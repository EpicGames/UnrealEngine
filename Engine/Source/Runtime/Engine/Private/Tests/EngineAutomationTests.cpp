// Copyright Epic Games, Inc. All Rights Reserved.


#include "Engine/Level.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/PackageName.h"
#include "GameMapsSettings.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"
#include "Serialization/MemoryReader.h"
#include "Tests/AutomationTestSettings.h"

#if WITH_EDITOR
#include "FileHelpers.h"
#endif

#include "Tests/AutomationCommon.h"
#include "PlatformFeatures.h"
#include "SaveGameSystem.h"
#include "GameFramework/DefaultPawn.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	UWorld* GetSimpleEngineAutomationTestGameWorld(const EAutomationTestFlags TestFlags)
	{
		// Accessing the game world is only valid for game-only 
		check((TestFlags & EAutomationTestFlags_ApplicationContextMask) == EAutomationTestFlags::ClientContext);
		check(GEngine->GetWorldContexts().Num() == 1);
		check(GEngine->GetWorldContexts()[0].WorldType == EWorldType::Game);

		return GEngine->GetWorldContexts()[0].World();
	}

	/**
	* Populates the test names and commands for complex tests that are ran on all available maps
	*
	* @param OutBeautifiedNames - The list of map names
	* @param OutTestCommands - The list of commands for each test (The file names in this case)
	*/
	void PopulateTestsForAllAvailableMaps(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands)
	{
		TArray<FString> FileList;
#if WITH_EDITOR
		FEditorFileUtils::FindAllPackageFiles(FileList);
#else
		// Look directly on disk. Very slow!
		FPackageName::FindPackagesInDirectory(FileList, *FPaths::ProjectContentDir());
#endif

		// Iterate over all files, adding the ones with the map extension..
		for (int32 FileIndex = 0; FileIndex < FileList.Num(); FileIndex++)
		{
			const FString& Filename = FileList[FileIndex];

			// Disregard filenames that don't have the map extension if we're in MAPSONLY mode.
			if (FPaths::GetExtension(Filename, true) == FPackageName::GetMapPackageExtension())
			{
				if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
				{
					OutBeautifiedNames.Add(FPaths::GetBaseFilename(Filename));
					OutTestCommands.Add(Filename);
				}
			}
		}
	}
}

#if PLATFORM_DESKTOP

/**
 * SetRes Verification - Verify changing resolution works
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetResTest, "System.Windows.Set Resolution", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

/** 
 * Change resolutions, wait, and change back
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FSetResTest::RunTest(const FString& Parameters)
{

	//Gets the default map that the game uses.
	const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
	const FString& MapName = GameMapsSettings->GetGameDefaultMap();

	//Opens the actual default map in game.
	GEngine->Exec(GetSimpleEngineAutomationTestGameWorld(GetTestFlags()), *FString::Printf(TEXT("Open %s"), *MapName));

	//Gets the current resolution.
	int32 ResX = GSystemResolution.ResX;
	int32 ResY = GSystemResolution.ResY;
	FString RestoreResolutionString = FString::Printf(TEXT("setres %dx%d"), ResX, ResY);

	//Change the resolution and then restore it.
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.0f));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("setres 640x480")));
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.0f));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(RestoreResolutionString));
	return true;
}

#endif

/**
 * Stats verification - Toggle various "stats" commands
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStatsVerificationMapTest, "System.Maps.Stats Verification", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

/** 
 * Execute the loading of one map to verify screen captures and performance captures work
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FStatsVerificationMapTest::RunTest(const FString& Parameters)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	if ( AutomationTestSettings->AutomationTestmap.IsValid() )
	{
		FString MapName = AutomationTestSettings->AutomationTestmap.GetLongPackageName();

		GEngine->Exec(GetSimpleEngineAutomationTestGameWorld(GetTestFlags()), *FString::Printf(TEXT("Open %s"), *MapName));
	}
	else
	{
		UE_LOGF(LogEngineAutomationTests, Log, "Automation test map doesn't exist or is not set: %ls.  \nUsing the currently loaded map.", *AutomationTestSettings->AutomationTestmap.GetLongPackageName());
	}

	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat game")));
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat game")));

	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat scenerendering")));
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat scenerendering")));

	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat memory")));
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat memory")));

	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat slate")));
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(1.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat slate")));

	return true;
}

/**
 * Latent command to take a screenshot of the viewport
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FTakeViewportScreenshotCommand, FString, ScreenshotFileName);

bool FTakeViewportScreenshotCommand::Update()
{
	const bool bShowUI = false;
	const bool bAddFilenameSuffix = false;
	FScreenshotRequest::RequestScreenshot( ScreenshotFileName, bShowUI, bAddFilenameSuffix );
	return true;
}

/**
 * SaveGameTest
 * Test makes sure a save game (without UI) saves and loads correctly
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSaveGameTest, "System.Engine.Game.Noninteractive Save", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

/** 
 * Saves and loads a savegame file
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FSaveGameTest::RunTest(const FString& Parameters)
{
	// automation save name
	const TCHAR* SaveName = TEXT("AutomationSaveTest");
	uint32 SavedData = 99;

	// the blob we are going to write out
	TArray<uint8> Blob;
	FMemoryWriter WriteAr(Blob);
	WriteAr << SavedData;

	// get the platform's save system
	ISaveGameSystem* Save = IPlatformFeaturesModule::Get().GetSaveGameSystem();

	// write it out
	if (Save->SaveGame(false, SaveName, 0, Blob) == false)
	{
		return false;
	}

	// make sure it was written
	if (Save->DoesSaveGameExist(SaveName, 0) == false)
	{
		return false;
	}

	// read it back in
	Blob.Empty();
	if (Save->LoadGame(false, SaveName, 0, Blob) == false)
	{
		return false;
	}

	// make sure it's the same data
	FMemoryReader ReadAr(Blob);
	uint32 LoadedData;
	ReadAr << LoadedData;

	// try to delete it (not all platforms can)
	if (Save->DeleteGame(false, SaveName, 0))
	{
		// make sure it's no longer there
		if (Save->DoesSaveGameExist(SaveName, 0) == true)
		{
			return false;
		}
	}

	return LoadedData == SavedData;
}

/**
 * FCVarEnvironmentTest
 * Test makes sure that CVars are set and restore properly
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCVarEnvironmentTest, "System.Engine.Automation.Environment.CVar", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

/**
 * Set and restore a CVar
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FCVarEnvironmentTest::RunTest(const FString& Parameters)
{
	const FString DummyTestName = "Automation.DummyTestVariable";
	const int32 TestValue = 12345;

	TAutoConsoleVariable<int32> CVarDummyTestVariable(
		*DummyTestName,
		111,
		TEXT("Used for the purposes of testing if the CVar is getting set and reset."),
		ECVF_Default);

	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*DummyTestName);
	if (!ConsoleVariable)
	{
		AddError(FString::Format(TEXT("Could not find CVar for '{0}'"), { DummyTestName }));
		return false;
	}

	int32 InitialValue = ConsoleVariable->GetInt();
	if (InitialValue == TestValue)
	{
		AddError(FString::Format(TEXT("Initial and values to test are the same '{0}'"), { InitialValue }));
		return false;
	}

	// Because we're testing that the CVar is properly restored, we want to create our FTestEnvironment object inside a scope to be destructed
	{
		TSharedPtr<FScopedTestEnvironment> TestCVarEnvironment = FScopedTestEnvironment::Get();
		TestCVarEnvironment->SetConsoleVariableValue(DummyTestName, FString::FromInt(TestValue));

		// Verify that setting the CVar through our test environment actually sets the Console Variable
		int32 CurrentValue = ConsoleVariable->GetInt();
		if (CurrentValue == InitialValue)
		{
			AddError(FString::Format(TEXT("CVar was not set as the current value matches the initial value of '{0}'"), { InitialValue }));
			return false;
		}

		// Verify that retrieving the CVar from our test environment matches the value fetched directly from the CVar
		FString ConsoleVariableValue;
		bool bWasCVarSet = TestCVarEnvironment->TryGetConsoleVariableValue(DummyTestName, &ConsoleVariableValue);
		if (!bWasCVarSet)
		{
			AddError(TEXT("CVar was not found as being set in the ScopedTestEnvironment."));
			return false;
		}

		if (ConsoleVariableValue != FString::FromInt(CurrentValue))
		{
			AddError(FString::Format(TEXT("CVar value of '{0}' does not match the CVar value fetched from ScopedTestEnvironment '{1}'"), { CurrentValue, ConsoleVariableValue }));
			return false;
		}
	}

	int32 CurrentValue = ConsoleVariable->GetInt();
	AddErrorIfFalse(CurrentValue == InitialValue, FString::Format(TEXT("CVar was not reset as the current value of '{0}' does not match the initial value of '{1}'"), { CurrentValue, InitialValue }));

	return !HasAnyErrors();
}

/**
 * FCVarEnvironmentReuseTest
 * Test makes sure that a CVar can be set multiple times before restoring back to the original value prior to being set
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCVarEnvironmentReuseTest, "System.Engine.Automation.Environment.CVar Reuse", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

/**
 * Set a CVar multiple times before restoring to the original value
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FCVarEnvironmentReuseTest::RunTest(const FString& Parameters)
{
	const FString DummyTestName = "Automation.DummyTestVariable";

	TAutoConsoleVariable<int32> CVarDummyTestVariable(
		*DummyTestName,
		111,
		TEXT("Used for the purposes of testing if the CVar is getting set and reset."),
		ECVF_Default);

	IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*DummyTestName);
	if (!ConsoleVariable)
	{
		AddError(FString::Format(TEXT("Could not find CVar for '{0}'"), { DummyTestName }));
		return false;
	}

	int32 InitialValue = ConsoleVariable->GetInt();

	// Because we're testing that the CVar is properly restored, we want to create our FTestEnvironment object inside a scope to be destructed
	{
		TSharedPtr<FScopedTestEnvironment> TestCVarEnvironment = FScopedTestEnvironment::Get();

		// Loop through a range of values to set our CVar
		for (int32 TestValue = 0; TestValue < 5; ++TestValue)
		{
			TestCVarEnvironment->SetConsoleVariableValue(DummyTestName, FString::FromInt(TestValue));
			
			int32 CurrentValue = ConsoleVariable->GetInt();
			if (TestValue != CurrentValue)
			{
				AddError(FString::Format(TEXT("CVar was not set as the current value '{0}' does not match the expected value of '{1}'"), { CurrentValue, TestValue }));
				return false;
			}
		}
	}

	int32 CurrentValue = ConsoleVariable->GetInt();
	AddErrorIfFalse(CurrentValue == InitialValue, FString::Format(TEXT("CVar was not reset as the current value of '{0}' does not match the initial value of '{1}'"), { CurrentValue, InitialValue }));

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationLogAddMessage, "TestFramework.Log.Add Log Message", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FAutomationLogAddMessage::RunTest(const FString& Parameters)
{
	//** TEST **//
	AddInfo(TEXT("Test log message."));

	//** VERIFY **//
	TestEqual<FString>(TEXT("Test log message was not added to the ExecutionInfo.Log array."), ExecutionInfo.GetEntries().Last().Event.Message, TEXT("Test log message."));
	
	//** TEARDOWN **//
	// We have to empty this log array so that it doesn't show in the automation results window as it may cause confusion.
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Info);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationLogAddWarning, "TestFramework.Log.Add Warning Message", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FAutomationLogAddWarning::RunTest(const FString& Parameters)
{
	//** TEST **//
	AddWarning(TEXT("Test warning message."));

	//** VERIFY **//
	FString CurrentWarningMessage = ExecutionInfo.GetEntries().Last().Event.Message;
	// The warnings array is emptied so that it doesn't cause a false positive warning for this test.
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Warning);

	TestEqual<FString>(TEXT("Test warning message was not added to the ExecutionInfo.Warning array."), CurrentWarningMessage, TEXT("Test warning message."));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationLogAddError, "TestFramework.Log.Add Error Message", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FAutomationLogAddError::RunTest(const FString& Parameters)
{
	//** TEST **//
	AddError(TEXT("Test error message"));
	
	//** VERIFY **//
	FString CurrentErrorMessage = ExecutionInfo.GetEntries().Last().Event.Message;
	// The errors array is emptied so that this doesn't cause a false positive failure for this test.
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);

	TestEqual<FString>(TEXT("Test error message was not added to the ExecutionInfo.Error array."), CurrentErrorMessage, TEXT("Test error message"));
	
	return true;
}

class FAutomationNearlyEqualTest : public FAutomationTestBase
{
public:
	FAutomationNearlyEqualTest(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

protected:
	template<typename T>
	int32 RunSimpleTest(const FString& What, const T ActualValue, const T ExpectedValue, float Tolerance)
	{
		int32 CasesCheckedTotal = 0;

		TestNearlyEqual(What, ActualValue, ExpectedValue, Tolerance);
		++CasesCheckedTotal;

		return CasesCheckedTotal;
	}

	// Note: this method is used to avoid rising of error C2666 2 overloads have similar conversions
	template<>
	int32 RunSimpleTest<double>(const FString& What, double ActualValue, double ExpectedValue, float Tolerance)
	{
		int32 CasesCheckedTotal = 0;

		TestNearlyEqual(What, ActualValue, ExpectedValue, static_cast<double>(Tolerance));
		++CasesCheckedTotal;

		return CasesCheckedTotal;
	}

	int32 RunFloatMutationTest(const FString& WhatPrefix, float BaseValue, float Difference, float Tolerance)
	{
		check(Difference != 0.f);
		check(Tolerance > 0.f);

		int32 CasesCheckedTotal = 0;

		// Perform tests with mutated values
		TestNearlyEqual(FString::Format(*ActualValueIsIncreasedByFormatString, { *WhatPrefix, Difference }),
			BaseValue + Difference, BaseValue, Tolerance);
		++CasesCheckedTotal;

		TestNearlyEqual(FString::Format(*ExpectedValueIsIncreasedByFormatString, { *WhatPrefix, Difference }),
			BaseValue, BaseValue + Difference, Tolerance);
		++CasesCheckedTotal;

		return CasesCheckedTotal;
	}

	int32 RunDoubleMutationTest(const FString& WhatPrefix, double BaseValue, double Difference, float Tolerance)
	{
		check(Difference != 0.f);
		check(Tolerance > 0.f);

		int32 CasesCheckedTotal = 0;

		// Perform tests with mutated values
		TestNearlyEqual(FString::Format(*ActualValueIsIncreasedByFormatString, { *WhatPrefix, Difference }),
			BaseValue + Difference, BaseValue, static_cast<double>(Tolerance));
		++CasesCheckedTotal;
		TestNearlyEqual(FString::Format(*ExpectedValueIsIncreasedByFormatString, { *WhatPrefix, Difference }),
			BaseValue, BaseValue + Difference, static_cast<double>(Tolerance));
		++CasesCheckedTotal;

		return CasesCheckedTotal;
	}

	using GetWhatCallable = TFunction<FString(const FString& WhatPrefix, uint32 ActualValueMutationBitMask, 
		uint32 ExpectedValueMutationBitMask, double Difference)>;

	template<typename T>
	using GetMutatedValueCallable = TFunction<T(const T& BaseValue, uint32 MutationBitMask, double Difference)>;

	int32 RunFVectorMutationTest(const FString& WhatPrefix, const FVector& BaseValue, double Difference, float Tolerance)
	{
		GetWhatCallable GetWhatCallableImpl = [](const FString& WhatPrefix, uint32 ActualValueMutationBitMask, uint32 ExpectedValueMutationBitMask, double Difference)
		{
			return FString::Printf(
				TEXT(
					"%s: the actual FVector value is not nearly equal to the expected FVector value\n"
					"(mutation mask for actual value is (%c, %c, %c), mutation mask for expected value is (%c, %c, %c), values were increased by %f)"
				),
				*WhatPrefix,
				GetNthBitAsChar(ActualValueMutationBitMask, 2),
				GetNthBitAsChar(ActualValueMutationBitMask, 1),
				GetNthBitAsChar(ActualValueMutationBitMask, 0),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 2),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 1),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 0),
				Difference);
		};

		GetMutatedValueCallable<FVector> GetMutatedValueCallableImpl = [](const FVector& BaseValue, uint32 MutationBitMask, double Difference)
		{
			return FVector(
				BaseValue.X + GetNthBitAsUInt32(MutationBitMask, 2) * Difference,
				BaseValue.Y + GetNthBitAsUInt32(MutationBitMask, 1) * Difference,
				BaseValue.Z + GetNthBitAsUInt32(MutationBitMask, 0) * Difference
			);
		};

		return RunMutationTestImpl<FVector>(WhatPrefix, BaseValue, MaxFVectorMutationBitMask, Difference, Tolerance,
			GetWhatCallableImpl, GetMutatedValueCallableImpl);
	}

	int32 RunFRotatorMutationTest(const FString& WhatPrefix, const FRotator& BaseValue, double Difference, float Tolerance)
	{
		GetWhatCallable GetWhatCallableImpl = [](const FString& WhatPrefix, uint32 ActualValueMutationBitMask, uint32 ExpectedValueMutationBitMask, double Difference)
		{
			return FString::Printf(
				TEXT(
					"%s: the actual FRotator value is not nearly equal to the expected FRotator value\n"
					"(mutation mask for actual value is (%c, %c, %c), mutation mask for expected value is (%c, %c, %c), values were increased by %f)"
				),
				*WhatPrefix,
				GetNthBitAsChar(ActualValueMutationBitMask, 2),
				GetNthBitAsChar(ActualValueMutationBitMask, 1),
				GetNthBitAsChar(ActualValueMutationBitMask, 0),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 2),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 1),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 0),
				Difference);
		};

		GetMutatedValueCallable<FRotator> GetMutatedValueCallableImpl = [](const FRotator& BaseValue, uint32 MutationBitMask, double Difference)
		{
			return FRotator(
				BaseValue.Pitch + GetNthBitAsUInt32(MutationBitMask, 2) * Difference,
				BaseValue.Yaw + GetNthBitAsUInt32(MutationBitMask, 1) * Difference,
				BaseValue.Roll + GetNthBitAsUInt32(MutationBitMask, 0) * Difference
			);
		};

		return RunMutationTestImpl<FRotator>(WhatPrefix, BaseValue, MaxFRotatorMutationBitMask, Difference, Tolerance,
			GetWhatCallableImpl, GetMutatedValueCallableImpl);
	}

	int32 RunFTransformMutationTest(const FString& WhatPrefix, const FTransform& BaseValue, double Difference, float Tolerance)
	{
		GetWhatCallable GetWhatCallableImpl = [](const FString& WhatPrefix, uint32 ActualValueMutationBitMask, uint32 ExpectedValueMutationBitMask, double Difference)
		{
			return FString::Printf(
				TEXT(
					"%s: the actual FTransform value is not nearly equal to the expected FTransform value\n"
					"(mutation mask for actual value is (%c, %c, %c, %c, %c, %c, %c, %c, %c), "
					"mutation mask for expected value is (%c, %c, %c, %c, %c, %c, %c, %c, %c), values were increased by %f)"
				),
				*WhatPrefix,
				GetNthBitAsChar(ActualValueMutationBitMask, 8),
				GetNthBitAsChar(ActualValueMutationBitMask, 7),
				GetNthBitAsChar(ActualValueMutationBitMask, 6),
				GetNthBitAsChar(ActualValueMutationBitMask, 5),
				GetNthBitAsChar(ActualValueMutationBitMask, 4),
				GetNthBitAsChar(ActualValueMutationBitMask, 3),
				GetNthBitAsChar(ActualValueMutationBitMask, 2),
				GetNthBitAsChar(ActualValueMutationBitMask, 1),
				GetNthBitAsChar(ActualValueMutationBitMask, 0),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 8),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 7),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 6),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 5),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 4),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 3),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 2),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 1),
				GetNthBitAsChar(ExpectedValueMutationBitMask, 0),
				Difference);
		};

		GetMutatedValueCallable<FTransform> GetMutatedValueCallableImpl = [](const FTransform& BaseValue, uint32 MutationBitMask, double Difference)
		{
			return FTransform(
				FRotator(
					BaseValue.Rotator().Pitch + GetNthBitAsUInt32(MutationBitMask, 8) * Difference,
					BaseValue.Rotator().Yaw + GetNthBitAsUInt32(MutationBitMask, 7) * Difference,
					BaseValue.Rotator().Roll + GetNthBitAsUInt32(MutationBitMask, 6) * Difference
				),
				FVector(
					BaseValue.GetTranslation().X + GetNthBitAsUInt32(MutationBitMask, 5) * Difference,
					BaseValue.GetTranslation().Y + GetNthBitAsUInt32(MutationBitMask, 4) * Difference,
					BaseValue.GetTranslation().Z + GetNthBitAsUInt32(MutationBitMask, 3) * Difference
				),
				FVector(
					BaseValue.GetScale3D().X + GetNthBitAsUInt32(MutationBitMask, 2) * Difference,
					BaseValue.GetScale3D().Y + GetNthBitAsUInt32(MutationBitMask, 1) * Difference,
					BaseValue.GetScale3D().Z + GetNthBitAsUInt32(MutationBitMask, 0) * Difference
				)
			);
		};

		return RunMutationTestImpl<FTransform>(WhatPrefix, BaseValue, MaxFTransformMutationBitMask, Difference, Tolerance,
			GetWhatCallableImpl, GetMutatedValueCallableImpl);
	}

	static const float NullTolerance;
	static const float PositiveTolerance;
	static const float PositiveDifference;
	static const float PositiveHalfDifference;

	// Max mutation masks for complex classes/structs
	// Each bit represents whether (value 1) or not (value 0) mutation will be applied to the object's constructor paramter
	static const uint32 MaxFVectorMutationBitMask;
	static const uint32 MaxFRotatorMutationBitMask;
	static const uint32 MaxFTransformMutationBitMask;

	static const FString TestFailMessage;

	static const float BaseFloatValue;
	static const float ActualFloatValue;
	static const float ExpectedFloatValue;
	static const float ExpectedFloatValueForNullTolerance;
	static const float FloatDifferenceToGetOutOfTolerance;
	static const float ExpectedFloatValueOutOfTolerance;

	static const double BaseDoubleValue;
	static const double ActualDoubleValue;
	static const double ExpectedDoubleValue;
	static const double ExpectedDoubleValueForNullTolerance;
	static const double DoubleDifferenceToGetOutOfTolerance;
	static const double ExpectedDoubleValueOutOfTolerance;

	static const FVector ActualFVectorValue;
	static const FVector& ExpectedFVectorValue;
	static const FVector& BaseFVectorValue;

	static const FRotator ActualFRotatorValue;
	static const FRotator& ExpectedFRotatorValue;
	static const FRotator& BaseFRotatorValue;

	static const FTransform ActualFTransformValue;
	static const FTransform& ExpectedFTransformValue;
	static const FTransform& BaseFTransformValue;

private:
	template<typename T>
	int32 RunMutationTestImpl(const FString& WhatPrefix, const T& BaseValue, uint32 MaxMutationBitMask, float Difference, float Tolerance,
		GetWhatCallable GetWhatCallableImpl, GetMutatedValueCallable<T> GetMutatedValueCallableImpl)
	{
		check(Difference != 0.f);
		check(Tolerance > 0.f);

		int32 CasesCheckedTotal = 0;

		for (uint32 ActualValueMutationBitMask = 0; ActualValueMutationBitMask <= MaxMutationBitMask; ++ActualValueMutationBitMask)
		{
			for (uint32 ExpectedValueMutationBitMask = 0; ExpectedValueMutationBitMask <= MaxMutationBitMask; ++ExpectedValueMutationBitMask)
			{
				if (ActualValueMutationBitMask == ExpectedValueMutationBitMask)
				{
					// The values' mutation submasks are the same, we should skip this combination
					continue;
				}

				// Perform test with mutated values in accordance to the current MutationBitMask
				const FString WhatMessage(GetWhatCallableImpl(WhatPrefix, ActualValueMutationBitMask, ExpectedValueMutationBitMask, Difference));
				const T ActualValue(GetMutatedValueCallableImpl(BaseValue, ActualValueMutationBitMask, Difference));
				const T ExpectedValue(GetMutatedValueCallableImpl(BaseValue, ExpectedValueMutationBitMask, Difference));

				TestNearlyEqual(WhatMessage, ActualValue, ExpectedValue, Tolerance);
				++CasesCheckedTotal;
			}
		}

		return CasesCheckedTotal;
	}

	static uint32 GetNthBitAsUInt32(uint32 Value, uint32 BitIndex)
	{
		return ((Value & (1 << BitIndex)) == 0 ? 0 : 1);
	}

	static char GetNthBitAsChar(uint32 Value, uint32 BitIndex)
	{
		return (GetNthBitAsUInt32(Value, BitIndex) == 1 ? '1' : '0');
	}

	static const FString ActualValueIsIncreasedByFormatString;
	static const FString ExpectedValueIsIncreasedByFormatString;
	static const FString DifferenceAndOrToleranceAreNotValidFormatString;
};

const float FAutomationNearlyEqualTest::NullTolerance(0.f);
const float FAutomationNearlyEqualTest::PositiveTolerance(1.e-4f);
const float FAutomationNearlyEqualTest::PositiveDifference(1.e-4f);
const float FAutomationNearlyEqualTest::PositiveHalfDifference((1.e-4f) / 2.f);
const FString FAutomationNearlyEqualTest::TestFailMessage(TEXT("Total amount of errors is not equal to the expected amount"));
const uint32 FAutomationNearlyEqualTest::MaxFVectorMutationBitMask(0b111);
const uint32 FAutomationNearlyEqualTest::MaxFRotatorMutationBitMask(0b111);
const uint32 FAutomationNearlyEqualTest::MaxFTransformMutationBitMask(0b111);

const float FAutomationNearlyEqualTest::BaseFloatValue(0.f);
const float FAutomationNearlyEqualTest::ActualFloatValue(BaseFloatValue);
const float FAutomationNearlyEqualTest::ExpectedFloatValue(BaseFloatValue);
const float FAutomationNearlyEqualTest::ExpectedFloatValueForNullTolerance(0.1f);
const float FAutomationNearlyEqualTest::FloatDifferenceToGetOutOfTolerance(PositiveTolerance + 0.1f);
const float FAutomationNearlyEqualTest::ExpectedFloatValueOutOfTolerance(ActualFloatValue + FloatDifferenceToGetOutOfTolerance + 0.1f);

const double FAutomationNearlyEqualTest::BaseDoubleValue(0.0);
const double FAutomationNearlyEqualTest::ActualDoubleValue(BaseDoubleValue);
const double FAutomationNearlyEqualTest::ExpectedDoubleValue(BaseDoubleValue);
const double FAutomationNearlyEqualTest::ExpectedDoubleValueForNullTolerance(0.1);
const double FAutomationNearlyEqualTest::DoubleDifferenceToGetOutOfTolerance(PositiveTolerance + 0.1);
const double FAutomationNearlyEqualTest::ExpectedDoubleValueOutOfTolerance(ActualDoubleValue + DoubleDifferenceToGetOutOfTolerance);

const FVector FAutomationNearlyEqualTest::ActualFVectorValue(0.f, -1.f, 1.f);
const FVector& FAutomationNearlyEqualTest::ExpectedFVectorValue(ActualFVectorValue);
const FVector& FAutomationNearlyEqualTest::BaseFVectorValue(ActualFVectorValue);

const FRotator FAutomationNearlyEqualTest::ActualFRotatorValue(0.001f, -1.002f, 1.003f);
const FRotator& FAutomationNearlyEqualTest::ExpectedFRotatorValue(ActualFRotatorValue);
const FRotator& FAutomationNearlyEqualTest::BaseFRotatorValue(ActualFRotatorValue);

const FTransform FAutomationNearlyEqualTest::ActualFTransformValue(FRotator(0.f, -1.f, 1.f), FVector(0.1f, -1.2f, 1.3f), FVector(0.01f, -1.02f, 1.03f));
const FTransform& FAutomationNearlyEqualTest::ExpectedFTransformValue(ActualFTransformValue);
const FTransform& FAutomationNearlyEqualTest::BaseFTransformValue(ActualFTransformValue);

const FString FAutomationNearlyEqualTest::ActualValueIsIncreasedByFormatString(TEXT("{0} (actual value is increased by {1})"));
const FString FAutomationNearlyEqualTest::ExpectedValueIsIncreasedByFormatString(TEXT("{0} (expected value is increased by {1})"));
const FString FAutomationNearlyEqualTest::DifferenceAndOrToleranceAreNotValidFormatString(TEXT("Difference and/or Tolerance are not valid. Difference: {0}, Tolerance: {1}"));

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFloatPositive, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFloatPositive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFloatPositive::RunTest(const FString& Parameters)
{
	//** TEST **//
	RunSimpleTest<float>(TEXT("The same float values with null tolerance"),
		ActualFloatValue, ExpectedFloatValue, NullTolerance);
	RunSimpleTest<float>(TEXT("The same float values with positive tolerance"),
		ActualFloatValue, ExpectedFloatValue, PositiveTolerance);
	RunFloatMutationTest(TEXT("Mutation of base float value with the same positive difference and tolerance (edge case)"),
		BaseFloatValue, PositiveDifference, PositiveTolerance);
	RunFloatMutationTest(TEXT("Mutation of base float value with negative difference and positive tolerance that are equal after being placed in Abs"),
		BaseFloatValue, -PositiveDifference, PositiveTolerance);
	RunFloatMutationTest(TEXT("Mutation of base float value with positive half difference and positive tolerance"),
		BaseFloatValue, PositiveHalfDifference, PositiveTolerance);
	RunFloatMutationTest(TEXT("Mutation of base float value with negative half difference and positive tolerance"),
		BaseFloatValue, -PositiveHalfDifference, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	const int32 ExpectedErrorTotal = 0;

	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, ExpectedErrorTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFloatNegative, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFloatNegative", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFloatNegative::RunTest(const FString& Parameters)
{
	int32 CasesCheckedTotal = 0;

	//** TEST **//
	CasesCheckedTotal += RunSimpleTest<float>(TEXT("Different float values with null tolerance"),
		ActualFloatValue, ExpectedFloatValueForNullTolerance, NullTolerance);
	CasesCheckedTotal += RunSimpleTest<float>(TEXT("Different float values with positive tolerance"),
		ActualFloatValue, ExpectedFloatValueOutOfTolerance, PositiveTolerance);
	CasesCheckedTotal += RunFloatMutationTest(TEXT("Mutation of base float value with positive difference that is greater than positive tolerance"),
		BaseFloatValue, FloatDifferenceToGetOutOfTolerance, PositiveTolerance);
	CasesCheckedTotal += RunFloatMutationTest(TEXT("Mutation of base float value with negative difference which absolute value is greater than positive tolerance"),
		BaseFloatValue, -FloatDifferenceToGetOutOfTolerance, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, CasesCheckedTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualDoublePositive, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualDoublePositive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualDoublePositive::RunTest(const FString& Parameters)
{
	//** TEST **//
	RunSimpleTest<double>(TEXT("The same double values with null tolerance"),
		ActualDoubleValue, ExpectedDoubleValue, NullTolerance);
	RunSimpleTest<double>(TEXT("The same double values with positive tolerance"),
		ActualDoubleValue, ExpectedDoubleValue, PositiveTolerance);
	RunDoubleMutationTest(TEXT("Mutation of base double value with the same positive difference and tolerance (edge case)"),
		BaseDoubleValue, PositiveDifference, PositiveTolerance);
	RunDoubleMutationTest(TEXT("Mutation of base double value with negative difference and positive tolerance that are equal after being placed in Abs"),
		BaseDoubleValue, -PositiveDifference, PositiveTolerance);
	RunDoubleMutationTest(TEXT("Mutation of base double value with positive half difference and positive tolerance"),
		BaseDoubleValue, PositiveHalfDifference, PositiveTolerance);
	RunDoubleMutationTest(TEXT("Mutation of base double value with negative half difference and positive tolerance"),
		BaseDoubleValue, -PositiveHalfDifference, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	const int32 ExpectedErrorTotal = 0;

	TestEqual(TestFailMessage, ErrorTotal, ExpectedErrorTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualDoubleNegative, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualDoubleNegative", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualDoubleNegative::RunTest(const FString& Parameters)
{
	int32 CasesCheckedTotal = 0;

	//** TEST **//
	CasesCheckedTotal += RunSimpleTest<double>(TEXT("Different double values with null tolerance"),
		ActualDoubleValue, ExpectedDoubleValueForNullTolerance, NullTolerance);
	CasesCheckedTotal += RunSimpleTest<double>(TEXT("Different double values with positive tolerance"),
		ActualDoubleValue, ExpectedDoubleValueOutOfTolerance, PositiveTolerance);
	CasesCheckedTotal += RunDoubleMutationTest(TEXT("Mutation of base double value with positive difference that is greater than positive tolerance"),
		BaseDoubleValue, DoubleDifferenceToGetOutOfTolerance, PositiveTolerance);
	CasesCheckedTotal += RunDoubleMutationTest(TEXT("Mutation of base double value with negative difference which absolute value is greater than positive tolerance"),
		BaseDoubleValue, -DoubleDifferenceToGetOutOfTolerance, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, CasesCheckedTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFVectorPositive, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFVectorPositive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFVectorPositive::RunTest(const FString& Parameters)
{
	//** TEST **//
	RunSimpleTest<FVector>(TEXT("The same FVector values with null tolerance"),
		ActualFVectorValue, ExpectedFVectorValue, NullTolerance);
	RunSimpleTest<FVector>(TEXT("The same FVector values with positive tolerance"),
		ActualFVectorValue, ExpectedFVectorValue, PositiveTolerance);
	RunFVectorMutationTest(TEXT("Mutation of base FVector value with the same positive difference and tolerance (edge case)"),
		BaseFVectorValue, PositiveDifference, PositiveTolerance);
	RunFVectorMutationTest(TEXT("Mutation of base FVector value with negative difference and positive tolerance that are equal after being placed in Abs"),
		BaseFVectorValue, -PositiveDifference, PositiveTolerance);
	RunFVectorMutationTest(TEXT("Mutation of base FVector value with positive half difference and positive tolerance"),
		BaseFVectorValue, PositiveHalfDifference, PositiveTolerance);
	RunFVectorMutationTest(TEXT("Mutation of base FVector value with negative half difference and positive tolerance"),
		BaseFVectorValue, -PositiveHalfDifference, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	const int32 ExpectedErrorTotal = 0;

	TestEqual(TestFailMessage, ErrorTotal, ExpectedErrorTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFVectorNegative, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFVectorNegative", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFVectorNegative::RunTest(const FString& Parameters)
{
	int32 CasesCheckedTotal = 0;

	//** TEST **//
	CasesCheckedTotal += RunFVectorMutationTest(TEXT("Mutation of base FVector value with positive difference that is greater than positive tolerance"),
		BaseFVectorValue, PositiveDifference + 0.1f, PositiveTolerance);
	CasesCheckedTotal += RunFVectorMutationTest(TEXT("Mutation of base FVector value with negative difference which absolute value is greater than positive tolerance"),
		BaseFVectorValue, -PositiveDifference - 0.1f, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, CasesCheckedTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFRotatorPositive, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFRotatorPositive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFRotatorPositive::RunTest(const FString& Parameters)
{
	//** TEST **//
	RunSimpleTest<FRotator>(TEXT("The same FRotator values with null tolerance"),
		ActualFRotatorValue, ExpectedFRotatorValue, NullTolerance);
	RunSimpleTest<FRotator>(TEXT("The same FRotator values with positive tolerance"),
		ActualFRotatorValue, ExpectedFRotatorValue, PositiveTolerance);
	RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with the same positive difference and tolerance (edge case)"),
		BaseFRotatorValue, PositiveDifference, PositiveTolerance);
	RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with negative difference and positive tolerance that are equal after being placed in Abs"),
		BaseFRotatorValue, -PositiveDifference, PositiveTolerance);
	RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with positive half difference and positive tolerance"),
		BaseFRotatorValue, PositiveHalfDifference, PositiveTolerance);
	RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with negative half difference and positive tolerance"),
		BaseFRotatorValue, -PositiveHalfDifference, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	const int32 ExpectedErrorTotal = 0;

	TestEqual(TestFailMessage, ErrorTotal, ExpectedErrorTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFRotatorNegative, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFRotatorNegative", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFRotatorNegative::RunTest(const FString& Parameters)
{
	int32 CasesCheckedTotal = 0;

	//** TEST **//
	CasesCheckedTotal += RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with positive difference that is greater than positive tolerance"),
		BaseFRotatorValue, PositiveDifference + 1, PositiveTolerance);
	CasesCheckedTotal += RunFRotatorMutationTest(TEXT("Mutation of base FRotator value with negative difference which absolute value is greater than positive tolerance"),
		BaseFRotatorValue, -PositiveDifference - 1, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, CasesCheckedTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFTransformPositive, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFTransformPositive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFTransformPositive::RunTest(const FString& Parameters)
{
	//** TEST **//
	RunSimpleTest<FTransform>(TEXT("The same FTransform values with null tolerance"),
		ActualFTransformValue, ExpectedFTransformValue, NullTolerance);
	RunSimpleTest<FTransform>(TEXT("The same FTransform values with positive tolerance"),
		ActualFTransformValue, ExpectedFTransformValue, PositiveTolerance);
	RunFTransformMutationTest(TEXT("Mutation of base FTransform value with the same positive difference and tolerance (edge case)"),
		BaseFTransformValue, PositiveDifference, PositiveTolerance);
	RunFTransformMutationTest(TEXT("Mutation of base FTransform value with negative difference and positive tolerance that are equal after being placed in Abs"),
		BaseFTransformValue, -PositiveDifference, PositiveTolerance);
	RunFTransformMutationTest(TEXT("Mutation of base FTransform value with positive half difference and positive tolerance"),
		BaseFTransformValue, PositiveHalfDifference, PositiveTolerance);
	RunFTransformMutationTest(TEXT("Mutation of base FTransform value with negative half difference and positive tolerance"),
		BaseFTransformValue, -PositiveHalfDifference, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	const int32 ExpectedErrorTotal = 0;

	TestEqual(TestFailMessage, ErrorTotal, ExpectedErrorTotal);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestNearlyEqualFTransformNegative, FAutomationNearlyEqualTest, "TestFramework.Validation.TestNearlyEqualFTransformNegative", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestNearlyEqualFTransformNegative::RunTest(const FString& Parameters)
{
	int32 CasesCheckedTotal = 0;

	//** TEST **//
	CasesCheckedTotal += RunFTransformMutationTest(TEXT("Mutation of base FTransform value with positive difference that is greater than positive tolerance"),
		BaseFTransformValue, PositiveDifference + 0.1f, PositiveTolerance);
	CasesCheckedTotal += RunFTransformMutationTest(TEXT("Mutation of base FTransform value with negative difference which absolute value is greater than positive tolerance"),
		BaseFTransformValue, -PositiveDifference - 0.1f, PositiveTolerance);

	//** VERIFY **//
	const int32 ErrorTotal = ExecutionInfo.GetErrorTotal();
	ExecutionInfo.RemoveAllEvents(EAutomationEventType::Error);
	TestEqual(TestFailMessage, ErrorTotal, CasesCheckedTotal);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityBool, "TestFramework.Validation.TestInequalityBool", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityBool::RunTest(const FString& Parameters)
{
	TestTrue("True constant", true);
	TestTrue("True int", 1);
	TestFalse("False constant", false);
	TestFalse("False int", 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityPointer, "TestFramework.Validation.TestInequalityPointer", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityPointer::RunTest(const FString& Parameters)
{
	int32 StackValue = 42;
	int32* StackPointer = &StackValue;
	int32* SameStackPointer = &StackValue;
	int32 OtherStackValue = 42;
	int32* OtherStackPointer = &OtherStackValue;
	UWorld* World = UWorld::CreateWorld(EWorldType::Game, false);
	UWorld* SameWorld = World;
	UWorld* OtherWorld = UWorld::CreateWorld(EWorldType::Game, false);
	
	TestSamePtr("Identity stack primitive", StackPointer, StackPointer);
	TestSamePtr("Identity world object", World, World);
	TestSamePtr("Same stack primitive", SameStackPointer, StackPointer);
	TestSamePtr("Same world object", SameWorld, World);
	TestNotSamePtr("Other stack primitive", OtherStackPointer, StackPointer);
	TestNotSamePtr("Other world object", OtherWorld, World);
	TestNotNull("Stack primitive not null", StackPointer);
	TestNotNull("Constructed World object not null", World);
	TestNull("Nullptr", nullptr);

	World->DestroyWorld(false);
	OtherWorld->DestroyWorld(false);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityReference, "TestFramework.Validation.TestInequalityReference", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityReference::RunTest(const FString& Parameters)
{
	int32 StackValue = 42;
	int32& StackRef = StackValue;
	int32& SameStackRef = StackValue;
	int32 OtherStackValue = 42;
	int32& OtherStackRef = OtherStackValue;
	int32&& StackRValRef = 42;
	int32& StackLValFromRvalRef = StackRValRef;

	TestSame("Identity primitive", StackRef, StackRef);
	TestSame("Identity value", StackValue, StackRef);
	TestSame("Same primitive", SameStackRef, StackRef);
	TestSame("Identity rvalue", StackRValRef, StackRValRef);
	TestSame("Same rvalue and lvalue", StackLValFromRvalRef, StackRValRef);
	TestNotSame("Other primitive", OtherStackRef, StackRef);
	TestNotSame("Other value", OtherStackValue, StackRef);
	TestNotSame("Other rvalue", StackRValRef, StackRef);
	TestNotSame("Other lvalue from rvalue", StackLValFromRvalRef, StackRef);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityInt32, "TestFramework.Validation.TestInequalityInt32", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityInt32::RunTest(const FString& Parameters)
{
	int32 Expected(42);
	int32 Identical(Expected);
	int32 Zero(0);
	int32 LargePositive(2048);
	int32 LargeNegative(-2048);
	int32 MaxPositive(INT32_MAX);
	int32 MaxNegative(INT32_MIN);

	TestEqual(TEXT("Identity equal"), Identical, Expected);
	TestNotEqual(TEXT("Zero unequal"), Zero, Expected);
	TestNotEqual(TEXT("Positive unequal"), LargePositive, Expected);
	TestNotEqual(TEXT("Negative unequal"), LargeNegative, Expected);
	TestNotEqual(TEXT("Max unequal"), MaxPositive, Expected);
	TestNotEqual(TEXT("Min unequal"), MaxNegative, Expected);
	TestLessEqual(TEXT("Identity LE"), Identical, Expected);
	TestLessEqual(TEXT("Less LE"), LargeNegative, Expected);
	TestLessThan(TEXT("Min LE"), MaxNegative, Expected);
	TestLessThan(TEXT("Less than"), LargeNegative, Expected);
	TestLessThan(TEXT("Min less than"), MaxNegative, Expected);
	TestGreaterEqual(TEXT("Identity GE"), Identical, Expected);
	TestGreaterEqual(TEXT("Less GE"), LargePositive, Expected);
	TestGreaterEqual(TEXT("Max GE"), MaxPositive, Expected);
	TestGreaterThan(TEXT("Greater than"), LargePositive, Expected);
	TestGreaterThan(TEXT("Max greater than"), MaxPositive, Expected);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityInt64, "TestFramework.Validation.TestInequalityInt64", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityInt64::RunTest(const FString& Parameters)
{
	int64 Expected(42);
	int64 Identical(Expected);
	int64 Zero(0);
	int64 LargePositive(2048);
	int64 LargeNegative(-2048);
	int64 MaxPositive(INT64_MAX);
	int64 MaxNegative(INT64_MIN);

	TestEqual(TEXT("Identity equal"), Identical, Expected);
	TestNotEqual(TEXT("Zero unequal"), Zero, Expected);
	TestNotEqual(TEXT("Positive unequal"), LargePositive, Expected);
	TestNotEqual(TEXT("Negative unequal"), LargeNegative, Expected);
	TestNotEqual(TEXT("Max unequal"), MaxPositive, Expected);
	TestNotEqual(TEXT("Min unequal"), MaxNegative, Expected);
	TestLessEqual(TEXT("Identity LE"), Identical, Expected);
	TestLessEqual(TEXT("Less LE"), LargeNegative, Expected);
	TestLessThan(TEXT("Min LE"), MaxNegative, Expected);
	TestLessThan(TEXT("Less than"), LargeNegative, Expected);
	TestLessThan(TEXT("Min less than"), MaxNegative, Expected);
	TestGreaterEqual(TEXT("Identity GE"), Identical, Expected);
	TestGreaterEqual(TEXT("Less GE"), LargePositive, Expected);
	TestGreaterEqual(TEXT("Max GE"), MaxPositive, Expected);
	TestGreaterThan(TEXT("Greater than"), LargePositive, Expected);
	TestGreaterThan(TEXT("Max greater than"), MaxPositive, Expected);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalitySizeT, "TestFramework.Validation.TestInequalitySizeT", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalitySizeT::RunTest(const FString& Parameters)
{
	SIZE_T Expected(42);
	SIZE_T Identical(Expected);
	SIZE_T Zero(0);
	SIZE_T LargePositive(2048);
	SIZE_T SmallPositive(17);

	TestEqual(TEXT("Identity equal"), Identical, Expected);
	TestNotEqual(TEXT("Zero unequal"), Zero, Expected);
	TestNotEqual(TEXT("Positive unequal"), LargePositive, Expected);
	TestLessEqual(TEXT("Identity LE"), Identical, Expected);
	TestLessEqual(TEXT("Less LE"), SmallPositive, Expected);
	TestLessThan(TEXT("Less than"), SmallPositive, Expected);
	TestGreaterEqual(TEXT("Identity GE"), Identical, Expected);
	TestGreaterEqual(TEXT("Less GE"), LargePositive, Expected);
	TestGreaterThan(TEXT("Greater than"), LargePositive, Expected);
#ifdef SIZE_T_MAX
	SIZE_T MaxPositive(SIZE_T_MAX);
	TestNotEqual(TEXT("Max unequal"), MaxPositive, Expected);
	TestGreaterEqual(TEXT("Max GE"), MaxPositive, Expected);
	TestGreaterThan(TEXT("Max greater than"), MaxPositive, Expected);
#endif //SIZE_T_MAX

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityFloat, "TestFramework.Validation.TestInequalityFloat", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityFloat::RunTest(const FString& Parameters)
{
	float Expected(42);
	float Identical(Expected);
	float Zero(0);
	float LargePositive(2048);
	float LargeNegative(-2048);
	float LargeTolerance(1);
	float SmallTolerance(UE_SMALL_NUMBER);
	float KindaClosePositive(Expected + (UE_KINDA_SMALL_NUMBER * 0.9));
	float KindaCloseNegative(Expected - (UE_KINDA_SMALL_NUMBER * 0.9));
	float ExtremelyClosePositive(Expected + (UE_SMALL_NUMBER * 0.9));
	float ExtremelyCloseNegative(Expected - (UE_SMALL_NUMBER * 0.9));

	TestNearlyEqual(TEXT("Identity equal"), Identical, Expected);
	TestNearlyEqual(TEXT("Identity equal low tolerance"), Identical, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Identity equal high tolerance"), Identical, Expected, SmallTolerance);
	TestNearlyEqual(TEXT("Nearby positive equal"), KindaClosePositive, Expected);
	TestNearlyEqual(TEXT("Nearby positive equal low tolerance"), KindaClosePositive, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Nearby positive equal high tolerance"), ExtremelyClosePositive, Expected, SmallTolerance);
	TestNearlyEqual(TEXT("Nearby negative equal"), KindaCloseNegative, Expected);
	TestNearlyEqual(TEXT("Nearby negative equal low tolerance"), KindaCloseNegative, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Nearby negative equal high tolerance"), ExtremelyCloseNegative, Expected, SmallTolerance);

	TestEqual(TEXT("Identity equal (forwards to TestNearlyEqual)"), Identical, Expected);
	TestEqual(TEXT("Identity equal low tolerance (forwards to TestNearlyEqual)"), Identical, Expected, LargeTolerance);
	TestEqual(TEXT("Identity equal high tolerance (forwards to TestNearlyEqual)"), Identical, Expected, SmallTolerance);
	TestEqual(TEXT("Nearby positive equal (forwards to TestNearlyEqual)"), KindaClosePositive, Expected);
	TestEqual(TEXT("Nearby positive equal low tolerance (forwards to TestNearlyEqual)"), KindaClosePositive, Expected, LargeTolerance);
	TestEqual(TEXT("Nearby positive equal high tolerance (forwards to TestNearlyEqual)"), ExtremelyClosePositive, Expected, SmallTolerance);
	TestEqual(TEXT("Nearby negative equal (forwards to TestNearlyEqual)"), KindaCloseNegative, Expected);
	TestEqual(TEXT("Nearby negative equal low tolerance (forwards to TestNearlyEqual)"), KindaCloseNegative, Expected, LargeTolerance);
	TestEqual(TEXT("Nearby negative equal high tolerance (forwards to TestNearlyEqual)"), ExtremelyCloseNegative, Expected, SmallTolerance);

	TestNotEqual(TEXT("Zero unequal"), Zero, Expected);
	TestNotEqual(TEXT("Positive unequal"), LargePositive, Expected);
	TestNotEqual(TEXT("Nearby positive unequal due to high tolerance"), KindaClosePositive, Expected, SmallTolerance);
	TestNotEqual(TEXT("Negative unequal"), LargeNegative, Expected);
	TestNotEqual(TEXT("Nearby negative unequal due to high tolerance"), KindaClosePositive, Expected, SmallTolerance);
	TestNotEqual(TEXT("Max unequal"), FLT_MAX, Expected);
	TestNotEqual(TEXT("Min unequal"), FLT_MIN, Expected);

	TestLessEqual(TEXT("Identity LE"), Identical, Expected);
	TestLessEqual(TEXT("Identity LE low tolerance"), Identical, Expected, LargeTolerance);
	TestLessEqual(TEXT("Identity LE high tolerance"), Identical, Expected, SmallTolerance);
	TestLessEqual(TEXT("Less LE"), LargeNegative, Expected);
	TestLessThan(TEXT("Min LE"), FLT_MIN, Expected);
	TestLessThan(TEXT("Less than"), LargeNegative, Expected);
	TestLessThan(TEXT("Min less than"), FLT_MIN, Expected);

	TestGreaterEqual(TEXT("Identity GE"), Identical, Expected);
	TestGreaterEqual(TEXT("Identity GE low tolerance"), Identical, Expected, LargeTolerance);
	TestGreaterEqual(TEXT("Identity GE high tolerance"), Identical, Expected, SmallTolerance);
	TestGreaterEqual(TEXT("Less GE"), LargePositive, Expected);
	TestGreaterEqual(TEXT("Max GE"), FLT_MAX, Expected);
	TestGreaterThan(TEXT("Greater than"), LargePositive, Expected);
	TestGreaterThan(TEXT("Max greater than"), FLT_MAX, Expected);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityDouble, "TestFramework.Validation.TestInequalityDouble", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityDouble::RunTest(const FString& Parameters)
{
	double Expected(42);
	double Identical(Expected);
	double Zero(0);
	double LargePositive(2048);
	double LargeNegative(-2048);
	double SmallTolerance(UE_SMALL_NUMBER);
	double LargeTolerance(1);
	double KindaClosePositive(Expected + (UE_KINDA_SMALL_NUMBER * 0.9));
	double KindaCloseNegative(Expected - (UE_KINDA_SMALL_NUMBER * 0.9));
	double ExtremelyClosePositive(Expected + (UE_SMALL_NUMBER * 0.9));
	double ExtremelyCloseNegative(Expected - (UE_SMALL_NUMBER * 0.9));

	TestNearlyEqual(TEXT("Identity equal"), Identical, Expected);
	TestNearlyEqual(TEXT("Identity equal low tolerance"), Identical, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Identity equal high tolerance"), Identical, Expected, SmallTolerance);
	TestNearlyEqual(TEXT("Nearby positive equal"), KindaClosePositive, Expected);
	TestNearlyEqual(TEXT("Nearby positive equal low tolerance"), KindaClosePositive, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Nearby positive equal high tolerance"), ExtremelyClosePositive, Expected, SmallTolerance);
	TestNearlyEqual(TEXT("Nearby negative equal"), KindaCloseNegative, Expected);
	TestNearlyEqual(TEXT("Nearby negative equal low tolerance"), KindaCloseNegative, Expected, LargeTolerance);
	TestNearlyEqual(TEXT("Nearby negative equal high tolerance"), ExtremelyCloseNegative, Expected, SmallTolerance);

	TestEqual(TEXT("Identity equal (forwards to TestNearlyEqual)"), Identical, Expected);
	TestEqual(TEXT("Identity equal low tolerance (forwards to TestNearlyEqual)"), Identical, Expected, LargeTolerance);
	TestEqual(TEXT("Identity equal high tolerance (forwards to TestNearlyEqual)"), Identical, Expected, SmallTolerance);
	TestEqual(TEXT("Nearby positive equal (forwards to TestNearlyEqual)"), KindaClosePositive, Expected);
	TestEqual(TEXT("Nearby positive equal low tolerance (forwards to TestNearlyEqual)"), KindaClosePositive, Expected, LargeTolerance);
	TestEqual(TEXT("Nearby positive equal high tolerance (forwards to TestNearlyEqual)"), ExtremelyClosePositive, Expected, SmallTolerance);
	TestEqual(TEXT("Nearby negative equal (forwards to TestNearlyEqual)"), KindaCloseNegative, Expected);
	TestEqual(TEXT("Nearby negative equal low tolerance (forwards to TestNearlyEqual)"), KindaCloseNegative, Expected, LargeTolerance);
	TestEqual(TEXT("Nearby negative equal high tolerance (forwards to TestNearlyEqual)"), ExtremelyCloseNegative, Expected, SmallTolerance);

	TestNotEqual(TEXT("Zero unequal"), Zero, Expected);
	TestNotEqual(TEXT("Positive unequal"), LargePositive, Expected);
	TestNotEqual(TEXT("Nearby positive unequal due to high tolerance"), KindaClosePositive, Expected, SmallTolerance);
	TestNotEqual(TEXT("Negative unequal"), LargeNegative, Expected);
	TestNotEqual(TEXT("Nearby negative unequal due to high tolerance"), KindaClosePositive, Expected, SmallTolerance);
	TestNotEqual(TEXT("Max unequal"), DBL_MAX, Expected);
	TestNotEqual(TEXT("Min unequal"), DBL_MIN, Expected);

	TestLessEqual(TEXT("Identity LE"), Identical, Expected);
	TestLessEqual(TEXT("Identity LE low tolerance"), Identical, Expected, LargeTolerance);
	TestLessEqual(TEXT("Identity LE high tolerance"), Identical, Expected, SmallTolerance);
	TestLessEqual(TEXT("Less LE"), LargeNegative, Expected);
	TestLessThan(TEXT("Min LE"), DBL_MIN, Expected);
	TestLessThan(TEXT("Less than"), LargeNegative, Expected);
	TestLessThan(TEXT("Min less than"), DBL_MIN, Expected);

	TestGreaterEqual(TEXT("Identity GE"), Identical, Expected);
	TestGreaterEqual(TEXT("Identity GE low tolerance"), Identical, Expected, LargeTolerance);
	TestGreaterEqual(TEXT("Identity GE high tolerance"), Identical, Expected, SmallTolerance);
	TestGreaterEqual(TEXT("Less GE"), LargePositive, Expected);
	TestGreaterEqual(TEXT("Max GE"), DBL_MAX, Expected);
	TestGreaterThan(TEXT("Greater than"), LargePositive, Expected);
	TestGreaterThan(TEXT("Max greater than"), DBL_MAX, Expected);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityString, "TestFramework.Validation.TestInequalityString", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTestInequalityString::RunTest(const FString& Parameters)
{
	FString ExpectedString("Forty-two");
	const TCHAR* ExpectedCharPtr = TEXT("Forty-two");
	const UTF8CHAR* ExpectedCharPtrUtf8 = UTF8TEXT("Forty-two");
	FString IdenticalString("Forty-two");
	const TCHAR* IdenticalCharPtr = TEXT("Forty-two");
	FString LowercaseString("forty-two");
	const TCHAR* LowercaseCharPtr = TEXT("forty-two");
	FString UppercaseString("FORTY-TWO");
	const TCHAR* UppercaseCharPtr = TEXT("FORTY-TWO");
	FString EmptyString;
	const TCHAR* EmptyCharPtr = TEXT("");
	FString DifferentString("42");
	const TCHAR* DifferentCharPtr = TEXT("42");
	const TCHAR* NullCharPtr = nullptr;
	const UTF8CHAR* NullCharPtrUtf8 = nullptr;

	TestEqual(TEXT("String identity equal"), IdenticalString, ExpectedString);
	TestEqual(TEXT("char* identity equal"), IdenticalCharPtr, ExpectedCharPtr);
	TestEqual(TEXT("String equals char*"), ExpectedString, ExpectedCharPtr);
	TestEqual(TEXT("char* equals string"), ExpectedCharPtr, ExpectedString);
	TestEqual(TEXT("String equals char* empty"), EmptyString, EmptyCharPtr);
	TestEqual(TEXT("char* equals string empty"), EmptyCharPtr, EmptyString);

	TestNotEqual(TEXT("String unequal"), DifferentString, ExpectedString);
	TestNotEqual(TEXT("char* unequal"), DifferentCharPtr, ExpectedCharPtr);	
	TestNotEqual(TEXT("String unequal empty"), EmptyString, ExpectedString);
	TestNotEqual(TEXT("char* unequal empty"), EmptyCharPtr, ExpectedCharPtr);
	TestNotEqual(TEXT("char* unequal null"), NullCharPtr, ExpectedCharPtr);

	TestEqual(TEXT("String insensitive equal identity"), IdenticalString, ExpectedString);
	TestEqual(TEXT("char* insensitive equal identity"), IdenticalCharPtr, ExpectedCharPtr);
	TestEqual(TEXT("String insensitive equal lower"), LowercaseString, ExpectedString);
	TestEqual(TEXT("char* insensitive equal lower"), LowercaseCharPtr, ExpectedCharPtr);
	TestEqual(TEXT("String insensitive equal upper"), UppercaseString, ExpectedString);
	TestEqual(TEXT("char* insensitive equal upper"), UppercaseCharPtr, ExpectedCharPtr);
	TestNotEqual(TEXT("String insensitive unequal"), DifferentString, ExpectedString);
	TestNotEqual(TEXT("char* insensitive unequal"), DifferentCharPtr, ExpectedCharPtr);
	TestNotEqual(TEXT("char* insensitive unequal null"), NullCharPtr, ExpectedCharPtr);

	TestEqualSensitive(TEXT("String sensitive equal identity"), IdenticalString, ExpectedString);
	TestEqualSensitive(TEXT("char* sensitive equal identity"), IdenticalCharPtr, ExpectedCharPtr);	
	TestNotEqualSensitive(TEXT("String sensitive unequal lower"), LowercaseString, ExpectedString);
	TestNotEqualSensitive(TEXT("char* sensitive unequal lower"), LowercaseCharPtr, ExpectedCharPtr);
	TestNotEqualSensitive(TEXT("String sensitive unequal upper"), UppercaseString, ExpectedString);
	TestNotEqualSensitive(TEXT("char* sensitive unequal upper"), UppercaseCharPtr, ExpectedCharPtr);	
	TestNotEqualSensitive(TEXT("String sensitive unequal"), DifferentString, ExpectedString);
	TestNotEqualSensitive(TEXT("char* sensitive unequal"), DifferentCharPtr, ExpectedCharPtr);
	TestNotEqualSensitive(TEXT("char* sensitive unequal null"), NullCharPtr, ExpectedCharPtr);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTestInequalityStringNulls, "TestFramework.Validation.TestInequalityStringNulls", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::NegativeFilter);
bool FAutomationTestInequalityStringNulls::RunTest(const FString& Parameters)
{
	const TCHAR* ExpectedCharPtr = TEXT("Forty-two");
	const UTF8CHAR* ExpectedCharPtrUtf8 = UTF8TEXT("Forty-two");
	const TCHAR* NullCharPtr = nullptr;
	const UTF8CHAR* NullCharPtrUtf8 = nullptr;

	// TCHAR*
	TestEqual(TEXT("char* equal null null"), NullCharPtr, NullCharPtr);
	TestEqual(TEXT("char* equal null string"), NullCharPtr, ExpectedCharPtr);
	TestEqual(TEXT("char* equal string null"), ExpectedCharPtr, NullCharPtr);

	TestNotEqual(TEXT("char* unequal null null"), NullCharPtr, NullCharPtr);
	TestNotEqual(TEXT("char* unequal null string"), NullCharPtr, ExpectedCharPtr);
	TestNotEqual(TEXT("char* unequal string null"), ExpectedCharPtr, NullCharPtr);

	TestEqualSensitive(TEXT("char* equal(sensitive) null null"), NullCharPtr, NullCharPtr);
	TestEqualSensitive(TEXT("char* equal(sensitive) null string"), NullCharPtr, ExpectedCharPtr);
	TestEqualSensitive(TEXT("char* equal(sensitive) string null"), ExpectedCharPtr, NullCharPtr);

	TestNotEqualSensitive(TEXT("char* unequal(sensitive) null null"), NullCharPtr, NullCharPtr);
	TestNotEqualSensitive(TEXT("char* unequal(sensitive) null string"), NullCharPtr, ExpectedCharPtr);
	TestNotEqualSensitive(TEXT("char* unequal(sensitive) string null"), ExpectedCharPtr, NullCharPtr);

	// TStringView
	TestEqual(TEXT("stringview equal null null"), MakeStringView(NullCharPtr), MakeStringView(NullCharPtr));
	TestEqual(TEXT("stringview equal null string"), MakeStringView(NullCharPtr), MakeStringView(ExpectedCharPtr));
	TestEqual(TEXT("stringview equal string null"), MakeStringView(ExpectedCharPtr), MakeStringView(NullCharPtr));

	TestNotEqual(TEXT("stringview unequal null null"), MakeStringView(NullCharPtr), MakeStringView(NullCharPtr));
	TestNotEqual(TEXT("stringview unequal null string"), MakeStringView(NullCharPtr), MakeStringView(ExpectedCharPtr));
	TestNotEqual(TEXT("stringview unequal string null"), MakeStringView(ExpectedCharPtr), MakeStringView(NullCharPtr));

	TestEqualSensitive(TEXT("stringview equal(sensitive) null null"), MakeStringView(NullCharPtr), MakeStringView(NullCharPtr));
	TestEqualSensitive(TEXT("stringview equal(sensitive) null string"), MakeStringView(NullCharPtr), MakeStringView(ExpectedCharPtr));
	TestEqualSensitive(TEXT("stringview equal(sensitive) string null"), MakeStringView(ExpectedCharPtr), MakeStringView(NullCharPtr));

	TestNotEqualSensitive(TEXT("stringview unequal(sensitive) null null"), MakeStringView(NullCharPtr), MakeStringView(NullCharPtr));
	TestNotEqualSensitive(TEXT("stringview unequal(sensitive) null string"), MakeStringView(NullCharPtr), MakeStringView(ExpectedCharPtr));
	TestNotEqualSensitive(TEXT("stringview unequal(sensitive) string null"), MakeStringView(ExpectedCharPtr), MakeStringView(NullCharPtr));

	// FUtf8StringView
	TestEqual(TEXT("stringview8 equal null null"), MakeStringView(NullCharPtrUtf8), MakeStringView(NullCharPtrUtf8));
	TestEqual(TEXT("stringview8 equal null string"), MakeStringView(NullCharPtrUtf8), MakeStringView(ExpectedCharPtrUtf8));
	TestEqual(TEXT("stringview8 equal string null"), MakeStringView(ExpectedCharPtrUtf8), MakeStringView(NullCharPtrUtf8));

	TestNotEqual(TEXT("stringview8 unequal null null"), MakeStringView(NullCharPtrUtf8), MakeStringView(NullCharPtrUtf8));
	TestNotEqual(TEXT("stringview8 unequal null string"), MakeStringView(NullCharPtrUtf8), MakeStringView(ExpectedCharPtrUtf8));
	TestNotEqual(TEXT("stringview8 unequal string null"), MakeStringView(ExpectedCharPtrUtf8), MakeStringView(NullCharPtrUtf8));

	TestEqualSensitive(TEXT("stringview8 equal(sensitive) null null"), MakeStringView(NullCharPtrUtf8), MakeStringView(NullCharPtrUtf8));
	TestEqualSensitive(TEXT("stringview8 equal(sensitive) null string"), MakeStringView(NullCharPtrUtf8), MakeStringView(ExpectedCharPtrUtf8));
	TestEqualSensitive(TEXT("stringview8 equal(sensitive) string null"), MakeStringView(ExpectedCharPtrUtf8), MakeStringView(NullCharPtrUtf8));

	TestNotEqualSensitive(TEXT("stringview8 unequal(sensitive) null null"), MakeStringView(NullCharPtrUtf8), MakeStringView(NullCharPtrUtf8));
	TestNotEqualSensitive(TEXT("stringview8 unequal(sensitive) null string"), MakeStringView(NullCharPtrUtf8), MakeStringView(ExpectedCharPtrUtf8));
	TestNotEqualSensitive(TEXT("stringview8 unequal(sensitive) string null"), MakeStringView(ExpectedCharPtrUtf8), MakeStringView(NullCharPtrUtf8));

	return true;
}

class FAutomationUTestMacrosExpr : public FAutomationTestBase
{
public:
	FAutomationUTestMacrosExpr(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

protected:

	static const float PositiveToleranceFloat;
	static const float ActualFloatValue;
	static const float ExpectedFloatValue;
	static const float WrongFloatValue;
	static const float ExpectedFloatValueOutOfTolerance;
	static const float ExpectedFloatValueOutOfToleranceNegative;
	static const float ExpectedFloatValueLess;
	static const float ExpectedFloatValueGreater;
	static const FString ActualFStringValue;
	static const FString ActualFStringValueCopy;
	static const FString ExpectedFStringValueLowerCase;
	static const FString UnexpectedFStringValueLowerCase;
	static const FString CustomDescriptionString;
};

const float FAutomationUTestMacrosExpr::PositiveToleranceFloat(1.e-4f);
const float FAutomationUTestMacrosExpr::ActualFloatValue(0.f);
const float FAutomationUTestMacrosExpr::ExpectedFloatValue(ActualFloatValue);
const float FAutomationUTestMacrosExpr::WrongFloatValue(ActualFloatValue + 1.f);
const float FAutomationUTestMacrosExpr::ExpectedFloatValueOutOfTolerance(ActualFloatValue + PositiveToleranceFloat);
const float FAutomationUTestMacrosExpr::ExpectedFloatValueOutOfToleranceNegative(ActualFloatValue - PositiveToleranceFloat);
const float FAutomationUTestMacrosExpr::ExpectedFloatValueLess(ActualFloatValue + (PositiveToleranceFloat*2)); //actual < expected
const float FAutomationUTestMacrosExpr::ExpectedFloatValueGreater(ActualFloatValue - (PositiveToleranceFloat*2)); //actual > expected
const FString FAutomationUTestMacrosExpr::ActualFStringValue(TEXT("EQUALS"));
const FString FAutomationUTestMacrosExpr::ActualFStringValueCopy(TEXT("EQUALS"));
const FString FAutomationUTestMacrosExpr::ExpectedFStringValueLowerCase(TEXT("equals"));
const FString FAutomationUTestMacrosExpr::UnexpectedFStringValueLowerCase(TEXT("not-equals"));
const FString FAutomationUTestMacrosExpr::CustomDescriptionString(TEXT("Error string appears when UTEST_ macro diverges from _EXPR variant"));

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationEqualEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestEqual", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationEqualEXPR::RunTest(const FString& Parameters)
{

	UTEST_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValue);
	UTEST_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValue);
	UTEST_NEARLY_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValueOutOfTolerance, PositiveToleranceFloat);
	UTEST_NEARLY_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueOutOfTolerance, PositiveToleranceFloat);
	UTEST_EQUAL_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValueOutOfTolerance, PositiveToleranceFloat);
	UTEST_EQUAL_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueOutOfTolerance, PositiveToleranceFloat);
	UTEST_NOT_EQUAL_EXPR(ActualFloatValue, WrongFloatValue);
	UTEST_NOT_EQUAL(CustomDescriptionString, ActualFloatValue, WrongFloatValue);
	UTEST_EQUAL_INSENSITIVE_EXPR(*ActualFStringValue, *ExpectedFStringValueLowerCase);
	UTEST_EQUAL_INSENSITIVE(*CustomDescriptionString, *ActualFStringValue, *ExpectedFStringValueLowerCase);
	UTEST_NOT_EQUAL_INSENSITIVE_EXPR(*ActualFStringValue, *UnexpectedFStringValueLowerCase);
	UTEST_NOT_EQUAL_INSENSITIVE(*CustomDescriptionString, *ActualFStringValue, *UnexpectedFStringValueLowerCase);
	UTEST_EQUAL_SENSITIVE_EXPR(*ActualFStringValue, *ActualFStringValueCopy);
	UTEST_EQUAL_SENSITIVE(*CustomDescriptionString, *ActualFStringValue, *ActualFStringValueCopy);
	UTEST_NOT_EQUAL_SENSITIVE_EXPR(*ActualFStringValue, *ExpectedFStringValueLowerCase);
	UTEST_NOT_EQUAL_SENSITIVE(*CustomDescriptionString, *ActualFStringValue, *ExpectedFStringValueLowerCase);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationSameNotSameEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestSameNotSame", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationSameNotSameEXPR::RunTest(const FString& Parameters)
{

	UTEST_SAME_EXPR(ActualFStringValue, ActualFStringValue);
	UTEST_SAME(CustomDescriptionString, ActualFStringValue, ActualFStringValue);
	UTEST_NOT_SAME_EXPR(ActualFStringValue, ExpectedFStringValueLowerCase);
	UTEST_NOT_SAME(CustomDescriptionString, ActualFStringValue, ExpectedFStringValueLowerCase);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationSameNotSamePtrEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestSameNotSamePtr", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationSameNotSamePtrEXPR::RunTest(const FString& Parameters)
{

	UTEST_SAME_PTR_EXPR(&ActualFStringValue, &ActualFStringValue);
	UTEST_SAME_PTR(CustomDescriptionString, &ActualFStringValue, &ActualFStringValue);
	UTEST_NOT_SAME_PTR_EXPR(&ActualFStringValue, &ExpectedFStringValueLowerCase);
	UTEST_NOT_SAME_PTR(CustomDescriptionString, &ActualFStringValue, &ExpectedFStringValueLowerCase);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTrueFalseEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestTrueFalse", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationTrueFalseEXPR::RunTest(const FString& Parameters)
{
	UTEST_TRUE_EXPR(ActualFloatValue == ExpectedFloatValue);
	UTEST_TRUE(CustomDescriptionString, ActualFloatValue == ExpectedFloatValue);
	UTEST_FALSE_EXPR(ActualFloatValue > ExpectedFloatValue);
	UTEST_FALSE(CustomDescriptionString, ActualFloatValue > ExpectedFloatValue);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationValidInvalidEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestValidInvalid", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationValidInvalidEXPR::RunTest(const FString& Parameters)
{
	struct FHasIsValid
	{
		explicit FHasIsValid(bool bInIsValid)
			: bIsValid(bInIsValid)
		{  }
		
		bool IsValid() const { return bIsValid; }

	private:
		bool bIsValid;
	};
	
	//** TEST **//
	TSharedPtr<FVector> ValidSharedPtr = MakeShared<FVector>();
	TSharedPtr<UObject> InvalidSharedPtr = nullptr;

	FHasIsValid ValidObject(true);
	FHasIsValid InvalidObject(false);

	//** VERIFY **//
	UTEST_VALID_EXPR(ValidSharedPtr);
	UTEST_VALID(CustomDescriptionString, ValidSharedPtr);
	UTEST_INVALID_EXPR(InvalidSharedPtr);
	UTEST_INVALID(CustomDescriptionString, InvalidSharedPtr);

	UTEST_VALID_EXPR(ValidObject);
	UTEST_VALID(CustomDescriptionString, ValidObject);
	UTEST_INVALID_EXPR(InvalidObject);
	UTEST_INVALID(CustomDescriptionString, InvalidObject);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationNullNotNullPtrEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestNullNotNull", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationNullNotNullPtrEXPR::RunTest(const FString& Parameters)
{
	UWorld* World = UWorld::CreateWorld(EWorldType::Game, false);

	UTEST_NULL_EXPR(nullptr);
	UTEST_NULL(CustomDescriptionString, nullptr);
	UTEST_NOT_NULL_EXPR(World);
	UTEST_NOT_NULL(CustomDescriptionString, World);

	World->DestroyWorld(false);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationLessGreaterEXPR, FAutomationUTestMacrosExpr, "TestFramework.Validation.UTestLessGreater", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAutomationLessGreaterEXPR::RunTest(const FString& Parameters)
{
	// inequality
	UTEST_LESS_EXPR(ActualFloatValue, ExpectedFloatValueLess);
	UTEST_LESS(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueLess);
	UTEST_LESS_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValueLess, PositiveToleranceFloat);
	UTEST_LESS_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueLess, PositiveToleranceFloat);
	UTEST_LESS_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValueLess);
	UTEST_LESS_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueLess);
	UTEST_LESS_EQUAL_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValueLess, PositiveToleranceFloat);
	UTEST_LESS_EQUAL_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueLess, PositiveToleranceFloat);
	UTEST_GREATER_EXPR(ActualFloatValue, ExpectedFloatValueGreater);
	UTEST_GREATER(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueGreater);
	UTEST_GREATER_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValueGreater, PositiveToleranceFloat);
	UTEST_GREATER_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueGreater, PositiveToleranceFloat);
	UTEST_GREATER_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValueGreater);
	UTEST_GREATER_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueGreater);
	UTEST_GREATER_EQUAL_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValueGreater, PositiveToleranceFloat);
	UTEST_GREATER_EQUAL_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValueGreater, PositiveToleranceFloat);

	// equality
	UTEST_LESS_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValue);
	UTEST_LESS_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValue);
	UTEST_LESS_EQUAL_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValue, PositiveToleranceFloat);
	UTEST_LESS_EQUAL_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValue, PositiveToleranceFloat);
	UTEST_GREATER_EQUAL_EXPR(ActualFloatValue, ExpectedFloatValue);
	UTEST_GREATER_EQUAL(CustomDescriptionString, ActualFloatValue, ExpectedFloatValue);
	UTEST_GREATER_EQUAL_TOLERANCE_EXPR(ActualFloatValue, ExpectedFloatValue, PositiveToleranceFloat);
	UTEST_GREATER_EQUAL_TOLERANCE(CustomDescriptionString, ActualFloatValue, ExpectedFloatValue, PositiveToleranceFloat);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationAttachment, "System.Engine.Attachment", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#define DUMP_EXPECTED_TRANSFORMS 0
#define TEST_NEW_ATTACHMENTS 1
#define TEST_DEPRECATED_ATTACHMENTS 0

namespace AttachTestConstants
{
	const FVector ParentLocation(1.0f, -2.0f, 4.0f);
	const FQuat ParentRotation(FRotator(0.0f, 45.0f, 45.0f).Quaternion());
	const FVector ParentScale(1.25f, 1.25f, 1.25f);
	const FVector ChildLocation(2.0f, -8.0f, -4.0f);
	const FQuat ChildRotation(FRotator(0.0f, 45.0f, 20.0f).Quaternion());
	const FVector ChildScale(1.25f, 1.25f, 1.25f);
}

void AttachmentTest_CommonTests(AActor* ParentActor, AActor* ChildActor, FAutomationTestBase* Test)
{
#if TEST_DEPRECATED_ATTACHMENTS
	static const FTransform LegacyExpectedChildTransforms[4][2] =
	{
		{
			FTransform(
				FQuat(-0.49031073f, -0.11344112f, 0.64335662f, 0.57690436f),
				FVector(10.26776695f, -7.73223305f, 7.53553343f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
			FTransform(
				FQuat(-0.49031067f, -0.11344092f, 0.64335656f, 0.57690459f),
				FVector(10.26776695f, -7.73223305f, 7.53553343f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
		},
		{
			FTransform(
				FQuat(-0.49031067f, -0.11344092f, 0.64335662f, 0.57690459f),
				FVector(10.26776695f, -7.73223305f, 7.53553343f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
			FTransform(
				FQuat(-0.49031061f, -0.11344086f, 0.64335656f, 0.57690465f),
				FVector(10.26776695f, -7.73223305f, 7.53553343f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
		},
		{
			FTransform(
				FQuat(-0.35355338f, -0.14644660f, 0.35355338f, 0.85355335f),
				FVector(1.00000000f, -2.00000000f, 4.00000000f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
			FTransform(
				FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
				FVector(1.00000000f, -2.00000000f, 4.00000000f),
				FVector(1.56250000f, 1.56250000f, 1.56250000f)
			),
		},
		{
			FTransform(
				FQuat(-0.35355338f, -0.14644660f, 0.35355338f, 0.85355335f),
				FVector(1.00000000f, -2.00000000f, 4.00000000f),
				FVector(1.25000000f, 1.25000000f, 1.25000000f)
			),
			FTransform(
				FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
				FVector(1.00000000f, -2.00000000f, 4.00000000f),
				FVector(1.25000000f, 1.25000000f, 1.25000000f)
			),
		},
	};

	for (uint8 LocationInteger = (uint8)EAttachLocation::KeepRelativeOffset; LocationInteger <= (uint8)EAttachLocation::SnapToTargetIncludingScale; ++LocationInteger)
	{
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "{");
#endif
		EAttachLocation::Type Location = (EAttachLocation::Type)LocationInteger;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ChildActor->AttachRootComponentToActor(ParentActor, NAME_None, Location, true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// check parent actor is unaffected by attachment
		Test->TestEqual<FVector>(TEXT("Parent location was affected by attachment"), ParentActor->GetActorLocation(), AttachTestConstants::ParentLocation);
		Test->TestEqual<FQuat>(TEXT("Parent rotation was affected by attachment"), ParentActor->GetActorQuat(), AttachTestConstants::ParentRotation);
		Test->TestEqual<FVector>(TEXT("Parent scale was affected by attachment"), ParentActor->GetActorScale3D(), AttachTestConstants::ParentScale);

		// check we have expected transforms for each mode
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "\tFTransform(");
		UE_LOGF(LogTemp, Log, "\t\tFQuat(%.8ff, %.8ff, %.8ff, %.8ff),", ChildActor->GetActorQuat().X, ChildActor->GetActorQuat().Y, ChildActor->GetActorQuat().Z, ChildActor->GetActorQuat().W);
		UE_LOGF(LogTemp, Log, "\t\tFVector(%.8ff, %.8ff, %.8ff),", ChildActor->GetActorLocation().X, ChildActor->GetActorLocation().Y, ChildActor->GetActorLocation().Z);
		UE_LOGF(LogTemp, Log, "\t\tFVector(%.8ff, %.8ff, %.8ff)", ChildActor->GetActorScale3D().X, ChildActor->GetActorScale3D().Y, ChildActor->GetActorScale3D().Z);
		UE_LOGF(LogTemp, Log, "\t),");
#endif

		Test->TestTrue(FString::Printf(TEXT("Child world location was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorLocation().ToString(), *LegacyExpectedChildTransforms[LocationInteger][0].GetLocation().ToString()), ChildActor->GetActorLocation().Equals(LegacyExpectedChildTransforms[LocationInteger][0].GetLocation(), KINDA_SMALL_NUMBER));
		Test->TestTrue(FString::Printf(TEXT("Child world rotation was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorQuat().ToString(), *LegacyExpectedChildTransforms[LocationInteger][0].GetRotation().ToString()), ChildActor->GetActorQuat().Equals(LegacyExpectedChildTransforms[LocationInteger][0].GetRotation(), KINDA_SMALL_NUMBER));
		Test->TestTrue(FString::Printf(TEXT("Child world scale was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorScale3D().ToString(), *LegacyExpectedChildTransforms[LocationInteger][0].GetScale3D().ToString()), ChildActor->GetActorScale3D().Equals(LegacyExpectedChildTransforms[LocationInteger][0].GetScale3D(), KINDA_SMALL_NUMBER));

		ChildActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

		// check we have expected values after detachment
		Test->TestEqual<FVector>(TEXT("Parent location was affected by detachment"), ParentActor->GetActorLocation(), AttachTestConstants::ParentLocation);
		Test->TestEqual<FQuat>(TEXT("Parent rotation was affected by detachment"), ParentActor->GetActorQuat(), AttachTestConstants::ParentRotation);
		Test->TestEqual<FVector>(TEXT("Parent scale was affected by detachment"), ParentActor->GetActorScale3D(), AttachTestConstants::ParentScale);

		// check we have expected transforms for each mode
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "\tFTransform(");
		UE_LOGF(LogTemp, Log, "\t\tFQuat(%.8ff, %.8ff, %.8ff, %.8ff),", ChildActor->GetActorQuat().X, ChildActor->GetActorQuat().Y, ChildActor->GetActorQuat().Z, ChildActor->GetActorQuat().W);
		UE_LOGF(LogTemp, Log, "\t\tFVector(%.8ff, %.8ff, %.8ff),", ChildActor->GetActorLocation().X, ChildActor->GetActorLocation().Y, ChildActor->GetActorLocation().Z);
		UE_LOGF(LogTemp, Log, "\t\tFVector(%.8ff, %.8ff, %.8ff)", ChildActor->GetActorScale3D().X, ChildActor->GetActorScale3D().Y, ChildActor->GetActorScale3D().Z);
		UE_LOGF(LogTemp, Log, "\t),");
#endif

		Test->TestTrue(FString::Printf(TEXT("Child relative location was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorLocation().ToString(), *LegacyExpectedChildTransforms[LocationInteger][1].GetLocation().ToString()), ChildActor->GetActorLocation().Equals(LegacyExpectedChildTransforms[LocationInteger][1].GetLocation(), KINDA_SMALL_NUMBER));
		Test->TestTrue(FString::Printf(TEXT("Child relative rotation was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorQuat().ToString(), *LegacyExpectedChildTransforms[LocationInteger][1].GetRotation().ToString()), ChildActor->GetActorQuat().Equals(LegacyExpectedChildTransforms[LocationInteger][1].GetRotation(), KINDA_SMALL_NUMBER));
		Test->TestTrue(FString::Printf(TEXT("Child relative scale was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorScale3D().ToString(), *LegacyExpectedChildTransforms[LocationInteger][1].GetScale3D().ToString()), ChildActor->GetActorScale3D().Equals(LegacyExpectedChildTransforms[LocationInteger][1].GetScale3D(), KINDA_SMALL_NUMBER));
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "},");
#endif
	}
#endif

#if TEST_NEW_ATTACHMENTS
	// Check each component against each rule in all combinations, pre and post-detachment
	static const FTransform ExpectedChildTransforms[3][3][3][2] =
	{
		{
			{
				{
					FTransform(
						FQuat(-0.49031073f, -0.11344108f, 0.64335668f, 0.57690459f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.16042995f, -0.06645225f, 0.37686956f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.49031073f, -0.11344108f, 0.64335668f, 0.57690459f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.16042995f, -0.06645225f, 0.37686956f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.49031073f, -0.11344108f, 0.64335668f, 0.57690459f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.16042995f, -0.06645225f, 0.37686956f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.16042994f, -0.06645226f, 0.37686956f, 0.90984380f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.16042991f, -0.06645230f, 0.37686959f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.16042991f, -0.06645229f, 0.37686959f, 0.90984380f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.16042989f, -0.06645229f, 0.37686959f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.16042989f, -0.06645229f, 0.37686956f, 0.90984380f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.16042989f, -0.06645229f, 0.37686959f, 0.90984380f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(10.26776695f, -7.73223495f, 7.53553295f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(2.00000000f, -8.00000000f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
		},
		{
			{
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.99999976f, -8.00000000f, -4.00000095f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999976f, -8.00000000f, -4.00000095f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.99999881f, -8.00000095f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999881f, -8.00000095f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.99999857f, -8.00000191f, -4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
		},
		{
			{
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.60355335f, -0.24999997f, 0.60355341f, 0.45710698f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644657f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
			{
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.56250000f, 1.56250000f, 1.56250000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
				{
					FTransform(
						FQuat(-0.35355335f, -0.14644656f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
					FTransform(
						FQuat(-0.35355335f, -0.14644659f, 0.35355335f, 0.85355347f),
						FVector(1.00000000f, -2.00000000f, 4.00000000f),
						FVector(1.25000000f, 1.25000000f, 1.25000000f)
					),
				},
			},
		},
	};

	const FVector ParentPreAttachmentLocation = ParentActor->GetActorLocation();
	const FQuat ParentPreAttachmentRotation = ParentActor->GetActorQuat();
	const FVector ParentPreAttachmentScale = ParentActor->GetActorScale3D();

	for (uint8 RuleInteger0 = (uint8)EAttachmentRule::KeepRelative; RuleInteger0 <= (uint8)EAttachmentRule::SnapToTarget; ++RuleInteger0)
	{
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "{");
#endif
		for (uint8 RuleInteger1 = (uint8)EAttachmentRule::KeepRelative; RuleInteger1 <= (uint8)EAttachmentRule::SnapToTarget; ++RuleInteger1)
		{
#if DUMP_EXPECTED_TRANSFORMS
			UE_LOGF(LogTemp, Log, "\t{");
#endif
			for (uint8 RuleInteger2 = (uint8)EAttachmentRule::KeepRelative; RuleInteger2 <= (uint8)EAttachmentRule::SnapToTarget; ++RuleInteger2)
			{
#if DUMP_EXPECTED_TRANSFORMS
				UE_LOGF(LogTemp, Log, "\t\t{");
#endif
				EAttachmentRule Rule0 = (EAttachmentRule)RuleInteger0;
				EAttachmentRule Rule1 = (EAttachmentRule)RuleInteger1;
				EAttachmentRule Rule2 = (EAttachmentRule)RuleInteger2;

				FAttachmentTransformRules Rules(Rule0, Rule1, Rule2, false);

				ChildActor->AttachToActor(ParentActor, Rules);

				// check parent actor is unaffected by attachment
				Test->TestEqual<FVector>(TEXT("Parent location was affected by attachment"), ParentActor->GetActorLocation(), ParentPreAttachmentLocation);
				Test->TestEqual<FQuat>(TEXT("Parent rotation was affected by attachment"), ParentActor->GetActorQuat(), ParentPreAttachmentRotation);
				Test->TestEqual<FVector>(TEXT("Parent scale was affected by attachment"), ParentActor->GetActorScale3D(), ParentPreAttachmentScale);

				// check we have expected transforms for each mode
#if DUMP_EXPECTED_TRANSFORMS
				UE_LOGF(LogTemp, Log, "\t\t\tFTransform(");
				UE_LOGF(LogTemp, Log, "\t\t\t\tFQuat(%.8ff, %.8ff, %.8ff, %.8ff),", ChildActor->GetActorQuat().X, ChildActor->GetActorQuat().Y, ChildActor->GetActorQuat().Z, ChildActor->GetActorQuat().W);
				UE_LOGF(LogTemp, Log, "\t\t\t\tFVector(%.8ff, %.8ff, %.8ff),", ChildActor->GetActorLocation().X, ChildActor->GetActorLocation().Y, ChildActor->GetActorLocation().Z);
				UE_LOGF(LogTemp, Log, "\t\t\t\tFVector(%.8ff, %.8ff, %.8ff)", ChildActor->GetActorScale3D().X, ChildActor->GetActorScale3D().Y, ChildActor->GetActorScale3D().Z);
				UE_LOGF(LogTemp, Log, "\t\t\t),");
#endif

				Test->TestTrue(FString::Printf(TEXT("Child world location was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorLocation().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetLocation().ToString()), ChildActor->GetActorLocation().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetLocation(), UE_KINDA_SMALL_NUMBER));
				Test->TestTrue(FString::Printf(TEXT("Child world rotation was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorQuat().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetRotation().ToString()), ChildActor->GetActorQuat().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetRotation(), UE_KINDA_SMALL_NUMBER));
				Test->TestTrue(FString::Printf(TEXT("Child world scale was incorrect after attachment (was %s, should be %s)"), *ChildActor->GetActorScale3D().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetScale3D().ToString()), ChildActor->GetActorScale3D().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][0].GetScale3D(), UE_KINDA_SMALL_NUMBER));

				ChildActor->DetachFromActor(FDetachmentTransformRules(Rules, true));

				// check we have expected values after detachment
				Test->TestEqual<FVector>(TEXT("Parent location was affected by detachment"), ParentActor->GetActorLocation(), ParentPreAttachmentLocation);
				Test->TestEqual<FQuat>(TEXT("Parent rotation was affected by detachment"), ParentActor->GetActorQuat(), ParentPreAttachmentRotation);
				Test->TestEqual<FVector>(TEXT("Parent scale was affected by detachment"), ParentActor->GetActorScale3D(), ParentPreAttachmentScale);

				// check we have expected transforms for each mode
#if DUMP_EXPECTED_TRANSFORMS
				UE_LOGF(LogTemp, Log, "\t\t\tFTransform(");
				UE_LOGF(LogTemp, Log, "\t\t\t\tFQuat(%.8ff, %.8ff, %.8ff, %.8ff),", ChildActor->GetActorQuat().X, ChildActor->GetActorQuat().Y, ChildActor->GetActorQuat().Z, ChildActor->GetActorQuat().W);
				UE_LOGF(LogTemp, Log, "\t\t\t\tFVector(%.8ff, %.8ff, %.8ff),", ChildActor->GetActorLocation().X, ChildActor->GetActorLocation().Y, ChildActor->GetActorLocation().Z);
				UE_LOGF(LogTemp, Log, "\t\t\t\tFVector(%.8ff, %.8ff, %.8ff)", ChildActor->GetActorScale3D().X, ChildActor->GetActorScale3D().Y, ChildActor->GetActorScale3D().Z);
				UE_LOGF(LogTemp, Log, "\t\t\t),");
#endif

				Test->TestTrue(FString::Printf(TEXT("Child relative location was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorLocation().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetLocation().ToString()), ChildActor->GetActorLocation().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetLocation(), UE_KINDA_SMALL_NUMBER));
				Test->TestTrue(FString::Printf(TEXT("Child relative rotation was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorQuat().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetRotation().ToString()), ChildActor->GetActorQuat().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetRotation(), UE_KINDA_SMALL_NUMBER));
				Test->TestTrue(FString::Printf(TEXT("Child relative scale was incorrect after detachment (was %s, should be %s)"), *ChildActor->GetActorScale3D().ToString(), *ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetScale3D().ToString()), ChildActor->GetActorScale3D().Equals(ExpectedChildTransforms[RuleInteger0][RuleInteger1][RuleInteger2][1].GetScale3D(), UE_KINDA_SMALL_NUMBER));
#if DUMP_EXPECTED_TRANSFORMS
				UE_LOGF(LogTemp, Log, "\t\t},");
#endif
			}
#if DUMP_EXPECTED_TRANSFORMS
			UE_LOGF(LogTemp, Log, "\t},");
#endif
		}
#if DUMP_EXPECTED_TRANSFORMS
		UE_LOGF(LogTemp, Log, "},");
#endif
	}
#endif // TEST_NEW_ATTACHMENTS
}


void AttachmentTest_SetupParentAndChild(UWorld* World, AActor*& InOutParentActor, AActor*& InOutChildActor)
{
	ADefaultPawn* ParentActor = NewObject<ADefaultPawn>(World->PersistentLevel);
	ParentActor->SetActorLocation(AttachTestConstants::ParentLocation);
	ParentActor->SetActorRotation(AttachTestConstants::ParentRotation);
	ParentActor->SetActorScale3D(AttachTestConstants::ParentScale);

	ADefaultPawn* ChildActor = NewObject<ADefaultPawn>(World->PersistentLevel);
	ChildActor->SetActorLocation(AttachTestConstants::ChildLocation);
	ChildActor->SetActorRotation(AttachTestConstants::ChildRotation);
	ChildActor->SetActorScale3D(AttachTestConstants::ChildScale);

	InOutParentActor = ParentActor;
	InOutChildActor = ChildActor;
}

void AttachmentTest_AttachWhenNotAttached(UWorld* World, FAutomationTestBase* Test)
{
	AActor* ParentActor = nullptr;
	AActor* ChildActor = nullptr;
	AttachmentTest_SetupParentAndChild(World, ParentActor, ChildActor);

	AttachmentTest_CommonTests(ParentActor, ChildActor, Test);
}

void AttachmentTest_AttachWhenAttached(UWorld* World, FAutomationTestBase* Test)
{
	ADefaultPawn* PreviousParentActor = NewObject<ADefaultPawn>(World->PersistentLevel);
	PreviousParentActor->SetActorLocation(FVector::ZeroVector);
	PreviousParentActor->SetActorRotation(FQuat::Identity);
	PreviousParentActor->SetActorScale3D(FVector(1.0f));

	AActor* ParentActor = nullptr;
	AActor* ChildActor = nullptr;
	AttachmentTest_SetupParentAndChild(World, ParentActor, ChildActor);

#if TEST_DEPRECATED_ATTACHMENTS
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ChildActor->AttachRootComponentToActor(PreviousParentActor, NAME_None, EAttachLocation::KeepWorldPosition, true);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
	ChildActor->AttachToActor(PreviousParentActor, FAttachmentTransformRules(EAttachmentRule::KeepWorld, false));
#endif

	AttachmentTest_CommonTests(ParentActor, ChildActor, Test);
}

bool FAutomationAttachment::RunTest(const FString& Parameters)
{
	// This will get cleaned up when it leaves scope
	FTestWorldWrapper WorldWrapper;
	WorldWrapper.CreateTestWorld(EWorldType::Game);
	UWorld* World = WorldWrapper.GetTestWorld();

	if (World)
	{
		WorldWrapper.BeginPlayInTestWorld();
		AttachmentTest_AttachWhenNotAttached(World, this);
		AttachmentTest_AttachWhenAttached(World, this);
		WorldWrapper.ForwardErrorMessages(this);

		return !HasAnyErrors();
	}
	return false;
}

constexpr auto ExampleTag = "[TestExampleTag][MustPass]";
constexpr auto OtherTag = "[SomeOtherTag]";
constexpr auto NegativeTag = "[DoNotWant]";
constexpr auto NegativeAndExampleTags = "[TestExampleTag][DoNotWant]";
constexpr auto NegativeAndExampleTagsReversed = "[TestExampleTag][DoNotWant]";

constexpr auto FullTestNameTagsExist = "TestFramework.Tags.TagsExist";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsExist, FullTestNameTagsExist, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsExist, FullTestNameTagsExist, ExampleTag) //Other tests expect ExampleTag to only be registered once, and only for this test
bool FAutomationTagsExist::RunTest(const FString& Parameters)
{
	//registration above performs necessary setup
	FString MyTags = FAutomationTestFramework::Get().GetTagsForAutomationTest(FullTestNameTagsExist);

	TestEqual(TEXT("Tags statically register"), MyTags, ExampleTag);
	return true;
}

constexpr auto FullTestNameSelect = "TestFramework.Tags.TagsAreSelectable";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsSelect, FullTestNameSelect, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsSelect, FullTestNameSelect, NegativeAndExampleTags)
bool FAutomationTagsSelect::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString PositiveFilter("[TestExampleTag]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, PositiveFilter);

	TestGreaterEqual("Tags get selected", TestNames.Num(), 1);
	bool FoundThisTest = false;
	for (FString Element : TestNames)
	{
		if (Element.Equals(FullTestNameSelect))
		{
			FoundThisTest = true;
		}
	}
	TestTrue("Current test was selected", FoundThisTest);
	return true;
}

constexpr auto FullTestNameUnion = "TestFramework.Tags.UnionSelection";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsUnion, FullTestNameUnion, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
//The two tests above are used, no extra tags need to be registered
bool FAutomationTagsUnion::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString UnionFilter("[TestExampleTag] OR [SomeOtherTag]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, UnionFilter);

	TestGreaterEqual("Tags get selected", TestNames.Num(), 1);
	bool FoundExample = false;
	bool FoundOther = false;
	for (FString Element : TestNames)
	{
		if (Element.Equals(FullTestNameTagsExist))
		{
			FoundExample = true;
		}
		else {
			if (Element.Equals(FullTestNameSelect))
			{
				FoundOther = true;
			}
		}
	}
	TestTrue("First test was selected", FoundExample);
	TestTrue("Second test was selected", FoundOther);
	return true;
}

constexpr auto FullTestNameNoBracket = "TestFramework.Tags.SelectWithoutBrackets";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsFilterNoBrackets, FullTestNameNoBracket, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsFilterNoBrackets, FullTestNameNoBracket, OtherTag)
bool FAutomationTagsFilterNoBrackets::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString PositiveFilter("SomeOtherTag");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, PositiveFilter);

	TestGreaterEqual("Tags get selected", TestNames.Num(), 1);
	bool FoundThisTest = false;
	for (FString Element : TestNames)
	{
		if (Element.Equals(FullTestNameNoBracket))
		{
			FoundThisTest = true;
		}
	}
	TestTrue("Current test was selected", FoundThisTest);
	return true;
}

constexpr auto FullTestNameFilter = "TestFramework.Tags.TagsCanFilter";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsFilter, FullTestNameFilter, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsFilter, FullTestNameFilter, NegativeAndExampleTags)
bool FAutomationTagsFilter::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString NegativeFilter("[TestExampleTag] AND NOT [DoNotWant]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, NegativeFilter);

	TestEqual("One element", TestNames.Num(), 1 );
	TestEqual("Current test is not selected", TestNames[0], FullTestNameTagsExist);
	return true;
}

constexpr auto FullTestNameFilterReversePattern = "TestFramework.Tags.TagsFilterPatternOrderIndependent";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsFilterPatternOrder, FullTestNameFilterReversePattern, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsFilterPatternOrder, FullTestNameFilterReversePattern, NegativeAndExampleTags)
bool FAutomationTagsFilterPatternOrder::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString NegativeFilter("NOT [DoNotWant] AND [TestExampleTag]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, NegativeFilter);

	TestEqual("One element", TestNames.Num(), 1);
	TestEqual("Current test is not selected", TestNames[0], FullTestNameTagsExist);
	return true;
}

constexpr auto FullTestNameFilterReverseTags = "TestFramework.Tags.TagsFilterTagOrderIndependent";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsFilterTagOrder, FullTestNameFilterReverseTags, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsFilterTagOrder, FullTestNameFilterReverseTags, NegativeAndExampleTagsReversed)
bool FAutomationTagsFilterTagOrder::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString NegativeFilter("[TestExampleTag] AND NOT [DoNotWant]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, NegativeFilter);

	TestEqual("One element", TestNames.Num(), 1);
	TestEqual("Current test is not selected", TestNames[0], FullTestNameTagsExist);
	return true;
}

constexpr auto FullTestNameFilterReverseBoth = "TestFramework.Tags.TagsFilterReverseOrderIndependent";
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutomationTagsFilterBothReverseOrder, FullTestNameFilterReverseBoth, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FAutomationTagsFilterBothReverseOrder, FullTestNameFilterReverseBoth, NegativeAndExampleTagsReversed)
bool FAutomationTagsFilterBothReverseOrder::RunTest(const FString& Parameters)
{
	TArray<FString> TestNames;
	FString NegativeFilter("NOT [DoNotWant] AND [TestExampleTag]");

	FAutomationTestFramework::Get().GetTestFullNamesMatchingTagPattern(TestNames, NegativeFilter);

	TestEqual("One element", TestNames.Num(), 1);
	TestEqual("Current test is not selected", TestNames[0], FullTestNameTagsExist);
	return true;
}

constexpr auto ComplexTags1 = "[TestExampleTagComplex][FirstExample][MustPass]";
constexpr auto ComplexTags2 = "[TestExampleTagComplex][SecondExample][MustPass]";
constexpr auto ComplexTagTestPath = "TestFramework.Tags.Complex";
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FAutomationTagsForComplexSuite, ComplexTagTestPath, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FAutomationTagsForComplexSuite::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FAutomationTestFramework& Framework = FAutomationTestFramework::Get();

	OutBeautifiedNames.Add("First");
	Framework.RegisterComplexAutomationTestTags(this, "First", ComplexTags1);

	OutBeautifiedNames.Add("Second");
	Framework.RegisterComplexAutomationTestTags(this, "Second", ComplexTags2);
	
	OutTestCommands = OutBeautifiedNames; //pass names as parameters
}

bool FAutomationTagsForComplexSuite::RunTest(const FString& Parameters)
{
	//registration in GetTests() performs necessary setup
	FString MyName = GetTestFullName();
	TestTrue(TEXT("Complex tests construct names as expected"), MyName.EndsWith(Parameters));

	FString MyTags = FAutomationTestFramework::Get().GetTagsForAutomationTest(MyName);
	TestFalse(TEXT("Tag is found"), MyTags.IsEmpty());

	FString ExpectedTags;
	if( Parameters.Equals("First"))
	{
		ExpectedTags = FString(ComplexTags1);
	}
	else {
		ExpectedTags = FString(ComplexTags2);
	}
	TestEqual(TEXT("Tags dynamically registered for intended complex test"), MyTags, ExpectedTags);
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
