// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Engine.h"

#include "Algo/BinarySearch.h"
#include "Engine/GameEngine.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "Misc/CoreMisc.h"
#include "Misc/OutputDeviceMemory.h"
#include "Serialization/MemoryWriter.h"


#if WITH_DEV_AUTOMATION_TESTS && UE_ALLOW_EXEC_COMMANDS

/**
 * Validates the execution of a deferred exec command
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDeferredExecCommandTest, "System.Engine.ExecCommands.Deferred Command",
	                             EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FDeferredExecCommandTest::RunTest(const FString& Parameters)
{
	if (!GEngine)
	{
		return false;
	}

	struct FExecCommandHandler : FSelfRegisteringExec
	{
		bool bCommandExecuted = false;

		virtual bool Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			if (FParse::Command(&Cmd, TEXT("Test_DeferredCommand")))
			{
				bCommandExecuted = true;
				return true;
			}

			return false;
		}
	} ExecCommandHandler;

	// Add to the DeferredCommands array
	{
		GEngine->DeferredCommands.Add(TEXT("Test_DeferredCommand"));
		GEngine->TickDeferredCommands();

		TestTrue(TEXT("Deferred Exec Command failed to execute."), ExecCommandHandler.bCommandExecuted);
	}

	ExecCommandHandler.bCommandExecuted = false;

	// Use the DEFER prefix to defer the execution
	{
		GEngine->Exec(nullptr, TEXT("DEFER Test_DeferredCommand"));
		TestFalse(TEXT("Deferred Exec Command executed without deferring."), ExecCommandHandler.bCommandExecuted);

		GEngine->TickDeferredCommands();
		TestTrue(TEXT("Deferred Exec Command failed to execute."), ExecCommandHandler.bCommandExecuted);
	}

	return true;
}

/**
 * Validates the execution of a self registering exec command
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSelfRegisteringExecCommandTest, "System.Engine.ExecCommands.Self Registering",
	                             EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FSelfRegisteringExecCommandTest::RunTest(const FString& Parameters)
{
	if (!GEngine)
	{
		return false;
	}

	struct FExecCommandHandler : FSelfRegisteringExec
	{
		bool bCommandExecuted = false;

		virtual bool Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			if (FParse::Command(&Cmd, TEXT("Test_Command")))
			{
				bCommandExecuted = true;
				return true;
			}

			return false;
		}
	} ExecCommandHandler;

	
	GEngine->Exec(nullptr, TEXT("Test_Command"));

	TestTrue(TEXT("Exec Command failed to execute."), ExecCommandHandler.bCommandExecuted);

	return true;
}

/**
 * Validates that we can execute FConsoleCommands through GEngine->Exec
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConsoleCommandExecCommandTest, "System.Engine.ExecCommands.Console Command",
	                             EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FConsoleCommandExecCommandTest::RunTest(const FString& Parameters)
{
	if (!GEngine)
	{
		return false;
	}

	const TCHAR* ConsoleCommandName = TEXT("TestConsoleCommand");

	bool bConsoleCommandExecuted = false;

	FAutoConsoleCommand ConsoleCommand(
		ConsoleCommandName,
		TEXT(""),
		FConsoleCommandDelegate::CreateLambda(
			[&bConsoleCommandExecuted]()
			{
				bConsoleCommandExecuted = true;
			})
	);

	GEngine->Exec(nullptr, ConsoleCommandName);

	TestTrue(TEXT("Console command failed to execute through Exec."), bConsoleCommandExecuted);

	return true;
}

/**
 * Validates that the exec command routing respects the target (editor vs dev vs runtime)
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FExecCommandTargetVariantsTest, "System.Engine.ExecCommands.Target Variants",
	                             EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FExecCommandTargetVariantsTest::RunTest(const FString& Parameters)
{
	if (!GEngine)
	{
		return false;
	}

	struct FExecCommandHandler : FSelfRegisteringExec
	{
		bool bEditorCommandExecuted = false;
		bool bDevCommandExecuted = false;
		bool bRuntimeCommandExecuted = false;

		virtual bool Exec_Editor(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			if (FParse::Command(&Cmd, TEXT("Test_EditorCommand")))
			{
				bEditorCommandExecuted = true;
				return true;
			}

			return false;
		}

		virtual bool Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			if (FParse::Command(&Cmd, TEXT("Test_DevCommand")))
			{
				bDevCommandExecuted = true;
				return true;
			}

			return false;
		}

		virtual bool Exec_Runtime(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
		{
			if (FParse::Command(&Cmd, TEXT("Test_RuntimeCommand")))
			{
				bRuntimeCommandExecuted = true;
				return true;
			}

			return false;
		}
	} ExecCommandHandler;

	GEngine->Exec(nullptr, TEXT("Test_EditorCommand"));
	GEngine->Exec(nullptr, TEXT("Test_DevCommand"));
	GEngine->Exec(nullptr, TEXT("Test_RuntimeCommand"));

#if UE_ALLOW_EXEC_EDITOR
	TestTrue(TEXT("Editor Exec Command failed to execute."), ExecCommandHandler.bEditorCommandExecuted);
#else
	TestFalse(TEXT("Editor Exec Command executed in an invalid context."), ExecCommandHandler.bEditorCommandExecuted);
#endif

#if UE_ALLOW_EXEC_DEV
	TestTrue(TEXT("Dev Exec Command failed to execute."), ExecCommandHandler.bDevCommandExecuted);
#else
	TestFalse(TEXT("Dev Exec Command executed in an invalid context."), ExecCommandHandler.bDevCommandExecuted);
#endif

	TestTrue(TEXT("Runtime Exec Command failed to execute."), ExecCommandHandler.bRuntimeCommandExecuted);

	return true;
}

/**
 * Validates that some of the most commonly used commands are reachable
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCommonCommandsExecCommandTest, "System.Engine.ExecCommands.Common Commands",
	                             EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

bool FCommonCommandsExecCommandTest::RunTest(const FString& Parameters)
{
	if (!GEngine)
	{
		return false;
	}

	TArray<FString> CommonCommands;
	CommonCommands.Reserve(5);

	CommonCommands.Emplace(TEXT("recompileshaders"));

#if STATS
	CommonCommands.Emplace(TEXT("stat")); // Prefix for the stat commands. The specific commands themselves are not gathered by DumpConsoleCommands.
#endif

	if (GEngine->IsA<UGameEngine>())
	{
		CommonCommands.Emplace(TEXT("quit"));
	}
	else if (GIsEditor)
	{
		CommonCommands.Emplace(TEXT("quit_editor"));
	}

	const int32 PreserveSize = 256 * 1024;
	const int32 BufferSize = 2048 * 1024;
	const bool bSuppressEventTag = true;
	FOutputDeviceMemory MemoryOutput(PreserveSize, BufferSize, bSuppressEventTag);

	const bool bDumpConsoleCommandsExecuted = GEngine->Exec(nullptr, TEXT("DumpConsoleCommands"), MemoryOutput);
	TestTrue(TEXT("DumpConsoleCommands failed to execute."), bDumpConsoleCommandsExecuted);

	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryOutput.Dump(MemoryWriter);

	if (static_cast<ANSICHAR>(Buffer.Last()) != '\0')
	{
		Buffer.Add('\0');
	}

	FString ConsoleCommandsDump(reinterpret_cast<ANSICHAR*>(Buffer.GetData()));

	TArray<FString> CommandList;
	ConsoleCommandsDump.ParseIntoArrayLines(CommandList);

	for ( const FString& CommonCommand : CommonCommands )
	{
		const bool bCommandFound = ( Algo::BinarySearch(CommandList, CommonCommand) != INDEX_NONE );
		TestTrue( FString::Printf( TEXT("%s failed to execute."), *CommonCommand ), bCommandFound );
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS && UE_ALLOW_EXEC_COMMANDS
