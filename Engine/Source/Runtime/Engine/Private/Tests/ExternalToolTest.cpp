// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationTestSettings.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogBaseAutomationTests, Log, All);

/**
 * Holds the process info for an external tool
 */
namespace ExternalProcessHelper
{
	struct ExternalProcessInfo
	{
		ExternalProcessInfo()
			: ProcessHandle()
			, ReadPipe(NULL)
			, WritePipe(NULL)
		{
		}
		// Holds the process handle
		FProcHandle		ProcessHandle;
		// Holds the read pipe.
		void* ReadPipe;
		// Holds the write pipe.
		void* WritePipe;
	};
}
/**
 * Wait for an external process to finish
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForProcessToCompleteLatentCommand, ExternalProcessHelper::ExternalProcessInfo, ProcessInfo);

bool FWaitForProcessToCompleteLatentCommand::Update()
{
	if( ProcessInfo.ProcessHandle.IsValid() )
	{
		//Check for any output
		FString ProcessOutput = FPlatformProcess::ReadPipe(ProcessInfo.ReadPipe);
		if( ProcessOutput.Len() > 0 )
		{
			TArray<FString> OutputLines;
			ProcessOutput.ParseIntoArray(OutputLines, TEXT("\n"), false);
			for( int32 i = 0; i < OutputLines.Num(); i++ )
			{
				UE_LOGF(LogBaseAutomationTests, Log, "%ls", *OutputLines[i]);
			}
		}

		//We aren't done until the process stops
		if( FPlatformProcess::IsProcRunning(ProcessInfo.ProcessHandle) )
		{
			return false;
		}

		FPlatformProcess::ClosePipe(ProcessInfo.ReadPipe,ProcessInfo.WritePipe);
		ProcessInfo.ReadPipe = NULL;
		ProcessInfo.WritePipe = NULL;

		//Check for any return codes
		int32 ReturnCode;
		if( !FPlatformProcess::GetProcReturnCode(ProcessInfo.ProcessHandle,&ReturnCode) )
		{
			ReturnCode = -1;
		}

		if( ReturnCode != 0 )
		{
			UE_LOGF(LogBaseAutomationTests, Error, "External Tool failed with error code: %i",ReturnCode);
		}
	}

	return true;
}

/**
 * RunExternalToolTest
 */
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FRunExternalToolTest, "External", EAutomationTestFlags::EditorContext | EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::EngineFilter)

/** 
 * Find all the external too commands
 */
void FRunExternalToolTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	for( int32 i = 0; i < AutomationTestSettings->ExternalTools.Num(); i++ )
	{
		const FExternalToolDefinition& ToolIt = AutomationTestSettings->ExternalTools[i];
		if( ToolIt.ExecutablePath.FilePath.Len() > 0 )
		{
			// Check for multiple scripts if we have a script extension to look for
			if( ToolIt.ScriptExtension.Len() > 0 && ToolIt.ScriptDirectory.Path.Len() > 0 )
			{
				TArray<FString> ScriptFileNames;

				//Remove the "." before the extension if it exists
				const FString ScriptExtension = ToolIt.ScriptExtension.StartsWith(TEXT(".")) ? ToolIt.ScriptExtension.Right(ToolIt.ScriptExtension.Len() - 1) : ToolIt.ScriptExtension;

				IFileManager::Get().FindFiles(ScriptFileNames, *(ToolIt.ScriptDirectory.Path / TEXT("*.") + ScriptExtension), true, false);

				for (TArray<FString>::TConstIterator ScriptIt(ScriptFileNames); ScriptIt; ++ScriptIt)
				{
					const FString ScriptName = FString::Printf(TEXT("%s: %s"),*ToolIt.ToolName,*FPaths::GetBaseFilename(*ScriptIt));
					OutBeautifiedNames.Add(*ScriptName);

					const FString SctiptCommand = FString::Printf(TEXT("%s;%s;%s"),*ToolIt.ExecutablePath.FilePath,**ScriptIt,ToolIt.WorkingDirectory.Path.Len() > 0 ? *ToolIt.WorkingDirectory.Path : *ToolIt.ScriptDirectory.Path);
					OutTestCommands.Add(*SctiptCommand);
				}
			}
			else
			{
				OutBeautifiedNames.Add(*ToolIt.ToolName);
				const FString ToolCommand = FString::Printf(TEXT("%s;%s;%s"),*ToolIt.ExecutablePath.FilePath,*ToolIt.CommandLineOptions,*ToolIt.WorkingDirectory.Path);
				OutTestCommands.Add(*ToolCommand);
			}
		}
	}
}

/** 
 * Launch the external tool as a process and wait for it to complete
 *
 * @param Parameters - The command to launch the external tool
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FRunExternalToolTest::RunTest(const FString& Parameters)
{
	TArray<FString> Commands;
	Parameters.ParseIntoArray(Commands,TEXT(";"),false);

	if( Commands.Num() != 3 )
	{
		UE_LOGF(LogBaseAutomationTests, Error, "ERROR Parsing commands for external tool: %ls",*Parameters);
		return false;
	}

	const FString& Executable       = Commands[0];
	const FString& Options          = Commands[1];
	const FString& WorkingDirectory = Commands[2];

	//Create the pipes that we can use to read the output of the process
	ExternalProcessHelper::ExternalProcessInfo ProcessInfo;
	if (!FPlatformProcess::CreatePipe(ProcessInfo.ReadPipe, ProcessInfo.WritePipe))
	{
		return false;
	}
	
	//Create the new process
	ProcessInfo.ProcessHandle = FPlatformProcess::CreateProc(*Executable,
															*Options,
															true,
															false,
															false,
															NULL,
															0,
															WorkingDirectory.Len() > 0 ? *WorkingDirectory : NULL,
															ProcessInfo.WritePipe);

	//Check that we got a valid process handle
	if( ProcessInfo.ProcessHandle.IsValid() )
	{
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForProcessToCompleteLatentCommand(ProcessInfo));
	}
	else
	{
		UE_LOGF(LogBaseAutomationTests, Error, "Failed to launch executable (%ls) for external tool %ls",*Executable, *Parameters);
		FPlatformProcess::ClosePipe(ProcessInfo.ReadPipe, ProcessInfo.WritePipe);
		return false;
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
