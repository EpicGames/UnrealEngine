// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#define TEST_NAME_ROOT "System.Engine.InstancedStaticMesh"

namespace InstancedStaticMeshTests
{
	constexpr EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

	// =======================================================================
	// Helpers
	// =======================================================================

	/**
	 * Verify the invariant: PerInstanceSMCustomData.Num() == GetInstanceCount() * NumCustomDataFloats
	 */
	static bool VerifyCustomDataInvariant(FAutomationTestBase& Test, const UInstancedStaticMeshComponent* ISM, const TCHAR* Context)
	{
		const int32 ExpectedCount = ISM->GetInstanceCount() * ISM->NumCustomDataFloats;
		const int32 ActualCount = ISM->PerInstanceSMCustomData.Num();
		return Test.TestEqual(
			FString::Printf(TEXT("%s: PerInstanceSMCustomData.Num() should be %d (Instances=%d * NumCustomDataFloats=%d), got %d"),
				Context, ExpectedCount, ISM->GetInstanceCount(), ISM->NumCustomDataFloats, ActualCount),
			ActualCount,
			ExpectedCount);
	}

	/**
	 * Simulate what the property editor does when removing an element from the Instances array:
	 * 1. The editor removes the entry from PerInstanceSMData before notifying the component.
	 * 2. PostEditChangeChainProperty is called with EPropertyChangeType::ArrayRemove.
	 * 3. Inside, RemoveInstanceInternal(index, true) is called.
	 */
	static void SimulateEditorArrayRemove(UInstancedStaticMeshComponent* ISM, int32 IndexToRemove)
	{
		// Step 1: The property editor removes the entry from PerInstanceSMData
		ISM->PerInstanceSMData.RemoveAt(IndexToRemove);

		// Step 2: Construct the property changed event to match what the editor sends
		FProperty* PerInstanceSMDataProp = UInstancedStaticMeshComponent::StaticClass()->FindPropertyByName(
			GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMData));
		check(PerInstanceSMDataProp);

		TMap<FString, int32> ArrayIndices;
		ArrayIndices.Add(PerInstanceSMDataProp->GetFName().ToString(), IndexToRemove);

		FPropertyChangedEvent ChangeEvent(PerInstanceSMDataProp, EPropertyChangeType::ArrayRemove);
		ChangeEvent.ObjectIteratorIndex = 0;
		ChangeEvent.SetArrayIndexPerObject(MakeArrayView(&ArrayIndices, 1));

		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(PerInstanceSMDataProp);

		FPropertyChangedChainEvent ChainEvent(PropertyChain, ChangeEvent);
		ISM->PostEditChangeChainProperty(ChainEvent);
	}

	// =======================================================================
	// CustomData tests
	// =======================================================================

	// -----------------------------------------------------------------------
	// Control test: RemoveInstance (public API) correctly maintains invariant
	// -----------------------------------------------------------------------

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FISMCustomData_PublicRemoveKeepsInvariant,
		TEST_NAME_ROOT ".CustomData.PublicRemoveKeepsInvariant",
		TestFlags)

	bool FISMCustomData_PublicRemoveKeepsInvariant::RunTest(const FString& Parameters)
	{
		UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
		ISM->NumCustomDataFloats = 2;

		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);

		TestEqual(TEXT("Instance count after adds"), ISM->GetInstanceCount(), 3);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding 3 instances"));

		// Remove middle instance via public API (calls RemoveInstanceInternal with InstanceAlreadyRemoved=false)
		ISM->RemoveInstance(1);

		TestEqual(TEXT("Instance count after remove"), ISM->GetInstanceCount(), 2);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After public API removal of middle instance"));

		// Add another instance
		ISM->AddInstance(FTransform::Identity);

		TestEqual(TEXT("Instance count after re-add"), ISM->GetInstanceCount(), 3);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After re-adding instance"));

		return true;
	}

	// -----------------------------------------------------------------------
	// Bug test: Editor removal of middle instance should maintain invariant
	// -----------------------------------------------------------------------

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FISMCustomData_EditorRemoveMiddleKeepsInvariant,
		TEST_NAME_ROOT ".CustomData.EditorRemoveMiddleKeepsInvariant",
		TestFlags)

	bool FISMCustomData_EditorRemoveMiddleKeepsInvariant::RunTest(const FString& Parameters)
	{
		UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
		ISM->NumCustomDataFloats = 2;

		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);

		TestEqual(TEXT("Instance count after adds"), ISM->GetInstanceCount(), 3);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding 3 instances"));

		// Simulate editor removing instance at index 1 (middle).
		// This exercises the InstanceAlreadyRemoved=true path in RemoveInstanceInternal.
		SimulateEditorArrayRemove(ISM, 1);

		TestEqual(TEXT("Instance count after editor remove"), ISM->GetInstanceCount(), 2);
		// With the bug, PerInstanceSMCustomData still has 6 entries instead of 4.
		VerifyCustomDataInvariant(*this, ISM, TEXT("After editor removal of middle instance"));

		return true;
	}

	// -----------------------------------------------------------------------
	// Bug test: Editor removal of last instance should maintain invariant.
	// Exercises the IsValidIndex edge case where InstanceIndex == PerInstanceSMData.Num().
	// -----------------------------------------------------------------------

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FISMCustomData_EditorRemoveLastKeepsInvariant,
		TEST_NAME_ROOT ".CustomData.EditorRemoveLastKeepsInvariant",
		TestFlags)

	bool FISMCustomData_EditorRemoveLastKeepsInvariant::RunTest(const FString& Parameters)
	{
		UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
		ISM->NumCustomDataFloats = 2;

		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);

		TestEqual(TEXT("Instance count after adds"), ISM->GetInstanceCount(), 3);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding 3 instances"));

		// Simulate editor removing the last instance (index 2).
		// After PerInstanceSMData.RemoveAt(2), IsValidIndex(2) returns false,
		// which causes RemoveInstanceInternal to skip the entire body.
		SimulateEditorArrayRemove(ISM, 2);

		TestEqual(TEXT("Instance count after editor remove"), ISM->GetInstanceCount(), 2);
		// With the bug, PerInstanceSMCustomData still has 6 entries instead of 4.
		VerifyCustomDataInvariant(*this, ISM, TEXT("After editor removal of last instance"));

		return true;
	}

	// -----------------------------------------------------------------------
	// Bug test: Adding instances after editor removal causes further divergence.
	// Each editor remove leaves stale custom data; each add appends more,
	// making the mismatch grow. This is what ultimately triggers the
	// check() in Scatter() at runtime.
	// -----------------------------------------------------------------------

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FISMCustomData_AddAfterEditorRemoveKeepsInvariant,
		TEST_NAME_ROOT ".CustomData.AddAfterEditorRemoveKeepsInvariant",
		TestFlags)

	bool FISMCustomData_AddAfterEditorRemoveKeepsInvariant::RunTest(const FString& Parameters)
	{
		UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
		ISM->NumCustomDataFloats = 2;

		// Add 3 instances
		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);
		ISM->AddInstance(FTransform::Identity);

		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding 3 instances"));

		// Simulate editor removing instance at index 1
		SimulateEditorArrayRemove(ISM, 1);

		TestEqual(TEXT("Instance count after editor remove"), ISM->GetInstanceCount(), 2);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After editor removal"));

		// Add another instance.
		// With the bug, PerInstanceSMCustomData has 6 (stale) + 2 (new) = 8 instead of 6.
		ISM->AddInstance(FTransform::Identity);

		TestEqual(TEXT("Instance count after re-add"), ISM->GetInstanceCount(), 3);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding instance post-editor-removal"));

		return true;
	}

	// -----------------------------------------------------------------------
	// Bug test: Multiple sequential editor removals accumulate stale entries.
	// -----------------------------------------------------------------------

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FISMCustomData_MultipleEditorRemovalsKeepInvariant,
		TEST_NAME_ROOT ".CustomData.MultipleEditorRemovalsKeepInvariant",
		TestFlags)

	bool FISMCustomData_MultipleEditorRemovalsKeepInvariant::RunTest(const FString& Parameters)
	{
		UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
		ISM->NumCustomDataFloats = 3;

		// Add 5 instances
		for (int32 i = 0; i < 5; ++i)
		{
			ISM->AddInstance(FTransform::Identity);
		}

		TestEqual(TEXT("Instance count after adds"), ISM->GetInstanceCount(), 5);
		VerifyCustomDataInvariant(*this, ISM, TEXT("After adding 5 instances"));

		// Remove instances one by one via editor path (always remove index 0 for simplicity)
		for (int32 i = 4; i >= 2; --i)
		{
			SimulateEditorArrayRemove(ISM, 0);
			TestEqual(
				*FString::Printf(TEXT("Instance count after removal %d"), 5 - i),
				ISM->GetInstanceCount(), i);
			VerifyCustomDataInvariant(*this, ISM,
				*FString::Printf(TEXT("After editor removal %d (remaining: %d)"), 5 - i, i));
		}

		return true;
	}
}

#undef TEST_NAME_ROOT

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
