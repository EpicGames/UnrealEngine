// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/AutomationTest.h"
#include "Engine/IntSerialization.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIntSerializationTest, "System.Engine.Int Serialization", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FIntSerializationTest::RunTest (const FString& Parameters)
{
	//Create object for serialization
	//UIntSerialization SerializableObject;
	//SerializableObject->
	auto SerializableObject = NewObject<UIntSerialization>();
	SerializableObject->UnsignedInt8Variable = 255U;
	SerializableObject->UnsignedInt16Variable = 65535U;
	SerializableObject->UnsignedInt32Variable = 4294967295U;
	SerializableObject->UnsignedInt64Variable = 18446744073709551615U;
	SerializableObject->SignedInt8Variable = -128;
	SerializableObject->SignedInt16Variable = -32768;
	SerializableObject->SignedInt32Variable = 2147483647 ;
	SerializableObject->SignedInt64Variable = 9223372036854775807;
	//Serialize object
	//FArchive 
	//FMemoryWriter SerializedData = new FMemoryWriter(;

	TArray<uint8> SaveData;
	FMemoryWriter OutAr(SaveData, true);
	SerializableObject->Serialize(OutAr);

	//Deserialize into new object
	FMemoryReader InAr(SaveData, true);
	auto DeSerializableObject = NewObject<UIntSerialization>();
	DeSerializableObject->Serialize(InAr);

	//Compare test
	TestEqual(TEXT("int8 serialised and deserialised incorrectly"), SerializableObject->SignedInt8Variable, DeSerializableObject->SignedInt8Variable);
	TestEqual(TEXT("int16 serialised and deserialised incorrectly"), SerializableObject->SignedInt16Variable, DeSerializableObject->SignedInt16Variable);
	TestEqual(TEXT("int32 serialised and deserialised incorrectly"), SerializableObject->SignedInt32Variable, DeSerializableObject->SignedInt32Variable);
	TestEqual(TEXT("int64 serialised and deserialised incorrectly"), SerializableObject->SignedInt64Variable, DeSerializableObject->SignedInt64Variable);
	TestEqual(TEXT("uint8 serialised and deserialised incorrectly"), SerializableObject->UnsignedInt8Variable, DeSerializableObject->UnsignedInt8Variable);
	TestEqual(TEXT("uint16 serialised and deserialised incorrectly"), SerializableObject->UnsignedInt16Variable, DeSerializableObject->UnsignedInt16Variable);
	TestEqual(TEXT("uint32 serialised and deserialised incorrectly"), SerializableObject->UnsignedInt32Variable, DeSerializableObject->UnsignedInt32Variable);
	TestEqual(TEXT("uint64 serialised and deserialised incorrectly"), SerializableObject->UnsignedInt64Variable, DeSerializableObject->UnsignedInt64Variable);


	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
