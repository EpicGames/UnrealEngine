// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextKey.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"
#include "UObject/TextProperty.h"
#include "Tests/TextPropertyTestObject.h"

#define LOCTEXT_NAMESPACE "TextPropertyTest"

UTextPropertyTestObject::UTextPropertyTestObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), DefaultedText(LOCTEXT("DefaultedText", "DefaultValue"))
{
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTextPropertyTest, "System.Engine.Internationalization.Text Property Test", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FTextPropertyTest::RunTest (const FString& Parameters)
{
	UClass* const TextPropertyTestObjectClass = UTextPropertyTestObject::StaticClass();
	FTextProperty* const DefaultedTextProperty = FindFProperty<FTextProperty>(TextPropertyTestObjectClass, "DefaultedText");
	FTextProperty* const UndefaultedTextProperty = FindFProperty<FTextProperty>(TextPropertyTestObjectClass, "UndefaultedText");
	const UTextPropertyTestObject* const TextPropertyTestCDO = Cast<UTextPropertyTestObject>( TextPropertyTestObjectClass->GetDefaultObject(false) );

	{
		UTextPropertyTestObject* NewUObject = NewObject<UTextPropertyTestObject>(GetTransientPackage(), TextPropertyTestObjectClass);

		// Test Identical - Newly constructed object properties should be identical to class default object properties.
		if ((DefaultedTextProperty->Identical(&(NewUObject->DefaultedText), &(TextPropertyTestCDO->DefaultedText), 0) != true)
			||
			(UndefaultedTextProperty->Identical(&(NewUObject->UndefaultedText), &(TextPropertyTestCDO->UndefaultedText), 0) != true))
		{
			AddError(TEXT("FTextProperty::Identical failed to return true comparing a newly constructed object and the class default object."));
		}

		// Test ExportText - Export text should provide the localized form of the text.
		{
			FString ExportedStringValue;
			DefaultedTextProperty->ExportTextItem_Direct(ExportedStringValue, &(NewUObject->DefaultedText), nullptr, nullptr, 0, nullptr);
			FText ExportedTextValue;
			FTextStringHelper::ReadFromBuffer(*ExportedStringValue, ExportedTextValue);
			if (ExportedTextValue.ToString() != NewUObject->DefaultedText.ToString())
			{
				AddError(TEXT("FTextProperty::ExportTextItem failed to provide the display string."));
			}
		}

		// Test ImportText - Import text should set the source string to the input string.
		{
			FString ImportedStringValue = "ImportValue";
			DefaultedTextProperty->ImportText_Direct(*ImportedStringValue, &(NewUObject->DefaultedText), nullptr, 0);
			const FString* const SourceString = FTextInspector::GetSourceString(NewUObject->DefaultedText);
			if( !SourceString || ImportedStringValue != *SourceString )
			{
				AddError(TEXT("FTextProperty::ImportText failed to alter the source string to the provided value."));
			}
		}
	}

	// Test Identical - Altered text properties should not be identical to class default object properties.
	{
		UTextPropertyTestObject* NewUObject = NewObject<UTextPropertyTestObject>(GetTransientPackage(), TextPropertyTestObjectClass);

		NewUObject->DefaultedText = LOCTEXT("ModifiedDefaultedText", "Modified DefaultedText Value");
		NewUObject->UndefaultedText = LOCTEXT("ModifiedUndefaultedText", "Modified UndefaultedText Value");
		if( 
			DefaultedTextProperty->Identical(&(NewUObject->DefaultedText), &(TextPropertyTestCDO->DefaultedText), 0)
			||
			UndefaultedTextProperty->Identical(&(NewUObject->UndefaultedText), &(TextPropertyTestCDO->UndefaultedText), 0)
		  )
		{
			AddError(TEXT("FTextProperty::Identical failed to return false comparing a modified object and the class default object."));
		}
	}

	{
		TArray<uint8> BackingStore;
		
		auto SavedObject = NewObject<UTextPropertyTestObject>();

		SavedObject->DefaultedText = FText::AsLocalizable_Advanced( TEXT("TextPropertyTest"), TEXT("DefaultedText"), TEXT("DefaultValue") );
		SavedObject->UndefaultedText = LOCTEXT("ModifiedUndefaultedText", "Modified UndefaultedText Value");
		const FText TransientText = FText::Format( LOCTEXT("TransientTest", "{0}"), LOCTEXT("TransientTestMessage", "Testing Transient serialization detection") );
		SavedObject->TransientText = TransientText;

		// Test Identical - Text properties with the same source as class default object properties should be considered identical. 
		if( !( DefaultedTextProperty->Identical(&(SavedObject->DefaultedText), &(TextPropertyTestCDO->DefaultedText), 0) ) )
		{
			AddError(TEXT("FTextProperty::Identical failed to return true comparing an FText with an identical source string to the class default object."));
		}

		// Save.
		{
			FMemoryWriter MemoryWriter(BackingStore, true);
			SavedObject->Serialize(MemoryWriter);
		}

		auto LoadedObject = NewObject<UTextPropertyTestObject>();

		// Load.
		{
			FMemoryReader MemoryReader(BackingStore, true);
			LoadedObject->Serialize(MemoryReader);
		}

		// Test Serialization - Loaded object should be identical to saved object. 
		if( 
			!( DefaultedTextProperty->Identical(&(LoadedObject->DefaultedText), &(SavedObject->DefaultedText), 0) )
			||
			!( UndefaultedTextProperty->Identical(&(LoadedObject->UndefaultedText), &(SavedObject->UndefaultedText), 0) )
		  )
		{
			AddError(TEXT("Saving and loading a serialized object containing FText properties failed to maintain FText values."));
		}

		// Test Identical - Text properties with the same source as the class default object property should save and load as the class default object property.
		if( !( DefaultedTextProperty->Identical(&(LoadedObject->DefaultedText), &(TextPropertyTestCDO->DefaultedText), 0) ) )
		{
			AddError(TEXT("FTextProperty::Identical failed to collapse identical source strings into the same namespace and key during serialization."));
		}

		// Test Transient - Transient text properties should save out an error message instead of their actual string value
		const FString* const LoadedTransientTextString = FTextInspector::GetSourceString(LoadedObject->TransientText);
		const FString* const TransientTextString = FTextInspector::GetSourceString(TransientText);
		if ( GIsEditor && LoadedTransientTextString && TransientTextString && *(LoadedTransientTextString) != *(TransientTextString) )
		{
			AddError(TEXT("Transient Texts should not exist in the editor."));
		}
		else if ( !GIsEditor && LoadedObject->TransientText.ToString() != FText::Format( LOCTEXT("Error_SerializationFailure", "ERR: Transient text cannot be serialized \"{0}\"."), TransientText ).ToString() )
		{
			//AddError(TEXT("Transient Texts should persist an error message when they are serialized."));
		}
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS

#undef LOCTEXT_NAMESPACE
