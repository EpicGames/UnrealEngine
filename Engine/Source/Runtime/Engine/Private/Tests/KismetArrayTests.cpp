// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Kismet/KismetArrayLibrary.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Kismet::Test
{
	namespace Private
	{
		template<typename ElementType, typename InSortFunc, typename... InArgs>
		void VerifySort(
			const TArray<ElementType>& InArray,
			const TArray<ElementType>& InExpected,
			InSortFunc SortFunc,
			InArgs... ExtraArgs)
		{
			const auto RunTest = [&](const TCHAR* SortType, bool bStableSort, EArraySortOrder SortOrder)
			{
				TArray<ElementType> ArrayCopy = InArray;
				TArray<ElementType> ExpectedCopy = InExpected;
				if (SortOrder == EArraySortOrder::Descending)
				{
					Algo::Reverse(ExpectedCopy);
				}

				Invoke(SortFunc, ArrayCopy, bStableSort, SortOrder, ExtraArgs...);

				const auto ToStringHelper = [](const ElementType& Element) { return LexToString(Element); };

				const FString ErrorMessage = FString::Printf(
					TEXT("Sorted array doesn't match expected array for %s sort. Sorted: %s, Expected: %s."),
					SortType,
					*FString::JoinBy(ArrayCopy, TEXT(","), ToStringHelper),
					*FString::JoinBy(ExpectedCopy, TEXT(","), ToStringHelper));

				REQUIRE_MESSAGE(*ErrorMessage, ArrayCopy == ExpectedCopy);
			};

			// Test each combo of stable/sort order.
			RunTest(TEXT("unstable-ascending"), false, EArraySortOrder::Ascending);
			RunTest(TEXT("unstable-descending"), false, EArraySortOrder::Descending);
			RunTest(TEXT("stable-ascending"),true, EArraySortOrder::Ascending);
			RunTest(TEXT("stable-descending"),true, EArraySortOrder::Descending);
		}
	}

	TEST_CASE_NAMED(
		KismetArray_SortString_Tests,
		"UE::KismetArray::SortStrings",
		"[KismetArray][ClientContext][EditorContext][EngineFilter]")
	{
		{
			Private::VerifySort(
				TArray<FString>{ TEXT("Billy"), TEXT("Carol"), TEXT("Thomas"), TEXT("Riley"), TEXT("Adam") },
				TArray<FString>{ TEXT("Adam"), TEXT("Billy"), TEXT("Carol"), TEXT("Riley"), TEXT("Thomas") },
				&UKismetArrayLibrary::SortStringArray);
		}

		// Mixed casing
		{
			Private::VerifySort(
				TArray<FString>{ TEXT("Billy"), TEXT("BILLY"), TEXT("billy"), TEXT("BiLlY") },
				TArray<FString>{ TEXT("billy"), TEXT("Billy"), TEXT("BiLlY"), TEXT("BILLY") },
				&UKismetArrayLibrary::SortStringArray);
		}

		// Stable sort
		{
			TArray<FString> InArray{ TEXT("Billy"), TEXT("Billy"), TEXT("Billy") };
			TArray<const FString*> ElementOrder;
			for (const FString& Str : InArray)
			{
				ElementOrder.Add(&Str);
			}

			UKismetArrayLibrary::SortStringArray(InArray, true);

			for (int32 Index = 0; Index < InArray.Num(); ++Index)
			{
				if (ElementOrder[Index] != &InArray[Index])
				{
					FAIL_CHECK(TEXT("Identical elements didn't remain in original order after stable sort"));
				}
			}
		}
	}

	TEST_CASE_NAMED(
		KismetArray_SortName_Tests,
		"UE::KismetArray::SortNames",
		"[KismetArray][ClientContext][EditorContext][EngineFilter]")
	{
		const auto SortNamesRedirector = [](TArray<FName>& TargetArray, bool bStableSort, EArraySortOrder SortOrder, bool bLexicalSort)
		{
			UKismetArrayLibrary::SortNameArray(TargetArray, bStableSort, bLexicalSort, SortOrder);
		};

		// Test non-lexical sort
		{
			TArray<FName> InArray{ TEXT("Billy"), TEXT("Carol"), TEXT("Thomas"), TEXT("Riley"), TEXT("Adam") };
			TArray<FName> Expected = InArray;
			// Sorted by comparison index so can't easily hard-code expected array.
			Expected.Sort(FNameFastLess{});

			constexpr bool bLexicalSort = false;
			Private::VerifySort(InArray, Expected, SortNamesRedirector, bLexicalSort);
		}

		// Test lexical sort
		{
			constexpr bool bLexicalSort = true;
			Private::VerifySort(
				TArray<FName>{ TEXT("Billy"), TEXT("Carol"), TEXT("Thomas"), TEXT("Riley"), TEXT("Adam") },
				TArray<FName>{ TEXT("Adam"), TEXT("Billy"), TEXT("Carol"), TEXT("Riley"), TEXT("Thomas") },
				SortNamesRedirector,
				bLexicalSort);
		}
	}

	TEST_CASE_NAMED(
		KismetArray_SortIntegerType_Tests,
		"UE::KismetArray::SortIntegerTypes",
		"[KismetArray][ClientContext][EditorContext][EngineFilter]")
	{
		// byte array
		{
			Private::VerifySort(
				TArray<uint8>{ 10, 1, 22, 244, 90, 3 },
				TArray<uint8>{ 1, 3, 10, 22, 90, 244 },
				&UKismetArrayLibrary::SortByteArray);
		}

		// int32
		{
			Private::VerifySort(
				TArray<int32>{ 10, 1, 22, 244, 90, 3 },
				TArray<int32>{ 1, 3, 10, 22, 90, 244 },
				&UKismetArrayLibrary::SortIntArray);
		}

		// int64
		{
			Private::VerifySort(
				TArray<int64>{ 10, 1, 22, 244, 90, 3 },
				TArray<int64>{ 1, 3, 10, 22, 90, 244 },
				&UKismetArrayLibrary::SortInt64Array);
		}

		// double
		{
			Private::VerifySort(
				TArray<double>{ 10.0, 1.0, 22.0, 244.0, 90.0, 3.0 },
				TArray<double>{ 1.0, 3.0, 10.0, 22.0, 90.0, 244.0 },
				&UKismetArrayLibrary::SortFloatArray);
		}
	}

} // UE::Kismet::Test
#endif // WITH_TESTS