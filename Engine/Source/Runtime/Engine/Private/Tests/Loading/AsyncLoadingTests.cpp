// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Serialization/PackageStore.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectHash.h"

#if WITH_DEV_AUTOMATION_TESTS

#undef TEST_NAME_ROOT
#define TEST_NAME_ROOT "System.Engine.Loading"

/**
 * This test demonstrate that LoadPackageAsync is thread-safe and can be called from multiple workers at the same time.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FThreadSafeAsyncLoadingTest, TEXT(TEST_NAME_ROOT ".ThreadSafeAsyncLoadingTest"), EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)
bool FThreadSafeAsyncLoadingTest::RunTest(const FString& Parameters)
{
	// We use the asset registry to get a list of asset to load. 
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();
	AssetRegistry.WaitForCompletion();

	// Limit the number of packages we're going to load for the test in case the project is very big.
	constexpr int32 MaxPackageCount = 5000;

	TSet<FName> UniquePackages;
	AssetRegistry.EnumerateAllAssets(
		[&UniquePackages](const FAssetData& AssetData)
		{
			if (UniquePackages.Num() < MaxPackageCount)
			{
				if (LoadingTestsUtils::IsAssetSuitableForTests(AssetData))
				{
					UniquePackages.FindOrAdd(AssetData.PackageName);
				}

				return true;
			}
			
			return false;
		},
		UE::AssetRegistry::EEnumerateAssetsFlags::OnlyOnDiskAssets
	);

	TArray<FName> PackagesToLoad(UniquePackages.Array());
	TArray<int32> RequestIDs;
	RequestIDs.SetNum(PackagesToLoad.Num());

	ParallelFor(PackagesToLoad.Num(),
		[&PackagesToLoad, &RequestIDs](int32 Index)
		{
			RequestIDs[Index] = LoadPackageAsync(PackagesToLoad[Index].ToString());
		}
	);

	FlushAsyncLoading(RequestIDs);

	return true;
}

/**
 * Ensure we can properly handle Serialize implementations that might invalidate exports during preload.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAsyncLoadingTestInvalidateExportDuringPreload, TEXT(TEST_NAME_ROOT ".InvalidateExportDuringPreload"), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAsyncLoadingTestInvalidateExportDuringPreload::RunTest(const FString& Parameters)
{
	auto VerifyLoad = [this](FLoadingTestsScope& LoadingTestScope, bool bExpectToFindObject)
		{
			UPackage* Package = LoadPackage(nullptr, LoadingTestScope.PackagePath1, LOAD_None);
			TestTrue(TEXT("The package should load successfully"), Package != nullptr);
			
			// Exclude garbage objects as the GC won't have run yet but invalidated objects shdould be marked as garbage by this point
			UAsyncLoadingTests_Shared* Object1 = FindObjectFast<UAsyncLoadingTests_Shared>(Package, LoadingTestScope.ObjectName, EFindObjectFlags::ExactClass, EObjectFlags::RF_MirroredGarbage);
			if (bExpectToFindObject)
			{
				TestTrue(TEXT("The object should have been loaded"), Object1 != nullptr);
			}
			else
			{
				TestTrue(TEXT("The object should not have been loaded"), Object1 == nullptr);
			}
		};

	{
		FLoadingTestsScope LoadingTestScope(this);


		VerifyLoad(LoadingTestScope, true /*bExpectToFindObject*/);
	}
	
	{
		FLoadingTestsScope LoadingTestScope(this);

		UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
			[this](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
			{
				if (Ar.IsLoading())
				{
					if (FLinkerLoad* Linker = Object->GetLinker())
					{
						Object->MarkAsGarbage();
						Linker->InvalidateExport(Object, true /*bHidesGarbageObjects*/);
					}
				}
			}
		);

		VerifyLoad(LoadingTestScope, false /*bExpectToFindObject*/);
	}
	return true;
}

/**
 * Validates imported package failures are propagated such that failing imports prevent loading of the importing package
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAsyncLoadingTestFailedImportPropagation, TEXT(TEST_NAME_ROOT ".FailedImportPropagation"), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAsyncLoadingTestFailedImportPropagation::RunTest(const FString& Parameters)
{
	class FMockPackageStoreBackend final : public IPackageStoreBackend
	{
	public:
		virtual EPackageLoader GetSupportedLoaders() override { return EPackageLoader::Zen | EPackageLoader::LinkerLoad; }
		virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext> Context) override {}
		virtual void BeginRead() override {}
		virtual void EndRead() override {}
		virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override { return false; }

		FMockPackageStoreBackend(const FLoadingTestsScope& LoadingTestScope)
		{
			const FPackageId PackageId1 = FPackageId::FromName(FName(LoadingTestScope.PackagePath1));
			const FPackageId PackageId2 = FPackageId::FromName(FName(LoadingTestScope.PackagePath2));
			const FPackageId PackageId3 = FPackageId::FromName(FName(LoadingTestScope.PackagePath3));

			PackageImportPackageIds.Add(PackageId1, { PackageId2, PackageId3 });
			PackageImportPackageIds.Add(PackageId2, { PackageId3 });
			PackageImportPackageIds.Add(PackageId3, { });

			Entries.Add(PackageId1,
				{
#if WITH_EDITORONLY_DATA
					.LinkerLoadCaseCorrectedPackageName = LoadingTestScope.PackagePath1,
#endif
					.PackageExtension = EPackageExtension::Asset,
					.LoaderType = EPackageLoader::LinkerLoad,
					.ImportedPackageIds = *PackageImportPackageIds.Find(PackageId1)
				});
			Entries.Add(PackageId2,
				{
#if WITH_EDITORONLY_DATA
					.LinkerLoadCaseCorrectedPackageName = LoadingTestScope.PackagePath2,
#endif
					.PackageExtension = EPackageExtension::Asset,
					.LoaderType = EPackageLoader::LinkerLoad,
					.ImportedPackageIds = *PackageImportPackageIds.Find(PackageId2)
				});
			Entries.Add(PackageId3,
				{
#if WITH_EDITORONLY_DATA
					.LinkerLoadCaseCorrectedPackageName = LoadingTestScope.PackagePath3,
#endif
					.PackageExtension = EPackageExtension::Asset,
					.LoaderType = EPackageLoader::LinkerLoad,
					.ImportedPackageIds = *PackageImportPackageIds.Find(PackageId3)
				});
		}

		virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry) override
		{
			if (FPackageStoreEntry* Entry = Entries.Find(PackageId))
			{
				OutPackageStoreEntry = *Entry;

				if (NotInstalledPackages.Contains(PackageId))
				{
					return EPackageStoreEntryStatus::NotInstalled;
				}

				return EPackageStoreEntryStatus::Ok;
			}
			return EPackageStoreEntryStatus::Missing;
		}

		void MarkPackageAsNotInstalled(const FPackageId PackageId)
		{
			NotInstalledPackages.Add(PackageId);
		}

		void MarkPackageAsAvailable(const FPackageId PackageId)
		{
			NotInstalledPackages.Remove(PackageId);
		}
	private:
		TSet<FPackageId> NotInstalledPackages;
		TMap<FPackageId, TArray<FPackageId>> PackageImportPackageIds;
		TMap<FPackageId, FPackageStoreEntry> Entries;
	};

	// Package 1 has a hard ref to package 2 which has a hard ref to package 3.
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object3;
		};

	FLoadingTestsScope LoadingTestScope(this, MutateObjects);
	const FPackageId PackageId3 = FPackageId::FromName(FName(LoadingTestScope.PackagePath3));

	TSharedPtr<FMockPackageStoreBackend> MockBackend = MakeShared<FMockPackageStoreBackend>(LoadingTestScope);
	FPackageStore::Get().Mount(MockBackend.ToSharedRef(), INT_MAX /*Priority*/);

	TArray<const TCHAR*> PathsToLoad = { LoadingTestScope.PackagePath3, LoadingTestScope.PackagePath2, LoadingTestScope.PackagePath1 };

	MockBackend->MarkPackageAsNotInstalled(PackageId3);

	// We should the following log message twice; once for Package1 and again for Package2 due to failed Imports. Package3 fails because it does not exist, not due to failed imports
	AddExpectedMessage(TEXT("will not be loaded due to the NotInstalled imported package"), EAutomationExpectedErrorFlags::Contains, 2 /*Occurrences*/);

	// Verify that no matter which package we load, since a leaf imported package is NotInstalled, none of the packages are loaded
	for (const TCHAR* PackagePath : PathsToLoad)
	{
		TestTrue(TEXT("Package with a failed transitive import should not have loaded"), LoadPackage(nullptr, PackagePath, LOAD_None) == nullptr);
		TestTrue(TEXT("Package1 should not have been loaded"), FindPackage(nullptr, LoadingTestScope.PackagePath1) == nullptr);
		TestTrue(TEXT("Package2 should not have been loaded"), FindPackage(nullptr, LoadingTestScope.PackagePath2) == nullptr);
		TestTrue(TEXT("Package3 should not have been loaded"), FindPackage(nullptr, LoadingTestScope.PackagePath3) == nullptr);
	}
	
	MockBackend->MarkPackageAsAvailable(PackageId3);

	// Now that Package3 is available, verify all packages are now loadable
	TestTrue(TEXT("Package1 is expected to be loaded"), LoadPackage(nullptr, LoadingTestScope.PackagePath1, LOAD_None) != nullptr);
	TestTrue(TEXT("Package2 is expected to be implicitly loaded by Package1"), FindPackage(nullptr, LoadingTestScope.PackagePath2) != nullptr);
	TestTrue(TEXT("Package3 is expected to be implicitly loaded by Package2"), FindPackage(nullptr, LoadingTestScope.PackagePath3) != nullptr);

	return true;
}

#undef TEST_NAME_ROOT
#endif // WITH_DEV_AUTOMATION_TESTS
