// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncLoadingTests_ConvertFromType.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncLoadingTests_ConvertFromType)

#if WITH_DEV_AUTOMATION_TESTS

class FClassRedirectScope
{
	TArray<FCoreRedirect> Redirects;
public:
	FClassRedirectScope(UClass* OldClass, UClass* NewClass)
	{
		Redirects.Emplace(ECoreRedirectFlags::Type_Class, OldClass->GetName(), NewClass->GetName());
		check(FCoreRedirects::AddRedirectList(Redirects, TEXT("Tests")));
	}

	~FClassRedirectScope()
	{
		check(FCoreRedirects::RemoveRedirectList(Redirects, TEXT("Tests")));
	}
};

/**
 * This test validates ConvertFromType thread-safety.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConvertFromType_SoftToHard, TEXT("System.Engine.Loading.ConvertFromType_SoftToHard"), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FConvertFromType_SoftToHard::RunTest(const FString& Parameters)
{
	// Just make sure the async loading queue is empty before beginning.
	FlushAsyncLoading();

	constexpr const TCHAR* ObjectName   = TEXT("TestObject");
	constexpr const TCHAR* PackagePath1 = TEXT("/Engine/TestConvertFromType_SoftToHard1");
	constexpr const TCHAR* ObjectPath1  = TEXT("/Engine/TestConvertFromType_SoftToHard1.TestObject");
	constexpr const TCHAR* PackagePath2 = TEXT("/Engine/TestConvertFromType_SoftToHard2");
	constexpr const TCHAR* ObjectPath2  = TEXT("/Engine/TestConvertFromType_SoftToHard2.TestObject");

	// Creation phase
	{
		UPackage* Package1 = CreatePackage(PackagePath1);
		UAsyncLoadingTests_ConvertFromType_V1* Object1 = NewObject<UAsyncLoadingTests_ConvertFromType_V1>(Package1, ObjectName, RF_Public | RF_Standalone);

		UPackage* Package2 = CreatePackage(PackagePath2);
		UAsyncLoadingTests_ConvertFromType_V1* Object2 = NewObject<UAsyncLoadingTests_ConvertFromType_V1>(Package2, ObjectName, RF_Public | RF_Standalone);
	
		// We need inter package dependency to trigger a load during ConvertFromType.
		Object1->Reference = Object2;

		// To avoid an error on save, we need to mark the package as fully loaded.
		Package1->MarkAsFullyLoaded();
		Package2->MarkAsFullyLoaded();

		// Save packages to disk.
		check(UPackage::SavePackage(Package1, nullptr, *FPackageName::LongPackageNameToFilename(PackagePath1, FPackageName::GetAssetPackageExtension()), FSavePackageArgs()));
		check(UPackage::SavePackage(Package2, nullptr, *FPackageName::LongPackageNameToFilename(PackagePath2, FPackageName::GetAssetPackageExtension()), FSavePackageArgs()));

		// Remove RF_Standalone from top level object.
		Object1->ClearFlags(RF_Standalone);
		Object2->ClearFlags(RF_Standalone);

		// GC and make sure everything gets cleaned up before loading
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V1>(nullptr, ObjectPath1) != nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath1) != nullptr);
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V1>(nullptr, ObjectPath2) != nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath2) != nullptr);

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		check(FindObject<UAsyncLoadingTests_ConvertFromType_V1>(nullptr, ObjectPath1) == nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath1) == nullptr);
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V1>(nullptr, ObjectPath2) == nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath2) == nullptr);
	}

	// Loading phase
	{
		// We're going to deserialize with the V2 so we get a ConvertFromType call.
		FClassRedirectScope RedirectScope(UAsyncLoadingTests_ConvertFromType_V1::StaticClass(), UAsyncLoadingTests_ConvertFromType_V2::StaticClass());

		UPackage* LoadedPackage = nullptr;
		UAsyncLoadingTests_ConvertFromType_V2* LoadedObject = nullptr;
		int32 RequestID = LoadPackageAsync(PackagePath1,
			FLoadPackageAsyncDelegate::CreateLambda(
				[&LoadedPackage, &LoadedObject, &ObjectName](const FName& InPackageName, UPackage* InLoadedPackage, EAsyncLoadingResult::Type InResult)
				{
					LoadedPackage = InLoadedPackage;
					LoadedObject  = FindObject<UAsyncLoadingTests_ConvertFromType_V2>(LoadedPackage, ObjectName);
				}
			));
		FlushAsyncLoading(RequestID);

		TestTrue(TEXT("The object should have been properly loaded"), LoadedObject != nullptr);
		if (LoadedObject)
		{
			TestTrue(TEXT("The hard-ref should now point to the object inside the second package"), LoadedObject->Reference.Get() != nullptr);
			TestTrue(TEXT("The hard-ref should now point to the object inside the second package"), LoadedObject->Reference->GetPathName() == ObjectPath2);
		}
	}

	// Cleanup phase
	{
		// GC and make sure everything gets cleaned up before exiting
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath1) != nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath1) != nullptr);
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath2) != nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath2) != nullptr);

		// Remove RF_Standalone from top level object.
		FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath1)->ClearFlags(RF_Standalone);
		FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath2)->ClearFlags(RF_Standalone);

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		check(FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath1) == nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath1) == nullptr);
		check(FindObject<UAsyncLoadingTests_ConvertFromType_V2>(nullptr, ObjectPath2) == nullptr);
		check(FindObject<UPackage>(nullptr, PackagePath2) == nullptr);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
