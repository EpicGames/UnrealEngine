// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "AsyncLoadingTests_ConvertFromType.generated.h"

UCLASS()
class UAsyncLoadingTests_ConvertFromType_V1 : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TSoftObjectPtr<UObject> Reference;
};

UCLASS()
class UAsyncLoadingTests_ConvertFromType_V2 : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UObject> Reference;
};
