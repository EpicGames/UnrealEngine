// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_ExternalReadDependencies,
	TEXT("System.Engine.Loading.ExternalReadDependencies"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_ExternalReadDependencies::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	std::atomic<int32> NumExternalReads = 0;
	UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
		[this, &NumExternalReads](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			FExternalReadCallback ExternalReadCallback =
				[&NumExternalReads](double RemainingTime)
				{
					NumExternalReads++;
					return true;
				};

			Ar.AttachExternalReadDependency(ExternalReadCallback);
		}
	);

	// Trigger the async loading
	int32 RequestId = LoadPackageAsync(FLoadingTestsScope::PackagePath1);

	// When a flush occurs, we have no choice but to ignore bIsReadyForAsyncPostLoad
	FlushAsyncLoading(RequestId);

	TestTrue("ExternalReadDependency callbacks should have been called", NumExternalReads != 0);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
