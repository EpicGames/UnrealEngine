// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Algo/AllOf.h"
#include "Misc/AutomationTest.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName.h"
#include "Async/ManualResetEvent.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SavePackage.h"
#include "Tasks/Task.h"

#if WITH_DEV_AUTOMATION_TESTS

// All Flush tests should run on zenloader only as the other loaders are not compliant.
typedef FLoadingTests_ZenLoaderOnly_Base FLoadingTests_Flush_Base;

/**
 * This test validates loading an object synchronously during serialize.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_InvalidFromWorker,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.InvalidFromWorker"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_InvalidFromWorker::RunTest(const FString& Parameters)
{
	AddExpectedError(TEXT("is unable to FlushAsyncLoading from the current thread"), EAutomationExpectedErrorFlags::Contains);
	AddExpectedError(TEXT("[Callstack]"), EAutomationExpectedErrorFlags::Contains, 0 /* At least 1 occurrence */);
	
	FLoadingTestsScope LoadingTestScope(this);

	UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
		[this](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			if (Ar.IsLoading())
			{
				// Use event instead of waiting on the task to prevent retraction as we really want that task
				// to execute on a worker thread instead of being retracted in the serialize thread.
				UE::FManualResetEvent Event;
				UE::Tasks::Launch(TEXT("FlushAsyncLoading"), [&Event]() { FlushAsyncLoading(); Event.Notify(); });
				Event.Wait();
			}
		}
	);

	LoadingTestScope.LoadObjects();

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_ValidFromCallback,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.ValidFromCallback"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_ValidFromCallback::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	LoadPackageAsync(FLoadingTestsScope::PackagePath1,
		FLoadPackageAsyncDelegate::CreateLambda([](const FName&, UPackage*, EAsyncLoadingResult::Type) { FlushAsyncLoading(); }));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_LoadFromPostLoad,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_LoadFromPostLoad"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_LoadFromPostLoad::RunTest(const FString& Parameters)
{
	TArray<const TCHAR*> CompletionCallbackOrder;
	FLoadPackageAsyncDelegate LoadCompletedFn; LoadCompletedFn.BindLambda([&CompletionCallbackOrder](const FName& PackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
		{
			if (PackageName == FLoadingTestsScope::PackagePath1)
			{
				CompletionCallbackOrder.AddUnique(FLoadingTestsScope::PackagePath1);
			}
			else if (PackageName == FLoadingTestsScope::PackagePath2)
			{
				CompletionCallbackOrder.AddUnique(FLoadingTestsScope::PackagePath2);
			}
		});
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda([&CompletionCallbackOrder, &LoadCompletedFn](UAsyncLoadingTests_Shared* Object)
		{
			UPackage* Package = Object->GetPackage();
			if (!Package)
			{
				return;
			}
			FString PackageName = Package->GetFName().ToString();
			if (PackageName.Equals(FLoadingTestsScope::PackagePath1))
			{
				FlushAsyncLoading(LoadPackageAsync(FLoadingTestsScope::PackagePath2, LoadCompletedFn));
			}
		});

	FLoadingTestsScope LoadingTestScope(this);
	FlushAsyncLoading(LoadPackageAsync(FLoadingTestsScope::PackagePath1, LoadCompletedFn));

	// If we sync load a package while loading an existing one, we expect the callbacks to be ordered as packages are completed
	const TArray<const TCHAR*> ExpectedOrder = { FLoadingTestsScope::PackagePath2, FLoadingTestsScope::PackagePath1 };
	TestEqual(TEXT("Expected LoadCompletion callback order to be identical"), CompletionCallbackOrder, ExpectedOrder);
	return true;
}

enum class EVerifyOrder : uint8
{
	None			= 0,
	Serialize		= 1 << 0,
	PostLoad		= 1 << 1,
	LoadCompleted	= 1 << 2,
	All				= (uint8) ~0
};
ENUM_CLASS_FLAGS(EVerifyOrder);

template <typename TMutateObjectsFn, typename TIsOrderedFn>
void TestLoadPhases(FAutomationTestBase* AutomationTest, TMutateObjectsFn& MutateObjectsFn, const TArray<const TCHAR*>& PackagesToFlush, TIsOrderedFn& IsOrderedFn, EVerifyOrder VerifyOrder)
{
	TArray<const TCHAR*> SerializedPackages; SerializedPackages.Reserve(3);
	TArray<const TCHAR*> PostLoadedPackages; PostLoadedPackages.Reserve(3);
	TArray<const TCHAR*> LoadCompletedPackages; LoadCompletedPackages.Reserve(3);

	auto CheckNameFn = [](UObject* Object, TArray<const TCHAR*>& OutPackagesFound)
		{
			UPackage* Package = Cast<UPackage>(Object);
			if (!Package)
			{
				Package = Object->GetPackage();
			}
			FString PackageName = Package->GetFName().ToString();
			
			if (PackageName == FLoadingTestsScope::PackagePath1)
			{
				OutPackagesFound.AddUnique(FLoadingTestsScope::PackagePath1);
			}
			else if (PackageName == FLoadingTestsScope::PackagePath2)
			{
				OutPackagesFound.AddUnique(FLoadingTestsScope::PackagePath2);
			}
			else if (PackageName == FLoadingTestsScope::PackagePath3)
			{
				OutPackagesFound.AddUnique(FLoadingTestsScope::PackagePath3);
			}
			else if (PackageName == FLoadingTestsScope::PackagePath4)
			{
				OutPackagesFound.AddUnique(FLoadingTestsScope::PackagePath4);
			}
		};

	UAsyncLoadingTests_Shared::OnSerialize.BindLambda([&SerializedPackages, &CheckNameFn](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			CheckNameFn(Object, SerializedPackages);
		});
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda([&PostLoadedPackages, &CheckNameFn](UAsyncLoadingTests_Shared* Object)
		{
			CheckNameFn(Object, PostLoadedPackages);
		});

	FLoadPackageAsyncDelegate LoadCompletedFn; LoadCompletedFn.BindLambda([&LoadCompletedPackages, &CheckNameFn](const FName& PackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
		{
			if (Result != EAsyncLoadingResult::Type::Succeeded)
			{
				return;
			}
			CheckNameFn(LoadedPackage, LoadCompletedPackages);
		});

	FLoadingTestsScope LoadingTestScope(AutomationTest, MutateObjectsFn);
	LoadPackageAsync(FLoadingTestsScope::PackagePath1, LoadCompletedFn);
	LoadPackageAsync(FLoadingTestsScope::PackagePath2, LoadCompletedFn);
	LoadPackageAsync(FLoadingTestsScope::PackagePath3, LoadCompletedFn);
	LoadPackageAsync(FLoadingTestsScope::PackagePath4, LoadCompletedFn);

	// Flush the packages requested, but use a new load request so that we can also verify 
	// we handle aliasing request ids for packages in our ordering validation
	TArray<int32, TInlineAllocator<4>> FlushRequests;
	for (const TCHAR* PackagePath : PackagesToFlush)
	{
		FlushRequests.Add(LoadPackageAsync(PackagePath, LoadCompletedFn));
	}
	FlushAsyncLoading(FlushRequests);

	auto VerifyLoads = [AutomationTest,&IsOrderedFn](const TCHAR* OrderType, const TArray<const TCHAR*>& Actual, bool bVerifyOrder)
		{
			const bool bOnlyVerifyExistence = !bVerifyOrder;
			const bool bMatch = IsOrderedFn(Actual, bOnlyVerifyExistence);
			if (!bMatch)
			{
				TStringBuilder<256> Builder;
				Builder.Appendf(TEXT("Expected to %s packages with '%s' ordering for packages 1 to 4:\n\t"), OrderType,
					(bVerifyOrder ? TEXT("custom") : TEXT("any")));
				Builder.Append(TEXT("Actual order:\n\t"));
				if (!Actual.IsEmpty())
				{
					for (const TCHAR* Path : Actual)
					{
						Builder.Append(Path);
						Builder.Append(TEXT(", "));
					}
					Builder.RemoveSuffix(2);
				}
				Builder.Append(TEXT("\nFailed to find all 4 package, or failed to find the expected package ordering (refer to IsOrdered lambda in test)."));
				AutomationTest->TestTrue(Builder.ToString(), false);
			}
		};

	// Currently we do not provide any Serialize order guarantee with respect to dependencies.
	// We likely never want this so we can enable parallel preloading opportunities. However
	// if we should want preload ordering, re-enable to ensure this is done.
	VerifyLoads(TEXT("Serialize"), SerializedPackages, false /*!!(VerifyOrder& EVerifyOrder::Serialize)*/);
	VerifyLoads(TEXT("PostLoad"), PostLoadedPackages, !!(VerifyOrder & EVerifyOrder::PostLoad));
	VerifyLoads(TEXT("LoadCompletedCallbacks"), LoadCompletedPackages, !!(VerifyOrder & EVerifyOrder::LoadCompleted));
}

// The majority of the time we expect Packages 1 to 4 to be loaded 4, 3, 2, 1
bool IsLoadOrderReverseSorted(const TArray<const TCHAR*> FlushedPackages, const bool bOnlyVerifyExistence)
	{
		const int IndexOfPackage1 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath1); if (IndexOfPackage1 == INDEX_NONE) return false;
		const int IndexOfPackage2 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath2); if (IndexOfPackage2 == INDEX_NONE) return false;
		const int IndexOfPackage3 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath3); if (IndexOfPackage3 == INDEX_NONE) return false;
		const int IndexOfPackage4 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath4); if (IndexOfPackage4 == INDEX_NONE) return false;
		if (bOnlyVerifyExistence)
		{
			return true;
		}
		return IndexOfPackage4 < IndexOfPackage3 && IndexOfPackage3 < IndexOfPackage2 && IndexOfPackage2 < IndexOfPackage1;
	};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_A,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_A"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_A::RunTest(const FString& Parameters)
{
	// Package 1 has a hard ref to package 2 which has a hard ref to package 3.
	// 1 -> 2 -> 3 -> 4
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object3;
			Scope.Object3->HardReference = Scope.Object4;
		};

	bool bVerifyExactOrder = true;
	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::All);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_B,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_B"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_B::RunTest(const FString& Parameters)
{
	// Flush a sub-tree then flush the rest of the tree
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object3;
			Scope.Object3->HardReference = Scope.Object4;
		};

	bool bVerifyExactOrder = true;
	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath3, FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::All);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_C,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_C"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_C::RunTest(const FString& Parameters)
{
	// Flush a sub-tree via distinct root then flush the rest of the tree
	//      3 -> 4
	// 1 -> 2 -> 4
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object4;

			Scope.Object3->HardReference = Scope.Object4;
		};

	{
		bool bVerifyExactOrder = true;
		const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath3, FLoadingTestsScope::PackagePath1 };
		TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::LoadCompleted);
	}

	{
		// Swap flush order. We should still see imports before their referencers when loading
		const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1, FLoadingTestsScope::PackagePath3 };
		auto IsOrdered = [](const TArray<const TCHAR*> FlushedPackages, const bool bOnlyVerifyExistence)
			{
				const int IndexOfPackage1 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath1); if (IndexOfPackage1 == INDEX_NONE) return false;
				const int IndexOfPackage2 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath2); if (IndexOfPackage2 == INDEX_NONE) return false;
				const int IndexOfPackage3 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath3); if (IndexOfPackage3 == INDEX_NONE) return false;
				const int IndexOfPackage4 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath4); if (IndexOfPackage4 == INDEX_NONE) return false;
				if (bOnlyVerifyExistence)
				{
					return true;
				}
				// Ensure we load leaf nodes first for the first flush and then load anything outstanding in the second (4,2,1 then 3)
				return IndexOfPackage4 < IndexOfPackage2 && IndexOfPackage2 < IndexOfPackage1 && IndexOfPackage1 < IndexOfPackage3;
			};
		TestLoadPhases(this, MutateObjects, PackagesToFlush, IsOrdered, EVerifyOrder::LoadCompleted);
	}
	return true;
}

// Circular dependencies are currently chaotic in the loader and the order for processing is a mixture of request
// order, import load state, and import table ordering. The tests below confirm that load completion callbacks
// are exactly ordered, and all other load orderings are only verified to ensure that we see load events (Serialize/Postload)
// from all packages but we do not expect an exact ordering.
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_CircularDependencies_A,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_CircularDependencies_A"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_CircularDependencies_A::RunTest(const FString& Parameters)
{
	// Circular reference
	// 1 -> 2 -> 3 -> 4 -> 1
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object3;
			Scope.Object3->HardReference = Scope.Object4;
			// Circular ref back to Package1
			Scope.Object4->HardReference = Scope.Object1;
		};

	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::LoadCompleted);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_CircularDependencies_B,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_CircularDependencies_B"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_CircularDependencies_B::RunTest(const FString& Parameters)
{
	// Circular reference
	// 1 -> 2 -> 3 <-> 4
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object2->HardReference = Scope.Object3;
			Scope.Object3->HardReference = Scope.Object4;
			// Circular ref back to Package3
			Scope.Object4->HardReference = Scope.Object3;
		};

	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::LoadCompleted);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_CircularDependencies_C,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_CircularDependencies_C"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_CircularDependencies_C::RunTest(const FString& Parameters)
{
	// Two circular references
	// 1 <-> 2 <-> 3 -> 4
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			// Circular ref back to Package1
			Scope.Object2->HardReference = Scope.Object1;
			Scope.Object2->HardReference2 = Scope.Object3;
			// Circular ref back to Package2
			Scope.Object3->HardReference = Scope.Object2;
			Scope.Object3->HardReference2 = Scope.Object4;
		};

	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::LoadCompleted);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_CircularDependencies_D,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_CircularDependencies_D"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_CircularDependencies_D::RunTest(const FString& Parameters)
{
	// One circular reference. 3<->4, but 1 is also able to reach the cycle
	// 1 -> 2 -> 3 
	// |         ^
	// |         |
	// |         v
	// +-------> 4
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object1->HardReference2 = Scope.Object4;
			Scope.Object2->HardReference = Scope.Object3;
			Scope.Object3->HardReference = Scope.Object4;
			// Circular ref back to Package3
			Scope.Object4->HardReference = Scope.Object3;
		};

	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsLoadOrderReverseSorted, EVerifyOrder::LoadCompleted);
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Flush_LoadOrder_CircularDependencies_E,
	FLoadingTests_Flush_Base,
	TEXT("System.Engine.Loading.Flush.LoadOrder_CircularDependencies_E"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Flush_LoadOrder_CircularDependencies_E::RunTest(const FString& Parameters)
{
	// One circular reference. 3<->2, but 1 and 4 can both reach the cycle
	// 1 -> 3 -> 4
	// |    ^    |
	// v    |    |
	// 2 <--+<---+
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object1->HardReference = Scope.Object2;
			Scope.Object1->HardReference2 = Scope.Object3;
			Scope.Object2->HardReference = Scope.Object3;
			// Circular ref back to Package2
			Scope.Object3->HardReference = Scope.Object2;
			Scope.Object3->HardReference2 = Scope.Object4;
			Scope.Object4->HardReference = Scope.Object2;
		};

	const TArray<const TCHAR*> PackagesToFlush = { FLoadingTestsScope::PackagePath1 };
	auto IsOrdered = [](const TArray<const TCHAR*> FlushedPackages, const bool bOnlyVerifyExistence)
		{
			const int IndexOfPackage1 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath1); if (IndexOfPackage1 == INDEX_NONE) return false;
			const int IndexOfPackage2 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath2); if (IndexOfPackage2 == INDEX_NONE) return false;
			const int IndexOfPackage3 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath3); if (IndexOfPackage3 == INDEX_NONE) return false;
			const int IndexOfPackage4 = FlushedPackages.IndexOfByKey(FLoadingTestsScope::PackagePath4); if (IndexOfPackage4 == INDEX_NONE) return false;
			if (bOnlyVerifyExistence)
			{
				return true;
			}

			// We only care that depth is sorted (handle leaf nodes before the cycle first). For nodes at the same depth, the order does not need to be strictly ordered.
			return IndexOfPackage4 == 0 && (IndexOfPackage3 == 1 || IndexOfPackage3 == 2) && (IndexOfPackage2 == 1 || IndexOfPackage2 == 2) && IndexOfPackage1 == 3;
		};
	TestLoadPhases(this, MutateObjects, PackagesToFlush, IsOrdered, EVerifyOrder::LoadCompleted);
	return true;
}
#endif // WITH_DEV_AUTOMATION_TESTS
