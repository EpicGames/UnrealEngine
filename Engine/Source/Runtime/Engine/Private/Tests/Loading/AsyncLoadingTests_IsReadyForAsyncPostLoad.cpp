// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_IsReadyForAsyncPostLoad,
	TEXT("System.Engine.Loading.IsReadyForAsyncPostLoad.Tick"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_IsReadyForAsyncPostLoad::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	// Return IsReadyForAsyncPostLoad until directed otherwise.
	bool bIsReadyForAsyncPostLoad = false;
	UAsyncLoadingTests_Shared::OnIsReadyForAsyncPostLoad.BindLambda(
		[&bIsReadyForAsyncPostLoad](const UAsyncLoadingTests_Shared* Object) 
		{
			return bIsReadyForAsyncPostLoad;
		}
	);

	// Sentinel to validate at which point postload is called exactly
	bool bPostLoadCalled = false;
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[&bPostLoadCalled](UAsyncLoadingTests_Shared* Object) 
		{ 
			bPostLoadCalled = true; 
		} 
	);

	// Trigger the async loading
	int32 RequestId = LoadPackageAsync(FLoadingTestsScope::PackagePath1);

	// Wait for a good amount of time to make sure we're really timing out because bIsReadyForAsyncPostLoad is false.
	EAsyncPackageState::Type State = ProcessAsyncLoadingUntilComplete([&bPostLoadCalled]() { return bPostLoadCalled; }, 5.0);
	TestNotEqual("Async loading should not have been able to complete due to bIsReadyForAsyncPostLoad being false", State, EAsyncPackageState::Complete);
	TestFalse("No postload should have been called", bPostLoadCalled);

	// Start returning that we are ready for postload
	bIsReadyForAsyncPostLoad = true;
	State = ProcessAsyncLoadingUntilComplete([&bPostLoadCalled]() { return bPostLoadCalled; }, 1.0);

	// Now we expect this to have completed in a timely manner
	TestEqual("AsyncLoading should now be complete", State, EAsyncPackageState::Complete);
	TestTrue("Postload should have been called", bPostLoadCalled);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_IsReadyForAsyncPostLoad_Flush,
	FLoadingTests_ZenLoaderOnly_Base,
	TEXT("System.Engine.Loading.IsReadyForAsyncPostLoad.Flush"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_IsReadyForAsyncPostLoad_Flush::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	// Return IsReadyForAsyncPostLoad until directed otherwise.
	bool bIsReadyForAsyncPostLoad = false;
	UAsyncLoadingTests_Shared::OnIsReadyForAsyncPostLoad.BindLambda(
		[&bIsReadyForAsyncPostLoad](const UAsyncLoadingTests_Shared* Object)
		{
			return bIsReadyForAsyncPostLoad;
		}
	);

	// Sentinel to validate at which point postload is called exactly
	bool bPostLoadCalled = false;
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[&bPostLoadCalled](UAsyncLoadingTests_Shared* Object)
		{
			bPostLoadCalled = true;
		}
	);

	// Trigger the async loading
	int32 RequestId = LoadPackageAsync(FLoadingTestsScope::PackagePath1);

	// When a flush occurs, we have no choice but to ignore bIsReadyForAsyncPostLoad
	FlushAsyncLoading(RequestId);

	// Now we expect this to have completed in a timely manner
	TestTrue("Postload should have been called", bPostLoadCalled);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_IsReadyForAsyncPostLoad_FullFlush,
	FLoadingTests_ZenLoaderOnly_Base,
	TEXT("System.Engine.Loading.IsReadyForAsyncPostLoad.FullFlush"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_IsReadyForAsyncPostLoad_FullFlush::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	// Return IsReadyForAsyncPostLoad until directed otherwise.
	bool bIsReadyForAsyncPostLoad = false;
	UAsyncLoadingTests_Shared::OnIsReadyForAsyncPostLoad.BindLambda(
		[&bIsReadyForAsyncPostLoad](const UAsyncLoadingTests_Shared* Object)
		{
			return bIsReadyForAsyncPostLoad;
		}
	);

	// Sentinel to validate at which point postload is called exactly
	bool bPostLoadCalled = false;
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[&bPostLoadCalled](UAsyncLoadingTests_Shared* Object)
		{
			bPostLoadCalled = true;
		}
	);

	// Trigger the async loading
	LoadPackageAsync(FLoadingTestsScope::PackagePath1);

	// When a flush occurs, we have no choice but to ignore bIsReadyForAsyncPostLoad
	FlushAsyncLoading();

	// Now we expect this to have completed in a timely manner
	TestTrue("Postload should have been called", bPostLoadCalled);

	return true;
}


#endif // WITH_DEV_AUTOMATION_TESTS
