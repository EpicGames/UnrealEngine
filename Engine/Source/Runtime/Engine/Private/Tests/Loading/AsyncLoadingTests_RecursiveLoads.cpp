// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SavePackage.h"

#if WITH_DEV_AUTOMATION_TESTS

// All RecursiveLoads tests should run on zenloader only as the other loaders are not compliant.
typedef FLoadingTests_ZenLoaderOnly_Base FLoadingTests_RecursiveLoads_Base;

/**
 * This test validates loading an object synchronously during serialize.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FromSerialize, 
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FromSerialize"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FromSerialize::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	UE::FManualResetEvent PostLoadEvent;
	std::atomic<bool> bHasBeenPostloaded = false;
	std::atomic<bool> bIsMultithreadedLoading = false;

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this, &PostLoadEvent, &bHasBeenPostloaded, &bIsMultithreadedLoading](UAsyncLoadingTests_Shared* Object)
		{
			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath2)
			{
				check(IsInGameThread());

				if (bIsMultithreadedLoading)
				{
					// If we run with ALT enabled, we want just enough time to make sure the flush
					// can return before this function exits, proving that we're not waiting on postload
					// during serialize.
					PostLoadEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1.0));
				}

				bHasBeenPostloaded = true;
			}
		}
	);

	UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
		[this, &bHasBeenPostloaded, &PostLoadEvent, &bIsMultithreadedLoading](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			// Loading from serialize is not supported in the multithreaded workers as it can lead to deadlocks.
			// Classes that end up doing it should be added to the .ini (search ParallelPreloadExclusions for examples) to prevent them from running wide.
			if (IsInParallelLoadingThread())
			{
				return;
			}

			if (Ar.IsLoading())
			{
				bIsMultithreadedLoading = !IsInGameThread();

				if (UObject* Obj = Object->SoftReference.LoadSynchronous())
				{
					TestTrue(TEXT("Recursive loads in serialize should be deserialized"), !Obj->HasAnyFlags(RF_NeedLoad));
					if (!IsInGameThread())
					{
						TestFalse(TEXT("Recursive loads in serialize skip thread-unsafe postloads when run from the ALT"), bHasBeenPostloaded);

						PostLoadEvent.Notify();
					}
				}
			}
		}
	);

	LoadingTestScope.LoadObjects();

	return true;
}

/**
 * This test validates loading an object with a thread-safe postload synchronously inside a thread-safe postload.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FromPostLoad_ThreadSafe,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FromPostLoad_ThreadSafe"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FromPostLoad_ThreadSafe::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	// Set both object for thread-safe postloads so we get called earlier.
	UAsyncLoadingTests_Shared::OnIsPostLoadThreadSafe.BindLambda(
		[](const UAsyncLoadingTests_Shared* Object)
		{
			return true;
		}
	);

	// When running with async loading thread, this should be called on ALT.
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this](UAsyncLoadingTests_Shared* Object)
		{
			// We expect objects that are thread-safe to postload, to have been postloaded before returning from their sync load.
			if (UObject* Obj = Object->SoftReference.LoadSynchronous())
			{
				TestFalse(TEXT("Sync loads inside thread-safe postload of objects that are thread-safe to postload should be fully loaded"), Obj->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}
		}
	);

	LoadingTestScope.LoadObjects();

	return true;
}

/**
 * This test validates loading an object synchronously inside a thread-safe (non-deferred) postload.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FromPostLoad_ThreadUnsafe,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FromPostLoad_ThreadUnsafe"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FromPostLoad_ThreadUnsafe::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	// Make the first postloads as thread-safe
	UAsyncLoadingTests_Shared::OnIsPostLoadThreadSafe.BindLambda(
		[](const UAsyncLoadingTests_Shared* Object)
		{
			return Object->GetPathName() == FLoadingTestsScope::ObjectPath1;
		}
	);

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this](UAsyncLoadingTests_Shared* Object)
		{
			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath1)
			{
				if (IsAsyncLoadingMultithreaded())
				{
					TestFalse(TEXT("Thread-safe postloads should get called from the async loading thread when it is active"), IsInGameThread());
				}
			
				if (UObject* Obj = Object->SoftReference.LoadSynchronous())
				{
					TestTrue(TEXT("Sync loads of non thread-safe objects from thread-safe postload should be deserialized"), !Obj->HasAnyFlags(RF_NeedLoad));

					// Since the Object returned will continue to postload on the game-thread, we can't verify the RF_NeedPostLoad flag as we could race
					// trying to look at the value depending on its state on the game-thread.
					
					// What we can do is verify below that postload is called from the game-thread on the object
				}
			}
			else
			{
				TestTrue(TEXT("Sync loads of non thread-safe objects from thread-safe postload should have their postload deferred on the game-thread"), IsInGameThread());
			}
		}
	);

	LoadingTestScope.LoadObjects();

	return true;
}

/**
 * This test validates loading an object synchronously during postload.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FromDeferredPostLoad,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FromDeferredPostLoad"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FromDeferredPostLoad::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this](UAsyncLoadingTests_Shared* Object)
		{
			if (UObject* Obj = Object->SoftReference.LoadSynchronous())
			{
				TestFalse(TEXT("Recursive loads in postload should be fully loaded"), Obj->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}
		}
	);

	LoadingTestScope.LoadObjects();

	return true;
}

/**
 * This test validates an error is emitted when flushing a requestid that is not a partial load from inside a recursive function.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FullFlushFrom_Serialize,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FullFlushFrom.Serialize"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FullFlushFrom_Serialize::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	AddExpectedMessage(TEXT("to avoid deadlock when it was recursively flushed by package"), EAutomationExpectedErrorFlags::Contains);

	UE::FManualResetEvent PostLoadEvent;
	std::atomic<bool> bHasBeenPostloaded = false;
	std::atomic<bool> bIsMultithreadedLoading = false;

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this, &PostLoadEvent, &bHasBeenPostloaded, &bIsMultithreadedLoading](UAsyncLoadingTests_Shared* Object)
		{
			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath2)
			{
				check(IsInGameThread());

				if (bIsMultithreadedLoading)
				{
					// If we run with ALT enabled, we want just enough time to make sure the flush
					// can return before this function exits, proving that we're not waiting on postload
					// during serialize.
					PostLoadEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1.0));
				}

				bHasBeenPostloaded = true;
			}
		}
	);

	int32 RequestId = 0;
	UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
		[this, &RequestId, &PostLoadEvent, &bHasBeenPostloaded, &bIsMultithreadedLoading](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			// Do not try to flush ourself as this would lead to a fatal error :)
			// Just flush Package2 when we're in Package1
			if (Ar.IsLoading() && Object->GetPathName() == FLoadingTestsScope::ObjectPath1)
			{
				bIsMultithreadedLoading = !IsInGameThread();

				// Flush the requestId that has been created outside of the recursive load. This request
				// should be a full request and flushing it should result in an error being reported.
				FlushAsyncLoading(RequestId);

				UAsyncLoadingTests_Shared* Object2 = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);
				TestFalse(TEXT("The object should be serialized"), Object2->HasAnyFlags(RF_NeedLoad));
				TestFalse(TEXT("The object should not have been postloaded"), bHasBeenPostloaded);

				PostLoadEvent.Notify();
			}
		}
	);

	// Create a request before starting the loading test so we get a request that is not tagged as partial.
	RequestId = LoadPackageAsync(FLoadingTestsScope::PackagePath2);

	LoadingTestScope.LoadObjects();

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FullFlushFrom_PostLoad,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FullFlushFrom.Postload"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FullFlushFrom_PostLoad::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	int32 RequestId = 0;
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this, &RequestId](UAsyncLoadingTests_Shared* Object)
		{
			// Do not try to flush ourself as this would lead to a fatal error :)
			// Just flush Package2 when we're in Package1
			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath1)
			{
				// Flush the requestId that has been created outside of the recursive load. This request
				// should be a full request and flushing it should result in an error being reported.
				FlushAsyncLoading(RequestId);

				UAsyncLoadingTests_Shared* Object2 = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);
				TestFalse(TEXT("The object should be serialized and postloaded"), Object2->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}
		}
	);

	// Create a request before starting the loading test so we get a request that is not tagged as partial.
	RequestId = LoadPackageAsync(FLoadingTestsScope::PackagePath2);

	LoadingTestScope.LoadObjects();

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FullFlushFrom_CompletionCallback,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FullFlushFrom.CompletionCallback"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_FullFlushFrom_CompletionCallback::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	bool bIsCallback1Called = false;
	bool bIsCallback2Called = false;
	int32 RequestId1 = 0;
	
	// Create a request before starting the loading test so we get a request that is not tagged as partial.
	RequestId1 = LoadPackageAsync(FLoadingTestsScope::PackagePath1,
		FLoadPackageAsyncDelegate::CreateLambda(
			[this, &bIsCallback1Called, &bIsCallback2Called](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
			{
				bIsCallback1Called = true;

				int32 RequestId2 = 
					LoadPackageAsync(FLoadingTestsScope::PackagePath2,
						FLoadPackageAsyncDelegate::CreateLambda(
							[&bIsCallback2Called](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
							{
								bIsCallback2Called = true;
							}
						));

				FlushAsyncLoading(RequestId2);
				TestTrue(TEXT("Callback function 2 should have been called"), bIsCallback2Called);
			}
		));

	FlushAsyncLoading(RequestId1);
	TestTrue(TEXT("Callback function 1 should have been called"), bIsCallback1Called);
	TestTrue(TEXT("Callback function 2 should have been called"), bIsCallback2Called);

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_FromBothSerializeAndPostLoad,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.FromBothSerializeAndPostLoad"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)

bool FLoadingTests_RecursiveLoads_FromBothSerializeAndPostLoad::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this,
		[](FLoadingTestsScope& TestScope)
		{
			TestScope.DefaultMutateObjects();
		}
	);

	std::atomic<bool> PartialLoadAchieved = false;
	UE::FManualResetEvent Event;
	int32 SerializeCount = 0;

	UE::FManualResetEvent PostLoadEvent;
	std::atomic<bool> bHasBeenPostloaded = false;
	std::atomic<bool> bIsMultithreadedLoading = false;
	
	// On serialize we try to force load, this should add the newly loaded package as a dynamic import of the package that requested them.
	UAsyncLoadingTests_Shared::OnSerialize.BindLambda(
		[this, &LoadingTestScope, &PartialLoadAchieved, &Event, &SerializeCount, &bHasBeenPostloaded, &PostLoadEvent, &bIsMultithreadedLoading](FArchive& Ar, UAsyncLoadingTests_Shared* Object)
		{
			if (Ar.IsLoading() && Object->GetPathName() == FLoadingTestsScope::ObjectPath1)
			{
				SerializeCount++;
				check(SerializeCount == 1);
				bIsMultithreadedLoading = !IsInGameThread();

				// Loading Object2 while being in Object1 serialize should trigger partial load of Object2
				if (UObject* Obj = Object->SoftReference.LoadSynchronous())
				{
					TestTrue(TEXT("Recursive loads in serialize should be deserialized"), !Obj->HasAnyFlags(RF_NeedLoad));
					if (!IsInGameThread())
					{
						TestFalse(TEXT("Recursive loads in serialize skip thread-unsafe postloads when run from the ALT"), bHasBeenPostloaded);
					}

					PostLoadEvent.Notify();

					PartialLoadAchieved = true;

					// When everything runs on GT, we will have no choice but to use the whole time
					// but when running with ALT, we will be able to resolve earlier as the GT will
					// unlock us sooner.
					Event.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1.0f)); 
				}
			}
		}
	);

	// Once in postload of object3, we now force load the same softref a second time, expecting to be able to postload it without deadlocking because of the merged postload groups.
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this, &LoadingTestScope, &bHasBeenPostloaded, &PostLoadEvent, &bIsMultithreadedLoading](UAsyncLoadingTests_Shared* Object)
		{
			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath2)
			{
				check(IsInGameThread());

				if (bIsMultithreadedLoading)
				{
					// If we run with ALT enabled, we want just enough time to make sure the flush
					// can return before this function exits, proving that we're not waiting on postload
					// during serialize.
					PostLoadEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1.0));
				}

				bHasBeenPostloaded = true;
			}

			if (Object->GetPathName() == FLoadingTestsScope::ObjectPath3)
			{
				// Trying to load Package2 completely while being in package 3 postload should be able to succeed
				// even if the Package2 is now a dynamic import of Package1.
				if (UObject* Obj = LoadObject<UObject>(nullptr, LoadingTestScope.ObjectPath2))
				{
					TestTrue(TEXT("Recursive loads in postload should be deserialized"), !Obj->HasAnyFlags(RF_NeedLoad));
					TestTrue(TEXT("Recursive loads in postload should be able to postload"), !Obj->HasAnyFlags(RF_NeedPostLoad));
				}
			}
		}
	);

	int32 Request1 = LoadPackageAsync(LoadingTestScope.PackagePath1);

	// Let the first package run until we reach the serialization part then we'll 
	// back-off to start another package and finish it.
	while (!PartialLoadAchieved)
	{
		ProcessAsyncLoadingUntilComplete([&PartialLoadAchieved](){ return PartialLoadAchieved.load(); }, 0.1f);
	}

	// Hopefully, the Object1 and Object2 are still being loaded when we reach this point.
	int32 Request2 = LoadPackageAsync(LoadingTestScope.PackagePath3);

	// Unlock the loading thread faster than the timeout if we reach here first.
	Event.Notify();

	FlushAsyncLoading(Request2);
	FlushAsyncLoading(Request1);

	return true;
}

/**
 * These tests validate loading an object that loads itself synchronously during postload.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_LoadSelfFromPostLoad_Direct,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.LoadSelfFromPostLoad_Direct"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_LoadSelfFromPostLoad_Direct::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this,
		[](FLoadingTestsScope& TestScope)
		{
			TestScope.Object1->SoftReference = TestScope.Object1;
		}
	);

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this](UAsyncLoadingTests_Shared* Object)
		{
			if (UObject* Obj = Object->SoftReference.LoadSynchronous())
			{
				TestFalse(TEXT("Recursive loads in postload should be fully loaded"), Obj->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}
		}
	);

	LoadPackage(nullptr, LoadingTestScope.PackagePath1, LOAD_None);

	UAsyncLoadingTests_Shared* LocalObject1 = FindObject<UAsyncLoadingTests_Shared>(nullptr, LoadingTestScope.ObjectPath1);
	TestTrue(TEXT("The object should have been properly loaded recursively"), LocalObject1 != nullptr);
	if (LocalObject1)
	{
		TestFalse(TEXT("The object should have been properly loaded recursively"), LocalObject1->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
	}

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_RecursiveLoads_LoadSelfFromPostLoad_Indirect,
	FLoadingTests_RecursiveLoads_Base,
	TEXT("System.Engine.Loading.RecursiveLoads.LoadSelfFromPostLoad_Indirect"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_RecursiveLoads_LoadSelfFromPostLoad_Indirect::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this,
		[](FLoadingTestsScope& TestScope)
		{
			// Soft-circular reference back to Object1
			TestScope.Object1->SoftReference = TestScope.Object2;
			TestScope.Object2->HardReference = TestScope.Object1;
		}
	);

	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this](UAsyncLoadingTests_Shared* Object)
		{
			if (UObject* Obj = Object->SoftReference.LoadSynchronous())
			{
				TestFalse(TEXT("Recursive loads in postload should be fully loaded"), Obj->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
			}
		}
	);

	LoadPackage(nullptr, LoadingTestScope.PackagePath1, LOAD_None);

	UAsyncLoadingTests_Shared* LocalObject1 = FindObject<UAsyncLoadingTests_Shared>(nullptr, LoadingTestScope.ObjectPath1);
	TestTrue(TEXT("The object should have been properly loaded recursively"), LocalObject1 != nullptr);
	if (LocalObject1)
	{
		TestFalse(TEXT("The object should have been properly loaded recursively"), LocalObject1->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
	}

	UAsyncLoadingTests_Shared* LocalObject2 = FindObject<UAsyncLoadingTests_Shared>(nullptr, LoadingTestScope.ObjectPath2);
	TestTrue(TEXT("The object should have been properly loaded recursively"), LocalObject2 != nullptr);
	if (LocalObject2)
	{
		TestFalse(TEXT("The object should have been properly loaded recursively"), LocalObject2->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
