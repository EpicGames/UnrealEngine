// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SavePackage.h"

#if WITH_DEV_AUTOMATION_TESTS

// All Renames tests should run on zenloader only as the other loaders are not compliant.
class FLoadingTests_Renames_Base : public FLoadingTests_ZenLoaderOnly_Base
{
public:
	using FLoadingTests_ZenLoaderOnly_Base::FLoadingTests_ZenLoaderOnly_Base;

protected:
	void DoRenameTest(bool bWithSave, bool bWithGC)
	{
		// Package 1 has a hard ref to package 2.
		auto MutateObjects =
			[](FLoadingTestsScope& Scope)
			{
				Scope.Object1->HardReference = Scope.Object2;
			};

		FLoadingTestsScope LoadingTestScope(this, MutateObjects);

		UAsyncLoadingTests_Shared* Object2 = LoadObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);

		// Move the Object in package 2 to a new package 3, leaving a redirector
		// This leaves the zenloader global import store in a bad state since it still thinks object in Package 2 is still an export.
		UPackage* NewPackage = LoadingTestScope.CreatePackage();
		Object2->Rename(nullptr, NewPackage);

		FString MovedObjectName = Object2->GetFullName();

		if (bWithSave)
		{
			LoadingTestScope.SavePackages();
		}

		if (bWithGC)
		{
			LoadingTestScope.GarbageCollect();
		}

		// Load object 1, which should trigger a fixup during import because object2 will be resolved in the wrong package.
		UAsyncLoadingTests_Shared* Object1 = LoadObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);

		TestNotEqual<UObject*>("Object1 should have been loaded", Object1, nullptr);
		if (Object1)
		{
			TestNotEqual<UObject*>("Object1's hard reference should have been loaded", Object1->HardReference, nullptr);

			if (Object1->HardReference)
			{
				TestEqual("Object1's hard reference object name should be the moved object", Object1->HardReference.GetFullName(), MovedObjectName);
			}
		}

		// Try to reload package2 for good measure as the previous operation might have caused the loader to become confused.
		UPackage* Package = LoadPackage(nullptr, FLoadingTestsScope::PackagePath2, LOAD_None);

		TestNotEqual<UObject*>("Package should have been loaded", Package, nullptr);
		if (Package)
		{
			TestEqual("Package name is wrong", Package->GetPathName(), FLoadingTestsScope::PackagePath2);
		}
		
		Object2 = LoadObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);

		TestNotEqual<UObject*>("Object should have been loaded", Object2, nullptr);
		if (Object2)
		{
			TestEqual("Object name is wrong", Object2->GetFullName(), MovedObjectName);
		}
	}
};

/**
 * This test validates the loader behavior when an object is moved to a different package in memory only.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Renames_ObjectMovedToAnotherPackage_InMemory,
	FLoadingTests_Renames_Base,
	TEXT("System.Engine.Loading.Renames.ObjectMovedToAnotherPackage.InMemory"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Renames_ObjectMovedToAnotherPackage_InMemory::RunTest(const FString& Parameters)
{
	static constexpr bool bWithSave = false;
	static constexpr bool bWithGC   = false;
	DoRenameTest(bWithSave, bWithGC);

	return true;
}

/**
 * This test validates the loader behavior when an object is moved to a different package and dirtied packages are written to disk.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Renames_ObjectMovedToAnotherPackage_OnDisk,
	FLoadingTests_Renames_Base,
	TEXT("System.Engine.Loading.Renames.ObjectMovedToAnotherPackage.OnDisk"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Renames_ObjectMovedToAnotherPackage_OnDisk::RunTest(const FString& Parameters)
{
	static constexpr bool bWithSave = true;
	static constexpr bool bWithGC = false;
	DoRenameTest(bWithSave, bWithGC);

	return true;
}

/**
 * This test validates the loader behavior when an object is moved to a different package and dirtied packages are written to disk, then reloaded after a full GC.
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Renames_ObjectMovedToAnotherPackage_OnDiskReload,
	FLoadingTests_Renames_Base,
	TEXT("System.Engine.Loading.Renames.ObjectMovedToAnotherPackage.OnDiskReload"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Renames_ObjectMovedToAnotherPackage_OnDiskReload::RunTest(const FString& Parameters)
{
	static constexpr bool bWithSave = true;
	static constexpr bool bWithGC = true;
	DoRenameTest(bWithSave, bWithGC);

	return true;
}

/**
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLoadingTests_Renames_ImportReplacedObject, TEXT("System.Engine.Loading.Renames.ImportReplacedObject"), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FLoadingTests_Renames_ImportReplacedObject::RunTest(const FString& Parameters)
{
	// Simulate Parent (Object1) <- Child (Object2) relationship 
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object2->HardReference = Scope.Object1;
		};

	FLoadingTestsScope LoadingTestScope(this, MutateObjects);

	// Load Parent package
	UPackage* ParentPackage = LoadPackage(nullptr, FLoadingTestsScope::PackagePath1, LOAD_None);

	// Rename one of the objects so the package is now effectively missing a public export
	UAsyncLoadingTests_Shared* Parent = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);
	TestNotNull(TEXT("Expected to be able to load our Parent object"), Parent);

	FName OriginalName = Parent->GetFName();
	Parent->Rename(TEXT("TRASH_TestObject"), nullptr, REN_DontCreateRedirectors);
	Parent->ClearFlags(RF_Standalone);
	Parent->MarkAsGarbage();

	// Create a new object on top of where the missing public export should be
	UAsyncLoadingTests_Shared* ReplacementObject = NewObject<UAsyncLoadingTests_Shared>(ParentPackage, OriginalName);
	UAsyncLoadingTests_Shared* NewParent = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);
	TestEqual(TEXT("Expect new object named ontop of orignal object name to be the same instance"), ReplacementObject, NewParent);

	// Load Child Package which imports the Parent package
	LoadPackage(nullptr, FLoadingTestsScope::PackagePath2, LOAD_None);

	UAsyncLoadingTests_Shared* Child = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);
	TestNotNull(TEXT("Expected to be able to load our Child object"), Child);
	TestEqual(TEXT("Expected Child package to import the new renamed Parent object, not the trashed object or a new object create from disk."), Cast<UAsyncLoadingTests_Shared>(Child->HardReference), NewParent);

	ReplacementObject->ClearFlags(RF_Standalone);
	ReplacementObject->MarkAsGarbage();

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLoadingTests_Renames_ImportRenamedObjectReloadsImport, TEXT("System.Engine.Loading.Renames.ImportRenamedObjectReloadsImport"), EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FLoadingTests_Renames_ImportRenamedObjectReloadsImport::RunTest(const FString& Parameters)
{
	// Simulate Parent (Object1) <- Child (Object2) relationship 
	auto MutateObjects =
		[](FLoadingTestsScope& Scope)
		{
			Scope.Object2->HardReference = Scope.Object1;
		};

	FLoadingTestsScope LoadingTestScope(this, MutateObjects);

	// Load Parent package
	UPackage* ParentPackage = LoadPackage(nullptr, FLoadingTestsScope::PackagePath1, LOAD_None);

	// Rename one of the objects so the package is now effectively missing a public export
	UAsyncLoadingTests_Shared* Parent = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);
	TestNotNull(TEXT("Expected to be able to load our Parent object"), Parent);

	FName OriginalName = Parent->GetFName();
	Parent->Rename(TEXT("TRASH_TestObject"), nullptr, REN_DontCreateRedirectors);
	Parent->ClearFlags(RF_Standalone);
	Parent->MarkAsGarbage();

	UAsyncLoadingTests_Shared* ReloadedParent = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);
	TestNull(TEXT("Expected to be able Parent to no longer exist at the name FLoadingTestsScope::ObjectPath1"), ReloadedParent);

	// Load Child Package which imports the Parent package
	LoadPackage(nullptr, FLoadingTestsScope::PackagePath2, LOAD_None);

	UAsyncLoadingTests_Shared* Child = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath2);
	TestNotNull(TEXT("Expected to be able to load our Child object"), Child);

	ReloadedParent = FindObject<UAsyncLoadingTests_Shared>(nullptr, FLoadingTestsScope::ObjectPath1);
	TestNotNull(TEXT("Expected to be able to load our ReloadedParent object"), ReloadedParent);

	TestEqual(TEXT("Expected Child package to import the renamed Parent object"), Cast<UAsyncLoadingTests_Shared>(Child->HardReference), ReloadedParent);
	TestNotEqual(TEXT("Expected the reloaded parent to be a new instance and not the trashed parent object"), Parent, ReloadedParent);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
