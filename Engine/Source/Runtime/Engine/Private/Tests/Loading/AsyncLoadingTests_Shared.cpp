// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Misc/RedirectCollector.h"
#include "UObject/MetaData.h"
#include "UObject/SavePackage.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncLoadingTests_Shared)

UAsyncLoadingTests_Shared::FOnConstructDelegate UAsyncLoadingTests_Shared::OnConstruct;
UAsyncLoadingTests_Shared::FOnPostLoadDelegate UAsyncLoadingTests_Shared::OnPostLoad;
UAsyncLoadingTests_Shared::FOnSerializeDelegate UAsyncLoadingTests_Shared::OnSerialize;
UAsyncLoadingTests_Shared::FOnIsReadyForAsyncPostLoadDelegate UAsyncLoadingTests_Shared::OnIsReadyForAsyncPostLoad;
UAsyncLoadingTests_Shared::FOnIsPostLoadThreadSafeDelegate UAsyncLoadingTests_Shared::OnIsPostLoadThreadSafe;

#if WITH_DEV_AUTOMATION_TESTS

void FLoadingTestsScope::CreateObjects()
{
	Package1 = CreatePackage();
	Object1 = NewObject<UAsyncLoadingTests_Shared>(Package1, ObjectName, RF_Public | RF_Standalone);

	Package2 = CreatePackage();
	Object2 = NewObject<UAsyncLoadingTests_Shared>(Package2, ObjectName, RF_Public | RF_Standalone);
	
	Package3 = CreatePackage();
	Object3 = NewObject<UAsyncLoadingTests_Shared>(Package3, ObjectName, RF_Public | RF_Standalone);

	Package4 = CreatePackage();
	Object4 = NewObject<UAsyncLoadingTests_Shared>(Package4, ObjectName, RF_Public | RF_Standalone);
}

void FLoadingTestsScope::DefaultMutateObjects()
{
	// This is the soft reference that we want to test loading for
	Object1->SoftReference = Object2;
}

void FLoadingTestsScope::SavePackages()
{
	// To avoid an error on save, we need to mark the package as fully loaded.
	for (const FString& PackageName : PackageNames)
	{
		if (UPackage* Package = FindObject<UPackage>(nullptr, *PackageName))
		{
			Package->MarkAsFullyLoaded();
		}
	}

	// Save packages to disk.
	for (const FString& PackageName : PackageNames)
	{
		if (UPackage* Package = FindObject<UPackage>(nullptr, *PackageName))
		{
			check(UPackage::SavePackage(Package, nullptr, *FPackageName::LongPackageNameToFilename(*PackageName, FPackageName::GetAssetPackageExtension()), FSavePackageArgs()));
		}
	}
}

void FLoadingTestsScope::GarbageCollect()
{
	GarbageCollect(PackageNames, AutomationTest);
}

void FLoadingTestsScope::GarbageCollect(const TArray<FString>& PackageNames, FAutomationTestBase& AutomationTest)
{
	TArray<FString> ObjectPaths;

	// Remove RF_Standalone from package and objects inside it.
	for (const FString& PackageName : PackageNames)
	{
		if (UPackage* Package = FindObject<UPackage>(nullptr, *PackageName))
		{
			ObjectPaths.Add(PackageName);

			ForEachObjectWithPackage(Package,
				[&ObjectPaths](UObject* Object)
				{
					ObjectPaths.Add(Object->GetPathName());
					Object->ClearFlags(RF_Standalone);
					return true;
				}
			);
		}
	}

	// Make sure everything we gathered can be properly found
	for (const FString& ObjectPath : ObjectPaths)
	{
		AutomationTest.TestTrue(FString::Printf(TEXT("%s should be present in memory"), *ObjectPath), FindObject<UObject>(nullptr, *ObjectPath) != nullptr);
	}

	{
		// Make sure the GIsInitialLoad flag is false.  Otherwise GC does nothing
		TGuardValue<bool> GuardIsInitialLoad(GIsInitialLoad, false);

		// GC and make sure everything gets cleaned up before loading
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	// Now make sure everything is gone
	for (const FString& ObjectPath : ObjectPaths)
	{
		AutomationTest.TestTrue(FString::Printf(TEXT("%s should have been garbage collected"), *ObjectPath), FindObject<UObject>(nullptr, *ObjectPath) == nullptr);
	}
}

void FLoadingTestsScope::LoadObjects()
{
	LoadPackage(nullptr, PackagePath1, LOAD_None);

	UAsyncLoadingTests_Shared* LocalObject1 = FindObject<UAsyncLoadingTests_Shared>(nullptr, ObjectPath1);
	AutomationTest.TestTrue(TEXT("The object should have been properly loaded recursively"), LocalObject1 != nullptr);
	if (LocalObject1)
	{
		AutomationTest.TestFalse(TEXT("The object should have been properly loaded recursively"), LocalObject1->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad));
	}
}

void FLoadingTestsScope::CleanupObjects()
{
	// Remove any asset path redirections that were registered for test packages
	// (e.g., from ObjectRedirectors created by Rename operations and picked up by the asset registry).
	// These persist in GRedirectCollector across tests and would corrupt soft object path resolution.
#if WITH_EDITOR
	for (const FString& PackageName : PackageNames)
	{
		FSoftObjectPath ObjectPath(PackageName + TEXT(".") + ObjectName);
		GRedirectCollector.RemoveAssetPathRedirection(ObjectPath);
	}
#endif

	GarbageCollect();

	// Delete on-disk package files to avoid stale assets leaking between tests
	for (const FString& PackageName : PackageNames)
	{
		FString FilePath = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
		IFileManager::Get().Delete(*FilePath);
	}

	UAsyncLoadingTests_Shared::OnConstruct.Unbind();
	UAsyncLoadingTests_Shared::OnPostLoad.Unbind();
	UAsyncLoadingTests_Shared::OnSerialize.Unbind();
	UAsyncLoadingTests_Shared::OnIsPostLoadThreadSafe.Unbind();
	UAsyncLoadingTests_Shared::OnIsReadyForAsyncPostLoad.Unbind();
}

#endif // WITH_DEV_AUTOMATION_TESTS

namespace LoadingTestsUtils
{
	extern bool IsAssetSuitableForTests(const FAssetData& AssetData)
	{
		FString PackageName = AssetData.PackageName.ToString();

		// Assets from plugins can be problematic because they are either not accessible, or sometimes have issues,
		// so let's limit ourselves to ones from the engine or the game.
		if (!PackageName.StartsWith(TEXT("/Engine/")) && !PackageName.StartsWith(TEXT("/Game/")))
		{
			return false;
		}

		// We're skipping WorldPartition assets because some HLOD Layers (engine objects) reference settings objects
		// defined in editor-only plugins, which obviously fails to load on non-editor targets.
		if (PackageName.StartsWith(TEXT("/Game/Tests/WorldPartition")))
		{
			return false;
		}

		return true;
	}
}
