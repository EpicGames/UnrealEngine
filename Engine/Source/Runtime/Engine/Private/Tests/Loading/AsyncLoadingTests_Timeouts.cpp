// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Loading/AsyncLoadingTests_Shared.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Async/ManualResetEvent.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FLoadingTests_Timeouts_Postload,
	TEXT("System.Engine.Loading.Timeouts.Postload"),
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)
bool FLoadingTests_Timeouts_Postload::RunTest(const FString& Parameters)
{
	FLoadingTestsScope LoadingTestScope(this);

	UE::FManualResetEvent PostLoadWaitEvent;

	int32 PostLoadCount = 0;
	UAsyncLoadingTests_Shared::OnPostLoad.BindLambda(
		[this, &PostLoadCount](UAsyncLoadingTests_Shared* Object)
		{
			PostLoadCount++;
			FPlatformProcess::Sleep(1.5);
		}
	);

	// Trigger the async loading
	LoadPackageAsync(FLoadingTestsScope::PackagePath1);
	LoadPackageAsync(FLoadingTestsScope::PackagePath2);

	// Try to wait until all postload are finished but we're timing out after 1 seconds, which doesn't let enough time to process both postloads.
	EAsyncPackageState::Type State = ProcessAsyncLoadingUntilComplete([&PostLoadCount]() { return PostLoadCount == 2; }, 1.0);
	
	// The first call should have postloaded a single object and returned due to the elapse time.
	TestEqual("Async loading should have timed out", State, EAsyncPackageState::TimeOut);
	TestEqual("Only the first package should have been postloaded due to the timeout", PostLoadCount, 1);

	// On the second iteration, we should be able to get throught the second postload
	State = ProcessAsyncLoadingUntilComplete([&PostLoadCount]() { return PostLoadCount == 2; }, 5.0);

	// The old loader could return PendingImports... do we want to make a special case for zenloader? 
	TestTrue("Async loading should not have timeout on the second wait", State == EAsyncPackageState::Complete || State == EAsyncPackageState::PendingImports); 
	TestEqual("All packages should have been postloaded by now", PostLoadCount, 2);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
