// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Tasks/Task.h"
#include "Tests/Loading/AsyncLoadingTests_Shared.h"

#if WITH_DEV_AUTOMATION_TESTS

#undef TEST_NAME_ROOT
#define TEST_NAME_ROOT "System.Engine.Loading"

namespace UnattendedLoadTestImpl
{
	// We're still missing a simple multi-consumer thread-safe queue in the engine...
	// So make a custom one for our purpose.
	template <typename T>
	class TThreadSafeQueue
	{
		mutable FRWLock Lock;
		TArray<T> Array;
	public:
		TThreadSafeQueue(TArray<T>&& InArray)
			: Array(MoveTemp(InArray))
		{
		}

		bool Dequeue(T& OutName)
		{
			FWriteScopeLock Scope(Lock);
			if (Array.IsEmpty())
			{
				return false;
			}

			OutName = Array.Pop(EAllowShrinking::No);
			return true;
		}

		int32 Num() const
		{
			FReadScopeLock Scope(Lock);
			return Array.Num();
		}
	};

	// We use this to make sure the Lambda is only moved and never copied
	// and gets called at least one time before being deleted.
	// This is for testing purpose and is not required for the unattended method to work.
	class FExecutionCheck
	{
		enum class EExecutionCheck
		{
			NotExecuted,
			Executed,
			MovedFrom
		};

		EExecutionCheck ExecutionState = EExecutionCheck::NotExecuted;
	public:
		FExecutionCheck() = default;
		~FExecutionCheck()
		{
			check(ExecutionState == EExecutionCheck::Executed || ExecutionState == EExecutionCheck::MovedFrom);
		}
		FExecutionCheck(const FExecutionCheck&)
		{
			checkf(false, TEXT("Delegate should be moved, not copied"));
		}
		FExecutionCheck& operator= (const FExecutionCheck&)
		{
			checkf(false, TEXT("Delegate should be moved, not copied"));
			return *this;
		}
		FExecutionCheck(FExecutionCheck&& Other)
		{
			checkf(ExecutionState != EExecutionCheck::Executed, TEXT("Probably don't want to move from an executed object"));
			ExecutionState = Other.ExecutionState;
			Other.ExecutionState = EExecutionCheck::MovedFrom;
		}
		FExecutionCheck& operator=(FExecutionCheck&& Other)
		{
			FExecutionCheck Temp = MoveTemp(Other);
			Swap(Temp, *this);
			return *this;
		}
		void SetHasBeenExecuted() 
		{
			check(ExecutionState != EExecutionCheck::MovedFrom);
			ExecutionState = EExecutionCheck::Executed;
		}
		bool HasBeenExecuted() const { return ExecutionState == EExecutionCheck::Executed; }
	};

	// The UnattendedLoader will queue the next async loading as soon as a package finishes its serialization phase
	// or finishes loading for any reason. 
	class FUnattendedLoader
	{
		bool bFinished = false;
		TThreadSafeQueue<FName> PackagesToLoad;

		void LoadNextPackage()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadNextPackage);

			FName PackageName;
			if (PackagesToLoad.Dequeue(PackageName))
			{
				UE_LOGF(LogTemp, Display, "Loading %ls (Remaining: %d)", *PackageName.ToString(), PackagesToLoad.Num());
				LoadPackageAsync(PackageName.ToString(),
					{
						.ProgressDelegate = MakeShared<FLoadPackageAsyncProgressDelegate>(
							FLoadPackageAsyncProgressDelegate::DelegateType::CreateLambda(
								[this, ExecutionCheck = FExecutionCheck()](FLoadPackageAsyncProgressParams& Params) mutable
								{
									// If we already triggered the next load from this callback, don't do anything else
									if (ExecutionCheck.HasBeenExecuted())
									{
										return;
									}
							
									switch (Params.ProgressType)
									{
										// Filter out some progress types as we wait until later to start loading the next package
										case EAsyncLoadingProgress::Started:
											[[fallthrough]];
										case EAsyncLoadingProgress::Read:
											return;

										// Whenever we receive a progress type that indicate the request has finished being processed
										// we schedule the next package to be loaded.
										// LoadNextPackage is thread-safe when using zenloader, so we can run directly from the
										// callback even if it comes from the ALT thread.
										case EAsyncLoadingProgress::Failed:
											[[fallthrough]];
										case EAsyncLoadingProgress::FullyLoaded:
											[[fallthrough]];
										case EAsyncLoadingProgress::Serialized:
											// Serialized is an intermediate state that happens just before postloading but it generally
											// indicate that what's left to be processed needs to happen on the game-thread.
											// This event is expected to be called directly from ALT, so this is a perfect place to
											// schedule the next load in our unattended test. 
											LoadNextPackage();
											// We expect further progress to happen on the same package, so we set this
											// to make sure only one package gets scheduled per load request.
											ExecutionCheck.SetHasBeenExecuted();
											break;
										default:
											checkNoEntry();
											break;
									}
								}))
					}
				);
			}
			else
			{
				UE_LOGF(LogTemp, Display, "Loading finished");
				bFinished = true;
			}
		}
	public:
		FUnattendedLoader(TArray<FName>&& InPackagesToLoad)
			: PackagesToLoad(MoveTemp(InPackagesToLoad))
		{
			LoadNextPackage();
		}

		bool HasFinished() const
		{
			return bFinished;
		}
	};
}

/**
 * This test demonstrate how to consume a list of asset that needs loading directly from the async loading thread
 * without involving the game thread by monitoring asset loading progress and starting the next async load once the
 * asset currently being loaded reach a certain state.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnattendedLoadingTest, TEXT(TEST_NAME_ROOT ".UnattendedAsyncLoadingTest"), EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)
bool FUnattendedLoadingTest::RunTest(const FString& Parameters)
{
	using namespace UnattendedLoadTestImpl;

	// We use the asset registry to get a list of asset to load. 
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();
	AssetRegistry.WaitForCompletion();

	// Limit the number of packages we're going to load for the test in case the project is very big.
	constexpr int32 MaxPackageCount = 5000;

	TSet<FName> UniquePackages;
	AssetRegistry.EnumerateAllAssets(
		[&UniquePackages](const FAssetData& AssetData)
		{
			if (UniquePackages.Num() < MaxPackageCount)
			{
				if (LoadingTestsUtils::IsAssetSuitableForTests(AssetData))
				{
					UniquePackages.FindOrAdd(AssetData.PackageName);
				}

				return true;
			}
			
			return false;
		},
		UE::AssetRegistry::EEnumerateAssetsFlags::OnlyOnDiskAssets
	);

	// Use the unattended loader to load all unique packages gathered in the list.
	// The loader will self-feed package to load directly from the callback which
	// should happen on the ALT when zenloader and async loading thread are both
	// active.
	FUnattendedLoader Loader(UniquePackages.Array());

	// For the purpose of this test, we manually tick the GT so it can process postloads
	// and finalize some package loading steps.
	while (GetNumAsyncPackages() != 0 || !Loader.HasFinished())
	{
		ProcessAsyncLoading(true/*bUseTimeLimit*/, false/*bUseFullTimeLimit*/, 1.0f);
	}
	
	return true;
}

#undef TEST_NAME_ROOT
#endif // WITH_DEV_AUTOMATION_TESTS