// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/NetTestHelpers.h"

#include "CoreGlobals.h"
#include "Engine/NetworkObjectList.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/ChildConnection.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "Engine/GameViewportClient.h"
#include "Engine/PackageMapClient.h"
#include "EngineUtils.h"
#include "GameFramework/PlayerState.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMath.h"
#include "Kismet/GameplayStatics.h"
#include "Net/OnlineEngineInterface.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/Trace/NetTrace.h"

#if WITH_EDITOR
#include "Settings/LevelEditorPlaySettings.h"
#endif

#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::Net
{

FScopedNetEmulationOverride::FScopedNetEmulationOverride(UNetDriver* InNetDriver, const FPacketSimulationSettings& InSettings)
	: WeakNetDriver(InNetDriver)
#if DO_ENABLE_NET_TEST
	, SavedSettings(InNetDriver->PacketSimulationSettings)
#endif
{
#if DO_ENABLE_NET_TEST
	InNetDriver->PacketSimulationSettings = InSettings;
	InNetDriver->OnPacketSimulationSettingsChanged();
#else
	ensureMsgf(false, TEXT("FScopedNetEmulationOverride will have no effect since DO_ENABLE_NET_TEST is not defined."));
#endif
}

FScopedNetEmulationOverride::~FScopedNetEmulationOverride()
{
#if DO_ENABLE_NET_TEST
	UNetDriver* NetDriver = WeakNetDriver.Get();
	if (ensureMsgf(NetDriver, TEXT("~FScopedNetEmulationOverride: WeakNetDriver is invalid.")))
	{
		NetDriver->PacketSimulationSettings = SavedSettings;
		NetDriver->OnPacketSimulationSettingsChanged();
	}
#endif
}

#if WITH_EDITOR

bool IsClientConnected(FTestWorldInstance& Instance)
{
	UNetDriver* NetDriver = Instance.GetNetDriver();
	if (NetDriver && NetDriver->ServerConnection)
	{
		if (NetDriver->ServerConnection->GetConnectionState() == USOCK_Open)
		{
			return true;
		}

		for (UChildConnection* ChildConnection : NetDriver->ServerConnection->Children)
		{
			if (ChildConnection->GetConnectionState() != USOCK_Open)
			{
				return false;
			}
		}
	}

	return false;
}

bool IsServerConnected(FTestWorldInstance& Instance)
{
	UNetDriver* NetDriver = Instance.GetNetDriver();
	if (NetDriver)
	{
		for (UNetConnection* Connection : NetDriver->ClientConnections)
		{
			if (Connection->GetConnectionState() != USOCK_Open)
			{
				return false;
			}

			for (UChildConnection* ChildConnection : Connection->Children)
			{
				if (ChildConnection->GetConnectionState() != USOCK_Open)
				{
					return false;
				}
			}
		}

		return true;
	}

	return false;
}

FTestWorldInstance FTestWorldInstance::CreateServer(const TCHAR* InURL, const EReplicationSystem ReplicationSystem)
{
	const bool bPrevShouldUseIrisReplication = ShouldUseIrisReplication();
	if (ReplicationSystem == EReplicationSystem::Iris)
	{
		SetUseIrisReplication(true);
	}
	else if (ReplicationSystem == EReplicationSystem::Generic)
	{
		SetUseIrisReplication(false);
	}

	FGameInstancePIEParameters ServerParams;
	ServerParams.bSimulateInEditor = false;
	ServerParams.bAnyBlueprintErrors = false;
	ServerParams.bStartInSpectatorMode = false;
	ServerParams.bRunAsDedicated = true;
	ServerParams.bIsPrimaryPIEClient = false;
	ServerParams.WorldFeatureLevel = ERHIFeatureLevel::Num;
	ServerParams.EditorPlaySettings = nullptr;
	ServerParams.NetMode = PIE_ListenServer;
	ServerParams.OverrideMapURL = TEXT("/Engine/Maps/Entry"); // always start from an empty map. Otherwise it will clone the current editor map to start the server in.

	FTestWorldInstance NewInstance(ServerParams);

	{
		// If something calls a Function during initialization of this world it needs to have the PIE instance ID accurate or cleared, see UEngine::GetGlobalFunctionCallspace where GetCurrentPlayWorld is called.
		FScopedNetTestPIERestoration PieScope;
		UE::SetPlayInEditorID(-1);

		FURL LocalURL(nullptr, InURL, TRAVEL_Absolute);
		FString BrowseError;
		const EBrowseReturnVal::Type Result = GEngine->Browse(*NewInstance.GameInstance->GetWorldContext(), LocalURL, BrowseError);
		NewInstance.bIsLoaded = (Result == EBrowseReturnVal::Success);
	}

	if (ReplicationSystem != EReplicationSystem::Default)
	{
		SetUseIrisReplication(bPrevShouldUseIrisReplication);
	}

	return NewInstance;
}

FTestWorldInstance FTestWorldInstance::CreateClient(int32 ServerPort, int32 NumLocalPlayers)
{
	FGameInstancePIEParameters ClientParams;
	ClientParams.bSimulateInEditor = false;
	ClientParams.bAnyBlueprintErrors = false;
	ClientParams.bStartInSpectatorMode = false;
	ClientParams.bRunAsDedicated = false;
	ClientParams.bIsPrimaryPIEClient = false;
	ClientParams.WorldFeatureLevel = ERHIFeatureLevel::Num;
	ClientParams.EditorPlaySettings = nullptr;
	ClientParams.NetMode = PIE_Client;

	FTestWorldInstance NewInstance(ClientParams);

	FWorldContext* ClientWorldContext = NewInstance.GameInstance->GetWorldContext();

	UGameViewportClient* ViewportClient = NewObject<UGameViewportClient>(GEngine, GEngine->GameViewportClientClass);
	ViewportClient->Init(*ClientWorldContext, NewInstance.GameInstance);
	ClientWorldContext->GameViewport = ViewportClient;

	FString OutCreatePlayerError;
	ViewportClient->SetupInitialLocalPlayer(OutCreatePlayerError);
	{
		// If something calls a Function during initialization of this world it needs to have the PIE instance ID accurate or cleared, see UEngine::GetGlobalFunctionCallspace where GetCurrentPlayWorld is called.
		FScopedNetTestPIERestoration PieScope;
		UE::SetPlayInEditorID(-1);
		GEngine->BrowseToDefaultMap(*ClientWorldContext);

		FString ClientURLString = FString::Format(TEXT("127.0.0.1:{0}"), { ServerPort });

		FURL ClientURL(nullptr, *ClientURLString, TRAVEL_Absolute);
		FString ClientBrowseError;
		const EBrowseReturnVal::Type Result = GEngine->Browse(*ClientWorldContext, ClientURL, ClientBrowseError);
		NewInstance.bIsLoaded = (Result == EBrowseReturnVal::Success);
	}

	for (int32 LocalPlayerIndex = 1; LocalPlayerIndex < NumLocalPlayers; LocalPlayerIndex++)
	{
		FString Error;
		NewInstance.GameInstance->CreateLocalPlayer(LocalPlayerIndex, Error, true);
	}

	return NewInstance;
}

FTestWorldInstance FTestWorldInstance::CreateProxy()
{
	FGameInstancePIEParameters ServerParams;
	ServerParams.bSimulateInEditor = false;
	ServerParams.bAnyBlueprintErrors = false;
	ServerParams.bStartInSpectatorMode = false;
	ServerParams.bRunAsDedicated = true;
	ServerParams.bIsPrimaryPIEClient = false;
	ServerParams.WorldFeatureLevel = ERHIFeatureLevel::Num;
	ServerParams.EditorPlaySettings = nullptr;
	ServerParams.NetMode = PIE_ListenServer;
	ServerParams.OverrideMapURL = TEXT("/Engine/Maps/Entry"); // always start from an empty map. Otherwise it will clone the current editor map to start the server in.
		
	FTestWorldInstance NewInstance(ServerParams);

	FURL LocalURL(nullptr, TEXT("/Engine/Maps/Entry?listen?NetDriverDef=ProxyNetDriver"), TRAVEL_Absolute);
	FString BrowseError;
	GEngine->Browse(*NewInstance.GameInstance->GetWorldContext(), LocalURL, BrowseError);

	return NewInstance;
}

FTestWorldInstance::FTestWorldInstance(const FGameInstancePIEParameters& InstanceParams)
{
	GameInstance = NewObject<UGameInstance>(GEngine, UGameInstance::StaticClass());
	GameInstance->AddToRoot();

	GameInstance->InitializeForPlayInEditor(FindUnusedPIEInstance(), InstanceParams);

	FWorldContext* WorldContext = GameInstance->GetWorldContext();
}

FTestWorldInstance& FTestWorldInstance::operator=(FTestWorldInstance&& Other)
{
	if (this != &Other)
	{
		Shutdown();

		GameInstance = Other.GameInstance;
		bIsLoaded = Other.bIsLoaded;

		Other.GameInstance = nullptr;
		Other.bIsLoaded = false;
	}

	return *this;
}

FTestWorldInstance::~FTestWorldInstance()
{
	Shutdown();
}

void FTestWorldInstance::Shutdown()
{
	UWorld* World = GetWorld();
	FWorldContext* WorldContext = nullptr;

	if (World)
	{
		TGuardConsoleVariable<bool> CVarOverrideRPCSendWarnings(TEXT("net.Iris.EnableRPCSendWarnings"), false);
		World->EndPlay(EEndPlayReason::EndPlayInEditor);
	}

	if (GameInstance)
	{
		WorldContext = GetWorldContext();
		GameInstance->Shutdown();
		GameInstance->RemoveFromRoot();
	}
	
	if (World)
	{
		GEngine->ShutdownWorldNetDriver(World);

		// Cleanup online subsystems instantiated during test
		UOnlineEngineInterface* Online = UOnlineEngineInterface::Get();
		if (Online && WorldContext)
		{
			FName OnlineIdentifier = Online->GetOnlineIdentifier(*WorldContext);
			if (Online->DoesInstanceExist(OnlineIdentifier))
			{
				Online->ShutdownOnlineSubsystem(OnlineIdentifier);
				Online->DestroyOnlineSubsystem(OnlineIdentifier);
			}
		}

		constexpr bool bInformEngineOfDestroyedWorld = true;
		World->DestroyWorld(bInformEngineOfDestroyedWorld);
		GEngine->DestroyWorldContext(World);
	}
}

int32 FTestWorldInstance::FindUnusedPIEInstance()
{
	if (!GEngine)
	{
		return INDEX_NONE;
	}

	int32 MaxUsedPIEInstance = INDEX_NONE;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		MaxUsedPIEInstance = FMath::Max(MaxUsedPIEInstance, Context.PIEInstance);
	}

	return MaxUsedPIEInstance + 1;
}

FTestWorldInstance::FContext FTestWorldInstance::GetTestContext() const
{
	UReplicationSystem* RepSystem = GetNetDriver() ? GetNetDriver()->GetReplicationSystem() : nullptr;
	UObjectReplicationBridge* RepBridge  = RepSystem ? RepSystem->GetReplicationBridge() : nullptr;

	return FContext
	{ 
		.World = GetWorld(),
		.NetDriver = GetNetDriver(),
		.IrisRepSystem = RepSystem,
		.IrisRepBridge = RepBridge,
	};
}

UWorld* FTestWorldInstance::GetWorld() const
{
	return GameInstance ? GameInstance->GetWorld() : nullptr;
}

FWorldContext* FTestWorldInstance::GetWorldContext() const
{
	return GameInstance->GetWorldContext();
}

UNetDriver* FTestWorldInstance::GetNetDriver() const
{
	UWorld* World = GetWorld();
	return World ? World->GetNetDriver() : nullptr;
}

void FTestWorldInstance::Tick(float DeltaSeconds)
{
	GEngine->TickWorldTravel(*GetWorldContext(), DeltaSeconds);
	if (UWorld* World = GetWorld())
	{
		World->Tick(LEVELTICK_All, DeltaSeconds);
		World->UpdateLevelStreaming();
	}
}

int32 FTestWorldInstance::GetPort() const
{
	if (const UWorld* const World = GetWorld())
	{
		if (UNetDriver* const NetDriver = World->GetNetDriver())
		{
			if (NetDriver->GetLocalAddr().IsValid())
			{
				return NetDriver->GetLocalAddr()->GetPort();
			}
		}
	}

	return 0;
}

void FTestWorldInstance::LoadStreamingLevel(FName LevelName)
{
	FLatentActionInfo LoadLatentInfo;
	LoadLatentInfo.UUID = LevelStreamRequestUUID++;
	constexpr bool bMakeVisibleAfterLoad = true;
	constexpr bool bShouldBlockOnLoad = true;
	UGameplayStatics::LoadStreamLevel(GetWorld(), LevelName, bMakeVisibleAfterLoad, bShouldBlockOnLoad, LoadLatentInfo);
}

void FTestWorldInstance::UnloadStreamingLevel(FName LevelName)
{
	FLatentActionInfo UnloadLatentInfo;
	UnloadLatentInfo.UUID = LevelStreamRequestUUID++;
	constexpr bool bShouldBlockOnUnload = true;
	UGameplayStatics::UnloadStreamLevel(GetWorld(), LevelName, UnloadLatentInfo, bShouldBlockOnUnload);
}

//============================================================================
// FTestWorlds
//============================================================================

FTestWorlds::FTestWorlds(const EReplicationSystem ReplicationSystem)
	: Server(true)
{
	Server = FTestWorldInstance::CreateServer(TEXT("/Engine/Maps/Entry?listen?Game=/Script/Engine.GameMode"), ReplicationSystem);
}

FTestWorlds::FTestWorlds(const FString& MapName, const FString& GameModeName, const EReplicationSystem ReplicationSystem)
	: Server(true)
{
	FString URL = FString::Printf(TEXT("%s?listen?Game=%s"), *MapName, *GameModeName);

	Server = FTestWorldInstance::CreateServer(*URL, ReplicationSystem);
}

FTestWorlds::FTestWorlds(const TCHAR* ServerURL, const EReplicationSystem ReplicationSystem)
	: Server(true)
{
	Server = FTestWorldInstance::CreateServer(ServerURL, ReplicationSystem);
}

FTestWorlds::~FTestWorlds()
{
}

void FTestWorlds::SetTickInSeconds(float TickInSeconds)
{
	TickDeltaSeconds = TickInSeconds;
}

bool FTestWorlds::CreateAndConnectClient()
{
	Clients.Emplace(FTestWorldInstance::CreateClient(Server.GetPort()));
	const bool bConnected = WaitForClientConnect(Clients.Last());
	return bConnected;
}

int32 FTestWorlds::ConnectNewClient()
{
	Clients.Emplace(FTestWorldInstance::CreateClient(Server.GetPort()));
	const bool bConnected = WaitForClientConnect(Clients.Last());
	return bConnected ? (Clients.Num()-1) : INDEX_NONE;
}

int32 FTestWorlds::FindClientIdFromWorld(const UWorld* ClientWorld) const
{
	const int32 Index = Clients.IndexOfByPredicate([ClientWorld](const FTestWorldInstance& ClientInstance)
	{
		return ClientInstance.GetWorld() == ClientWorld;
	});
	
	return Index;
}

FTestWorldInstance::FContext FTestWorlds::GetClientContext(int32 ClientId) const
{
	if (Clients.IsValidIndex(ClientId))
	{
		return Clients[ClientId].GetTestContext();
	}

	return {};
}

bool FTestWorlds::WaitForClientConnect(FTestWorldInstance& Client)
{
	bool bConnected = TickAllUntil([&Client]()
	{
		if (UWorld* World = Client.GetWorld())
		{
			APlayerController* PC = World->GetFirstPlayerController();
			return IsValid(PC) && (PC->GetLocalRole() == ROLE_AutonomousProxy);
		}
		return false;
	});

	if (bConnected)
	{
		OnClientConnected.Broadcast(Client);
		UE_NET_TRACE_UPDATE_INSTANCE(Client.GetNetDriver()->GetNetTraceId(), false, *Description);
	}

	return bConnected;
}

void FTestWorlds::SetDescription(const FString& InDescription)
{
	Description = InDescription;

#if UE_NET_TRACE_ENABLED
	if (Server.GetNetDriver())
	{
		UE_NET_TRACE_UPDATE_INSTANCE(Server.GetNetDriver()->GetNetTraceId(), true, *Description);
	}
	for (FTestWorldInstance& ClientIt : Clients)
	{
		if (ClientIt.GetNetDriver())
		{
			UE_NET_TRACE_UPDATE_INSTANCE(ClientIt.GetNetDriver()->GetNetTraceId(), false, *Description);
		}
	}
#endif
}

void FTestWorlds::TickAll(int32 NumTicks)
{
	PreTickAllDelegate.Broadcast();

	for (int32 i = 0; i < NumTicks; ++i)
	{
		TickServer();
		TickClients();
		GFrameCounter++;
	}
}

void FTestWorlds::TickServer()
{
	Server.Tick(TickDeltaSeconds);
}

void FTestWorlds::TickClients()
{
	for (FTestWorldInstance& Client : Clients)
	{
		Client.Tick(TickDeltaSeconds);
	}
}

void FTestWorlds::TickServerAndDrop()
{
#if DO_ENABLE_NET_TEST
	UNetDriver* NetDriver = Server.GetNetDriver();
	NetDriver->PacketSimulationSettings.PktLoss = 100;
	NetDriver->OnPacketSimulationSettingsChanged();
	
	Server.Tick(TickDeltaSeconds);

	NetDriver->PacketSimulationSettings.PktLoss = 0;
	NetDriver->OnPacketSimulationSettingsChanged();
#else
	UE_LOGF(LogNet, Error, "FTestWorlds::TickServerAndDrop does not work called without NetDriver Simulation Settings");
#endif
}

void FTestWorlds::TickClientsAndDrop()
{
#if DO_ENABLE_NET_TEST
	for (FTestWorldInstance& Client : Clients)
	{
		UNetDriver* NetDriver = Client.GetNetDriver();
		NetDriver->PacketSimulationSettings.PktLoss = 100;
		NetDriver->OnPacketSimulationSettingsChanged();

		Client.Tick(TickDeltaSeconds);
		
		NetDriver->PacketSimulationSettings.PktLoss = 0;
		NetDriver->OnPacketSimulationSettingsChanged();
	}
#else
	UE_LOGF(LogNet, Error, "FTestWorlds::TickClientsAndDrop does not work without NetDriver Simulation Settings");
#endif
}

void FTestWorlds::TickServerAndDelay(uint32 NumFramesToDelay)
{
#if DO_ENABLE_NET_TEST
	UNetDriver* NetDriver = Server.GetNetDriver();
	NetDriver->PacketSimulationSettings.PktFrameDelay = NumFramesToDelay;
	NetDriver->OnPacketSimulationSettingsChanged();

	Server.Tick(TickDeltaSeconds);

	NetDriver->PacketSimulationSettings.PktFrameDelay = 0;
	NetDriver->OnPacketSimulationSettingsChanged();
#else
	UE_LOGF(LogNet, Error, "FTestWorlds::TickServerAndDelay does not work without NetDriver Simulation Settings");
#endif
}

void FTestWorlds::TickClientsAndDelay(uint32 NumFramesToDelay)
{
#if DO_ENABLE_NET_TEST
	for (FTestWorldInstance& Client : Clients)
	{
		UNetDriver* NetDriver = Client.GetNetDriver();
		NetDriver->PacketSimulationSettings.PktFrameDelay = NumFramesToDelay;
		NetDriver->OnPacketSimulationSettingsChanged();

		Client.Tick(TickDeltaSeconds);

		NetDriver->PacketSimulationSettings.PktFrameDelay = 0;
		NetDriver->OnPacketSimulationSettingsChanged();
	}
#else
	UE_LOGF(LogNet, Error, "FTestWorlds::TickClientsAndDelay does not work without NetDriver Simulation Settings");
#endif
}

 APlayerController* FTestWorlds::GetServerPlayerControllerOfClient(uint32 ClientIndex)
{
	if (Clients.IsValidIndex(ClientIndex) == false)
	{
		// No client exists for that index
		return nullptr;
	}

	int32 PlayerId = -1;
	// Get the unique info from the PlayerController on the client world
	{
		APlayerController* ClientPC = Clients[ClientIndex].GetWorld()->GetFirstPlayerController();
		if (!ClientPC || !ClientPC->PlayerState)
		{
			return nullptr;
		}

		PlayerId = ClientPC->PlayerState->GetPlayerId();
	}

	// Find the PlayerController on the server related to to this client
	UWorld* ServerWorld = Server.GetWorld();
	for (FConstPlayerControllerIterator PCIter = ServerWorld->GetPlayerControllerIterator(); PCIter; ++PCIter)
	{
		if (APlayerController* PC = PCIter->Get())
		{
			if (PC->PlayerState && PC->PlayerState->GetPlayerId() == PlayerId)
			{
				return PC;
			}
		}
	}

	return nullptr;
}

void FTestWorlds::MovePlayerToLocation(uint32 ClientIndex, const FVector& Location)
{
	if (Clients.IsValidIndex(ClientIndex) == false)
	{
		ensureMsgf(false, TEXT("No client exists for that index"));
		return;
	}

	APlayerController* ClientPC = GetServerPlayerControllerOfClient(ClientIndex);
	if (!ClientPC)
	{
		ensureMsgf(false, TEXT("No PlayerController found for this client"));
		return;
	}

	AActor* ViewTarget = ClientPC->GetViewTarget();
	ViewTarget->SetActorLocation(Location);
}

UObject* FTestWorlds::FindReplicatedObjectOnClient(UObject* ServerObject, uint32 ClientIndex) const
{
	using namespace UE::Net;

	if (ClientIndex == UINT32_MAX)
	{
		if (Clients.IsEmpty())
		{
			ensureMsgf(false, TEXT("%hs failed. No client world exist"), __FUNCTION__);
			return nullptr;
		}

		// Use the first client
		ClientIndex = 0;
	}

	if (!Clients.IsValidIndex(ClientIndex))
	{
		ensureMsgf(false, TEXT("FTestWorlds::FindReplicatedObjectOnClient received invalid ClientIndex: %u"), ClientIndex);
		return nullptr;
	}

	if (!ServerObject)
	{
		ensureMsgf(false, TEXT("Expected valid pointer"));
		return nullptr;
	}

	if (ServerObject->GetWorld() != Server.GetWorld() && ServerObject->GetOuter() != GetTransientPackage())
	{
		ensureMsgf(false, TEXT("FTestWorlds::FindReplicatedObjectOnClient received object %s not part of the Server world"), *GetFullNameSafe(ServerObject));
		return nullptr;
	}

	const FTestWorldInstance::FContext ServerContext = Server.GetTestContext();
	const FTestWorldInstance::FContext ClientContext = Clients[ClientIndex].GetTestContext();

	if (ServerContext.NetDriver->IsUsingIrisReplication())
	{
		const FNetRefHandle NetHandle = ServerContext.IrisRepBridge->GetReplicatedRefHandle(ServerObject);

		if (!NetHandle.IsValid())
		{
			ensureMsgf(false, TEXT("FTestWorlds::FindReplicatedObjectOnClient ServerObject: %s is not replicated."), *GetFullNameSafe(ServerObject));
			return nullptr;
		}

		return ClientContext.IrisRepBridge ? ClientContext.IrisRepBridge->GetReplicatedObject(NetHandle) : nullptr;
	}
	else
	{
		const FNetworkGUID NetGUID = ServerContext.NetDriver->GetNetGuidCache()->GetNetGUID(ServerObject);

		if (!NetGUID.IsValid())
		{
			ensureMsgf(false, TEXT("FTestWorlds::FindReplicatedObjectOnClient ServerObject: %s is not replicated."), *GetFullNameSafe(ServerObject));
			return nullptr;
		}

		return ClientContext.NetDriver->GetNetGuidCache()->GetObjectFromNetGUID(NetGUID, false);
	}
}

bool FTestWorlds::DoesReplicatedObjectExistOnClient(UObject* ServerObject, uint32 ClientIndex) const
{
	return IsValid(FindReplicatedObjectOnClient(ServerObject, ClientIndex));
}

bool FTestWorlds::IsServerObjectReplicated(const UObject* ServerObject) const
{
	using namespace UE::Net;

	if (ServerObject->GetWorld() != Server.GetWorld() && ServerObject->GetOuter() != GetTransientPackage())
	{
		ensureMsgf(false, TEXT("FTestWorlds::FindReplicatedObjectOnClient received object %s not part of the Server world"), *GetFullNameSafe(ServerObject));
		return false;
	}

	const FTestWorldInstance::FContext ServerContext = Server.GetTestContext();
	
	if (ServerContext.NetDriver->IsUsingIrisReplication())
	{
		const FNetRefHandle NetHandle = ServerContext.IrisRepBridge->GetReplicatedRefHandle(ServerObject);
		return NetHandle.IsValid();
	}
	else
	{
		const FNetworkGUID NetGUID = ServerContext.NetDriver->GetNetGuidCache()->GetNetGUID(ServerObject);
		return NetGUID.IsValid();
	}
}

//------------------------------------------------------------------------

FScopedCVarOverrideInt::FScopedCVarOverrideInt(const TCHAR* VariableName, int32 Value)
	: Variable(IConsoleManager::Get().FindConsoleVariable(VariableName, false /* false to avoid frequent find warnings */ ))
{
	if (Variable)
	{
		SavedValue = Variable->GetInt();
		Variable->Set(Value, ECVF_SetByConsole /* Use highest priority so we override even console setters */);
	}
}

FScopedCVarOverrideInt::~FScopedCVarOverrideInt()
{
	if (Variable)
	{
		Variable->Set(SavedValue, ECVF_SetByConsole);
	}
}

//------------------------------------------------------------------------

FScopedCVarOverrideFloat::FScopedCVarOverrideFloat(const TCHAR* VariableName, float Value)
: Variable(IConsoleManager::Get().FindConsoleVariable(VariableName, false /* false to avoid frequent find warnings */))
{
	if (Variable)
	{
		SavedValue = Variable->GetFloat();
		Variable->Set(Value, ECVF_SetByConsole /* Use highest priority so we override even console setters */);
	}
}

FScopedCVarOverrideFloat::~FScopedCVarOverrideFloat()
{
	if (Variable)
	{
		Variable->Set(SavedValue, ECVF_SetByConsole);
	}
}


//------------------------------------------------------------------------

FScopedTestSettings::FScopedTestSettings()
	: AddressResolutionDisabled(TEXT("net.IpConnectionDisableResolution"), 1)
	, BandwidthThrottlingDisabled(TEXT("net.DisableBandwithThrottling"), 1)
	, RepGraphBandwidthThrottlingDisabled(TEXT("Net.RepGraph.DisableBandwithLimit"), 1)
	, RandomNetUpdateDelayDisabled(TEXT("net.DisableRandomNetUpdateDelay"), 1)
	, GameplayDebuggerDisabled(TEXT("GameplayDebugger.AutoCreateGameplayDebuggerManager"), 0)
#if WITH_EDITOR
	, ClientsShareAudioDevice(TEXT("au.ForcePieClientsToShareAudio"), 1)
#endif
{
	NetDriverCreatedHandle = FWorldDelegates::OnNetDriverCreated.AddLambda([](UWorld* InWorld, UNetDriver* InNetDriver)
	{
		// Make sure NetDriver will tick every engine frame
		InNetDriver->MaxNetTickRate = 0;
	});
}

FScopedTestSettings::~FScopedTestSettings()
{
	FWorldDelegates::OnNetDriverCreated.Remove(NetDriverCreatedHandle);
}

//------------------------------------------------------------------------

FScopedNetTestPIERestoration::FScopedNetTestPIERestoration()
	: OldGWorld(GWorld)
	, OldPIEID(UE::GetPlayInEditorID())
	, OldGIsPlayInEditorWorld(GIsPlayInEditorWorld)
{

}

FScopedNetTestPIERestoration::~FScopedNetTestPIERestoration()
{
	GWorld = OldGWorld;
	UE::SetPlayInEditorID(OldPIEID);
	GIsPlayInEditorWorld = OldGIsPlayInEditorWorld;
}

#endif // WITH_EDITOR

} // end namespace UE::Net
