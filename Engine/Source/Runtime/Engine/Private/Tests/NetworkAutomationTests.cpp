// Copyright Epic Games, Inc. All Rights Reserved.


#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogNetworkAutomationTests, Log, All);


//enum for multiplayer test
namespace EMultiplayerAutomationRoles
{
	enum Type
	{
		Host = 0,
		Client0,

		MaxNumParticipants,
	};
};


/**
 * 2-Multiplayer session automation test
 * Verification for 2 player multiplayer session start up and tear down
 */
IMPLEMENT_NETWORKED_AUTOMATION_TEST(FMultiplayer4PlayerTest, "System.Networking.Multiplayer.TwoPlayerSessionStartupShutdown", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter, EMultiplayerAutomationRoles::MaxNumParticipants)


/** 
 * Load up a game session, invite players to join, accept invitations, quit
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FMultiplayer4PlayerTest::RunTest(const FString& Parameters)
{
	// Accessing the game world is only valid for game-only 
	check((GetTestFlags() & EAutomationTestFlags_ApplicationContextMask) == EAutomationTestFlags::ClientContext);
	check(GEngine->GetWorldContexts().Num() == 1);
	check(GEngine->GetWorldContexts()[0].WorldType == EWorldType::Game);

	START_NETWORK_AUTOMATION_COMMAND(OpenMap)
	{
		//Load Map
		FString MapName = "AutomationTest";
		GEngine->Exec(GEngine->GetWorldContexts()[0].World(), *FString::Printf(TEXT("Open %s"), *MapName));
		ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.0));
	}
	END_NETWORK_AUTOMATION_COMMAND(OpenMap, EMultiplayerAutomationRoles::Host)


	// TODO: Ask JB about this....
	for (int32 i = 0; i < EMultiplayerAutomationRoles::MaxNumParticipants; ++i)
	{
		//Invite Players
		START_NETWORK_AUTOMATION_COMMAND(InvitePlayers)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat memory")));
			ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(2.0));
			ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("stat memory")));
		}
		END_NETWORK_AUTOMATION_COMMAND(InvitePlayers, i)
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
