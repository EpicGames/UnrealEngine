// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Tests/AutomationCommon.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogHackAutomationTests, Log, All);


//////////////////////////////////////////////////////////////////////////

/*
 * Goes through OSS commands
 */

//////////////////////////////////////////////////////////////////////////


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPFriendsTest, "System.OSS.MCP.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPFriendsTest::RunTest(const FString& Parameters)
{

	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test friends")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAmazonFriendsTest, "System.OSS.Amazon.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAmazonFriendsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=amazon test friends")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFacebookFriendsTest, "System.OSS.Facebook.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FFacebookFriendsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=facebook test friends")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSteamFriendsTest, "System.OSS.Steam.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSteamFriendsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=steam test friends")));

	return true;
}

#if PLATFORM_IOS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIOSFriendsTest, "System.OSS.IOS.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FIOSFriendsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=ios test friends")));

	return true;
}

#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOculusFriendsTest, "System.OSS.Oculus.Test Friends", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FOculusFriendsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=oculus test friends")));

	return true;
}

/*

/////////////Currently crashing//////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNullFriendsTest,"System.OSS.NULL.Test Friends", EAutomationTestFlags::EditorContext )

bool FNullFriendsTest::RunTest(const FString& Parameters)
{
	
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=null test friends")));

	return true;
}
*/

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPCloudTest, "System.OSS.MCP.Test Cloud", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPCloudTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test cloud")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSteamCloudTest, "System.OSS.Steam.Test Cloud", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSteamCloudTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=steam test cloud")));

	return true;
}

/*

/////////////Currently crashing//////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNullCloudTest,"System.OSS.NULL.Test Cloud", EAutomationTestFlags::EditorContext )

bool FNullCloudTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=null test cloud")));

	return true;
}
*/

/*

/////////////Currently crashing//////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNullLeaderBoardTest,"System.OSS.NULL.Test Leaderboards", EAutomationTestFlags::EditorContext)

bool FNullLeaderBoardTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=null test leaderboards")));

	return true;
}
*/

#if PLATFORM_IOS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIosLeaderBoardTest, "System.OSS.IOS.Test Leaderboards", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FIosLeaderBoardTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=ios test leaderboards")));

	return true;
}

#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSteamLeaderBoardTest, "System.OSS.Steam.Test Leaderboards", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSteamLeaderBoardTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=steam test leaderboards")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPTimeTest, "System.OSS.MCP.Test Time", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPTimeTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test time")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPIdentityTest, "System.OSS.MCP.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test identity")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSteamIdentityTest, "System.OSS.Steam.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSteamIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=steam test identity")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAmazonIdentityTest, "System.OSS.Amazon.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAmazonIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=amazon test identity")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFacebookIdentityTest, "System.OSS.Facebook.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FFacebookIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=facebook test identity ")));

	return true;
}

#if PLATFORM_IOS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIOSIdentityTest, "System.OSS.IOS.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FIOSIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=ios test identity ")));

	return true;
}

#endif

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOculusIdentityTest, "System.OSS.Oculus.Test Identity", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FOculusIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=oculus test identity ")));

	return true;
}

/*

/////////////Currently crashing//////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNullIdentityTest,"System.OSS.NULL.Test Identity", EAutomationTestFlags::EditorContext )

bool FNullIdentityTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=null test identity ")));

	return true;
}
*/

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPEntitlementsTest, "System.OSS.MCP.Test Entitlements", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPEntitlementsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test entitlements")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMCPTitleFileTest, "System.OSS.MCP.Test Title File", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FMCPTitleFileTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=mcp test titlefile")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSteamAchievementsTest, "System.OSS.Steam.Test Achievements", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FSteamAchievementsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=steam test achievements")));

	return true;
}

/*

/////////////Currently crashing//////////////

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNullAchievementsTest,"System.OSS.NULL.Test Achievements", EAutomationTestFlags::EditorContext )

bool FNullAchievementsTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FEngineWaitLatentCommand(3.0));
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("online sub=null test achievements")));

	return true;
}
*/

#endif //WITH_DEV_AUTOMATION_TESTS

#endif
