// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "PieFixupTestObjects.generated.h"

USTRUCT()
struct FPieFixupStructWithSoftObjectPath
{
    GENERATED_BODY()

    UPROPERTY()
    FSoftObjectPath Path;

    UPROPERTY()
    TSoftObjectPtr<AActor> TypedPtr;
};

UCLASS()
class UPieFixupTestObject : public UObject
{
	GENERATED_BODY()

public:
    UPROPERTY()
    FSoftObjectPath Path;

    UPROPERTY()
    TSoftObjectPtr<AActor> TypedPtr;
    
    UPROPERTY()
    FPieFixupStructWithSoftObjectPath Struct;

    UPROPERTY()
    TArray<FPieFixupStructWithSoftObjectPath> Array;
};
