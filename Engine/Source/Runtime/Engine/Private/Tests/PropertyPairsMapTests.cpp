// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "PropertyPairsMap.h"

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.PropertyPairsMap"

namespace PropertyPairsMapTests
{
	constexpr const EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPropertyPairsMapTests, TEST_NAME_ROOT, TestFlags)
	bool FPropertyPairsMapTests::RunTest(const FString& Parameters)
	{
		FPropertyPairsMap PropertyPairsMap;

		TestFalse(TEXT("PropertyPairsMap.HasProperty(Invalid)"), PropertyPairsMap.HasProperty(TEXT("Invalid")));

		FName WithoutValueValue("Invalid");
		PropertyPairsMap.AddProperty(TEXT("WithoutValue"));
		TestTrue(TEXT("PropertyPairsMap.HasProperty(WithoutValue)"), PropertyPairsMap.HasProperty(TEXT("WithoutValue")));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithoutValue)"), PropertyPairsMap.GetProperty(TEXT("WithoutValue")));
		TestTrue(TEXT("WithoutValueValue is Invalid"), WithoutValueValue.ToString() == TEXT("Invalid"));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithVoutValue)"), PropertyPairsMap.GetProperty(TEXT("WithoutValue"), &WithoutValueValue));
		TestTrue(TEXT("WithoutValueValue is None"), WithoutValueValue.IsNone());

		FName WithValueValue("Invalid");
		PropertyPairsMap.AddProperty(TEXT("WithValue"), TEXT("Value1"));
		TestTrue(TEXT("PropertyPairsMap.HasProperty(WithValue)"), PropertyPairsMap.HasProperty(TEXT("WithValue")));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithValue)"), PropertyPairsMap.GetProperty(TEXT("WithValue")));
		TestTrue(TEXT("WithValueValue is Invalid"), WithValueValue.ToString() == TEXT("Invalid"));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithValue)"), PropertyPairsMap.GetProperty(TEXT("WithValue"), &WithValueValue));
		TestTrue(TEXT("WithValueValue is Value1"), WithValueValue.ToString() == TEXT("Value1"));
		PropertyPairsMap.AddProperty(TEXT("WithValue"), TEXT("Value2"));
		TestTrue(TEXT("PropertyPairsMap.HasProperty(WithValue)"), PropertyPairsMap.HasProperty(TEXT("WithValue")));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithValue)"), PropertyPairsMap.GetProperty(TEXT("WithValue")));
		TestTrue(TEXT("WithValueValue is Value1"), WithValueValue.ToString() == TEXT("Value1"));
		TestTrue(TEXT("PropertyPairsMap.GetProperty(WithValue)"), PropertyPairsMap.GetProperty(TEXT("WithValue"), &WithValueValue));
		TestTrue(TEXT("WithValueValue is Value2"), WithValueValue.ToString() == TEXT("Value2"));

		PropertyPairsMap.ForEachProperty([this](FName Name, FName Value)
		{
			TestTrue(TEXT("Unbreakable iterator"), (Name == TEXT("WithoutValue")) || (Name == TEXT("WithValue")));
			TestTrue(TEXT("Unbreakable iterator"), Value.IsNone() || (Value == TEXT("Value2")));
		});

		PropertyPairsMap.ForEachProperty([this](FName Name, FName Value)
		{
			TestTrue(TEXT("Breakable iterator"), (Name == TEXT("WithoutValue")) || (Name == TEXT("WithValue")));
			TestTrue(TEXT("Breakable iterator"), Value.IsNone() || (Value == TEXT("Value2")));
			return true;
		});

		return true;
	}
}

#undef TEST_NAME_ROOT

#endif 
