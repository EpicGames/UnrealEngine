// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "ReferenceCluster.h"

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.ReferenceCluster"

namespace ReferenceClusterTests
{
	constexpr const EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReferenceClusterTest, TEST_NAME_ROOT, TestFlags)

	TArray<TArray<FGuid>> GenerateObjectsClustersReference(const TArray<TPair<FGuid, TArray<FGuid>>>& InObjects)
	{
		TMap<FGuid, FGuid> ObjectToClusters;
		TMap<FGuid, TSet<FGuid>> Clusters;

		for (const auto& Object : InObjects)
		{
			const FGuid& ObjectGuid = Object.Key;
			FGuid ClusterGuid = ObjectToClusters.FindRef(ObjectGuid);

			if (!ClusterGuid.IsValid())
			{
				ClusterGuid = FGuid::NewGuid();
				TSet<FGuid>& Cluster = Clusters.Add(ClusterGuid);
				ObjectToClusters.Add(ObjectGuid, ClusterGuid);
				Cluster.Add(ObjectGuid);
			}

			for (const FGuid& ReferenceGuid : Object.Value)
			{
				FGuid ReferenceClusterGuid = ObjectToClusters.FindRef(ReferenceGuid);

				if (ReferenceClusterGuid.IsValid())
				{
					if (ReferenceClusterGuid != ClusterGuid)
					{
						TSet<FGuid> ReferenceCluster = Clusters.FindAndRemoveChecked(ReferenceClusterGuid);
						TSet<FGuid>& Cluster = Clusters.FindChecked(ClusterGuid);
						Cluster.Append(ReferenceCluster);
						for (const FGuid& OtherReferenceGuid : ReferenceCluster)
						{
							ObjectToClusters.FindChecked(OtherReferenceGuid) = ClusterGuid;
						}
					}
				}
				else
				{
					TSet<FGuid>& Cluster = Clusters.FindChecked(ClusterGuid);
					Cluster.Add(ReferenceGuid);
				}

				ObjectToClusters.Add(ReferenceGuid, ClusterGuid);
			}
		}
	
		TArray<TArray<FGuid>> Result;
		Result.Reserve(Clusters.Num());

		for (const auto& Cluster : Clusters)
		{
			Result.AddDefaulted_GetRef() = Cluster.Value.Array();
		}

		return MoveTemp(Result);
	}

	bool FReferenceClusterTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		TArray<TPair<FGuid, TArray<FGuid>>> ObjectsWithRefs;
		ObjectsWithRefs.AddDefaulted(10);

		for (auto& ObjectWithRefs : ObjectsWithRefs)
		{
			ObjectWithRefs.Key = FGuid::NewGuid();
		}

		// Basic test
		{
			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ObjectsWithRefs);

			TestTrue(TEXT("Reference Cluster Basic"), Clusters.Num() == 10);
			for (const TArray<FGuid>& Cluster : Clusters)
			{
				TestTrue(TEXT("Reference Cluster Basic"), Cluster.Num() == 1);
			}
		}

		//
		// This forms 6 clusters:
		// 
		// 0 --> 1     4     6     7     8     9
		// |     ^     |
		// v     |     v
		// 2 --> 3     5
		ObjectsWithRefs[0].Value.Add(ObjectsWithRefs[1].Key);
		ObjectsWithRefs[0].Value.Add(ObjectsWithRefs[2].Key);
		ObjectsWithRefs[2].Value.Add(ObjectsWithRefs[3].Key);
		ObjectsWithRefs[3].Value.Add(ObjectsWithRefs[1].Key);
		ObjectsWithRefs[4].Value.Add(ObjectsWithRefs[5].Key);

		// Simple clustering test
		{
			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ObjectsWithRefs);
			TestTrue(TEXT("Reference Cluster Simple"), Clusters.Num() == 6);
		}

		//
		// This forms 5 clusters:
		// 
		// 0 --> 1     4     6     7     8     9
		// |     ^     |
		// v     |     v
		// 2 --> 3 <-- 5
		ObjectsWithRefs[5].Value.Add(ObjectsWithRefs[3].Key);

		// Simple clusters merging test
		{
			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ObjectsWithRefs);
			TestTrue(TEXT("Reference Cluster Simple"), Clusters.Num() == 5);
		}

		//
		// This forms 2 clusters:
		// 
		//                   /-----.-----------\
		//                   |     |           |
		//                   v     v           |
		// 0 --> 1     4     6     7     8 <-- 9
		// |     ^     |                  
		// v     |     v                  
		// 2 --> 3 <-- 5                  
		ObjectsWithRefs[9].Value.Add(ObjectsWithRefs[8].Key);
		ObjectsWithRefs[9].Value.Add(ObjectsWithRefs[7].Key);
		ObjectsWithRefs[9].Value.Add(ObjectsWithRefs[6].Key);

		// Double clusters test
		{
			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ObjectsWithRefs);
			TestTrue(TEXT("Reference Cluster Double"), Clusters.Num() == 2);
		}

		//
		// This forms a single cluster:
		// 
		//                   /-----.-----------\
		//                   |     |           |
		//                   v     v           |
		// 0 --> 1     4     6     7     8 <-- 9
		// |     ^     |                 |
		// v     |     v                 |
		// 2 --> 3 <-- 5 <---------------/
		ObjectsWithRefs[8].Value.Add(ObjectsWithRefs[5].Key);

		// Complex clusters merging test
		{
			TArray<TArray<FGuid>> Clusters = GenerateObjectsClusters(ObjectsWithRefs);

			TestTrue(TEXT("Reference Cluster Complex Merging"), Clusters.Num() == 1);
			TestTrue(TEXT("Reference Cluster Complex Merging"), Clusters[0].Num() == 10);
		}

		// Random clusters test
		{
			const int32 NumObjects = 25000;

			TArray<FGuid> Objects;
			Objects.Reserve(NumObjects);
			for (int32 i=0; i<NumObjects; i++)
			{
				Objects.Emplace(FGuid::NewGuid());
			}

			ObjectsWithRefs.Empty(Objects.Num());
			for (int32 i=0; i<Objects.Num(); i++)
			{
				TPair<FGuid, TArray<FGuid>>& Pair = ObjectsWithRefs.AddDefaulted_GetRef();
				Pair.Key = Objects[i];

				const int32 NumRefs = FMath::RandRange(-10, 10);

				if (NumRefs > 0)
				{
					Pair.Value.Reserve(NumRefs);
					for (int32 j=0; j<NumRefs; j++)
					{
						Pair.Value.Add(Objects[FMath::RandHelper(Objects.Num())]);
					}
				}
			}

			const double StartTimeRef = FPlatformTime::Seconds();
			TArray<TArray<FGuid>> ClustersRef = GenerateObjectsClustersReference(ObjectsWithRefs);
			const double RunTimeRef = FPlatformTime::Seconds() - StartTimeRef;
			AddInfo(FString::Printf(TEXT("Clustered %d objects into %d clusters in %s (reference)"), ObjectsWithRefs.Num(), ClustersRef.Num(), *FPlatformTime::PrettyTime(RunTimeRef)));

			const double StartTimeCur = FPlatformTime::Seconds();
			TArray<TArray<FGuid>> ClustersCur = GenerateObjectsClusters(ObjectsWithRefs);
			const double RunTimeCur = FPlatformTime::Seconds() - StartTimeCur;
			AddInfo(FString::Printf(TEXT("Clustered %d objects into %d clusters in %s (current)"), ObjectsWithRefs.Num(), ClustersCur.Num(), *FPlatformTime::PrettyTime(RunTimeCur)));

			auto SortInnerClusters = [](const TArray<FGuid>& A, const TArray<FGuid>& B)
			{
				const FGuid Guid0;
				const FGuid& GuidA = A.Num() ? A[0] : Guid0;
				const FGuid& GuidB = B.Num() ? B[0] : Guid0;
				return GuidA < GuidB;
			};

			for (TArray<FGuid>& Cluster : ClustersRef) { Cluster.Sort(); }
			ClustersRef.Sort(SortInnerClusters);

			for (TArray<FGuid>& Cluster : ClustersCur) { Cluster.Sort(); }
			ClustersCur.Sort(SortInnerClusters);

			TestTrue(TEXT("Clustering against reference algorithm"), ClustersRef == ClustersCur);
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif 