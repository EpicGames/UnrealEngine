// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSAMTestTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SSAMTestTypes)

#if WITH_DEV_AUTOMATION_TESTS

#include "ContentStreaming.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Streaming/TextureInstanceView.h"

// ---------------------------------------------------------------------------
// USSAMTestStreamableAsset
// ---------------------------------------------------------------------------

void USSAMTestStreamableAsset::BeginDestroy()
{
	if (FSimpleStreamableAssetManager::IsAssetRegistered(GetSSAMAssetHandle()))
	{
		FSimpleStreamableAssetManager::UnregisterAsset(this);
	}
	// Prevent UnlinkStreaming from calling into the real FRenderAssetStreamingManager
	// (our test assets were never registered there via LinkStreaming).
	StreamingIndex = INDEX_NONE;
	Super::BeginDestroy();
}

// ---------------------------------------------------------------------------
// USSAMTestComponent
// ---------------------------------------------------------------------------

FPrimitiveSceneProxy* USSAMTestComponent::CreateSceneProxy()
{
	if (bUseFallbackProxy)
	{
		return new FSSAMTestFallbackSceneProxy(this);
	}

	TArray<FStreamingRenderAssetPrimitiveInfo> AssetInfos;
	AssetInfos.Reserve(TestAssets.Num());

	for (const FAssetEntry& Entry : TestAssets)
	{
		if (Entry.Asset)
		{
			FStreamingRenderAssetPrimitiveInfo Info(
				Entry.Asset,
				TestBounds,
				Entry.TexelFactor,
				PackedRelativeBox_Identity,
				false,	// bAllowInvalidTexelFactorWhenUnregistered
				Entry.bAffectedByComponentScale
			);
			AssetInfos.Add(MoveTemp(Info));
		}
	}

	return new FSSAMTestSceneProxy(this, MoveTemp(AssetInfos), bSupportScaleFactor);
}

void USSAMTestComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	if (bUseFallbackProxy)
	{
		for (const FAssetEntry& Entry : TestAssets)
		{
			if (Entry.Asset)
			{
				OutStreamingRenderAssets.Emplace(
					Entry.Asset,
					GetBounds(),
					Entry.TexelFactor,
					PackedRelativeBox_Identity,
					false,	// bAllowInvalidTexelFactorWhenUnregistered
					Entry.bAffectedByComponentScale
				);
			}
		}
		return;
	}

	Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
}

// ---------------------------------------------------------------------------
// FSSAMTestProxyDesc
// ---------------------------------------------------------------------------

FSSAMTestProxyDesc::FSSAMTestProxyDesc()
{
	CustomPrimitiveData = &TestCustomPrimitiveData;
	Mobility = EComponentMobility::Movable;
	bIsVisible = true;
}

// ---------------------------------------------------------------------------
// FSSAMTestSceneProxy
// ---------------------------------------------------------------------------

SIZE_T FSSAMTestSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FSSAMTestSceneProxy::FSSAMTestSceneProxy(const UPrimitiveComponent* InComponent, TArray<FStreamingRenderAssetPrimitiveInfo>&& InAssets, bool bInSupportScaleFactor)
	: FPrimitiveSceneProxy(InComponent)
	, CapturedAssets(MoveTemp(InAssets))
	, bSupportScaleFactor(bInSupportScaleFactor)
{
	bImplementsStreamableAssetGathering = true;
}

FSSAMTestSceneProxy::FSSAMTestSceneProxy(const FPrimitiveSceneProxyDesc* InDesc, TArray<FStreamingRenderAssetPrimitiveInfo>&& InAssets, bool bInSupportScaleFactor)
	: FPrimitiveSceneProxy(*InDesc)
	, CapturedAssets(MoveTemp(InAssets))
	, bSupportScaleFactor(bInSupportScaleFactor)
{
	bImplementsStreamableAssetGathering = true;
}

bool FSSAMTestSceneProxy::CanApplyStreamableRenderAssetScaleFactor() const
{
	return bSupportScaleFactor;
}

void FSSAMTestSceneProxy::GetStreamableRenderAssetInfo(const FBoxSphereBounds& PrimitiveBounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamableRenderAssets) const
{
	OutStreamableRenderAssets = CapturedAssets;
}

FPrimitiveViewRelevance FSSAMTestSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Relevance;
	Relevance.bDrawRelevance = false;
	return Relevance;
}

// ---------------------------------------------------------------------------
// FSSAMTestFallbackSceneProxy
// ---------------------------------------------------------------------------

SIZE_T FSSAMTestFallbackSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FSSAMTestFallbackSceneProxy::FSSAMTestFallbackSceneProxy(const UPrimitiveComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
{
}

FPrimitiveViewRelevance FSSAMTestFallbackSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Relevance;
	Relevance.bDrawRelevance = false;
	return Relevance;
}

// ---------------------------------------------------------------------------
// FSSAMTestEnvironment
// ---------------------------------------------------------------------------

FSSAMTestEnvironment::FSSAMTestEnvironment()
	: bDidInit(false)
	, bWasEnabled(FSimpleStreamableAssetManager::IsEnabled())
{
	if (!bWasEnabled)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("s.StreamableAssets.UseSimpleStreamableAssetManager"));
		if (CVar)
		{
			CVar->Set(1, ECVF_SetByCode);
		}
	}

	if (!FSimpleStreamableAssetManager::GetCriticalSection())
	{
		FSimpleStreamableAssetManager::Init();
		bDidInit = true;
	}

	WorldWrapper.CreateTestWorld(EWorldType::Game);
	WorldWrapper.BeginPlayInTestWorld();
}

FSSAMTestEnvironment::~FSSAMTestEnvironment()
{
	CleanupAssets();
	WorldWrapper.DestroyTestWorld(true);

	if (bDidInit)
	{
		FSimpleStreamableAssetManager::Shutdown();
	}

	if (!bWasEnabled)
	{
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("s.StreamableAssets.UseSimpleStreamableAssetManager"));
		if (CVar)
		{
			CVar->Set(0, ECVF_SetByCode);
		}
	}
}

USSAMTestStreamableAsset* FSSAMTestEnvironment::CreateTestAsset()
{
	USSAMTestStreamableAsset* Asset = NewObject<USSAMTestStreamableAsset>();
	Asset->SetStreamingIndexForTest(0);
	Asset->AddToRoot();
	CreatedAssets.Add(Asset);
	return Asset;
}

USSAMTestComponent* FSSAMTestEnvironment::SpawnTestComponent(const TArray<FSSAMTestAssetEntry>& Assets, const FBoxSphereBounds& Bounds, const FVector& Location)
{
	UWorld* World = GetWorld();
	check(World);

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
	check(Actor);

	USSAMTestComponent* Comp = NewObject<USSAMTestComponent>(Actor);
	Comp->TestAssets = Assets;
	Comp->TestBounds = Bounds;
	if (Actor->GetRootComponent())
	{
		Comp->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	}
	Actor->SetActorLocation(Location);
	Comp->RegisterComponent();

	return Comp;
}

USSAMTestComponent* FSSAMTestEnvironment::SpawnTestFallbackComponent(const TArray<FSSAMTestAssetEntry>& Assets, const FBoxSphereBounds& Bounds, const FVector& Location)
{
	UWorld* World = GetWorld();
	check(World);

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
	check(Actor);

	USSAMTestComponent* Comp = NewObject<USSAMTestComponent>(Actor);
	Comp->TestAssets = Assets;
	Comp->TestBounds = Bounds;
	Comp->bUseFallbackProxy = true;
	if (Actor->GetRootComponent())
	{
		Comp->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	}
	Actor->SetActorLocation(Location);
	Comp->RegisterComponent();

	return Comp;
}

void FSSAMTestEnvironment::CleanupAssets()
{
	for (USSAMTestStreamableAsset* Asset : CreatedAssets)
	{
		if (Asset)
		{
			Asset->SetStreamingIndexForTest(INDEX_NONE);
			Asset->RemoveFromRoot();
		}
	}
	CreatedAssets.Empty();
}

void FSSAMTestEnvironment::ReleaseAssetsForGC(TArrayView<USSAMTestStreamableAsset*> Assets)
{
	for (USSAMTestStreamableAsset* Asset : Assets)
	{
		CreatedAssets.Remove(Asset);
		Asset->RemoveFromRoot();
	}
}

void FSSAMTestEnvironment::UpdateBoundSizes(const FVector& ViewOrigin, float ScreenSize, float BoostFactor, float LastUpdateTime)
{
	UpdateBoundSizes(MakeArrayView(&ViewOrigin, 1), ScreenSize, BoostFactor, LastUpdateTime);
}

void FSSAMTestEnvironment::UpdateBoundSizes(TArrayView<const FVector> ViewOrigins, float ScreenSize, float BoostFactor, float LastUpdateTime)
{
	TArray<FStreamingViewInfo> ViewInfos;
	FStreamingViewInfoExtraArray ViewInfoExtras;
	ViewInfos.Reserve(ViewOrigins.Num());
	ViewInfoExtras.Reserve(ViewOrigins.Num());

	for (const FVector& Origin : ViewOrigins)
	{
		ViewInfos.Emplace(Origin, ScreenSize, ScreenSize, BoostFactor, false, 0.f, TWeakObjectPtr<AActor>(), TWeakObjectPtr<UWorld>());
		FStreamingViewInfoExtra Extra;
		Extra.ScreenSizeFloat = ScreenSize;
		Extra.ExtraBoostForVisiblePrimitiveFloat = BoostFactor;
		ViewInfoExtras.Add(Extra);
	}

	FRenderAssetStreamingSettings Settings;
	FSimpleStreamableAssetManager::UpdateBoundSizes(ViewInfos, ViewInfoExtras, LastUpdateTime, Settings);
}

FSSAMTestEnvironment::FScreenSizeResult FSSAMTestEnvironment::QueryScreenSize(FSSAMAssetHandle Handle, EStreamableRenderAssetType AssetType)
{
	FScreenSizeResult Result;
	FSimpleStreamableAssetManager::GetRenderAssetScreenSize(AssetType, Handle, Result.MaxSize, Result.MaxSizeVisibleOnly, Result.MaxNumForcedLODs, FLT_MAX, MAX_TEXTURE_MIP_COUNT, TEXT("Test"));
	return Result;
}

#else

void USSAMTestStreamableAsset::BeginDestroy()
{
	Super::BeginDestroy();
}

FPrimitiveSceneProxy* USSAMTestComponent::CreateSceneProxy()
{
	return nullptr;
}

void USSAMTestComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	Super::GetStreamingRenderAssetInfo(LevelContext, OutStreamingRenderAssets);
}

#endif // WITH_DEV_AUTOMATION_TESTS
