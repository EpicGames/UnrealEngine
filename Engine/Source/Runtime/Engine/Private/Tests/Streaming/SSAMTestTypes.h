// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StreamableRenderAsset.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/PrimitiveComponent.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneProxyDesc.h"
#include "PrimitiveSceneDesc.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "Tests/AutomationCommon.h"
#if USING_INSTRUMENTATION
#include "Sanitizer/RaceDetector.h"
#endif
#endif // WITH_DEV_AUTOMATION_TESTS

#include "SSAMTestTypes.generated.h"

// ---------------------------------------------------------------------------
// USSAMTestStreamableAsset
//
// A minimal UStreamableRenderAsset subclass that SSAM treats as a real
// streamable texture. No GPU resources, no render data — just the shared
// index pointer that SSAM needs for tracking.
//
// Note: UCLASS cannot be inside #if WITH_DEV_AUTOMATION_TESTS (UHT restriction).
// The class is harmless in non-test builds — it's Transient and never instantiated.
// ---------------------------------------------------------------------------
UCLASS(Transient)
class USSAMTestStreamableAsset : public UStreamableRenderAsset
{
	GENERATED_BODY()

public:
	virtual EStreamableRenderAssetType GetRenderAssetType() const override
	{
		return EStreamableRenderAssetType::Texture;
	}

	// StreamingIndex is protected in the base class. Setting it to a
	// non-INDEX_NONE value makes IsStreamable() return true, without
	// calling LinkStreaming() (which would register with the real
	// streaming manager and cause side effects).
	void SetStreamingIndexForTest(int32 InIndex)
	{
		StreamingIndex = InIndex;
	}

	// Override BeginDestroy to route asset cleanup through SSAM's UnregisterAsset
	// directly, bypassing UnlinkStreaming which calls into the real
	// FRenderAssetStreamingManager (our test assets were never registered there).
	virtual void BeginDestroy() override;

	virtual int32 CalcCumulativeLODSize(int32 NumLODs) const override
	{
		// Return a fake size: 1KB per mip, doubling each level.
		int32 Size = 0;
		for (int32 i = 0; i < NumLODs; ++i)
		{
			Size += 1024 << i;
		}
		return Size;
	}

	// Set up minimal CachedSRRState so the streaming manager can create a
	// valid FStreamingRenderAsset for this test asset (needs non-zero mip counts).
	void SetupFakeStreamingState(int32 NumMips = 10, int32 NumNonStreaming = 1)
	{
		CachedSRRState.bSupportsStreaming = true;
		CachedSRRState.MaxNumLODs = NumMips;
		CachedSRRState.NumNonStreamingLODs = NumNonStreaming;
		CachedSRRState.NumNonOptionalLODs = NumMips;
		CachedSRRState.NumResidentLODs = NumMips;
		CachedSRRState.NumRequestedLODs = NumMips;
		CachedSRRState.AssetLODBias = 0;
	}
};

// ---------------------------------------------------------------------------
// USSAMTestComponent
//
// Primitive component that creates an FSSAMTestSceneProxy (defined below,
// guarded by WITH_DEV_AUTOMATION_TESTS). Configure TestAssets and TestBounds
// before registering.
// ---------------------------------------------------------------------------
UCLASS(Transient)
class USSAMTestComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	struct FAssetEntry
	{
		UStreamableRenderAsset* Asset = nullptr;
		float TexelFactor = 1.0f;
		bool bAffectedByComponentScale = true;
	};

	TArray<FAssetEntry> TestAssets;
	FBoxSphereBounds TestBounds = FBoxSphereBounds(FVector::ZeroVector, FVector(100.f), 100.f);
	bool bUseFallbackProxy = false;
	bool bSupportScaleFactor = false;

	void AddTestAsset(USSAMTestStreamableAsset* Asset, float TexelFactor = 1.0f, bool bAffectedByComponentScale = true)
	{
		TestAssets.Add(FAssetEntry{ Asset, TexelFactor, bAffectedByComponentScale });
	}

	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override
	{
		return TestBounds.TransformBy(LocalToWorld);
	}

	// Mirrors the proxy-side CanApplyStreamableRenderAssetScaleFactor opt-in: when
	// bSupportScaleFactor is true, the fallback FRegister picks up the transform's
	// max axis scale through this knob, matching the primary path's behavior.
	virtual float GetStreamingScale() const override
	{
		return bSupportScaleFactor ? GetComponentTransform().GetMaximumAxisScale() : 1.f;
	}

	// Override the streaming asset gathering used by the FALLBACK path.
	// When using the fallback proxy, ProcessFallbackRegistrations_GameThread
	// calls FStreamingTextureLevelContext::GetStreamingRenderAssetInfoWithNULLRemoval
	// which calls Component->GetStreamingRenderAssetInfo. This override
	// returns our test assets directly, bypassing the material system.
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
};

// ===========================================================================
// Everything below requires the test framework and is stripped from shipping.
// ===========================================================================
#if WITH_DEV_AUTOMATION_TESTS

// Alias for convenience in test code
using FSSAMTestAssetEntry = USSAMTestComponent::FAssetEntry;

// ---------------------------------------------------------------------------
// FSSAMTestProxyDesc
//
// Minimal FPrimitiveSceneProxyDesc for creating proxies without a UPrimitiveComponent.
// Overrides GetUsedMaterials to avoid the check(Component) in the base class,
// which is called unconditionally in editor builds from FPrimitiveSceneProxy's constructor.
// ---------------------------------------------------------------------------
struct FSSAMTestProxyDesc : public FPrimitiveSceneProxyDesc
{
	FCustomPrimitiveData TestCustomPrimitiveData;

	FSSAMTestProxyDesc();

	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override
	{
	}
};

// ---------------------------------------------------------------------------
// FSSAMTestSceneProxy
//
// Scene proxy that implements GetStreamableRenderAssetInfo, taking the
// PRIMARY SSAM registration path (not the fallback component path).
// The asset list is captured from the component at construction time.
// ---------------------------------------------------------------------------
class FSSAMTestSceneProxy : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;
	uint32 GetMemoryFootprint() const override { return sizeof(*this); }

	FSSAMTestSceneProxy(const UPrimitiveComponent* InComponent, TArray<FStreamingRenderAssetPrimitiveInfo>&& InAssets, bool bInSupportScaleFactor = false);
	FSSAMTestSceneProxy(const FPrimitiveSceneProxyDesc* InDesc, TArray<FStreamingRenderAssetPrimitiveInfo>&& InAssets, bool bInSupportScaleFactor = false);

	virtual bool CanApplyStreamableRenderAssetScaleFactor() const override;

	virtual void GetStreamableRenderAssetInfo(const FBoxSphereBounds& PrimitiveBounds, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamableRenderAssets) const override;

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

private:
	TArray<FStreamingRenderAssetPrimitiveInfo> CapturedAssets;
	bool bSupportScaleFactor = false;
};

// ---------------------------------------------------------------------------
// FSSAMTestFallbackSceneProxy
//
// Scene proxy that does NOT implement GetStreamableRenderAssetInfo.
// bImplementsStreamableAssetGathering stays false (default), forcing
// Register to take the FALLBACK path through
// ProcessFallbackRegistrations_GameThread.
// ---------------------------------------------------------------------------
class FSSAMTestFallbackSceneProxy : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;
	uint32 GetMemoryFootprint() const override { return sizeof(*this); }

	FSSAMTestFallbackSceneProxy(const UPrimitiveComponent* InComponent);

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
};

// ---------------------------------------------------------------------------
// FSSAMTestEnvironment
//
// RAII helper that enables SSAM for the duration of a test. Handles:
// - Setting the CVar (GUseSimpleStreamableAssetManager)
// - Initializing / shutting down the SSAM singleton if needed
// - Creating and managing a test world via FTestWorldWrapper
// ---------------------------------------------------------------------------
class FSSAMTestEnvironment
{
#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::FRaceDetectorScope RaceDetectorScope;
#endif

public:
	FSSAMTestEnvironment();
	~FSSAMTestEnvironment();

	UWorld* GetWorld() const { return WorldWrapper.GetTestWorld(); }

	void Tick(float DeltaTime = 0.01f)
	{
		WorldWrapper.TickTestWorld(DeltaTime);
	}

	void ProcessSSAM()
	{
		FSimpleStreamableAssetManager::Process();
	}

	// Convenience: create a test asset that SSAM will treat as streamable
	USSAMTestStreamableAsset* CreateTestAsset();

	// Convenience: spawn actor, attach a configured test component, register it.
	// Component registration triggers the full engine flow:
	// RegisterComponent -> CreateSceneProxy -> AddPrimitive -> BatchAddPrimitivesInternal -> Register
	USSAMTestComponent* SpawnTestComponent(const TArray<FSSAMTestAssetEntry>& Assets, const FBoxSphereBounds& Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector(100.f), 100.f), const FVector& Location = FVector::ZeroVector);

	// Spawn a component that uses the FALLBACK proxy (bImplementsStreamableAssetGathering = false).
	// Registration goes through Register's fallback path -> FallbackComponentRegistration queue.
	// ProcessFallbackRegistrations_GameThread() must be called to complete registration.
	USSAMTestComponent* SpawnTestFallbackComponent(const TArray<FSSAMTestAssetEntry>& Assets, const FBoxSphereBounds& Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector(100.f), 100.f), const FVector& Location = FVector::ZeroVector);

	void CleanupAssets();

	// Drain the GT-only fallback queue (proxies without GetStreamableRenderAssetInfo).
	void ProcessFallbackRegistrations()
	{
		FSimpleStreamableAssetManager::ProcessFallbackRegistrations_GameThread();
	}

	// Release assets so GC can collect them through the normal BeginDestroy →
	// UnlinkStreaming → UnregisterAsset path. Unlike CleanupAssets(), this does
	// NOT set StreamingIndex = INDEX_NONE, preserving the real teardown flow.
	// Released assets are removed from CreatedAssets so the destructor skips them.
	void ReleaseAssetsForGC(TArrayView<USSAMTestStreamableAsset*> Assets);

	// Streaming query helpers — eliminate repeated view setup boilerplate.
	struct FScreenSizeResult
	{
		float MaxSize = 0.f;
		float MaxSizeVisibleOnly = 0.f;
		int32 MaxNumForcedLODs = 0;
	};

	void UpdateBoundSizes(const FVector& ViewOrigin, float ScreenSize = 1920.f, float BoostFactor = 1.f, float LastUpdateTime = 0.f);
	void UpdateBoundSizes(TArrayView<const FVector> ViewOrigins, float ScreenSize = 1920.f, float BoostFactor = 1.f, float LastUpdateTime = 0.f);
	FScreenSizeResult QueryScreenSize(FSSAMAssetHandle Handle, EStreamableRenderAssetType AssetType = EStreamableRenderAssetType::Texture);

private:
	FTestWorldWrapper WorldWrapper;
	TArray<USSAMTestStreamableAsset*> CreatedAssets;
	bool bDidInit;
	bool bWasEnabled;
};

#endif // WITH_DEV_AUTOMATION_TESTS
