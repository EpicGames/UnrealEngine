// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TextImportTest.h"
#include "Tests/TestHarnessAdapter.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextImportTest)

#if WITH_TESTS

TEST_CASE_NAMED(FTextImportStructPropertyTest, "System::Engine::TextImport::StructProperty", "[Engine][TextImport][StructProperty]")
{
	struct FTextImportTestCase
	{
		bool bShouldError = false;
		FTextImportTestStruct TargetOutput = FTextImportTestStruct();
		FStringView TextInput = FStringView();
	};

	FTextImportTestCase TestCases[] =
	{
		{ false, { ETextImportTestFlags::FlagA, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=\"ETextImportTestFlags::FlagA\",TestInt=2, TestString=\"String,With,Commas\")")},
		{ false, { ETextImportTestFlags::FlagA, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=\"FlagA \",TestInt=2, TestString=\"String,With,Commas\")")},
		{ false, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=\"ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB\",TestInt=2, TestString=\"String,With,Commas\")")},
		{ false, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=\"FlagA | FlagB\",TestInt=2, TestString=\"String,With,Commas\")")},
		{ false, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB,TestInt=2, TestString=\"String,With,Commas\")")},
		{ false, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=FlagA | FlagB,TestInt=2, TestString=\"String,With,Commas\")")},
		{ true, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=\"FlagA | FlagB,TestInt=2, TestString=\"String,With,Commas\")")}, //Missing end quote
		{ true, { ETextImportTestFlags::FlagA | ETextImportTestFlags::FlagB, 2, "String,With,Commas" }, TEXTVIEW("(EmbeddedFlags=FlagA | FlagB\",TestInt=2, TestString=\"String,With,Commas\")")}, //Missing start quote
		{ true, { ETextImportTestFlags::TestStructDefault, 1, "" }, TEXTVIEW("(EmbeddedFlags=\"\",TestInt=\"\", TestString=\"\")")}, //Empty Quotes - Should give invalid enum error (same behaviour as for non-struct enums)
		{ false, { ETextImportTestFlags::TestStructDefault, 1, "DefaultString" }, TEXTVIEW("(EmbeddedFlags=,TestInt=, TestString=)")}, //Empty No-Quotes - Returns default Struct 

		{ false, { ETextImportTestFlags::TestStructDefault, 1, "DefaultString" }, TEXTVIEW("(WrongNameA=\"ETextImportTestFlags::FlagA\",WrongNameB=2, WrongNameC=\"String,With,Commas\")")},
		{ true, { ETextImportTestFlags::TestStructDefault, 1, "" }, TEXTVIEW("(EmbeddedFlags=(,TestInt=, TestString=)")}, //Extra Bracket - Should give invalid enum error
		{ true, { ETextImportTestFlags::TestStructDefault, 1, "" }, TEXTVIEW("(EmbeddedFlags=\",TestInt=, TestString=)")}, //Extra Quote - Should give Bad quoted string error
	};
	

	UTextImportContainer* Container = NewObject<UTextImportContainer>();
	check(Container);
	const FStructProperty* ObjProp = FindFProperty<FStructProperty>(Container->GetClass(), TEXT("ResultStruct"));
	check(ObjProp);
	check(ObjProp->Struct);

	int32 PortFlags = 0;

	for (const FTextImportTestCase& TestCase : TestCases)
	{
		//Reset struct to defaults
		Container->ResultStruct = FTextImportTestStruct();

		//Parse the text into the ResultStruct
		const TCHAR* InBuffer = TestCase.TextInput.GetData();
		FStringOutputDevice ImportError;
		ObjProp->Struct->ImportText(InBuffer, (void*)&Container->ResultStruct, Container, PortFlags, &ImportError, FString("FTextImportTestStruct"), true);

		const bool bImportErrored = ImportError.Len() > 0;
		if (!TestCase.bShouldError && bImportErrored)
		{
			FString ErrorString = FString::Printf(TEXT("Test case failed due to ImportError %s for Test Case: %s"), *ImportError, InBuffer);
			ADD_ERROR(ErrorString);
		}
		
		CHECK_MESSAGE(FString::Printf(TEXT("Import Error Mismatch. Result: %s Expected: %s for TestCase: %s"), bImportErrored ? TEXT("Error") : TEXT("NoError"), TestCase.bShouldError ? TEXT("Error") : TEXT("NoError"), InBuffer), TestCase.bShouldError == bImportErrored);
		if (!TestCase.bShouldError)
		{
			CHECK_MESSAGE(FString::Printf(TEXT("Import Results are Incorrect: %s Expected: %s for TestCase: %s"), *Container->ResultStruct.ToString(), *TestCase.TargetOutput.ToString(), InBuffer), TestCase.TargetOutput == Container->ResultStruct);
		}
	}
}

#endif //WITH_TESTS
