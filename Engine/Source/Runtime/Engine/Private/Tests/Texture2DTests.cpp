// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Engine/Texture2D.h"
#include "UObject/Package.h"
#include "UObject/ReferenceChainSearch.h"
#include "AssetCompilingManager.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace Texture2DTest
{

constexpr const EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

// A simple test to make sure that basic functionality in UTexture2D::CreateTransient works as it seems to be a 
// fairly uncommon code path in our samples/test games etc.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTexture2DTestTransient, "System.Engine.Texture2D.CreateTransient", TestFlags)
bool FTexture2DTestTransient::RunTest(const FString& Parameters)
{
	{
		// Each test in this scope is expected to give one warning about invalid parameters
		AddExpectedError(TEXT("Negative size specified for UTexture2D::CreateTransient()"), EAutomationExpectedErrorFlags::Contains, 3);

		UTexture2D* ZeroSizedTexture = UTexture2D::CreateTransient(0, 0);
		TestTrue(TEXT("Creating a transient texture with a zero length dimension should fail!"), ZeroSizedTexture == nullptr);

		UTexture2D* ZeroWidthTexture = UTexture2D::CreateTransient(0, 32);
		TestTrue(TEXT("Creating a transient texture with a zero length dimension should fail!"), ZeroWidthTexture == nullptr);

		UTexture2D* ZeroHeightTexture = UTexture2D::CreateTransient(32, 0);
		TestTrue(TEXT("Creating a transient texture with a zero length dimension should fail!"), ZeroHeightTexture == nullptr);
	}

	UTexture2D* TransientTexture = UTexture2D::CreateTransient(32, 32);
	TestTrue(TEXT("Failed to create a 32*32 transient texture!"), TransientTexture != nullptr);

	return true;
}

#if WITH_EDITORONLY_DATA

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTexture2DTestLockingWhenEmpty, "System.Engine.Texture2D.LockingWhenEmpty", TestFlags)
bool FTexture2DTestLockingWhenEmpty::RunTest(const FString& Parameters)
{
	// Create a texture with no valid dimensions and no data
	{
		UTexture2D* EmptyTexture = NewObject<UTexture2D>(GetTransientPackage());
		EmptyTexture->Source.Init2DWithMipChain(0, 0, ETextureSourceFormat::TSF_BGRA8);

		// Test that we can lock and unlock repeatedly
		uint8* FirstLockPtr = EmptyTexture->Source.LockMip(0);
		TestNull(TEXT("Locking an empty texture"), FirstLockPtr);

		// If LockMip returns null you do not get the lock, you do not need to unlock it
		uint8* SecondLockPtr = EmptyTexture->Source.LockMip(0);
		TestNull(TEXT("Locking an empty texture a second time"), SecondLockPtr);

		// If we fail to get the lock because the mip does not exist then we do not get
		// a lock and don't need to unlock it.
		uint8* InvalidLockPtr = EmptyTexture->Source.LockMip(1);
		TestNull(TEXT("Locking a submip of an empty texture"), InvalidLockPtr);

		FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadOnly,&EmptyTexture->Source,0);
		TestFalse( TEXT("MipLock on empty texture should not be valid"), MipLock.IsValid() );
	}

	// Create a texture with  valid dimensions and default data
	{
		UTexture2D* Texture = NewObject<UTexture2D>(GetTransientPackage());
		Texture->Source.Init2DWithMipChain(1024, 1024, ETextureSourceFormat::TSF_BGRA8);

		// Test that we can lock and unlock repeatedly
		uint8* FirstLockPtr = Texture->Source.LockMip(0);
		TestNotNull(TEXT("Locking a valid texture"), FirstLockPtr);
		Texture->Source.UnlockMip(0);

		uint8* SecondLockPtr = Texture->Source.LockMip(0);
		TestNotNull(TEXT("Locking a valid a second time"), SecondLockPtr);
		Texture->Source.UnlockMip(0);

		{
			FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadOnly,&Texture->Source,0);
			TestTrue( TEXT("MipLock on valid texture should be valid"), MipLock.IsValid() );
		}

		// Test that we can lock each mip before unlocking them all
		for (int32 MipIndex = 0; MipIndex < Texture->Source.GetNumMips(); ++MipIndex)
		{
			uint8* MipPtr = Texture->Source.LockMip(MipIndex);
			TestNotNull(TEXT("Locking a valid texture mip"), MipPtr);
		}

		for (int32 MipIndex = 0; MipIndex < Texture->Source.GetNumMips(); ++MipIndex)
		{
			Texture->Source.UnlockMip(MipIndex);
		}
		
		for (int32 MipIndex = 0; MipIndex < Texture->Source.GetNumMips(); ++MipIndex)
		{
			FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadOnly,&Texture->Source,MipIndex);
			TestTrue( TEXT("MipLock on valid texture should be valid"), MipLock.IsValid() );
			
			// does fail, but fails with a check, so we can't test it :
			//uint8* MipPtr = Texture->Source.LockMip(MipIndex);
			//TestNull(TEXT("Locking for write after readonly should fail"), MipPtr);
		}
	}

	return true;
}

#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTexture2DAsyncCompileCancelation, "System.Engine.Texture2D.AsyncCompileCancelation", TestFlags)
bool FTexture2DAsyncCompileCancelation::RunTest(const FString& Parameters)
{
	constexpr int32 TextureSize = 2 * 1024;
	constexpr int32 NumIteration = 10;

	auto DoTest =
		[this](bool bVirtualStreaming)
		{
			double CompilationTime = 0.0;

			for (int32 Iteration = 0; Iteration < NumIteration; ++Iteration)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::Iteration);

				UTexture2D* TestTexture = nullptr;
		
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::CreateTransient);
					TestTexture = UTexture2D::CreateTransient(TextureSize, TextureSize);
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::PreEditChange);
					TestTexture->PreEditChange(nullptr);
				}

				TestTexture->VirtualTextureStreaming = bVirtualStreaming;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::WriteSource);
					TestTexture->Source.Init(TextureSize, TextureSize, 1, 1, TSF_BGRA8);
					int32* Data = (int32*)TestTexture->Source.LockMip(0);

					ParallelFor(TextureSize * TextureSize, [&](int32 Index) { Data[Index] = FMath::Rand(); });

					TestTexture->Source.UnlockMip(0);
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::PostEditChange);
					TestTexture->PostEditChange();
				}
	
				// on first iteration, gather the compilation time
				if (Iteration == 0)
				{
					double StartTime = FPlatformTime::Seconds();
					FAssetCompilingManager::Get().FinishCompilationForObjects({TestTexture});
					CompilationTime = FPlatformTime::Seconds() - StartTime;
					continue;
				}

				TWeakObjectPtr<UTexture> WeakPtr(TestTexture);

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::RandomSleep);
					// On subsequent iterations, sleep a random amount of time that should span the compilation time
					// to test various cancellation points
					FPlatformProcess::Sleep(FMath::RandRange(0.0, CompilationTime));
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::GC);
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}
		
				TestFalse(TEXT("There shouldn't be anything preventing the texture from being GCed during async compilation"), WeakPtr.IsValid());
			}
		};

	{
		// Normal and VT textures have different cancelation points
		DoTest(false /*bVirtualStreaming*/);
		DoTest(true  /*bVirtualStreaming*/);
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTexture2DMissingPreEditChange, "System.Engine.Texture2D.MissingPreEditChange", TestFlags)
bool FTexture2DMissingPreEditChange::RunTest(const FString& Parameters)
{
	constexpr int32 TextureSize = 2 * 1024;
	constexpr int32 NumIteration = 3;

	auto DoTest =
		[this](bool bVirtualStreaming)
		{
			double CompilationTime = 0.0;

			for (int32 Iteration = 0; Iteration < NumIteration; ++Iteration)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DMissingPreEditChange::Iteration);

				UTexture2D* TestTexture = nullptr;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::CreateTransient);
					TestTexture = UTexture2D::CreateTransient(TextureSize, TextureSize);
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DMissingPreEditChange::PreEditChange);
					TestTexture->PreEditChange(nullptr);
				}

				TestTexture->VirtualTextureStreaming = bVirtualStreaming;

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DMissingPreEditChange::WriteSource);
					TestTexture->Source.Init(TextureSize, TextureSize, 1, 1, TSF_BGRA8);
					int32* Data = (int32*)TestTexture->Source.LockMip(0);

					ParallelFor(TextureSize * TextureSize, [&](int32 Index) { Data[Index] = FMath::Rand(); });

					TestTexture->Source.UnlockMip(0);
				}

				// This will start a build async
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DMissingPreEditChange::PostEditChange);
					TestTexture->PostEditChange();
				}
				
				// Overwrite source again without calling PreEditChange, this should log some errors
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DAsyncCompileCancelation::OverwriteWriteSource);
					TestTexture->Source.Init(TextureSize, TextureSize, 1, 1, TSF_BGRA8);
					int32* Data = (int32*)TestTexture->Source.LockMip(0);

					ParallelFor(TextureSize * TextureSize, [&](int32 Index) { Data[Index] = FMath::Rand(); });

					TestTexture->Source.UnlockMip(0);
				}

				FAssetCompilingManager::Get().FinishCompilationForObjects({ TestTexture });
				
				TWeakObjectPtr<UTexture> WeakPtr(TestTexture);

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FTexture2DMissingPreEditChange::GC);
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}

				TestFalse(TEXT("There shouldn't be anything preventing the texture from being GCed during async compilation"), WeakPtr.IsValid());
			}
		};

	AddExpectedMessage(TEXT("missing call to PreEditChange"), EAutomationExpectedErrorFlags::Contains, 0 /* Must be seen at least once for the test to pass*/);
	{
		// Normal and VT textures have different cancelation points
		DoTest(false /*bVirtualStreaming*/);
		DoTest(true  /*bVirtualStreaming*/);
	}

	return true;
}

#endif

} // namespace Texture2DTest

#endif //WITH_DEV_AUTOMATION_TESTS
