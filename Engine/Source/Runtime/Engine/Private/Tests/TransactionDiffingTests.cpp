// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/TransactionDiffingTests.h"
#include "Misc/AutomationTest.h"
#include "Misc/TransactionCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransactionDiffingTests)

void UTransactionDiffingTestObject::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	Record << SA_VALUE(TEXT("NonPropertyData"), NonPropertyData);
}

#if WITH_DEV_AUTOMATION_TESTS

namespace TransactionDiffingTests
{

constexpr const EAutomationTestFlags TestFlags = EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditPropertyDataTest, "System.Engine.Transactions.EditPropertyData", TestFlags)
bool FEditPropertyDataTest::RunTest(const FString& Parameters)
{
	const UTransactionDiffingTestObject* DefaultObject = GetDefault<UTransactionDiffingTestObject>();
	UTransactionDiffingTestObject* ModifiedObject = NewObject<UTransactionDiffingTestObject>();
	
	const UE::Transaction::FDiffableObject DefaultDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(DefaultObject);

	ModifiedObject->PropertyData = 10;

	const UE::Transaction::FDiffableObject ModifiedDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('PropertyData')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, PropertyData)));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditNonPropertyDataTest, "System.Engine.Transactions.EditNonPropertyData", TestFlags)
bool FEditNonPropertyDataTest::RunTest(const FString& Parameters)
{
	const UTransactionDiffingTestObject* DefaultObject = GetDefault<UTransactionDiffingTestObject>();
	UTransactionDiffingTestObject* ModifiedObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject DefaultDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(DefaultObject);

	ModifiedObject->NonPropertyData = 10;

	const UE::Transaction::FDiffableObject ModifiedDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject);

		TestTrue(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 0);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditNamesTest, "System.Engine.Transactions.EditNames", TestFlags)
bool FEditNamesTest::RunTest(const FString& Parameters)
{
	const UTransactionDiffingTestObject* DefaultObject = GetDefault<UTransactionDiffingTestObject>();
	UTransactionDiffingTestObject* ModifiedObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject DefaultDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(DefaultObject);

	ModifiedObject->AdditionalName = "Test0";

	const UE::Transaction::FDiffableObject ModifiedDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalName')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalName)));
	}

	ModifiedObject->NamesArray.Add("Test1");

	const UE::Transaction::FDiffableObject ModifiedDiffableObject2 = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 2);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalName')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalName)));
		TestTrue(TEXT("ChangedProperties.Contains('NamesArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, NamesArray)));
	}

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(ModifiedDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('NamesArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, NamesArray)));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditObjectsTest, "System.Engine.Transactions.EditObjects", TestFlags)
bool FEditObjectsTest::RunTest(const FString& Parameters)
{
	const UTransactionDiffingTestObject* DefaultObject = GetDefault<UTransactionDiffingTestObject>();
	UTransactionDiffingTestObject* ModifiedObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject DefaultDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(DefaultObject);

	ModifiedObject->AdditionalObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject ModifiedDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalObject')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalObject)));
	}

	ModifiedObject->ObjectsArray.Add(NewObject<UTransactionDiffingTestObject>());

	const UE::Transaction::FDiffableObject ModifiedDiffableObject2 = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 2);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalObject')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalObject)));
		TestTrue(TEXT("ChangedProperties.Contains('ObjectsArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, ObjectsArray)));
	}

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(ModifiedDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('ObjectsArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, ObjectsArray)));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditSoftObjectsTest, "System.Engine.Transactions.EditSoftObjects", TestFlags)
bool FEditSoftObjectsTest::RunTest(const FString& Parameters)
{
	const UTransactionDiffingTestObject* DefaultObject = GetDefault<UTransactionDiffingTestObject>();
	UTransactionDiffingTestObject* ModifiedObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject DefaultDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(DefaultObject);

	ModifiedObject->AdditionalSoftObject = NewObject<UTransactionDiffingTestObject>();

	const UE::Transaction::FDiffableObject ModifiedDiffableObject = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalSoftObject')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalSoftObject)));
	}

	ModifiedObject->SoftObjectsArray.Add(NewObject<UTransactionDiffingTestObject>());

	const UE::Transaction::FDiffableObject ModifiedDiffableObject2 = UE::Transaction::DiffUtil::GetDiffableObject(ModifiedObject);

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(DefaultDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 2);
		TestTrue(TEXT("ChangedProperties.Contains('AdditionalSoftObject')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, AdditionalSoftObject)));
		TestTrue(TEXT("ChangedProperties.Contains('SoftObjectsArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, SoftObjectsArray)));
	}

	{
		const FTransactionObjectDeltaChange DeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(ModifiedDiffableObject, ModifiedDiffableObject2);

		TestFalse(TEXT("bHasNonPropertyChanges"), DeltaChange.bHasNonPropertyChanges);
		TestEqual(TEXT("ChangedProperties.Num()"), DeltaChange.ChangedProperties.Num(), 1);
		TestTrue(TEXT("ChangedProperties.Contains('SoftObjectsArray')"), DeltaChange.ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(UTransactionDiffingTestObject, SoftObjectsArray)));
	}

	return true;
}

} // namespace TransactionDiffingTests

#endif //WITH_DEV_AUTOMATION_TESTS

