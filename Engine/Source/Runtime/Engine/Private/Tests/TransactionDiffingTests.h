// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "TransactionDiffingTests.generated.h"

UCLASS()
class UTransactionDiffingTestObject : public UObject
{
	GENERATED_BODY()

public:
	virtual void Serialize(FStructuredArchive::FRecord Record) override;

	UPROPERTY()
	TArray<FName> NamesArray;

	UPROPERTY()
	FName AdditionalName;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ObjectsArray;

	UPROPERTY()
	TObjectPtr<UObject> AdditionalObject;

	UPROPERTY()
	TArray<TSoftObjectPtr<UObject>> SoftObjectsArray;

	UPROPERTY()
	TSoftObjectPtr<UObject> AdditionalSoftObject;

	UPROPERTY()
	int32 PropertyData = 0;

	int32 NonPropertyData = 0;
};
