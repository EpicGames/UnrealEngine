// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Font.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Canvas::Private
{

static UFont* GetTestFont()
{
	if (!GEngine)
	{
		return nullptr;
	}
	if (UFont* Font = GEngine->GetTinyFont())
	{
		return Font;
	}
	if (UFont* Font = GEngine->GetSmallFont())
	{
		return Font;
	}
	if (UFont* Font = GEngine->GetMediumFont())
	{
		return Font;
	}
	return nullptr;
}

} // namespace UE::Canvas::Private

namespace UE::Canvas
{

TEST_CASE_NAMED(CanvasBasicMultilineWidth, "UE::Engine::Canvas::BasicMultilineWidth", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Multi-line width should equal widest individual line, not sum
	constexpr FStringView SingleLine = TEXTVIEW("Hello");
	constexpr FStringView MultiLine = TEXTVIEW("Hello\nWorld\nTest");

	UCanvas::CanvasStringSize(Params, SingleLine);
	const float HelloWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("World"));
	const float WorldWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("Test"));
	const float TestWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, MultiLine);
	const float MultiLineWidth = Params.DrawXL;

	const float ExpectedWidth = FMath::Max(HelloWidth, WorldWidth, TestWidth);

	REQUIRE_MESSAGE(TEXT("Multi-line width should equal widest individual line"),
		FMath::IsNearlyEqual(MultiLineWidth, ExpectedWidth, 0.1f));
}

TEST_CASE_NAMED(CanvasSingleVsMultilineHeight, "UE::Engine::Canvas::SingleVsMultilineHeight", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Single line height
	UCanvas::CanvasStringSize(Params, TEXTVIEW("Hello"));
	const float SingleLineHeight = Params.DrawYL;

	// Two line height should be approximately single + max font height
	UCanvas::CanvasStringSize(Params, TEXTVIEW("Hello\nWorld"));
	const float TwoLineHeight = Params.DrawYL;

	const float MaxCharHeight = TestFont->GetMaxCharHeight();
	const float ExpectedTwoLineHeight = MaxCharHeight + SingleLineHeight;

	REQUIRE_MESSAGE(TEXT("Two-line height should be single line height plus font max height"),
		FMath::IsNearlyEqual(TwoLineHeight, ExpectedTwoLineHeight, MaxCharHeight * 0.1f));
}

TEST_CASE_NAMED(CanvasEmptyLines, "UE::Engine::Canvas::EmptyLines", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Empty lines should still contribute to height
	UCanvas::CanvasStringSize(Params, TEXTVIEW("Hello\n\nWorld"));
	const float ThreeLineHeight = Params.DrawYL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("Hello\nWorld"));
	const float TwoLineHeight = Params.DrawYL;

	const float MaxCharHeight = TestFont->GetMaxCharHeight();

	REQUIRE_MESSAGE(TEXT("Empty line should add one more line advance"),
		FMath::IsNearlyEqual(ThreeLineHeight, TwoLineHeight + MaxCharHeight, MaxCharHeight * 0.1f));
}

TEST_CASE_NAMED(CanvasLineWidthVariation, "UE::Engine::Canvas::LineWidthVariation", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Lines with very different widths
	constexpr FStringView VaryingWidths = TEXTVIEW("A\nThis is a much longer line\nB");

	UCanvas::CanvasStringSize(Params, VaryingWidths);
	const float MultiLineWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("This is a much longer line"));
	const float LongestLineWidth = Params.DrawXL;

	REQUIRE_MESSAGE(TEXT("Multi-line width should match the longest individual line"),
		FMath::IsNearlyEqual(MultiLineWidth, LongestLineWidth, 0.1f));
}

TEST_CASE_NAMED(CanvasStoppingOffsetMultiline, "UE::Engine::Canvas::StoppingOffsetMultiline", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Stopping offset on second line
	constexpr FStringView MultiLine = TEXTVIEW("A\nWorld");

	// Get width of "Wo" to calculate offset into second line
	UCanvas::CanvasStringSize(Params, TEXTVIEW("Wo"));
	const float TwoCharsWidth = Params.DrawXL;

	// Stop after "Wo" on second line
	int32 OutIndex;
	UCanvas::MeasureStringInternal(Params, MultiLine, static_cast<int32>(TwoCharsWidth), UCanvas::ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset, OutIndex);

	// Should stop at index 3 ("A\nWo" = 4 chars, but LastWholeBefore gives index 3)
	int32 ExpectedIndex = 3; // Position of 'o' in "World"

	REQUIRE_MESSAGE(TEXT("Multi-line stopping offset should work correctly across line boundaries"),
		OutIndex == ExpectedIndex);
}

TEST_CASE_NAMED(CanvasScalingConsistency, "UE::Engine::Canvas::ScalingConsistency", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params1(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);
	FTextSizingParameters Params2(TestFont, /* ScaleX = */ 2.0f, /* ScaleY = */ 2.0f);

	constexpr FStringView TestText = TEXTVIEW("Hello\nWorld");

	UCanvas::CanvasStringSize(Params1, TestText);
	const float Width1 = Params1.DrawXL;
	const float Height1 = Params1.DrawYL;

	UCanvas::CanvasStringSize(Params2, TestText);
	const float Width2 = Params2.DrawXL;
	const float Height2 = Params2.DrawYL;

	REQUIRE_MESSAGE(TEXT("Scaled width should be approximately 2x original"),
		FMath::IsNearlyEqual(Width2, Width1 * 2.0f, Width1 * 0.1f));
	REQUIRE_MESSAGE(TEXT("Scaled height should be approximately 2x original"),
		FMath::IsNearlyEqual(Height2, Height1 * 2.0f, Height1 * 0.1f));
}

TEST_CASE_NAMED(CanvasKerningAcrossLines, "UE::Engine::Canvas::KerningAcrossLines", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters Params(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);

	// Kerning should not apply across line boundaries
	UCanvas::CanvasStringSize(Params, TEXTVIEW("A"));
	const float AWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("V"));
	const float VWidth = Params.DrawXL;

	UCanvas::CanvasStringSize(Params, TEXTVIEW("A\nV"));
	const float MultiLineWidth = Params.DrawXL;

	// Multi-line version should have width = Max(AWidth, VWidth) without kerning
	const float ExpectedWidth = FMath::Max(AWidth, VWidth);

	REQUIRE_MESSAGE(TEXT("Kerning should not apply across line boundaries"),
		FMath::IsNearlyEqual(MultiLineWidth, ExpectedWidth, ExpectedWidth * 0.1f));
}

TEST_CASE_NAMED(CanvasSpacingAdjustments, "UE::Engine::Canvas::SpacingAdjustments", "[Engine][Canvas][Text][EngineFilter]")
{
	UFont* TestFont = Private::GetTestFont();
	REQUIRE_MESSAGE(TEXT("Engine font must be available for testing"), TestFont != nullptr);

	FTextSizingParameters ParamsNoSpacing(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);
	ParamsNoSpacing.SpacingAdjust = FVector2D(0.0f, 0.0f);
	
	FTextSizingParameters ParamsWithSpacing(TestFont, /* ScaleX = */ 1.0f, /* ScaleY = */ 1.0f);
	ParamsWithSpacing.SpacingAdjust = FVector2D(5.0f, 10.0f);

	constexpr FStringView TestText = TEXTVIEW("Hello\nWorld");

	UCanvas::CanvasStringSize(ParamsNoSpacing, TestText);
	const float WidthNoSpacing = ParamsNoSpacing.DrawXL;
	const float HeightNoSpacing = ParamsNoSpacing.DrawYL;

	UCanvas::CanvasStringSize(ParamsWithSpacing, TestText);
	const float WidthWithSpacing = ParamsWithSpacing.DrawXL;
	const float HeightWithSpacing = ParamsWithSpacing.DrawYL;

	REQUIRE_MESSAGE(TEXT("Horizontal spacing adjustment should increase width"),
		WidthWithSpacing > WidthNoSpacing);
	REQUIRE_MESSAGE(TEXT("Vertical spacing adjustment should increase height"),
		HeightWithSpacing > HeightNoSpacing);
}

} // namespace UE::Canvas

#endif // WITH_TESTS