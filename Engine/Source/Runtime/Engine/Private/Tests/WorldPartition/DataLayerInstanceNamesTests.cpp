// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDataLayerInstanceNamesTest, TEST_NAME_ROOT ".DataLayerInstanceNames", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FDataLayerInstanceNamesTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		// Tests for FDataLayerInstanceNames
		TArray<FName> DataLayerNames = { TEXT("DLD"), TEXT("DLC"), TEXT("DLB"), TEXT("DLA") };
		TArray<FName> DataLayerNamesWithEDL = { TEXT("EDL"), TEXT("DLA"), TEXT("DLB"), TEXT("DLC") };
		TArray<FName> DataLayerNames2 = { TEXT("DLD"), TEXT("DLE"), TEXT("DLF") };

		FDataLayerInstanceNames DataLayersNoEDL(DataLayerNames, NAME_None);
		if (!TestTrue(TEXT("Expected no External Data Layer"), !DataLayersNoEDL.HasExternalDataLayer()))
		{
			return false;
		}

		FDataLayerInstanceNames DataLayersNoEDL2(DataLayerNames2, NAME_None);
		if (!TestTrue(TEXT("Expected no External Data Layer"), !DataLayersNoEDL2.HasExternalDataLayer()))
		{
			return false;
		}

		FDataLayerInstanceNames DataLayersWithEDL(DataLayerNamesWithEDL, true);
		if (!TestTrue(TEXT("Expected External Data Layer"), DataLayersWithEDL.HasExternalDataLayer()))
		{
			return false;
		}

		FDataLayerInstanceNames MergeEDLWithNoEDL = DataLayersWithEDL;
		MergeEDLWithNoEDL.Append(DataLayersNoEDL);
		if (!TestTrue(TEXT("Expected merged ELD with No EDL to contain an External Data Layer"), MergeEDLWithNoEDL.HasExternalDataLayer()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged ELD with No EDL to contain 5 Data Layers"), MergeEDLWithNoEDL.Num() == 5))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged ELD with No EDL to contain 4 non-external Data Layer"), MergeEDLWithNoEDL.GetNonExternalDataLayers().Num() == 4))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged ELD with No EDL to have External Data Layer == 'EDL'"), MergeEDLWithNoEDL.GetExternalDataLayer() == TEXT("EDL")))
		{
			return false;
		}

		FDataLayerInstanceNames MergeEDLWithNoEDLForceEmpty = DataLayersWithEDL;
		MergeEDLWithNoEDLForceEmpty.ForceEmptyNonExternalDataLayers();
		MergeEDLWithNoEDLForceEmpty.Append(DataLayersNoEDL);
		if (!TestTrue(TEXT("Expected merged ELD with No EDL to be marked as ForcedEmptyNonExternalDataLayers"), MergeEDLWithNoEDLForceEmpty.IsForcedEmptyNonExternalDataLayers()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged ELD with No EDL to contain 1 Data Layer"), MergeEDLWithNoEDLForceEmpty.Num() == 1))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged ELD with No EDL to have External Data Layer == 'EDL'"), MergeEDLWithNoEDLForceEmpty.GetExternalDataLayer() == TEXT("EDL")))
		{
			return false;
		}

		FDataLayerInstanceNames MergeNoEDLWithEDL = DataLayersNoEDL;
		MergeNoEDLWithEDL.Append(DataLayersWithEDL);
		if (!TestTrue(TEXT("Expected merged No ELD with EDL to contain an External Data Layer"), MergeNoEDLWithEDL.HasExternalDataLayer()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with EDL to contain 5 Data Layers"), MergeNoEDLWithEDL.Num() == 5))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with EDL to contain 4 non-external Data Layer"), MergeNoEDLWithEDL.GetNonExternalDataLayers().Num() == 4))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with EDL to have External Data Layer == 'EDL'"), MergeNoEDLWithEDL.GetExternalDataLayer() == TEXT("EDL")))
		{
			return false;
		}

		FDataLayerInstanceNames MergeNoEDLWithEDLForceEmpty = DataLayersNoEDL;
		MergeNoEDLWithEDLForceEmpty.ForceEmptyNonExternalDataLayers();
		MergeNoEDLWithEDLForceEmpty.Append(DataLayersWithEDL);
		if (!TestTrue(TEXT("Expected merged No ELD with EDL to be marked as ForcedEmptyNonExternalDataLayers"), MergeNoEDLWithEDLForceEmpty.IsForcedEmptyNonExternalDataLayers()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with EDL to contain 1 Data Layer"), MergeNoEDLWithEDLForceEmpty.Num() == 1))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with EDL to contain 1 Data Layer"), MergeNoEDLWithEDLForceEmpty.GetExternalDataLayer() == TEXT("EDL")))
		{
			return false;
		}

		FDataLayerInstanceNames MergeNoEDLWithNoEDL2 = DataLayersNoEDL;
		MergeNoEDLWithNoEDL2.Append(DataLayersNoEDL2);

		if (!TestTrue(TEXT("Expected merged No ELD with No EDL to contain no External Data Layer"), !MergeNoEDLWithNoEDL2.HasExternalDataLayer()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with No EDL to contain 6 Data Layers"), MergeNoEDLWithNoEDL2.Num() == 6))
		{
			return false;
		}

		if (!TestTrue(TEXT("Expected merged No ELD with No EDL to contain 6 non-external Data Layer"), MergeNoEDLWithNoEDL2.GetNonExternalDataLayers().Num() == 6))
		{
			return false;
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif 
