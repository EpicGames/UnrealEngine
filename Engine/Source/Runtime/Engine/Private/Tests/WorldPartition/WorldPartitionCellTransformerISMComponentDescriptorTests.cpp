// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartitionCellTransformerISMComponentDescriptor.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionCellTransformerISMComponentDescriptorTest, TEST_NAME_ROOT ".WorldPartitionCellTransformerISMComponentDescriptor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FWorldPartitionCellTransformerISMComponentDescriptorTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		auto MakeSMC = [](ECollisionEnabled::Type Coll, ECanBeCharacterBase Step)
		{
			UStaticMeshComponent* SMC = NewObject<UStaticMeshComponent>(GetTransientPackage());
			SMC->BodyInstance.SetCollisionEnabled(Coll, /*bUpdatePhysicsFilterData=*/false);
			SMC->CanCharacterStepUpOn = Step;
			return SMC;
		};

		// Off-mode: CanCharacterStepUpOn must not affect bucketing (backward compat).
		{
			FWorldPartitionCellTransformerISMComponentDescriptor A, B;
			A.bStrictBucketing = false;
			B.bStrictBucketing = false;
			A.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_Yes));
			B.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			if (!TestTrue(TEXT("Off-mode: descriptors equal regardless of CanCharacterStepUpOn"), A == B))
			{
				return false;
			}
			if (!TestTrue(TEXT("Off-mode: hashes match"), A.GetTypeHash() == B.GetTypeHash()))
			{
				return false;
			}
		}

		// Strict + query collision: divergent CanCharacterStepUpOn must split buckets.
		{
			FWorldPartitionCellTransformerISMComponentDescriptor A, B;
			A.bStrictBucketing = true;
			B.bStrictBucketing = true;
			A.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_Yes));
			B.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			if (!TestFalse(TEXT("Strict + query: descriptors differ"), A == B))
			{
				return false;
			}
			if (!TestTrue(TEXT("Strict + query: hashes differ"), A.GetTypeHash() != B.GetTypeHash()))
			{
				return false;
			}
		}

		// Strict + query, same value: must bucket together.
		{
			FWorldPartitionCellTransformerISMComponentDescriptor A, B;
			A.bStrictBucketing = true;
			B.bStrictBucketing = true;
			A.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			B.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			if (!TestTrue(TEXT("Strict + query, matching value: equal"), A == B))
			{
				return false;
			}
			if (!TestTrue(TEXT("Strict + query, matching value: hashes match"), A.GetTypeHash() == B.GetTypeHash()))
			{
				return false;
			}
		}

		// Strict + NoCollision: CanCharacterStepUpOn must be irrelevant.
		{
			FWorldPartitionCellTransformerISMComponentDescriptor A, B;
			A.bStrictBucketing = true;
			B.bStrictBucketing = true;
			A.InitFrom(MakeSMC(ECollisionEnabled::NoCollision, ECB_Yes));
			B.InitFrom(MakeSMC(ECollisionEnabled::NoCollision, ECB_No));
			if (!TestTrue(TEXT("Strict + NoCollision: descriptors equal (CanCharacterStepUpOn irrelevant)"), A == B))
			{
				return false;
			}
			if (!TestTrue(TEXT("Strict + NoCollision: hashes match"), A.GetTypeHash() == B.GetTypeHash()))
			{
				return false;
			}
		}

		// InitComponent: strict + query propagates the value to the merged ISM.
		{
			FWorldPartitionCellTransformerISMComponentDescriptor D;
			D.bStrictBucketing = true;
			D.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
			D.InitComponent(ISM);
			if (!TestTrue(TEXT("Strict + query: ISM receives descriptor's CanCharacterStepUpOn"), ISM->CanCharacterStepUpOn == ECB_No))
			{
				return false;
			}
		}

		// InitComponent: off-mode leaves the merged ISM at the UPrimitiveComponent ctor default.
		{
			FWorldPartitionCellTransformerISMComponentDescriptor D;
			D.bStrictBucketing = false;
			D.InitFrom(MakeSMC(ECollisionEnabled::QueryOnly, ECB_No));
			UInstancedStaticMeshComponent* ISM = NewObject<UInstancedStaticMeshComponent>(GetTransientPackage());
			D.InitComponent(ISM);
			if (!TestTrue(TEXT("Off-mode: ISM keeps ctor default ECB_Yes"), ISM->CanCharacterStepUpOn == ECB_Yes))
			{
				return false;
			}
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif
