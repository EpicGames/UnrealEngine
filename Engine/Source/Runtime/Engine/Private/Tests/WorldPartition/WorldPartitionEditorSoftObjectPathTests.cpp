// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "EditorWorldUtils.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "AssetToolsModule.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "Misc/EditorPathHelper.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

COREUOBJECT_API extern bool GIsEditorPathFeatureEnabled;

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionEditorSoftObjectPathTest, TEST_NAME_ROOT ".EditorPaths", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FWorldPartitionEditorSoftObjectPathTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		ON_SCOPE_EXIT
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		};

		TGuardValue<bool> EnableEditorPathFeature(GIsEditorPathFeatureEnabled, true);

		FScopedEditorWorld ScopedEditorWorld(
			TEXTVIEW("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths"),
			UWorld::InitializationValues()
			.RequiresHitProxies(false)
			.ShouldSimulatePhysics(false)
			.EnableTraceCollision(false)
			.CreateNavigation(false)
			.CreateAISystem(false)
			.AllowAudioPlayback(false)
			.CreatePhysicsScene(true)
		);

		UWorld* World = ScopedEditorWorld.GetWorld();
		if (!TestNotNull(TEXT("Missing World Object"), World))
		{
			return false;
		}

		check(World);
		UWorldPartition* WorldPartition = World->GetWorldPartition();
		if (!TestNotNull(TEXT("Missing World Partition Object"), WorldPartition))
		{
			return false;
		}

		// Make sure all level instances are loaded see (UE-194631)
		World->BlockTillLevelStreamingCompleted();

		// Editor Path resolving
		{
			// Test that we can resolve Actor
			FSoftObjectPath EditorPath(TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.LevelInstance_UAID_E04F43E60CCA16A401_1407795222.StaticMeshActor_UAID_E04F43E60CCA16A401_1407522221"));
			AActor* ResolvedActor = Cast<AActor>(EditorPath.ResolveObject());
			if (!TestNotNull(TEXT("Failed to resolve editor path"), ResolvedActor))
			{
				return false;
			}

			// Test that FEditorPathHelper::GetEditorPath(ResolvedActor) returns the same path as EditorPath
			FSoftObjectPath NewEditorPath = FEditorPathHelper::GetEditorPath(ResolvedActor);
			if (!TestEqual(TEXT("Resolved actor editor path mismatch"), EditorPath, NewEditorPath))
			{
				return false;
			}

			// Test that we can resolve a LevelInstance through Editor Path
			FSoftObjectPath LevelInstanceEditorPath(TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877"));
			ALevelInstance* LevelInstance = Cast<ALevelInstance>(LevelInstanceEditorPath.ResolveObject());
			if (!TestNotNull(TEXT("Failed to resolve editor path"), LevelInstance))
			{
				return false;
			}

			// Test that FEditorPathHelper::GetEditorPathFromReferencer(ResolvedActor, LevelInstance) returns an Editor Path relative to the Level Instance (Context)
			EditorPath = FEditorPathHelper::GetEditorPathFromReferencer(ResolvedActor, LevelInstance);
			if (!TestEqual(TEXT("Failed to create an editor path relative to Level Instance"), FSoftObjectPath(LevelInstance).GetAssetPath(), EditorPath.GetAssetPath()))
			{
				return false;
			}

			// Test that this relative path resolves to the same actor as the AActor::GetEditorPath(nullptr)
			AActor* ResolvedSameActor = Cast<AActor>(EditorPath.ResolveObject());
			if (!TestEqual(TEXT("Failed to resolve sub container editor path"), ResolvedActor, ResolvedSameActor))
			{
				return false;
			}
		}

		// Runtime Path testing
		{
			UWorldPartition::FGenerateStreamingParams Params = UWorldPartition::FGenerateStreamingParams();

			UWorldPartition::FGenerateStreamingContext Context;
			WorldPartition->GenerateStreaming(Params, Context);

			// Test conversion of Editor Path to Runtime Path (1 Level deep StaticMeshActor)	
			FSoftObjectPath RuntimePath;
			FSoftObjectPath EditorPath(TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.StaticMeshActor_UAID_E04F43E60CCA14A401_2106052870"));
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_b3e12442c23432cb_2106052870")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (1 Level deep StaticMeshComponent)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.StaticMeshActor_UAID_E04F43E60CCA14A401_2106052870.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_b3e12442c23432cb_2106052870.StaticMeshComponent0")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (1 Level deep StaticMeshActor, different top level instance)
			EditorPath =(TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.StaticMeshActor_UAID_E04F43E60CCA14A401_2106052870"));
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_855651f5ca95d6b9_2106052870")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (1 Level deep StaticMeshComponent, different top level instance)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.StaticMeshActor_UAID_E04F43E60CCA14A401_2106052870.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_855651f5ca95d6b9_2106052870.StaticMeshComponent0")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (2 Level deep StaticMeshActor)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.StaticMeshActor_UAID_E04F43E60CCA14A401_2145611875");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_023ce1270d5e3394_2145611875")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (2 Level deep StaticMeshComponent)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.StaticMeshActor_UAID_E04F43E60CCA14A401_2145611875.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_023ce1270d5e3394_2145611875.StaticMeshComponent0")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (2 Level deep StaticMeshActor, different top level instance)
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.StaticMeshActor_UAID_E04F43E60CCA14A401_2145611875");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_b48c4d72c8d10646_2145611875")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (2 Level deep StaticMeshComponent, different top level instance)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.StaticMeshActor_UAID_E04F43E60CCA14A401_2145611875.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA14A401_b48c4d72c8d10646_2145611875.StaticMeshComponent0")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (3 Levels deep StaticMeshActor)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.LevelInstance_UAID_E04F43E60CCA16A401_1407795222.StaticMeshActor_UAID_E04F43E60CCA16A401_1407522221");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA16A401_c72e7f8fcd5f70c5_1407522221")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (3 Levels deep StaticMeshComponent)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA14A401_2106307871.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.LevelInstance_UAID_E04F43E60CCA16A401_1407795222.StaticMeshActor_UAID_E04F43E60CCA16A401_1407522221.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA16A401_c72e7f8fcd5f70c5_1407522221.StaticMeshComponent0")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (3 Levels deep StaticMeshActor, different top level instance)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.LevelInstance_UAID_E04F43E60CCA16A401_1407795222.StaticMeshActor_UAID_E04F43E60CCA16A401_1407522221");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA16A401_0f31c5c4ff7639d5_1407522221")))
			{
				return false;
			}

			// Test conversion of Editor Path to Runtime Path (3 Levels deep StaticMeshComponent, different top level instance)	
			EditorPath = TEXT("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths.EditorPaths:PersistentLevel.LevelInstance_UAID_E04F43E60CCA16A401_1435304224.LevelInstance_UAID_E04F43E60CCA14A401_2145683877.LevelInstance_UAID_E04F43E60CCA16A401_1407795222.StaticMeshActor_UAID_E04F43E60CCA16A401_1407522221.StaticMeshComponent0");
			FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(EditorPath, RuntimePath);
			if (!TestEqual(TEXT("Invalid runtime path"), RuntimePath, FSoftObjectPath("/Engine/WorldPartition/UnitTests/EditorPaths/EditorPaths/_Generated_/2WBX502V75SLTW3UKBT9S2GG6.EditorPaths:PersistentLevel.StaticMeshActor_UAID_E04F43E60CCA16A401_0f31c5c4ff7639d5_1407522221.StaticMeshComponent0")))
			{
				return false;
			}
		}

#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif