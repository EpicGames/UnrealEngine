// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "EditorWorldUtils.h"
#include "PackageTools.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

#if WITH_EDITOR
struct FWorldPartitionActorDescUnitTestAcccessor
{
	inline static uint32 GetSoftRefCount(const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		return ActorDescInstance->GetSoftRefCount();
	}

	inline static uint32 GetHardRefCount(const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		return ActorDescInstance->GetHardRefCount();
	}
};
#endif

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionSoftRefTest, TEST_NAME_ROOT ".Handle", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#if WITH_EDITOR
	template <class LoadingContextType>
	void PerformTests(FWorldPartitionSoftRefTest* Test)
	{
		auto TestTrue = [Test](const TCHAR* What, bool Value) -> bool { return Test->TestTrue(What, Value); };
		auto TestFalse = [Test](const TCHAR* What, bool Value) -> bool { return Test->TestFalse(What, Value); };

		TUniquePtr<FScopedEditorWorld> ScopedEditorWorld = MakeUnique<FScopedEditorWorld>(
			TEXTVIEW("/Engine/WorldPartition/WorldPartitionUnitTest"),
			UWorld::InitializationValues()
				.RequiresHitProxies(false)
				.ShouldSimulatePhysics(false)
				.EnableTraceCollision(false)
				.CreateNavigation(false)
				.CreateAISystem(false)
				.AllowAudioPlayback(false)
				.CreatePhysicsScene(true),
			EWorldType::None
		);
		
		UWorld* World = ScopedEditorWorld->GetWorld();
		if (!Test->TestNotNull(TEXT("Missing World Object"), World))
		{
			return;
		}

		TestTrue(TEXT("World type"), World->WorldType == EWorldType::None);

		UActorDescContainerInstance* ActorDescContainerInstance = NewObject<UActorDescContainerInstance>(World);
		ActorDescContainerInstance->Initialize({ TEXT("/Engine/WorldPartition/WorldPartitionUnitTest") });

		FWorldPartitionHandle Handle = FWorldPartitionHandle(ActorDescContainerInstance, FGuid(TEXT("5D9F93BA407A811AFDDDAAB4F1CECC6A")));

		{
			FWorldPartitionHandle ActorWithChildActorComponentHandle = FWorldPartitionHandle(ActorDescContainerInstance, FGuid(TEXT("538856174ECD465948488F9441AE9251")));
			FWorldPartitionReference ActorWithChildActorComponentReference = ActorWithChildActorComponentHandle.ToReference();
		}		

		FWorldPartitionReference Reference;
		{
			LoadingContextType LoadingContext;
			Reference = FWorldPartitionReference(ActorDescContainerInstance, FGuid(TEXT("0D2B04D240BE5DE58FE437A8D2DBF5C9")));
		}

		TestTrue(TEXT("Handle container"), Handle->GetContainerInstance() == ActorDescContainerInstance);
		TestTrue(TEXT("Reference container"), Reference->GetContainerInstance() == ActorDescContainerInstance);

		TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
		TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
		TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

		// Pin handle scope test
		{
			FWorldPartitionHandlePinRefScope PinRefScopeHandle(Handle);
			TestTrue(TEXT("Pin to Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
			TestTrue(TEXT("Pin to Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		}

		// Pin reference scope test
		{
			FWorldPartitionHandlePinRefScope PinRefScopeReference(Reference);
			TestTrue(TEXT("Pin to Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
			TestTrue(TEXT("Pin to Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 2);
		}

		TestTrue(TEXT("Handle/Reference equality"), Handle != Reference);
		TestTrue(TEXT("Reference/Handle equality"), Reference != Handle);
		
		TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
		TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
		TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

		// Conversions
		{
			FWorldPartitionHandle HandleToReference = Reference.ToHandle();
			TestTrue(TEXT("Handle/Reference equality"), HandleToReference == Reference);
			TestTrue(TEXT("Reference/Handle equality"), Reference == HandleToReference);
			TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 1);
			TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

			FWorldPartitionReference ReferenceToHandle = Handle.ToReference();
			TestTrue(TEXT("Handle/Reference equality"), ReferenceToHandle == Handle);
			TestTrue(TEXT("Handle/Reference equality"), Handle == ReferenceToHandle);
			TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 1);
			TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
		}

		TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
		TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
		TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

		// inplace new test
		{
			TOptional<FWorldPartitionHandle> HandlePtr;
			HandlePtr.Emplace(Reference.ToHandle());

			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 1);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

			HandlePtr.Reset();

			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
		}

		// TArray test
		{
			TArray<FWorldPartitionHandle> HandleList;
			HandleList.Add(Handle);

			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 2);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);

			FWorldPartitionReference ReferenceToHandle = Handle.ToReference();
			TestTrue(TEXT("Handle/Reference equality"), ReferenceToHandle == Handle);
			TestTrue(TEXT("Handle/Reference equality"), Handle == ReferenceToHandle);
			TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 2);
			TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 1);

			TestTrue(TEXT("Handle array contains handle"), HandleList.Contains(Handle));
			TestTrue(TEXT("Handle array contains reference"), HandleList.Contains(ReferenceToHandle));

			HandleList.Add(Reference.ToHandle());
			
			TestTrue(TEXT("Handle array contains reference"), HandleList.Contains(Reference));
			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 1);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
			
			HandleList.Remove(Handle);
			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 1);

			HandleList.Remove(Reference.ToHandle());
			TestTrue(TEXT("Handle array soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
			TestTrue(TEXT("Handle array hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
		}

		TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
		TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
		TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

		// TSet test
		{
			TSet<FWorldPartitionHandle> HandleSet;
			HandleSet.Add(Handle);

			TestTrue(TEXT("Handle set soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 2);
			TestTrue(TEXT("Handle set hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);

			FWorldPartitionReference ReferenceToHandle = Handle.ToReference();
			TestTrue(TEXT("Handle/Reference equality"), ReferenceToHandle == Handle);
			TestTrue(TEXT("Handle/Reference equality"), Handle == ReferenceToHandle);
			TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 2);
			TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 1);

			TestTrue(TEXT("Handle set contains handle"), HandleSet.Contains(Handle));
			TestTrue(TEXT("Handle set contains reference"), HandleSet.Contains(ReferenceToHandle.ToHandle()));

			HandleSet.Add(Reference.ToHandle());
			TestTrue(TEXT("Handle set contains reference"), HandleSet.Contains(Reference.ToHandle()));

			TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 1);
			TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
		}

		TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
		TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
		TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
		TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

		// Move tests
		{
			// Handle move
			TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
			TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
			{
				FWorldPartitionHandle HandleCopy(MoveTemp(Handle));
				TestTrue(TEXT("Handle move src not valid"), !Handle.IsValid());
				TestTrue(TEXT("Handle move dst valid"), HandleCopy.IsValid());
				TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*HandleCopy) == 1);
				TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*HandleCopy) == 0);

				Handle = MoveTemp(HandleCopy);
				TestTrue(TEXT("Handle move src not valid"), !HandleCopy.IsValid());
				TestTrue(TEXT("Handle move dst valid"), Handle.IsValid());
				TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
				TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
			}

			// Reference move
			TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
			TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
			{
				FWorldPartitionReference ReferenceCopy(MoveTemp(Reference));
				TestTrue(TEXT("Reference move src not valid"), !Reference.IsValid());
				TestTrue(TEXT("Reference move dst valid"), ReferenceCopy.IsValid());
				TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*ReferenceCopy) == 0);
				TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*ReferenceCopy) == 1);

				Reference = MoveTemp(ReferenceCopy);
				TestTrue(TEXT("Reference move src not valid"), !ReferenceCopy.IsValid());
				TestTrue(TEXT("Reference move dst valid"), Reference.IsValid());
				TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
				TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
			}

			// Handle reference move
			{
				FWorldPartitionHandle HandleFromReference(MoveTemp(Reference));
				TestTrue(TEXT("Handle move src not valid"), !Reference.IsValid());
				TestTrue(TEXT("Handle move dst valid"), HandleFromReference.IsValid());
				TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*HandleFromReference) == 1);
				TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*HandleFromReference) == 0);

				Reference = MoveTemp(HandleFromReference);
				TestTrue(TEXT("Handle move src not valid"), !HandleFromReference.IsValid());
				TestTrue(TEXT("Reference move dst valid"), Reference.IsValid());
				TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
				TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);
			}

			// Reference handle move
			{
				FWorldPartitionReference ReferenceFromHandle(MoveTemp(Handle));
				TestTrue(TEXT("Handle move src not valid"), !Handle.IsValid());
				TestTrue(TEXT("Reference move dst valid"), ReferenceFromHandle.IsValid());
				TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*ReferenceFromHandle) == 0);
				TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*ReferenceFromHandle) == 1);

				Handle = MoveTemp(ReferenceFromHandle);
				TestTrue(TEXT("Reference move src not valid"), !ReferenceFromHandle.IsValid());
				TestTrue(TEXT("Handle move dst valid"), Handle.IsValid());
				TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
				TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
			}

			// Reset
			{
				FWorldPartitionReference ReferenceFromHandle(Handle.ToReference());
				TestTrue(TEXT("Reference valid"), ReferenceFromHandle.IsValid());
				TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*ReferenceFromHandle) == 1);
				TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*ReferenceFromHandle) == 1);

				ReferenceFromHandle.Reset();
				TestTrue(TEXT("Reference not valid"), !ReferenceFromHandle.IsValid());
				TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
				TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
			}
		}

		// Make sure the container gets destroyed so we can test destructing dangling handles
		{
			TestTrue(TEXT("Invalid container test"), Handle.IsValid());
			TestTrue(TEXT("Invalid container test"), Reference.IsValid());

			TestTrue(TEXT("Handle soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Handle) == 1);
			TestTrue(TEXT("Handle hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Handle) == 0);
			TestTrue(TEXT("Reference soft refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetSoftRefCount(*Reference) == 0);
			TestTrue(TEXT("Reference hard refcount"), FWorldPartitionActorDescUnitTestAcccessor::GetHardRefCount(*Reference) == 1);

			TWeakObjectPtr<AActor> LoadedActor = Reference->GetActor();
			TestTrue(TEXT("Loaded actor"), LoadedActor.IsValid());
			TestTrue(TEXT("Registered actor"), World->PersistentLevel->Actors.Find(LoadedActor.Get()) != INDEX_NONE);

			ActorDescContainerInstance->Uninitialize();

			TestTrue(TEXT("Registered actor"), World->PersistentLevel->Actors.Find(LoadedActor.Get()) == INDEX_NONE);

			// Make sure to cleanup world before collecting garbage so it gets uninitialized
			ScopedEditorWorld.Reset();
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			TestFalse(TEXT("Invalid container test"), Handle.IsValid());
			TestFalse(TEXT("Invalid container test"), Reference.IsValid());
		}
	};
#endif

	bool FWorldPartitionSoftRefTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		// Handle tests
		WorldPartitionTests::PerformTests<FWorldPartitionLoadingContext::FImmediate>(this);
		WorldPartitionTests::PerformTests<FWorldPartitionLoadingContext::FDeferred>(this);

		// Serialization tests
		UActorDescContainerInstance* ActorDescContainerInstance = NewObject<UActorDescContainerInstance>(GetTransientPackage());
		ActorDescContainerInstance->Initialize({ TEXT("/Engine/WorldPartition/WorldPartitionUnitTest") });

		for (UActorDescContainerInstance::TIterator<> Iterator(ActorDescContainerInstance); Iterator; ++Iterator)
		{
			UClass* ActorNativeClass = UClass::TryFindTypeSlow<UClass>(Iterator->GetNativeClass().ToString(), EFindFirstObjectOptions::ExactClass);
			TestFalse(TEXT("Actor Descriptor Serialization"), !ActorNativeClass);

			if (ActorNativeClass)
			{
				TUniquePtr<FWorldPartitionActorDesc> NewActorDesc(AActor::StaticCreateClassActorDesc(ActorNativeClass));

				FWorldPartitionActorDescInitData ActorDescInitData = FWorldPartitionActorDescInitData()
					.SetNativeClass(ActorNativeClass)
					.SetPackageName(Iterator->GetActorPackage())
					.SetActorPath(Iterator->GetActorSoftPath());
			
				Iterator->GetActorDesc()->SerializeTo(ActorDescInitData.GetSerializedData());
				NewActorDesc->Init(ActorDescInitData);

				TestTrue(TEXT("Actor Descriptor Serialization"), NewActorDesc->Equals(Iterator->GetActorDesc()));
			}
		}

		ActorDescContainerInstance->Uninitialize();
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif 
