// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "EditorWorldUtils.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionPathConversionTest, TEST_NAME_ROOT ".PathConversion", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FWorldPartitionPathConversionTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		ON_SCOPE_EXIT
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		};

		FScopedEditorWorld ScopedEditorWorld(
			TEXTVIEW("/Engine/WorldPartition/WorldPartitionUnitTest"),
			UWorld::InitializationValues()
				.RequiresHitProxies(false)
				.ShouldSimulatePhysics(false)
				.EnableTraceCollision(false)
				.CreateNavigation(false)
				.CreateAISystem(false)
				.AllowAudioPlayback(false)
				.CreatePhysicsScene(true)
		);
		
		UWorld* World = ScopedEditorWorld.GetWorld();
		if (!TestNotNull(TEXT("Missing World Object"), World))
		{
			return false;
		}

		check(World);
		UWorldPartition* WorldPartition = World->GetWorldPartition();
		if (!TestNotNull(TEXT("Missing World Partition Object"), WorldPartition))
		{
			return false;
		}
		
		check(WorldPartition);
		UActorDescContainerInstance* BaseActorDescContainerInstance = WorldPartition->GetActorDescContainerInstance();
		if (!TestNotNull(TEXT("Missing World Partition Container"), BaseActorDescContainerInstance))
		{
			return false;
		}

		FActorDescContainerInstanceCollection Collection({ TObjectPtr<UActorDescContainerInstance>(BaseActorDescContainerInstance) });
		UWorldPartition::FGenerateStreamingParams Params = UWorldPartition::FGenerateStreamingParams()
			.SetContainerInstanceCollection(Collection, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs);
		UWorldPartition::FGenerateStreamingContext Context;

		if (!TestTrue(TEXT("World Partition Generate Streaming"), WorldPartition->GenerateContainerStreaming(Params, Context)))
		{
			return false;
		}

		FWorldPartitionReference ActorRef(BaseActorDescContainerInstance, FGuid(TEXT("5D9F93BA407A811AFDDDAAB4F1CECC6A")));
		if (!TestTrue(TEXT("Invalid Actor Reference"), ActorRef.IsValid()))
		{
			return false;
		}

		AActor* Actor = ActorRef.GetActor();
		if (!TestNotNull(TEXT("Missing Actor"), Actor))
		{
			return false;
		}

		FWorldPartitionHandle ActorHandle(BaseActorDescContainerInstance, FGuid(TEXT("0D2B04D240BE5DE58FE437A8D2DBF5C9")));
		if (!TestTrue(TEXT("Invalid Actor Handle"), ActorHandle.IsValid()))
		{
			return false;
		}

		if (!TestNull(TEXT("Actor Handle Not Loaded"), ActorHandle.GetActor()))
		{
			return false;
		}

		UObject* ResolvedObject = ActorHandle->GetActorSoftPath().TryLoad();
		if (!TestNotNull(TEXT("Actor Handle Loaded"), ActorHandle.GetActor()))
		{
			return false;
		}

		if (!TestTrue(TEXT("Resolving Runtime Actor From Editor Path Failed"), ResolvedObject == ActorHandle.GetActor()))
		{
			return false;
		}

		auto TestEditorRuntimePathConversion = [this](const FSoftObjectPath& InEditorPath) -> bool
		{
			FSoftObjectPath RuntimePath;
			if (!TestTrue(TEXT("Path Editor to Runtime Conversion Failed"), FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(InEditorPath, RuntimePath)))
			{
				return false;
			}

			FSoftObjectPath EditorPath;
			if (!TestTrue(TEXT("Path Runtime to Editor Conversion Failed"), FWorldPartitionHelpers::ConvertRuntimePathToEditorPath(RuntimePath, EditorPath)))
			{
				return false;
			}

			if (!TestTrue(TEXT("Path Editor to Runtime to Editor Conversion Failed"), EditorPath == InEditorPath))
			{
				return false;
			}

			return true;
		};

		// Test path conversions to an actor
		if (!TestEditorRuntimePathConversion(FSoftObjectPath(Actor)))
		{
			return false;
		}

		// Test path conversions to an actor component
		if (!TestEditorRuntimePathConversion(FSoftObjectPath(Actor->GetRootComponent())))
		{
			return false;
		}

		// Test path conversions to an instanced level
		UPackage* WorldPackage = World->GetPackage();
		WorldPackage->Rename(*FString(WorldPackage->GetName() + TEXT("_LevelInstance_1")));

		// Test path conversions to an instanced actor
		if (!TestEditorRuntimePathConversion(FSoftObjectPath(Actor)))
		{
			return false;
		}

		// Test path conversions to an instanced actor component
		if (!TestEditorRuntimePathConversion(FSoftObjectPath(Actor->GetRootComponent())))
		{
			return false;
		}

		// Test path conversions to a newly spawned actor
		AActor* NewActor = World->SpawnActor(AActor::StaticClass());
		if (!TestNotNull(TEXT("Sapwning Actor Failed"), NewActor))
		{
			return false;
		}

		if (!TestEditorRuntimePathConversion(FSoftObjectPath(NewActor)))
		{
			return false;
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif 
