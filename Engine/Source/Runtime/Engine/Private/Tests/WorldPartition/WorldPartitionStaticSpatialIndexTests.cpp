// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "Math/StaticSpatialIndex.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionStaticSpatialIndexTest, TEST_NAME_ROOT ".StaticSpatialIndex", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#if WITH_EDITOR
	template <typename BoxType>
	const TCHAR* GetSpaceString()
	{
		if constexpr (std::is_same<BoxType, FBox2D>::value)
		{
			return TEXT("2d");
		}
		else
		{
			return TEXT("3d");
		}
	}

	template <typename Profile, class Class>
	FORCENOINLINE void PerformTests(FWorldPartitionStaticSpatialIndexTest* Test, const TCHAR* Name, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, TArray<int32>& Results)
	{
		Class SpatialIndex;
		SpatialIndex.Init(Elements);

		const double StartTime = FPlatformTime::Seconds();
		for (int32 ListNumTests = 0; ListNumTests < Tests.Num(); ListNumTests++)
		{
			FStaticSpatialIndex::FSphere Sphere(Tests[ListNumTests].Center, Tests[ListNumTests].W);
			SpatialIndex.ForEachIntersectingElement(Sphere, [&Results](const int32& Value) { Results.Add(Value); });
		}
		const double RunTime = FPlatformTime::Seconds() - StartTime;

		Test->AddInfo(FString::Printf(TEXT("%s(%s): %d tests in %s (%.2f/s, %s)"), Name, GetSpaceString<typename Profile::FBox>(), Tests.Num(), *FPlatformTime::PrettyTime(RunTime), Tests.Num() / RunTime, *FGenericPlatformMemory::PrettyMemory(SpatialIndex.GetAllocatedSize())));
	}

	template <typename Profile, int32 MaxNumElementsPerNode, int32 MaxNumElementsPerLeaf>
	FORCENOINLINE void PerformNoSortTest(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		TArray<int32> RTreeNoSortResults;
		RTreeNoSortResults.Reserve(ReferenceResults.Num());
		FString TestName = FString::Printf(TEXT("TStaticSpatialIndexRTree(%s-NoSort-%d-%d)"), GetSpaceString<typename Profile::FBox>(), MaxNumElementsPerNode, MaxNumElementsPerLeaf);
		PerformTests<Profile, TStaticSpatialIndexRTree<int32, FStaticSpatialIndex::TNodeSorterNoSort<Profile>, Profile>>(Test, *TestName, Elements, Tests, RTreeNoSortResults);
		RTreeNoSortResults.Sort();
		Test->TestTrue(TestName, RTreeNoSortResults == ReferenceResults);
	}

	template <typename Profile>
	FORCENOINLINE void PerformNoSortTests(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		PerformNoSortTest<Profile, 16, 16>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 16, 64>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 16, 256>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 16, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 64, 16>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 64, 64>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 64, 256>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 64, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 256, 16>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 256, 64>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 256, 256>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 256, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 1024, 16>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 1024, 64>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 1024, 256>(Test, Elements, Tests, ReferenceResults);
		PerformNoSortTest<Profile, 1024, 1024>(Test, Elements, Tests, ReferenceResults);
	}

	template <typename Profile, int32 MaxNumElementsPerNode, int32 MaxNumElementsPerLeaf>
	FORCENOINLINE void PerformMinXTest(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		TArray<int32> RTreeMinXResults;
		RTreeMinXResults.Reserve(ReferenceResults.Num());
		FString TestName = FString::Printf(TEXT("TStaticSpatialIndexRTree(%s-minx-%d-%d)"), GetSpaceString<typename Profile::FBox>(), MaxNumElementsPerNode, MaxNumElementsPerLeaf);
		PerformTests<Profile, TStaticSpatialIndexRTree<int32, FStaticSpatialIndex::TNodeSorterMinX<Profile>, Profile>>(Test, *TestName, Elements, Tests, RTreeMinXResults);
		RTreeMinXResults.Sort();
		Test->TestTrue(TestName, RTreeMinXResults == ReferenceResults);
	}

	template <typename Profile>
	FORCENOINLINE void PerformMinXTests(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		PerformMinXTest<Profile, 16, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 16, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 16, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 16, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 64, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 64, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 64, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 64, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 256, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 256, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 256, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 256, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 1024, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 1024, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 1024, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMinXTest<Profile, 1024, 1024>(Test, Elements, Tests, ReferenceResults);
	}

	template <typename Profile, int32 BucketSize, int32 MaxNumElementsPerNode, int32 MaxNumElementsPerLeaf>
	FORCENOINLINE void PerformMortonTest(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		TArray<int32> RTreeMortonResults;
		RTreeMortonResults.Reserve(ReferenceResults.Num());
		FString TestName = FString::Printf(TEXT("TStaticSpatialIndexRTree(%s-morton-%dk-%d-%d)"), GetSpaceString<typename Profile::FBox>(), BucketSize >> 10, MaxNumElementsPerNode, MaxNumElementsPerLeaf);
		PerformTests<Profile, TStaticSpatialIndexRTree<int32, FStaticSpatialIndex::TNodeSorterMorton<Profile, BucketSize>, Profile>>(Test, *TestName, Elements, Tests, RTreeMortonResults);
		RTreeMortonResults.Sort();
		Test->TestTrue(TestName, RTreeMortonResults == ReferenceResults);
	}

	template <typename Profile, int32 BucketSize>
	FORCENOINLINE void PerformMortonTests(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		PerformMortonTest<Profile, BucketSize, 16, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 16, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 16, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 16, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 64, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 64, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 64, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 64, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 256, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 256, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 256, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 256, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 1024, 16>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 1024, 64>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 1024, 256>(Test, Elements, Tests, ReferenceResults);
		PerformMortonTest<Profile, BucketSize, 1024, 1024>(Test, Elements, Tests, ReferenceResults);
	}

	template <typename Profile, int32 BucketSize, int32 MaxNumElementsPerNode, int32 MaxNumElementsPerLeaf>
	FORCENOINLINE void PerformHilbertTest(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		TArray<int32> RTreeHilbertResults;
		RTreeHilbertResults.Reserve(ReferenceResults.Num());
		FString TestName = FString::Printf(TEXT("TStaticSpatialIndexRTree(%s-hilbert-%dk-%d-%d)"), GetSpaceString<typename Profile::FBox>(), BucketSize >> 10, MaxNumElementsPerNode, MaxNumElementsPerLeaf);
		PerformTests<Profile, TStaticSpatialIndexRTree<int32, FStaticSpatialIndex::TNodeSorterHilbert<Profile, BucketSize>, Profile>>(Test, *TestName, Elements, Tests, RTreeHilbertResults);
		RTreeHilbertResults.Sort();
		Test->TestTrue(TestName, RTreeHilbertResults == ReferenceResults);
	}

	template <typename Profile, int32 BucketSize>
	FORCENOINLINE void PerformHilbertTests(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests, const TArray<int32>& ReferenceResults)
	{
		PerformHilbertTest<Profile, BucketSize, 16, 16>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 16, 64>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 16, 256>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 16, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 64, 16>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 64, 64>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 64, 256>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 64, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 256, 16>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 256, 64>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 256, 256>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 256, 1024>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 1024, 16>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 1024, 64>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 1024, 256>(Test, Elements, Tests, ReferenceResults);
		PerformHilbertTest<Profile, BucketSize, 1024, 1024>(Test, Elements, Tests, ReferenceResults);
	}

	template <typename Profile>
	FORCENOINLINE void PerformTests(FWorldPartitionStaticSpatialIndexTest* Test, const TArray<TPair<typename Profile::FBox, int32>>& Elements, const TArray<FSphere>& Tests)
	{
		TArray<int32> ListResults;
		PerformTests<Profile, TStaticSpatialIndexList<int32, FStaticSpatialIndex::TNodeSorterNoSort<Profile>, Profile>>(Test, TEXT("TStaticSpatialIndexList"), Elements, Tests, ListResults);
		ListResults.Sort();

		PerformNoSortTests<Profile>(Test, Elements, Tests, ListResults);

		PerformMinXTests<Profile>(Test, Elements, Tests, ListResults);

		PerformMortonTests<Profile, 4096>(Test, Elements, Tests, ListResults);
		PerformMortonTests<Profile, 16384>(Test, Elements, Tests, ListResults);
		PerformMortonTests<Profile, 65536>(Test, Elements, Tests, ListResults);
		PerformMortonTests<Profile, 262144>(Test, Elements, Tests, ListResults);

		PerformHilbertTests<Profile, 4096>(Test, Elements, Tests, ListResults);
		PerformHilbertTests<Profile, 16384>(Test, Elements, Tests, ListResults);
		PerformHilbertTests<Profile, 65536>(Test, Elements, Tests, ListResults);
		PerformHilbertTests<Profile, 262144>(Test, Elements, Tests, ListResults);
	}
#endif

	bool FWorldPartitionStaticSpatialIndexTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		const int32 NumBoxes = 100000;
		const int32 NumTests = 10000;

		TArray<TPair<FBox, int32>> Elements;
		TArray<TPair<FBox2D, int32>> Elements2D;
		Elements.Reserve(NumBoxes);
		Elements2D.Reserve(NumBoxes);
		for (int32 i=0; i<NumBoxes; i++)
		{
			const FSphere Sphere(FMath::VRand() * 10000000, FMath::RandRange(1, 100000));
			const FBox ElementBox(FBoxSphereBounds(Sphere).GetBox());
			Elements.Emplace(ElementBox, i);
			Elements2D.Emplace(FBox2D(FVector2D(ElementBox.Min), FVector2D(ElementBox.Max)), i);
		}

		TArray<FSphere> Tests;
		Tests.Reserve(NumTests);
		for (int32 i=0; i<NumTests; i++)
		{
			const FSphere Sphere(FMath::VRand() * 10000000, FMath::RandRange(1, 100000));
			Tests.Add(Sphere);
		}

		PerformTests<FStaticSpatialIndex::FSpatialIndexProfile3D>(this, Elements, Tests);
		PerformTests<FStaticSpatialIndex::FSpatialIndexProfile2D>(this, Elements2D, Tests);
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif