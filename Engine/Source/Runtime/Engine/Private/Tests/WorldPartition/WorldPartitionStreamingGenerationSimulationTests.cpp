// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionUtils.h"
#include "EditorWorldUtils.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionStreamingGenerationSimulationTest, TEST_NAME_ROOT ".StreamingGenerationSimulation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FWorldPartitionStreamingGenerationSimulationTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		ON_SCOPE_EXIT
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		};

		FScopedEditorWorld ScopedEditorWorld(
			TEXTVIEW("/Engine/WorldPartition/UnitTests/WPUnitTests"),
			UWorld::InitializationValues()
				.RequiresHitProxies(false)
				.ShouldSimulatePhysics(false)
				.EnableTraceCollision(false)
				.CreateNavigation(false)
				.CreateAISystem(false)
				.AllowAudioPlayback(false)
				.CreatePhysicsScene(true)
		);
		
		UWorld* World = ScopedEditorWorld.GetWorld();
		if (!TestNotNull(TEXT("Missing World Object"), World))
		{
			return false;
		}

		FWorldPartitionUtils::FSimulateCookedSession SimulateCookedSession(World);		
		if (!TestTrue(TEXT("Simulate Cooking"), SimulateCookedSession.IsValid()))
		{
			return false;
		}

		FWorldPartitionStreamingQuerySource QuerySource;
		QuerySource.bSpatialQuery = true;
		QuerySource.Radius = HALF_WORLD_MAX;

		TArray<const IWorldPartitionCell*> Cells;
		if (!TestTrue(TEXT("GetIntersectingCells"), SimulateCookedSession.GetIntersectingCells({ QuerySource }, Cells)))
		{
			return false;
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif