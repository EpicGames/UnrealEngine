// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "EditorWorldUtils.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "AssetToolsModule.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS

#define TEST_NAME_ROOT "System.Engine.WorldPartition"

namespace WorldPartitionTests
{
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FWorldPartitionStreamingGenerationTest, TEST_NAME_ROOT ".StreamingGeneration", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool FWorldPartitionStreamingGenerationTest::RunTest(const FString& Parameters)
	{
#if WITH_EDITOR
		ON_SCOPE_EXIT
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		};

		FScopedEditorWorld ScopedEditorWorld(
			TEXTVIEW("/Engine/WorldPartition/UnitTests/WPUnitTests"),
			UWorld::InitializationValues()
				.RequiresHitProxies(false)
				.ShouldSimulatePhysics(false)
				.EnableTraceCollision(false)
				.CreateNavigation(false)
				.CreateAISystem(false)
				.AllowAudioPlayback(false)
				.CreatePhysicsScene(true)
		);
		
		UWorld* World = ScopedEditorWorld.GetWorld();
		if (!TestNotNull(TEXT("Missing World Object"), World))
		{
			return false;
		}

		check(World);
		UWorldPartition* WorldPartition = World->GetWorldPartition();
		if (!TestNotNull(TEXT("Missing World Partition Object"), WorldPartition))
		{
			return false;
		}

		UWorldPartition::FGenerateStreamingParams Params;
		UWorldPartition::FGenerateStreamingContext Context;
		WorldPartition->GenerateStreaming(Params, Context);

		if (!TestTrue(TEXT("Missing Generated Output Log Path"), Context.OutputLogFilename.IsSet()))
		{
			return false;
		}

		// Read reference output log
		FString ReferenceOuputLog;
		const FString ReferenceOutputLogPath(FPaths::GetPath(World->GetPackage()->GetLoadedPath().GetLocalFullPath()) / TEXT("WPUnitTests.log"));
		if (!TestTrue(TEXT("Error Reading Reference Output Log File"), FFileHelper::LoadFileToString(ReferenceOuputLog, *ReferenceOutputLogPath)))
		{
			return false;
		}

		// Read generated output log
		FString GeneratedOuputLog;
		const FString GeneratedOutputLogPath(*Context.OutputLogFilename);
		if (!TestTrue(TEXT("Error Reading Generated Output Log File"), FFileHelper::LoadFileToString(GeneratedOuputLog, *GeneratedOutputLogPath)))
		{
			return false;
		}

		if (!TestTrue(TEXT("Generated Output Log Don't Match Reference Output Log"), GeneratedOuputLog == ReferenceOuputLog))
		{
			if (!FParse::Param(FCommandLine::Get(), TEXT("unattended")))
			{
				FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;
				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateDiffProcess(DiffCommand, ReferenceOutputLogPath, GeneratedOutputLogPath);
			}
			return false;
		}
#endif
		return true;
	}
}

#undef TEST_NAME_ROOT

#endif