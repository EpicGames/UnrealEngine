// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2D.cpp: Implementation of UTexture2D.
=============================================================================*/

#include "Engine/Texture2D.h"

#include "Algo/AnyOf.h"
#include "AsyncCompilationHelpers.h"
#include "Containers/ResourceArray.h"
#include "EngineLogs.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "RenderUtils.h"
#include "RenderGraphBuilder.h"
#include "EngineUtils.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DerivedDataCache.h"
#include "Math/GuardedInt.h"
#include "Rendering/Texture2DResource.h"
#include "Streaming/Texture2DStreamOut_AsyncCreate.h"
#include "RenderingThread.h"
#include "Streaming/Texture2DStreamOut_AsyncReallocate.h"
#include "Streaming/Texture2DStreamOut_Virtual.h"
#include "Streaming/Texture2DStreamIn_DDC_AsyncCreate.h"
#include "Streaming/Texture2DStreamIn_DDC_AsyncReallocate.h"
#include "Streaming/Texture2DStreamIn_DerivedData.h"
#include "Streaming/Texture2DStreamIn_IO_AsyncCreate.h"
#include "Streaming/Texture2DStreamIn_IO_AsyncReallocate.h"
#include "Streaming/Texture2DStreamIn_IO_Virtual.h"
// Generic path
#include "Streaming/TextureStreamIn.h"
#include "Streaming/Texture2DMipAllocator_AsyncCreate.h"
#include "Streaming/Texture2DMipAllocator_AsyncReallocate.h"
#include "Streaming/Texture2DMipAllocator_Virtual.h"
#include "Streaming/Texture2DMipDataProvider_DDC.h"
#include "Streaming/Texture2DMipDataProvider_IO.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "EngineModule.h"
#include "VT/UploadingVirtualTexture.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VT/VirtualTextureScalability.h"
#include "ImageUtils.h"
#include "TextureCompiler.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/StrongObjectPtr.h"
#include "VirtualTextureEnum.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

#if WITH_EDITOR
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "Interfaces/ITargetPlatform.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Texture2D)

#define LOCTEXT_NAMESPACE "UTexture2D"

UTexture2D::UTexture2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PrivatePlatformData(nullptr)
	, ResourceMem(nullptr)
{
	PendingUpdate = nullptr;
	SRGB = true;

	// AddressX is default-constructed by Uproperty to enum value 0 which is TA_Wrap
	check( AddressX == TA_Wrap );
}

/*-----------------------------------------------------------------------------
	Global helper functions
-----------------------------------------------------------------------------*/

/** CVars */
static TAutoConsoleVariable<float> CVarSetMipMapLODBias(
	TEXT("r.MipMapLODBias"),
	0.0f,
	TEXT("Apply additional mip map bias for all 2D textures, range of -15.0 to 15.0"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

TAutoConsoleVariable<int32> CVarFlushRHIThreadOnSTreamingTextureLocks(
	TEXT("r.FlushRHIThreadOnSTreamingTextureLocks"),
	0,
	TEXT("If set to 0, we won't do any flushes for streaming textures. This is safe because the texture streamer deals with these hazards explicitly."),
	ECVF_RenderThreadSafe);
static TAutoConsoleVariable<int32> CVarMobileReduceLoadedMips(
	TEXT("r.MobileReduceLoadedMips"),
	0,
	TEXT("Reduce loaded texture mipmaps for nonstreaming mobile platforms.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileMaxLoadedMips(
	TEXT("r.MobileMaxLoadedMips"),
	MAX_TEXTURE_MIP_COUNT,
	TEXT("Maximum number of loaded mips for nonstreaming mobile platforms.\n"),
	ECVF_RenderThreadSafe);


int32 GUseGenericStreamingPath = 0;
static FAutoConsoleVariableRef CVarUseGenericStreamingPath(
	TEXT("r.Streaming.UseGenericStreamingPath"),
	GUseGenericStreamingPath,
	TEXT("Control when to use the mip data provider implementation: (default=0)\n")
	TEXT("0 to use it when there is a custom asset override.\n")
	TEXT("1 to always use it.\n")
	TEXT("2 to never use it."),
	ECVF_Default
);

static int32 MobileReduceLoadedMips(int32 NumTotalMips)
{
	// apply cvar options to reduce the number of mips created at runtime
	// note they are still cooked  &shipped
	int32 NumReduceMips = FMath::Max(0, CVarMobileReduceLoadedMips.GetValueOnAnyThread());
	int32 MaxLoadedMips = FMath::Clamp(CVarMobileMaxLoadedMips.GetValueOnAnyThread(), 1, GMaxTextureMipCount);

	int32 NumMips = NumTotalMips;
	// Reduce number of mips as requested
	NumMips = FMath::Max(NumMips - NumReduceMips, 1);
	// Clamp number of mips as requested
	NumMips = FMath::Min(NumMips, MaxLoadedMips);
	
	return NumMips;
}

/** Number of times to retry to reallocate a texture before trying a panic defragmentation, the first time. */
int32 GDefragmentationRetryCounter = 10;
/** Number of times to retry to reallocate a texture before trying a panic defragmentation, subsequent times. */
int32 GDefragmentationRetryCounterLong = 100;

struct FStreamingRenderAsset;
struct FRenderAssetStreamingManager;

/** Turn on ENABLE_RENDER_ASSET_TRACKING in ContentStreaming.cpp and setup GTrackedTextures to track specific textures/meshes through the streaming system. */
extern bool TrackRenderAssetEvent( FStreamingRenderAsset* StreamingTexture, UStreamableRenderAsset* Texture, bool bForceMipLevelsToBeResident, const FRenderAssetStreamingManager* Manager );

/*-----------------------------------------------------------------------------
	FTexture2DMipMap
-----------------------------------------------------------------------------*/

#if !WITH_EDITORONLY_DATA
static_assert(sizeof(FTexture2DMipMap) <= 80, "FTexture2DMipMap was packed to reduce its memory footprint and fit into an 80 bytes bin of MallocBinned2/3");
#endif

void FTexture2DMipMap::Serialize(FArchive& Ar, UObject* Owner, int32 MipIdx, bool bSerializeMipData)
{
	if (bSerializeMipData)
	{
#if WITH_EDITORONLY_DATA
		BulkData.Serialize(Ar, Owner, MipIdx, false, FileRegionType);
#else
		BulkData.Serialize(Ar, Owner, MipIdx, false);
#endif
	}
	else if (Ar.IsLoading())
	{
		// in case we're deserializing into an existing object, clear out BulkData
		BulkData.RemoveBulkData();
	}

	int32 XSize = this->SizeX;
	int32 YSize = this->SizeY;
	int32 ZSize = this->SizeZ;
	Ar << XSize;
	Ar << YSize;
	Ar << ZSize;
	this->SizeX = XSize;
	this->SizeY = YSize;
	this->SizeZ = ZSize;

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << FileRegionType;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		Ar << bPagedToDerivedData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}

	// Streaming mips are saved with a size of 0 because they are stored separately.
	// IsBulkDataLoaded() returns true for this empty bulk data. Remove the empty
	// bulk data to allow unloaded streaming mips to be detected.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	const bool bLocalPagedToDerivedData = bPagedToDerivedData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	if (BulkData.GetBulkDataSize() == 0 && bLocalPagedToDerivedData)
	{
		BulkData.RemoveBulkData();
	}
#endif // #if WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
int64 FTexture2DMipMap::StoreInDerivedDataCache(const FStringView InKey, const FStringView InName, const bool bInReplaceExisting)
{
	using namespace UE;
	using namespace UE::DerivedData;

	const int64 BulkDataSizeInBytes = BulkData.GetBulkDataSize();
	check(BulkDataSizeInBytes > 0);

	const FSharedString Name = InName;
	const FCacheKey Key = ConvertLegacyCacheKey(InKey);
	FValue Value = FValue::Compress(FSharedBuffer::MakeView(BulkData.Lock(LOCK_READ_ONLY), BulkDataSizeInBytes));
	BulkData.Unlock();

	FRequestOwner AsyncOwner(EPriority::Normal);
	const ECachePolicy Policy = bInReplaceExisting ? ECachePolicy::Store : ECachePolicy::Default;
	GetCache().PutValue({{Name, Key, MoveTemp(Value), Policy}}, AsyncOwner);
	AsyncOwner.KeepAlive();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bPagedToDerivedData = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	DerivedData = FDerivedData(Name, Key);
	BulkData.RemoveBulkData();
	return BulkDataSizeInBytes;
}
#endif // #if WITH_EDITORONLY_DATA

/*-----------------------------------------------------------------------------
	UTexture2D
-----------------------------------------------------------------------------*/

/**
 * Get the optimal placeholder to use during texture compilation
 */ 
#if WITH_EDITOR
static UTexture2D* GetDefaultTexture2D(const UTexture2D* Texture)
{
	static TStrongObjectPtr<UTexture2D> CheckerboardTexture;
	static TStrongObjectPtr<UTexture2D> WhiteTexture;
	static TStrongObjectPtr<UTexture2D> NormalMapTexture;
	static TStrongObjectPtr<UTexture2D> EmptyTexture;

	if (!NormalMapTexture.IsValid())
	{
		NormalMapTexture.Reset(FImageUtils::CreateCheckerboardTexture(FColor(128, 128, 255), FColor(128, 128, 255)));
	}

	if (!EmptyTexture.IsValid())
	{
		EmptyTexture.Reset(FImageUtils::CreateCheckerboardTexture(FColor(0, 0, 0, 0), FColor(0, 0, 0, 0)));
	}

	if (!CheckerboardTexture.IsValid())
	{
		CheckerboardTexture.Reset(FImageUtils::CreateCheckerboardTexture(FColor(200, 200, 200, 128), FColor(128, 128, 128, 128)));
	}

	if (!WhiteTexture.IsValid())
	{
		WhiteTexture.Reset(FImageUtils::CreateCheckerboardTexture(FColor(255, 255, 255), FColor(255, 255, 255)));
	}

	// Normal maps requires a special default value
	if (Texture->IsNormalMap())
	{
		return NormalMapTexture.Get();
	}

	// Disable masks and displacement effects until they are compiled
	// otherwise could cause major visual artefacts.
	if (Texture->LODGroup == TEXTUREGROUP_Terrain_Heightmap ||
		Texture->LODGroup == TEXTUREGROUP_Terrain_Weightmap ||
		Texture->CompressionSettings == TC_Masks ||
		Texture->CompressionSettings == TC_Displacementmap ||
		Texture->CompressionSettings == TC_VectorDisplacementmap)
	{
		return EmptyTexture.Get();
	}

	if (Texture->LODGroup == TEXTUREGROUP_Lightmap ||
		Texture->LODGroup == TEXTUREGROUP_Shadowmap ||
		Texture->LODGroup == TEXTUREGROUP_ColorLookupTable)
	{
		return WhiteTexture.Get();
	}

	// Anything that is not a basecolor will be effectively
	// removed during the compilation phase to reduce visual
	// artefacts to a minimum.
	if (Texture->SRGB == false ||
		Texture->CompressionSettings != TC_Default)
	{
		return EmptyTexture.Get();
	}

	return CheckerboardTexture.Get();
}
#endif

FTexturePlatformData** UTexture2D::GetRunningPlatformData()
{
	// @todo DC GetRunningPlatformData is fundamentally unsafe but almost unused... should we replace it with Get/SetRunningPlatformData directly in the base class
	return &PrivatePlatformData;
}

void UTexture2D::SetPlatformData(FTexturePlatformData* InPlatformData)
{
	if (PrivatePlatformData)
	{
		ReleaseResource();
		delete PrivatePlatformData;
	}
	PrivatePlatformData = InPlatformData;
}

// Any direct access to GetPlatformData() will stall until the structure
// is safe to use. It is advisable to replace those use case with
// async aware code to avoid stalls where possible.
const FTexturePlatformData* UTexture2D::GetPlatformData() const
{
#if WITH_EDITOR
	if (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTexture2D::GetPlatformDataStall);
		UE_LOGF(LogTexture, Log, "Call to GetPlatformData() is forcing a wait on data that is not yet ready.");

		FText Msg = FText::Format(LOCTEXT("WaitOnTextureCompilation", "Waiting on texture compilation {0} ..."), FText::FromString(GetName()));
		FScopedSlowTask Progress(1.f, Msg, true);
		Progress.MakeDialog(true);
		uint64 StartTime = FPlatformTime::Cycles64();
		PrivatePlatformData->FinishCache();
		AsyncCompilationHelpers::SaveStallStack(FPlatformTime::Cycles64() - StartTime);
	}
#endif

	return PrivatePlatformData;
}

FTexturePlatformData* UTexture2D::GetPlatformData()
{
	// For now, this is the same implementation as the const version.
	const UTexture2D* ConstThis = this;
	return const_cast<FTexturePlatformData*>(ConstThis->GetPlatformData());
}

// While compiling the platform data in editor, we will return the 
// placeholders value to ensure rendering works as expected and that
// there are no thread-unsafe access to the platform data being built.
// Any process requiring a fully up-to-date platform data is expected to
// call FTextureCompilingManager:Get().FinishCompilation on UTexture first.
int32 UTexture2D::GetSizeX() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			// any calculation that actually uses this is garbage
			return GetDefaultTexture2D(this)->GetSizeX();
		}
#endif
		return PrivatePlatformData->SizeX;
	}
	return 0;
}

int32 UTexture2D::GetSizeY() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			// any calculation that actually uses this is garbage
			return GetDefaultTexture2D(this)->GetSizeY();
		}
#endif
		return PrivatePlatformData->SizeY;
	}
	return 0;
}

int32 UTexture2D::GetNumMips() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2D(this)->GetNumMips();
		}
#endif
		if (IsCurrentlyVirtualTextured())
		{
			return PrivatePlatformData->GetNumVTMips();
		}
		return PrivatePlatformData->Mips.Num();
	}
	return 0;
}

EPixelFormat UTexture2D::GetPixelFormat(uint32 LayerIndex) const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2D(this)->GetPixelFormat(LayerIndex);
		}
#endif
		return PrivatePlatformData->GetLayerPixelFormat(LayerIndex);
	}
	return PF_Unknown;
}

int32 UTexture2D::GetMipTailBaseIndex() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2D(this)->GetMipTailBaseIndex();
		}
#endif
		const int32 NumMipsInTail = GetPlatformData()->GetNumMipsInTail();
		return FMath::Max(0, NumMipsInTail > 0 ? (GetPlatformData()->Mips.Num() - NumMipsInTail) : (GetPlatformData()->Mips.Num() - 1));
	}
	return 0;
}

const TIndirectArray<FTexture2DMipMap>& UTexture2D::GetPlatformMips() const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTexture2D(this)->GetPlatformMips();
	}
#endif
	return PrivatePlatformData->Mips;
}

int32 UTexture2D::GetExtData() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2D(this)->GetExtData();
		}
#endif
		return PrivatePlatformData->GetExtData();
	}
	return 0;
}

bool UTexture2D::GetResourceMemSettings(int32 FirstMipIdx, int32& OutSizeX, int32& OutSizeY, int32& OutNumMips, uint32& OutTexCreateFlags)
{
	return false;
}

void UTexture2D::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::TextureMetaData);

	Super::Serialize(Ar);

	FStripDataFlags StripDataFlags(Ar);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (Ar.IsCooking() || bCooked)
	{
		bool bSerializeMipData = true;

		if (Ar.IsSaving())
		{
			// if there is an all mip provider, then we don't need to serialize the PlatformData mip data
			bSerializeMipData = (GetAllMipProvider() == nullptr);
		}

		// since the binary serialization format depends on this bool, we need to serialize it to know what format to expect at load time
		Ar << bSerializeMipData;
		
		SerializeCookedPlatformData(Ar, bSerializeMipData);
	}
}

int32 UTexture2D::GetNumResidentMips() const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTexture2D(this)->GetNumResidentMips();
	}
#endif

	if (GetResource())
	{
		if (IsCurrentlyVirtualTextured())
		{
			/*
			For VT this is obviously a bit abstract. We could return:
			- 0 -> No mips are guaranteed to be resident
			- Mips that are currently fully resident -> Not sure what the use of that would be
			- Mips that are currently partially resident
			- Mips that may be made resident by the VT system

			=> We currently return the last value as it seems to best fit use of this function throughout editor and engine, namely to query the actual
			in-game	resolution of the texture as it's currently loaded. An other option would be "Mips that are partially resident" as that would cover
			somewhat the same but knowing this is additional burden on the VT system and interfaces.
			*/
			return static_cast<const FVirtualTexture2DResource*>(GetResource())->GetNumMips();
		}
		else if (CachedSRRState.IsValid())
		{
			return CachedSRRState.NumResidentLODs;
		}
		else
		{
			return GetResource()->GetCurrentMipCount();
		}
	}
	return 0;
}

#if WITH_EDITOR

bool UTexture2D::IsDefaultTexture() const
{
	return (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete()) || (GetResource() && GetResource()->IsProxy());
}

void UTexture2D::PostEditUndo()
{
	FPropertyChangedEvent Undo(NULL);
	PostEditChangeProperty(Undo);
}

void UTexture2D::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTexture2D, AddressX)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTexture2D, AddressY)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTexture2D, VirtualTextureStreaming))
	{
		// VT need to recompile shaders when address mode changes
		// Non-VT still needs to potentially update sampler state in the materials
		NotifyMaterials();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

float UTexture2D::GetAverageBrightness(bool bIgnoreTrueBlack, bool bUseGrayscale)
{
	float AvgBrightness = -1.0f;
#if WITH_EDITOR

	// @todo Oodle : GetAverageColor is done in a few places ; factor out to an FImage helper?

	TArray64<uint8> RawData;
	// use the source art if it exists
	if (Source.IsValid() && Source.GetFormat() == TSF_BGRA8)
	{
		Source.GetMipData(RawData, 0);
	}
	else
	{
		UE_LOGF(LogTexture, Log, "No SourceArt available for %ls", *GetPathName());
	}

	if (RawData.Num() > 0)
	{
		int32 SizeX = Source.GetSizeX();
		int32 SizeY = Source.GetSizeY();
		double PixelSum = 0.0f;
		int32 Divisor = SizeX * SizeY;
		const FColor* ColorData = (const FColor*)RawData.GetData();
		for (int32 Y = 0; Y < SizeY; Y++)
		{
			for (int32 X = 0; X < SizeX; X++)
			{
				if ((ColorData->R == 0) && (ColorData->G == 0) && (ColorData->B == 0) && bIgnoreTrueBlack)
				{
					ColorData++;
					Divisor--;
					continue;
				}

				FLinearColor CurrentColor;
				if (SRGB == true)
				{
					CurrentColor = bUseLegacyGamma ? FLinearColor::FromPow22Color(*ColorData) : FLinearColor(*ColorData);
				}
				else
				{
					CurrentColor.R = float(ColorData->R) / 255.0f;
					CurrentColor.G = float(ColorData->G) / 255.0f;
					CurrentColor.B = float(ColorData->B) / 255.0f;
				}

				if (bUseGrayscale == true)
				{
					PixelSum += CurrentColor.R * 0.30f + CurrentColor.G * 0.59f + CurrentColor.B * 0.11f;
				}
				else
				{
					PixelSum += FMath::Max<float>(CurrentColor.R, FMath::Max<float>(CurrentColor.G, CurrentColor.B));
				}

				ColorData++;
			}
		}
		if (Divisor > 0)
		{
			AvgBrightness = PixelSum / Divisor;
		}
	}
#endif // #if WITH_EDITOR
	return AvgBrightness;
}

bool UTexture2D::IsReadyForAsyncPostLoad() const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return true;
	}
#endif

	return !PrivatePlatformData || PrivatePlatformData->IsReadyForAsyncPostLoad();
}

#if WITH_EDITOR
FIntPoint UTexture2D::GetImportedSize() const
{
	if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		return Source.GetLogicalSize();
	}
	return ImportedSize;
}
#endif // #if WITH_EDITOR

void UTexture2D::PostLoad()
{
#if WITH_EDITOR
	if (!GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		ImportedSize = Source.GetLogicalSize();
	}

	if (FApp::CanEverRender())
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
			BeginCachePlatformData();
		}
		else
		{
			FinishCachePlatformData();
		}
	}
#endif // #if WITH_EDITOR

	// Route postload, which will update bIsStreamable as UTexture::PostLoad calls UpdateResource.
	Super::PostLoad();
}

void UTexture2D::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITOR
	if( bTemporarilyDisableStreaming )
	{
		bTemporarilyDisableStreaming = false;
		UpdateResource();
	}

	// #TODO DC This is redundant code coming from UTexture::Presave that can be removed once we remove the above streaming code.

	// Ensure that compilation has finished before saving the package
	// otherwise async compilation might try to read the bulkdata
	// while it's being serialized to the package.
	if (IsCompiling())
	{
		FTextureCompilingManager::Get().FinishCompilation({ this });
	}
#endif
}

#if WITH_EDITOR
void UTexture2D::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		const ITargetPlatform* TargetPlatform = CookContext.GetTargetPlatform();
		check(TargetPlatform);
		CookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::SettingsObject(
			&TargetPlatform->GetTextureLODSettings()));
	}
}
#endif

void UTexture2D::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UTexture2D::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	FIntPoint SourceSize = GetImportedSize();

	const FString DimensionsStr = FString::Printf(TEXT("%dx%d"), SourceSize.X, SourceSize.Y);
	Context.AddTag( FAssetRegistryTag("Dimensions", DimensionsStr, FAssetRegistryTag::TT_Dimensional) );
	
	// This "Has Alpha Channel" is whether the GPU format can represent alpha in the format (eg. is it DXT1 vs DXT5)
	//	it does not tell you if the texture actually has non-opaque alpha
	Context.AddTag( FAssetRegistryTag("HasAlphaChannel", HasAlphaChannel() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical) );
	Context.AddTag( FAssetRegistryTag("Format", GPixelFormats[GetPixelFormat()].Name, FAssetRegistryTag::TT_Alphabetical) );

	Super::GetAssetRegistryTags(Context);
}

#if WITH_EDITOR
void UTexture2D::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Add("Dimensions",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UTexture2D", "Dimensions_Label", "Dimensions"))
		);

	OutMetadata.Add("HasAlphaChannel",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UTexture2D", "HasAlphaChannel_Label", "Has Alpha Channel"))
		);

	OutMetadata.Add("Format",
		FAssetRegistryTagMetadata()
			.SetDisplayName(NSLOCTEXT("UTexture2D", "Format_Label", "Format"))
		);
}
#endif

void UTexture2D::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture2D::UpdateResource);

	WaitForPendingInitOrStreaming();

#if WITH_EDITOR
	// Invalidate the CPU texture in case we changed sources - it'll recreate
	// on access if needed.
	CPUCopyTexture = nullptr;

	bool bAsyncFlags = false;
	// Recache platform data if the source has changed.
	if (!EnumHasAnyFlags(InFlags, EUpdateResourceFlags::Synchronous) &&
		FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		bAsyncFlags = true;
	}

	CachePlatformData(bAsyncFlags, bAsyncFlags, bAsyncFlags, nullptr, EnumHasAnyFlags(InFlags, EUpdateResourceFlags::ForceRebuild));
	
	// clear all the cooked cached platform data if the source could have changed... 
	ClearAllCachedCookedPlatformData();
#else
	// Note that using TF_FirstMip disables texture streaming, because the mip data becomes lost.
	// Also, the cleanup of the platform data must go between UpdateCachedLODBias() and UpdateResource().
	const bool bLoadOnlyFirstMip = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetMipLoadOptions(this) == ETextureMipLoadOptions::OnlyFirstMip;
	if (bLoadOnlyFirstMip && GetPlatformData() && GetPlatformData()->Mips.Num() > 0 && FPlatformProperties::RequiresCookedData())
	{
		// If we aren't pow2 there are failure modes here on some platforms that will cause crashes if we
		// strip tail mips without updating padding.
		// If you are using this to get around that fact that LODBias doesn't work with NoMipMaps, see the Downscale property.
		FTexturePlatformData* PlatformData = GetPlatformData();
		bool bIsPow2 = FMath::IsPowerOfTwo(PlatformData->Mips[0].SizeX) && FMath::IsPowerOfTwo(PlatformData->Mips[0].SizeY);
		if (bIsPow2)
		{
			const int32 NumMipsInTail = PlatformData->GetNumMipsInTail();
			const int32 MipTailBaseIndex = FMath::Max(0, NumMipsInTail > 0 ? (PlatformData->Mips.Num() - NumMipsInTail) : (PlatformData->Mips.Num() - 1));

			const int32 FirstMip = FMath::Min(FMath::Max(0, GetCachedLODBias()), MipTailBaseIndex);
			if (FirstMip < MipTailBaseIndex)
			{
				// Remove any mips after the first mip.
				PlatformData->Mips.RemoveAt(FirstMip + 1, PlatformData->Mips.Num() - FirstMip - 1);
				PlatformData->OptData.NumMipsInTail = 0;
			}
			// Remove any mips before the first mip.
			PlatformData->Mips.RemoveAt(0, FirstMip);
			// Update the texture size for the memory usage metrics.
			PlatformData->SizeX = PlatformData->Mips[0].SizeX;
			PlatformData->SizeY = PlatformData->Mips[0].SizeY;
		}
	}
#endif // #if WITH_EDITOR

	// Route to super.
	Super::UpdateResourceWithParams(InFlags);
}


#if WITH_EDITOR
void UTexture2D::PostLinkerChange()
{
	// Changing the linker requires re-creating the resource to make sure streaming behavior is right.
	if( !HasAnyFlags( RF_BeginDestroyed | RF_NeedLoad | RF_NeedPostLoad ) && !IsUnreachable() && !HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading) )
	{
		// Update the resource. We have to complete any builds beforehand.
		BlockOnAnyAsyncBuild();
		UpdateResource();
	}
}
#endif

void UTexture2D::BeginDestroy()
{
	// Route BeginDestroy.
	Super::BeginDestroy();

	TrackRenderAssetEvent(nullptr, this, false, 0);
}

FString UTexture2D::GetDesc()
{
	const int32 MaxResMipBias = GetNumMips() - GetNumMipsAllowed(false);
	return FString::Printf( TEXT("%s %dx%d [%s]"), 
		VirtualTextureStreaming ? TEXT("Virtual") : (NeverStream ? TEXT("NeverStreamed") : TEXT("Streamed")), 
		FMath::Max<int32>(GetSizeX() >> MaxResMipBias, 1), 
		FMath::Max<int32>(GetSizeY() >> MaxResMipBias, 1), 
		GPixelFormats[GetPixelFormat()].Name
		);
}

int32 UTexture2D::CalcTextureMemorySize( int32 MipCount ) const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTexture2D(this)->CalcTextureMemorySize(MipCount);
	}
#endif

	int32 Size = 0;
	if (FTexturePlatformData const* PlatformDataPtr = GetPlatformData())
	{
		if (FVirtualTextureBuiltData const* VTData = PlatformDataPtr->VTData)
		{
			const uint32 TileSize = VTData->GetPhysicalTileSize();
			const uint32 LayerCount = VTData->GetNumLayers();
			
			uint32 TileMemorySize = 0;				
			for (uint32 LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
			{
				const EPixelFormat Format = VTData->LayerTypes[LayerIndex];
				TileMemorySize += CalculateImageBytes(TileSize, TileSize, 1, Format);
			}

			// Get number of tiles in all mips up to and including this mip level.
			// Stop iteration when we get down to one tile since this tile is shared by the mip tail.
			const int32 MipLevel = FMath::Max<int32>(VTData->GetNumMips() - MipCount, 0);
			uint32 TileCountX = FMath::Max(VTData->GetWidthInTiles() >> MipLevel, 1u);
			uint32 TileCountY = FMath::Max(VTData->GetHeightInTiles() >> MipLevel, 1u);
			uint32 TileCount = TileCountX * TileCountY;

			while (TileCountX > 1 && TileCountY > 1)
			{
				TileCountX = FMath::DivideAndRoundUp(TileCountX, 2u);
				TileCountY = FMath::DivideAndRoundUp(TileCountY, 2u);
				TileCount += TileCountX * TileCountY;
			}

			Size = TileCount * TileMemorySize;
		}
		else
		{
			static TConsoleVariableData<int32>* CVarReducedMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PRT.ReducedMemory"));
			check(CVarReducedMode);

			ETextureCreateFlags TexCreateFlags = (SRGB ? TexCreate_SRGB : TexCreate_None) | (bNoTiling ? TexCreate_NoTiling : TexCreate_None) | (bNotOfflineProcessed ? TexCreate_None : TexCreate_OfflineProcessed) | TexCreate_Streamable;
			const bool bCanUsePartiallyResidentMips = CanCreateWithPartiallyResidentMips(TexCreateFlags);

			const int32 SizeX = GetSizeX();
			const int32 SizeY = GetSizeY();
			const int32 NumMips = GetNumMips();
			const int32 FirstMip = FMath::Max(0, NumMips - MipCount);
			const EPixelFormat Format = GetPixelFormat();

			// Must be consistent with the logic in FTexture2DResource::InitRHI
			if (IsStreamable() && bCanUsePartiallyResidentMips && (!CVarReducedMode->GetValueOnAnyThread() || MipCount > UTexture2D::GetMinTextureResidentMipCount()))
			{
				const FRHITextureDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("Temp"), SizeX, SizeY, Format)
					.SetNumMips(NumMips)
					.SetFlags(TexCreateFlags | TexCreate_Virtual);

				Size = RHICalcTexturePlatformSize(Desc, FirstMip).Size;
			}
			else
			{
				const FIntPoint MipExtents = CalcMipMapExtent(SizeX, SizeY, Format, FirstMip);

				const FRHITextureDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("Temp"), MipExtents, Format)
					.SetNumMips(FMath::Max(1, MipCount))
					.SetFlags(TexCreateFlags)
					.SetExtData(GetPlatformData()->GetExtData());

				Size = RHICalcTexturePlatformSize(Desc).Size;
			}
		}
	}
	return Size;
}

int32 UTexture2D::GetNumMipsAllowed(bool bIgnoreMinResidency) const
{
	// this function is trying to get the number of mips that will be in the texture after cooking
	//	(eg. after "drop mip" lod bias is applied)
	// but it doesn't exactly replicate the behavior of Serialize
	// it's also similar to Texture::GetResourcePostInitState but not the same
	// yay

	const int32 NumMips = GetNumMips();

	// Compute the number of mips that will be available after cooking, as some mips get cooked out.
	// See the logic around FirstMipToSerialize in TextureDerivedData.cpp, SerializePlatformData().
	const int32 LODBiasNoCinematics = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->CalculateLODBias(this, false);
	const int32 CookedMips = FMath::Clamp<int32>(NumMips - LODBiasNoCinematics, 1, GMaxTextureMipCount);
	const int32 MinResidentMipCount = GetMinTextureResidentMipCount();

	// If the data is already cooked, then mips bellow min resident can't be stripped out.
	// This would happen if the data is cooked with some texture group settings, but launched
	// with other settings, adding more constraints on the cooked data.
	if (bIgnoreMinResidency && !FPlatformProperties::RequiresCookedData())
	{
		return CookedMips;
	}
	else if (NumMips > MinResidentMipCount)
	{
		// In non cooked, the engine can not partially load the resident mips.
		return FMath::Max<int32>(CookedMips, MinResidentMipCount);
	}
	else
	{
		return NumMips;
	}
}


uint32 UTexture2D::CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const
{
	if (IsCurrentlyVirtualTextured())
	{
		// Virtual textures "take no space". I.e. the space used by them (Caches translation tables, ...) should already be accounted for elsewhere.
		return 0;
	}

	if ( Enum == TMC_ResidentMips )
	{
		return CalcTextureMemorySize( GetNumResidentMips() );
	}
	else if( Enum == TMC_AllMipsBiased)
	{
		return CalcTextureMemorySize( GetNumMipsAllowed(false) );
	}
	else
	{
		return CalcTextureMemorySize( GetNumMips() );
	}
}


bool UTexture2D::GetSourceArtCRC(uint32& OutSourceCRC)
{
	bool bResult = false;
	TArray64<uint8> RawData;
#if WITH_EDITOR
	// use the source art if it exists
	if (Source.IsValid())
	{
		// Decompress source art.
		Source.GetMipData(RawData, 0);
	}
	else
	{
		UE_LOGF(LogTexture, Log, "No SourceArt available for %ls", *GetPathName());
	}

	if (RawData.Num() > 0)
	{
		OutSourceCRC = FCrc::MemCrc_DEPRECATED((void*)(RawData.GetData()), RawData.Num());
		bResult = true;
	}
#endif // #if WITH_EDITOR
	return bResult;
}

bool UTexture2D::HasSameSourceArt(UTexture2D* InTexture)
{
	bool bResult = false;

#if WITH_EDITOR

	if ( ! Source.IsValid() || ! InTexture->Source.IsValid() )
	{
		return false;
	}

	TArray64<uint8> RawData1;
	TArray64<uint8> RawData2;
	int32 SizeX = 0;
	int32 SizeY = 0;

	// Need to handle UDIM here?
	if ((Source.GetSizeX() == InTexture->Source.GetSizeX()) && 
		(Source.GetSizeY() == InTexture->Source.GetSizeY()) &&
		(Source.GetNumMips() == InTexture->Source.GetNumMips()) &&
		(Source.GetNumMips() == 1) &&
		(Source.GetFormat() == InTexture->Source.GetFormat()) &&
		(SRGB == InTexture->SRGB))
	{
		Source.GetMipData(RawData1, 0);
		InTexture->Source.GetMipData(RawData2, 0);
	}

	if ((RawData1.Num() > 0) && (RawData1.Num() == RawData2.Num()))
	{
		if (RawData1 == RawData2)
		{
			bResult = true;
		}
	}
#endif // #if WITH_EDITOR

	return bResult;
}

bool UTexture2D::HasAlphaChannel() const
{
	// This "Has Alpha Channel" is whether the GPU format can represent alpha in the format (eg. is it DXT1 vs DXT5)
	//	it does not tell you if the texture actually has non-opaque alpha

	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTexture2D(this)->HasAlphaChannel();
		}
#endif
		return EnumHasAnyFlags(GetPixelFormatValidChannels(PrivatePlatformData->PixelFormat), EPixelFormatChannelFlags::A);
	}
	return false;
}

FTextureResource* UTexture2D::CreateResource()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture2D::CreateResource)

#if WITH_EDITOR
	if (PrivatePlatformData)
	{
		if (PrivatePlatformData->IsAsyncWorkComplete())
		{
			// Make sure AsyncData has been destroyed in case it still exists to avoid
			// IsDefaultTexture thinking platform data is still being computed.
			PrivatePlatformData->FinishCache();
		}
		else
		{
			FTextureCompilingManager::Get().AddTextures({ this });

			UnlinkStreaming();
			return new FTexture2DResource(this, GetDefaultTexture2D(this)->GetResource()->GetTexture2DResource());
		}
	}
#endif

	if (PrivatePlatformData)
	{
		if (IsCurrentlyVirtualTextured())
		{
			FVirtualTexture2DResource* ResourceVT = new FVirtualTexture2DResource(this, PrivatePlatformData->VTData, GetCachedLODBias());
			return ResourceVT;
		}
		else
		{
			const EPixelFormat PixelFormat = GetPixelFormat();

			int32 NumMips = FMath::Min3<int32>(PrivatePlatformData->Mips.Num(), GMaxTextureMipCount, FStreamableRenderResourceState::MAX_LOD_COUNT);
#if !PLATFORM_SUPPORTS_TEXTURE_STREAMING // eg, Android
			NumMips = MobileReduceLoadedMips(NumMips);
#endif
	
			if (!NumMips)
			{
#if WITH_EDITOR
				bool bIsServerCookedPackage = false;
				if (UPackage* Package = GetPackage())
				{
					if ((Package->GetPackageFlags() & PKG_Cooked) ||
						(Package->GetPackageFlags() & PKG_FilterEditorOnly))
					{
						// this is likely a cooked package for a server and the mip data is removed intentionally
						bIsServerCookedPackage = true;
					}
				}
				if (bIsServerCookedPackage)
				{
					// reduce this to a log because we don't need no errors in this case
					UE_LOGF(LogTexture, Log, "%ls contains no miplevels! Please delete. (Format: %d)", *GetFullName(), (int)PixelFormat);
				}
				else
				{
					UE_LOGF(LogTexture, Warning, "%ls contains no miplevels! This could happen if this texture is a thumbnail and hasn't been generated (Format: %d)", *GetFullName(), (int)PixelFormat);
				}
#else
				UE_LOGF(LogTexture, Error, "%ls contains no miplevels! Please delete. (Format: %d)", *GetFullName(), (int)PixelFormat);
#endif
				
			}
			else if (!GPixelFormats[PixelFormat].Supported)
			{
				UE_LOGF(LogTexture, Error, "%ls is %ls [raw type %d] which is not supported.", *GetFullName(), GPixelFormats[PixelFormat].Name, static_cast<int32>(PixelFormat));
			}
			else if (NumMips == 1 && FMath::Max(GetSizeX(), GetSizeY()) > (int32)GetMax2DTextureDimension())
			{
				UE_LOGF(LogTexture, Warning, "%ls cannot be created, exceeds this rhi's maximum dimension (%d) and has no mip chain to fall back on.", *GetFullName(), GetMax2DTextureDimension());
			}
			else
			{
				// Should be as big as the mips we have already directly loaded into GPU mem
				FStreamableRenderResourceState PostInitState;
				if (UTextureAllMipDataProviderFactory* ProviderFactory = GetAllMipProvider())
				{
					// All Mip Providers get to control the initial streaming state
					PostInitState = ProviderFactory->GetResourcePostInitState(this, !bTemporarilyDisableStreaming);
				}
				else
				{
					PostInitState = GetResourcePostInitState(GetPlatformData(), !bTemporarilyDisableStreaming, ResourceMem ? ResourceMem->GetNumMips() : 0, NumMips);
				}

				FTexture2DResource* Texture2DResource = new FTexture2DResource(this, PostInitState);
				// preallocated memory for the UTexture2D resource is now owned by this resource
				// and will be freed by the RHI resource or when the FTexture2DResource is deleted
				ResourceMem = nullptr;

				return Texture2DResource;
			}
		}
	}

	return nullptr;
}

EMaterialValueType UTexture2D::GetMaterialType() const
{
	if (VirtualTextureStreaming)
	{
		return MCT_TextureVirtual;
	}
	return MCT_Texture2D;
}

void UTexture2D::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (IsCurrentlyVirtualTextured())
	{
		CumulativeResourceSize.AddUnknownMemoryBytes(GetPlatformData()->VTData->GetMemoryFootprint());
	}
	else
	{
		if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::Exclusive)
		{
			// Use only loaded mips
			CumulativeResourceSize.AddDedicatedVideoMemoryBytes(CalcTextureMemorySize(GetNumResidentMips()));
		}
		else
		{
			// Use all possible mips
			CumulativeResourceSize.AddDedicatedVideoMemoryBytes(CalcTextureMemorySize(GetNumMipsAllowed(true)));
		}
	}
}


UTexture2D* UTexture2D::CreateTransientFromImage(const FImage* InImage, const FName InName)
{
	LLM_SCOPE(ELLMTag::Textures);
	if (InImage == nullptr)
	{
		return nullptr;
	}

	TArray64<uint8> ConvertedData;

	EPixelFormat Format = PF_B8G8R8A8;
	switch (InImage->Format)
	{
	case ERawImageFormat::BGRE8:
		{
			Format = PF_A32B32G32R32F; 
			ConvertedData.AddUninitialized(InImage->SizeX * InImage->SizeY * sizeof(FLinearColor));
			FLinearColor* Output = (FLinearColor*)ConvertedData.GetData();
			const FColor* Input = (const FColor*)InImage->RawData.GetData();

			uint64 Pixels = (uint64)InImage->SizeX * InImage->SizeY;
			for (uint64 Pixel = 0; Pixel < Pixels; Pixel++)
			{
				Output[Pixel] = Input[Pixel].FromRGBE();
			}
			break;
		}
	case ERawImageFormat::BGRA8: Format = PF_B8G8R8A8; break;
	case ERawImageFormat::RGBA16F: Format = PF_FloatRGBA; break;
	case ERawImageFormat::RGBA32F: Format = PF_A32B32G32R32F; break;
	case ERawImageFormat::R16F: Format = PF_R16F; break;
	case ERawImageFormat::R32F: Format = PF_R32_FLOAT; break;
	case ERawImageFormat::G8:
		{
			Format = PF_B8G8R8A8;
			ConvertedData.AddUninitialized(InImage->SizeX * InImage->SizeY * sizeof(FColor));
			FColor* Output = (FColor*)ConvertedData.GetData();
			const uint8* Input = (const uint8*)InImage->RawData.GetData();
			
			uint64 Pixels = (uint64)InImage->SizeX * InImage->SizeY;
			for (uint64 Pixel = 0; Pixel < Pixels; Pixel++)
			{
				Output[Pixel].R = Output[Pixel].G = Output[Pixel].B = Input[Pixel];
				Output[Pixel].A = 255;
			}

			break;
		}
	case ERawImageFormat::G16:
		{
			Format = PF_A16B16G16R16;
			ConvertedData.AddUninitialized(InImage->SizeX * InImage->SizeY * 4 * sizeof(uint16));
			uint16* Output = (uint16*)ConvertedData.GetData();
			const uint16* Input = (const uint16*)InImage->RawData.GetData();

			uint64 Pixels = (uint64)InImage->SizeX * InImage->SizeY;
			for (uint64 Pixel = 0; Pixel < Pixels; Pixel++)
			{
				Output[Pixel * 4 + 3] = 65535;
				Output[Pixel * 4 + 2] = Input[Pixel];
				Output[Pixel * 4 + 1] = Input[Pixel];
				Output[Pixel * 4 + 0] = Input[Pixel];
			}

			break;
		}
	case ERawImageFormat::RGBA16:
		{
			Format = PF_A16B16G16R16;
			break;
		}
	default:
		{
			UE_LOGF(LogTexture, Error, "Invalid raw image format in UTexture2D::CreateTransientFromImage()");
			return nullptr;
		}
	}

	// We only want to provide one slice if the source is for an array/cube
	TConstArrayView64<uint8> ImageData = InImage->RawData;
	if (InImage->NumSlices > 1)
	{
		int64 SliceBytes = ImageData.Num() / InImage->NumSlices;
		ImageData.LeftInline(SliceBytes);
	}

	return CreateTransient(InImage->GetWidth(), InImage->GetHeight(), Format, InName, ConvertedData.Num() ? ConvertedData : ImageData);
}


UTexture2D* UTexture2D::CreateTransient(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat, const FName InName, TConstArrayView64<uint8> InImageData)
{
	LLM_SCOPE(ELLMTag::Textures);

	const int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
	const int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;

	if (InSizeX <= 0 || InSizeY <= 0)
	{
		UE_LOGF(LogTexture, Warning, "Negative size specified for UTexture2D::CreateTransient()");
		return nullptr;
	}

	if ((InSizeX % GPixelFormats[InFormat].BlockSizeX) ||
		(InSizeY % GPixelFormats[InFormat].BlockSizeY))
	{
		UE_LOGF(LogTexture, Warning, "Size specified isn't valid for block-based pixel format in UTexture2D::CreateTransient()");
		return nullptr;
	}

	FGuardedInt64 BytesForImageValidation = FGuardedInt64(NumBlocksX) * NumBlocksY * GPixelFormats[InFormat].BlockBytes;
	if (BytesForImageValidation.IsValid() == false)
	{
		UE_LOGF(LogTexture, Warning, "Size specified overflows in UTexture2D::CreateTransient()");
		return nullptr;
	}

	int64 BytesForImage = BytesForImageValidation.Get(0);

	// If they provided data, it needs to be the right size.
	if (InImageData.Num() && InImageData.Num() != BytesForImage)
	{
		UE_LOGF(LogTexture, Warning, "Image data provided is incorrect size (%llu provided, %llu wanted) in UTexture2D::CreateTransient()", InImageData.Num(), BytesForImage);
		return nullptr;
	}

	UTexture2D* NewTexture = NewObject<UTexture2D>(
		GetTransientPackage(),
		InName,
		RF_Transient
		);

	NewTexture->SetPlatformData(new FTexturePlatformData());
	NewTexture->GetPlatformData()->SizeX = InSizeX;
	NewTexture->GetPlatformData()->SizeY = InSizeY;
	NewTexture->GetPlatformData()->SetNumSlices(1);
	NewTexture->GetPlatformData()->PixelFormat = InFormat;
	NewTexture->bNotOfflineProcessed = true;

	// Allocate first mipmap.
	FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY, 1);
	NewTexture->GetPlatformData()->Mips.Add(Mip);
	Mip->BulkData.Lock(LOCK_READ_WRITE);
	void* DestImageData = Mip->BulkData.Realloc(BytesForImage);
	if (InImageData.Num())
	{
		FMemory::Memcpy(DestImageData, InImageData.GetData(), BytesForImage);
	}
	Mip->BulkData.Unlock();
	if (InImageData.Num())
	{
		NewTexture->UpdateResource();
	}
	return NewTexture;
}

int32 UTexture2D::Blueprint_GetSizeX() const
{
#if WITH_EDITORONLY_DATA
	// When cooking, blueprint construction scripts are ran before textures get postloaded.
	// In that state, the texture size is 0. Here we compute the resolution once cooked.
	if (!GetSizeX())
	{
		//beware: this is wrong in a variety of ways
		//	MaxTextureSize, PadForPow2, Downscale, etc. are not applied

		const UTextureLODSettings* LODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
		const int32 CookedLODBias = LODSettings->CalculateLODBias(Source.SizeX, Source.SizeY, GetMinTextureSize(), GetMaxTextureSize(), LODGroup, LODBias, 0, MipGenSettings, IsCurrentlyVirtualTextured(), IsStreamable());
		return FMath::Max<int32>(Source.SizeX >> CookedLODBias, 1);
	}
#endif
	return GetSizeX();
}

#if WITH_EDITORONLY_DATA
UTexture2D* UTexture2D::GetCPUCopyTexture()
{
	if (CPUCopyTexture)
	{
		return CPUCopyTexture;
	}

	FSharedImageConstRef CPUCopy = GetCPUCopy(); // blocks in the editor during encoding
	if (CPUCopy)
	{
		CPUCopyTexture = UTexture2D::CreateTransientFromImage(CPUCopy.GetReference());
	}
	return CPUCopyTexture;
}
#endif


FSharedImageConstRef UTexture2D::GetCPUCopy() const
{
	// early out to avoid stalling when not a CPU texture
	if ( Availability != ETextureAvailability::CPU)
	{
		return FSharedImageConstRef();
	}

	// could stall if texture isn't built!
	const FTexturePlatformData* LocalPlatformData = GetPlatformData();

	if (LocalPlatformData && LocalPlatformData->GetHasCpuCopy())
	{
		return LocalPlatformData->CPUCopy;
	}

	return FSharedImageConstRef();

}

FSharedImageConstRefBlueprint UTexture2D::Blueprint_GetCPUCopy() const
{
	// When cooking, blueprint construction scripts are run before textures get postloaded.
	// We don't have valid platformdata, so we always have a null reference here which passes
	// back to the caller.
	FSharedImageConstRefBlueprint Result;
	Result.Reference = GetCPUCopy();
	return Result;
}

int32 UTexture2D::Blueprint_GetSizeY() const
{
#if WITH_EDITORONLY_DATA
	// When cooking, blueprint construction scripts are ran before textures get postloaded.
	// In that state, the texture size is 0. Here we compute the resolution once cooked.
	if (!GetSizeY())
	{
		const UTextureLODSettings* LODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
		const int32 CookedLODBias = LODSettings->CalculateLODBias(Source.SizeX, Source.SizeY, GetMinTextureSize(), GetMaxTextureSize(), LODGroup, LODBias, 0, MipGenSettings, IsCurrentlyVirtualTextured(), IsStreamable());
		return FMath::Max<int32>(Source.SizeY >> CookedLODBias, 1);
	}
#endif
	return GetSizeY();
}

void UTexture2D::UpdateTextureRegions(int32 MipIndex, uint32 NumRegions, const FUpdateTextureRegion2D* Regions, uint32 SrcPitch, uint32 SrcBpp, uint8* SrcData, TFunction<void(uint8* SrcData, const FUpdateTextureRegion2D* Regions)> DataCleanupFunc)
{
	if (IsCurrentlyVirtualTextured())
	{
		UE_LOGF(LogTexture, Log, "UpdateTextureRegions called for %ls which is virtual.", *GetPathName());
		return;
	}

	FTexture2DResource* Texture2DResource = GetResource() ? GetResource()->GetTexture2DResource() : nullptr;
	if (!bTemporarilyDisableStreaming && IsStreamable())
	{
		UE_LOGF(LogTexture, Log, "UpdateTextureRegions called for %ls without calling TemporarilyDisableStreaming", *GetPathName());
	}
	else if (Texture2DResource)
	{
		struct FUpdateTextureRegionsData
		{
			FTexture2DResource* Texture2DResource;
			int32 MipIndex;
			uint32 NumRegions;
			const FUpdateTextureRegion2D* Regions;
			uint32 SrcPitch;
			uint32 SrcBpp;
			uint8* SrcData;
		};

		FUpdateTextureRegionsData* RegionData = new FUpdateTextureRegionsData;

		RegionData->Texture2DResource = Texture2DResource;
		RegionData->MipIndex = MipIndex;
		RegionData->NumRegions = NumRegions;
		RegionData->Regions = Regions;
		RegionData->SrcPitch = SrcPitch;
		RegionData->SrcBpp = SrcBpp;
		RegionData->SrcData = SrcData;

		ENQUEUE_RENDER_COMMAND(UpdateTextureRegionsData)(
			[RegionData, DataCleanupFunc](FRHICommandListImmediate& RHICmdList)
			{
				for (uint32 RegionIndex = 0; RegionIndex < RegionData->NumRegions; ++RegionIndex)
				{
					int32 CurrentFirstMip = RegionData->Texture2DResource->State.AssetLODBias;
					if (RegionData->MipIndex >= CurrentFirstMip)
					{
						// Some RHIs don't support source offsets. Offset source data pointer now and clear source offsets
						FUpdateTextureRegion2D RegionCopy = RegionData->Regions[RegionIndex];
						const uint8* RegionSourceData = RegionData->SrcData
							+ RegionCopy.SrcY * RegionData->SrcPitch
							+ RegionCopy.SrcX * RegionData->SrcBpp;
						RegionCopy.SrcX = 0;
						RegionCopy.SrcY = 0;

						RHICmdList.UpdateTexture2D(
							RegionData->Texture2DResource->TextureRHI->GetTexture2D(),
							RegionData->MipIndex - CurrentFirstMip,
							RegionCopy,
							RegionData->SrcPitch,
							RegionSourceData);
					}
				}

				// The deletion of source data may need to be deferred to the RHI thread after the updates occur
				RHICmdList.EnqueueLambda([RegionData, DataCleanupFunc](FRHICommandList&)
				{
					DataCleanupFunc(RegionData->SrcData, RegionData->Regions);
					delete RegionData;
				});
			});
	}
}

#if WITH_EDITOR
void UTexture2D::TemporarilyDisableStreaming()
{
	BlockOnAnyAsyncBuild();
	if( !bTemporarilyDisableStreaming )
	{
		bTemporarilyDisableStreaming = true;
		UpdateResource();
	}
}
#endif

float UTexture2D::GetGlobalMipMapLODBias()
{
	float BiasOffset = CVarSetMipMapLODBias.GetValueOnAnyThread(); // called from multiple threads.
	return FMath::Clamp(BiasOffset, -15.0f, 15.0f);
}

void UTexture2D::RefreshSamplerStates()
{
	if (GetResource())
	{
		if (FTexture2DResource* Texture2DResource = GetResource()->GetTexture2DResource())
		{
			Texture2DResource->CacheSamplerStateInitializer(this);
			ENQUEUE_RENDER_COMMAND(RefreshSamplerStatesCommand)([Texture2DResource](FRHICommandList& RHICmdList)
			{
				Texture2DResource->RefreshSamplerStates();
			});
		}
	}
}

bool UTexture2D::IsCurrentlyVirtualTextured() const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return false;
	}
#endif

	if (VirtualTextureStreaming && GetPlatformData() && GetPlatformData()->VTData)
	{
		return true;
	}
	return false;
}

FVirtualTexture2DResource::FVirtualTexture2DResource()
{
	//NOTE: Empty constructor for use with media textures (which do not derive from UTexture2D).

	// Initialize this resource FeatureLevel, so it gets re-created on FeatureLevel changes
	SetFeatureLevel(GMaxRHIFeatureLevel);

	VirtualTextureStreamingPriority = EVTProducerPriority::Normal;
}

FVirtualTexture2DResource::FVirtualTexture2DResource(const UTexture2D* InOwner, FVirtualTextureBuiltData* InVTData, int32 InFirstMipToUse)
{
	check(InOwner);
	bSRGB = InOwner->SRGB;
	bGreyScaleFormat = UE::TextureDefines::ShouldUseGreyScaleEditorVisualization(InOwner->CompressionSettings);
	TextureReferenceRHI = InOwner->TextureReference.TextureReferenceRHI;

	TextureName = InOwner->GetFName();
	PackageName = InOwner->GetOutermost()->GetFName();
	FullNameHash = GetTypeHash(InOwner->GetName().ToLower());

	UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
	Filter = (ESamplerFilter)TextureLODSettings->GetSamplerFilter(InOwner);
	AddressU = InOwner->AddressX == TA_Wrap ? AM_Wrap : (InOwner->AddressX == TA_Clamp ? AM_Clamp : AM_Mirror);
	AddressV = InOwner->AddressY == TA_Wrap ? AM_Wrap : (InOwner->AddressY == TA_Clamp ? AM_Clamp : AM_Mirror);

	TexCreateFlags = InOwner->SRGB ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None;
	TexCreateFlags |= InOwner->bNotOfflineProcessed ? ETextureCreateFlags::None : ETextureCreateFlags::OfflineProcessed;
	TexCreateFlags |= InOwner->bNoTiling ? ETextureCreateFlags::NoTiling : ETextureCreateFlags::None;

	bSinglePhysicalSpace = InOwner->IsVirtualTexturedWithSinglePhysicalSpace();
	bRequiresSinglePhysicalPool = InOwner->IsVirtualTexturedWithSinglePhysicalPool();
	
	if (InOwner->bUseVirtualTextureStreamingPriority)
	{
		VirtualTextureStreamingPriority = InOwner->VirtualTextureStreamingPriority;
	}
	else
	{
		VirtualTextureStreamingPriority = TextureLODSettings->GetTextureLODGroup(InOwner->LODGroup).VirtualTextureStreamingPriority;
	}
	
	check(InVTData);
	VTData = InVTData;

	// Don't allow input mip bias to drop size below a single tile
	const uint32 SizeInTiles = FMath::Max(VTData->GetWidthInTiles(), VTData->GetHeightInTiles());
	const uint32 MaxMip = FMath::CeilLogTwo(SizeInTiles);
	FirstMipToUse = FMath::Min((int32)MaxMip, InFirstMipToUse);

	// Initialize this resource FeatureLevel, so it gets re-created on FeatureLevel changes
	SetFeatureLevel(GMaxRHIFeatureLevel);
}

FVirtualTexture2DResource::~FVirtualTexture2DResource()
{
}

void FVirtualTexture2DResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTexture2DResource::InitRHI);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(PackageName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), PackageName);

	uint32 MaxAnisotropy = 0;
	if (VirtualTextureScalability::IsAnisotropicFilteringEnabled())
	{
		// Limit HW MaxAnisotropy to avoid sampling outside VT borders
		MaxAnisotropy = FMath::Min<int32>(VirtualTextureScalability::GetMaxAnisotropy(), VTData->TileBorderSize);
	}

	// We always create a sampler state if we're attached to a texture. This is used to sample the cache texture during actual rendering and the miptails editor resource.
	// If we're not attached to a texture it likely means we're light maps which have sampling handled differently.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		// This will ensure nearest/linear/trilinear which does matter when sampling both the cache and the miptail
		Filter,

		// This doesn't really matter when sampling the cache texture but it does when sampling the miptail texture
		AddressU,
		AddressV,
		AM_Wrap,

		// This doesn't really matter when sampling the cache texture (as it only has a level 0, so whatever the bias that is sampled) but it does when we sample miptail texture
		0, // VT currently ignores global mip bias ensure the miptail works the same -> UTexture2D::GetGlobalMipMapLODBias()
		MaxAnisotropy
	);

	if (MaxAnisotropy == 0u)
	{
		if (SamplerStateInitializer.Filter == SF_AnisotropicLinear || SamplerStateInitializer.Filter == SF_AnisotropicPoint)
		{
			SamplerStateInitializer.Filter = SF_Bilinear;
		}
	}

	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

	const int32 MaxLevel = VTData->GetNumMips() - FirstMipToUse - 1;
	check(MaxLevel >= 0);

	FVTProducerDescription ProducerDesc;
	ProducerDesc.Name = TextureName;
	ProducerDesc.FullNameHash = FullNameHash;
	ProducerDesc.Dimensions = 2;
	ProducerDesc.TileSize = VTData->TileSize;
	ProducerDesc.TileBorderSize = VTData->TileBorderSize;
	ProducerDesc.BlockWidthInTiles = FMath::DivideAndRoundUp<uint32>(GetNumTilesX(), VTData->WidthInBlocks);
	ProducerDesc.BlockHeightInTiles = FMath::DivideAndRoundUp<uint32>(GetNumTilesY(), VTData->HeightInBlocks);
	ProducerDesc.WidthInBlocks = VTData->WidthInBlocks;
	ProducerDesc.HeightInBlocks = VTData->HeightInBlocks;
	ProducerDesc.DepthInTiles = 1u;
	ProducerDesc.MaxLevel = MaxLevel;
	ProducerDesc.NumTextureLayers = VTData->GetNumLayers();
	ProducerDesc.NumPhysicalGroups = bSinglePhysicalSpace ? 1 : VTData->GetNumLayers();
	for (uint32 LayerIndex = 0u; LayerIndex < VTData->GetNumLayers(); ++LayerIndex)
	{
		ProducerDesc.LayerFormat[LayerIndex] = VTData->LayerTypes[LayerIndex];
		ProducerDesc.LayerFallbackColor[LayerIndex] = VTData->LayerFallbackColors[LayerIndex];
		ProducerDesc.PhysicalGroupIndex[LayerIndex] = bSinglePhysicalSpace ? 0 : LayerIndex;
		ProducerDesc.bIsLayerSRGB[LayerIndex] = bSRGB;
	}
	ProducerDesc.bRequiresSinglePhysicalPool = bRequiresSinglePhysicalPool;
	ProducerDesc.Priority = VirtualTextureStreamingPriority;

	FUploadingVirtualTexture* VirtualTexture = new FUploadingVirtualTexture(ProducerDesc.Name, VTData, FirstMipToUse);
	ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(RHICmdList, ProducerDesc, VirtualTexture);

	// Only create the miptails mini-texture in-editor.
#if WITH_EDITOR
	InitializeEditorResources(RHICmdList, VirtualTexture);
#endif

	if (TextureRHI.IsValid())
	{
		TextureRHI->SetOwnerName(GetOwnerName());
	}
}

#if WITH_EDITOR
void FVirtualTexture2DResource::InitializeEditorResources(FRHICommandListBase& RHICmdList, IVirtualTexture* InVirtualTexture)
{
	// Create a texture resource for use when we try to access the VT as a regular texture.
	// This may happen for debugging or when we are in a preview mode that doesn't support VT.
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Choose a mip level for the thumbnail texture to ensure proper size
		const uint32 MaxTargetSize = FMath::Min(1024u, GetMax2DTextureDimension());
		const uint32 TargetSize = FMath::Clamp(128u, GetTileSize(), MaxTargetSize);
		const uint32 MaxMipLevel = VTData->GetNumMips() - 1u;
		
		uint32 MipLevel = 0;
		uint32 MipWidth = GetSizeX();
		uint32 MipHeight = GetSizeY();
		while (((MipWidth > TargetSize && MipHeight > TargetSize) || MipWidth > MaxTargetSize || MipHeight > MaxTargetSize) && MipLevel < MaxMipLevel)
		{
			++MipLevel;
			MipWidth = FMath::DivideAndRoundUp(MipWidth, 2u);
			MipHeight = FMath::DivideAndRoundUp(MipHeight, 2u);
		}

		FString Name = TextureName.ToString();
		EPixelFormat PixelFormat = VTData->LayerTypes[0];

		FTextureRHIRef Texture2DRHI = nullptr;

		// We only handle the single layer case.
		bool bUseFallbackTexture = GetNumLayers() != 1;
		// If the VT texture is very skewed on one dimension so that other dimension is very small then we may have less than VT block size.
		bUseFallbackTexture |= IsBlockCompressedFormat(PixelFormat) && ((MipWidth % 4 != 0) || (MipHeight % 4 != 0));

		if (bUseFallbackTexture)
		{
			// Fallback to simple 1x1 texture of the fallback color.
			const FRHITextureCreateDesc TextureDesc =
				FRHITextureCreateDesc::Create2D(*Name, 1, 1, EPixelFormat::PF_B8G8R8A8)
				.AddFlags(TexCreateFlags)
				.SetInitActionInitializer();

			FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(TextureDesc);

			{
				FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);
				Subresource.WriteColor(VTData->LayerFallbackColors[0].ToFColor(true));
			}

			Texture2DRHI = Initializer.Finalize();
		}
		else
		{
			FRDGBuilder GraphBuilder(RHICmdList.GetAsImmediate());

			// We want to strip borders when compositing tiles since we're just laying out tiles in a regular texture.
			// But if we have block compressed formats with border not divisible by 4 then doing this will lead to an unaligned copy. We keep the small unwanted borders in that case.
			const bool bCopyUnwantedBordersForAlignment = (VTData->TileBorderSize % 4 != 0) && IsBlockCompressedFormat(PixelFormat);
			const uint32 MipScaleFactor = (1u << MipLevel);
			const uint32 MipWidthInTiles = FMath::DivideAndRoundUp(GetNumTilesX(), MipScaleFactor);
			const uint32 MipHeightInTiles = FMath::DivideAndRoundUp(GetNumTilesY(), MipScaleFactor);
			const uint32 TileSizeInPixels = bCopyUnwantedBordersForAlignment ? GetTileSize() + 2 * GetBorderSize() : GetTileSize();
			const uint32 LayerMask = 1u;			
			const ETextureCreateFlags RHITexCreateFlags = ETextureCreateFlags::ShaderResource | (this->TexCreateFlags & TexCreate_SRGB);

			// Get or create the target texture to composite tiles to.
			// First try reusing the current referenced texture.
			// We do that because swapping the referenced texture can be expensive if we are doing a recreate of all textures.
			if (FRHITexture* ReferencedTexture = TextureReferenceRHI->GetReferencedTexture())
			{
				FRHITextureDesc const& ExistingDesc = ReferencedTexture->GetDesc();
				if (ExistingDesc.Format == PixelFormat && ExistingDesc.GetSize() == FIntVector(MipWidthInTiles * TileSizeInPixels, MipHeightInTiles * TileSizeInPixels, 1) &&
					ExistingDesc.Flags == RHITexCreateFlags )
				{
					Texture2DRHI = ReferencedTexture;
				}
			}

			// If we can't reuse the referenced texture then create it.
			if (Texture2DRHI == nullptr)
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("VTEditorResource"), MipWidthInTiles * TileSizeInPixels, MipHeightInTiles * TileSizeInPixels, PixelFormat)
					.SetFlags(RHITexCreateFlags);

				Texture2DRHI = RHICmdList.CreateTexture(Desc);
			}
			
			TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = CreateRenderTarget(Texture2DRHI, TEXT("VTEditorResource"));

			// Produce all the tiles.
			TArray<IVirtualTextureFinalizer*> Finalizers;

			for (uint32 TileY = 0u; TileY < MipHeightInTiles; ++TileY)
			{
				for (uint32 TileX = 0u; TileX < MipWidthInTiles; ++TileX)
				{
					const uint32 vAddress = FMath::MortonCode2(TileX) | (FMath::MortonCode2(TileY) << 1);
					const FVTRequestPageResult RequestResult = InVirtualTexture->RequestPageData(RHICmdList, ProducerHandle, LayerMask, MipLevel, vAddress, EVTRequestPagePriority::High);

					// High priority request should never be Saturated.
					// It's possible for status to be Invalid, if requesting data from a mip level that doesn't exist for the given producer (when using sparse UDIMs).
					// Technically we could try to handle this, by checking LocalMipBias, grabbing lower resolution tile, and resizing...but that would make this code much more complex for very little gain.
					if (VTRequestPageStatus_HasData(RequestResult.Status))
					{
						FVTProduceTargetLayer TargetLayer;
						TargetLayer.PooledRenderTarget = PooledRenderTarget;
						TargetLayer.pPageLocation = FIntVector(TileX, TileY, 0);

						IVirtualTextureFinalizer* Finalizer = InVirtualTexture->ProducePageData(
							RHICmdList,
							GMaxRHIFeatureLevel,
							bCopyUnwantedBordersForAlignment ? EVTProducePageFlags::None : EVTProducePageFlags::SkipPageBorders,
							ProducerHandle, LayerMask, MipLevel, vAddress,
							RequestResult.Handle,
							&TargetLayer);

						Finalizers.AddUnique(Finalizer);
					}
				}
			}

			for (IVirtualTextureFinalizer* Finalizer : Finalizers)
			{
				if (Finalizer)
				{
					Finalizer->Finalize(GraphBuilder);
				}
			}

			GraphBuilder.Execute();

			RHICmdList.BindDebugLabelName(Texture2DRHI, *Name);
		}

		TextureRHI = Texture2DRHI;
		
		// Only update texture reference if necessary.
		if (TextureRHI != TextureReferenceRHI->GetReferencedTexture())
		{
			RHICmdList.UpdateTextureReference(TextureReferenceRHI, TextureRHI);
		}
	}
}
#endif // WITH_EDITOR

void FVirtualTexture2DResource::ReleaseRHI()
{
	ReleaseAllocatedVT();

	GetRendererModule().ReleaseVirtualTextureProducer(ProducerHandle);
	ProducerHandle = FVirtualTextureProducerHandle();
}

class IAllocatedVirtualTexture* FVirtualTexture2DResource::AcquireAllocatedVT()
{
	check(IsInRenderingThread());
	if (!AllocatedVT)
	{
		FAllocatedVTDescription VTDesc;
		VTDesc.Dimensions = 2;
		VTDesc.TileSize = GetTileSize();
		VTDesc.TileBorderSize = GetBorderSize();
		VTDesc.NumTextureLayers = GetNumLayers();
		VTDesc.bShareDuplicateLayers = bSinglePhysicalSpace;

		for (uint32 LayerIndex = 0u; LayerIndex < VTDesc.NumTextureLayers; ++LayerIndex)
		{
			VTDesc.ProducerHandle[LayerIndex] = ProducerHandle; // use the same producer for each layer
			VTDesc.ProducerLayerIndex[LayerIndex] = LayerIndex;
		}
		AllocatedVT = GetRendererModule().AllocateVirtualTexture(VTDesc);
	}
	return AllocatedVT;
}

void FVirtualTexture2DResource::ReleaseAllocatedVT()
{
	if (AllocatedVT)
	{
		GetRendererModule().DestroyVirtualTexture(AllocatedVT);
		AllocatedVT = nullptr;
	}
}

uint32 FVirtualTexture2DResource::GetSizeX() const
{
	return FMath::Max(VTData->Width >> FirstMipToUse, 1u);
}

uint32 FVirtualTexture2DResource::GetSizeY() const
{
	return FMath::Max(VTData->Height >> FirstMipToUse, 1u);
}

EPixelFormat FVirtualTexture2DResource::GetFormat(uint32 LayerIndex) const
{
	return VTData->LayerTypes[LayerIndex];
}

FIntPoint FVirtualTexture2DResource::GetSizeInBlocks() const
{
	return FIntPoint(VTData->WidthInBlocks, VTData->HeightInBlocks);
}

uint32 FVirtualTexture2DResource::GetNumTilesX() const
{
	return FMath::Max(VTData->GetWidthInTiles() >> FirstMipToUse, 1u);
}

uint32 FVirtualTexture2DResource::GetNumTilesY() const
{
	return FMath::Max(VTData->GetHeightInTiles() >> FirstMipToUse, 1u);
}

uint32 FVirtualTexture2DResource::GetBorderSize() const
{
	return VTData->TileBorderSize;
}

uint32 FVirtualTexture2DResource::GetNumMips() const
{
	ensure((int32)VTData->GetNumMips() > FirstMipToUse);
	return VTData->GetNumMips() - FirstMipToUse;
}

uint32 FVirtualTexture2DResource::GetNumLayers() const
{
	return VTData->GetNumLayers();
}

uint32 FVirtualTexture2DResource::GetTileSize() const
{
	return VTData->TileSize;
}

uint32 FVirtualTexture2DResource::GetAllocatedvAddress() const
{
	if (AllocatedVT)
	{
		return AllocatedVT->GetVirtualAddress();
	}
	return ~0;
}

FIntPoint FVirtualTexture2DResource::GetPhysicalTextureSize(uint32 LayerIndex) const
{
	if (AllocatedVT)
	{
		const uint32 PhysicalTextureSize = AllocatedVT->GetPhysicalTextureSize(LayerIndex);
		return FIntPoint(PhysicalTextureSize, PhysicalTextureSize);
	}
	return FIntPoint(0, 0);
}

static void StreamVirtualTextureMips(FVirtualTexture2DResource* VirtualTexture2DResource, FStreamableRenderResourceState& CachedSRRState, int32 NewMipCount)
{
	CachedSRRState.NumResidentLODs = CachedSRRState.NumRequestedLODs;

	ENQUEUE_RENDER_COMMAND(RequestVTTiles)([VirtualTexture2DResource, NewMipCount](FRHICommandListImmediate& RHICmdList)
	{
		const int32 MipLevel = VirtualTexture2DResource->GetNumMips() - NewMipCount;
		GetRendererModule().LockVirtualTextureTiles(VirtualTexture2DResource->GetProducerHandle(), MipLevel);
	});
}

bool UTexture2D::StreamIn(int32 NewMipCount, bool bHighPrio)
{
	check(IsInGameThread());

	FTextureResource* TextureResource = GetResource();

	FVirtualTexture2DResource* VirtualTexture2DResource = TextureResource ? TextureResource->GetVirtualTexture2DResource() : nullptr;
	if (VirtualTexture2DResource != nullptr)
	{
		const bool bOK = CachedSRRState.StreamIn(NewMipCount);
		if (bOK)
		{
			StreamVirtualTextureMips(VirtualTexture2DResource, CachedSRRState, NewMipCount);
		}
		return bOK;
	}

	FTexture2DResource* Texture2DResource = TextureResource ? TextureResource->GetTexture2DResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamIn(NewMipCount) && ensure(Texture2DResource))
	{
		FTextureMipDataProvider* CustomMipDataProvider = nullptr;
		if (GUseGenericStreamingPath != 2)
		{
			for (UAssetUserData* UserData : AssetUserData)
			{
				UTextureMipDataProviderFactory* CustomMipDataProviderFactory = Cast<UTextureMipDataProviderFactory>(UserData);
				if (CustomMipDataProviderFactory)
				{
					CustomMipDataProvider = CustomMipDataProviderFactory->AllocateMipDataProvider(this);
					if (CustomMipDataProvider)
					{
						break;
					}
				}
			}
		}

		if (!CustomMipDataProvider && GUseGenericStreamingPath != 1)
		{
			const auto HasDerivedData = [](UTexture2D* Texture) -> bool
			{
				if (FTexturePlatformData* LocalPlatformData = Texture->GetPlatformData())
				{
					return Algo::AnyOf(LocalPlatformData->Mips, &FTexture2DMipMap::DerivedData) ||
						(LocalPlatformData->VTData && Algo::AnyOf(LocalPlatformData->VTData->Chunks, &FVirtualTextureDataChunk::DerivedData));
				}
				return false;
			};

	#if WITH_EDITORONLY_DATA
			if (FPlatformProperties::HasEditorOnlyData() && !GetOutermost()->bIsCookedForEditor && !GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
			{
				if (GRHISupportsAsyncTextureCreation)
				{
					PendingUpdate = new FTexture2DStreamIn_DDC_AsyncCreate(this);
				}
				else
				{
					PendingUpdate = new FTexture2DStreamIn_DDC_AsyncReallocate(this);
				}
			}
			else
	#endif
			if (HasDerivedData(this))
			{
				// If the future texture is to be a virtual texture, use the virtual stream in path.
				if (Texture2DResource && Texture2DResource->bUsePartiallyResidentMips)
				{
					PendingUpdate = new UE::FTexture2DStreamIn_DerivedData_Virtual(this, bHighPrio);
				}
				// If the platform supports creating the new texture on an async thread, use that path.
				else if (GRHISupportsAsyncTextureCreation)
				{
					PendingUpdate = new UE::FTexture2DStreamIn_DerivedData_AsyncCreate(this, bHighPrio);
				}
				// Otherwise use the default path.
				else
				{
					PendingUpdate = new UE::FTexture2DStreamIn_DerivedData_AsyncReallocate(this, bHighPrio);
				}
			}
			else
			{
				// If the future texture is to be a virtual texture, use the virtual stream in path.
				if (Texture2DResource && Texture2DResource->bUsePartiallyResidentMips)
				{
					PendingUpdate = new FTexture2DStreamIn_IO_Virtual(this, bHighPrio);
				}
				// If the platform supports creating the new texture on an async thread, use that path.
				else if (GRHISupportsAsyncTextureCreation)
				{
					PendingUpdate = new FTexture2DStreamIn_IO_AsyncCreate(this, bHighPrio);
				}
				// Otherwise use the default path.
				else
				{
					PendingUpdate = new FTexture2DStreamIn_IO_AsyncReallocate(this, bHighPrio);
				}
			}
		}
		else // Generic path
		{
			FTextureMipAllocator* MipAllocator = nullptr;
			FTextureMipDataProvider* DefaultMipDataProvider = nullptr;

	#if WITH_EDITORONLY_DATA
			if (FPlatformProperties::HasEditorOnlyData() && !GetOutermost()->bIsCookedForEditor && !GetOutermost()->HasAnyPackageFlags(PKG_Cooked))
			{
				DefaultMipDataProvider = new FTexture2DMipDataProvider_DDC(this);
			}
			else 
#endif
			{
				DefaultMipDataProvider = new FTexture2DMipDataProvider_IO(this, bHighPrio);
			}

			if (Texture2DResource && Texture2DResource->bUsePartiallyResidentMips)
			{
				MipAllocator = new FTexture2DMipAllocator_Virtual(this);
			}
			else if (GRHISupportsAsyncTextureCreation)
			{
				MipAllocator = new FTexture2DMipAllocator_AsyncCreate(this);
			}
			else
			{
				MipAllocator = new FTexture2DMipAllocator_AsyncReallocate(this);
			}

			PendingUpdate = new FTextureStreamIn(this, MipAllocator, CustomMipDataProvider, DefaultMipDataProvider);
		}
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

bool UTexture2D::StreamOut(int32 NewMipCount)
{
	check(IsInGameThread());

	FTextureResource* TextureResource = GetResource();
	
	FVirtualTexture2DResource* VirtualTexture2DResource = TextureResource ? TextureResource->GetVirtualTexture2DResource() : nullptr;
	if (VirtualTexture2DResource != nullptr)
	{
		const bool bOK = CachedSRRState.StreamOut(NewMipCount);
		if (bOK)
		{
			StreamVirtualTextureMips(VirtualTexture2DResource, CachedSRRState, NewMipCount);
		}
		return bOK;
	}

	FTexture2DResource* Texture2DResource = TextureResource ? TextureResource->GetTexture2DResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamOut(NewMipCount) && ensure(Texture2DResource))
	{
		if (Texture2DResource && Texture2DResource->bUsePartiallyResidentMips)
		{
			PendingUpdate = new FTexture2DStreamOut_Virtual(this);
		}
		// If the platform supports creating the new texture on an async thread, use that path.
		else if (GRHISupportAsyncTextureStreamOut)
		{
			PendingUpdate = new FTexture2DStreamOut_AsyncCreate(this);
		}
		else
		{
			PendingUpdate = new FTexture2DStreamOut_AsyncReallocate(this);
		}
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

