// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Texture2DDynamic.cpp: Implementation of UTexture2DDynamic.
=============================================================================*/

#include "Engine/Texture2DDynamic.h"
#include "EngineLogs.h"
#include "UObject/Package.h"
#include "TextureResource.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Texture2DDynamic)

/*-----------------------------------------------------------------------------
	FTexture2DDynamicResource
-----------------------------------------------------------------------------*/

/** Initialization constructor. */
FTexture2DDynamicResource::FTexture2DDynamicResource(UTexture2DDynamic* InOwner)
:	Owner(InOwner)
{
}

/** Returns the width of the texture in pixels. */
uint32 FTexture2DDynamicResource::GetSizeX() const
{
	return Owner->SizeX;
}

/** Returns the height of the texture in pixels. */
uint32 FTexture2DDynamicResource::GetSizeY() const
{
	return Owner->SizeY;
}

/** Called when the resource is initialized. This is only called by the rendering thread. */
void FTexture2DDynamicResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Create the sampler state RHI resource.
	ESamplerAddressMode SamplerAddressMode = Owner->SamplerAddressMode;
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter( Owner ),
		SamplerAddressMode,
		SamplerAddressMode,
		SamplerAddressMode
	);
	SamplerStateRHI = GetOrCreateSamplerState( SamplerStateInitializer );

	FString Name = Owner->GetName();

	FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(*Name, GetSizeX(), GetSizeY(), Owner->Format)
		.SetNumMips(Owner->NumMips);

	if (Owner->bIsResolveTarget)
	{
		Desc.AddFlags(ETextureCreateFlags::ResolveTargetable);
	}
	else if (Owner->SRGB)
	{
		Desc.AddFlags(ETextureCreateFlags::SRGB);
	}

	if (Owner->bNoTiling)
	{
		Desc.AddFlags(ETextureCreateFlags::NoTiling);
	}

	Texture2DRHI = RHICmdList.CreateTexture(Desc);

	TextureRHI = Texture2DRHI;
	TextureRHI->SetName(Owner->GetFName());
	RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI,TextureRHI);
}

/** Called when the resource is released. This is only called by the rendering thread. */
void FTexture2DDynamicResource::ReleaseRHI()
{
	RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	FTextureResource::ReleaseRHI();
	Texture2DRHI.SafeRelease();
}

/** Returns the Texture2DRHI, which can be used for locking/unlocking the mips. */
FTextureRHIRef FTexture2DDynamicResource::GetTexture2DRHI()
{
	return Texture2DRHI;
}

#if !UE_SERVER
void FTexture2DDynamicResource::WriteRawToTexture_RenderThread(TArrayView64<const uint8> RawData)
{
	check(IsInRenderingThread());

	const uint32 Width = Texture2DRHI->GetSizeX();
	const uint32 Height = Texture2DRHI->GetSizeY();

	// Prevent from locking texture if the source is empty or of size 0 or if source is is too small.
	const uint64 SourceSize = RawData.Num();
	if (!ensure(Width * Height != 0 && SourceSize >= Width * Height && Texture2DRHI->GetDesc().Format == EPixelFormat::PF_B8G8R8A8))
	{
		return;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	uint32 DestStride = 0;
	uint8* DestData = reinterpret_cast<uint8*>(RHICmdList.LockTexture2D(Texture2DRHI, 0, RLM_WriteOnly, DestStride, false, false));


	for (uint32 y = 0; y < Height; y++)
	{
		const uint64 CurrentLine = ((uint64)Height - 1 - y);
		uint8* DestPtr = &DestData[CurrentLine * DestStride];

		const FColor* SrcPtr = &((FColor*)(RawData.GetData()))[CurrentLine * Width];
		for (uint32 x = 0; x < Width; x++)
		{
			*DestPtr++ = SrcPtr->B;
			*DestPtr++ = SrcPtr->G;
			*DestPtr++ = SrcPtr->R;
			*DestPtr++ = SrcPtr->A;
			SrcPtr++;
		}
	}

	RHICmdList.UnlockTexture2D(Texture2DRHI, 0, false, false);
}

#endif

/*-----------------------------------------------------------------------------
	UTexture2DDynamic
-----------------------------------------------------------------------------*/
UTexture2DDynamic::UTexture2DDynamic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NeverStream = true;
	Format = PF_B8G8R8A8;
	SamplerAddressMode = AM_Wrap;
}


void UTexture2DDynamic::Init( int32 InSizeX, int32 InSizeY, EPixelFormat InFormat/*=2*/, bool InIsResolveTarget/*=false*/ )
{
	SizeX = InSizeX;
	SizeY = InSizeY;
	Format = (EPixelFormat) InFormat;
	NumMips = 1;
	bIsResolveTarget = InIsResolveTarget;

	// Initialize the resource.
	UpdateResource();
}

FTextureResource* UTexture2DDynamic::CreateResource()
{
	return new FTexture2DDynamicResource(this);
}

float UTexture2DDynamic::GetSurfaceWidth() const
{
	return SizeX;
}

float UTexture2DDynamic::GetSurfaceHeight() const
{
	return SizeY;
}

UTexture2DDynamic* UTexture2DDynamic::Create(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat)
{
	FTexture2DDynamicCreateInfo CreateInfo(InFormat);

	return Create(InSizeX, InSizeY, CreateInfo);
}

UTexture2DDynamic* UTexture2DDynamic::Create(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat, bool InIsResolveTarget)
{
	FTexture2DDynamicCreateInfo CreateInfo(InFormat, InIsResolveTarget);

	return Create(InSizeX, InSizeY, CreateInfo);
}

UTexture2DDynamic* UTexture2DDynamic::Create(int32 InSizeX, int32 InSizeY, const FTexture2DDynamicCreateInfo& InCreateInfo)
{
	EPixelFormat DesiredFormat = EPixelFormat(InCreateInfo.Format);
	if (InSizeX > 0 && InSizeY > 0 )
	{
		
		auto NewTexture = NewObject<UTexture2DDynamic>(GetTransientPackage(), NAME_None, RF_Transient);
		if (NewTexture != NULL)
		{
			NewTexture->Filter = InCreateInfo.Filter;
			NewTexture->SamplerAddressMode = InCreateInfo.SamplerAddressMode;
			NewTexture->SRGB = InCreateInfo.bSRGB;

			// Disable compression
			NewTexture->CompressionSettings		= TC_Default;
#if WITH_EDITORONLY_DATA
			NewTexture->CompressionNone			= true;
			NewTexture->MipGenSettings			= TMGS_NoMipmaps;
			NewTexture->CompressionNoAlpha		= true;
			NewTexture->DeferCompression		= false;
#endif // #if WITH_EDITORONLY_DATA
			if ( InCreateInfo.bIsResolveTarget )
			{
				NewTexture->bNoTiling			= false;
			}
			else
			{
				// Untiled format
				NewTexture->bNoTiling			= true;
			}

			NewTexture->Init(InSizeX, InSizeY, DesiredFormat, InCreateInfo.bIsResolveTarget);
		}
		return NewTexture;
	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UTexture2DDynamic::Create()");
		return NULL;
	}
}

