// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TextureCollection.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "RenderingThread.h"
#include "ShaderPlatformConfig.h"
#include "TextureResource.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "ObjectCacheContext.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCollection)

#define LOCTEXT_NAMESPACE "UTextureCollection"

FTextureCollectionResource::FTextureCollectionResource(UTextureCollection* InParent)
{
	CollectionName = InParent->GetFName();
}

FBindlessTextureCollectionResource::FBindlessTextureCollectionResource(UTextureCollection* InParent) : FTextureCollectionResource(InParent)
{
	TConstArrayView<TObjectPtr<UTexture>> ParentTextures = InParent->Textures;
	InputTextureResources.Reserve(ParentTextures.Num());

	for (TObjectPtr<UTexture> Texture : ParentTextures)
	{
		// TODO: figure out what we do with null textures
		UTexture* ActualTexture = Texture != nullptr ? (UTexture*)Texture : (UTexture*)GEngine->DefaultTexture;
		InputTextureResources.Emplace(ActualTexture->GetResource());
	}
}

void FBindlessTextureCollectionResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	TArray<FRHIResourceCollectionMember> CollectionMembers;
	CollectionMembers.Reserve(InputTextureResources.Num());

	for (FTextureResource* TextureResource : InputTextureResources)
	{
		FRHITextureReference* TextureReference = TextureResource ? TextureResource->GetTextureReference() : nullptr;
		CollectionMembers.Emplace(TextureReference);
	}

	ResourceCollectionRHI = RHICmdList.CreateResourceCollection(CollectionMembers);
}

void FBindlessTextureCollectionResource::ReleaseRHI()
{
	
}

#if WITH_EDITOR
void UTextureCollection::NotifyMaterials()
{
	FMaterialUpdateContext UpdateContext;
	FObjectCacheContextScope ObjectCache;

	// Notify any material that uses this texture
	TSet<UMaterial*> BaseMaterialsThatUseThisTexture;
	for (UMaterialInterface* MaterialInterface : ObjectCache.GetContext().GetMaterialsAffectedByTextureCollection(this))
	{
		UpdateContext.AddMaterialInterface(MaterialInterface);
		// This is a bit tricky. We want to make sure all materials using this texture are
		// updated. Materials are always updated. Material instances may also have to be
		// updated and if they have static permutations their children must be updated
		// whether they use the texture or not! The safe thing to do is to add the instance's
		// base material to the update context causing all materials in the tree to update.
		BaseMaterialsThatUseThisTexture.Add(MaterialInterface->GetMaterial());
	}

	// Go ahead and update any base materials that need to be.
	for (TSet<UMaterial*>::TConstIterator It(BaseMaterialsThatUseThisTexture); It; ++It)
	{
		(*It)->PostEditChange();
	}
}

void UTextureCollection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateResource();
}
#endif // WITH_EDITOR

void UTextureCollection::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		// Make sure all referenced textures are PostLoad-ed before we start accessing them in UpdateResource()
		for (UTexture* Texture : this->Textures)
		{
			if (Texture)
			{
				Texture->ConditionalPostLoad();
			}
		}

		UpdateResource(false);
	}
}

void UTextureCollection::SetResource(FTextureCollectionResource* InResource)
{
	check (!IsInActualRenderingThread() && !IsInRHIThread());

	PrivateResource = InResource;
	ENQUEUE_RENDER_COMMAND(SetResourceRenderThread)([this, InResource](FRHICommandListImmediate& RHICmdList)
	{
		PrivateResourceRenderThread = InResource;
	});
}

FTextureCollectionResource* UTextureCollection::GetResource() const
{
	if (IsInParallelGameThread() || IsInGameThread() || IsInSlateThread() || IsInAsyncLoadingThread())
	{
		return PrivateResource;
	}

	if (IsInParallelRenderingThread() || IsInRHIThread())
	{
		return PrivateResourceRenderThread;
	}

	ensureMsgf(false, TEXT("Attempted to access a texture resource from an unkown thread."));
	return nullptr;
}

FTextureCollectionResource* UTextureCollection::GetResource()
{
	if (IsInParallelGameThread() || IsInGameThread() || IsInSlateThread() || IsInAsyncLoadingThread())
	{
		return PrivateResource;
	}

	if (IsInParallelRenderingThread() || IsInRHIThread())
	{
		return PrivateResourceRenderThread;
	}

	ensureMsgf(false, TEXT("Attempted to access a texture resource from an unkown thread."));
	return nullptr;
}

FTextureCollectionResource* UTextureCollection::CreateResource()
{
	// TODO: christopher.waters - We should add a way to query the RHI to see if bindless is enabled.
	ERHIBindlessConfiguration BindlessResourcesConfig = FShaderPlatformConfig::GetBindlessConfiguration(GMaxRHIShaderPlatform);
	if (BindlessResourcesConfig == ERHIBindlessConfiguration::Disabled || !GRHIGlobals.bSupportsBindless)
	{
		return nullptr;
	}

	return new FBindlessTextureCollectionResource(this);
}

void UTextureCollection::ReleaseResource()
{
	if (PrivateResource)
	{
		check(!IsInActualRenderingThread() && !IsInRHIThread());

		// Free the resource.
		ENQUEUE_RENDER_COMMAND(DeleteResource)([this, ToDelete = PrivateResource](FRHICommandListImmediate& RHICmdList)
		{
			PrivateResourceRenderThread = nullptr;
			ToDelete->ReleaseResource();
			delete ToDelete;
		});
		PrivateResource = nullptr;
	}
}

void UTextureCollection::UpdateResource(bool bNotifyMaterials)
{
	// Release the existing texture resource.
	ReleaseResource();

	// Dedicated servers have no texture internals
	if (!FApp::CanEverRender() || HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (FTextureCollectionResource* NewResource = CreateResource())
	{
		LLM_SCOPE(ELLMTag::Textures);

		PrivateResource = NewResource;
#if RHI_ENABLE_RESOURCE_INFO
		NewResource->SetOwnerName(FName(GetPathName()));
#endif

		ENQUEUE_RENDER_COMMAND(SetTextureCollectionResource)([this, NewResource](FRHICommandListImmediate& RHICmdList)
		{
			PrivateResourceRenderThread = NewResource;
			NewResource->InitResource(RHICmdList);
		});
	}
	else
	{
		SetResource(nullptr);
	}

#if WITH_EDITOR
	if (bNotifyMaterials)
	{
		NotifyMaterials();
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
