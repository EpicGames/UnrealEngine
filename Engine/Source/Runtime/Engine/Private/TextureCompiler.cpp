// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureCompiler.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR

#include "AsyncCompilationHelpers.h"
#include "AssetCompilingManager.h"
#include "Editor.h"
#include "ObjectCacheContext.h"
#include "EngineLogs.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "RenderingThread.h"
#include "UObject/StrongObjectPtr.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "TextureDerivedDataTask.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Components/PrimitiveComponent.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "TextureCompiler"

static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncTextureStandard(
	TEXT("Texture"),
	TEXT("textures"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FTextureCompilingManager::Get().FinishAllCompilation();
		}
	));

static bool GAsyncTextureCompilationCancelable = false;
static FAutoConsoleVariableRef CVarAsyncTextureCompilationCancelable(
	TEXT("Editor.AsyncTextureCompilationCancelable"),
	GAsyncTextureCompilationCancelable,
	TEXT("Whether or not to allow early cancelation of textures during async compilation."),
	ECVF_Default);

namespace TextureCompilingManagerImpl
{
	static FString GetLODGroupName(UTexture* Texture)
	{
		return StaticEnum<TextureGroup>()->GetMetaData(TEXT("DisplayName"), Texture->LODGroup);
	}

	static EQueuedWorkPriority GetBasePriority(UTexture* InTexture)
	{
		switch (InTexture->LODGroup)
		{
		case TEXTUREGROUP_UI:
			return EQueuedWorkPriority::High;
		case TEXTUREGROUP_Terrain_Heightmap:
			return EQueuedWorkPriority::Normal;
		default:
			return EQueuedWorkPriority::Lowest;
		}
	}

	static EQueuedWorkPriority GetBoostPriority(UTexture* InTexture)
	{
		return (EQueuedWorkPriority)(FMath::Max((uint8)EQueuedWorkPriority::Highest, (uint8)GetBasePriority(InTexture)) - 1);
	}

	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;

			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("texture"),
				CVarAsyncTextureStandard.AsyncCompilation,
				CVarAsyncTextureStandard.AsyncCompilationMaxConcurrency,
				GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bEnableAsyncTextureCompilation));
		}
	}
}

FTextureCompilingManager::FTextureCompilingManager()
	: Notification(MakeUnique<FAsyncCompilationNotification>(GetAssetNameFormat()))
{
	TextureCompilingManagerImpl::EnsureInitializedCVars();
}

bool FTextureCompilingManager::IsAsyncCompilationCancelable() const
{
	return GAsyncTextureCompilationCancelable;
}

FName FTextureCompilingManager::GetStaticAssetTypeName()
{
	return TEXT("UE-Texture");
}

bool FTextureCompilingManager::IsCompilingTexture(UTexture* InTexture) const 
{
	check(IsInGameThread());

	if (!InTexture)
	{
		return false;
	}

	const TWeakObjectPtr<UTexture> InWeakTexturePtr = InTexture;
	uint32 Hash = GetTypeHash(InWeakTexturePtr);
	for (const TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
	{
		if (Bucket.ContainsByHash(Hash, InWeakTexturePtr))
		{
			return true;
		}
	}

	return false;
}

FName FTextureCompilingManager::GetAssetTypeName() const
{
	return GetStaticAssetTypeName();
}

TArrayView<FName> FTextureCompilingManager::GetDependentTypeNames() const
{
	return TArrayView<FName>{ };
}

FTextFormat FTextureCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("TextureNameFormat", "{0}|plural(one=Texture,other=Textures)");
}

EQueuedWorkPriority FTextureCompilingManager::GetBasePriority(UTexture* InTexture) const
{
	return TextureCompilingManagerImpl::GetBasePriority(InTexture);
}

FQueuedThreadPool* FTextureCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolWrapper* GTextureThreadPool = nullptr;
	if (GTextureThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		const auto TexturePriorityMapper = [](EQueuedWorkPriority TexturePriority) { return FMath::Max(TexturePriority, EQueuedWorkPriority::Low); };

		// Textures will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GTextureThreadPool = new FQueuedThreadPoolWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, TexturePriorityMapper);

		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GTextureThreadPool,
			CVarAsyncTextureStandard.AsyncCompilation,
			CVarAsyncTextureStandard.AsyncCompilationResume,
			CVarAsyncTextureStandard.AsyncCompilationMaxConcurrency
		);
	}

	return GTextureThreadPool;
}

void FTextureCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingTextures())
	{
		TArray<UTexture*> PendingTextures;
		PendingTextures.Reserve(GetNumRemainingTextures());

		for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
		{
			for (TWeakObjectPtr<UTexture>& WeakTexture : Bucket)
			{
				if (WeakTexture.IsValid())
				{
					UTexture* Texture = WeakTexture.Get();
					
					if (!Texture->TryCancelCachePlatformData())
					{
						PendingTextures.Add(Texture);
					}
				}
			}
		}

		// Wait on textures already in progress we couldn't cancel
		FinishCompilation(PendingTextures);
	}
}

bool FTextureCompilingManager::IsAsyncTextureCompilationEnabled() const
{
	if (bHasShutdown || !FPlatformProcess::SupportsMultithreading())
	{
		return false;
	}

	return CVarAsyncTextureStandard.AsyncCompilation.GetValueOnAnyThread() != 0;
}

TRACE_DECLARE_INT_COUNTER(QueuedTextureCompilation, TEXT("AsyncCompilation/QueuedTexture"));
void FTextureCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedTextureCompilation, GetNumRemainingTextures());
	Notification->Update(GetNumRemainingTextures());
}

bool FTextureCompilingManager::PostCompilation(UTexture* Texture)
{
	TGuardValue PostCompilationGuard(bIsRoutingPostCompilation, true);
	TGuardValue PostCompilationTextureGuard(PostCompilationTexture, Texture);
	bIsRoutingPostCompilationError = false;

	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::PostCompilation);

	UE_LOGF(LogTexture, Verbose, "Refreshing texture %ls because it is ready", *Texture->GetName());

	Texture->FinishCachePlatformData();

	// Track the DDC key suffix of the texture we are done with so that if we re-enter we can
	// log info and hopefully be able to do some post-mortem on it.

	CurrentPostCompilationDDCKey.Empty();
	if (FTexturePlatformData** RunningPlatformData = Texture->GetRunningPlatformData(); RunningPlatformData && *RunningPlatformData)
	{
		// only works for ddc1 right now... 
		if (FString* DDCKey = RunningPlatformData[0]->DerivedDataKey.TryGet<FString>(); DDCKey)
		{
			CurrentPostCompilationDDCKey = *DDCKey;
		}
	}

	Texture->UpdateResource();

	// Most likely a recursion happened during UpdateResource to add the texture again
	// so we must return an error and take action on it.
	if (bIsRoutingPostCompilationError)
	{
		return false;
	}

	// Generate an empty property changed event, to force the asset registry tag
	// to be refreshed now that pixel format and alpha channels are available.
	FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Texture, EmptyPropertyChangedEvent);

	return true;
}

bool FTextureCompilingManager::IsAsyncCompilationAllowed(UTexture* Texture) const
{
	return IsAsyncTextureCompilationEnabled();
}

FTextureCompilingManager& FTextureCompilingManager::Get()
{
	static FTextureCompilingManager Singleton;
	return Singleton;
}

int32 FTextureCompilingManager::GetNumRemainingTextures() const
{
	int32 Num = 0;
	for (const TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
	{
		Num += Bucket.Num();
	}

	return Num;
}

int32 FTextureCompilingManager::GetNumRemainingAssets() const
{
	return GetNumRemainingTextures();
}

void FTextureCompilingManager::AddTextures(TArrayView<UTexture* const> InTextures)
{
	check(IsInGameThread());

	// If you hit this, it's because above this in the stack you'll see PostCompilation(). In that function you'll see:
	// 	Texture->FinishCachePlatformData();
	//	Texture->UpdateResource();
	// UpdateResource ends up doing another CachePlatformData() - so what's happened is you finished pulling in the derived data
	// and then immediately tried again - and then tried to launch another build because the ddc keys changed. This means that
	// during the async build, a property or otherwise that is an input to the ddc key changed. This shouldn't happen because
	// PreEditChange completes the async build before allowing the change.
	// Debugging this can be a huge pain. NEW AND IMPROVED: We should now be printing the relevant DDC keys below (if DDC1). This
	// should facilitate at least finding out what property is getting changed, as well as what texture. If you can't divine what's
	// causing the change from that, you'll need to put a data breakpoint on it and see who is doing it.
	//
	// **
	//
	// One thing to be aware of is this can be caused by a system manually calling FinishCachePlatformData + UpdateResource
	// instead of calling BlockOnAnyAsyncBuild. This causes the async task to become null,
	// which prevents any IsCompiling / BlockOnAnyAsyncBuild from detecting it, even though it's still pending a PostCompilation
	// in here. As a result you can edit the DDC key any time between the FinishCachePlatformData and the subsequent CreateResource
	// call and get this crash. If you have a repro, best bet is to try and get a breakpoint on FinishCachePlatformData for the texture
	// in question - only the compilation manager shoulid be calling that for editor resources.
	if (bIsRoutingPostCompilation)
	{
		TStringBuilder<1024> StringBuilder;

		StringBuilder.Appendf(
			TEXT("Registering a texture to the compile manager from inside a texture postcompilation usually\n")
			TEXT(" indicates that the previous async operation wasn't completed (i.e. missing call to PreEditChange) before modifying a texture property.\n")
		);

		StringBuilder.Appendf(TEXT("  PostCompilation Texture: %s\n"), PostCompilationTexture ? *PostCompilationTexture->GetPathName() : TEXT("<nullptr>"));

		// Empty keys most likely means we are on ddc2.
		StringBuilder.Appendf(TEXT("  PostCompilation DDCKey: %s\n"), *CurrentPostCompilationDDCKey);
		StringBuilder.Appendf(TEXT("  AddTextures Count: %d\n"), InTextures.Num());

		for (const UTexture* ConstTexture : InTextures)
		{
			if (ConstTexture)
			{
				if (ConstTexture == PostCompilationTexture)
				{
					bIsRoutingPostCompilationError = true;
				}

				// We're about to crash anyway.
				UTexture* Texture = const_cast<UTexture*>(ConstTexture);

				StringBuilder.Appendf(TEXT("    %s:\n"), *Texture->GetPathName());

				FTexturePlatformData** RunningPlatformData = Texture->GetRunningPlatformData();
				if (!RunningPlatformData || !RunningPlatformData[0])
				{
					StringBuilder.Appendf(TEXT("      -> No RunningPlatformData!\n"));
				}
				else
				{
					FString* Key = RunningPlatformData[0]->FetchFirstDerivedDataKey.TryGet<FString>();
					StringBuilder.Appendf(TEXT("       FetchFirstKey: %s\n"), Key ? **Key : TEXT("<empty, likely new texture build flow?>"));
					Key = RunningPlatformData[0]->FetchOrBuildDerivedDataKey.TryGet<FString>();
					StringBuilder.Appendf(TEXT("       FetchOrBuildKey: %s\n"), Key ? **Key : TEXT("<empty, likely new texture build flow?>"));
				}
			}
		}

		if (bIsRoutingPostCompilationError && InTextures.Num() == 1)
		{
			// Just log as error, will be catched and returned by PostCompilation, and acted on accordingly.
			UE_LOGF(LogTexture, Error, "%ls", StringBuilder.ToString());
			return;
		}

		// Treat any other case that we didn't plan on as fatal...
		UE_LOGF(LogTexture, Fatal, "%ls", StringBuilder.ToString());
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::AddTextures)

	// Register new textures after ProcessTextures to avoid
	// potential reentrant calls to CreateResource on the
	// textures being added. This would cause multiple
	// TextureResource to be created and assigned to the same Owner
	// which would obviously be bad and causing leaks including
	// in the RHI.
	for (UTexture* Texture : InTextures)
	{
		if (Texture)
		{
			int32 TexturePriority = 2;
			switch (Texture->LODGroup)
			{
				case TEXTUREGROUP_UI:
					TexturePriority = 0;
				break;
				case TEXTUREGROUP_Terrain_Heightmap:
					TexturePriority = 1;
				break;
			}

			if (RegisteredTextureBuckets.Num() <= TexturePriority)
			{
				RegisteredTextureBuckets.SetNum(TexturePriority + 1);
			}
			RegisteredTextureBuckets[TexturePriority].Emplace(Texture);
		}
	}

	TRACE_COUNTER_SET(QueuedTextureCompilation, GetNumRemainingTextures());
}

void FTextureCompilingManager::ForceDeferredTextureRebuildAnyThread(TArrayView<const TWeakObjectPtr<UTexture>> InTextures)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::AddTexturesDeferredAnyThread)

	for (const TWeakObjectPtr<UTexture>& Texture : InTextures)
	{
		DeferredRebuildRequestQueue.ProduceItem(Texture);
	}
}

void FTextureCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::FinishCompilationForObjects);

	TSet<UTexture*> Textures;
	for (UObject* Object : InObjects)
	{
		if (UTexture* Texture = Cast<UTexture>(Object))
		{
			Textures.Add(Texture);
		}
	}

	if (Textures.Num())
	{
		FinishCompilation(Textures.Array());
	}
}

void FTextureCompilingManager::MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects)
{
	if (InObjects.Num())
	{
		TSet<UTexture*> Textures;
		for (UObject* Object : InObjects)
		{
			if (UTexture* Texture = Cast<UTexture>(Object))
			{
				Textures.Add(Texture);
			}
		}

		if (Textures.Num())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::MarkCompilationAsCanceled);

			auto CancelOrMarkCanceled = [&Textures](TSet<TWeakObjectPtr<UTexture>>& Set)
			{
				for (auto Iterator = Set.CreateIterator(); Iterator; ++Iterator)
				{
					UTexture* Texture = Iterator->GetEvenIfUnreachable();
					if (Texture && Textures.Contains(Texture))
					{
						UE_LOGF(LogTexture, Verbose, "Canceling texture %ls async compilation as requested", *Texture->GetName());

						// On success, we can remove them from the list right away.
						// Otherwise, they are marked as canceled and will finish ASAP if the tasks support early cancellation.
						if (Texture->TryCancelCachePlatformData())
						{
							Iterator.RemoveCurrent();
						}
					}
				}
			};

			for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
			{
				CancelOrMarkCanceled(Bucket);
			}
		}
	}
}

void FTextureCompilingManager::FinishCompilation(TArrayView<UTexture* const> InTextures)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::FinishCompilation);

	if (InTextures.Num() == 0)
	{
		return;
	}

	if (bIsRoutingPostCompilation)
	{
		// This ends up modifying the registered texture buckets which is not allowed
		// when we are routing PostCompilation. Plus, it doesn't make much sense to 
		// be calling FinishCompilation while we are in the middle of finishing
		// compilations!
		// This is likely because a worker task got scheduled during a wait inside
		// PostCompilation and it's randomly running during the wait, causing crashes.
		// Workers that need to interact with textures should do that work in response to
		// a game tick via e.g. ExecuteOnGameThread
		UE_LOGF(LogTexture, Fatal, "Calling FinishCompilation is not allowed during PostCompilation. NumTextures = %d, Texture[0] = %ls", InTextures.Num(), *InTextures[0]->GetPathName());
	}

	using namespace TextureCompilingManagerImpl;
	check(IsInGameThread());

	TSet<UTexture*> PendingTextures;
	PendingTextures.Reserve(InTextures.Num());

	int32 TextureIndex = 0;
	for (UTexture* Texture : InTextures)
	{
		for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
		{
			if (Bucket.Contains(Texture))
			{
				PendingTextures.Add(Texture);
			}
		}
	}

	if (!PendingTextures.Num())
	{
		return;
	}

	class FCompilableTexture final : public AsyncCompilationHelpers::ICompilable
	{
	public:
		FCompilableTexture(UTexture* InTexture)
			: Texture(InTexture)
		{
		}

		FTextureAsyncCacheDerivedDataTask* GetAsyncTask()
		{
			if (FTexturePlatformData** PlatformDataPtr = Texture->GetRunningPlatformData())
			{
				if (FTexturePlatformData* PlatformData = *PlatformDataPtr)
				{
					return PlatformData->AsyncTask;
				}
			}

			return nullptr;
		}

		void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) final
		{
			if (FTextureAsyncCacheDerivedDataTask* AsyncTask = GetAsyncTask())
			{
				AsyncTask->SetPriority(InPriority);
			}
		}

		bool WaitCompletionWithTimeout(float TimeLimitSeconds) final
		{
			if (FTextureAsyncCacheDerivedDataTask* AsyncTask = GetAsyncTask())
			{
				return AsyncTask->WaitWithTimeout(TimeLimitSeconds);
			}
			return true;
		}

		FName GetName() override { return Texture->GetOutermost()->GetFName(); }

		TStrongObjectPtr<UTexture> Texture;
	};

	TArray<UTexture*> InitialTextures(PendingTextures.Array());
	while (PendingTextures.Num())
	{
		TArray<UTexture*> UniqueTextures(PendingTextures.Array());
		TArray<FCompilableTexture> CompilableTextures(UniqueTextures);
		using namespace AsyncCompilationHelpers;
		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableTextures](int32 Index)	-> ICompilable& { return CompilableTextures[Index]; },
			CompilableTextures.Num(),
			LOCTEXT("Textures", "Textures"),
			LogTexture,
			[this, &PendingTextures](ICompilable* Object)
			{
				UTexture* Texture = static_cast<FCompilableTexture*>(Object)->Texture.Get();
				// If the compilation doesn't succeed, it means we need to do another pass on it
				// to wait and postcompile again, so leave it in the PendingTextures.
				if (PostCompilation(Texture))
				{
					PendingTextures.Remove(Texture);

					for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
					{
						Bucket.Remove(Texture);
					}
				}
			}
		);
	}

	// Do the group post compilation only once after all textures has successfully passed individual PostCompilation.
	PostCompilation(InitialTextures);
}

void FTextureCompilingManager::PostCompilation(TArrayView<UTexture* const> InCompiledTextures)
{
	using namespace TextureCompilingManagerImpl;
	if (InCompiledTextures.Num())
	{
		FObjectCacheContextScope ObjectCacheScope;
		TRACE_CPUPROFILER_EVENT_SCOPE(PostTextureCompilation);
		{
			TSet<UMaterialInterface*> AffectedMaterials;
			for (UTexture* Texture : InCompiledTextures)
			{
				for (UMaterialInterface* Material : ObjectCacheScope.GetContext().GetMaterialsAffectedByTexture(Texture))
				{
					AffectedMaterials.Add(Material);
				}
			}

			if (AffectedMaterials.Num())
			{
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UpdateMaterials);

					for (UMaterialInterface* MaterialToUpdate : AffectedMaterials)
					{
						FMaterialRenderProxy* RenderProxy = MaterialToUpdate->GetRenderProxy();
						if (RenderProxy)
						{
							ENQUEUE_RENDER_COMMAND(TextureCompiler_RecacheUniformExpressions)(
								[RenderProxy](FRHICommandListImmediate& RHICmdList)
								{
									RenderProxy->CacheUniformExpressions(RHICmdList, false);
								});
						}
					}
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UpdatePrimitives);

					TSet<IPrimitiveComponent*> AffectedPrimitives;
					for (UMaterialInterface* MaterialInterface : AffectedMaterials)
					{
						for (IPrimitiveComponent* Component : ObjectCacheScope.GetContext().GetPrimitivesAffectedByMaterial(MaterialInterface))
						{
							AffectedPrimitives.Add(Component);
						}
					}

					for (IPrimitiveComponent* AffectedPrimitive : AffectedPrimitives)
					{
						AffectedPrimitive->MarkRenderStateDirty();
					}
				}
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

			TArray<FAssetCompileData> AssetsData;
			AssetsData.Reserve(InCompiledTextures.Num());

			for (UTexture* Texture : InCompiledTextures)
			{
				AssetsData.Emplace(Texture);
			}

			// Calling this delegate during app exit might be quite dangerous and lead to crash
			// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
			if (!GExitPurge && !IsGarbageCollecting())
			{
				FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
				OnTexturePostCompileEvent().Broadcast(InCompiledTextures);
			}
		}
	}
}

void FTextureCompilingManager::FinishAllCompilation()
{
	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Finish Texture Compilation"));

	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::FinishAllCompilation)

	if (GetNumRemainingTextures())
	{
		TArray<UTexture*> PendingTextures;
		PendingTextures.Reserve(GetNumRemainingTextures());

		for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
		{
			for (TWeakObjectPtr<UTexture>& Texture : Bucket)
			{
				if (Texture.IsValid())
				{
					PendingTextures.Add(Texture.Get());
				}
			}
		}

		FinishCompilation(PendingTextures);
	}
}

bool FTextureCompilingManager::GetCurrentPriority(UTexture* InTexture, EQueuedWorkPriority& OutPriority)
{
	using namespace TextureCompilingManagerImpl;
	if (InTexture)
	{
		FTexturePlatformData** Data = InTexture->GetRunningPlatformData();
		if (Data && *Data)
		{
			FTextureAsyncCacheDerivedDataTask* AsyncTask = (*Data)->AsyncTask;
			if (AsyncTask)
			{
				OutPriority = AsyncTask->GetPriority();
				return true;
			}
		}
	}

	return false;
}

bool FTextureCompilingManager::RequestPriorityChange(UTexture* InTexture, EQueuedWorkPriority InPriority)
{
	using namespace TextureCompilingManagerImpl;
	if (InTexture)
	{
		FTexturePlatformData** Data = InTexture->GetRunningPlatformData();
		if (Data && *Data)
		{
			FTextureAsyncCacheDerivedDataTask* AsyncTask = (*Data)->AsyncTask;
			if (AsyncTask)
			{
				EQueuedWorkPriority OldPriority = AsyncTask->GetPriority();
				if (OldPriority != InPriority)
				{
					if (AsyncTask->SetPriority(InPriority))
					{
						UE_LOGF(
							LogTexture,
							Verbose,
							"Changing priority of %ls (%ls) from %ls to %ls",
							*InTexture->GetName(),
							*GetLODGroupName(InTexture),
							LexToString(OldPriority),
							LexToString(InPriority)
						);

						return true;
					}
				}
			}
		}
	}

	return false;
}

void FTextureCompilingManager::ProcessTextures(bool bLimitExecutionTime, int32 MaximumPriority)
{
	using namespace TextureCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::ProcessTextures);
	const double MaxSecondsPerFrame = 0.016;

	if (GetNumRemainingTextures())
	{
		FObjectCacheContextScope ObjectCacheScope;
		TArray<UTexture*> ProcessedTextures;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedTextures);

			double TickStartTime = FPlatformTime::Seconds();

			if (MaximumPriority == -1 || MaximumPriority > RegisteredTextureBuckets.Num())
			{
				MaximumPriority = RegisteredTextureBuckets.Num();
			}
			
			for (int32 PriorityIndex = 0; PriorityIndex < MaximumPriority; ++PriorityIndex)
			{
				TSet<TWeakObjectPtr<UTexture>>& TexturesToProcess = RegisteredTextureBuckets[PriorityIndex];
				if (TexturesToProcess.Num())
				{
					const bool bIsHighestPrio = PriorityIndex == 0;
			
					TSet<TWeakObjectPtr<UTexture>> TexturesToPostpone;
					for (TWeakObjectPtr<UTexture>& Texture : TexturesToProcess)
					{
						if (Texture.IsValid())
						{
							const bool bHasTimeLeft = bLimitExecutionTime ? ((FPlatformTime::Seconds() - TickStartTime) < MaxSecondsPerFrame) : true;
							if ((bIsHighestPrio || bHasTimeLeft) && Texture->IsAsyncCacheComplete() && PostCompilation(Texture.Get()))
							{
								ProcessedTextures.Add(Texture.Get());
							}
							else
							{
								TexturesToPostpone.Emplace(MoveTemp(Texture));
							}
						}
					}

					RegisteredTextureBuckets[PriorityIndex] = MoveTemp(TexturesToPostpone);
				}
			}
		}

		if (GEngine && FPlatformTime::Seconds() - LastReschedule > 1.0f)
		{
			LastReschedule = FPlatformTime::Seconds();

			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::Reschedule);

			auto TryRescheduleTexture = 
				[this, &ObjectCacheScope](UTexture* Texture)
			{
				// Do not process anything for a texture that already has been prioritized
				EQueuedWorkPriority OutCurrentPriority;
				if (GetCurrentPriority(Texture, OutCurrentPriority) && OutCurrentPriority == GetBoostPriority(Texture))
				{
					return;
				}

				// Reschedule any texture that have been rendered with slightly higher priority 
				// to improve the editor experience for low-core count.
				//
				// Keep in mind that some textures are only accessed once during the construction
				// of a virtual texture, so we can't count on the LastRenderTime to be updated
				// continuously for those even if they're in view.
				if ((Texture->GetResource() && Texture->GetResource()->LastRenderTime > 0.0f) ||
					Texture->TextureReference.GetLastRenderTime() > 0.0f)
				{
					RequestPriorityChange(Texture, GetBoostPriority(Texture));
				}
				else
				{
					for (UMaterialInterface* MaterialInterface : ObjectCacheScope.GetContext().GetMaterialsAffectedByTexture(Texture))
					{
						for (IPrimitiveComponent* Component : ObjectCacheScope.GetContext().GetPrimitivesAffectedByMaterial(MaterialInterface))
						{
							if (Component->IsRegistered() && Component->IsRenderStateCreated() && Component->GetLastRenderTimeOnScreen() > 0.0f)
							{
								RequestPriorityChange(Texture, GetBoostPriority(Texture));
								return;
							}
						}
					}
				}
			};

			for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
			{
				for (TWeakObjectPtr<UTexture>& WeakPtr : Bucket)
				{
					if (UTexture* Texture = WeakPtr.Get())
					{
						TryRescheduleTexture(Texture);
					}
				}
			}
		}

		PostCompilation(ProcessedTextures);
	}
}

void FTextureCompilingManager::FinishCompilationsForGame()
{
	if (GetNumRemainingTextures())
	{
		// Supports both Game and PIE mode
		const bool bIsPlaying =
			(GWorld && !GWorld->IsEditorWorld()) ||
			(GEditor && GEditor->PlayWorld && !GEditor->IsSimulateInEditorInProgress());

		if (bIsPlaying)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCompilingManager::FinishCompilationsForGame);

			TSet<UTexture*> TexturesRequiredForGame;
			for (TSet<TWeakObjectPtr<UTexture>>& Bucket : RegisteredTextureBuckets)
			{
				for (TWeakObjectPtr<UTexture>& WeakTexture : Bucket)
				{
					if (UTexture* Texture = WeakTexture.Get())
					{
						switch (Texture->LODGroup)
						{
						case TEXTUREGROUP_Terrain_Heightmap:
						case TEXTUREGROUP_Terrain_Weightmap:
							TexturesRequiredForGame.Add(Texture);
							break;
						default:
							break;
						}
					}
				}
			}

			if (TexturesRequiredForGame.Num())
			{
				FinishCompilation(TexturesRequiredForGame.Array());
			}
		}
	}
}

void FTextureCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	AssetCompilation::FProcessAsyncTaskParams Params;
	Params.bLimitExecutionTime = bLimitExecutionTime;
	Params.bPlayInEditorAssetsOnly = false;
	ProcessAsyncTasks(Params);
}

void FTextureCompilingManager::ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params)
{
	if (bIsRoutingPostCompilation)
	{
		// This potentially affects RegisteredTextureBuckets which can't be touched inside PostCompilation.
		// This is likely because a worker task got scheduled during a wait inside
		// PostCompilation and it's randomly running during the wait, causing crashes.
		// Workers that need to interact with textures should do that work in response to
		// a game tick via e.g. ExecuteOnGameThread
		UE_LOGF(LogTexture, Fatal, "Calling ProcessAsyncTasks is not allowed during PostCompilation.");
	}

	FObjectCacheContextScope ObjectCacheScope;
	ProcessDeferredRequests();
	FinishCompilationsForGame();

	if (!Params.bPlayInEditorAssetsOnly)
	{
		ProcessTextures(Params.bLimitExecutionTime);
	}

	UpdateCompilationNotification();
}

void FTextureCompilingManager::ProcessDeferredRequests()
{
	TSet<UTexture*> DeferredTextures;
	DeferredRebuildRequestQueue.ConsumeAllFifo([this, &DeferredTextures](TWeakObjectPtr<UTexture> WeakTexture)
	{
		if (UTexture* Texture = WeakTexture.Get())
		{
			if (Texture->IsAsyncCacheComplete() && IsAsyncCompilationAllowed(Texture) && !IsCompilingTexture(Texture))
			{
				DeferredTextures.Add(Texture);
			}
		}
	});

	if (!DeferredTextures.IsEmpty())
	{
		for (UTexture* DeferredTexture : DeferredTextures)
		{
			DeferredTexture->UpdateResourceWithParams(UTexture::EUpdateResourceFlags::ForceRebuild);

			// The texture should now be in the normal texture compilation path. Since we don't need it immediately
			// we don't FinishCompilation on it.
		}
	}
}


#undef LOCTEXT_NAMESPACE

#endif // #if WITH_EDITOR
