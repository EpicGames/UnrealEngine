// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureCube.cpp: UTextureCube implementation.
=============================================================================*/

#include "Engine/TextureCube.h"
#include "EngineLogs.h"
#include "Misc/CoreStats.h"
#include "RenderUtils.h"
#include "Stats/StatsTrace.h"
#include "TextureResource.h"
#include "EngineUtils.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "TextureCompiler.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "ImageUtils.h"
#include "UObject/ReleaseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCube)

#define LOCTEXT_NAMESPACE "UTextureCube"

UTextureCube* UTextureCube::CreateTransient(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat, const FName InName)
{
	UTextureCube* NewTexture = nullptr;
	if (InSizeX > 0 && InSizeY > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0)
	{
		NewTexture = NewObject<UTextureCube>(
			GetTransientPackage(),
			InName,
			RF_Transient
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->SetIsCubemap(true);
		NewTexture->GetPlatformData()->SetNumSlices(6);
		NewTexture->GetPlatformData()->PixelFormat = InFormat;
		NewTexture->bNotOfflineProcessed = true;

		// Allocate first mipmap.
		int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY, 1);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc((int64)6 * NumBlocksX * NumBlocksY * GPixelFormats[InFormat].BlockBytes);
		Mip->BulkData.Unlock();
	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UTextureCube::CreateTransient()");
	}
	return NewTexture;
}

/**
 * Get the optimal placeholder to use during texture compilation
 */
static UTextureCube* GetDefaultTextureCube(const UTextureCube* Texture)
{
	static TStrongObjectPtr<UTextureCube> CheckerboardTexture;

	if (!CheckerboardTexture.IsValid())
	{
		CheckerboardTexture.Reset(FImageUtils::CreateCheckerboardCubeTexture(FColor(200, 200, 200, 128), FColor(128, 128, 128, 128)));
	}

	return CheckerboardTexture.Get();
}

UTextureCube::UTextureCube(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PrivatePlatformData(nullptr)
{
	SRGB = true;
}

FTexturePlatformData** UTextureCube::GetRunningPlatformData()
{
	// @todo DC GetRunningPlatformData is fundamentally unsafe but almost unused... should we replace it with Get/SetRunningPlatformData directly in the base class
	return &PrivatePlatformData;
}

void UTextureCube::SetPlatformData(FTexturePlatformData* InPlatformData)
{
	if (PrivatePlatformData)
	{
		ReleaseResource();
		delete PrivatePlatformData;
	}
	PrivatePlatformData = InPlatformData;
}

// Any direct access to GetPlatformData() will stall until the structure
// is safe to use. It is advisable to replace those use case with
// async aware code to avoid stalls where possible.
const FTexturePlatformData* UTextureCube::GetPlatformData() const
{
#if WITH_EDITOR
	if (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTextureCube::GetPlatformDataStall);
		UE_LOGF(LogTexture, Log, "Call to GetPlatformData() is forcing a wait on data that is not yet ready.");

		FText Msg = FText::Format(LOCTEXT("WaitOnTextureCompilation", "Waiting on texture compilation {0} ..."), FText::FromString(GetName()));
		FScopedSlowTask Progress(1.f, Msg, true);
		Progress.MakeDialog(true);
		PrivatePlatformData->FinishCache();
	}
#endif // #if WITH_EDITOR
	return PrivatePlatformData;
}

FTexturePlatformData* UTextureCube::GetPlatformData()
{
	// For now, this is the same implementation as the const version.
	const UTextureCube* ConstThis = this;
	return const_cast<FTexturePlatformData*>(ConstThis->GetPlatformData());
}

void UTextureCube::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTextureCube::Serialize"), STAT_TextureCube_Serialize, STATGROUP_LoadTime);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	Super::Serialize(Ar);

	FStripDataFlags StripFlags(Ar);
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (bCooked || Ar.IsCooking())
	{
		SerializeCookedPlatformData(Ar);
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Source.GetNumSlices() == 1 && MaxTextureSize == 0 && Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::LonglatTextureCubeDefaultMaxResolution)
	{
		// Previously default maximum resolution for cubemaps generated from long-lat sources was set to 512 pixels in the texture building code.
		// This value is now explicitly set for cubemaps loaded from earlier versions of the stream in order to avoid texture rebuild.
		MaxTextureSize = 512;
		UE_LOGF(LogTexture, Log, "Default maximum texture size for cubemaps generated from long-lat sources has been changed from 512 to unlimited. In order to preserve old behavior for '%ls', its maximum texture size has been explicitly set to 512.", *GetPathName());
	}
#endif // #if WITH_EDITORONLY_DATA
}

void UTextureCube::PostLoad()
{
#if WITH_EDITOR
	if (FApp::CanEverRender())
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
				BeginCachePlatformData();
		}
		else
		{
			FinishCachePlatformData();
		}
	}
#endif // #if WITH_EDITOR

	Super::PostLoad();

	// note UTexture::PostLoad casts to UTextureCube:: and calls ->UpdateResource  (should just do that here instead))
}

void UTextureCube::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UTextureCube::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	int32 SizeX = Source.GetSizeX();
	int32 SizeY = Source.GetSizeY();
	// Our source may be a LongLat cubemap which is a single texture
	const TCHAR* SourceFaceCount = Source.IsLongLatCubemap() ? TEXT("") : TEXT("x6");
#else
	int32 SizeX = 0;
	int32 SizeY = 0;
	const TCHAR* SourceFaceCount = TEXT("");
#endif

	const FString Dimensions = FString::Printf(TEXT("%dx%d%s"), SizeX, SizeY, SourceFaceCount);
	Context.AddTag( FAssetRegistryTag("Dimensions", Dimensions, FAssetRegistryTag::TT_Dimensional) );
	Context.AddTag( FAssetRegistryTag("Format", GPixelFormats[GetPixelFormat()].Name, FAssetRegistryTag::TT_Alphabetical) );

	Super::GetAssetRegistryTags(Context);
}

void UTextureCube::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
#if WITH_EDITOR
	bool bAsyncFlags = false;
	// Recache platform data if the source has changed.
	if (!EnumHasAnyFlags(InFlags, EUpdateResourceFlags::Synchronous) &&
		FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		bAsyncFlags = true;
	}

	CachePlatformData(bAsyncFlags, bAsyncFlags, bAsyncFlags, nullptr, EnumHasAnyFlags(InFlags, EUpdateResourceFlags::ForceRebuild));
#endif // #if WITH_EDITOR

	// Route to super.
	Super::UpdateResourceWithParams(InFlags);
}

FString UTextureCube::GetDesc()
{
	return FString::Printf(TEXT("Cube: %dx%d [%s]"),
		GetSizeX(),
		GetSizeY(),
		GPixelFormats[GetPixelFormat()].Name
		);
}

uint32 UTextureCube::CalcTextureMemorySize( int32 MipCount ) const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTextureCube(this)->CalcTextureMemorySize(MipCount);
	}
#endif

	uint32 Size = 0;
	if (GetPlatformData())
	{
		int32 SizeX = GetSizeX();
		int32 SizeY = GetSizeY();
		int32 NumMips = GetNumMips();
		EPixelFormat Format = GetPixelFormat();

		ensureMsgf(SizeX == SizeY, TEXT("Cubemap faces expected to be square.  Actual sizes are: %i, %i"), SizeX, SizeY);

		// Figure out what the first mip to use is.
		int32 FirstMip	= FMath::Max( 0, NumMips - MipCount );		
		FIntPoint MipExtents = CalcMipMapExtent(SizeX, SizeY, Format, FirstMip);

		const FRHITextureDesc Desc =
			FRHITextureCreateDesc::CreateCube(TEXT("Temp"), MipExtents.X, Format)
			.SetNumMips(FMath::Max(1, MipCount))
			.SetExtData(GetPlatformData()->GetExtData());

		Size = RHICalcTexturePlatformSize(Desc).Size;
	}
	return Size;
}

uint32 UTextureCube::CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const
{
	if ( Enum == TMC_ResidentMips || Enum == TMC_AllMipsBiased )
	{
		return CalcTextureMemorySize( GetNumMips() - GetCachedLODBias() );
	}
	else
	{
		return CalcTextureMemorySize( GetNumMips() );
	}
}

// While compiling the platform data in editor, we will return the 
// placeholders value to ensure rendering works as expected and that
// there are no thread-unsafe access to the platform data being built.
// Any process requiring a fully up-to-date platform data is expected to
// call FTextureCompiler:Get().FinishCompilation on UTexture first.
int32 UTextureCube::GetSizeX() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCube(this)->GetSizeX();
		}
#endif
		return PrivatePlatformData->SizeX;
	}

	return 0;
}

int32 UTextureCube::GetSizeY() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCube(this)->GetSizeY();
		}
#endif
		return PrivatePlatformData->SizeY;
	}

	return 0;
}

int32 UTextureCube::GetNumMips() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCube(this)->GetNumMips();
		}
#endif
		return PrivatePlatformData->Mips.Num();
	}

	return 0;
}

EPixelFormat UTextureCube::GetPixelFormat() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCube(this)->GetPixelFormat();
		}
#endif
		return PrivatePlatformData->PixelFormat;
	}
	return PF_Unknown;
}

class FTextureCubeResource : public FTextureResource
{
	/** The FName of the LODGroup-specific stat	*/
	FName					LODGroupStatName;

public:
	/**
	 * Minimal initialization constructor.
	 * @param InOwner - The UTextureCube which this FTextureCubeResource represents.
	 */
	FTextureCubeResource(UTextureCube* InOwner)
	: Owner(InOwner)
	, TextureSize(0)
	, ProxiedResource(nullptr)
	{
		//Initialize the MipData array
		for ( int32 FaceIndex=0;FaceIndex<6; FaceIndex++)
		{
			for( int32 MipIndex=0; MipIndex<UE_ARRAY_COUNT(MipData[FaceIndex]); MipIndex++ )
			{
				MipData[FaceIndex][MipIndex] = NULL;
			}
		}

		check(Owner->GetNumMips() > 0);

		TIndirectArray<FTexture2DMipMap>& Mips = InOwner->GetPlatformData()->Mips;
		const int32 FirstMipTailIndex = Mips.Num() - FMath::Max(1, InOwner->GetPlatformData()->GetNumMipsInTail());
		for (int32 MipIndex = 0; MipIndex <= FirstMipTailIndex; MipIndex++)
		{
			FTexture2DMipMap& Mip = Mips[MipIndex];
			if( Mip.BulkData.GetBulkDataSize() <= 0 )
			{
				UE_LOGF(LogTexture, Error, "Corrupt texture [%ls]! Missing bulk data for MipIndex=%d",*InOwner->GetFullName(),MipIndex );
			}
			else			
			{
				TextureSize += Mip.BulkData.GetBulkDataSize();
				uint32 MipSize = Mip.BulkData.GetBulkDataSize() / 6;

				uint8* In = (uint8*)Mip.BulkData.Lock(LOCK_READ_ONLY);

				for(uint32 Face = 0; Face < 6; ++Face)
				{
					MipData[Face][MipIndex] = FMemory::Malloc(MipSize);
					FMemory::Memcpy(MipData[Face][MipIndex], In + MipSize * Face, MipSize);
				}

				Mip.BulkData.Unlock();
			}
		}
		STAT( LODGroupStatName = TextureGroupStatFNames[InOwner->LODGroup] );
	}

	/**
	 * Minimal initialization constructor.
	 * @param InOwner           - The UTextureCube which this FTextureCubeResource represents.
	 * @param InProxiedResource - The resource to proxy.
	 */
	FTextureCubeResource(UTextureCube* InOwner, const FTextureCubeResource* InProxiedResource)
	: Owner(InOwner)
	, TextureSize(0)
	, ProxiedResource(InProxiedResource)
	{
		//Initialize the MipData array
		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			for (int32 MipIndex = 0; MipIndex < UE_ARRAY_COUNT(MipData[FaceIndex]); MipIndex++)
			{
				MipData[FaceIndex][MipIndex] = NULL;
			}
		}
	}

	/**
	 * Destructor, freeing MipData in the case of resource being destroyed without ever 
	 * having been initialized by the rendering thread via InitRHI.
	 */	
	~FTextureCubeResource()
	{
		// Make sure we're not leaking memory if InitRHI has never been called.
		for (int32 i=0; i<6; i++)
		{
			for( int32 MipIndex=0; MipIndex<UE_ARRAY_COUNT(MipData[i]); MipIndex++ )
			{
				// free any mip data that was copied 
				if( MipData[i][MipIndex] )
				{
					FMemory::Free( MipData[i][MipIndex] );
				}
				MipData[i][MipIndex] = NULL;
			}
		}
	}
	
	/**
	 * Called when the resource is initialized. This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (ProxiedResource)
		{
			TextureCubeRHI = ProxiedResource->GetTextureCubeRHI();
			TextureRHI = TextureCubeRHI;
			SamplerStateRHI = ProxiedResource->SamplerStateRHI;
			RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
			return;
		}

		INC_DWORD_STAT_BY( STAT_TextureMemory, TextureSize );
		INC_DWORD_STAT_FNAME_BY( LODGroupStatName, TextureSize );

		// Create the RHI texture.
		const ETextureCreateFlags TexCreateFlags = (Owner->SRGB ? TexCreate_SRGB : TexCreate_None)  | (Owner->bNotOfflineProcessed ? TexCreate_None : TexCreate_OfflineProcessed);
		const FString Name = Owner->GetPathName();
		const static FLazyName ClassName(TEXT("FTextureCubeResource"));

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCube(*Name)
			.SetExtent(Owner->GetSizeX())
			.SetFormat(Owner->GetPixelFormat())
			.SetNumMips(Owner->GetNumMips())
			.SetFlags(TexCreateFlags)
			.SetExtData(Owner->GetPlatformData() ? Owner->GetPlatformData()->GetExtData() : 0)
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		TextureCubeRHI = RHICmdList.CreateTexture(Desc);

		TextureRHI = TextureCubeRHI;
		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		// Read the mip-levels into the RHI texture.
		int32 NumMips = Owner->GetNumMips();
		check(NumMips <= MAX_TEXTURE_MIP_COUNT);
		for( int32 FaceIndex=0; FaceIndex<6; FaceIndex++ )
		{
			for(int32 MipIndex=0; MipIndex < NumMips; MipIndex++)
			{
				if( MipData[FaceIndex][MipIndex] != NULL )
				{
					uint32 DestStride;
					void* TheMipData = RHICmdList.GetAsImmediate().LockTextureCubeFace( TextureCubeRHI, FaceIndex, 0, MipIndex, RLM_WriteOnly, DestStride, false );
					GetData( FaceIndex, MipIndex, TheMipData, DestStride );
					RHICmdList.GetAsImmediate().UnlockTextureCubeFace( TextureCubeRHI, FaceIndex, 0, MipIndex, false );
				}
			}
		}

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter( Owner ),
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		bGreyScaleFormat = UE::TextureDefines::ShouldUseGreyScaleEditorVisualization( Owner->CompressionSettings );
	}

	virtual void ReleaseRHI() override
	{
		DEC_DWORD_STAT_BY( STAT_TextureMemory, TextureSize );
		DEC_DWORD_STAT_FNAME_BY( LODGroupStatName, TextureSize );
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
		TextureCubeRHI.SafeRelease();
		FTextureResource::ReleaseRHI();
	}

	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override
	{
		if (ProxiedResource)
		{
			return ProxiedResource->GetSizeX();
		}

		return Owner->GetSizeX();
	}

	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const override
	{
		if (ProxiedResource)
		{
			return ProxiedResource->GetSizeY();
		}

		return Owner->GetSizeY();
	}

	/**
	 * Accessor
	 * @return Texture2DRHI
	 */
	FTextureRHIRef GetTextureCubeRHI() const
	{
		return TextureCubeRHI;
	}

	virtual bool IsProxy() const override { return ProxiedResource != nullptr; }

	const FTextureCubeResource* GetProxiedResource() const { return ProxiedResource; }
private:
	/** A reference to the texture's RHI resource as a cube-map texture. */
	FTextureRHIRef TextureCubeRHI;

	/** Local copy/ cache of mip data. Only valid between creation and first call to InitRHI */
	void* MipData[6][MAX_TEXTURE_MIP_COUNT];

	/** The UTextureCube which this resource represents. */
	const UTextureCube* Owner;

	// Cached texture size for stats. */
	int32	TextureSize;

	const FTextureCubeResource* const ProxiedResource;
	/**
	 * Writes the data for a single mip-level into a destination buffer.
	 * @param FaceIndex		The index of the face of the mip-level to read.
	 * @param MipIndex		The index of the mip-level to read.
	 * @param Dest			The address of the destination buffer to receive the mip-level's data.
	 * @param DestPitch		Number of bytes per row
	 */
	void GetData( int32 FaceIndex, int32 MipIndex, void* Dest, uint32 DestPitch )
	{
		check(MipIndex < MAX_TEXTURE_MIP_COUNT);
		check( MipData[FaceIndex][MipIndex] );

		// for platforms that returned 0 pitch from Lock, we need to just use the bulk data directly, never do 
		// runtime block size checking, conversion, or the like
		if (DestPitch == 0)
		{
			FMemory::Memcpy(Dest, MipData[FaceIndex][MipIndex], Owner->GetPlatformData()->Mips[MipIndex].BulkData.GetBulkDataSize() / 6);
		}
		else
		{
			EPixelFormat PixelFormat = Owner->GetPixelFormat();
			uint32 NumRows = 0;
			uint32 SrcPitch = 0;
			uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;	// Block width in pixels
			uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;	// Block height in pixels
			uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;

			FIntPoint MipExtent = CalcMipMapExtent(Owner->GetSizeX(), Owner->GetSizeY(), PixelFormat, MipIndex);

			uint32 NumColumns = (MipExtent.X + BlockSizeX - 1) / BlockSizeX;	// Num-of columns in the source data (in blocks)
			NumRows    = (MipExtent.Y + BlockSizeY - 1) / BlockSizeY;	// Num-of rows in the source data (in blocks)
			SrcPitch   = NumColumns * BlockBytes;		// Num-of bytes per row in the source data
				
			SIZE_T MipSizeInBytes = CalcTextureMipMapSize(MipExtent.X, MipExtent.Y, PixelFormat, 0);

			if (SrcPitch == DestPitch)
			{

				// Copy data, not taking into account stride!
				FMemory::Memcpy(Dest, MipData[FaceIndex][MipIndex], MipSizeInBytes);
			}
			else
			{
				// Copy data, taking the stride into account!
				uint8 *Src = (uint8*) MipData[FaceIndex][MipIndex];
				uint8 *Dst = (uint8*) Dest;
				for ( uint32 Row=0; Row < NumRows; ++Row )
				{
					FMemory::Memcpy( Dst, Src, SrcPitch );
					Src += SrcPitch;
					Dst += DestPitch;
				}
				check( (PTRINT(Src) - PTRINT(MipData[FaceIndex][MipIndex])) == PTRINT(MipSizeInBytes) );
			}
		}

		FMemory::Free( MipData[FaceIndex][MipIndex] );
		MipData[FaceIndex][MipIndex] = NULL;
	}
};

FTextureResource* UTextureCube::CreateResource()
{
#if WITH_EDITOR
	if (PrivatePlatformData)
	{
		if (PrivatePlatformData->IsAsyncWorkComplete())
		{
			// Make sure AsyncData has been destroyed in case it still exists to avoid
			// IsDefaultTexture thinking platform data is still being computed.
			PrivatePlatformData->FinishCache();
		}
		else
		{
			FTextureCompilingManager::Get().AddTextures({ this });
			UnlinkStreaming();
			return new FTextureCubeResource(this, (const FTextureCubeResource*)GetDefaultTextureCube(this)->GetResource());
		}
	}
#endif

	FTextureResource* NewResource = NULL;
	if (GetNumMips() > 0)
	{
		NewResource = new FTextureCubeResource(this);
	}
	return NewResource;
}

void UTextureCube::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::Exclusive)
	{
		// Use only loaded mips
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(CalcTextureMemorySizeEnum(TMC_ResidentMips));
	}
	else
	{
		// Use all possible mips
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(CalcTextureMemorySizeEnum(TMC_AllMipsBiased));
	}
}

bool UTextureCube::NeedsLoadForTargetPlatform(const ITargetPlatform* TargetPlatform) const
{
	// TC_EncodedReflectionCapture is no longer used and could be deleted
	if (CompressionSettings == TC_EncodedReflectionCapture)
	{
		const static FName EncodedHDR("EncodedHDR");
		TArray<FName> Formats;

		TargetPlatform->GetReflectionCaptureFormats(Formats);

		return Formats.Contains(EncodedHDR);
	}
	return true;
}

#if WITH_EDITOR
bool UTextureCube::IsDefaultTexture() const
{
	return (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete()) || (GetResource() && GetResource()->IsProxy());
}

uint32 UTextureCube::GetMaximumDimension() const
{
	return GetMaxCubeTextureDimension();
}
#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE

