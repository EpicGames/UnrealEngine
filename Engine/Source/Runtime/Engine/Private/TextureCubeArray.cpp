// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureCubeArray.cpp: UTextureCubeArray implementation.
=============================================================================*/

#include "Engine/TextureCubeArray.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/TextureCube.h"
#include "EngineUtils.h"
#include "EngineLogs.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"
#include "Misc/CoreStats.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderUtils.h"
#include "Stats/StatsTrace.h"
#include "TextureCompiler.h"
#include "TextureResource.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_EDITOR
#include "AsyncCompilationHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCubeArray)

#define LOCTEXT_NAMESPACE "UTextureCubeArray"

class FTextureCubeArrayResource : public FTextureResource
{
	/** The FName of the LODGroup-specific stat	*/
	FName					LODGroupStatName;

public:
	/**
	 * Minimal initialization constructor.
	 * @param InOwner - The UTextureCube which this FTextureCubeResource represents.
	 */
	FTextureCubeArrayResource(UTextureCubeArray* InOwner)
		: Owner(InOwner)
		, TextureSize(0)
		, ProxiedResource(nullptr)
	{
		check(Owner);
		check(Owner->GetNumMips() > 0);

		TIndirectArray<FTexture2DMipMap>& Mips = InOwner->GetPlatformData()->Mips;
		const int32 FirstMipTailIndex = Mips.Num() - FMath::Max(1, InOwner->GetPlatformData()->GetNumMipsInTail());

		NumSlices = Owner->GetNumSlices();
		MipData.Empty(NumSlices * (FirstMipTailIndex + 1));

		for (int32 MipIndex = 0; MipIndex <= FirstMipTailIndex; MipIndex++)
		{
			FTexture2DMipMap& Mip = Mips[MipIndex];
			if (Mip.BulkData.GetBulkDataSize() <= 0)
			{
				UE_LOGF(LogTexture, Error, "Corrupt texture [%ls]! Missing bulk data for MipIndex=%d", *InOwner->GetFullName(), MipIndex);
			}
			else
			{
				TextureSize += Mip.BulkData.GetBulkDataSize();
				uint32 MipSize = Mip.BulkData.GetBulkDataSize() / NumSlices;

				uint8* In = (uint8*)Mip.BulkData.Lock(LOCK_READ_ONLY);
				for (uint32 Face = 0; Face < NumSlices; ++Face)
				{
					void* MipMemory = FMemory::Malloc(MipSize);
					FMemory::Memcpy(MipMemory, In + MipSize * Face, MipSize);
					MipData.Add(MipMemory);
				}
				Mip.BulkData.Unlock();
			}
		}
		STAT(LODGroupStatName = TextureGroupStatFNames[InOwner->LODGroup]);
	}

	/**
	 * Minimal initialization constructor.
	 * @param InOwner           - The UTextureCube which this FTextureCubeResource represents.
	 * @param InProxiedResource - The resource to proxy.
	 */
	FTextureCubeArrayResource(UTextureCubeArray* InOwner, const FTextureCubeArrayResource* InProxiedResource)
		: Owner(InOwner)
		, NumSlices(0u)
		, TextureSize(0)
		, ProxiedResource(InProxiedResource)
	{
		check(Owner);
	}

	/**
	 * Destructor, freeing MipData in the case of resource being destroyed without ever
	 * having been initialized by the rendering thread via InitRHI.
	 */
	~FTextureCubeArrayResource()
	{
		// Make sure we're not leaking memory if InitRHI has never been called.
		for (void* Mip : MipData)
		{
			if (Mip)
			{
				FMemory::Free(Mip);
			}
		}
	}

	/**
	 * Called when the resource is initialized. This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (ProxiedResource)
		{
			TextureCubeRHI = ProxiedResource->GetTextureCubeRHI();
			TextureRHI = TextureCubeRHI;
			SamplerStateRHI = ProxiedResource->SamplerStateRHI;
			RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
			return;
		}

		INC_DWORD_STAT_BY(STAT_TextureMemory, TextureSize);
		INC_DWORD_STAT_FNAME_BY(LODGroupStatName, TextureSize);

		const uint32 ArraySize = NumSlices / 6u;

		check(Owner->GetNumMips() > 0);

		// Create the RHI texture.
		const ETextureCreateFlags TexCreateFlags = (Owner->SRGB ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None) | (Owner->bNotOfflineProcessed ? ETextureCreateFlags::None : ETextureCreateFlags::OfflineProcessed);
		const FString Name = Owner->GetPathName();
		const static FLazyName ClassName(TEXT("FTextureCubeArrayResource"));

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCubeArray(*Name)
			.SetExtent(Owner->GetSizeX())
			.SetArraySize(ArraySize)
			.SetFormat(Owner->GetPixelFormat())
			.SetNumMips(Owner->GetNumMips())
			.SetFlags(TexCreateFlags)
			.SetExtData(Owner->GetPlatformData()->GetExtData())
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		TextureCubeRHI = RHICmdList.CreateTexture(Desc);
		TextureRHI = TextureCubeRHI;

		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		// Read the mip-levels into the RHI texture.
		const int32 FirstMipTailIndex = Owner->GetNumMips() - FMath::Max(1, Owner->GetPlatformData()->GetNumMipsInTail());
		for (int32 MipIndex = 0; MipIndex <= FirstMipTailIndex; MipIndex++)
		{
			for (uint32 ArrayIndex = 0u; ArrayIndex < ArraySize; ++ArrayIndex)
			{
				for (uint32 FaceIndex = 0u; FaceIndex < 6u; ++FaceIndex)
				{
					uint32 DestStride;
					void* TheMipData = RHICmdList.GetAsImmediate().LockTextureCubeFace(TextureCubeRHI, FaceIndex, ArrayIndex, MipIndex, RLM_WriteOnly, DestStride, false);
					GetData(ArrayIndex, FaceIndex, MipIndex, TheMipData, DestStride);
					RHICmdList.GetAsImmediate().UnlockTextureCubeFace(TextureCubeRHI, FaceIndex, ArrayIndex, MipIndex, false);
				}
			}
		}

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(Owner),
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
		
		bGreyScaleFormat = UE::TextureDefines::ShouldUseGreyScaleEditorVisualization( Owner->CompressionSettings );
	}

	virtual void ReleaseRHI() override
	{
		DEC_DWORD_STAT_BY(STAT_TextureMemory, TextureSize);
		DEC_DWORD_STAT_FNAME_BY(LODGroupStatName, TextureSize);
		RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
		TextureCubeRHI.SafeRelease();
		FTextureResource::ReleaseRHI();
	}

	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override
	{
		if (ProxiedResource)
		{
			return ProxiedResource->GetSizeX();
		}

		return Owner->GetSizeX();
	}

	/** Returns the height of the texture in pixels. */
	virtual uint32 GetSizeY() const override
	{
		if (ProxiedResource)
		{
			return ProxiedResource->GetSizeY();
		}

		return Owner->GetSizeY();
	}

	/** Returns the depth of the texture in pixels. */
	virtual uint32 GetSizeZ() const
	{
		if (ProxiedResource)
		{
			return ProxiedResource->GetSizeZ();
		}

		return Owner->GetNumSlices();
	}

	FTextureRHIRef GetTextureCubeRHI() const
	{
		return TextureCubeRHI;
	}

	virtual bool IsProxy() const override { return ProxiedResource != nullptr; }

	const FTextureCubeArrayResource* GetProxiedResource() const { return ProxiedResource; }
private:
	/** A reference to the texture's RHI resource as a cube-map texture. */
	FTextureRHIRef TextureCubeRHI;

	/** Local copy/ cache of mip data. Only valid between creation and first call to InitRHI */
	TArray<void*> MipData;

	/** The UTextureCube which this resource represents. */
	const UTextureCubeArray* Owner;

	/** Number of 2D faces per mip, equal to array size  6 */
	uint32 NumSlices;

	// Cached texture size for stats. */
	int32	TextureSize;

	const FTextureCubeArrayResource* const ProxiedResource;
	/**
	 * Writes the data for a single mip-level into a destination buffer.
	 * @param FaceIndex		The index of the face of the mip-level to read.
	 * @param MipIndex		The index of the mip-level to read.
	 * @param Dest			The address of the destination buffer to receive the mip-level's data.
	 * @param DestPitch		Number of bytes per row
	 */
	void GetData(uint32 SliceIndex, int32 FaceIndex, int32 MipIndex, void* Dest, uint32 DestPitch)
	{
		// Mips are stored sequentially
		// For each mip, we store 6 faces per array index
		const uint32 Index = MipIndex * NumSlices + SliceIndex * 6u + FaceIndex;
		check(MipData[Index]);

		// for platforms that returned 0 pitch from Lock, we need to just use the bulk data directly, never do 
		// runtime block size checking, conversion, or the like
		if (DestPitch == 0)
		{
			FMemory::Memcpy(Dest, MipData[Index], Owner->GetPlatformData()->Mips[MipIndex].BulkData.GetBulkDataSize() / Owner->GetNumSlices());
		}
		else
		{
			EPixelFormat PixelFormat = Owner->GetPixelFormat();
			uint32 NumRows = 0;
			uint32 SrcPitch = 0;
			uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;	// Block width in pixels
			uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;	// Block height in pixels
			uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;

			FIntPoint MipExtent = CalcMipMapExtent(Owner->GetSizeX(), Owner->GetSizeY(), PixelFormat, MipIndex);

			uint32 NumColumns = (MipExtent.X + BlockSizeX - 1) / BlockSizeX;	// Num-of columns in the source data (in blocks)
			NumRows = (MipExtent.Y + BlockSizeY - 1) / BlockSizeY;	// Num-of rows in the source data (in blocks)
			SrcPitch = NumColumns * BlockBytes;		// Num-of bytes per row in the source data

			SIZE_T MipSizeInBytes = CalcTextureMipMapSize(MipExtent.X, MipExtent.Y, PixelFormat, 0);

			if (SrcPitch == DestPitch)
			{
				// Copy data, not taking into account stride!
				FMemory::Memcpy(Dest, MipData[Index], MipSizeInBytes);
			}
			else
			{
				// Copy data, taking the stride into account!
				uint8* Src = (uint8*)MipData[Index];
				uint8* Dst = (uint8*)Dest;
				for (uint32 Row = 0; Row < NumRows; ++Row)
				{
					FMemory::Memcpy(Dst, Src, SrcPitch);
					Src += SrcPitch;
					Dst += DestPitch;
				}
				check((PTRINT(Src) - PTRINT(MipData[Index])) == PTRINT(MipSizeInBytes));
			}
		}

		FMemory::Free(MipData[Index]);
		MipData[Index] = nullptr;
	}
};

UTextureCubeArray* UTextureCubeArray::CreateTransient(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat, const FName InName)
{
	UTextureCubeArray* NewTexture = nullptr;
	if (InSizeX > 0 && InSizeY > 0 && InArraySize > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0)
	{
		NewTexture = NewObject<UTextureCubeArray>(
			GetTransientPackage(),
			InName,
			RF_Transient
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->SetIsCubemap(true);
		NewTexture->GetPlatformData()->SetNumSlices(InArraySize * 6);
		NewTexture->GetPlatformData()->PixelFormat = InFormat;
		NewTexture->bNotOfflineProcessed = true;

		// Allocate first mipmap.
		int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY, InArraySize * 6);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc((int64)GPixelFormats[InFormat].BlockBytes * NumBlocksX * NumBlocksY * 6 * InArraySize);
		Mip->BulkData.Unlock();
	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UTextureCubeArray::CreateTransient()");
	}
	return NewTexture;
}

/**
 * Get the optimal placeholder to use during texture compilation
 */
static UTextureCubeArray* GetDefaultTextureCubeArray(const UTextureCubeArray* Texture)
{
	static TStrongObjectPtr<UTextureCubeArray> CheckerboardTexture;
	if (!CheckerboardTexture.IsValid())
	{
		CheckerboardTexture.Reset(FImageUtils::CreateCheckerboardTextureCubeArray(FColor(200, 200, 200, 128), FColor(128, 128, 128, 128)));
	}
	return CheckerboardTexture.Get();
}

UTextureCubeArray::UTextureCubeArray(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PrivatePlatformData(nullptr)
{
#if WITH_EDITORONLY_DATA
	SRGB = true;
	MipGenSettings = TMGS_NoMipmaps;
#endif
}

FTexturePlatformData** UTextureCubeArray::GetRunningPlatformData()
{
	// @todo DC GetRunningPlatformData is fundamentally unsafe but almost unused... should we replace it with Get/SetRunningPlatformData directly in the base class
	return &PrivatePlatformData;
}

void UTextureCubeArray::SetPlatformData(FTexturePlatformData* InPlatformData)
{
	if (PrivatePlatformData)
	{
		ReleaseResource();
		delete PrivatePlatformData;
	}
	PrivatePlatformData = InPlatformData;
}

// Any direct access to GetPlatformData() will stall until the structure
// is safe to use. It is advisable to replace those use case with
// async aware code to avoid stalls where possible.
const FTexturePlatformData* UTextureCubeArray::GetPlatformData() const
{
#if WITH_EDITOR
	if (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UTextureCubeArray::GetPlatformDataStall);
		UE_LOGF(LogTexture, Log, "Call to GetPlatformData() is forcing a wait on data that is not yet ready.");

		FText Msg = FText::Format(LOCTEXT("WaitOnTextureCompilation", "Waiting on texture compilation {0} ..."), FText::FromString(GetName()));
		FScopedSlowTask Progress(1.f, Msg, true);
		Progress.MakeDialog(true);
		uint64 StartTime = FPlatformTime::Cycles64();
		PrivatePlatformData->FinishCache();
		AsyncCompilationHelpers::SaveStallStack(FPlatformTime::Cycles64() - StartTime);
	}
#endif // #if WITH_EDITOR

	return PrivatePlatformData;
}

FTexturePlatformData* UTextureCubeArray::GetPlatformData()
{
	// For now, this is the same implementation as the const version.
	const UTextureCubeArray* ConstThis = this;
	return const_cast<FTexturePlatformData*>(ConstThis->GetPlatformData());
}

FTextureResource* UTextureCubeArray::CreateResource()
{
#if WITH_EDITOR
	if (PrivatePlatformData)
	{
		if (PrivatePlatformData->IsAsyncWorkComplete())
		{
			// Make sure AsyncData has been destroyed in case it still exists to avoid
			// IsDefaultTexture thinking platform data is still being computed.
			PrivatePlatformData->FinishCache();
		}
		else
		{
			FTextureCompilingManager::Get().AddTextures({ this });
			UnlinkStreaming();
			return new FTextureCubeArrayResource(this, (const FTextureCubeArrayResource*)GetDefaultTextureCubeArray(this)->GetResource());
		}
	}
#endif

	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat()];
	if (GetNumMips() > 0 && FormatInfo.Supported)
	{
		return new FTextureCubeArrayResource(this);
	}
#if WITH_EDITORONLY_DATA
	else if (!SourceTextures.Num())
	{
		// empty arrays don't have built mips
		return nullptr;
	}
#endif
	else if (GetNumMips() == 0)
	{
		UE_LOGF(LogTexture, Warning, "%ls contains no miplevels! Please delete.", *GetFullName());
	}
	else if (!FormatInfo.Supported)
	{
		UE_LOGF(LogTexture, Warning, "%ls cannot be created, rhi does not support format %ls.", *GetFullName(), FormatInfo.Name);
	}
	return nullptr;
}

void UTextureCubeArray::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
#if WITH_EDITOR
	bool bAsyncFlags = false;
	// Recache platform data if the source has changed.
	if (!EnumHasAnyFlags(InFlags, EUpdateResourceFlags::Synchronous) &&
		FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		bAsyncFlags = true;
	}

	CachePlatformData(bAsyncFlags, bAsyncFlags, bAsyncFlags, nullptr, EnumHasAnyFlags(InFlags, EUpdateResourceFlags::ForceRebuild));
#endif // #if WITH_EDITOR

#if WITH_EDITORONLY_DATA
	bSourceGeneratedFromSourceTexturesArray = !Source.GetNumSlices() || SourceTextures.Num();
#endif

	Super::UpdateResourceWithParams(InFlags);
}

uint32 UTextureCubeArray::CalcTextureMemorySize(int32 MipCount) const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultTextureCubeArray(this)->CalcTextureMemorySize(MipCount);
	}
#endif

	uint32 Size = 0;
	if (GetPlatformData())
	{
		int32 SizeX = GetSizeX();
		int32 SizeY = GetSizeY();
		const int32 ArraySize = GetNumSlices() / 6;
		int32 NumMips = GetNumMips();
		EPixelFormat Format = GetPixelFormat();

		ensureMsgf(SizeX == SizeY, TEXT("Cubemap faces expected to be square.  Actual sizes are: %i, %i"), SizeX, SizeY);

		// Figure out what the first mip to use is.
		int32 FirstMip = FMath::Max(0, NumMips - MipCount);
		FIntPoint MipExtents = CalcMipMapExtent(SizeX, SizeY, Format, FirstMip);

		const FRHITextureDesc Desc =
			FRHITextureCreateDesc::CreateCubeArray(TEXT("Temp"), MipExtents.X, ArraySize, Format)
			.SetNumMips(FMath::Max(1, MipCount))
			.SetExtData(GetPlatformData()->GetExtData());

		Size = RHICalcTexturePlatformSize(Desc).Size;
	}
	return Size;
}

uint32 UTextureCubeArray::CalcTextureMemorySizeEnum(ETextureMipCount Enum) const
{
	if (Enum == TMC_ResidentMips || Enum == TMC_AllMipsBiased)
	{
		return CalcTextureMemorySize(GetNumMips() - GetCachedLODBias());
	}

	return CalcTextureMemorySize(GetNumMips());
}

// While compiling the platform data in editor, we will return the 
// placeholders value to ensure rendering works as expected and that
// there are no thread-unsafe access to the platform data being built.
// Any process requiring a fully up-to-date platform data is expected to
// call FTextureCompiler:Get().FinishCompilation on UTexture first.
int32 UTextureCubeArray::GetSizeX() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCubeArray(this)->GetSizeX();
		}
#endif
		return PrivatePlatformData->SizeX;
	}
	return 0;
}

int32 UTextureCubeArray::GetSizeY() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCubeArray(this)->GetSizeY();
		}
#endif
		return PrivatePlatformData->SizeY;
	}
	return 0;
}

int32 UTextureCubeArray::GetNumSlices() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCubeArray(this)->GetNumSlices();
		}
#endif
		return PrivatePlatformData->GetNumSlices();
	}
	return 0;
}

int32 UTextureCubeArray::GetNumMips() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCubeArray(this)->GetNumMips();
		}
#endif
		return PrivatePlatformData->Mips.Num();
	}
	return 0;
}

EPixelFormat UTextureCubeArray::GetPixelFormat() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultTextureCubeArray(this)->GetPixelFormat();
		}
#endif
		return PrivatePlatformData->PixelFormat;
	}
	return PF_Unknown;
}

#if WITH_EDITOR

ENGINE_API bool UTextureCubeArray::CheckArrayTexturesCompatibility()
{
	if (SourceTextures.Num() == 0)
	{
		return false;
	}

	for (int32 TextureIndex = 0; TextureIndex < SourceTextures.Num(); ++TextureIndex)
	{
		// Do not create array till all texture slots are filled.
		if (!SourceTextures[TextureIndex] || ! SourceTextures[TextureIndex]->Source.IsValid())
		{
			return false;
		}

		// Force the async texture build to complete
		// TODO - better way to do this?
		SourceTextures[TextureIndex]->GetPlatformData();
	}

	const FTextureSource& TextureSource = SourceTextures[0]->Source;
	const FString TextureName = SourceTextures[0]->GetName();
	const int32 SizeX = TextureSource.GetSizeX();
	const int32 SizeY = TextureSource.GetSizeY();
	const int32 NumSlices = TextureSource.GetNumSlices();

	ensure(NumSlices == 1 || NumSlices == 6); // either cubemap, or lat/long map

	bool bError = false;
	for (int32 TextureCmpIndex =1; TextureCmpIndex < SourceTextures.Num(); ++TextureCmpIndex)
	{
		const FTextureSource& TextureSourceCmp = SourceTextures[TextureCmpIndex]->Source;
		const FString TextureNameCmp = SourceTextures[TextureCmpIndex]->GetName();

		check( TextureSourceCmp.IsValid() );

		if (TextureSourceCmp.GetSizeX() != SizeX || TextureSourceCmp.GetSizeY() != SizeY)
		{
			UE_LOGF(LogTexture, Warning, "TextureCubeArray creation failed. Textures %ls and %ls have different sizes.", *TextureName, *TextureNameCmp);
			bError = true;
		}

		if (TextureSourceCmp.GetNumSlices() != NumSlices)
		{
			UE_LOGF(LogTexture, Warning, "TextureCubeArray creation failed. Textures %ls and %ls have different number of faces (some are long/lat, some are not).", *TextureName, *TextureNameCmp);
			bError = true;
		}
	}

	return (!bError);
}

ENGINE_API bool UTextureCubeArray::CheckLastTextureCompatibility()
{
	// If no last texture exists, return true -- can't have a compatibility mismatch if nothing is there.
	if (!SourceTextures.Num() || !SourceTextures.Last())
	{
		return true;
	}

	// Reject an explicitly invalid last texture
	const UTextureCube* LastTexture = SourceTextures.Last();
	if (!LastTexture->Source.IsValid())
	{
		return false;
	}
	const FTextureSource& LastSource = LastTexture->Source;

	// Check if the last matches the first valid
	for (UTextureCube* SourceTexture : SourceTextures)
	{
		if (SourceTexture && SourceTexture->Source.IsValid())
		{
			const FTextureSource& FirstValidSource = SourceTexture->Source;
			return
				FirstValidSource.GetSizeX() == LastSource.GetSizeX() &&
				FirstValidSource.GetSizeY() == LastSource.GetSizeY() &&
				FirstValidSource.GetNumSlices() == LastSource.GetNumSlices();
		}
	}

	return true;
}

ENGINE_API bool UTextureCubeArray::UpdateSourceFromSourceTextures(bool bCreatingNewTexture)
{
	// This should be false when texture is updated to avoid overriding user settings.
	if (bCreatingNewTexture)
	{
		// Pull settings from the first valid texture.  We do this before CheckArrayTexturesCompatibility, to support
		// cases where the array is initialized with multiple initially null elements, and the first non-null element
		// is set (CheckArrayTexturesCompatibility will return false if there are any null elements).
		for (UTextureCube* SourceTexture : SourceTextures)
		{
			if (SourceTexture && SourceTexture->Source.IsValid())
			{
				CompressionSettings = SourceTexture->CompressionSettings;
				MipGenSettings = TMGS_NoMipmaps;
				PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
				LODGroup = SourceTexture->LODGroup;
				NeverStream = true;
				break;
			}
		}
	}

	if (!CheckArrayTexturesCompatibility())
	{
		return false;
	}
	
	Modify();

	const FTextureSource& InitialSource = SourceTextures[0]->Source;
	check( InitialSource.IsValid() ); // checked by CheckArrayTexturesCompatibility

	// X,Y,Z size of the array.
	const int32 SizeX = InitialSource.GetSizeX();
	const int32 SizeY = InitialSource.GetSizeY();
	const int32 NumSlices = InitialSource.GetNumSlices();
	const uint32 ArraySize = SourceTextures.Num();

	// Detect format and gamma mismatches across source textures, finding a common format that can represent all
	// sources without information loss (ported from UTexture2DArray::UpdateSourceFromSourceTextures).
	ETextureSourceFormat Format = InitialSource.GetFormat();
	bool bSRGB = InitialSource.GetGammaSpace(0) == EGammaSpace::sRGB;
	bool bMismatchedFormats = false;
	bool bMismatchedGammaSpace = false;

	for (int32 SourceTextureIndex = 1; SourceTextureIndex < SourceTextures.Num(); ++SourceTextureIndex)
	{
		const FTextureSource& OtherSource = SourceTextures[SourceTextureIndex]->Source;

		if ((OtherSource.GetGammaSpace(0) == EGammaSpace::sRGB) != bSRGB)
		{
			bMismatchedGammaSpace = true;
			bSRGB = false;
			Format = TSF_RGBA32F;
		}

		if (Format != OtherSource.GetFormat())
		{
			bMismatchedFormats = true;
			Format = FImageCoreUtils::GetCommonSourceFormat(Format, OtherSource.GetFormat());
		}
	}

	if (bMismatchedGammaSpace)
	{
		UE_LOGF(LogTexture, Log, "TextureCubeArray: mismatched source gamma spaces, converting all to RGBA32F/Linear ...");
	}
	else if (bMismatchedFormats)
	{
		UE_LOGF(LogTexture, Log, "TextureCubeArray: mismatched source pixel formats, converting all to %ls/%s ...",
			ERawImageFormat::GetName(FImageCoreUtils::ConvertToRawImageFormat(Format)), bSRGB ? "sRGB" : "Linear");
	}

	SRGB = bSRGB;

	// Create the source texture for this UTexture.
	// Only copy the first mip from the source textures to array texture.
	Source.Init(SizeX, SizeY, ArraySize * NumSlices, 1, Format);
	Source.bLongLatCubemap = (NumSlices == 1);

	// this path sets bLongLatCubemap for CubeArray
	//  most paths do not, so it is not reliable

	uint8* DestMipData = Source.LockMip(0);
	const int64 MipSizePerElement = Source.CalcMipSize(0) / ArraySize;

	for (int32 SourceTexIndex = 0; SourceTexIndex < SourceTextures.Num(); ++SourceTexIndex)
	{
		FTextureSource& TextureSource = SourceTextures[SourceTexIndex]->Source;
		check( TextureSource.IsValid() ); // checked by CheckArrayTexturesCompatibility
		void* DestSliceData = DestMipData + MipSizePerElement * SourceTexIndex;

		if (TextureSource.GetFormat() == Format && ((TextureSource.GetGammaSpace(0) == EGammaSpace::sRGB) == bSRGB))
		{
			// Fast path: format and gamma match, direct copy.
			TArray64<uint8> RefCubeData;
			TextureSource.GetMipData(RefCubeData, 0);
			check(RefCubeData.Num() == MipSizePerElement);
			FMemory::Memcpy(DestSliceData, RefCubeData.GetData(), MipSizePerElement);
		}
		else
		{
			// Slow path: convert format/gamma via FImage.
			FImage SourceImage;
			TextureSource.GetMipImage(SourceImage, 0);

			if (TextureSource.GetFormat() != Format)
			{
				// When gamma space is mismatched we fall back to TSF_RGBA32F which is always linear,
				// so no separate gamma conversion is needed in that case.
				FImage ConvertedImage;
				SourceImage.CopyTo(ConvertedImage, FImageCoreUtils::ConvertToRawImageFormat(Format), bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
				ConvertedImage.Swap(SourceImage);
			}

			check(SourceImage.RawData.Num() == MipSizePerElement);
			FMemory::Memcpy(DestSliceData, SourceImage.RawData.GetData(), MipSizePerElement);
		}
	}

	Source.UnlockMip(0);

	SetLightingGuid();
	ValidateSettingsAfterImportOrEdit();
	UpdateResource();
	
	return true;
}

ENGINE_API void UTextureCubeArray::InvalidateTextureSource()
{
	Modify();

	SetPlatformData(nullptr);

	Source = FTextureSource();
	Source.SetOwner(this);

	UpdateResource();
}
#endif

void UTextureCubeArray::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UTextureCubeArray::Serialize"), STAT_TextureCubeArray_Serialize, STATGROUP_LoadTime);

	Super::Serialize(Ar);

	FStripDataFlags StripFlags(Ar);
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (bCooked || Ar.IsCooking())
	{
		SerializeCookedPlatformData(Ar);
	}
}

void UTextureCubeArray::PostLoad()
{
#if WITH_EDITOR
	if (FApp::CanEverRender())
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
			BeginCachePlatformData();
		}
		else
		{
			FinishCachePlatformData();
		}
	}
#endif // #if WITH_EDITOR
#if WITH_EDITORONLY_DATA
	bSourceGeneratedFromSourceTexturesArray = !Source.GetNumSlices() || SourceTextures.Num();
#endif

	Super::PostLoad();
};

void UTextureCubeArray::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UTextureCubeArray::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	int32 SizeX = Source.GetSizeX();
	int32 SizeY = Source.GetSizeY();
	// Our sources may be LongLat cubemaps which are single textures
	bool bSourceIsLongLat = Source.IsLongLatCubemap();
	const TCHAR* SourceFaceCount = bSourceIsLongLat ? TEXT("") : TEXT("x6"); 
	int32 ArraySize = bSourceIsLongLat ? Source.GetNumSlices() : Source.GetNumSlices() / 6;
#else
	int32 SizeX = 0;
	int32 SizeY = 0;
	const TCHAR* SourceFaceCount = TEXT(""); 
	int32 ArraySize = 0;
#endif
	const FString Dimensions = FString::Printf(TEXT("%dx%d%s*%d"), SizeX, SizeY, SourceFaceCount, ArraySize);
	Context.AddTag(FAssetRegistryTag("Dimensions", Dimensions, FAssetRegistryTag::TT_Dimensional));
	Context.AddTag(FAssetRegistryTag("Format", GPixelFormats[GetPixelFormat()].Name, FAssetRegistryTag::TT_Alphabetical));

	Super::GetAssetRegistryTags(Context);
}

FString UTextureCubeArray::GetDesc()
{
	return FString::Printf(TEXT("CubeArray: %dx%d*%d [%s]"),
		GetSizeX(),
		GetSizeY(),
		GetNumSlices() / 6,
		GPixelFormats[GetPixelFormat()].Name
	);
}

void UTextureCubeArray::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddUnknownMemoryBytes(CalcTextureMemorySizeEnum(TMC_ResidentMips));
}

#if WITH_EDITOR

bool UTextureCubeArray::IsDefaultTexture() const
{
	return (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete()) || (GetResource() && GetResource()->IsProxy());
}

uint32 UTextureCubeArray::GetMaximumDimension() const
{
	return GetMaxCubeTextureDimension();

}

ENGINE_API void UTextureCubeArray::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTextureCubeArray, SourceTextures))
	{
		int32 NumValid = 0;
		int32 NumRemoved = 0;
		for (int i=0; i < SourceTextures.Num(); i++)
		{
			UTextureCube * SourceTexture = SourceTextures[i];
			if (SourceTexture)
			{
				if (SourceTexture->Source.IsValid())
				{
					NumValid++;
				}
				else
				{
					UE_LOGF(LogTexture, Warning, "Texture has no Source, cannot be used [%ls]", *SourceTexture->GetFullName());
					SourceTextures[i] = nullptr;
					NumRemoved++;
				}
			}
		}

		// Empty SourceTextures, remove any resources if present.
		if (NumValid == 0)
		{
			InvalidateTextureSource();
		}
		// First valid entry into an empty texture array -- indicate to UpdateSourceFromSourceTextures that we are creating a new texture.
		else if (NumValid == 1)
		{
			// Except in the rare situation where an item was removed for being invalid above, don't treat that as creating a new texture.
			const bool bCreatingNewTexture = NumRemoved == 0;
			UpdateSourceFromSourceTextures(bCreatingNewTexture);
		}
		// Try to update from all source textures.
		else if (UpdateSourceFromSourceTextures(false) == false)
		{
			// If this is a change to the last element, and it's invalid, remove it to make the array valid.  Arrays don't support calling
			// OnShouldFilterAsset on child properties of the array, so this is a backstop to clean up such cases on newly added elements.
			// Invalid edits to an existing last element should be blocked by the filters, so this case should only be reached on an add.
			int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
			int32 LastIndex = SourceTextures.Num() - 1;

			if (ChangedIndex == LastIndex && !CheckLastTextureCompatibility())
			{
				SourceTextures.RemoveAt(LastIndex);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
