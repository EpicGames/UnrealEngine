// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureDerivedData.cpp: Derived data management for textures.
=============================================================================*/

#include "Algo/AllOf.h"
#include "EngineLogs.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "UObject/Package.h"
#include "GlobalRenderResources.h"
#include "TextureResource.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "TextureDerivedDataTask.h"
#include "Streaming/TextureStreamingHelpers.h"
#include "Engine/VolumeTexture.h"
#include "VT/VirtualTextureBuildSettings.h"
#include "VT/VirtualTextureBuiltData.h"
#include "Misc/ConfigCacheIni.h"
#include "RenderingThread.h"
#include "Interfaces/ITextureFormat.h"

#if WITH_EDITOR

#include "ChildTextureFormat.h"
#include "ColorManagement/ColorSpace.h"
#include "Compression/OodleDataCompressionUtil.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataRequestOwner.h"
#include "Hash/xxhash.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ProfilingDebugging/CookStats.h"
#include "UObject/ArchiveCookContext.h"
#include "VT/LightmapVirtualTexture.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryWriter.h"
#include "TextureBuildUtilities.h"
#include "TextureCompiler.h"
#include "TextureCompressorModule.h"
#include "TextureEncodingSettings.h"

static TAutoConsoleVariable<int32> CVarTexturesCookToDerivedDataReferences(
	TEXT("r.TexturesCookToDerivedDataReferences"),
	0,
	TEXT("Whether cooked textures are serialized using Derived Data References."),
	ECVF_ReadOnly);

/*------------------------------------------------------------------------------
	Versioning for texture derived data.
------------------------------------------------------------------------------*/

// The current version string is set up to mimic the old versioning scheme and to make
// sure the DDC does not get invalidated right now. If you need to bump the version, replace it
// with a guid ( ex.: TEXT("855EE5B3574C43ABACC6700C4ADC62E6") )
// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new
// guid as version
// this is put in the DDC1 and the DDC2 key

// next time this changes clean up SerializeForKey todo marks , search "@todo SerializeForKey"
#define TEXTURE_DERIVEDDATA_VER		TEXT("95BCE5A0BFB949539A18684748C633C9")

// This GUID is mixed into DDC version for virtual textures only, this allows updating DDC version for VT without invalidating DDC for all textures
// This is useful during development, but once large numbers of VT are present in shipped content, it will have the same problem as TEXTURE_DERIVEDDATA_VER
// This is put in the DDC1 key but NOT in the DDC2 key
// VT key bumped 02-27-2024 for Alpha change
#define TEXTURE_VT_DERIVEDDATA_VER	TEXT("7C16439390E24F1F9468894FB4D4BC55")

// TEXTURE_DDC_STB_IMAGE_RESIZE_VERSION should change whenever the stb_image_resize2.h version number changes
//	*if* it is a version change that changes output
//	if it's just a performance/compile fix that doesn't change output, do not change this version number
#define TEXTURE_DDC_STB_IMAGE_RESIZE_VERSION  TEXT("2.06")

// This GUID is mixed in for textures that are involved in shared linear encoded textures - both base and child. It's used
// to rebuild textures affects by shared linear in the case of bugs that only affect such textures so we don't force a global
// rebuild. This is in both texture build paths.
static const FGuid GTextureSLEDerivedDataVer(0xBD855730U, 0xA5B44BBBU, 0x89D051D0U, 0x695AC618U);
const FGuid& GetTextureSLEDerivedDataVersion() { return GTextureSLEDerivedDataVer; }

static bool IsUsingNewDerivedData()
{
	struct FTextureDerivedDataSetting
	{
		FTextureDerivedDataSetting()
		{
			bUseNewDerivedData = FParse::Param(FCommandLine::Get(), TEXT("DDC2AsyncTextureBuilds")) || FParse::Param(FCommandLine::Get(), TEXT("DDC2TextureBuilds"));
			if (!bUseNewDerivedData)
			{
				GConfig->GetBool(TEXT("TextureBuild"), TEXT("NewTextureBuilds"), bUseNewDerivedData, GEditorIni);
			}
			UE_CLOGF(bUseNewDerivedData, LogTexture, Log, "Using new texture derived data builds.");
		}
		bool bUseNewDerivedData;
	};
	static const FTextureDerivedDataSetting TextureDerivedDataSetting;
	return TextureDerivedDataSetting.bUseNewDerivedData;
}


#if ENABLE_COOK_STATS
namespace TextureCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStats::FDDCResourceUsageStats StreamingMipUsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("Texture.Usage"), TEXT("Inline"));
		StreamingMipUsageStats.LogStats(AddStat, TEXT("Texture.Usage"), TEXT("Streaming"));
	});
}
#endif

/*------------------------------------------------------------------------------
	Derived data key generation.
------------------------------------------------------------------------------*/

/**
 * Serialize build settings for use when generating the derived data key. (DDC1)
 * Must keep in sync with DDC2 Key WriteBuildSettings
 */
static void SerializeForKey(FArchive& Ar, const FTextureBuildSettings& Settings)
{
	uint32 TempUint32;
	float TempFloat;
	uint8 TempByte;
	FColor TempColor;
	FVector2f TempVector2f;
	FVector4f TempVector4f;
	UE::Color::FColorSpace TempColorSpace;
	FGuid TempGuid;
	FName TempName;

	TempFloat = Settings.ColorAdjustment.AdjustBrightness; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustBrightnessCurve; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustSaturation; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustVibrance; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustRGBCurve; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustHue; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustMinAlpha; Ar << TempFloat;
	TempFloat = Settings.ColorAdjustment.AdjustMaxAlpha; Ar << TempFloat;
	TempFloat = Settings.MipSharpening; Ar << TempFloat;
	TempUint32 = Settings.DiffuseConvolveMipLevel; Ar << TempUint32;
	TempUint32 = Settings.SharpenMipKernelSize; Ar << TempUint32;
	// NOTE: TextureFormatName is not stored in the key here.
	// NOTE: bHDRSource is not stored in the key here.
	TempByte = Settings.MipGenSettings; Ar << TempByte;
	TempByte = Settings.bCubemap; Ar << TempByte;
	TempByte = Settings.bTextureArray; Ar << TempByte;
	TempByte = Settings.bSRGB ? (Settings.bSRGB | ( Settings.bUseLegacyGamma ? 0 : 0x2 )) : 0; Ar << TempByte;

	if (Settings.SourceEncodingOverride != 0 /*UE::Color::EEncoding::None*/)
	{
		TempUint32 = UE::Color::ENCODING_TYPES_VER; Ar << TempUint32;
		TempByte = Settings.SourceEncodingOverride; Ar << TempByte;
	}

	if (Settings.bHasColorSpaceDefinition)
	{
		TempUint32 = UE::Color::COLORSPACE_VER; Ar << TempUint32;
		TempColorSpace = UE::Color::FColorSpace::GetWorking(); Ar << TempColorSpace;

		TempVector2f = FVector2f(Settings.RedChromaticityCoordinate); Ar << TempVector2f;
		TempVector2f = FVector2f(Settings.GreenChromaticityCoordinate); Ar << TempVector2f;
		TempVector2f = FVector2f(Settings.BlueChromaticityCoordinate); Ar << TempVector2f;
		TempVector2f = FVector2f(Settings.WhiteChromaticityCoordinate); Ar << TempVector2f;
		TempByte = Settings.ChromaticAdaptationMethod; Ar << TempByte;
	}

	if (Settings.SourceEncodingOverride != 0 || Settings.bHasColorSpaceDefinition)
	{
		TempUint32 = FTextureBuildSettings::GetOpenColorIOVersion(); Ar << TempUint32;
	}

	TempByte = Settings.bPreserveBorder; Ar << TempByte;

	// bDitherMipMapAlpha was removed from Texture
	//  serialize to DDC as if it was still around and false to keep keys the same:
	uint8 bDitherMipMapAlpha = 0;
	TempByte = bDitherMipMapAlpha; Ar << TempByte;

	if (Settings.bDoScaleMipsForAlphaCoverage)
	{
		check( Settings.AlphaCoverageThresholds != FVector4f(0, 0, 0, 0) );
		TempVector4f = Settings.AlphaCoverageThresholds; Ar << TempVector4f;
	}
	
	// Bokeh output version number bumped when processing changes
	TempByte = Settings.bComputeBokehAlpha ? 3 : 0; Ar << TempByte;
	TempByte = Settings.bReplicateRed; Ar << TempByte;
	TempByte = Settings.bReplicateAlpha; Ar << TempByte;
	TempByte = Settings.bDownsampleWithAverage; Ar << TempByte;
	
	{
		TempByte = Settings.bSharpenWithoutColorShift;

		if(Settings.bSharpenWithoutColorShift && Settings.MipSharpening != 0.0f)
		{
			// @todo SerializeForKey these can go away whenever we bump the overall ddc key
			// bSharpenWithoutColorShift prevented alpha sharpening. This got fixed
			// Here we update the key to get those cases recooked.
			TempByte = 2;
		}

		Ar << TempByte;
	}

	TempByte = Settings.bBorderColorBlack; Ar << TempByte;
	TempByte = Settings.bFlipGreenChannel; Ar << TempByte;
	TempByte = Settings.bApplyKernelToTopMip; Ar << TempByte;
	TempByte = Settings.CompositeTextureMode; Ar << TempByte;
	TempFloat = Settings.CompositePower; Ar << TempFloat;
	TempUint32 = Settings.MaxTextureResolution; Ar << TempUint32;
	TempByte = Settings.PowerOfTwoMode; Ar << TempByte;
	TempColor = Settings.PaddingColor; Ar << TempColor;
	TempByte = Settings.bChromaKeyTexture; Ar << TempByte;
	TempColor = Settings.ChromaKeyColor; Ar << TempColor;
	TempFloat = Settings.ChromaKeyThreshold; Ar << TempFloat;
	
	if ( Settings.PowerOfTwoMode >= ETexturePowerOfTwoSetting::Type::StretchToPowerOfTwo )
	{
		// Stretch power of two modes ResizeImage changed 10-31-2023
		TempName = TEXTURE_DDC_STB_IMAGE_RESIZE_VERSION;
		Ar << TempName;
	}

	// Avoid changing key for non-VT enabled textures
	if (Settings.bVirtualStreamable)
	{
		TempByte = Settings.bVirtualStreamable; Ar << TempByte;
		TempByte = Settings.VirtualAddressingModeX; Ar << TempByte;
		TempByte = Settings.VirtualAddressingModeY; Ar << TempByte;
		TempUint32 = Settings.VirtualTextureTileSize; Ar << TempUint32;
		TempUint32 = Settings.VirtualTextureBorderSize; Ar << TempUint32;
		// compresion options removed: keep serializing them as "off" to keep the key the same:
		TempByte = 0; Ar << TempByte;
		TempByte = 0; Ar << TempByte;
		TempByte = Settings.LossyCompressionAmount; Ar << TempByte; // Lossy compression currently only used by VT
		TempByte = Settings.bApplyYCoCgBlockScale; Ar << TempByte; // YCoCg currently only used by VT

		
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key:
		if ( Settings.bSRGB && Settings.bUseLegacyGamma )
		{
			// processing changed, modify ddc key :
			TempGuid = FGuid(0xA227BEFC,0x9F8643C6,0x81580369,0xC4C6F73E);
			Ar << TempGuid;
		}
	}

	// Avoid changing key if texture is not being downscaled
	if (Settings.Downscale > 1.0)
	{
		TempFloat = Settings.Downscale; Ar << TempFloat;
		TempByte = Settings.DownscaleOptions; Ar << TempByte;

		if ( Settings.bUseNewMipFilter )
		{
			// downscale behavior changed to use ResizeImage
			TempName = TEXTURE_DDC_STB_IMAGE_RESIZE_VERSION;
			Ar << TempName;
			TempName = TEXT("Downscale ResizeImage changed 02-29-2024");
			Ar << TempName;
		}
	}

	// this is done in a funny way to add the bool that wasn't being serialized before
	//  without changing DDC keys where the bool is not set
	// @todo SerializeForKey these can go away whenever we bump the overall ddc key - just serialize the bool
	if (Settings.bForceAlphaChannel)
	{
		TempGuid = FGuid(0x2C9DF7E3, 0xBC9D413B, 0xBF963C7A, 0x3F27E8B1);
		Ar << TempGuid;
	}
	// fix - bForceNoAlphaChannel is not in key !
	// @todo SerializeForKey these can go away whenever we bump the overall ddc key - just serialize the bool
	if (Settings.bForceNoAlphaChannel)
	{
		TempGuid = FGuid(0x748fc0d4, 0x62004afa, 0x9530460a, 0xf8149d02);
		Ar << TempGuid;
	}

	if ( Settings.bCubemap && Settings.bUseNewMipFilter )
	{
		if ( ( Settings.MipGenSettings >= TMGS_Sharpen0 && Settings.MipGenSettings <= TMGS_Sharpen10 ) ||
			( Settings.MipGenSettings >= TMGS_Blur1 && Settings.MipGenSettings <= TMGS_Blur5 ) )
		{
			// @todo SerializeForKey these can go away whenever we bump the overall ddc key
			// behavior of mip filter changed so modify the key :
			TempGuid = FGuid(0xB0420236,0x90064562,0x9C1F10B8,0x2771C31F);
			Ar << TempGuid;			
		}
	}

	if ( Settings.MaxTextureResolution != FTextureBuildSettings::MaxTextureResolutionDefault &&
		( Settings.MipGenSettings == TMGS_LeaveExistingMips || Settings.bDoScaleMipsForAlphaCoverage ) )
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		// behavior of MaxTextureResolution + LeaveExistingMips or bDoScaleMipsForAlphaCoverage changed, so modify the key :
		TempGuid = FGuid(0x418B8584, 0x72D54EA5, 0xBA8E8C2B, 0xECC880DE);
		Ar << TempGuid;
	}
	
	if ( Settings.MaxTextureResolution != FTextureBuildSettings::MaxTextureResolutionDefault && Settings.bUseNewMipFilter )
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		// behavior of MaxTextureResolution changed to ResizeImage 2/8/2024
		TempName = TEXTURE_DDC_STB_IMAGE_RESIZE_VERSION;
		Ar << TempName;

		if ( Settings.bCubemap || Settings.bTextureArray )
		{
			TempName = FName("Sliced Resize Bug Fix 03/07/2024");
			Ar << TempName;
		}
	}

	if (Settings.bDecodeForPCUsage)
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0x401AD2F7, 0x723E40A8, 0x8E07DCE8, 0x0D17B5DA);
		Ar << TempGuid;
	}

	if ( Settings.bVolume )
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0xCC4348B8,0x84714993,0xAB1E2C93,0x8EA6C9E0);
		Ar << TempGuid;
	}

	if ( Settings.bVirtualStreamable && Settings.bSRGB && Settings.bUseLegacyGamma )
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0xCAEDDFB6,0xEDC2455D,0x8D45B90C,0x3A1B7783);
		Ar << TempGuid;
	}

	// do not change key if old mip filter is used for old textures
	// @todo SerializeForKey these can go away whenever we bump the overall ddc key
	// instead just serialize bool 
	if (Settings.bUseNewMipFilter)
	{
		TempGuid = FGuid(0x27B79A99, 0xE1A5458E, 0xAB619475, 0xCD01AD2A);
		Ar << TempGuid;
	}
	
	// @todo SerializeForKey these can go away whenever we bump the overall ddc key
	// instead just serialize bool bNormalizeNormals
	if ( Settings.bNormalizeNormals )
	{
		TempGuid = FGuid(0x0F5221F6,0x992344D3,0x9C3CCED9,0x4AF08FB8);
		Ar << TempGuid;
	}

	if (Settings.bLongLatSource)
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		// texture processing for cubemaps generated from longlat sources changed, so modify the key :
		TempGuid = FGuid(0x3D642836, 0xEBF64714, 0x9E8E3241, 0x39F66906);
		Ar << TempGuid;
	}

	if (Settings.bCPUAccessible)
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0x583A3B04, 0xC41C4E2C, 0x9FB77E7D, 0xC7AEFE7E);
		Ar << TempGuid;
	}

	if (Settings.bPadWithBorderColor)
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0xB128BA67, 0x3F3C4797, 0x81C66E55, 0xDEEE78EB);
		Ar << TempGuid;
	}

	if (Settings.ResizeDuringBuildX || Settings.ResizeDuringBuildY)
	{
		// @todo SerializeForKey these can go away whenever we bump the overall ddc key
		TempGuid = FGuid(0xDAE8B3E9, 0x605B49DC, 0xADA3C221, 0x02D5567D); Ar << TempGuid;
		TempUint32 = Settings.ResizeDuringBuildX; Ar << TempUint32;
		TempUint32 = Settings.ResizeDuringBuildY; Ar << TempUint32;
	}

	if ( Settings.bUseNewMipFilter )
	{
		// @todo SerializeForKey : TextureAddressModeX is only used if bUseNewMipFilter is true
		//	so we hide it in here to avoid changing more DDC keys
		// todo: when there is an overall DDC key bump, remove ths if on NewFilter so this is just always written
		TempByte = Settings.TextureAddressModeX; Ar << TempByte;
		TempByte = Settings.TextureAddressModeY; Ar << TempByte;
		TempByte = Settings.TextureAddressModeZ; Ar << TempByte;
	}

	// Note - compression quality is added to the DDC by the formats (based on whether they
	// use them or not).
	// This is true for:
	//	LossyCompressionAmount
	//	CompressionQuality
	//	OodleEncodeEffort
	//	OodleUniversalTiling
	//  OodleTextureSdkVersion
	//	bOodlePreserveExtremes
}

/**
 * Computes the derived data key suffix for a texture with the specified compression settings.
 * @param Texture - The texture for which to compute the derived data key.
 * @param BuildSettings - Build settings for which to compute the derived data key.
 * @param OutKeySuffix - The derived data key suffix.
 */
void GetTextureDerivedDataKeySuffix(const UTexture& Texture, const FTextureBuildSettings* BuildSettingsPerLayer, FString& OutKeySuffix)
{
	uint16 Version = 0;
	TStringBuilder<1024> KeyBuilder;

	// Build settings for layer0 (used by default)
	const FTextureBuildSettings& BuildSettings = BuildSettingsPerLayer[0];

	// get the version for this texture's platform format
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const ITextureFormat* TextureFormat = NULL;
	if (TPM)
	{
		TextureFormat = TPM->FindTextureFormat(BuildSettings.TextureFormatName);
		if (TextureFormat)
		{
			Version = TextureFormat->GetVersion(BuildSettings.TextureFormatName, &BuildSettings);
		}
		// else error !?
	}
	// else error !?
	
	FString CompositeTextureStr;

	if(IsValid(Texture.GetCompositeTexture()) && Texture.CompositeTextureMode != CTM_Disabled && Texture.GetCompositeTexture()->Source.IsValid())
	{
		// CompositeTextureMode output changed so force a new DDC key value :
		CompositeTextureStr += TEXT("_Composite090802022_");
		CompositeTextureStr += Texture.GetCompositeTexture()->Source.GetIdString();
	}

	// child texture formats may need to know the mip dimensions in order to generate the ddc
	// key, however VTs don't ever use child texture formats so we just pass 0s
	FIntVector3 Mip0Dimensions = {};
	int32 MipCount = 0;
	if (BuildSettings.bVirtualStreamable == false)
	{
		MipCount = ITextureCompressorModule::GetMipCountForBuildSettings(
			Texture.Source.GetSizeX(), Texture.Source.GetSizeY(), Texture.Source.GetNumSlices(), Texture.Source.GetNumMips(), 
			BuildSettings, 
			Mip0Dimensions.X, Mip0Dimensions.Y, Mip0Dimensions.Z);
	}

	// build the key, but don't use include the version if it's 0 to be backwards compatible
	KeyBuilder.Appendf(TEXT("%s_%s%s%s_%02u_%s"),
		*BuildSettings.TextureFormatName.GetPlainNameString(),
		Version == 0 ? TEXT("") : *FString::Printf(TEXT("%d_"), Version),
		*Texture.Source.GetIdString(),
		*CompositeTextureStr,
		(uint32)NUM_INLINE_DERIVED_MIPS,
		(TextureFormat == NULL) ? TEXT("") : *TextureFormat->GetDerivedDataKeyString(BuildSettings, MipCount, Mip0Dimensions)
		);

	// Add key data for extra layers beyond the first
	const int32 NumLayers = Texture.Source.GetNumLayers();
	for (int32 LayerIndex = 1; LayerIndex < NumLayers; ++LayerIndex)
	{
		const FTextureBuildSettings& LayerBuildSettings = BuildSettingsPerLayer[LayerIndex];
		const ITextureFormat* LayerTextureFormat = NULL;
		if (TPM)
		{
			LayerTextureFormat = TPM->FindTextureFormat(LayerBuildSettings.TextureFormatName);
		}

		uint16 LayerVersion = 0;
		if (LayerTextureFormat)
		{
			LayerVersion = LayerTextureFormat->GetVersion(LayerBuildSettings.TextureFormatName, &LayerBuildSettings);
		}
		KeyBuilder.Appendf(TEXT("%s%d%s_"),
			*LayerBuildSettings.TextureFormatName.GetPlainNameString(),
			LayerVersion,
			(LayerTextureFormat == NULL) ? TEXT("") : *LayerTextureFormat->GetDerivedDataKeyString(LayerBuildSettings, MipCount, Mip0Dimensions));
	}

	if (BuildSettings.bVirtualStreamable)
	{
		// Additional GUID for virtual textures, make it easier to force these to rebuild while developing
		KeyBuilder.Appendf(TEXT("VT%s_"), TEXTURE_VT_DERIVEDDATA_VER);
	}

	if ( Texture.Source.GetNumBlocks() > 1 && Texture.Source.CalcMipOffset(0,0,0) != 0 )
	{
		// bug introduced in CL 32770500 4/5/2024 , incorrectly assumed CalcMipOffset(0,0,0) == 0
		// fix 09/10/2024
		KeyBuilder.Appendf(TEXT("UDIMOffsetBug_"));
	}

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	KeyBuilder.Append(TEXT("_arm64"));
#endif

	if (BuildSettings.bAffectedBySharedLinearEncoding)
	{
		GTextureSLEDerivedDataVer.AppendString(KeyBuilder, EGuidFormats::Digits);
	}

	// Serialize the compressor settings into a temporary array. The archive
	// is flagged as persistent so that machines of different endianness produce
	// identical binary results.
	TArray<uint8> TempBytes; 
	TempBytes.Reserve(1024);
	FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);
	SerializeForKey(Ar, BuildSettings);

	if (Texture.CompressionCacheId.IsValid())
	{
		FGuid TempGuid = Texture.CompressionCacheId;
		Ar << TempGuid;
	}

	for (int32 LayerIndex = 1; LayerIndex < NumLayers; ++LayerIndex)
	{
		const FTextureBuildSettings& LayerBuildSettings = BuildSettingsPerLayer[LayerIndex];
		SerializeForKey(Ar, LayerBuildSettings);
	}

	// Now convert the raw bytes to a string.
	const uint8* SettingsAsBytes = TempBytes.GetData();
	OutKeySuffix.Reset(KeyBuilder.Len() + TempBytes.Num()*2 /* 2 hex characters per byte*/);
	OutKeySuffix.Append(KeyBuilder.ToView());
	for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
	{
		ByteToHex(SettingsAsBytes[ByteIndex], OutKeySuffix);
	}
}

/**
 * Returns the texture derived data version.
 */
const FGuid& GetTextureDerivedDataVersion()
{
	static FGuid Version(TEXTURE_DERIVEDDATA_VER);
	return Version;
}

/**
 * Constructs a derived data key from the key suffix.
 * @param KeySuffix - The key suffix.
 * @param OutKey - The full derived data key.
 */
void GetTextureDerivedDataKeyFromSuffix(const FString& KeySuffix, FString& OutKey)
{
	static UE::DerivedData::FCacheBucket LegacyBucket(TEXTVIEW("LegacyTEXTURE"), TEXTVIEW("Texture"));
	OutKey = FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("TEXTURE"),
		TEXTURE_DERIVEDDATA_VER,
		*KeySuffix
		);
}

/**
 * Constructs the derived data key for an individual mip.
 * @param KeySuffix - The key suffix.
 * @param MipIndex - The mip index.
 * @param OutKey - The full derived data key for the mip.
 */
void GetTextureDerivedMipKey(
	int32 MipIndex,
	const FTexture2DMipMap& Mip,
	const FString& KeySuffix,
	FString& OutKey
	)
{
	OutKey = FDerivedDataCacheInterface::BuildCacheKey(
		TEXT("TEXTURE"),
		TEXTURE_DERIVEDDATA_VER,
		*FString::Printf(TEXT("%s_MIP%u_%dx%d"), *KeySuffix, MipIndex, Mip.SizeX, Mip.SizeY)
		);
}

/**
 * Computes the derived data key for a texture with the specified compression settings.
 * @param Texture - The texture for which to compute the derived data key.
 * @param BuildSettingsPerLayer - Array of FTextureBuildSettings (1 per layer) for which to compute the derived data key.
 * @param OutKey - The derived data key.
 */
static void GetTextureDerivedDataKey(
	const UTexture& Texture,
	const FTextureBuildSettings* BuildSettingsPerLayer,
	FString& OutKey
	)
{
	FString KeySuffix;
	GetTextureDerivedDataKeySuffix(Texture, BuildSettingsPerLayer, KeySuffix);
	GetTextureDerivedDataKeyFromSuffix(KeySuffix, OutKey);
}

#endif // #if WITH_EDITOR

/*------------------------------------------------------------------------------
	Texture compression.
------------------------------------------------------------------------------*/

#if WITH_EDITOR

struct FTextureEncodeSpeedOptions
{
	ETextureEncodeEffort Effort = ETextureEncodeEffort::Default;
	ETextureUniversalTiling Tiling = ETextureUniversalTiling::Disabled;
	bool bUsesRDO = false;
	uint8 RDOLambda = 30;
};

// InEncodeSpeed must be fast or final.
static void GetEncodeSpeedOptions(ETextureEncodeSpeed InEncodeSpeed, FTextureEncodeSpeedOptions* OutOptions)
{
	FResolvedTextureEncodingSettings const& EncodeSettings = FResolvedTextureEncodingSettings::Get();
	if (InEncodeSpeed == ETextureEncodeSpeed::Final)
	{
		OutOptions->bUsesRDO = EncodeSettings.Project.bFinalUsesRDO;
		OutOptions->Effort = EncodeSettings.Project.FinalEffortLevel;
		OutOptions->Tiling = EncodeSettings.Project.FinalUniversalTiling;
		OutOptions->RDOLambda = EncodeSettings.Project.FinalRDOLambda;
	}
	else
	{
		OutOptions->bUsesRDO = EncodeSettings.Project.bFastUsesRDO;
		OutOptions->Effort = EncodeSettings.Project.FastEffortLevel;
		OutOptions->Tiling = EncodeSettings.Project.FastUniversalTiling;
		OutOptions->RDOLambda = EncodeSettings.Project.FastRDOLambda;
	}
}

// this should be a strict over-estimate
// SizeZ is 6 for cubes, can be slices for arrays, etc
//	or it's volume depth and set IsVolume = true
static void GetBuiltTextureSizeBytesEstimate(
	const FTextureBuildSettings& BuildSettings,
	const ITextureFormat* TextureFormat,
	int64 TopMipSizeX,int64 TopMipSizeY,int64 TopMipSizeZ,
	bool bIsVolume,EPixelFormat PixelFormat,
	// fills :
	uint64 & OutTopMipSizeBytes, uint64 & OutTotalImageSizeBytes)
{
	check( PixelFormat != PF_Unknown );

	int64 NumMips = FImageCoreUtils::GetMipCountFromDimensions(TopMipSizeX,TopMipSizeY,TopMipSizeZ,bIsVolume);
	check( NumMips > 0 );

	bool bHasAlpha;
	BuildSettings.GetOutputAlphaFromKnownAlphaOrFallback(&bHasAlpha, true);

	FEncodedTextureDescription TextureDescription;
	BuildSettings.GetEncodedTextureDescription(&TextureDescription, TextureFormat, TopMipSizeX, TopMipSizeY, TopMipSizeZ, NumMips, bHasAlpha);
	check(TextureDescription.PixelFormat == PixelFormat);

	uint64 LinearTopMipSizeBytes = 0;
	uint64 LinearTotalImageSizeBytes = 0;

	// calculate bytes for linear unpadded/untiled layout :
	for (int32 MipIndex = 0; MipIndex < TextureDescription.NumMips; MipIndex++)
	{
		if (MipIndex == 0)
		{
			LinearTopMipSizeBytes = TextureDescription.GetMipSizeInBytes(0);
			LinearTotalImageSizeBytes = LinearTopMipSizeBytes;
		}
		else
		{
			LinearTotalImageSizeBytes += TextureDescription.GetMipSizeInBytes(MipIndex);
		}
	}
	
	check( LinearTotalImageSizeBytes > 0 );

	OutTopMipSizeBytes = LinearTopMipSizeBytes;
	OutTotalImageSizeBytes = LinearTotalImageSizeBytes;

	if ( LinearTotalImageSizeBytes < (2LL<<30) )
	{
		// only call GetExtendedDataForTexture if total size is under 2 GB
		//	because it calls into platform texture lib functions that are not 64-bit math safe

		int32 LODBias = 0;
		FEncodedTextureExtendedData ExtendedData = TextureFormat->GetExtendedDataForTexture(TextureDescription, LODBias);
		if ( ExtendedData.bIsTiled )
		{
			// ExtendedData is only valid for platform/tiled images
		
			uint64 TiledTopMipSizeBytes = 0;
			uint64 TiledTotalImageSizeBytes = 0;

			TiledTopMipSizeBytes = ExtendedData.MipSizesInBytes[0];
		
			TiledTotalImageSizeBytes = 0;
			for(const uint64 & MipSize : ExtendedData.MipSizesInBytes )
			{
				TiledTotalImageSizeBytes += MipSize;
			}
			
			check( TiledTotalImageSizeBytes > 0 );
			check( TiledTopMipSizeBytes >= LinearTopMipSizeBytes );
			check( TiledTotalImageSizeBytes >= LinearTotalImageSizeBytes );

			OutTopMipSizeBytes = TiledTopMipSizeBytes;
			OutTotalImageSizeBytes = TiledTotalImageSizeBytes;
		}
	}

}

// may reduce OutSettings.MaxTextureResolution
//	nop if called again
//	does not change anything else in OutSettings
//	OutSettings must be otherwise fully set up
static void ModifyMaxTextureResolutionBuildSettingsForPlatformLimit(
	const UTexture& Texture, 
	const ITargetPlatform* TargetPlatform,	
	const ITextureFormat* TextureFormat,
	FTextureBuildSettings& OutSettings)
{
	check( ! OutSettings.bVirtualStreamable );
	check( OutSettings.TextureFormatName != NAME_None );
	
	if (!Texture.Source.IsValid() ||
		OutSettings.BaseTextureFormat == nullptr) // can happen with missing format dlls.
	{
		// Nothing to do - texture can't be built.
		return;
	}

	// GetBuiltTextureSize is the size after LODBias
	int32 BuiltSizeX=0,BuiltSizeY=0,BuiltSizeZ=0;
	Texture.GetBuiltTextureSize(TargetPlatform,BuiltSizeX,BuiltSizeY,BuiltSizeZ);
	
	const int32 MaxDimension = UTexture::GetMaximumDimensionOfNonVT();
	
	// OriginalMaxTextureResolution is uint32_max if Texture did not have a max size set
	uint32 OriginalMaxTextureResolution = OutSettings.MaxTextureResolution;
	

	if ( BuiltSizeX > MaxDimension || BuiltSizeY > MaxDimension || BuiltSizeZ > MaxDimension )
	{
		// Only update the max texture resolution if we are affected by this so that previously conforming
		// textures don't get rebuilt.
		OutSettings.MaxTextureResolution = FMath::Min<uint32>(MaxDimension, OutSettings.MaxTextureResolution);

		// this should have already happened in Texture.cpp ValidateSettingsAfterImportOrEdit
		//	no harm in doing it again to make sure

		if ( BuiltSizeZ > MaxDimension && ! OutSettings.bVolume )
		{
			UE_LOGF(LogTexture, Error, "Texture %ls non-volume has huge Z depth!", 
				*Texture.GetPathName());

			OutSettings.MaxTextureResolution = 4;
			return;
		}
		else
		{
			UE_LOGF(LogTexture, Warning, "Texture %ls exceeds maximum dimensions : %d x %d x %d > %d , shrinking...", *Texture.GetPathName(),
				BuiltSizeX, BuiltSizeY, BuiltSizeZ, MaxDimension
				);
		}

		while ( BuiltSizeX > MaxDimension || BuiltSizeY > MaxDimension || BuiltSizeZ > MaxDimension )
		{
			BuiltSizeX = FMath::Max(1,BuiltSizeX>>1);
			BuiltSizeY = FMath::Max(1,BuiltSizeY>>1);
			if ( OutSettings.bVolume )
			{
				BuiltSizeZ = FMath::Max(1,BuiltSizeZ>>1);
			}
		}
	}

	uint64 MaxSurfaceBytes,MaxPackageBytes;
	TargetPlatform->GetTextureSizeLimits(MaxSurfaceBytes,MaxPackageBytes);
	
	EPixelFormat PixelFormat = UE::TextureBuildUtilities::GetOutputPixelFormatWithFallback(OutSettings, true);

	if ( PixelFormat == PF_Unknown )
	{
		UE_LOGF(LogTexture, Error, "Texture %ls failed GetOutputPixelFormatWithFallback (format=%ls)", 
			*Texture.GetPathName(),
			*OutSettings.TextureFormatName.ToString());
			
		PixelFormat = PF_FloatRGBA;
	}

	uint64 SurfaceBytes,TotalBytes;
	GetBuiltTextureSizeBytesEstimate(OutSettings,TextureFormat, BuiltSizeX,BuiltSizeY,BuiltSizeZ,OutSettings.bVolume,PixelFormat,SurfaceBytes,TotalBytes);

	if ( SurfaceBytes > MaxSurfaceBytes || TotalBytes > MaxPackageBytes )
	{
		UE_LOGF(LogTexture, Warning, "Texture %ls exceeds maximum size of surface or package: %d x %d x %d x %ls = {%lld,%lld bytes} exceeds limit {%lld,%lld bytes} shrinking...", *Texture.GetPathName(),
			BuiltSizeX, BuiltSizeY, BuiltSizeZ, GetPixelFormatString(PixelFormat),
			SurfaceBytes,TotalBytes,
			MaxSurfaceBytes,MaxPackageBytes
			);
		
		do
		{
			// change MaxTextureResolution so that it causes us to do one mip step down
			//	and adjust BuiltSize accordingly
			
			// BuiltSizeZ not affected by MaxTextureResolution
			OutSettings.MaxTextureResolution = FMath::RoundUpToPowerOfTwo( FMath::Max(BuiltSizeX,BuiltSizeY) )/2;
			check( (int64)OutSettings.MaxTextureResolution < (int64)BuiltSizeX || (int64)OutSettings.MaxTextureResolution < (int64)BuiltSizeY );
				
			BuiltSizeX = FMath::Max(1,BuiltSizeX>>1);
			BuiltSizeY = FMath::Max(1,BuiltSizeY>>1);
			if ( OutSettings.bVolume )
			{
				BuiltSizeZ = FMath::Max(1,BuiltSizeZ>>1);
			}

			check( (int64)BuiltSizeX <= (int64)OutSettings.MaxTextureResolution && (int64)BuiltSizeY <= (int64)OutSettings.MaxTextureResolution );

			// recalc size in bytes :
			GetBuiltTextureSizeBytesEstimate(OutSettings,TextureFormat, BuiltSizeX,BuiltSizeY,BuiltSizeZ,OutSettings.bVolume,PixelFormat,SurfaceBytes,TotalBytes);
		}
		while ( SurfaceBytes > MaxSurfaceBytes || TotalBytes > MaxPackageBytes );
	}

	if ( OutSettings.MaxTextureResolution != OriginalMaxTextureResolution )
	{
		// compensate for LODBias that will be applied
		// after scaling to MaxTextureResolution, LODBiasNoCinematics will be applied
			
		const UTextureLODSettings& LODSettings = TargetPlatform->GetTextureLODSettings();
 		const uint32 LODBiasNoCinematics = FMath::Max<int32>(LODSettings.CalculateLODBias(BuiltSizeX, BuiltSizeY, Texture.GetMinTextureSize(), Texture.GetMaxTextureSize(), Texture.LODGroup, Texture.LODBias, 0, Texture.MipGenSettings, OutSettings.bVirtualStreamable, Texture.IsStreamable()), 0);

		int64 MaxTextureResolutionUp = ((int64)OutSettings.MaxTextureResolution)<<LODBiasNoCinematics;

		OutSettings.MaxTextureResolution = (uint32) FMath::Min<int64>((int64)OriginalMaxTextureResolution,MaxTextureResolutionUp);

		// ensure MaxTextureResolution never goes up :
		OutSettings.MaxTextureResolution = FMath::Min(OriginalMaxTextureResolution,OutSettings.MaxTextureResolution);
	}
}


// Convert the baseline build settings for all layers to one for the given layer.
// Note this gets called twice for layer 0, so needs to be idempotent.
static void FinalizeBuildSettingsForLayer(
	const UTexture& Texture, 
	int32 LayerIndex, 
	const ITargetPlatform* TargetPlatform, 
	ETextureEncodeSpeed InEncodeSpeed, // must be Final or Fast.
	FTextureBuildSettings& OutSettings,
	FTexturePlatformData::FTextureEncodeResultMetadata* OutBuildResultMetadata // can be nullptr if not needed
	)
{
	FTextureFormatSettings FormatSettings;
	Texture.GetLayerFormatSettings(LayerIndex, FormatSettings);

	OutSettings.bHDRSource = Texture.HasHDRSource(LayerIndex);
	OutSettings.bSRGB = FormatSettings.SRGB;
	OutSettings.bForceNoAlphaChannel = FormatSettings.CompressionNoAlpha;
	OutSettings.bForceAlphaChannel = FormatSettings.CompressionForceAlpha;
	OutSettings.bApplyYCoCgBlockScale = FormatSettings.CompressionYCoCg;

	if (FormatSettings.CompressionSettings == TC_Displacementmap || FormatSettings.CompressionSettings == TC_DistanceFieldFont)
	{
		OutSettings.bReplicateAlpha = true;
	}
	else if (FormatSettings.CompressionSettings == TC_Grayscale || FormatSettings.CompressionSettings == TC_Alpha)
	{
		OutSettings.bReplicateRed = true;
	}

	// If we have channel boundary information, use that to determine whether we expect to have
	// a non opaque alpha.
	TArray<FTextureSourceLayerColorInfo> LayerColorInfo;
	Texture.Source.GetLayerColorInfo(LayerColorInfo);
	if (LayerIndex < LayerColorInfo.Num())
	{
		const FTextureSourceLayerColorInfo& LayerChannelBounds = LayerColorInfo[LayerIndex];

		OutSettings.bKnowAlphaTransparency = ITextureCompressorModule::DetermineAlphaChannelTransparency(OutSettings, 
			LayerChannelBounds.ColorMin, LayerChannelBounds.ColorMax, OutSettings.bHasTransparentAlpha);
	}

	// this is called once per Texture with OutSettings.TextureFormatName == None
	//	and then called again (per Layer) with OutSettings.TextureFormatName filled out

	if (OutSettings.bVirtualStreamable && ! OutSettings.TextureFormatName.IsNone())
	{
		// note : FinalizeVirtualTextureLayerFormat is run outside of the normal TextureFormatName set up ; fix?
		//	should be done inside GetPlatformTextureFormatNamesWithPrefix
		//	this is only used by Android & iOS (changes DXT to ASTC/ETC)
		//  the reason to do it here is we now have bVirtualStreamable, which is not available at the earlier call
		
		// The platform FinalizeVirtualTextureLayerFormat assumes (incorrectly) that it gets non-prefixed names, so remove them :

		// Virtual textures always strip the child format prefix prior to actual encode since VTs never tile.
		// must match VirtualTextureDataBuilder.cpp
		OutSettings.TextureFormatName = UE::TextureBuildUtilities::TextureFormatRemovePlatformPrefixFromName(OutSettings.TextureFormatName);

		// May still have an Oodle prefix, remove that :
		FName NameWithoutPrefix = UE::TextureBuildUtilities::TextureFormatRemovePrefixFromName(OutSettings.TextureFormatName);
		FName ModifiedName = TargetPlatform->FinalizeVirtualTextureLayerFormat(NameWithoutPrefix);
		if ( NameWithoutPrefix != ModifiedName )
		{
			// if the platform FinalizeVirtualTextureLayerFormat changed our format, take that change (with no Oodle) prefix)
			//	otherwise, leave TextureFormatName along (with Oodle prefix)
			// POSSIBLE BUG : if the Platform changed us to a different DXT format that should have Oodle prefix, we lost it here!
			OutSettings.TextureFormatName = ModifiedName;
		}
	}

	// Now that we know the texture format, we can make decisions based on it.
	
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const ITextureFormat* TextureFormat = nullptr;
	// this is called once first with NAME_None and then called again after Name is set up
	if ( ! OutSettings.TextureFormatName.IsNone() )
	{
		TextureFormat = TPM->FindTextureFormat(OutSettings.TextureFormatName);
	}

	bool bSupportsEncodeSpeed = false;

	// Can be null with first finalize (at the end of GetTextureBuildSettings)
	if (TextureFormat)
	{
		bSupportsEncodeSpeed = TextureFormat->SupportsEncodeSpeed(OutSettings.TextureFormatName, TargetPlatform->GetTargetPlatformSettings());

		const FChildTextureFormat* ChildTextureFormat = TextureFormat->GetChildFormat();

		if (ChildTextureFormat)
		{
			OutSettings.BaseTextureFormatName = ChildTextureFormat->GetBaseFormatName(OutSettings.TextureFormatName);
		}
		else
		{
			OutSettings.BaseTextureFormatName = OutSettings.TextureFormatName;
		}

		OutSettings.BaseTextureFormat = GetTextureFormatManager()->FindTextureFormat(OutSettings.BaseTextureFormatName);


		if (OutBuildResultMetadata)
		{
			OutBuildResultMetadata->Encoder = TextureFormat->GetEncoderName(OutSettings.TextureFormatName);
			OutBuildResultMetadata->bIsValid = true;
			OutBuildResultMetadata->bSupportsEncodeSpeed = bSupportsEncodeSpeed;

			// Storing the actual format we used at build time requires a ddc entry. Since this is rare and usually we
			// can figure it out, just try to figure it out. If we don't know, then we don't know.
			OutBuildResultMetadata->EncodedFormat = PF_Unknown;
			
			EPixelFormat WithAlphaFormat = TextureFormat->GetEncodedPixelFormat(OutSettings, true);
			EPixelFormat WithoutAlphaFormat = TextureFormat->GetEncodedPixelFormat(OutSettings, false);
			bool bHasAlpha = false;
			if (WithAlphaFormat == WithoutAlphaFormat)
			{
				OutBuildResultMetadata->EncodedFormat = WithAlphaFormat;
			}
			else if (OutSettings.GetOutputAlphaFromKnownAlphaOrFail(&bHasAlpha))
			{
				OutBuildResultMetadata->EncodedFormat = bHasAlpha ? WithAlphaFormat : WithoutAlphaFormat;
			}
		}

		if (ChildTextureFormat)
		{
			OutSettings.TilerEvenIfNotSharedLinear = ChildTextureFormat->GetTiler();
		}

		if (FResolvedTextureEncodingSettings::Get().Project.bSharedLinearTextureEncoding &&
			!OutSettings.bCPUAccessible) // CPU textures are not tiled.
		{
			//
			// We want to separate out textures involved in shared linear encoding in order to facilitate
			// fixing bugs without invalidating the world (even though we expect the exact same data to
			// get generated). However, virtual textures never tile, and so are exempt from this separation.
			//
			if (OutSettings.bVirtualStreamable == false)
			{
				OutSettings.bAffectedBySharedLinearEncoding = true;
			}

			// Shared linear encoding can only work if the base texture format does not expect to
			// do the tiling itself (SupportsTiling == false).
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (ChildTextureFormat && OutSettings.BaseTextureFormat->SupportsTiling() == false)
			{
				OutSettings.Tiler = ChildTextureFormat->GetTiler();
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	if (bSupportsEncodeSpeed)
	{
		FTextureEncodeSpeedOptions Options;
		GetEncodeSpeedOptions(InEncodeSpeed, &Options);

		// Always pass effort and tiling.
		OutSettings.OodleEncodeEffort = (uint8)Options.Effort;
		OutSettings.OodleUniversalTiling = (uint8)Options.Tiling;

		// LCA has no effect if disabled, and only override if not default.
		OutSettings.bOodleUsesRDO = Options.bUsesRDO;
		if (Options.bUsesRDO)
		{
			// If this mapping changes, update the tooltip in TextureEncodingSettings.h
			// this is an ETextureLossyCompressionAmount
			switch (OutSettings.LossyCompressionAmount)
			{
			default:
			case TLCA_Default: 
				{
					if (OutBuildResultMetadata)
					{
						OutBuildResultMetadata->RDOSource = FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Default;
					}
					OutSettings.OodleRDO = Options.RDOLambda; 
					break; // Use global defaults.
				}
			case TLCA_None:    OutSettings.OodleRDO = 0; break;		// "No lossy compression"
			case TLCA_Lowest:  OutSettings.OodleRDO = 1; break;		// "Lowest (Best Image quality, largest filesize)"
			case TLCA_Low:     OutSettings.OodleRDO = 10; break;	// "Low"
			case TLCA_Medium:  OutSettings.OodleRDO = 20; break;	// "Medium"
			case TLCA_High:    OutSettings.OodleRDO = 30; break;	// "High"
			case TLCA_Highest: OutSettings.OodleRDO = 40; break;	// "Highest (Worst Image quality, smallest filesize)"
			}
		}
		else
		{
			OutSettings.OodleRDO = 0;
		}

		if (OutBuildResultMetadata)
		{
			OutBuildResultMetadata->OodleRDO = OutSettings.OodleRDO;
			OutBuildResultMetadata->OodleEncodeEffort = OutSettings.OodleEncodeEffort;
			OutBuildResultMetadata->OodleUniversalTiling = OutSettings.OodleUniversalTiling;
		}
	}
	
	// this is called once first with NAME_None and then called again after Name is set up. TextureFormat might also
	// be null due to incorrect SDK configuration.
	if ( ! OutSettings.bVirtualStreamable && TextureFormat )
	{
		check( LayerIndex == 0 );
		ModifyMaxTextureResolutionBuildSettingsForPlatformLimit(Texture,TargetPlatform,TextureFormat,OutSettings);
	}
}

ENGINE_API ETextureEncodeSpeed UTexture::GetDesiredEncodeSpeed() const
{
	if ( CompressFinal )
	{
		return ETextureEncodeSpeed::Final;
	}

	return FResolvedTextureEncodingSettings::Get().EncodeSpeed;
}

int32 UTexture::GetMinTextureSize() const
{
	if (VirtualTextureStreaming)
	{
		FVirtualTextureBuildSettings VirtualTextureBuildSettings;
		GetVirtualTextureBuildSettings(VirtualTextureBuildSettings);
		return FVirtualTextureBuildSettings::ClampAndAlignTileSize(VirtualTextureBuildSettings.TileSize);
	}
	return 0;
}

// from Texture.cpp
extern FName GetLatestOodleTextureSdkVersion();

static FName ConditionalRemapOodleTextureSdkVersion(FName InOodleTextureSdkVersion, const ITargetPlatform* TargetPlatform)
{
#if WITH_EDITOR

	// optionally remap InOodleTextureSdkVersion
	
	bool bOodleTextureSdkForceLatestVersion = false;
	if ( TargetPlatform->GetConfigSystem()->GetBool(TEXT("AlternateTextureCompression"), TEXT("OodleTextureSdkForceLatestVersion"), bOodleTextureSdkForceLatestVersion, GEngineIni) &&
		bOodleTextureSdkForceLatestVersion )
	{
		static FName LatestOodleTextureSdkVersion = GetLatestOodleTextureSdkVersion();

		return LatestOodleTextureSdkVersion;
	}

	if ( InOodleTextureSdkVersion.IsNone() )
	{
		//	new (optional) pref : OodleTextureSdkVersionToUseIfNone

		FString OodleTextureSdkVersionToUseIfNone;
		if ( TargetPlatform->GetConfigSystem()->GetString(TEXT("AlternateTextureCompression"), TEXT("OodleTextureSdkVersionToUseIfNone"), OodleTextureSdkVersionToUseIfNone, GEngineIni) )
		{
			return FName(OodleTextureSdkVersionToUseIfNone);
		}
	}

	// @todo Oodle : possibly also remap non-none versions
	//	so you could set up mapping tables like "if it was 2.9.4, now use 2.9.6"

#endif

	return InOodleTextureSdkVersion;
}

/**
 * Sets texture build settings.
 * @param Texture - The texture for which to build compressor settings.
 * @param OutBuildSettings - Build settings.
 * 
 * This function creates the build settings that are shared across all layers - you can not
 * assume a texture format at this time (See FinalizeBuildSettingsForLayer)
 */
static void GetTextureBuildSettings(
	const UTexture& Texture,
	const UTextureLODSettings& TextureLODSettings,
	const ITargetPlatform& TargetPlatform,
	ETextureEncodeSpeed InEncodeSpeed, // must be Final or Fast.
	FTextureBuildSettings& OutBuildSettings,
	FTexturePlatformData::FTextureEncodeResultMetadata* OutBuildResultMetadata // can be nullptr if not needed
	)
{
	const bool bPlatformSupportsTextureStreaming = TargetPlatform.SupportsFeature(ETargetPlatformFeatures::TextureStreaming);

	if (OutBuildResultMetadata)
	{
		OutBuildResultMetadata->EncodeSpeed = (uint8)InEncodeSpeed;
	}
	OutBuildSettings.RepresentsEncodeSpeedNoSend = (uint8)InEncodeSpeed;

	OutBuildSettings.ColorAdjustment.AdjustBrightness = Texture.AdjustBrightness;
	OutBuildSettings.ColorAdjustment.AdjustBrightnessCurve = Texture.AdjustBrightnessCurve;
	OutBuildSettings.ColorAdjustment.AdjustVibrance = Texture.AdjustVibrance;
	OutBuildSettings.ColorAdjustment.AdjustSaturation = Texture.AdjustSaturation;
	OutBuildSettings.ColorAdjustment.AdjustRGBCurve = Texture.AdjustRGBCurve;
	OutBuildSettings.ColorAdjustment.AdjustHue = Texture.AdjustHue;
	OutBuildSettings.ColorAdjustment.AdjustMinAlpha = Texture.AdjustMinAlpha;
	OutBuildSettings.ColorAdjustment.AdjustMaxAlpha = Texture.AdjustMaxAlpha;
	OutBuildSettings.bUseLegacyGamma = Texture.bUseLegacyGamma;
	OutBuildSettings.bPreserveBorder = Texture.bPreserveBorder;

	// in Texture , the fields bDoScaleMipsForAlphaCoverage and AlphaCoverageThresholds are independent
	// but in the BuildSettings bDoScaleMipsForAlphaCoverage is only on if thresholds are valid (not all zero)
	if ( Texture.bDoScaleMipsForAlphaCoverage && Texture.AlphaCoverageThresholds != FVector4(0,0,0,0) )
	{
		OutBuildSettings.bDoScaleMipsForAlphaCoverage = Texture.bDoScaleMipsForAlphaCoverage;
		OutBuildSettings.AlphaCoverageThresholds = (FVector4f)Texture.AlphaCoverageThresholds;
	}
	else
	{
		OutBuildSettings.bDoScaleMipsForAlphaCoverage = false;
		OutBuildSettings.AlphaCoverageThresholds = FVector4f(0,0,0,0);
	}

	OutBuildSettings.bUseNewMipFilter = Texture.bUseNewMipFilter;
	OutBuildSettings.bNormalizeNormals = Texture.bNormalizeNormals && Texture.IsNormalMap();
	OutBuildSettings.bComputeBokehAlpha = (Texture.LODGroup == TEXTUREGROUP_Bokeh);
	OutBuildSettings.bReplicateAlpha = false;
	OutBuildSettings.bReplicateRed = false;
	OutBuildSettings.bVolume = false;
	OutBuildSettings.bCubemap = false;
	OutBuildSettings.bTextureArray = false;
	OutBuildSettings.DiffuseConvolveMipLevel = 0;
	OutBuildSettings.bLongLatSource = false;
	OutBuildSettings.SourceEncodingOverride = static_cast<uint8>(Texture.SourceColorSettings.EncodingOverride);
	OutBuildSettings.bHasColorSpaceDefinition = Texture.SourceColorSettings.ColorSpace != ETextureColorSpace::TCS_None;
	OutBuildSettings.RedChromaticityCoordinate = FVector2f(Texture.SourceColorSettings.RedChromaticityCoordinate);
	OutBuildSettings.GreenChromaticityCoordinate = FVector2f(Texture.SourceColorSettings.GreenChromaticityCoordinate);
	OutBuildSettings.BlueChromaticityCoordinate = FVector2f(Texture.SourceColorSettings.BlueChromaticityCoordinate);
	OutBuildSettings.WhiteChromaticityCoordinate = FVector2f(Texture.SourceColorSettings.WhiteChromaticityCoordinate);
	OutBuildSettings.ChromaticAdaptationMethod = static_cast<uint8>(Texture.SourceColorSettings.ChromaticAdaptationMethod);
	
	check( OutBuildSettings.MaxTextureResolution == FTextureBuildSettings::MaxTextureResolutionDefault );
	if (Texture.MaxTextureSize > 0)
	{
		OutBuildSettings.MaxTextureResolution = Texture.MaxTextureSize;
	}

	ETextureClass TextureClass = Texture.GetTextureClass();
	
	if ( TextureClass == ETextureClass::TwoD )
	{
		// nada
	}
	else if ( TextureClass == ETextureClass::Cube )
	{
		OutBuildSettings.bCubemap = true;
		OutBuildSettings.DiffuseConvolveMipLevel = GDiffuseConvolveMipLevel;
		check( Texture.Source.GetNumSlices() == 1 || Texture.Source.GetNumSlices() == 6 );
		OutBuildSettings.bLongLatSource = Texture.Source.IsLongLatCubemap();
	}
	else if ( TextureClass == ETextureClass::Array )
	{
		OutBuildSettings.bTextureArray = true;
	}
	else if ( TextureClass == ETextureClass::CubeArray )
	{
		OutBuildSettings.bCubemap = true;
		OutBuildSettings.bTextureArray = true;
		// beware IsLongLatCubemap
		// ambiguous with longlat cube arrays with multiple of 6 array size
		OutBuildSettings.bLongLatSource = Texture.Source.IsLongLatCubemap();
		check( ((Texture.Source.GetNumSlices()%6)==0) || OutBuildSettings.bLongLatSource );
	}
	else if ( TextureClass == ETextureClass::Volume )
	{
		OutBuildSettings.bVolume = true;
	}
	else if ( TextureClass == ETextureClass::TwoDDynamic ||
		TextureClass == ETextureClass::Other2DNoSource )
	{
		UE_LOGF(LogTexture, Warning, "Unexpected texture build for dynamic texture? (%ls)",*Texture.GetName());
	}
	else
	{
		// unknown TextureType ?
		UE_LOGF(LogTexture, Error, "Unexpected texture build for unknown texture class? (%ls)",*Texture.GetName());
	}

	bool bDownsampleWithAverage;
	bool bSharpenWithoutColorShift;
	bool bBorderColorBlack;
	TextureMipGenSettings MipGenSettings;
	TextureLODSettings.GetMipGenSettings( 
		Texture,
		MipGenSettings,
		OutBuildSettings.MipSharpening,
		OutBuildSettings.SharpenMipKernelSize,
		bDownsampleWithAverage,
		bSharpenWithoutColorShift,
		bBorderColorBlack
		);

	bool bVirtualTextureStreaming = Texture.VirtualTextureStreaming;

	if ( !bVirtualTextureStreaming && Texture.GetClass() == ULightMapVirtualTexture2D::StaticClass() )
	{
		// A ULightMapVirtualTexture2D with multiple layers saved in MapBuildData could be loaded with the r.VirtualTexture disabled, it will generate DDC before we decide to invalidate the light map data, to skip the ensure failure let it generate VT DDC anyway.
		// @@ pretty ugly hack here, this should have been fixed in PostLoad or something
		bVirtualTextureStreaming = true;
	}
	
	if ( bVirtualTextureStreaming && ! UTexture::IsVirtualTexturingEnabled(&TargetPlatform) )
	{
		bVirtualTextureStreaming = false;
	}

	if ( Texture.RequiresVirtualTexturing() && ! bVirtualTextureStreaming )
	{
		// should not get here; earlier call to CanBuildPlatformData() should have returned false
		UE_LOGF(LogTexture, Error, "Texture RequiresVirtualTexturing but VT is off (%ls)",*Texture.GetName());

		// no way to error out and abort the build from here (this function returns void)
		// return false;
		
		// turn it back on to avoid crashes?
		//	otherwise you will hit checks on NumLayers because we expect non-VT to always have 1 layer
		bVirtualTextureStreaming = true;
	}

	if (Texture.Availability == ETextureAvailability::CPU && TextureClass == ETextureClass::TwoD && 
		! Texture.RequiresVirtualTexturing())
	{
		// We are swapping with a placeholder - don't VT it.
		OutBuildSettings.bCPUAccessible = true;
		bVirtualTextureStreaming = false;
		MipGenSettings = TMGS_NoMipmaps;
	}
	
	OutBuildSettings.bVirtualStreamable = bVirtualTextureStreaming;

	// Virtual textures must have mips as VT memory management relies on a 1:1 texel/pixel mapping, which in turn
	// requires that we be able to swap in lower mips when that density gets too high for a given texture.
	if (bVirtualTextureStreaming && MipGenSettings == TMGS_NoMipmaps)
	{
		MipGenSettings = TMGS_SimpleAverage;
		UE_LOGF(LogTexture, Display, "Texture %ls is virtual and has NoMips - forcing to SimpleAverage.", *Texture.GetPathName());
	}
	if (bVirtualTextureStreaming && MipGenSettings == TMGS_LeaveExistingMips)
	{
		for (int32 BlockIndex = 0; BlockIndex < Texture.Source.GetNumBlocks(); BlockIndex++)
		{
			FTextureSourceBlock Block;
			Texture.Source.GetBlock(BlockIndex, Block);

			int32 ExpectedNumMips = FImageCoreUtils::GetMipCountFromDimensions(Block.SizeX, Block.SizeY, 0, false);
			if (Block.NumMips != ExpectedNumMips)
			{
				MipGenSettings = TMGS_SimpleAverage;
				UE_LOGF(LogTexture, Warning, "Texture %ls is virtual and has LeaveExistingMips with an incomplete mip chain - forcing to SimpleAverage (Block %d has %d mips, expected %d).", 
					*Texture.GetPathName(),
					BlockIndex,
					Block.NumMips,
					ExpectedNumMips
					);
			}
		}
	}
	if ( Texture.Source.GetNumBlocks() > 1 && !bVirtualTextureStreaming )
	{
		UE_LOGF(LogTexture, Warning, "Texture '%ls' has UDIM Blocks, but Virtual Texturing is off for platform '%ls'. Only the first UDIM block will be built for the platform.",
			*Texture.GetPathName(),
			*TargetPlatform.IniPlatformName());
	}

	const FIntPoint SourceSize = Texture.Source.GetLogicalSize();

	OutBuildSettings.MipGenSettings = MipGenSettings;
	OutBuildSettings.bDownsampleWithAverage = bDownsampleWithAverage;
	OutBuildSettings.bSharpenWithoutColorShift = bSharpenWithoutColorShift;
	OutBuildSettings.bBorderColorBlack = bBorderColorBlack;
	OutBuildSettings.bFlipGreenChannel = Texture.bFlipGreenChannel;
	
	// these are set even if Texture.CompositeTexture == null
	//	we should not do that, but keep it the same for now to preserve DDC keys
	OutBuildSettings.CompositeTextureMode = Texture.CompositeTextureMode;
	OutBuildSettings.CompositePower = Texture.CompositePower;

	if ( Texture.GetCompositeTexture() && !Texture.GetCompositeTexture()->Source.IsValid() )
	{
		// have a CompositeTexture but it has no source, don't use it :
		OutBuildSettings.CompositeTextureMode = CTM_Disabled;
	}

	OutBuildSettings.LODBias = TextureLODSettings.CalculateLODBias(SourceSize.X, SourceSize.Y, Texture.GetMinTextureSize(), Texture.GetMaxTextureSize(), Texture.LODGroup, Texture.LODBias, Texture.NumCinematicMipLevels, Texture.MipGenSettings, bVirtualTextureStreaming, Texture.IsStreamable());
	OutBuildSettings.LODBiasWithCinematicMips = TextureLODSettings.CalculateLODBias(SourceSize.X, SourceSize.Y, Texture.GetMinTextureSize(), Texture.GetMaxTextureSize(), Texture.LODGroup, Texture.LODBias, 0, Texture.MipGenSettings, bVirtualTextureStreaming, Texture.IsStreamable());
	OutBuildSettings.PowerOfTwoMode = Texture.PowerOfTwoMode;
	OutBuildSettings.PaddingColor = Texture.PaddingColor;
	OutBuildSettings.bPadWithBorderColor = Texture.bPadWithBorderColor;
	OutBuildSettings.ResizeDuringBuildX = Texture.ResizeDuringBuildX;
	OutBuildSettings.ResizeDuringBuildY = Texture.ResizeDuringBuildY;
	OutBuildSettings.ChromaKeyColor = Texture.ChromaKeyColor;
	OutBuildSettings.bChromaKeyTexture = Texture.bChromaKeyTexture;
	OutBuildSettings.ChromaKeyThreshold = Texture.ChromaKeyThreshold;
	OutBuildSettings.CompressionQuality = Texture.CompressionQuality - 1; // translate from enum's 0 .. 5 to desired compression (-1 .. 4, where -1 is default while 0 .. 4 are actual quality setting override)
	OutBuildSettings.bOodlePreserveExtremes = Texture.bOodlePreserveExtremes;

	// do remap here before we send to TBW's which may not have access to config :
	OutBuildSettings.OodleTextureSdkVersion = ConditionalRemapOodleTextureSdkVersion(Texture.OodleTextureSdkVersion,&TargetPlatform);

	// figure out the default astcenc version for the platform. Leave as NAME_None
	// if not specified
	{
		const FString& SectionName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(TargetPlatform.IniPlatformName()).TargetSettingsIniSectionName;
		FString ASTCVersion;
		if (TargetPlatform.GetConfigSystem()->GetString(*SectionName, TEXT("ASTCVersion"), ASTCVersion, GEngineIni) &&
			ASTCVersion.Len())
		{
			OutBuildSettings.AstcEncVersion = FName(ASTCVersion);
		}
	}

	// if LossyCompressionAmount is Default, inherit from LODGroup :
	if ( Texture.LossyCompressionAmount == TLCA_Default )
	{
		const FTextureLODGroup& LODGroup = TextureLODSettings.GetTextureLODGroup(Texture.LODGroup);
		OutBuildSettings.LossyCompressionAmount = LODGroup.LossyCompressionAmount;
		if (OutBuildResultMetadata)
		{
			OutBuildResultMetadata->RDOSource = FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::LODGroup;
		}
	}
	else
	{
		OutBuildSettings.LossyCompressionAmount = Texture.LossyCompressionAmount.GetValue();
		if (OutBuildResultMetadata)
		{
			OutBuildResultMetadata->RDOSource = FTexturePlatformData::FTextureEncodeResultMetadata::OodleRDOSource::Texture;
		}
	}

	OutBuildSettings.Downscale = 1.0f;

	float Downscale;
	ETextureDownscaleOptions DownscaleOptions;
	TextureLODSettings.GetDownscaleOptions(Texture, TargetPlatform, Downscale, DownscaleOptions);

	// Downscale only allowed if NoMipMaps, 2d, and not VT
	//	silently does nothing otherwise
	if (! bVirtualTextureStreaming &&
		MipGenSettings == TMGS_NoMipmaps && 
		Texture.IsA(UTexture2D::StaticClass()))	// TODO: support more texture types
	{
		OutBuildSettings.Downscale = Downscale;
		OutBuildSettings.DownscaleOptions = (uint8)DownscaleOptions;
	}
	// only show a warning for textures where Downscale setting would have effect if it was used
	else if (Downscale != 1.f)
	{
		UE_LOGF(LogTexture, Warning, "Downscale setting of %f was not used when building texture %ls%ls.", Downscale, *Texture.GetName(),
			bVirtualTextureStreaming ? TEXT(" because it is using virtual texturing") :
			MipGenSettings != TMGS_NoMipmaps ? TEXT(" because it is using mipmaps") :
			!Texture.IsA(UTexture2D::StaticClass()) ? TEXT(" because it is only supported for 2D textures") :
			TEXT("")
		);
	}
	
	// For virtual texturing we take the address mode into consideration
	if (OutBuildSettings.bVirtualStreamable)
	{
		const UTexture2D *Texture2D = Cast<UTexture2D>(&Texture);
		checkf(Texture2D, TEXT("Virtual texturing is only supported on 2D textures"));
		if (Texture.Source.GetNumBlocks() > 1)
		{
			// Multi-block textures (UDIM) interpret UVs outside [0,1) range as different blocks, so wrapping within a given block doesn't make sense
			// We want to make sure address mode is set to clamp here, otherwise border pixels along block edges will have artifacts
			OutBuildSettings.VirtualAddressingModeX = TA_Clamp;
			OutBuildSettings.VirtualAddressingModeY = TA_Clamp;
		}
		else
		{
			OutBuildSettings.VirtualAddressingModeX = Texture2D->AddressX;
			OutBuildSettings.VirtualAddressingModeY = Texture2D->AddressY;
		}

		FVirtualTextureBuildSettings VirtualTextureBuildSettings;
		Texture.GetVirtualTextureBuildSettings(VirtualTextureBuildSettings);
		OutBuildSettings.VirtualTextureTileSize = FVirtualTextureBuildSettings::ClampAndAlignTileSize(VirtualTextureBuildSettings.TileSize);

		// Apply any LOD group tile size bias here
		const int32 TileSizeBias = TextureLODSettings.GetTextureLODGroup(Texture.LODGroup).VirtualTextureTileSizeBias;
		OutBuildSettings.VirtualTextureTileSize >>= (TileSizeBias < 0) ? -TileSizeBias : 0;
		OutBuildSettings.VirtualTextureTileSize <<= (TileSizeBias > 0) ? TileSizeBias : 0;

		// Don't allow max resolution to be less than VT tile size
		OutBuildSettings.MaxTextureResolution = FMath::Max<uint32>(OutBuildSettings.MaxTextureResolution, OutBuildSettings.VirtualTextureTileSize);

		// 0 is a valid value for border size
		// 1 would be OK in some cases, but breaks BC compressed formats, since it will result in physical tiles that aren't divisible by block size (4)
		// Could allow border size of 1 for non BC compressed virtual textures, but somewhat complicated to get that correct, especially with multiple layers
		// Doesn't seem worth the complexity for now, so ensure we use multiple of 2
		OutBuildSettings.VirtualTextureBorderSize = FVirtualTextureBuildSettings::ClampAndAlignTileBorderSize(VirtualTextureBuildSettings.TileBorderSize);
	}
	else
	{
		OutBuildSettings.VirtualAddressingModeX = TA_Wrap;
		OutBuildSettings.VirtualAddressingModeY = TA_Wrap;
		OutBuildSettings.VirtualTextureTileSize = 0;
		OutBuildSettings.VirtualTextureBorderSize = 0;
	}
	
	OutBuildSettings.TextureAddressModeX = Texture.GetTextureAddressX();
	OutBuildSettings.TextureAddressModeY = Texture.GetTextureAddressY();
	OutBuildSettings.TextureAddressModeZ = Texture.GetTextureAddressZ();

	// By default, initialize settings for layer0
	FinalizeBuildSettingsForLayer(Texture, 0, &TargetPlatform, InEncodeSpeed, OutBuildSettings, OutBuildResultMetadata);
}

/**
 * Sets build settings for a texture on the target platform
 * @param Texture - The texture for which to build compressor settings.
 * @param OutBuildSettings - Array of desired texture settings
 */
static void GetBuildSettingsForTargetPlatform(
	const UTexture& Texture,
	const ITargetPlatform* TargetPlatform,
	ETextureEncodeSpeed InEncodeSpeed, //  must be Fast or Final
	TArray<FTextureBuildSettings>& OutSettingPerLayer,
	TArray<FTexturePlatformData::FTextureEncodeResultMetadata>* OutResultMetadataPerLayer // can be nullptr if not needed
)
{
	// Beware GetTargetPlatformBuildSettings and GetBuildSettingsForTargetPlatform are almost code dupes of each other; keep in sync!
	//	todo: refactor to dedupe

	check(TargetPlatform != NULL);

	const UTextureLODSettings* LODSettings = UDeviceProfileManager::Get().FindProfile(TargetPlatform->CookingDeviceProfileName())->GetTextureLODSettings();
	FTextureBuildSettings SourceBuildSettings;
	FTexturePlatformData::FTextureEncodeResultMetadata SourceMetadata;
	GetTextureBuildSettings(Texture, *LODSettings, *TargetPlatform, InEncodeSpeed, SourceBuildSettings, &SourceMetadata);

	TArray< TArray<FName> > PlatformFormats;
	Texture.GetPlatformTextureFormatNamesWithPrefix(TargetPlatform,PlatformFormats);

	// this code only uses PlatformFormats[0] , so it would be wrong for Android_Multi
	//	but it's only used for the platform running the Editor
	// ^^ Wrong now, when previewing platform data we run this. Since multi is also
	// exposed as other target platforms, we are fine with it only using [0].
	//check(PlatformFormats.Num() == 1);

	const int32 NumLayers = Texture.Source.GetNumLayers();
	check(PlatformFormats[0].Num() == NumLayers);

	OutSettingPerLayer.Reserve(NumLayers);
	if (OutResultMetadataPerLayer)
	{
		OutResultMetadataPerLayer->Reserve(NumLayers);
	}
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FTextureBuildSettings& OutSettings = OutSettingPerLayer.Add_GetRef(SourceBuildSettings);
		OutSettings.TextureFormatName = PlatformFormats[0][LayerIndex];

		FTexturePlatformData::FTextureEncodeResultMetadata* OutMetadata = nullptr;
		if (OutResultMetadataPerLayer)
		{
			OutMetadata = &OutResultMetadataPerLayer->Add_GetRef(SourceMetadata);
		}
			
		FinalizeBuildSettingsForLayer(Texture, LayerIndex, TargetPlatform, InEncodeSpeed, OutSettings, OutMetadata);
	}
}

/**
 * Sets build settings for a texture on the current running platform
 * @param Texture - The texture for which to build compressor settings.
 * @param OutBuildSettings - Array of desired texture settings
 */
static void GetBuildSettingsForRunningPlatform(
	const UTexture& Texture,
	ETextureEncodeSpeed InEncodeSpeed, //  must be Fast or Final
	TArray<FTextureBuildSettings>& OutSettingPerLayer,
	TArray<FTexturePlatformData::FTextureEncodeResultMetadata>* OutResultMetadataPerLayer // can be nullptr if not needed
	)
{
	// Compress to whatever formats the active target platforms want
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (TPM)
	{
		ITargetPlatform* TargetPlatform = TPM->GetRunningTargetPlatform();

		check(TargetPlatform != NULL);

		bool bNeedsDecode = false;
		if (Texture.OverrideRunningPlatformName != NAME_None)
		{
			if (Texture.VirtualTextureStreaming)
			{
				UE_LOGF(LogTexture, Display, "Platform viewing not supported with virtual textures (%ls)", *Texture.GetPathName());
			}
			else if (Texture.Availability == ETextureAvailability::GPU) // only makes sense if encoded!
			{
				ITargetPlatform* OverridePlatform = TPM->FindTargetPlatform(Texture.OverrideRunningPlatformName);
				if (OverridePlatform)
				{
					UE_LOGF(LogTexture, Display, "Overriding running platform for texture %ls from %ls to %ls", *Texture.GetPathName(), *TargetPlatform->PlatformName(), *OverridePlatform->PlatformName());
					TargetPlatform = OverridePlatform;
					bNeedsDecode = true;
				}
			}
		}

		GetBuildSettingsForTargetPlatform(Texture, TargetPlatform, InEncodeSpeed, OutSettingPerLayer, OutResultMetadataPerLayer);
		for (FTextureBuildSettings& LayerSettings : OutSettingPerLayer)
		{
			LayerSettings.bDecodeForPCUsage = bNeedsDecode;
		}
	}
}

static void GetBuildSettingsPerFormat(
	const UTexture& Texture, 
	const FTextureBuildSettings& SourceBuildSettings, 
	const FTexturePlatformData::FTextureEncodeResultMetadata* SourceResultMetadata, // can be nullptr if not capturing metadata
	const ITargetPlatform* TargetPlatform, 
	ETextureEncodeSpeed InEncodeSpeed, //  must be Fast or Final
	TArray< TArray<FTextureBuildSettings> >& OutBuildSettingsPerFormat,
	TArray< TArray<FTexturePlatformData::FTextureEncodeResultMetadata> >* OutResultMetadataPerFormat // can be nullptr if not capturing metadata
	)
{
	const int32 NumLayers = Texture.Source.GetNumLayers();

	TArray< TArray<FName> > PlatformFormats;
	Texture.GetPlatformTextureFormatNamesWithPrefix(TargetPlatform,PlatformFormats);

	// almost always == 1, except for Android_Multi, which makes an array of layer formats per variant
	// also OutFormats.Num() == 0 for server-only platforms

	OutBuildSettingsPerFormat.Reserve(PlatformFormats.Num());
	if (OutResultMetadataPerFormat)
	{
		OutResultMetadataPerFormat->Reserve(PlatformFormats.Num());
	}
	for (TArray<FName>& PlatformFormatsPerLayer : PlatformFormats)
	{
		check(PlatformFormatsPerLayer.Num() == NumLayers);
		TArray<FTextureBuildSettings>& OutSettingPerLayer = OutBuildSettingsPerFormat.AddDefaulted_GetRef();
		OutSettingPerLayer.Reserve(NumLayers);

		TArray<FTexturePlatformData::FTextureEncodeResultMetadata>* OutResultMetadataPerLayer = nullptr;
		if (OutResultMetadataPerFormat)
		{
			OutResultMetadataPerLayer = &OutResultMetadataPerFormat->AddDefaulted_GetRef();
			OutResultMetadataPerLayer->Reserve(NumLayers);
		}

		for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			FTextureBuildSettings& OutSettings = OutSettingPerLayer.Add_GetRef(SourceBuildSettings);
			OutSettings.TextureFormatName = PlatformFormatsPerLayer[LayerIndex];

			FTexturePlatformData::FTextureEncodeResultMetadata* OutResultMetadata = nullptr;
			if (OutResultMetadataPerLayer && SourceResultMetadata)
			{
				OutResultMetadata = &OutResultMetadataPerLayer->Add_GetRef(*SourceResultMetadata);			
			}

			FinalizeBuildSettingsForLayer(Texture, LayerIndex, TargetPlatform, InEncodeSpeed, OutSettings, OutResultMetadata);
		}
	}
}

/**
 * Stores derived data in the DDC.
 * After this returns, all bulk data from streaming (non-inline) mips will be sent separately to the DDC and the BulkData for those mips removed.
 * @param DerivedData - The data to store in the DDC.
 * @param DerivedDataKeySuffix - The key suffix at which to store derived data.
 * @param bForceAllMipsToBeInlined - Whether to store all mips in the main DDC. Relates to how the texture resources get initialized (not supporting streaming).
 * @return number of bytes put to the DDC (total, including all mips)
 */
int64 PutDerivedDataInCache(FTexturePlatformData* DerivedData, const FString& DerivedDataKeySuffix, const FStringView& TextureName, bool bForceAllMipsToBeInlined, bool bReplaceExistingDDC)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.PutDerivedDataInCache);

	TArray64<uint8> RawDerivedData;
	FString DerivedDataKey;
	int64 TotalBytesPut = 0;

	// Build the key with which to cache derived data.
	GetTextureDerivedDataKeyFromSuffix(DerivedDataKeySuffix, DerivedDataKey);

	FString LogString;

	// Write out individual mips to the derived data cache.
	const int32 MipCount = DerivedData->Mips.Num();
	const int32 FirstInlineMip = bForceAllMipsToBeInlined ? 0 : FMath::Max(0, MipCount - FMath::Max((int32)NUM_INLINE_DERIVED_MIPS, (int32)DerivedData->GetNumMipsInTail()));
	const int32 WritableMipCount = MipCount - ((DerivedData->GetNumMipsInTail() > 0) ? (DerivedData->GetNumMipsInTail() - 1) : 0);
	for (int32 MipIndex = 0; MipIndex < WritableMipCount; ++MipIndex)
	{
		FString MipDerivedDataKey;
		FTexture2DMipMap& Mip = DerivedData->Mips[MipIndex];
		const bool bInline = (MipIndex >= FirstInlineMip);
		GetTextureDerivedMipKey(MipIndex, Mip, DerivedDataKeySuffix, MipDerivedDataKey);

		const bool bDDCError = !bInline && !Mip.BulkData.GetBulkDataSize();
		if (UE_LOG_ACTIVE(LogTexture,Verbose) || bDDCError)
		{
			if (LogString.IsEmpty())
			{
				LogString = FString::Printf(
					TEXT("Storing texture in DDC:\n  Name: %s\n  Key: %s\n  Format: %s\n"),
					*FString(TextureName),
					*DerivedDataKey,
					GPixelFormats[DerivedData->PixelFormat].Name
				);
			}

			LogString += FString::Printf(TEXT("  Mip%d %dx%d %" UINT64_FMT " bytes%s %s\n"),
				MipIndex,
				Mip.SizeX,
				Mip.SizeY,
				Mip.BulkData.GetBulkDataSize(),
				bInline ? TEXT(" [inline]") : TEXT(""),
				*MipDerivedDataKey
				);
		}

		if (bDDCError)
		{
			UE_LOGF(LogTexture, Fatal, "Error %ls", *LogString);
		}

		// Note that calling StoreInDerivedDataCache() also calls RemoveBulkData().
		// This means that the resource needs to load differently inlined mips and non inlined mips.
		if (!bInline)
		{
			// store in the DDC, also drop the bulk data storage.
			TotalBytesPut += Mip.StoreInDerivedDataCache(MipDerivedDataKey, TextureName, bReplaceExistingDDC);
		}
	}

	// Write out each VT chunk to the DDC
	bool bReplaceExistingDerivedDataDDC = bReplaceExistingDDC;
	if (DerivedData->VTData)
	{
		const int32 ChunkCount = DerivedData->VTData->Chunks.Num();
		for (int32 ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex)
		{
			FVirtualTextureDataChunk& Chunk = DerivedData->VTData->Chunks[ChunkIndex];

			const FString ChunkDerivedDataKey = FDerivedDataCacheInterface::BuildCacheKey(
				TEXT("TEXTURE"), TEXTURE_VT_DERIVEDDATA_VER,
				*FString::Printf(TEXT("VTCHUNK%s"), *Chunk.BulkDataHash.ToString()));

			TotalBytesPut += Chunk.StoreInDerivedDataCache(ChunkDerivedDataKey, TextureName, bReplaceExistingDDC);
		}

		// VT always needs to replace the FVirtualTextureBuiltData in the DDC, otherwise we can be left in a situation where a local client is constantly attempting to rebuild chunks,
		// but failing to generate chunks that match the FVirtualTextureBuiltData in the DDC, due to non-determinism in texture generation
		bReplaceExistingDerivedDataDDC = true;
	}

	// Store derived data.
	// At this point we've stored all the non-inline data in the DDC, so this will only serialize and store the TexturePlatformData metadata and any inline mips
	FMemoryWriter64 Ar(RawDerivedData, /*bIsPersistent=*/ true);
	DerivedData->Serialize(Ar, NULL);
	const int64 RawDerivedDataSize = RawDerivedData.Num();
	TotalBytesPut += RawDerivedDataSize;

	using namespace UE::DerivedData;
	FRequestOwner AsyncOwner(EPriority::Normal);
	FValue Value = FValue::Compress(MakeSharedBufferFromArray(MoveTemp(RawDerivedData)));
	const ECachePolicy Policy = bReplaceExistingDerivedDataDDC ? ECachePolicy::Store : ECachePolicy::Default;
	GetCache().PutValue({{{TextureName}, ConvertLegacyCacheKey(DerivedDataKey), MoveTemp(Value), Policy}}, AsyncOwner);
	AsyncOwner.KeepAlive();

	UE_LOG(LogTexture, Verbose, TEXT("%s  Derived Data: %" INT64_FMT " bytes"), *LogString, RawDerivedDataSize);
	return TotalBytesPut;
}

#endif // #if WITH_EDITOR

#if WITH_EDITORONLY_DATA

FGuid UTexture::BuildLightingGuidFromHash()
{
#if !WITH_EDITOR
	return FGuid::NewGuid();
#else
	if (!Source.IsValid())
	{
		return FGuid::NewDeterministicGuid(GetPathName());
	}

	// Use the DDC key rather than reading just the texels, because the DDC key includes some lighting settings
	// that impact lighting on materials using the texture. Those settings are why we need a LightingGuid that is
	// separate from the texture's Id; the Id depends only on the texels.

	// Use a single TargetPlatform so we get the same LightingSettings in every editor platform, so that we get the
	// same LightingGuid no matter which platform we're running the editor on. We arbitrarily pick Windows for that
	// platform since it is the most commonly-used editor platform.
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (!TPM)
	{
		return FGuid::NewDeterministicGuid(GetPathName());
	}
	ITargetPlatform* SettingsPlatform = TPM->FindTargetPlatform(TEXT("Windows"));
	if (!SettingsPlatform)
	{
		return FGuid::NewDeterministicGuid(GetPathName());
	}

	TArray<FTextureBuildSettings> BuildSettings;
	GetBuildSettingsForTargetPlatform(*this, SettingsPlatform, ETextureEncodeSpeed::Final,
		BuildSettings, nullptr /* OutResultMetadataPerLayer */);
	if (BuildSettings.IsEmpty() || BuildSettings.Num() < Source.GetNumLayers())
	{
		return FGuid::NewDeterministicGuid(GetPathName());
	}

	FString DerivedDataKey;
	GetTextureDerivedDataKey(*this, BuildSettings.GetData(), DerivedDataKey);

	FXxHash128Builder Hasher;
	Hasher.Update(*DerivedDataKey, DerivedDataKey.Len() * sizeof(**DerivedDataKey));
	uint8 Hash[16];
	Hasher.Finalize().ToByteArray(Hash);
	return FGuid::NewGuidFromHashBytes(&Hash, sizeof(Hash));
#endif
}

#endif // WITH_EDITORONLY_DATA

/*------------------------------------------------------------------------------
	Derived data.
------------------------------------------------------------------------------*/

#if WITH_EDITOR

void FTexturePlatformData::Cache(
	UTexture& InTexture,
	const FTextureBuildSettings* InSettingsPerLayerFetchFirst, // can be null
	const FTextureBuildSettings* InSettingsPerLayerFetchOrBuild, // must be valid
	const FTexturePlatformData::FTextureEncodeResultMetadata* OutResultMetadataPerLayerFetchFirst, // can be nullptr
	const FTexturePlatformData::FTextureEncodeResultMetadata* OutResultMetadataPerLayerFetchOrBuild, // can be nullptr
	uint32 InFlags,
	ITextureCompressorModule* Compressor
	)
{
	//
	// Note this can be called off the main thread, despite referencing a UObject!
	// Be very careful!
	// (as of this writing, the shadow and light maps can call CachePlatformData
	// off the main thread via FAsyncEncode<>.)
	//
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FTexturePlatformData::Cache);

	// Flush any existing async task and ignore results.
	CancelCache();

	if ( InTexture.Source.GetNumLayers() == 0 )
	{
		UE_LOGF(LogTexture, Error, "Attempt to cache texture with zero layers [%ls]", *InTexture.GetName());
		return;
	}

	check( InSettingsPerLayerFetchOrBuild != nullptr );

	ETextureCacheFlags Flags = ETextureCacheFlags(InFlags);

	if (IsUsingNewDerivedData() && InTexture.Source.GetNumLayers() == 1 && !InSettingsPerLayerFetchOrBuild->bVirtualStreamable)
	{
		COOK_STAT(auto Timer = TextureCookStats::UsageStats.TimeSyncWork());
		COOK_STAT(Timer.TrackCyclesOnly());
		EQueuedWorkPriority Priority = FTextureCompilingManager::Get().GetBasePriority(&InTexture);
		AsyncTask = CreateTextureBuildTask(
			InTexture, 
			*this, 
			InSettingsPerLayerFetchFirst, 
			*InSettingsPerLayerFetchOrBuild, 
			OutResultMetadataPerLayerFetchFirst, 
			OutResultMetadataPerLayerFetchOrBuild, 
			Priority, 
			Flags);
		if (AsyncTask)
		{
			return;
		}
		UE_LOGF(LogTexture, Warning, "Failed to create requested DDC2 build task for texture %ls -- falling back to DDC1", *InTexture.GetName());
	}

	//
	// DDC1 from here on out.
	//

	static bool bForDDC = FString(FCommandLine::Get()).Contains(TEXT("Run=DerivedDataCache"));
	if (bForDDC)
	{
		Flags |= ETextureCacheFlags::ForDDCBuild;
	}

	bool bForceRebuild = EnumHasAnyFlags(Flags, ETextureCacheFlags::ForceRebuild);
	bool bAsync = EnumHasAnyFlags(Flags, ETextureCacheFlags::Async);

	if (!Compressor)
	{
		Compressor = &FModuleManager::LoadModuleChecked<ITextureCompressorModule>(TEXTURE_COMPRESSOR_MODULENAME);
	}

	if (InSettingsPerLayerFetchOrBuild[0].bVirtualStreamable)
	{
		Flags |= ETextureCacheFlags::ForVirtualTextureStreamingBuild;
	}

	if (bAsync)
	{
		FQueuedThreadPool*  TextureThreadPool = FTextureCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority      = FTextureCompilingManager::Get().GetBasePriority(&InTexture);

		COOK_STAT(auto Timer = TextureCookStats::UsageStats.TimeSyncWork());
		COOK_STAT(Timer.TrackCyclesOnly());
		FTextureAsyncCacheDerivedDataWorkerTask* LocalTask = new FTextureAsyncCacheDerivedDataWorkerTask(
			TextureThreadPool, 
			Compressor, 
			this, 
			&InTexture, 
			InSettingsPerLayerFetchFirst, 
			InSettingsPerLayerFetchOrBuild,
			OutResultMetadataPerLayerFetchFirst, 
			OutResultMetadataPerLayerFetchOrBuild,
			Flags);

		// LocalTask->TextureData Init may have failed and have bValid = false
		//	but we still go ahead and create the async task, perhaps wrongly so

		AsyncTask = LocalTask;
		LocalTask->StartBackgroundTask(TextureThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, LocalTask->GetTask().GetRequiredMemoryEstimate(), TEXT("TextureDerivedData"));
	}
	else
	{
		FTextureCacheDerivedDataWorker Worker(
			Compressor, 
			this, 
			&InTexture, 
			InSettingsPerLayerFetchFirst, 
			InSettingsPerLayerFetchOrBuild, 
			OutResultMetadataPerLayerFetchFirst,
			OutResultMetadataPerLayerFetchOrBuild,
			Flags);
		{
			COOK_STAT(auto Timer = TextureCookStats::UsageStats.TimeSyncWork());
			Worker.DoWork();
			Worker.Finalize();

			COOK_STAT(Timer.AddHitOrMiss(Worker.WasLoadedFromDDC() ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, Worker.GetBytesCached()));
		}
	}
}

bool FTexturePlatformData::TryCancelCache()
{
	if (AsyncTask && AsyncTask->Cancel())
	{
		delete AsyncTask;
		AsyncTask = nullptr;
	}
	return !AsyncTask;
}

void FTexturePlatformData::CancelCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTexturePlatformData::CancelCache)

	// If we're unable to cancel, it means it's already being processed, we must finish it then.
	if (!TryCancelCache())
	{
		FinishCache();
	}
}

bool FTexturePlatformData::IsAsyncWorkComplete() const
{
	return !AsyncTask || AsyncTask->Poll();
}

void FTexturePlatformData::FinishCache()
{
	if (AsyncTask)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTexturePlatformData::FinishCache)
		{
			COOK_STAT(auto Timer = TextureCookStats::UsageStats.TimeAsyncWait());
			bool bFoundInCache = false;
			uint64 ProcessedByteCount = 0;
			AsyncTask->Wait();
			AsyncTask->Finalize(bFoundInCache, ProcessedByteCount);
			COOK_STAT(Timer.AddHitOrMiss(bFoundInCache ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, int64(ProcessedByteCount)));
		}
		delete AsyncTask;
		AsyncTask = nullptr;
	}
}

void FTexturePlatformData::Reset()
{
	Mips.Empty();
	SizeX = 0;
	SizeY = 0;
	PixelFormat = PF_Unknown;
	PackedData = 0;
	OptData = FOptTexturePlatformData();
	if (VTData)
	{
		delete VTData;
	}
	VTData = nullptr;
	CPUCopy.SafeRelease();

#if WITH_EDITORONLY_DATA
	PreEncodeMipsHash = 0;
	ResultMetadata.bIsValid = false;
#endif
}


typedef TArray<uint32, TInlineAllocator<MAX_TEXTURE_MIP_COUNT> > FAsyncMipHandles;
typedef TArray<uint32> FAsyncVTChunkHandles;

/**
 * Executes async DDC gets for mips stored in the derived data cache.
 * @param Mip - Mips to retrieve.
 * @param FirstMipToLoad - Index of the first mip to retrieve.
 * @param Callback - Callback invoked for each mip as it loads.
 * 
 * This function must be called after the initial DDC fetch is complete,
 * so we know what our in-use key is. This might be on the worker immediately
 * after the fetch completes.
 */
static bool LoadDerivedStreamingMips(FTexturePlatformData& PlatformData, int32 FirstMipToLoad, int32 MaxMipCount, bool bTouchOnly, FStringView DebugContext, TFunctionRef<void (int32 MipIndex, FSharedBuffer MipData)> Callback)
{
	using namespace UE::DerivedData;

	bool bMiss = false;

	TIndirectArray<FTexture2DMipMap>& Mips = PlatformData.Mips;
	int32 ReadableMipCount = Mips.Num() - (PlatformData.GetNumMipsInTail() > 0 ? PlatformData.GetNumMipsInTail() - 1 : 0);
	ReadableMipCount = FMath::Min(MaxMipCount, ReadableMipCount);
	

	if (PlatformData.DerivedDataKey.IsType<FString>())
	{
		TArray<FCacheGetValueRequest, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> Requests;

		for (int32 MipIndex = FirstMipToLoad; MipIndex < ReadableMipCount; ++MipIndex)
		{
			const FTexture2DMipMap& Mip = Mips[MipIndex];
			if (Mip.IsPagedToDerivedData() && !Mip.BulkData.IsBulkDataLoaded())
			{
				TStringBuilder<256> MipNameBuilder;
				MipNameBuilder.Append(DebugContext).Appendf(TEXT(" [MIP %d]"), MipIndex);
				FCacheGetValueRequest& Request = Requests.AddDefaulted_GetRef();
				Request.Name = MipNameBuilder;
				Request.Key = ConvertLegacyCacheKey(PlatformData.GetDerivedDataMipKeyString(MipIndex, Mip));
				Request.UserData = MipIndex;
				if (bTouchOnly)
				{
					Request.Policy = ECachePolicy::Query | ECachePolicy::SkipData;
				}
			}
		}

		if (!Requests.IsEmpty())
		{
			if (bTouchOnly)
			{
				// We want to fire off the request but we honestly don't care about the results,
				// this is just to update server referencing timestamps.
				FRequestOwner Owner(EPriority::Lowest);
				Owner.KeepAlive();
				GetCache().GetValue(Requests, Owner, [](FCacheGetValueResponse&&) {} );
			}
			else
			{
				COOK_STAT(auto Timer = TextureCookStats::StreamingMipUsageStats.TimeSyncWork());
				uint64 Size = 0;
				FRequestOwner BlockingOwner(EPriority::Blocking);
				GetCache().GetValue(Requests, BlockingOwner, [Callback = MoveTemp(Callback), &Size, &bMiss](FCacheGetValueResponse&& Response)
					{
						Size += Response.Value.GetRawSize();
						if (Response.Status == EStatus::Ok)
						{
							Callback(int32(Response.UserData), Response.Value.GetData().Decompress());
						}
						else
						{
							bMiss = true;
						}
					});
				BlockingOwner.Wait();
				COOK_STAT(Timer.AddHitOrMiss(!bMiss ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, int64(Size)));
			}
		}
	}
	else if (PlatformData.DerivedDataKey.IsType<FCacheKeyProxy>())
	{
		TArray<FCacheGetChunkRequest, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> Requests;

		const FCacheKey& Key = *PlatformData.DerivedDataKey.Get<UE::DerivedData::FCacheKeyProxy>().AsCacheKey();
		for (int32 MipIndex = FirstMipToLoad; MipIndex < ReadableMipCount; ++MipIndex)
		{
			const FTexture2DMipMap& Mip = Mips[MipIndex];
			if (Mip.IsPagedToDerivedData() && !Mip.BulkData.IsBulkDataLoaded())
			{
				TStringBuilder<256> MipNameBuilder;
				MipNameBuilder.Append(DebugContext).Appendf(TEXT(" [MIP %d]"), MipIndex);
				FCacheGetChunkRequest& Request = Requests.AddDefaulted_GetRef();
				Request.Name = MipNameBuilder;
				Request.Key = Key;
				Request.Id = FTexturePlatformData::MakeMipId(MipIndex);
				Request.UserData = MipIndex;

				// We don't need to handle Touch because GetChunks uses CacheRecords and all mips
				// are touched if we touch any mip.
			}
		}

		if (!Requests.IsEmpty())
		{
			COOK_STAT(auto Timer = TextureCookStats::StreamingMipUsageStats.TimeSyncWork());
			uint64 Size = 0;
			FRequestOwner BlockingOwner(EPriority::Blocking);
			GetCache().GetChunks(Requests, BlockingOwner, [Callback = MoveTemp(Callback), &Size, &bMiss](FCacheGetChunkResponse&& Response)
			{
				Size += Response.RawSize;
				if (Response.Status == EStatus::Ok)
				{
					Callback(int32(Response.UserData), MoveTemp(Response.RawData));
				}
				else
				{
					bMiss = true;
				}
			});
			BlockingOwner.Wait();
			COOK_STAT(Timer.AddHitOrMiss(!bMiss ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, int64(Size)));
		}
	}
	else
	{
		UE_LOGF(LogTexture, Error, "Attempting to stream in mips for texture that has not generated a supported derived data key format.");
	}

	return !bMiss;
}

static bool LoadDerivedStreamingVTChunks(const TArray<FVirtualTextureDataChunk>& Chunks, FStringView DebugContext, TFunctionRef<void (int32 ChunkIndex, FSharedBuffer ChunkData)> Callback)
{
	using namespace UE;
	using namespace UE::DerivedData;
	TArray<FCacheGetValueRequest> Requests;

	for (int32 ChunkIndex = 0; ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		const FVirtualTextureDataChunk& Chunk = Chunks[ChunkIndex];
		if (!Chunk.DerivedDataKey.IsEmpty() && !Chunk.BulkData.IsBulkDataLoaded())
		{
			FCacheGetValueRequest& Request = Requests.AddDefaulted_GetRef();
			Request.Name = FSharedString(WriteToString<256>(DebugContext, TEXT(" [Chunk "), ChunkIndex, TEXT("]")));
			Request.Key = ConvertLegacyCacheKey(Chunk.DerivedDataKey);
			Request.UserData = ChunkIndex;
		}
	}

	bool bMiss = false;

	if (!Requests.IsEmpty())
	{
		COOK_STAT(auto Timer = TextureCookStats::StreamingMipUsageStats.TimeSyncWork());
		uint64 Size = 0;
		FRequestOwner BlockingOwner(EPriority::Blocking);
		GetCache().GetValue(Requests, BlockingOwner, [Callback = MoveTemp(Callback), &Size, &bMiss](FCacheGetValueResponse&& Response)
		{
			Size += Response.Value.GetRawSize();
			if (Response.Status == EStatus::Ok)
			{
				Callback(int32(Response.UserData), Response.Value.GetData().Decompress());
			}
			else
			{
				bMiss = true;
			}
		});
		BlockingOwner.Wait();
		COOK_STAT(Timer.AddHitOrMiss(!bMiss ? FCookStats::CallStats::EHitOrMiss::Hit : FCookStats::CallStats::EHitOrMiss::Miss, int64(Size)));
	}

	return !bMiss;
}

/** Logs a warning that MipSize is correct for the mipmap. */
static void CheckMipSize(FTexture2DMipMap& Mip, EPixelFormat PixelFormat, int64 MipSize)
{
	// this check is incorrect ; it does not account of platform tiling and padding done on textures
	// re-enable if fixed

	#if 0
	// Only volume can have SizeZ != 1
	if (MipSize != Mip.SizeZ * CalcTextureMipMapSize(Mip.SizeX, Mip.SizeY, PixelFormat, 0))
	{
		UE_LOG(LogTexture, Warning,
			TEXT("%dx%d mip of %s texture has invalid data in the DDC. Got %" INT64_FMT " bytes, expected %" SIZE_T_FMT ". Key=%s"),
			Mip.SizeX,
			Mip.SizeY,
			GPixelFormats[PixelFormat].Name,
			MipSize,
			CalcTextureMipMapSize(Mip.SizeX, Mip.SizeY, PixelFormat, 0),
			*Mip.DerivedDataKey
			);
	}
	#endif
}

//
// Retrieve all built texture data in to the associated arrays, and don't return unless there's an error
// or we have the data.
//
static bool FetchAllTextureDataSynchronous(FTexturePlatformData* PlatformData, FStringView DebugContext, TArray<TArray64<uint8>>& OutMipData, TArray<TArray64<uint8>>& OutVTChunkData)
{
	const auto StoreMip = [&OutMipData](int32 MipIndex, FSharedBuffer MipBuffer)
	{
		OutMipData[MipIndex].Append(static_cast<const uint8*>(MipBuffer.GetData()), MipBuffer.GetSize());
	};

	const int32 MipCount = PlatformData->Mips.Num();
	OutMipData.Empty(MipCount);
	OutMipData.AddDefaulted(MipCount);

	if (!LoadDerivedStreamingMips(*PlatformData, 0, MipCount, false, DebugContext, StoreMip))
	{
		return false;
	}

	for (int32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
	{
		FTexture2DMipMap& Mip = PlatformData->Mips[MipIndex];
		if (OutMipData[MipIndex].IsEmpty())
		{
			if (Mip.BulkData.IsBulkDataLoaded())
			{
				OutMipData[MipIndex].Append((uint8*)Mip.BulkData.LockReadOnly(), Mip.BulkData.GetBulkDataSize());
				Mip.BulkData.Unlock();
			}
			else
			{
				return false;
			}
		}
	}

	const auto StoreChunk = [&OutVTChunkData](int32 ChunkIndex, FSharedBuffer ChunkBuffer)
	{
		OutVTChunkData[ChunkIndex].Append(static_cast<const uint8*>(ChunkBuffer.GetData()), ChunkBuffer.GetSize());
	};

	const int32 ChunkCount = PlatformData->VTData ? PlatformData->VTData->Chunks.Num() : 0;
	OutVTChunkData.Empty(ChunkCount);
	if (ChunkCount)
	{
		OutVTChunkData.AddDefaulted(ChunkCount);

		if (!LoadDerivedStreamingVTChunks(PlatformData->VTData->Chunks, DebugContext, StoreChunk))
		{
			return false;
		}

		for (int32 ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex)
		{
			FVirtualTextureDataChunk& Chunk = PlatformData->VTData->Chunks[ChunkIndex];
			if (OutVTChunkData[ChunkIndex].IsEmpty())
			{
				if (Chunk.BulkData.IsBulkDataLoaded())
				{
					// The data is resident and we can just copy it.
					OutVTChunkData[ChunkIndex].Append((uint8*)Chunk.BulkData.LockReadOnly(), Chunk.BulkData.GetBulkDataSize());
					Chunk.BulkData.Unlock();
				}
				else
				{
					return false;
				}
			}
		}
	}

	return true;
}

//
// Chunk the input data in to blocks of the compression block size, then
// run Oodle on the separate chunks in order to get an estimate of how
// much space on disk the texture will take during deployment. This
// exists so the editor can show the benefits of increasing RDO levels 
// on a texture.
//
// This is not exact! Due to the nature of iostore, we can't know exactly
// whether our data will be chunked on the boundaries we've chosen. However
// it is illustrative.
//
static void EstimateOnDiskCompressionForTextureData(
	TArray<TArray64<uint8>> InMipData,
	TArray<TArray64<uint8>> InVTChunkData,
	FOodleDataCompression::ECompressor InOodleCompressor,
	FOodleDataCompression::ECompressionLevel InOodleCompressionLevel,
	uint32 InCompressionBlockSize,
	uint64& OutUncompressedByteCount,
	uint64& OutCompressedByteCount
)
{
	//
	// This is written such that you can have both classic mip data and
	// virtual texture data, however actual unreal textures don't have
	// both.
	//
	uint64 UncompressedByteCount = 0;
	for (TArray64<uint8>& Mip : InMipData)
	{
		UncompressedByteCount += Mip.Num();
	}
	for (TArray64<uint8>& VTChunk : InVTChunkData)
	{
		UncompressedByteCount += VTChunk.Num();
	}

	OutUncompressedByteCount = UncompressedByteCount;

	if (UncompressedByteCount == 0)
	{
		OutCompressedByteCount = 0;
		return;
	}

	int32 MipIndex = 0;
	int32 VTChunkIndex = 0;
	int64 CurrentOffsetInContainer = 0;
	uint64 CompressedByteCount = 0;

	// Array for compressed data so we don't have to realloc.
	TArray<uint8> Compressed;
	Compressed.Reserve(InCompressionBlockSize + 1024);

	// When we cross our input array boundaries, we accumulate in to here.
	TArray64<uint8> ContinuousMemory;
	for (;;)
	{
		TArray64<uint8>& CurrentContainer = MipIndex < InMipData.Num() ? InMipData[MipIndex] : InVTChunkData[VTChunkIndex];

		uint64 NeedBytes = InCompressionBlockSize - ContinuousMemory.Num();
		uint64 CopyBytes = CurrentContainer.Num() - CurrentOffsetInContainer;
		if (CopyBytes > NeedBytes)
		{
			CopyBytes = NeedBytes;
		}

		// Can we compressed without an intervening copy?
		if (NeedBytes == InCompressionBlockSize && // We don't have a partial block copied
			CopyBytes == InCompressionBlockSize) // we can fit in this chunk
		{
			// Direct.
			Compressed.SetNum(0,EAllowShrinking::No);
			FOodleCompressedArray::CompressData(
				Compressed,
				CurrentContainer.GetData() + CurrentOffsetInContainer,
				InCompressionBlockSize,
				InOodleCompressor,
				InOodleCompressionLevel);

			CompressedByteCount += Compressed.Num();
		}
		else
		{
			// Need to accumulate in to our temp buffer.

			if (ContinuousMemory.Num() == 0)
			{
				ContinuousMemory.Reserve(InCompressionBlockSize);
			}

			ContinuousMemory.Append(CurrentContainer.GetData() + CurrentOffsetInContainer, CopyBytes);

			if (ContinuousMemory.Num() == InCompressionBlockSize)
			{
				// Filled a block - kick.
				Compressed.SetNum(0,EAllowShrinking::No);
				FOodleCompressedArray::CompressData(
					Compressed,
					ContinuousMemory.GetData(),
					InCompressionBlockSize,
					InOodleCompressor,
					InOodleCompressionLevel);

				CompressedByteCount += Compressed.Num();
				ContinuousMemory.Empty();
			}
		}

		// Advance read cursor.
		CurrentOffsetInContainer += CopyBytes;
		if (CurrentOffsetInContainer >= CurrentContainer.Num())
		{
			CurrentOffsetInContainer = 0;

			if (MipIndex < InMipData.Num())
			{
				MipIndex++;
			}
			else if (VTChunkIndex < InVTChunkData.Num())
			{
				VTChunkIndex++;
			}

			if (MipIndex >= InMipData.Num() && VTChunkIndex >= InVTChunkData.Num())
			{
				// No more source data.
				break;
			}
		}
	}

	if (ContinuousMemory.Num())
	{
		// If we ran out of source data before we completely filled, kick here.
		Compressed.SetNum(0,EAllowShrinking::No);
		FOodleCompressedArray::CompressData(
			Compressed,
			ContinuousMemory.GetData(),
			ContinuousMemory.Num(),
			InOodleCompressor,
			InOodleCompressionLevel);

		CompressedByteCount += Compressed.Num();
	}

	OutCompressedByteCount = CompressedByteCount;
}

//
// Grabs the texture data and then kicks off a task to block compress it
// in order to try and mimic how iostore does on disk compression.
//
// Returns the future result of the compression, with the compressed byte count
// in the first of the pair and the total in the second.
//
TFuture<TTuple<uint64, uint64>> FTexturePlatformData::LaunchEstimateOnDiskSizeTask(
	FOodleDataCompression::ECompressor InOodleCompressor,
	FOodleDataCompression::ECompressionLevel InOodleCompressionLevel,
	uint32 InCompressionBlockSize,
	FStringView InDebugContext
	)
{
	TArray<TArray64<uint8>> MipData;
	TArray<TArray64<uint8>> VTChunkData;
	if (FetchAllTextureDataSynchronous(this, InDebugContext, MipData, VTChunkData) == false)
	{
		return TFuture<TTuple<uint64, uint64>>();
	}
	
	struct FAsyncEstimateState
	{
		TPromise<TPair<uint64, uint64>> Promise;
		TArray<TArray64<uint8>> MipData;
		TArray<TArray64<uint8>> VTChunkData;
		FOodleDataCompression::ECompressor OodleCompressor;
		FOodleDataCompression::ECompressionLevel OodleCompressionLevel;
		uint32 CompressionBlockSize;
	};
	
	FAsyncEstimateState* State = new FAsyncEstimateState();
	State->MipData = MoveTemp(MipData);
	State->VTChunkData = MoveTemp(VTChunkData);
	State->OodleCompressor = InOodleCompressor;
	State->OodleCompressionLevel = InOodleCompressionLevel;
	State->CompressionBlockSize = InCompressionBlockSize;

	// Grab the future before we kick the task so there's no race.
	// (unlikely since compression is so long...)
	TFuture<TTuple<uint64, uint64>> ResultFuture = State->Promise.GetFuture();

	// Kick off a task with no dependencies that does the compression
	// and posts the result to the future.
	FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady([State]()
	{
		uint64 CompressedByteCount=0, UncompressedByteCount=0;

		EstimateOnDiskCompressionForTextureData(
			State->MipData, 
			State->VTChunkData,
			State->OodleCompressor,
			State->OodleCompressionLevel,
			State->CompressionBlockSize,
			UncompressedByteCount,
			CompressedByteCount);

		State->Promise.SetValue(TTuple<uint64, uint64>(CompressedByteCount, UncompressedByteCount));
		delete State;
	}, TStatId(), nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);

	return ResultFuture;
}

void FTexturePlatformData::TouchStreamingMipDDCData(int32 InMipCount, FStringView DebugContext)
{
	LoadDerivedStreamingMips(*this, 0, InMipCount, true, DebugContext, [](int32, FSharedBuffer){});
}

bool FTexturePlatformData::TryInlineMipData(int32 FirstMipToLoad, FStringView DebugContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTexturePlatformData::TryInlineMipData);

	const auto StoreMip = [this](int32 MipIndex, FSharedBuffer MipBuffer)
	{
		FTexture2DMipMap& Mip = Mips[MipIndex];
		Mip.BulkData.Lock(LOCK_READ_WRITE);
		void* MipData = Mip.BulkData.Realloc(int64(MipBuffer.GetSize()));
		FMemory::Memcpy(MipData, MipBuffer.GetData(), MipBuffer.GetSize());
		Mip.BulkData.Unlock();
	};

	if (!LoadDerivedStreamingMips(*this, FirstMipToLoad, Mips.Num(), false, DebugContext, StoreMip))
	{
		return false;
	}

	const auto StoreChunk = [this](int32 ChunkIndex, FSharedBuffer ChunkBuffer)
	{
		FVirtualTextureDataChunk& Chunk = VTData->Chunks[ChunkIndex];
		Chunk.BulkData.Lock(LOCK_READ_WRITE);
		void* ChunkData = Chunk.BulkData.Realloc(int64(ChunkBuffer.GetSize()));
		FMemory::Memcpy(ChunkData, ChunkBuffer.GetData(), ChunkBuffer.GetSize());
		Chunk.BulkData.Unlock();
	};

	if (VTData && !LoadDerivedStreamingVTChunks(VTData->Chunks, DebugContext, StoreChunk))
	{
		return false;
	}

	return true;
}

#endif // #if WITH_EDITOR

FTexturePlatformData::FTexturePlatformData()
	: SizeX(0)
	, SizeY(0)
	, PackedData(0)
	, PixelFormat(PF_Unknown)
	, VTData(nullptr)
#if WITH_EDITORONLY_DATA
	, AsyncTask(nullptr)
#endif // #if WITH_EDITORONLY_DATA
{
}

FTexturePlatformData::~FTexturePlatformData()
{
#if WITH_EDITOR
	if (AsyncTask)
	{
		if (!AsyncTask->Cancel())
		{
			AsyncTask->Wait();
		}
		delete AsyncTask;
		AsyncTask = nullptr;
	}
#endif
	if (VTData) 
	{
		delete VTData;
	}
	CPUCopy = nullptr;
}

bool FTexturePlatformData::IsReadyForAsyncPostLoad() const
{
#if WITH_EDITOR
	// Can't touch the Mips until async work is finished
	if (!IsAsyncWorkComplete())
	{
		return false;
	}
#endif

	return true;
}

bool FTexturePlatformData::TryLoadMips(int32 FirstMipToLoad, void** OutMipData, FStringView DebugContext)
{
	// TryLoadMips fills mip pointers but not sizes
	//  dangerous, not robust, use TryLoadMipsWithSizes instead

	return TryLoadMipsWithSizes(FirstMipToLoad, OutMipData, nullptr, DebugContext);
}

bool FTexturePlatformData::TryLoadMipsWithSizes(int32 FirstMipToLoad, void** OutMipData, int64* OutMipSize, FStringView DebugContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTexturePlatformData::TryLoadMips);

	int32 NumMipsCached = 0;
	const int32 LoadableMips = Mips.Num() - ((GetNumMipsInTail() > 0) ? (GetNumMipsInTail() - 1) : 0);
	check(LoadableMips >= 0);

#if WITH_EDITOR
	const auto StoreMip = [this, OutMipData, OutMipSize, FirstMipToLoad, &NumMipsCached](int32 MipIndex, FSharedBuffer MipBuffer)
	{
		FTexture2DMipMap& Mip = Mips[MipIndex];

		const int64 MipSize = static_cast<int64>(MipBuffer.GetSize());
		CheckMipSize(Mip, PixelFormat, MipSize);
		NumMipsCached++;

		if (OutMipData)
		{
			OutMipData[MipIndex - FirstMipToLoad] = FMemory::Malloc(MipSize);
			FMemory::Memcpy(OutMipData[MipIndex - FirstMipToLoad], MipBuffer.GetData(), MipSize);
		}
		if ( OutMipSize ) 
		{
			OutMipSize[MipIndex - FirstMipToLoad] = MipSize;
		}
	};

	if (!LoadDerivedStreamingMips(*this, FirstMipToLoad, Mips.Num(), false, DebugContext, StoreMip))
	{
		return false;
	}
#endif // #if WITH_EDITOR

	// Handle the case where we inlined more mips than we intend to keep resident
	// Discard unneeded mips
	for (int32 MipIndex = 0; MipIndex < FirstMipToLoad && MipIndex < LoadableMips; ++MipIndex)
	{
		FTexture2DMipMap& Mip = Mips[MipIndex];
		if (Mip.BulkData.IsBulkDataLoaded())
		{
			Mip.BulkData.Lock(LOCK_READ_ONLY);
			Mip.BulkData.Unlock();
		}
	}

	// Load remaining mips (if any) from bulk data.
	for (int32 MipIndex = FirstMipToLoad; MipIndex < LoadableMips; ++MipIndex)
	{
		FTexture2DMipMap& Mip = Mips[MipIndex];
		const int64 BulkDataSize = Mip.BulkData.GetBulkDataSize();
		if (BulkDataSize > 0)
		{
			if (OutMipData)
			{
#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
				// We want to make sure that any non-streamed mips are coming from the texture asset file, and not from an external bulk file.
				// But because "r.TextureStreaming" is driven by the project setting as well as the command line option "-NoTextureStreaming", 
				// is it possible for streaming mips to be loaded in non streaming ways.
				// Also check if editor data is available, in which case we are probably loading cooked data in the editor.
				if (!FPlatformProperties::HasEditorOnlyData() && CVarSetTextureStreaming.GetValueOnAnyThread() != 0)
				{
					UE_CLOGF(Mip.BulkData.IsInSeparateFile(), LogTexture, Error, "Loading non-streamed mips from an external bulk file.  This is not desireable.  File %ls",
						*(Mip.BulkData.GetDebugName()) );
				}
#endif
				Mip.BulkData.GetCopy(&OutMipData[MipIndex - FirstMipToLoad], true);
			}
			if (OutMipSize)
			{
				OutMipSize[MipIndex - FirstMipToLoad] = BulkDataSize;
			}
			NumMipsCached++;
		}
	}

	if (NumMipsCached != (LoadableMips - FirstMipToLoad))
	{
		UE_LOGF(LogTexture, Verbose, "TryLoadMips failed for %.*ls, NumMipsCached: %d, LoadableMips: %d, FirstMipToLoad: %d",
			DebugContext.Len(), DebugContext.GetData(),
			NumMipsCached,
			LoadableMips,
			FirstMipToLoad);

		// Unable to cache all mips. Release memory for those that were cached.
		for (int32 MipIndex = FirstMipToLoad; MipIndex < LoadableMips; ++MipIndex)
		{
			FTexture2DMipMap& Mip = Mips[MipIndex];
			UE_LOG(LogTexture, Verbose, TEXT("  Mip %d, BulkDataSize: %" INT64_FMT),
				MipIndex,
				Mip.BulkData.GetBulkDataSize());

			if (OutMipData && OutMipData[MipIndex - FirstMipToLoad])
			{
				FMemory::Free(OutMipData[MipIndex - FirstMipToLoad]);
				OutMipData[MipIndex - FirstMipToLoad] = NULL;
			}
		}
		return false;
	}

	return true;
}

int32 FTexturePlatformData::GetNumNonStreamingMips(bool bIsStreamingPossible) const
{
	if (CanUseCookedDataPath())
	{
		// We're on a cooked platform so we should only be streaming mips that were not inlined in the texture by thecooker.
		int32 NumNonStreamingMips = Mips.Num();

		for (const FTexture2DMipMap& Mip : Mips)
		{
			if (Mip.DerivedData || Mip.BulkData.IsInSeparateFile() || !Mip.BulkData.IsInlined())
			{
				--NumNonStreamingMips;
			}
			else
			{
				break;
			}
		}

		if (NumNonStreamingMips == 0 && Mips.Num())
		{
			return 1;
		}
		else
		{
			if ( ! bIsStreamingPossible )
			{
				check( NumNonStreamingMips == Mips.Num() );
			}

			return NumNonStreamingMips;
		}
	}
	else if (Mips.Num() <= 1 || !bIsStreamingPossible)
	{
		return Mips.Num();
	}
	else
	{
		// MipCount >= 2 and bIsStreamingPossible
		return GetNumNonStreamingMipsDirect(Mips.Num(), Mips[0].SizeX, Mips[0].SizeY, PixelFormat, GetNumMipsInTail(), UTexture2D::GetStaticMinTextureResidentMipCount());
	}
}

int32 FTexturePlatformData::GetNumNonOptionalMips() const
{
	// TODO : Count from last mip to first.
	if (CanUseCookedDataPath())
	{
		int32 NumNonOptionalMips = Mips.Num();

		for (const FTexture2DMipMap& Mip : Mips)
		{
			if ((Mip.DerivedData && !EnumHasAnyFlags(Mip.DerivedData.GetFlags(), UE::EDerivedDataFlags::Required)) || Mip.BulkData.IsOptional())
			{
				--NumNonOptionalMips;
			}
			else
			{
				break;
			}
		}

		if (NumNonOptionalMips == 0 && Mips.Num())
		{
			return 1;
		}
		else
		{
			return NumNonOptionalMips;
		}
	}
	else // Otherwise, all mips are available.
	{
		return Mips.Num();
	}
}

bool FTexturePlatformData::CanBeLoaded() const
{
	for (const FTexture2DMipMap& Mip : Mips)
	{
		if (Mip.DerivedData)
		{
			return true;
		}
		if (Mip.BulkData.CanLoadFromDisk())
		{
			return true;
		}
	}
	return false;
}


int32 FTexturePlatformData::GetNumVTMips() const
{
	check(VTData);
	return VTData->GetNumMips();
}

EPixelFormat FTexturePlatformData::GetLayerPixelFormat(uint32 LayerIndex) const
{
	if (VTData)
	{
		check(LayerIndex < VTData->NumLayers);
		return VTData->LayerTypes[LayerIndex];
	}
	check(LayerIndex == 0u);
	return PixelFormat;
}

int64 FTexturePlatformData::GetPayloadSize(int32 MipBias) const
{
	int64 PayloadSize = 0;
	if (VTData)
	{
		int32 NumTiles = 0;
		for (uint32 MipIndex = MipBias; MipIndex < VTData->NumMips; MipIndex++)
		{
			NumTiles += VTData->TileOffsetData[MipIndex].Width * VTData->TileOffsetData[MipIndex].Height;
		}
		int32 TileSizeWithBorder = (int32)(VTData->TileSize + 2 * VTData->TileBorderSize);
		int32 TileBlockSizeX = FMath::DivideAndRoundUp(TileSizeWithBorder, GPixelFormats[PixelFormat].BlockSizeX);
		int32 TileBlockSizeY = FMath::DivideAndRoundUp(TileSizeWithBorder, GPixelFormats[PixelFormat].BlockSizeY);
		PayloadSize += (int64)GPixelFormats[PixelFormat].BlockBytes * TileBlockSizeX * TileBlockSizeY * VTData->NumLayers * NumTiles;
	}
	else
	{
		for (int32 MipIndex = MipBias; MipIndex < Mips.Num(); MipIndex++)
		{
			int32 BlockSizeX = FMath::DivideAndRoundUp((int32)Mips[MipIndex].SizeX, GPixelFormats[PixelFormat].BlockSizeX);
			int32 BlockSizeY = FMath::DivideAndRoundUp((int32)Mips[MipIndex].SizeY, GPixelFormats[PixelFormat].BlockSizeY);
			int32 BlockSizeZ = FMath::DivideAndRoundUp(FMath::Max(GetNumSlices(), 1), GPixelFormats[PixelFormat].BlockSizeZ);
			PayloadSize += (int64)GPixelFormats[PixelFormat].BlockBytes * BlockSizeX * BlockSizeY * BlockSizeZ;
		}
	}
	return PayloadSize;
}

bool FTexturePlatformData::CanUseCookedDataPath() const
{
#if WITH_IOSTORE_IN_EDITOR
	return Mips.Num() > 0 && (Mips[0].BulkData.IsUsingIODispatcher() || Mips[0].DerivedData.IsCooked());
#else	
	return FPlatformProperties::RequiresCookedData();
#endif //WITH_IOSTORE_IN_EDITOR
}

#if WITH_EDITOR
bool FTexturePlatformData::AreDerivedMipsAvailable(FStringView Context) const
{
	if (DerivedDataKey.IsType<FString>())
	{
		using namespace UE;
		using namespace UE::DerivedData;
		TArray<FCacheGetValueRequest, TInlineAllocator<16>> MipRequests;

		int32 MipIndex = 0;
		const FSharedString SharedContext = Context;
		for (const FTexture2DMipMap& Mip : Mips)
		{
			if (Mip.IsPagedToDerivedData())
			{
				const FCacheKey MipKey = ConvertLegacyCacheKey(GetDerivedDataMipKeyString(MipIndex, Mip));
				const ECachePolicy ExistsPolicy = ECachePolicy::Query | ECachePolicy::SkipData;
				MipRequests.Add({SharedContext, MipKey, ExistsPolicy});
			}
			++MipIndex;
		}

		if (MipRequests.IsEmpty())
		{
			return true;
		}

		// When performing async loading, prefetch the lowest streaming mip into local caches
		// to avoid high priority request stalls from the render thread.
		if (!IsInGameThread())
		{
			MipRequests.Last().Policy |= ECachePolicy::StoreLocal;
		}

		bool bAreDerivedMipsAvailable = true;
		FRequestOwner BlockingOwner(EPriority::Blocking);
		GetCache().GetValue(MipRequests, BlockingOwner, [&bAreDerivedMipsAvailable](FCacheGetValueResponse&& Response)
		{
			bAreDerivedMipsAvailable &= Response.Status == EStatus::Ok;
		});
		BlockingOwner.Wait();
		return bAreDerivedMipsAvailable;
	}
	else if (DerivedDataKey.IsType<UE::DerivedData::FCacheKeyProxy>())
	{
		return true;
	}

	return false;
}

bool FTexturePlatformData::AreDerivedVTChunksAvailable(FStringView Context) const
{
	check(VTData);

	using namespace UE;
	using namespace UE::DerivedData;
	TArray<FCacheGetValueRequest, TInlineAllocator<16>> ChunkRequests;

	int32 ChunkIndex = 0;
	const FSharedString SharedContext = Context;
	for (const FVirtualTextureDataChunk& Chunk : VTData->Chunks)
	{
		if (!Chunk.DerivedDataKey.IsEmpty())
		{
			const FCacheKey ChunkKey = ConvertLegacyCacheKey(Chunk.DerivedDataKey);
			const ECachePolicy ExistsPolicy = ECachePolicy::Query | ECachePolicy::SkipData;
			ChunkRequests.Add({SharedContext, ChunkKey, ExistsPolicy});
		}
		++ChunkIndex;
	}

	if (ChunkRequests.IsEmpty())
	{
		return true;
	}

	// When performing async loading, prefetch the last chunk into local caches
	// to avoid high priority request stalls from the render thread.
	if (!IsInGameThread())
	{
		ChunkRequests.Last().Policy |= ECachePolicy::StoreLocal;
	}

	bool bAreDerivedChunksAvailable = true;
	FRequestOwner BlockingOwner(EPriority::Blocking);
	GetCache().GetValue(ChunkRequests, BlockingOwner, [&bAreDerivedChunksAvailable](FCacheGetValueResponse&& Response)
	{
		bAreDerivedChunksAvailable &= Response.Status == EStatus::Ok;
	});
	BlockingOwner.Wait();
	return bAreDerivedChunksAvailable;
}

bool FTexturePlatformData::AreDerivedMipsAvailable() const
{
	return AreDerivedMipsAvailable(TEXTVIEW("DerivedMips"));
}

bool FTexturePlatformData::AreDerivedVTChunksAvailable() const
{
	return AreDerivedVTChunksAvailable(TEXTVIEW("DerivedVTChunks"));
}

#endif // #if WITH_EDITOR

// Transient flags used to control behavior of platform data serialization
enum class EPlatformDataSerializationFlags : uint8
{
	None = 0,
	Cooked = 1<<0,
	Streamable = 1<<1,
};
ENUM_CLASS_FLAGS(EPlatformDataSerializationFlags);

static void SerializePlatformData(
	FArchive& Ar,
	FTexturePlatformData* PlatformData,
	UTexture* Texture,
	EPlatformDataSerializationFlags Flags,
	const bool bSerializeMipData)
{
	// note: if BuildTexture failed, we still get called here,
	//	just with a default-constructed PlatformData
	//	(no mips, sizes=0, PF=Unknown)

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("SerializePlatformData"), STAT_Texture_SerializePlatformData, STATGROUP_LoadTime);

	UEnum* PixelFormatEnum = UTexture::GetPixelFormatEnum();

	const bool bCooked = (Flags & EPlatformDataSerializationFlags::Cooked) == EPlatformDataSerializationFlags::Cooked;
	const bool bStreamable = (Flags & EPlatformDataSerializationFlags::Streamable) == EPlatformDataSerializationFlags::Streamable;

	bool bIsVirtual = Ar.IsSaving() && PlatformData->VTData;
	int32 NumMips = PlatformData->Mips.Num();
	int32 FirstMipToSerialize = 0;
	int32 FirstInlineMip = 0;
	// TODO: Do we need to consider platforms saving texture assets as cooked files?
	//       The info used to calculate optional mips is part of the editor only data.
	int32 OptionalMips = 0;
	bool bDuplicateNonOptionalMips = false;

	if (bCooked && bIsVirtual)
	{
		check(NumMips == 0);
	}

#if WITH_EDITORONLY_DATA
	if (bCooked && Ar.IsSaving())
	{
		check(Texture);
		check(Ar.CookingTarget());

		const int32 Width = PlatformData->SizeX;
		const int32 Height = PlatformData->SizeY;
		const int32 LODGroup = Texture->LODGroup;
		const int32 LODBias = Texture->LODBias;
		const TextureMipGenSettings MipGenSetting = Texture->MipGenSettings;
		const int32 LastMip = FMath::Max(NumMips - 1, 0);
		const int32 FirstMipTailMip = NumMips - (int32)PlatformData->GetNumMipsInTail();
		check(FirstMipTailMip >= 0);

		FirstMipToSerialize = Ar.CookingTarget()->GetTextureLODSettings().CalculateLODBias(Width, Height, Texture->GetMinTextureSize(), Texture->GetMaxTextureSize(), LODGroup, LODBias, 0, MipGenSetting, bIsVirtual, Texture->IsStreamable());
		if (!bIsVirtual)
		{
			// Reassign NumMips as the number of mips starting from FirstMipToSerialize.
			FirstMipToSerialize = FMath::Clamp(FirstMipToSerialize, 0, PlatformData->GetNumMipsInTail() > 0 ? FirstMipTailMip : LastMip);
			NumMips = FMath::Max(0, NumMips - FirstMipToSerialize);
		}
		else
		{
			FirstMipToSerialize = FMath::Clamp(FirstMipToSerialize, 0, FMath::Max((int32)PlatformData->VTData->GetNumMips() - 1, 0));
		}

		// We can't reliably strip the mip tail on non-pow2 textures after tiling on all platforms, so this gets ignored for nonpow2
		// at runtime. Warn here to catch it earlier.
		ETextureMipLoadOptions MipLoadOptions = Ar.CookingTarget()->GetTextureLODSettings().GetMipLoadOptions(Texture);
		if (MipLoadOptions == ETextureMipLoadOptions::OnlyFirstMip)
		{
			// This property only applies to physical 2d textures and there's no point in warning if there's only 1 mip.
			if (!bIsVirtual && Texture->GetTextureClass() == ETextureClass::TwoD && NumMips > 1)
			{
				if (!FMath::IsPowerOfTwo(PlatformData->Mips[FirstMipToSerialize].SizeX) ||
					!FMath::IsPowerOfTwo(PlatformData->Mips[FirstMipToSerialize].SizeY))
				{
					// If you are here because you're trying to LODBias a texture with no mips and using this as a workaround,
					// look at the Downscale setting.
					UE_LOGF(LogTexture, Warning, "MipLoadOption OnlyFirstMip can't be applied to non pow2 textures, see Downscale option if this is being used as a workaround for LODBias on NoMipMaps. Texture: %ls", *Texture->GetName());
				}
			}
		}
	}
#endif // #if WITH_EDITORONLY_DATA

	// Force resident mips inline
	if (bCooked && Ar.IsSaving() && !bIsVirtual)
	{
		// bStreamable comes from IsCandidateForTextureStreaming
		//  if not bStreamable, all mips are written inline
		//  so the runtime will see NumNonStreamingMips = all

	#if WITH_EDITORONLY_DATA
		check(Ar.CookingTarget());
		// This also needs to check whether the project enables texture streaming.
		// Currently, there is no reliable way to implement this because there is no difference
		// between the project settings (CVar) and the command line setting (from -NoTextureStreaming)
		if (bStreamable && Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::TextureStreaming))
	#else
		if (bStreamable)
	#endif
		{
			check(Texture && Texture->IsPossibleToStream());

			int32 NumNonStreamingMips = PlatformData->GetNumNonStreamingMips(/*bIsStreamingPossible*/ true);
			// NumMips has been reduced by FirstMipToSerialize (LODBias)
			NumNonStreamingMips = FMath::Min(NumNonStreamingMips, NumMips);
			// NumNonStreamingMips is not serialized. The runtime will use NumNonStreamingMips = NumInlineMips.
			FirstInlineMip = NumMips - NumNonStreamingMips;

		#if WITH_EDITORONLY_DATA
			static bool bDisableOptionalMips = FParse::Param(FCommandLine::Get(), TEXT("DisableOptionalMips"));
			if (!bDisableOptionalMips && NumMips > 0 )
			{
				const int32 LODGroup = Texture->LODGroup;
				const int32 FirstMipWidth  = PlatformData->Mips[FirstMipToSerialize].SizeX;
				const int32 FirstMipHeight = PlatformData->Mips[FirstMipToSerialize].SizeY;

				OptionalMips = Ar.CookingTarget()->GetTextureLODSettings().CalculateNumOptionalMips(LODGroup, FirstMipWidth, FirstMipHeight, NumMips, FirstInlineMip, Texture->MipGenSettings);
				bDuplicateNonOptionalMips = Ar.CookingTarget()->GetTextureLODSettings().TextureLODGroups[LODGroup].DuplicateNonOptionalMips;

				// OptionalMips must be streaming mips.
				check(OptionalMips <= FirstInlineMip);
			}

		#endif

		#if WITH_EDITOR
			// TODO [chris.tchou] : we should probably query the All Mip Provider to see what streaming state is.
			// Otherwise we might disable streaming calculations, even though the AMP expects to stream.
			// (this feeds into the bHasNoStreamableTextures optimization that skips streaming calculations)

			// Record the use of streaming mips on the owner.
			if (NumNonStreamingMips < NumMips)
			{
				// Use FindChecked because this was previously added and set to false.
				const FString PlatformName = Ar.CookingTarget()->PlatformName();
				Texture->DidSerializeStreamingMipsForPlatform.FindChecked(PlatformName) = true;
			}
		#endif
		}
	}

#if WITH_EDITORONLY_DATA
	// Save cook tags
	if (bCooked && Ar.IsSaving() && Ar.GetCookContext() && Ar.GetCookContext()->GetCookTagList())
	{
		FCookTagList* CookTags = Ar.GetCookContext()->GetCookTagList();

		if (bIsVirtual)
		{
			FVirtualTextureBuiltData* VTData = PlatformData->VTData;
			CookTags->Add(Texture, "Size", FString::Printf(TEXT("%dx%d"), FMath::Max(VTData->Width >> FirstMipToSerialize, 1U), FMath::Max(VTData->Height >> FirstMipToSerialize, 1U)));
		}
		else if (PlatformData->Mips.Num() > 0) // PlatformData->Mips is empty if BuildTexture failed
		{
			FString DimensionsStr;
			FTexture2DMipMap& TopMip = PlatformData->Mips[FirstMipToSerialize];
			if (TopMip.SizeZ != 1)
			{
				DimensionsStr = FString::Printf(TEXT("%dx%dx%d"), TopMip.SizeX, TopMip.SizeY, TopMip.SizeZ);
			}
			else
			{
				DimensionsStr = FString::Printf(TEXT("%dx%d"), TopMip.SizeX, TopMip.SizeY);
			}
			CookTags->Add(Texture, "Size", MoveTemp(DimensionsStr));
		}

		check(Texture);
		CookTags->Add(Texture, "Format", FString(GPixelFormats[PlatformData->PixelFormat].Name));

		// Add in diff keys for change detection/blame.
		{
			// Did the source change?
			CookTags->Add(Texture, "Diff_10_Tex2D_Source", Texture->Source.GetIdString());

			// Did the settings change?
			if (const UE::DerivedData::FCacheKeyProxy* CacheKey = PlatformData->DerivedDataKey.TryGet<UE::DerivedData::FCacheKeyProxy>())
			{
				CookTags->Add(Texture, "Diff_20_Tex2D_CacheKey", *WriteToString<64>(*CacheKey->AsCacheKey()));
			}
			else if (const FString* DDK = PlatformData->DerivedDataKey.TryGet<FString>())
			{
				CookTags->Add(Texture, "Diff_20_Tex2D_DDK", FString(*DDK));
			}

			// Did something in the image processing change?
			// We haven't yet forced a rebuild of textures, so this hash might not exist in the
			// platform data.
			if (PlatformData->PreEncodeMipsHash != 0)
			{
				FXxHash64 XxHash;
				XxHash.Hash = PlatformData->PreEncodeMipsHash;
				TStringBuilder<33> HashStr;
				HashStr << XxHash;
				CookTags->Add(Texture, "Diff_30_Tex2D_PreEncodeHash", *HashStr);
			}
		}
	}
#endif

	// DO NOT SERIALIZE ANYTHING BEFORE THIS POINT IN THE FUNCTION!

	// The DerivedData and BulkData serialization paths are expected to be distinct.
	// Since 5.0, cooked textures using the BulkData serialization path have a 16-byte zero block
	// that acts as a placeholder for the DerivedData serialization path to be optionally enabled
	// without requiring unaffected textures to be patched.

	bool bUsingDerivedData = false;
	if (bCooked)
	{
		bUsingDerivedData = !bIsVirtual && Ar.IsSaving() && Ar.IsFilterEditorOnly();
	#if WITH_EDITOR
		bUsingDerivedData &= CVarTexturesCookToDerivedDataReferences.GetValueOnAnyThread() != 0;
	#endif
		Ar.Serialize(&bUsingDerivedData, sizeof(bool));
		static_assert(sizeof(bool) == 1);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// DERIVED DATA REFERENCE FORMAT BEGINS HERE

	if (bUsingDerivedData)
	{
		int32 MipSizeX = NumMips ? PlatformData->Mips[FirstMipToSerialize].SizeX : 0;
		int32 MipSizeY = NumMips ? PlatformData->Mips[FirstMipToSerialize].SizeY : 0;
		int32 MipSizeZ = NumMips ? PlatformData->Mips[FirstMipToSerialize].SizeZ : 0;

		Ar << bIsVirtual;

		// Serialize SizeX, SizeY
		if (bIsVirtual)
		{
			checkNoEntry();
		}
		else
		{
			Ar << MipSizeX;
			Ar << MipSizeY;
			Ar << MipSizeZ;

			if (Ar.IsLoading())
			{
				PlatformData->SizeX = MipSizeX;
				PlatformData->SizeY = MipSizeY;
			}
		}

		// Serialize PackedData, OptData
		Ar << PlatformData->PackedData;

		// The OptData describes the cooked mips; if !bSerializeMipData then clear it,
		// as it's describing data we aren't serializing.
		if (bSerializeMipData)
		{
			if (PlatformData->GetHasOptData())
			{
				Ar << PlatformData->OptData;
			}
		}
		else if (Ar.IsLoading()) // !bSerializeMipData and loading - make sure to clear opt data
		{
			PlatformData->SetOptData(FOptTexturePlatformData{});
		}

		// Serialize PixelFormat
		if (Ar.IsSaving())
		{
			FString PixelFormatString = PixelFormatEnum->GetNameByValue(PlatformData->PixelFormat).GetPlainNameString();
			Ar << PixelFormatString;
		}
		else
		{
			FString PixelFormatString;
			Ar << PixelFormatString;
			const int64 PixelFormatValue = PixelFormatEnum->GetValueByName(*PixelFormatString);
			if (PixelFormatValue != INDEX_NONE && PixelFormatValue < PF_MAX)
			{
				PlatformData->PixelFormat = (EPixelFormat)PixelFormatValue;
			}
			else
			{
				UE_LOGF(LogTexture, Warning, "Invalid pixel format '%ls' for texture '%ls'.", *PixelFormatString, Texture ? *Texture->GetPathName() : TEXT(""));
				PlatformData->PixelFormat = PF_Unknown;
			}
		}

		// Serialize DerivedData
		if (bIsVirtual)
		{
			checkNoEntry();
		}
		else
		{
			Ar << NumMips;
			check(NumMips >= (int32)PlatformData->GetNumMipsInTail());

			Ar << FirstInlineMip;
			check(FirstInlineMip >= 0 && FirstInlineMip <= NumMips);

			if (Ar.IsLoading())
			{
				PlatformData->Mips.Empty(NumMips);
				for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
				{
					PlatformData->Mips.Add(new FTexture2DMipMap(0, 0));
					PlatformData->Mips.Last().BulkData.RemoveBulkData();
				}
			}

			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				FTexture2DMipMap& Mip = PlatformData->Mips[FirstMipToSerialize + MipIndex];
				if (Ar.IsSaving())
				{
					if (Mip.SizeZ > 1 || MipSizeZ > 1)
					{
						checkf(Mip.SizeX == MipSizeX && Mip.SizeY == MipSizeY && Mip.SizeZ == MipSizeZ,
							TEXT("Expected %dx%dx%d mip and had %dx%dx%d mip for '%s'"),
							Mip.SizeX, Mip.SizeY, Mip.SizeZ, MipSizeX, MipSizeY, MipSizeZ, *Texture->GetPathName());
					}
					else
					{
						checkf(Mip.SizeX == MipSizeX && Mip.SizeY == MipSizeY,
							TEXT("Expected %dx%d mip and had %dx%d mip for '%s'"),
							Mip.SizeX, Mip.SizeY, MipSizeX, MipSizeY, *Texture->GetPathName());
					}
				}
				else
				{
					Mip.SizeX = MipSizeX;
					Mip.SizeY = MipSizeY;
					Mip.SizeZ = MipSizeZ;
				}
				MipSizeX = FMath::Max(MipSizeX / 2, 1);
				MipSizeY = FMath::Max(MipSizeY / 2, 1);
			}

			for (int32 MipIndex = 0; MipIndex < FirstInlineMip; ++MipIndex)
			{
				FTexture2DMipMap& Mip = PlatformData->Mips[FirstMipToSerialize + MipIndex];
				Mip.DerivedData.Serialize(Ar, Texture);
				check(Mip.DerivedData);
			}

			// from the first inline mip onwards, we serialize to inline bulk data
			for (int32 MipIndex = FirstInlineMip; MipIndex < NumMips; ++MipIndex)
			{
				FTexture2DMipMap& Mip = PlatformData->Mips[FirstMipToSerialize + MipIndex];
				Mip.BulkData.SerializeWithFlags(Ar, Texture, BULKDATA_ForceInlinePayload | BULKDATA_SingleUse);
			}
		}

		return;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// BULK DATA FORMAT BEGINS HERE

	if (bCooked)
	{
		constexpr int64 PlaceholderDerivedDataSize = 15;
		uint8 PlaceholderDerivedData[PlaceholderDerivedDataSize]{};
		Ar.Serialize(PlaceholderDerivedData, PlaceholderDerivedDataSize);
		check(Algo::AllOf(PlaceholderDerivedData, [](uint8 Value) { return Value == 0; }));
	}

	Ar << PlatformData->SizeX;
	Ar << PlatformData->SizeY;
	Ar << PlatformData->PackedData;
	if (Ar.IsLoading())
	{
		FString PixelFormatString;
		Ar << PixelFormatString;
		const int64 PixelFormatValue = PixelFormatEnum->GetValueByName(*PixelFormatString);
		if (PixelFormatValue != INDEX_NONE && PixelFormatValue < PF_MAX)
		{
			PlatformData->PixelFormat = (EPixelFormat)PixelFormatValue;
		}
		else
		{
			UE_LOGF(LogTexture, Warning, "Invalid pixel format '%ls' for texture '%ls'.", *PixelFormatString, Texture ? *Texture->GetPathName() : TEXT(""));
			PlatformData->PixelFormat = PF_Unknown;
		}
	}
	else if (Ar.IsSaving())
	{
		FString PixelFormatString = PixelFormatEnum->GetNameByValue(PlatformData->PixelFormat).GetPlainNameString();
		Ar << PixelFormatString;
	}

	// The OptData describes the cooked mips; if !bSerializeMipData then clear it,
	// as it's describing data we aren't serializing.
	if (bSerializeMipData)
	{
		if (PlatformData->GetHasOptData())
		{
			Ar << PlatformData->OptData;
		}
	}
	else if (Ar.IsLoading()) // !bSerializeMipData and loading - make sure to clear opt data
	{
		PlatformData->SetOptData(FOptTexturePlatformData{});
	}

	if (PlatformData->GetHasCpuCopy())
	{
		if (Ar.IsLoading())
		{
			PlatformData->CPUCopy = FSharedImageConstRef(new FSharedImage());
		}

		// we have to cast off the const since we load in to it here as well as save.
		FSharedImage* ImageToSerialize = (FSharedImage*)PlatformData->CPUCopy.GetReference();
		Ar << ImageToSerialize->SizeX;
		Ar << ImageToSerialize->SizeY;
		Ar << ImageToSerialize->NumSlices;
		Ar << (uint8&)ImageToSerialize->Format;
		Ar << ImageToSerialize->GammaSpace;
		Ar << ImageToSerialize->RawData;
	}

	if (bCooked)
	{
		Ar << FirstMipToSerialize;
		if (Ar.IsLoading())
		{
			check(Texture);
			FirstMipToSerialize = 0;
		}
	}

	TArray<uint32> BulkDataMipFlags;

	// Update BulkDataFlags for cooked textures before saving.
	if (bCooked && Ar.IsSaving())
	{
		if (bIsVirtual)
		{
			const int32 NumChunks = PlatformData->VTData->Chunks.Num();
			BulkDataMipFlags.Reserve(NumChunks);
			for (FVirtualTextureDataChunk& Chunk : PlatformData->VTData->Chunks)
			{
				BulkDataMipFlags.Add(Chunk.BulkData.GetBulkDataFlags());
				Chunk.BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
			}
		}
		else
		{
			BulkDataMipFlags.Reserve(FirstMipToSerialize + NumMips);
			for (const FTexture2DMipMap& Mip : PlatformData->Mips)
			{
				BulkDataMipFlags.Add(Mip.BulkData.GetBulkDataFlags());
			}

			// Optional Mips (Streaming)
			// OptionalMips == 0 when we don't have WITH_EDITORONLY
			const uint32 OptionalBulkDataFlags = BULKDATA_Force_NOT_InlinePayload | BULKDATA_OptionalPayload;
			for (int32 MipIndex = 0; MipIndex < OptionalMips; ++MipIndex) //-V654 //-V621
			{
				PlatformData->Mips[MipIndex + FirstMipToSerialize].BulkData.SetBulkDataFlags(OptionalBulkDataFlags);
			}

			// Streamed Mips (Non-Optional)
			const uint32 StreamedBulkDataFlags = BULKDATA_Force_NOT_InlinePayload | (bDuplicateNonOptionalMips ? BULKDATA_DuplicateNonOptionalPayload : 0);
			for (int32 MipIndex = OptionalMips; MipIndex < FirstInlineMip; ++MipIndex)
			{
				PlatformData->Mips[MipIndex + FirstMipToSerialize].BulkData.SetBulkDataFlags(StreamedBulkDataFlags);
			}

			// Inline Mips (Non-Optional)
			const uint32 InlineBulkDataFlags = BULKDATA_ForceInlinePayload | BULKDATA_SingleUse;
			for (int32 MipIndex = FirstInlineMip; MipIndex < NumMips; ++MipIndex)
			{
				PlatformData->Mips[MipIndex + FirstMipToSerialize].BulkData.SetBulkDataFlags(InlineBulkDataFlags);
			}
		}
	}

	Ar << NumMips;
	check(NumMips >= (int32)PlatformData->GetNumMipsInTail());
	if (Ar.IsLoading())
	{
		check(FirstMipToSerialize == 0);
		PlatformData->Mips.Empty(NumMips);
		for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
		{
			PlatformData->Mips.Add(new FTexture2DMipMap(0, 0));
		}
	}

	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		PlatformData->Mips[FirstMipToSerialize + MipIndex].Serialize(Ar, Texture, MipIndex, bSerializeMipData);
	}

	Ar << bIsVirtual;
	if (bIsVirtual)
	{
		if (Ar.IsLoading() && !PlatformData->VTData)
		{
			PlatformData->VTData = new FVirtualTextureBuiltData();
		}

		check(PlatformData->VTData);
		PlatformData->VTData->Serialize(Ar, Texture, FirstMipToSerialize);

		for (int32 ChunkIndex = 0; ChunkIndex < BulkDataMipFlags.Num(); ++ChunkIndex)
		{
			check(Ar.IsSaving() && bCooked);
			PlatformData->VTData->Chunks[ChunkIndex].BulkData.ResetBulkDataFlags(BulkDataMipFlags[ChunkIndex]);
		}
	}
	else
	{
		for (int32 MipIndex = 0; MipIndex < BulkDataMipFlags.Num(); ++MipIndex)
		{
			check(Ar.IsSaving());
			PlatformData->Mips[MipIndex].BulkData.ResetBulkDataFlags(BulkDataMipFlags[MipIndex]);
		}
	}
}

void FTexturePlatformData::Serialize(FArchive& Ar, UTexture* Owner)
{
	check(!Ar.IsCooking()); // this is not the path that handles serialization for cooking
	SerializePlatformData(Ar, this, Owner, EPlatformDataSerializationFlags::None, /* bSerializeMipData = */ true);
}

#if WITH_EDITORONLY_DATA

FString FTexturePlatformData::GetDerivedDataMipKeyString(int32 MipIndex, const FTexture2DMipMap& Mip) const
{
	const FString& KeyString = DerivedDataKey.Get<FString>();
	return FString::Printf(TEXT("%s_MIP%u_%dx%d"), *KeyString, MipIndex, Mip.SizeX, Mip.SizeY);
}

UE::DerivedData::FValueId FTexturePlatformData::MakeMipId(int32 MipIndex)
{
	return UE::DerivedData::FValueId::FromName(WriteToString<16>(TEXTVIEW("Mip"), MipIndex));
}

#endif // WITH_EDITORONLY_DATA

void FTexturePlatformData::SerializeCooked(FArchive& Ar, UTexture* Owner, bool bStreamable, const bool bSerializeMipData)
{
	EPlatformDataSerializationFlags Flags = EPlatformDataSerializationFlags::Cooked;
	if (bStreamable)
	{
		Flags |= EPlatformDataSerializationFlags::Streamable;
	}
	SerializePlatformData(Ar, this, Owner, Flags, bSerializeMipData);
	if (Ar.IsLoading())
	{
		// Patch up Size as due to mips being stripped out during cooking it could be wrong.
		if (Mips.Num() > 0)
		{
			SizeX = Mips[0].SizeX;
			SizeY = Mips[0].SizeY;
			
			// SizeZ is not the same as NumSlices for texture arrays and cubemaps.
			if (Cast<UVolumeTexture>(Owner))
			{
				SetNumSlices(Mips[0].SizeZ);
			}
		}
		else if (VTData)
		{
			SizeX = VTData->Width;
			SizeY = VTData->Height;
		}
	}
}

/*------------------------------------------------------------------------------
	Texture derived data interface.
------------------------------------------------------------------------------*/

bool UTexture2DArray::GetMipData(int32 InFirstMipToLoad, TArray<FUniqueBuffer, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>>& OutMipData)
{
	FTexturePlatformData* LocalPlatformData = GetPlatformData();
	check( LocalPlatformData != nullptr );
	const int32 ReadableMipCount = LocalPlatformData->Mips.Num() - (LocalPlatformData->GetNumMipsInTail() > 0 ? LocalPlatformData->GetNumMipsInTail() - 1 : 0);

	int32 OutputMipCount = ReadableMipCount - InFirstMipToLoad;

	check(OutputMipCount <= MAX_TEXTURE_MIP_COUNT);

	void* MipData[MAX_TEXTURE_MIP_COUNT] = {};
	int64 MipSizes[MAX_TEXTURE_MIP_COUNT];
	if (LocalPlatformData->TryLoadMipsWithSizes(InFirstMipToLoad, MipData, MipSizes, GetPathName()) == false)
	{
		// Unable to load mips from the cache. Rebuild the texture and try again.
		UE_LOGF(LogTexture, Warning, "UTexture2DArray GetMipData failed for %ls (%ls)",
			*GetPathName(), GPixelFormats[GetPixelFormat()].Name);
#if WITH_EDITOR
		if (!GetOutermost()->bIsCookedForEditor)
		{
			ForceRebuildPlatformData();
			LocalPlatformData = GetPlatformData();
			if (LocalPlatformData->TryLoadMipsWithSizes(InFirstMipToLoad, MipData, MipSizes, GetPathName()) == false)
			{
				UE_LOGF(LogTexture, Error, "TryLoadMipsWithSizes still failed after ForceRebuildPlatformData %ls.", *GetPathName());
				return false;
			}
		}
#else // #if WITH_EDITOR
		return false;
#endif // WITH_EDITOR
	}

	for (int32 MipIndex = 0; MipIndex < OutputMipCount; MipIndex++)
	{
		OutMipData.Emplace(FUniqueBuffer::TakeOwnership(MipData[MipIndex], MipSizes[MipIndex], [](void* Ptr) {FMemory::Free(Ptr);} ));
	}
	return true;
}

void UTexture2D::GetMipData(int32 FirstMipToLoad, void** OutMipData)
{
	// hack convert the unsafe inputs to the "safe" form
	// here we are hoping that the caller has allocated this number of elements in OutMipData...  :fingers_crossed:
	int32 NumberOfMipsToLoad = GetPlatformData()->Mips.Num() - FirstMipToLoad;
	TArrayView<int64> MipSizeView; // empty array - we don't need the sizes returned
	GetInitialMipData(FirstMipToLoad, MakeArrayView(OutMipData, NumberOfMipsToLoad), MipSizeView);
}

bool UTexture2D::GetInitialMipData(int32 FirstMipToLoad, TArrayView<void*> OutMipData, TArrayView<int64> OutMipSize)
{
	bool bLoaded = false;
	if (UTextureAllMipDataProviderFactory* ProviderFactory = GetAllMipProvider())
	{
		bLoaded = ProviderFactory->GetInitialMipData(FirstMipToLoad, OutMipData, OutMipSize, GetPathName());
	}
	else
	{
		check( GetPlatformData() != nullptr );
		bLoaded = GetPlatformData()->TryLoadMipsWithSizes(FirstMipToLoad, OutMipData.GetData(), OutMipSize.GetData(), GetPathName());
	}

	if (!bLoaded)
	{
		// Unable to load mips from the cache. Rebuild the texture and try again.
		UE_LOGF(LogTexture,Warning,"UTexture2D::GetInitialMipData failed for %ls (%ls)",
			*GetPathName(), GPixelFormats[GetPixelFormat()].Name);
#if WITH_EDITOR
		if (!GetOutermost()->bIsCookedForEditor)
		{
			ForceRebuildPlatformData();
			if (GetPlatformData()->TryLoadMipsWithSizes(FirstMipToLoad, OutMipData.GetData(), OutMipSize.GetData(), GetPathName()) == false)
			{
				UE_LOGF(LogTexture, Error, "TryLoadMips still failed after ForceRebuildPlatformData %ls.", *GetPathName());
			}
		}
#endif // #if WITH_EDITOR
	}
	return bLoaded;
}

void UTextureCube::GetMipData(int32 FirstMipToLoad, void** OutMipData)
{
	check( GetPlatformData() != nullptr );

	if (GetPlatformData()->TryLoadMips(FirstMipToLoad, OutMipData, GetPathName()) == false)
	{
		// Unable to load mips from the cache. Rebuild the texture and try again.
		UE_LOGF(LogTexture,Warning,"UTextureCube::GetMipData failed for %ls (%ls)",
			*GetPathName(), GPixelFormats[GetPixelFormat()].Name);
#if WITH_EDITOR
		if (!GetOutermost()->bIsCookedForEditor)
		{
			ForceRebuildPlatformData();
			if (GetPlatformData()->TryLoadMips(FirstMipToLoad, OutMipData, GetPathName()) == false)
			{
				UE_LOGF(LogTexture, Error, "TryLoadMips still failed after ForceRebuildPlatformData %ls.", *GetPathName());
			}
		}
#endif // #if WITH_EDITOR
	}
}

#if WITH_EDITORONLY_DATA
bool UTexture::RequiresVirtualTexturing() const
{
	if ( ! Source.IsValid() )
	{
		return false;
	}

	if ( Source.GetNumLayers() > 1 )
	{
		return true;
	}
	
	// NOTE: optional: if NumBlocks() > 1 , for UDIM
	//	it does work as a non-VT and will just show the first block
	//	we can either say RequiresVirtualTexturing or not in that case
	/*
	if ( Source.GetNumBlocks() > 1 )
	{
		return true;
	}
	*/

	// also check class == ULightMapVirtualTexture2D ?

	return false;
}
#endif

int32 UTexture::CalculateLODBias(bool bWithCinematicMipBias) const
{
	// Async caching of PlatformData must be done before calling this
	//	if you call while async CachePlatformData is in progress, you get garbage out

	return UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->CalculateLODBias(this,bWithCinematicMipBias);
}

#if WITH_EDITOR

bool UTexture::CanBuildPlatformData(const ITargetPlatformSettings * TargetPlatform) const
{
	if ( ! Source.IsValid() )
	{
		return false;
	}

	if ( RequiresVirtualTexturing() )
	{
		if ( ! IsVirtualTexturingEnabled(TargetPlatform) )
		{
			return false;
		}
	}

	return true;
}

void UTexture::CachePlatformData(bool bAsyncCache, bool bAllowAsyncBuild, bool bAllowAsyncLoading, ITextureCompressorModule* Compressor, bool bForceRebuild)
{
	//
	// NOTE this can be called off the main thread via FAsyncEncode<> for shadow/light maps!
	// This is why the compressor is passed in, to avoid calling LoadModule off the main thread.
	//

	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture::CachePlatformData);

	FTexturePlatformData** PlatformDataLinkPtr = GetRunningPlatformData();
	if (PlatformDataLinkPtr)
	{
		FTexturePlatformData*& PlatformDataLink = *PlatformDataLinkPtr;
		if ( FApp::CanEverRender() && CanBuildPlatformData() )
		{
			bool bPerformCache = false;

			ETextureCacheFlags CacheFlags =
				(bAsyncCache ? ETextureCacheFlags::Async : ETextureCacheFlags::None) |
				(bAllowAsyncBuild? ETextureCacheFlags::AllowAsyncBuild : ETextureCacheFlags::None) |
				(bAllowAsyncLoading? ETextureCacheFlags::AllowAsyncLoading : ETextureCacheFlags::None) |
				(bForceRebuild ? ETextureCacheFlags::ForceRebuild : ETextureCacheFlags::None);

			ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();

			//
			// Step 1 of the caching process is to determine whether or not we need to actually
			// do a cache. To check this, we compare the keys for the FetchOrBuild settings since we
			// know we always have those. If we need the FetchFirst key, we generate it later when
			// we know we're actually going to Cache()
			//
			TArray<FTextureBuildSettings> BuildSettingsFetchOrBuild;			
			TArray<FTexturePlatformData::FTextureEncodeResultMetadata> ResultMetadataFetchOrBuild;

			// These might be empty.
			TArray<FTextureBuildSettings> BuildSettingsFetchFirst;
			TArray<FTexturePlatformData::FTextureEncodeResultMetadata> ResultMetadataFetchFirst;

			switch (EncodeSpeed)
			{
			case ETextureEncodeSpeed::FinalIfAvailable:
				{
					GetBuildSettingsForRunningPlatform(*this, ETextureEncodeSpeed::Final, BuildSettingsFetchFirst, &ResultMetadataFetchFirst);
					GetBuildSettingsForRunningPlatform(*this, ETextureEncodeSpeed::Fast, BuildSettingsFetchOrBuild, &ResultMetadataFetchOrBuild);
					break;
				}
			case ETextureEncodeSpeed::Fast:
				{
					GetBuildSettingsForRunningPlatform(*this, ETextureEncodeSpeed::Fast, BuildSettingsFetchOrBuild, &ResultMetadataFetchOrBuild);
					break;
				}
			case ETextureEncodeSpeed::Final:
				{
					GetBuildSettingsForRunningPlatform(*this, ETextureEncodeSpeed::Final, BuildSettingsFetchOrBuild, &ResultMetadataFetchOrBuild);
					break;
				}
			default:
				{
					UE_LOGF(LogTexture, Fatal, "Invalid encode speed in CachePlatformData");
				}
			}

			// If we're open in a texture editor, then we might have custom build settings.
			if (TextureEditorCustomEncoding.IsValid())
			{
				TSharedPtr<FTextureEditorCustomEncode> CustomEncoding = TextureEditorCustomEncoding.Pin();
				if (CustomEncoding.IsValid() && // (threading) could have been destroyed between weak ptr IsValid and Pin
					CustomEncoding->bUseCustomEncode)
				{
					// If we are overriding, we don't want to have a fetch first, so just set our encode
					// speed to whatever we already have staged, then set those settings to the custom
					// ones.
					EncodeSpeed = (ETextureEncodeSpeed)BuildSettingsFetchOrBuild[0].RepresentsEncodeSpeedNoSend;
					BuildSettingsFetchFirst.Empty();
					ResultMetadataFetchFirst.Empty();

					for (int32 i = 0; i < BuildSettingsFetchOrBuild.Num(); i++)
					{
						FTextureBuildSettings& BuildSettings = BuildSettingsFetchOrBuild[i];
						FTexturePlatformData::FTextureEncodeResultMetadata& ResultMetadata = ResultMetadataFetchOrBuild[i];

						BuildSettings.OodleRDO = CustomEncoding->OodleRDOLambda;
						BuildSettings.bOodleUsesRDO = CustomEncoding->OodleRDOLambda ? true : false;
						BuildSettings.OodleEncodeEffort = CustomEncoding->OodleEncodeEffort;
						BuildSettings.OodleUniversalTiling = CustomEncoding->OodleUniversalTiling;

						ResultMetadata.OodleRDO = CustomEncoding->OodleRDOLambda;
						ResultMetadata.OodleEncodeEffort = CustomEncoding->OodleEncodeEffort;
						ResultMetadata.OodleUniversalTiling = CustomEncoding->OodleUniversalTiling;
						ResultMetadata.EncodeSpeed = (uint8)EncodeSpeed;

						ResultMetadata.bWasEditorCustomEncoding = true;
					}
				}
			}

			check(BuildSettingsFetchOrBuild.Num() == Source.GetNumLayers());

			// The only time we don't cache is if we a) have existing data and b) it matches what we want.
			bPerformCache = true;
			if (PlatformDataLink != nullptr && !EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForceRebuild))
			{
				bPerformCache = false;

				// Check if our keys match. If we have two, they both have to match, otherwise a change that only affects one
				// might not cause a rebuild, leading to confusion in the texture editor.
				if (IsUsingNewDerivedData() && (Source.GetNumLayers() == 1) && !BuildSettingsFetchOrBuild[0].bVirtualStreamable)
				{
					// DDC2 version
					using namespace UE::DerivedData;
					if (const FTexturePlatformData::FStructuredDerivedDataKey* ExistingDerivedDataKey = PlatformDataLink->FetchOrBuildDerivedDataKey.TryGet<FTexturePlatformData::FStructuredDerivedDataKey>())
					{
						if (*ExistingDerivedDataKey != CreateTextureDerivedDataKey(*this, CacheFlags, BuildSettingsFetchOrBuild[0]))
						{
							bPerformCache = true;
						}						
					}

					if (BuildSettingsFetchFirst.Num())
					{
						if (const FTexturePlatformData::FStructuredDerivedDataKey* ExistingDerivedDataKey = PlatformDataLink->FetchFirstDerivedDataKey.TryGet<FTexturePlatformData::FStructuredDerivedDataKey>())
						{
							if (*ExistingDerivedDataKey != CreateTextureDerivedDataKey(*this, CacheFlags, BuildSettingsFetchFirst[0]))
							{
								bPerformCache = true;
							}
						}
					}
				} // end if ddc2
				else
				{
					// DDC1 version.
					if (const FString* ExistingDerivedDataKey = PlatformDataLink->FetchOrBuildDerivedDataKey.TryGet<FString>())
					{
						FString DerivedDataKey;
						GetTextureDerivedDataKey(*this, BuildSettingsFetchOrBuild.GetData(), DerivedDataKey);
						if (*ExistingDerivedDataKey != DerivedDataKey)
						{
							bPerformCache = true;
						}
					}

					if (BuildSettingsFetchFirst.Num())
					{
						if (const FString* ExistingDerivedDataKey = PlatformDataLink->FetchFirstDerivedDataKey.TryGet<FString>())
						{
							FString DerivedDataKey;
							GetTextureDerivedDataKey(*this, BuildSettingsFetchFirst.GetData(), DerivedDataKey);
							if (*ExistingDerivedDataKey != DerivedDataKey)
							{
								bPerformCache = true;
							}
						}
					}
				} // end if ddc1
			} // end if checking existing data matches.

			if (bPerformCache)
			{
				// Release our resource if there is existing derived data.
				if (PlatformDataLink)
				{
					ReleaseResource();

					// Need to wait for any previous InitRHI() to complete before modifying PlatformData
					// We could remove this flush if InitRHI() was modified to not access PlatformData directly
					FlushRenderingCommands();
				}
				else
				{
					PlatformDataLink = new FTexturePlatformData();
				}

				PlatformDataLink->Cache(
					*this, 
					BuildSettingsFetchFirst.Num() ? BuildSettingsFetchFirst.GetData() : nullptr, 
					BuildSettingsFetchOrBuild.GetData(), 
					ResultMetadataFetchFirst.Num() ? ResultMetadataFetchFirst.GetData() : nullptr,
					ResultMetadataFetchOrBuild.GetData(),
					uint32(CacheFlags), 
					Compressor);
			}
		}
		else if (PlatformDataLink == NULL)
		{
			// If there is no source art available, create an empty platform data container.
			PlatformDataLink = new FTexturePlatformData();
		}
	}
}

void UTexture::BeginCachePlatformData()
{
	CachePlatformData(true, true, true);
}

void UTexture::BeginCacheForCookedPlatformData( const ITargetPlatform *TargetPlatform )
{
	// @todo Oodle : if TargetPlatform->IsServerOnly() early exit?

	if ( ! CanBuildPlatformData(TargetPlatform) )
	{
		return;
	}

	TMap<FString, FTexturePlatformData*>* CookedPlatformDataPtr = GetCookedPlatformData();
	if (CookedPlatformDataPtr && !GetOutermost()->HasAnyPackageFlags(PKG_FilterEditorOnly))
	{
		TMap<FString,FTexturePlatformData*>& CookedPlatformData = *CookedPlatformDataPtr;
		
		// Make sure the pixel format enum has been cached.
		UTexture::GetPixelFormatEnum();

		// Retrieve formats to cache for target platform.
		bool HaveFetch = false;
		TArray< TArray<FTextureBuildSettings> > BuildSettingsToCacheFetch; // can be empty
		TArray< TArray<FTextureBuildSettings> > BuildSettingsToCacheFetchOrBuild;
		ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();
		if (EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable)
		{			
			FTextureBuildSettings BuildSettingsFinal, BuildSettingsFast;
			GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Final, BuildSettingsFinal, nullptr);
			GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettingsFast, nullptr);

			// Try and fetch Final, but build Fast.
			GetBuildSettingsPerFormat(*this, BuildSettingsFinal, nullptr, TargetPlatform, ETextureEncodeSpeed::Final, BuildSettingsToCacheFetch, nullptr);
			GetBuildSettingsPerFormat(*this, BuildSettingsFast, nullptr, TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettingsToCacheFetchOrBuild, nullptr);
			HaveFetch = true;
		}
		else
		{
			FTextureBuildSettings BuildSettings;
			GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, EncodeSpeed, BuildSettings, nullptr);
			GetBuildSettingsPerFormat(*this, BuildSettings, nullptr, TargetPlatform, EncodeSpeed, BuildSettingsToCacheFetchOrBuild, nullptr);
		}
		
		// Cull redundant settings by comparing derived data keys.
		// There's an assumption here where we believe that if
		// a Fetch key is unique, so is its associated FetchOrBuild key,
		// and visa versa. Since we know we have FetchOrBuild, but not
		// necessarily Fetch, we just do the uniqueness check on FetchOrBuild.
		TArray<FString> BuildSettingsCacheKeysFetchOrBuild;
		for (int32 i=0; i<BuildSettingsToCacheFetchOrBuild.Num(); i++)
		{
			TArray<FTextureBuildSettings>& LayerBuildSettingsFetchOrBuild = BuildSettingsToCacheFetchOrBuild[i];
			check(LayerBuildSettingsFetchOrBuild.Num() == Source.GetNumLayers());

			FString DerivedDataKeyFetchOrBuild;
			GetTextureDerivedDataKey(*this, LayerBuildSettingsFetchOrBuild.GetData(), DerivedDataKeyFetchOrBuild);

			if (BuildSettingsCacheKeysFetchOrBuild.Find(DerivedDataKeyFetchOrBuild) != INDEX_NONE)
			{
				BuildSettingsToCacheFetchOrBuild.RemoveAtSwap(i);
				if (HaveFetch)
				{
					BuildSettingsToCacheFetch.RemoveAtSwap(i);
				}
				i--;
				continue;
			}

			BuildSettingsCacheKeysFetchOrBuild.Add(MoveTemp(DerivedDataKeyFetchOrBuild));
		}

		// Now have a unique list - kick off the caches.
		for (int32 SettingsIndex = 0; SettingsIndex < BuildSettingsCacheKeysFetchOrBuild.Num(); ++SettingsIndex)
		{
			// If we have two platforms that generate the same key, we can have duplicates (e.g. -run=DerivedDataCache  -TargetPlatform=WindowsEditor+Windows) 
			if (CookedPlatformData.Find(BuildSettingsCacheKeysFetchOrBuild[SettingsIndex]))
			{
				continue;
			}

			FTexturePlatformData* PlatformDataToCache = new FTexturePlatformData();
			PlatformDataToCache->Cache(
				*this,
				HaveFetch ? BuildSettingsToCacheFetch[SettingsIndex].GetData() : nullptr,
				BuildSettingsToCacheFetchOrBuild[SettingsIndex].GetData(),
				nullptr,
				nullptr,
				uint32(ETextureCacheFlags::Async | ETextureCacheFlags::InlineMips | ETextureCacheFlags::AllowAsyncBuild | ETextureCacheFlags::AllowAsyncLoading),
				nullptr
				);

			CookedPlatformData.Add(BuildSettingsCacheKeysFetchOrBuild[SettingsIndex], PlatformDataToCache);
		}
	}
}

void UTexture::ClearCachedCookedPlatformData( const ITargetPlatform* TargetPlatform )
{
	TMap<FString, FTexturePlatformData*> *CookedPlatformDataPtr = GetCookedPlatformData();

	if ( CookedPlatformDataPtr )
	{
		TMap<FString,FTexturePlatformData*>& CookedPlatformData = *CookedPlatformDataPtr;

		// Make sure the pixel format enum has been cached.
		UTexture::GetPixelFormatEnum();

		// Get the list of keys associated with the target platform so we know
		// what to evict from the CookedPlatformData array.

		// The cooked platform data map is keyed off of the FetchOrBuild ddc key, so we don't
		// bother generating the Fetch one.
		// Retrieve formats to cache for target platform.			
		TArray< TArray<FTextureBuildSettings> > BuildSettingsForPlatform;
		ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();
		if (EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable ||
			EncodeSpeed == ETextureEncodeSpeed::Fast)
		{
			FTextureBuildSettings BuildSettings;
			GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettings, nullptr);
			GetBuildSettingsPerFormat(*this, BuildSettings, nullptr, TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettingsForPlatform, nullptr);
		}
		else
		{
			FTextureBuildSettings BuildSettings;
			GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Final, BuildSettings, nullptr);
			GetBuildSettingsPerFormat(*this, BuildSettings, nullptr, TargetPlatform, ETextureEncodeSpeed::Final, BuildSettingsForPlatform, nullptr);
		}
		
		// If the cooked platform data contains our data, evict it
		// This also is likely to only be handful of entries... try using an array and having
		// FTargetPlatformSet track what platforms the data is valid for. Once all are cleared, wipe...
		for (int32 SettingsIndex = 0; SettingsIndex < BuildSettingsForPlatform.Num(); ++SettingsIndex)
		{
			check(BuildSettingsForPlatform[SettingsIndex].Num() == Source.GetNumLayers());

			FString DerivedDataKey;
			GetTextureDerivedDataKey(*this, BuildSettingsForPlatform[SettingsIndex].GetData(), DerivedDataKey);

			if ( CookedPlatformData.Contains( DerivedDataKey ) )
			{
				FTexturePlatformData *PlatformData = CookedPlatformData.FindAndRemoveChecked( DerivedDataKey );
				delete PlatformData;
			}
		}
	}
}

void UTexture::ClearAllCachedCookedPlatformData()
{
	TMap<FString, FTexturePlatformData*> *CookedPlatformDataPtr = GetCookedPlatformData();

	if ( CookedPlatformDataPtr )
	{
		TMap<FString, FTexturePlatformData*> &CookedPlatformData = *CookedPlatformDataPtr;

		for ( auto It : CookedPlatformData )
		{
			delete It.Value;
		}

		CookedPlatformData.Empty();
	}
}

bool UTexture::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	// @todo Oodle : if TargetPlatform->IsServerOnly() early exit?

	const TMap<FString, FTexturePlatformData*>* CookedPlatformDataPtr = GetCookedPlatformData();
	if (!CookedPlatformDataPtr)
	{
		// when WITH_EDITOR is 0, the derived classes don't compile their GetCookedPlatformData()
		// so this returns the base class (nullptr). Since this function only exists when
		// WITH_EDITOR is 1, we can assume we have this data. This code should never get hit.
		return true; 
	}

	if ( ! CanBuildPlatformData(TargetPlatform) )
	{
		return true; // signify that the cook should move on without us.
	}

	// CookedPlatformData is keyed off of FetchOrBuild settings.
	ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();

	TArray<TArray<FTextureBuildSettings>> BuildSettingsAllFormats;	
	if (EncodeSpeed == ETextureEncodeSpeed::Fast ||
		EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable)
	{
		FTextureBuildSettings BuildSettings;
		GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettings, nullptr);
		GetBuildSettingsPerFormat(*this, BuildSettings, nullptr, TargetPlatform, ETextureEncodeSpeed::Fast, BuildSettingsAllFormats, nullptr);
	}
	else
	{
		FTextureBuildSettings BuildSettings;
		GetTextureBuildSettings(*this, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, ETextureEncodeSpeed::Final, BuildSettings, nullptr);
		GetBuildSettingsPerFormat(*this, BuildSettings, nullptr, TargetPlatform, ETextureEncodeSpeed::Final, BuildSettingsAllFormats, nullptr);
	}
	
	// on server-only platforms, BuildSettingsAllFormats is empty

	for (const TArray<FTextureBuildSettings>& FormatBuildSettings : BuildSettingsAllFormats)
	{
		check(FormatBuildSettings.Num() == Source.GetNumLayers());

		FString DerivedDataKey;
		GetTextureDerivedDataKey(*this, FormatBuildSettings.GetData(), DerivedDataKey);

		FTexturePlatformData* PlatformData = (*CookedPlatformDataPtr).FindRef(DerivedDataKey);

		// begin cache hasn't been called
		if (!PlatformData)
		{
			if (!HasAnyFlags(RF_ClassDefaultObject) && Source.SizeX != 0 && Source.SizeY != 0)
			{
				// In case an UpdateResource happens, cooked platform data might be cleared and we might need to reschedule
				BeginCacheForCookedPlatformData(TargetPlatform);
			}
			return false;
		}

		if (PlatformData->AsyncTask && PlatformData->AsyncTask->Poll())
		{
			PlatformData->FinishCache();
		}

		if (PlatformData->AsyncTask)
		{
			return false;
		}
	}
	// if we get here all our stuff is cached :)
	return true;
}

bool UTexture::IsAsyncCacheComplete() const
{
	if (const FTexturePlatformData* const* RunningPlatformDataPtr = const_cast<UTexture*>(this)->GetRunningPlatformData())
	{
		if (const FTexturePlatformData* PlatformData = *RunningPlatformDataPtr)
		{
			if (!PlatformData->IsAsyncWorkComplete())
			{
				return false;
			}
		}
	}

	if (const TMap<FString, FTexturePlatformData*>* CookedPlatformDataPtr = const_cast<UTexture*>(this)->GetCookedPlatformData())
	{
		for (const TTuple<FString, FTexturePlatformData*>& Kvp : *CookedPlatformDataPtr)
		{
			if (const FTexturePlatformData* PlatformData = Kvp.Value)
			{
				if (!PlatformData->IsAsyncWorkComplete())
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool UTexture::TryCancelCachePlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture::TryCancelCachePlatformData);

	FTexturePlatformData* const* RunningPlatformDataPtr = GetRunningPlatformData();
	if (RunningPlatformDataPtr)
	{
		FTexturePlatformData* RunningPlatformData = *RunningPlatformDataPtr;
		if (RunningPlatformData && !RunningPlatformData->TryCancelCache())
		{
			return false;
		}
	}

	TMap<FString, FTexturePlatformData*>* CookedPlatformDataPtr = GetCookedPlatformData();
	if (CookedPlatformDataPtr)
	{
		for (TTuple<FString, FTexturePlatformData*>& Kvp : *CookedPlatformDataPtr)
		{
			FTexturePlatformData* PlatformData = Kvp.Value;
			if (PlatformData && !PlatformData->TryCancelCache())
			{
				return false;
			}
		}
	}

	return true;
}

void UTexture::FinishCachePlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture::FinishCachePlatformData);

	FTexturePlatformData** RunningPlatformDataPtr = GetRunningPlatformData();
	if (RunningPlatformDataPtr)
	{
		FTexturePlatformData*& RunningPlatformData = *RunningPlatformDataPtr;
		
		if (CanBuildPlatformData() && FApp::CanEverRender())
		{
			if ( RunningPlatformData == NULL )
			{
				// begin cache never called
				//  do a non-async cache :
				CachePlatformData();
			}
			else
			{
				// make sure async requests are finished
				RunningPlatformData->FinishCache();
			}
		}
	}

	// FinishCachePlatformData is not reliably called
	//  this is not a good place to put code that finalizes caching
}

void UTexture::ForceRebuildPlatformData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexture::ForceRebuildPlatformData)

	FTexturePlatformData** PlatformDataLinkPtr = GetRunningPlatformData();
	if (PlatformDataLinkPtr && *PlatformDataLinkPtr && FApp::CanEverRender())
	{
		// Cache() will clear FTexturePlatformData::Mips which can be accessed by the streaming update
		WaitForPendingInitOrStreaming();

		// Make sure the flush actually releases our resource.
		ReleaseResource();

		FTexturePlatformData *&PlatformDataLink = *PlatformDataLinkPtr;
		FlushRenderingCommands();

		ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();

		// Since we are forcing a rebuild, build what is desired rather than what is available
		if (EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable)
		{
			EncodeSpeed = ETextureEncodeSpeed::Final;
		}

		TArray<FTextureBuildSettings> BuildSettingsFetchOrBuild;
		TArray<FTexturePlatformData::FTextureEncodeResultMetadata> ResultMetadataFetchOrBuild;
		GetBuildSettingsForRunningPlatform(*this, EncodeSpeed, BuildSettingsFetchOrBuild, &ResultMetadataFetchOrBuild);
		
		check(BuildSettingsFetchOrBuild.Num() == Source.GetNumLayers());

		PlatformDataLink->Cache(
			*this,
			nullptr,
			BuildSettingsFetchOrBuild.GetData(),
			nullptr,
			ResultMetadataFetchOrBuild.GetData(),
			uint32(ETextureCacheFlags::ForceRebuild),
			nullptr
			);

		// The build was synchronous but we still need to complete the compilation.
		BlockOnAnyAsyncBuild();
	}
}

void UTexture::MarkPlatformDataTransient()
{
}
#endif // #if WITH_EDITOR

void UTexture::GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const
{
	OutSettings.Init();
}

void UTexture::CleanupCachedRunningPlatformData()
{
	FTexturePlatformData **RunningPlatformDataPtr = GetRunningPlatformData();

	if ( RunningPlatformDataPtr )
	{
		FTexturePlatformData *&RunningPlatformData = *RunningPlatformDataPtr;

		if ( RunningPlatformData != NULL )
		{
			delete RunningPlatformData;
			RunningPlatformData = NULL;
		}
	}
}

void UTexture::SerializeCookedPlatformData(FArchive& Ar, const bool bSerializeMipData)
{
	if (IsTemplate() )
	{
		return;
	}

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT("UTexture::SerializeCookedPlatformData"), STAT_Texture_SerializeCookedData, STATGROUP_LoadTime );

	UEnum* PixelFormatEnum = UTexture::GetPixelFormatEnum();

#if WITH_EDITOR
	if (Ar.IsCooking() && Ar.IsPersistent())
	{
		if (Ar.CookingTarget()->AllowAudioVisualData())
		{
			TArray<FTexturePlatformData*> PlatformDataToSerialize;

			if (GetOutermost()->bIsCookedForEditor)
			{
				// For cooked packages, simply grab the current platform data and serialize it
				FTexturePlatformData** RunningPlatformDataPtr = GetRunningPlatformData();
				if (RunningPlatformDataPtr == nullptr)
				{
					return;
				}
				FTexturePlatformData* RunningPlatformData = *RunningPlatformDataPtr;
				if (RunningPlatformData == nullptr)
				{
					return;
				}
				PlatformDataToSerialize.Add(RunningPlatformData);
			}
			else if (CanBuildPlatformData())
			{
				TMap<FString, FTexturePlatformData*> *CookedPlatformDataPtr = GetCookedPlatformData();
				if (CookedPlatformDataPtr == nullptr)
				{
					return;
				}

				// Kick off builds for anything we don't have on hand already.
				ETextureEncodeSpeed EncodeSpeed = GetDesiredEncodeSpeed();

				TArray< TArray<FTextureBuildSettings> > BuildSettingsToCacheFetch;
				TArray< TArray<FTextureBuildSettings> > BuildSettingsToCacheFetchOrBuild;
				if (EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable)
				{
					FTextureBuildSettings BuildSettingsFetch;
					GetTextureBuildSettings(*this, Ar.CookingTarget()->GetTextureLODSettings(), *Ar.CookingTarget(), ETextureEncodeSpeed::Final, BuildSettingsFetch, nullptr);
					GetBuildSettingsPerFormat(*this, BuildSettingsFetch, nullptr, Ar.CookingTarget(), ETextureEncodeSpeed::Final, BuildSettingsToCacheFetch, nullptr);

					FTextureBuildSettings BuildSettingsFetchOrBuild;
					GetTextureBuildSettings(*this, Ar.CookingTarget()->GetTextureLODSettings(), *Ar.CookingTarget(), ETextureEncodeSpeed::Fast, BuildSettingsFetchOrBuild, nullptr);
					GetBuildSettingsPerFormat(*this, BuildSettingsFetchOrBuild, nullptr, Ar.CookingTarget(), ETextureEncodeSpeed::Fast, BuildSettingsToCacheFetchOrBuild, nullptr);
				}
				else
				{
					FTextureBuildSettings BuildSettingsFetchOrBuild;
					GetTextureBuildSettings(*this, Ar.CookingTarget()->GetTextureLODSettings(), *Ar.CookingTarget(), EncodeSpeed, BuildSettingsFetchOrBuild, nullptr);
					GetBuildSettingsPerFormat(*this, BuildSettingsFetchOrBuild, nullptr, Ar.CookingTarget(), EncodeSpeed, BuildSettingsToCacheFetchOrBuild, nullptr);
				}

				for (int32 SettingIndex = 0; SettingIndex < BuildSettingsToCacheFetchOrBuild.Num(); SettingIndex++)
				{
					check(BuildSettingsToCacheFetchOrBuild[SettingIndex].Num() == Source.GetNumLayers());

					// CookedPlatformData is keyed off of the fetchorbuild key.
					FString DerivedDataKeyFetchOrBuild;
					GetTextureDerivedDataKey(*this, BuildSettingsToCacheFetchOrBuild[SettingIndex].GetData(), DerivedDataKeyFetchOrBuild);

					FTexturePlatformData *PlatformDataPtr = (*CookedPlatformDataPtr).FindRef(DerivedDataKeyFetchOrBuild);
					if (PlatformDataPtr == NULL)
					{
						PlatformDataPtr = new FTexturePlatformData();
						PlatformDataPtr->Cache(*this, 
							BuildSettingsToCacheFetch.Num() ? BuildSettingsToCacheFetch[SettingIndex].GetData() : nullptr,
							BuildSettingsToCacheFetchOrBuild[SettingIndex].GetData(), 
							nullptr,
							nullptr,
							uint32(ETextureCacheFlags::InlineMips | ETextureCacheFlags::Async), 
							nullptr);

						CookedPlatformDataPtr->Add(DerivedDataKeyFetchOrBuild, PlatformDataPtr);
					}
					PlatformDataToSerialize.Add(PlatformDataPtr);
				}
			}

			// set DidSerializeStreamingMipsForPlatform to false, then it will change to true if any SerializeCooked makes streaming mips
			const FString PlatformName = Ar.CookingTarget()->PlatformName();
			DidSerializeStreamingMipsForPlatform.Add( PlatformName, false );

			// this iteration is over NumLayers :
			for (FTexturePlatformData* PlatformDataToSave : PlatformDataToSerialize)
			{
				// wait for async build task to complete, if there is one
				PlatformDataToSave->FinishCache();

				FName PixelFormatName = PixelFormatEnum->GetNameByValue(PlatformDataToSave->PixelFormat);
				Ar << PixelFormatName;

				// reserve space in the archive to record the skip offset
				const int64 SkipOffsetLoc = Ar.Tell();
				int64 SkipOffset = 0;
				{
					Ar << SkipOffset;
				}

				// Pass streamable flag for inlining mips
				bool bTextureIsStreamable = GetTextureIsStreamableOnPlatform(*this, *Ar.CookingTarget());

				// serialize the platform data
				PlatformDataToSave->SerializeCooked(Ar, this, bTextureIsStreamable, bSerializeMipData);

				// go back and patch the skip offset
				SkipOffset = Ar.Tell() - SkipOffsetLoc;
				Ar.Seek(SkipOffsetLoc);
				Ar << SkipOffset;
				Ar.Seek(SkipOffsetLoc + SkipOffset);
			}
		}
		FName PixelFormatName = NAME_None;
		Ar << PixelFormatName;
	}
	else
#endif // #if WITH_EDITOR
	{

		FTexturePlatformData** RunningPlatformDataPtr = GetRunningPlatformData();
		if (RunningPlatformDataPtr == nullptr)
		{
			return;
		}

		CleanupCachedRunningPlatformData();
		FTexturePlatformData*& RunningPlatformData = *RunningPlatformDataPtr;
		check(RunningPlatformData == nullptr);
		RunningPlatformData = new FTexturePlatformData();

		FName PixelFormatName = NAME_None;
		Ar << PixelFormatName;
		while (PixelFormatName != NAME_None)
		{
			const int64 PixelFormatValue = PixelFormatEnum->GetValueByName(PixelFormatName);
			const EPixelFormat PixelFormat = (PixelFormatValue != INDEX_NONE && PixelFormatValue < PF_MAX) ? (EPixelFormat)PixelFormatValue : PF_Unknown;

			const int64 SkipOffsetLoc = Ar.Tell();
			int64 SkipOffset = 0;
			Ar << SkipOffset;
			if (RunningPlatformData->PixelFormat == PF_Unknown && GPixelFormats[PixelFormat].Supported)
			{
				// Extra arg is unused here because we're loading
				const bool bStreamable = false;
				RunningPlatformData->SerializeCooked(Ar, this, bStreamable, bSerializeMipData);
			}
			else
			{
				Ar.Seek(SkipOffsetLoc + SkipOffset);
			}
			Ar << PixelFormatName;
		}
	}

	if (Ar.IsLoading())
	{
		LODBias = 0;
	}
}

#if WITH_EDITOR
namespace UE::TextureBuildUtilities
{

bool TryWriteCookDeterminismDiagnostics(FCbWriter& Writer, UTexture* Texture, const ITargetPlatform* TargetPlatform)
{
	if (!TargetPlatform->AllowAudioVisualData())
	{
		return false;
	}
	TMap<FString, FTexturePlatformData*>* CookedPlatformDataPtr = Texture->GetCookedPlatformData();
	if (!CookedPlatformDataPtr)
	{
		return false;
	}

	ETextureEncodeSpeed EncodeSpeed = Texture->GetDesiredEncodeSpeed();
	TArray<TArray<FTextureBuildSettings>> BuildSettingsToCacheFetchOrBuild;
	if (EncodeSpeed == ETextureEncodeSpeed::FinalIfAvailable)
	{
		EncodeSpeed = ETextureEncodeSpeed::Fast;
	}
	FTextureBuildSettings BuildSettingsFetchOrBuild;
	GetTextureBuildSettings(*Texture, TargetPlatform->GetTextureLODSettings(), *TargetPlatform, EncodeSpeed, BuildSettingsFetchOrBuild, nullptr);
	GetBuildSettingsPerFormat(*Texture, BuildSettingsFetchOrBuild, nullptr, TargetPlatform, EncodeSpeed, BuildSettingsToCacheFetchOrBuild, nullptr);

	if (BuildSettingsToCacheFetchOrBuild.IsEmpty())
	{
		return false;
	}

	Writer.BeginObject();
	Writer.BeginArray("BuildSettings");
	for (int32 SettingIndex = 0; SettingIndex < BuildSettingsToCacheFetchOrBuild.Num(); SettingIndex++)
	{
		// CookedPlatformData is keyed off of the fetchorbuild key.
		FString DerivedDataKeyFetchOrBuild;
		GetTextureDerivedDataKey(*Texture, BuildSettingsToCacheFetchOrBuild[SettingIndex].GetData(), DerivedDataKeyFetchOrBuild);
		Writer.BeginObject();
		Writer << "DerivedDataKey" << DerivedDataKeyFetchOrBuild;
		Writer.EndObject();
	}
	Writer.EndArray();
	Writer.EndObject();
	return true;
}

}
#endif

int32 UTexture::GMinTextureResidentMipCount = NUM_INLINE_DERIVED_MIPS;

void UTexture::SetMinTextureResidentMipCount(int32 InMinTextureResidentMipCount)
{
	int32 MinAllowedMipCount = FPlatformProperties::RequiresCookedData() ? 1 : NUM_INLINE_DERIVED_MIPS;
	GMinTextureResidentMipCount = FMath::Max(InMinTextureResidentMipCount, MinAllowedMipCount);
}

#if WITH_EDITOR
// return value false for critical errors
// may return true even if nothing was done; check OutMadeChanges
// InOutImage is modified in place ; output image will be same format but changed dimensions
bool UTexture::DownsizeImageUsingTextureSettings(const ITargetPlatform* TargetPlatform, FImage& InOutImage, int32 TargetSize, int32 LayerIndex, bool & OutMadeChanges) const
{
	// resize so that the largest dimension is <= TargetSize
	OutMadeChanges = false;

	if (TargetSize <= 1 || LayerIndex < 0 || InOutImage.IsImageInfoValid() == false)
	{
		UE_LOGF(LogTexture, Error, "Invalid parameter supplied to DownsizeImageUsingTextureSettings target size = %d layer index = %d image valid: %ls",
			TargetSize, LayerIndex, InOutImage.IsImageInfoValid() ? TEXT("true") : TEXT("false"));
		return false;
	}

	if (TargetSize >= InOutImage.SizeX && TargetSize >= InOutImage.SizeY)
	{
		// both dimensions already small enough, early out
		// InOutImage is not changed
		return true;
	}

	// Ideally this code wouldn't live here but at the moment of writing this code the coupling between the texture and the texture compressor make it hard to move that logic elsewhere
	TArray<FTextureBuildSettings> SettingPerLayer;
	GetBuildSettingsForTargetPlatform(*this, TargetPlatform, ETextureEncodeSpeed::Final, SettingPerLayer, nullptr);

	if (LayerIndex >= SettingPerLayer.Num())
	{
		UE_LOGF(LogTexture, Error, "Invalid layer supplied to DownsizeImageUsingTextureSettings, layer index = %d", LayerIndex);
		return false;
	}

	// Teak the build setting to generate a mip for our image
	FTextureBuildSettings& BuildSettings = SettingPerLayer[LayerIndex];
	// even if we are a Cube or LatLong, tell it we are just 2d ?
	//  so the image is shrunk as a plain 2d
	BuildSettings.bCubemap = false;
	BuildSettings.bTextureArray = false;
	BuildSettings.bVolume = false;
	BuildSettings.bLongLatSource = false;
	
	// make sure modern options are set:
	BuildSettings.bUseNewMipFilter = true;
	BuildSettings.bSharpenWithoutColorShift = false;
	if ( IsNormalMap() )
	{
		BuildSettings.bNormalizeNormals = true;
	}

	if ( BuildSettings.MipGenSettings == TMGS_NoMipmaps ||
		BuildSettings.MipGenSettings == TMGS_LeaveExistingMips ||
		BuildSettings.MipGenSettings == TMGS_Angular )
	{
		// what kind of mipgen do we use here? (default from GetMipGenSettings will use 2x2 simple average)
		// external caller now prefers to use use ResizeImage in this case
		BuildSettings.MipGenSettings = TMGS_SimpleAverage;
	}

	// we turned off bCubeMap, make sure cube face filters clamp, not wrap
	//  see ComputeAddressMode
	if ( GetTextureClass() == ETextureClass::Cube || GetTextureClass() == ETextureClass::CubeArray )
	{
		// for 6-face cubes, just clamp
		// for LatLong we want to Clamp Y but Wrap X ; that's not supported so just clamp
		// external caller now prefers to use use ResizeImage for latlongs

		BuildSettings.TextureAddressModeX = TA_Clamp;
		BuildSettings.TextureAddressModeY = TA_Clamp;
	}

	FImage Temp;
	// convert to RGBA32F linear for the compressor
	InOutImage.CopyTo(Temp, ERawImageFormat::RGBA32F, EGammaSpace::Linear);

	if ( InOutImage.GetGammaSpace() == EGammaSpace::Pow22 )
	{
		// Pow22 is read only; now that we have converted to Linear, we will write output as sRGB
		// caller must also change Texture->bUseLegacyGamma
		InOutImage.GammaSpace = EGammaSpace::sRGB;
	}

	TArray<FImage> BuildSourceImageMips;
	// make sure BuildSourceImageMips doesn't reallocate :
	constexpr int BuildSourceImageMipsMaxCount = 20; // plenty
	BuildSourceImageMips.Empty(BuildSourceImageMipsMaxCount);

	ITextureCompressorModule::GenerateMipChain(BuildSettings, Temp, BuildSourceImageMips, 1);

	// keep halving while larger size is > Target
	while (BuildSourceImageMips.Last().SizeX > TargetSize ||
		BuildSourceImageMips.Last().SizeY > TargetSize)
	{
		ITextureCompressorModule::GenerateMipChain(BuildSettings, BuildSourceImageMips.Last(), BuildSourceImageMips, 1);
	}

	// now larger size must be <= TargetSize

	FImage* SelectedOutput = &BuildSourceImageMips.Last();

	if (SelectedOutput->Format == InOutImage.Format && SelectedOutput->GammaSpace == InOutImage.GammaSpace)
	{
		InOutImage = MoveTemp(*SelectedOutput);
	}
	else
	{
		SelectedOutput->CopyTo(InOutImage, InOutImage.Format, InOutImage.GammaSpace);
	}

	OutMadeChanges = true;

	return true;
}


void UTexture::GetTargetPlatformBuildSettings(const ITargetPlatform* TargetPlatform, TArray<TArray<FTextureBuildSettings>>& OutSettingPerSupportedFormatPerLayer) const
{
	// OutSettingPerSupportedFormatPerLayer is an array for platforms that have multiple different texture format modes (eg. Android_Multi)

	// Beware GetTargetPlatformBuildSettings and GetBuildSettingsForTargetPlatform are almost code dupes of each other; keep in sync!
	//	todo: refactor to dedupe
	//GetBuildSettingsForTargetPlatform(*this,TargetPlatform,ETextureEncodeSpeed::Final,...);

	ETextureEncodeSpeed EncodeSpeed = ETextureEncodeSpeed::Final;

	if (!TargetPlatform)
	{
		OutSettingPerSupportedFormatPerLayer.Empty();
		return;
	}

	const UTextureLODSettings* LODSettings = UDeviceProfileManager::Get().FindProfile(TargetPlatform->CookingDeviceProfileName())->GetTextureLODSettings();
	FTextureBuildSettings SourceBuildSettings;
	FTexturePlatformData::FTextureEncodeResultMetadata SourceMetadata;
	GetTextureBuildSettings(*this, *LODSettings, *TargetPlatform, EncodeSpeed, SourceBuildSettings, &SourceMetadata);

	TArray< TArray<FName> > PlatformFormats;
	GetPlatformTextureFormatNamesWithPrefix(TargetPlatform, PlatformFormats);

	int32 NumFormats = PlatformFormats.Num();
	OutSettingPerSupportedFormatPerLayer.SetNum(NumFormats);
	for ( int32 FormatIndex = 0; FormatIndex < NumFormats; ++FormatIndex)
	{
		const int32 NumLayers = Source.GetNumLayers();
		check(PlatformFormats[FormatIndex].Num() == NumLayers);

		OutSettingPerSupportedFormatPerLayer[FormatIndex].Reserve(NumLayers);
		for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			FTextureBuildSettings& OutSettings = OutSettingPerSupportedFormatPerLayer[FormatIndex].Add_GetRef(SourceBuildSettings);
			OutSettings.TextureFormatName = PlatformFormats[FormatIndex][LayerIndex];

			FTexturePlatformData::FTextureEncodeResultMetadata* OutMetadata = nullptr;
			FinalizeBuildSettingsForLayer(*this, LayerIndex, TargetPlatform, EncodeSpeed, OutSettings, OutMetadata);
		}
	}
}


#endif
