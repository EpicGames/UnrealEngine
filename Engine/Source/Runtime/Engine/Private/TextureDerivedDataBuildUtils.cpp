// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureDerivedDataBuildUtils.h"

#if WITH_EDITOR
#include "ColorManagement/ColorSpace.h"
#include "Containers/SharedString.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildFunctionRegistry.h"
#include "Engine/Texture.h"
#include "Interfaces/ITextureFormat.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/CompactBinaryWriter.h"
#include "TextureCompressorModule.h"
#include "TextureFormatManager.h"

const FGuid& GetTextureDerivedDataVersion();
const FGuid& GetTextureSLEDerivedDataVersion();
void GetTextureDerivedMipKey(int32 MipIndex, const FTexture2DMipMap& Mip, const FString& KeySuffix, FString& OutKey);

template <typename ValueType>
static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const ValueType& Value)
{
	Writer << Name << Value;
}

static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const FName & Value)
{
	Writer << Name << WriteToString<128>(Value);
}

static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const FColor& Value)
{
	Writer.BeginArray(Name);
	Writer.AddInteger(Value.A);
	Writer.AddInteger(Value.R);
	Writer.AddInteger(Value.G);
	Writer.AddInteger(Value.B);
	Writer.EndArray();
}

static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const FVector2f& Value)
{
	Writer.BeginArray(Name);
	Writer.AddFloat(Value.X);
	Writer.AddFloat(Value.Y);
	Writer.EndArray();
}

static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const FVector4f& Value)
{
	Writer.BeginArray(Name);
	Writer.AddFloat(Value.X);
	Writer.AddFloat(Value.Y);
	Writer.AddFloat(Value.Z);
	Writer.AddFloat(Value.W);
	Writer.EndArray();
}

static void WriteCbField(FCbWriter& Writer, FAnsiStringView Name, const FIntPoint& Value)
{
	Writer.BeginArray(Name);
	Writer.AddInteger(Value.X);
	Writer.AddInteger(Value.Y);
	Writer.EndArray();
}

template <typename ValueType>
static void WriteCbFieldWithDefault(FCbWriter& Writer, FAnsiStringView Name, ValueType Value, ValueType Default)
{
	if (Value != Default)
	{
		WriteCbField(Writer, Name, Forward<ValueType>(Value));
	}
}

static void WriteBuildSettings(FCbWriter& Writer, const FTextureBuildSettings& BuildSettings, const ITextureFormat* TextureFormat)
{
	FTextureBuildSettings DefaultSettings;

	Writer.BeginObject();

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	WriteCbField<bool>(Writer, "bBuildIsArm64", true);
#endif

	if (BuildSettings.FormatConfigOverride)
	{
		Writer.AddObject("FormatConfigOverride", BuildSettings.FormatConfigOverride);
	}
	else if (FCbObject TextureFormatConfig = TextureFormat->ExportGlobalFormatConfig(BuildSettings))
	{
		Writer.AddObject("FormatConfigOverride", TextureFormatConfig);
	}

	if (BuildSettings.ColorAdjustment.AdjustBrightness != DefaultSettings.ColorAdjustment.AdjustBrightness ||
		BuildSettings.ColorAdjustment.AdjustBrightnessCurve != DefaultSettings.ColorAdjustment.AdjustBrightnessCurve ||
		BuildSettings.ColorAdjustment.AdjustSaturation != DefaultSettings.ColorAdjustment.AdjustSaturation ||
		BuildSettings.ColorAdjustment.AdjustVibrance != DefaultSettings.ColorAdjustment.AdjustVibrance ||
		BuildSettings.ColorAdjustment.AdjustRGBCurve != DefaultSettings.ColorAdjustment.AdjustRGBCurve ||
		BuildSettings.ColorAdjustment.AdjustHue != DefaultSettings.ColorAdjustment.AdjustHue ||
		BuildSettings.ColorAdjustment.AdjustMinAlpha != DefaultSettings.ColorAdjustment.AdjustMinAlpha ||
		BuildSettings.ColorAdjustment.AdjustMaxAlpha != DefaultSettings.ColorAdjustment.AdjustMaxAlpha)
	{
		Writer.BeginObject("ColorAdjustment");
		WriteCbFieldWithDefault(Writer, "AdjustBrightness", BuildSettings.ColorAdjustment.AdjustBrightness, DefaultSettings.ColorAdjustment.AdjustBrightness);
		WriteCbFieldWithDefault(Writer, "AdjustBrightnessCurve", BuildSettings.ColorAdjustment.AdjustBrightnessCurve, DefaultSettings.ColorAdjustment.AdjustBrightnessCurve);
		WriteCbFieldWithDefault(Writer, "AdjustSaturation", BuildSettings.ColorAdjustment.AdjustSaturation, DefaultSettings.ColorAdjustment.AdjustSaturation);
		WriteCbFieldWithDefault(Writer, "AdjustVibrance", BuildSettings.ColorAdjustment.AdjustVibrance, DefaultSettings.ColorAdjustment.AdjustVibrance);
		WriteCbFieldWithDefault(Writer, "AdjustRGBCurve", BuildSettings.ColorAdjustment.AdjustRGBCurve, DefaultSettings.ColorAdjustment.AdjustRGBCurve);
		WriteCbFieldWithDefault(Writer, "AdjustHue", BuildSettings.ColorAdjustment.AdjustHue, DefaultSettings.ColorAdjustment.AdjustHue);
		WriteCbFieldWithDefault(Writer, "AdjustMinAlpha", BuildSettings.ColorAdjustment.AdjustMinAlpha, DefaultSettings.ColorAdjustment.AdjustMinAlpha);
		WriteCbFieldWithDefault(Writer, "AdjustMaxAlpha", BuildSettings.ColorAdjustment.AdjustMaxAlpha, DefaultSettings.ColorAdjustment.AdjustMaxAlpha);
		Writer.EndObject();
	}
	
	WriteCbFieldWithDefault<bool>(Writer, "bDoScaleMipsForAlphaCoverage", BuildSettings.bDoScaleMipsForAlphaCoverage, DefaultSettings.bDoScaleMipsForAlphaCoverage);
	if ( BuildSettings.bDoScaleMipsForAlphaCoverage )
	{
		// AlphaCoverageThresholds do not affect build if bDoScaleMipsForAlphaCoverage is off
		WriteCbFieldWithDefault(Writer, "AlphaCoverageThresholds", BuildSettings.AlphaCoverageThresholds, DefaultSettings.AlphaCoverageThresholds);
	}
	WriteCbFieldWithDefault<bool>(Writer, "bUseNewMipFilter", BuildSettings.bUseNewMipFilter, DefaultSettings.bUseNewMipFilter);
	WriteCbFieldWithDefault<bool>(Writer, "bNormalizeNormals", BuildSettings.bNormalizeNormals, DefaultSettings.bNormalizeNormals);
	WriteCbFieldWithDefault(Writer, "MipSharpening", BuildSettings.MipSharpening, DefaultSettings.MipSharpening);
	WriteCbFieldWithDefault(Writer, "DiffuseConvolveMipLevel", BuildSettings.DiffuseConvolveMipLevel, DefaultSettings.DiffuseConvolveMipLevel);
	WriteCbFieldWithDefault(Writer, "SharpenMipKernelSize", BuildSettings.SharpenMipKernelSize, DefaultSettings.SharpenMipKernelSize);
	WriteCbFieldWithDefault(Writer, "MaxTextureResolution", BuildSettings.MaxTextureResolution, DefaultSettings.MaxTextureResolution);
	WriteCbFieldWithDefault(Writer, "TextureFormatName", WriteToString<64>(BuildSettings.TextureFormatName).ToView(), TEXTVIEW(""));
	WriteCbFieldWithDefault(Writer, "bHDRSource", BuildSettings.bHDRSource, DefaultSettings.bHDRSource);
	WriteCbFieldWithDefault(Writer, "MipGenSettings", BuildSettings.MipGenSettings, DefaultSettings.MipGenSettings);
	WriteCbFieldWithDefault<bool>(Writer, "bCubemap", BuildSettings.bCubemap, DefaultSettings.bCubemap);
	WriteCbFieldWithDefault<bool>(Writer, "bTextureArray", BuildSettings.bTextureArray, DefaultSettings.bTextureArray);
	WriteCbFieldWithDefault<bool>(Writer, "bVolume", BuildSettings.bVolume, DefaultSettings.bVolume);
	WriteCbFieldWithDefault<bool>(Writer, "bLongLatSource", BuildSettings.bLongLatSource, DefaultSettings.bLongLatSource);
	WriteCbFieldWithDefault<bool>(Writer, "bSRGB", BuildSettings.bSRGB, DefaultSettings.bSRGB);

	if (BuildSettings.SourceEncodingOverride != 0 /*UE::Color::EEncoding::None*/)
	{
		WriteCbField<uint32>(Writer, "EncodingOverrideVersion", UE::Color::ENCODING_TYPES_VER);
		WriteCbFieldWithDefault(Writer, "SourceEncodingOverride", BuildSettings.SourceEncodingOverride, DefaultSettings.SourceEncodingOverride);
	}
	else // @todo SerializeForKey : remove else-case when overall key changes
	{
		WriteCbFieldWithDefault(Writer, "SourceEncodingOverride", BuildSettings.SourceEncodingOverride, DefaultSettings.SourceEncodingOverride);
	}

	WriteCbFieldWithDefault<bool>(Writer, "bHasColorSpaceDefinition", BuildSettings.bHasColorSpaceDefinition, DefaultSettings.bHasColorSpaceDefinition);
	if (BuildSettings.bHasColorSpaceDefinition)
	{
		WriteCbField<uint32>(Writer, "ColorSpaceVersion", UE::Color::COLORSPACE_VER);
		/*
		* The texture color transform depends on the chromaticities of both the source color space
		* and the destination (working) color space, per its project setting. We therefore include
		* the working color space chromaticities to incur a texture rebuild if/when changed.
		*/
		const TStaticArray<FVector2d, 4> DefaultChromaticities = UE::Color::FColorSpace::MakeChromaticities(UE::Color::EColorSpace::sRGB);
		const UE::Color::FColorSpace& WorkingColorSpace = UE::Color::FColorSpace::GetWorking();
		WriteCbFieldWithDefault(Writer, "WorkingRedChromaticity", FVector2f(WorkingColorSpace.GetRedChromaticity()), FVector2f(DefaultChromaticities[0]));
		WriteCbFieldWithDefault(Writer, "WorkingGreenChromaticity", FVector2f(WorkingColorSpace.GetGreenChromaticity()), FVector2f(DefaultChromaticities[1]));
		WriteCbFieldWithDefault(Writer, "WorkingBlueChromaticity", FVector2f(WorkingColorSpace.GetBlueChromaticity()), FVector2f(DefaultChromaticities[2]));
		WriteCbFieldWithDefault(Writer, "WorkingWhiteChromaticity", FVector2f(WorkingColorSpace.GetWhiteChromaticity()), FVector2f(DefaultChromaticities[3]));

		WriteCbFieldWithDefault(Writer, "RedChromaticityCoordinate", BuildSettings.RedChromaticityCoordinate, DefaultSettings.RedChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "GreenChromaticityCoordinate", BuildSettings.GreenChromaticityCoordinate, DefaultSettings.GreenChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "BlueChromaticityCoordinate", BuildSettings.BlueChromaticityCoordinate, DefaultSettings.BlueChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "WhiteChromaticityCoordinate", BuildSettings.WhiteChromaticityCoordinate, DefaultSettings.WhiteChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "ChromaticAdaptationMethod", BuildSettings.ChromaticAdaptationMethod, DefaultSettings.ChromaticAdaptationMethod);
	}
	else // @todo SerializeForKey : remove else-case when overall key changes
	{
		WriteCbFieldWithDefault(Writer, "RedChromaticityCoordinate", BuildSettings.RedChromaticityCoordinate, DefaultSettings.RedChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "GreenChromaticityCoordinate", BuildSettings.GreenChromaticityCoordinate, DefaultSettings.GreenChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "BlueChromaticityCoordinate", BuildSettings.BlueChromaticityCoordinate, DefaultSettings.BlueChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "WhiteChromaticityCoordinate", BuildSettings.WhiteChromaticityCoordinate, DefaultSettings.WhiteChromaticityCoordinate);
		WriteCbFieldWithDefault(Writer, "ChromaticAdaptationMethod", BuildSettings.ChromaticAdaptationMethod, DefaultSettings.ChromaticAdaptationMethod);
	}

	if (BuildSettings.SourceEncodingOverride != 0 || BuildSettings.bHasColorSpaceDefinition)
	{
		/*
		 * If any advanced color settings are in use, we link the OpenColorIO library version to the key
		 * since updates could cause changes in processing.
		 */ 
		WriteCbField<uint32>(Writer, "OpenColorIOVersion", FTextureBuildSettings::GetOpenColorIOVersion());
	}

	WriteCbFieldWithDefault<bool>(Writer, "bUseLegacyGamma", BuildSettings.bUseLegacyGamma, DefaultSettings.bUseLegacyGamma);
	WriteCbFieldWithDefault<bool>(Writer, "bPreserveBorder", BuildSettings.bPreserveBorder, DefaultSettings.bPreserveBorder);
	WriteCbFieldWithDefault<bool>(Writer, "bForceNoAlphaChannel", BuildSettings.bForceNoAlphaChannel, DefaultSettings.bForceNoAlphaChannel);
	WriteCbFieldWithDefault<bool>(Writer, "bForceAlphaChannel", BuildSettings.bForceAlphaChannel, DefaultSettings.bForceAlphaChannel);
	WriteCbFieldWithDefault<bool>(Writer, "bComputeBokehAlpha", BuildSettings.bComputeBokehAlpha, DefaultSettings.bComputeBokehAlpha);
	WriteCbFieldWithDefault<bool>(Writer, "bReplicateRed", BuildSettings.bReplicateRed, DefaultSettings.bReplicateRed);
	WriteCbFieldWithDefault<bool>(Writer, "bReplicateAlpha", BuildSettings.bReplicateAlpha, DefaultSettings.bReplicateAlpha);
	WriteCbFieldWithDefault<bool>(Writer, "bDownsampleWithAverage", BuildSettings.bDownsampleWithAverage, DefaultSettings.bDownsampleWithAverage);
	WriteCbFieldWithDefault<bool>(Writer, "bSharpenWithoutColorShift", BuildSettings.bSharpenWithoutColorShift, DefaultSettings.bSharpenWithoutColorShift);
	WriteCbFieldWithDefault<bool>(Writer, "bBorderColorBlack", BuildSettings.bBorderColorBlack, DefaultSettings.bBorderColorBlack);
	WriteCbFieldWithDefault<bool>(Writer, "bFlipGreenChannel", BuildSettings.bFlipGreenChannel, DefaultSettings.bFlipGreenChannel);
	WriteCbFieldWithDefault<bool>(Writer, "bApplyYCoCgBlockScale", BuildSettings.bApplyYCoCgBlockScale, DefaultSettings.bApplyYCoCgBlockScale);
	WriteCbFieldWithDefault<bool>(Writer, "bApplyKernelToTopMip", BuildSettings.bApplyKernelToTopMip, DefaultSettings.bApplyKernelToTopMip);
	WriteCbFieldWithDefault<bool>(Writer, "bRenormalizeTopMip", BuildSettings.bRenormalizeTopMip, DefaultSettings.bRenormalizeTopMip);
	WriteCbFieldWithDefault<bool>(Writer, "bCPUAccessible", BuildSettings.bCPUAccessible, DefaultSettings.bCPUAccessible);
	WriteCbFieldWithDefault(Writer, "CompositeTextureMode", BuildSettings.CompositeTextureMode, DefaultSettings.CompositeTextureMode);
	WriteCbFieldWithDefault(Writer, "CompositePower", BuildSettings.CompositePower, DefaultSettings.CompositePower);
	WriteCbFieldWithDefault(Writer, "LODBias", BuildSettings.LODBias, DefaultSettings.LODBias);
	WriteCbFieldWithDefault(Writer, "LODBiasWithCinematicMips", BuildSettings.LODBiasWithCinematicMips, DefaultSettings.LODBiasWithCinematicMips);
	WriteCbFieldWithDefault<bool>(Writer, "bStreamable", BuildSettings.bStreamable_Unused, DefaultSettings.bStreamable_Unused);
	WriteCbFieldWithDefault<bool>(Writer, "bVirtualStreamable", BuildSettings.bVirtualStreamable, DefaultSettings.bVirtualStreamable);
	WriteCbFieldWithDefault<bool>(Writer, "bChromaKeyTexture", BuildSettings.bChromaKeyTexture, DefaultSettings.bChromaKeyTexture);
	WriteCbFieldWithDefault(Writer, "PowerOfTwoMode", BuildSettings.PowerOfTwoMode, DefaultSettings.PowerOfTwoMode);
	WriteCbFieldWithDefault(Writer, "PaddingColor", BuildSettings.PaddingColor, DefaultSettings.PaddingColor);
	WriteCbFieldWithDefault<bool>(Writer, "bPadWithBorderColor", BuildSettings.bPadWithBorderColor, DefaultSettings.bPadWithBorderColor);
	WriteCbFieldWithDefault(Writer, "ResizeDuringBuildX", BuildSettings.ResizeDuringBuildX, DefaultSettings.ResizeDuringBuildX);
	WriteCbFieldWithDefault(Writer, "ResizeDuringBuildY", BuildSettings.ResizeDuringBuildY, DefaultSettings.ResizeDuringBuildY);
	WriteCbFieldWithDefault(Writer, "ChromaKeyColor", BuildSettings.ChromaKeyColor, DefaultSettings.ChromaKeyColor);
	WriteCbFieldWithDefault(Writer, "ChromaKeyThreshold", BuildSettings.ChromaKeyThreshold, DefaultSettings.ChromaKeyThreshold);
	WriteCbFieldWithDefault(Writer, "CompressionQuality", BuildSettings.CompressionQuality, DefaultSettings.CompressionQuality);
	WriteCbFieldWithDefault(Writer, "LossyCompressionAmount", BuildSettings.LossyCompressionAmount, DefaultSettings.LossyCompressionAmount);
	WriteCbFieldWithDefault(Writer, "Downscale", BuildSettings.Downscale, DefaultSettings.Downscale);
	WriteCbFieldWithDefault(Writer, "DownscaleOptions", BuildSettings.DownscaleOptions, DefaultSettings.DownscaleOptions);
	WriteCbFieldWithDefault(Writer, "VirtualAddressingModeX", BuildSettings.VirtualAddressingModeX, DefaultSettings.VirtualAddressingModeX);
	WriteCbFieldWithDefault(Writer, "VirtualAddressingModeY", BuildSettings.VirtualAddressingModeY, DefaultSettings.VirtualAddressingModeY);
	WriteCbFieldWithDefault(Writer, "VirtualTextureTileSize", BuildSettings.VirtualTextureTileSize, DefaultSettings.VirtualTextureTileSize);
	WriteCbFieldWithDefault(Writer, "VirtualTextureBorderSize", BuildSettings.VirtualTextureBorderSize, DefaultSettings.VirtualTextureBorderSize);

	WriteCbFieldWithDefault<uint8>(Writer, "OodleEncodeEffort", (uint8)BuildSettings.OodleEncodeEffort, (uint8)DefaultSettings.OodleEncodeEffort);	
	WriteCbFieldWithDefault<uint8>(Writer, "OodleUniversalTiling", (uint8)BuildSettings.OodleUniversalTiling, (uint8)DefaultSettings.OodleUniversalTiling);
	WriteCbFieldWithDefault<uint8>(Writer, "OodleRDO", BuildSettings.OodleRDO, DefaultSettings.OodleRDO);
	WriteCbFieldWithDefault<bool>(Writer, "bOodleUsesRDO", BuildSettings.bOodleUsesRDO, DefaultSettings.bOodleUsesRDO);
	WriteCbFieldWithDefault<bool>(Writer, "bOodlePreserveExtremes", BuildSettings.bOodlePreserveExtremes, DefaultSettings.bOodlePreserveExtremes);

	WriteCbFieldWithDefault(Writer, "OodleTextureSdkVersion", BuildSettings.OodleTextureSdkVersion, DefaultSettings.OodleTextureSdkVersion);

	WriteCbFieldWithDefault(Writer, "TextureAddressModeX", BuildSettings.TextureAddressModeX, DefaultSettings.TextureAddressModeX);
	WriteCbFieldWithDefault(Writer, "TextureAddressModeY", BuildSettings.TextureAddressModeY, DefaultSettings.TextureAddressModeY);
	WriteCbFieldWithDefault(Writer, "TextureAddressModeZ", BuildSettings.TextureAddressModeZ, DefaultSettings.TextureAddressModeZ);

	// @todo SerializeForKey : remove these when overall key changes
	if ( BuildSettings.bVolume )
	{
		WriteCbField<bool>(Writer, "bVolume_ForceNewDDcKey", true); 
	}
	if ( BuildSettings.bVirtualStreamable && BuildSettings.bSRGB && BuildSettings.bUseLegacyGamma )
	{
		WriteCbField<bool>(Writer, "VTPow22_ForceNewDDcKey", true); 
	}
	// @todo: see further above, remove the two else-cases per comments.

	Writer.EndObject();
}

static void WriteSource(FCbWriter& Writer, const UTexture& Texture, int32 LayerIndex, const FTextureBuildSettings& BuildSettings)
{
	const FTextureSource& Source = Texture.Source;

	Writer.BeginObject();

	Writer.AddInteger("SourceFormat", Source.GetFormat(LayerIndex));
	Writer.AddInteger("GammaSpace", static_cast<uint8>(Source.GetGammaSpace(LayerIndex)));
	Writer.AddInteger("NumSlices", (BuildSettings.bCubemap || BuildSettings.bTextureArray || BuildSettings.bVolume) ? Source.GetNumSlices() : 1);
	Writer.AddInteger("SizeX", Source.GetSizeX());
	Writer.AddInteger("SizeY", Source.GetSizeY());
	Writer.BeginArray("Mips");
	int32 NumMips = BuildSettings.MipGenSettings == TMGS_LeaveExistingMips ? Source.GetNumMips() : 1;
	int64 Offset = 0;
	for (int32 MipIndex = 0, MipCount = NumMips; MipIndex < MipCount; ++MipIndex)
	{
		Writer.BeginObject();
		Writer.AddInteger("Offset", Offset);
		const int64 MipSize = Source.CalcMipSize(MipIndex);
		Writer.AddInteger("Size", MipSize);
		Offset += MipSize;
		Writer.EndObject();
	}
	Writer.EndArray();

	Writer.EndObject();
}

static FRWLock GTextureBuildFunctionLock;
static TMap<FName, UE::FUtf8SharedString> GTextureBuildFunctionMap;

UE::FUtf8SharedString FindTextureBuildFunction(const FName TextureFormatName)
{
	using namespace UE::DerivedData;

	{
		FReadScopeLock Lock(GTextureBuildFunctionLock);
		if (const UE::FUtf8SharedString* Function = GTextureBuildFunctionMap.Find(TextureFormatName))
		{
			return *Function;
		}
	}

	FName TextureFormatModuleName;
	ITextureFormatManagerModule* TFM = GetTextureFormatManager();
	if (TFM == nullptr)
	{
		return {};
	}

	ITextureFormatModule* TextureFormatModule = nullptr;
	if (!TFM->FindTextureFormatAndModule(TextureFormatName, TextureFormatModuleName, TextureFormatModule))
	{
		return {};
	}

	TStringBuilder<128> FunctionName;

	// Texture format modules are inconsistent in their naming, e.g., TextureFormatUncompressed, <Platform>TextureFormat.
	// Attempt to unify the naming of build functions as <Format>Texture.
	FunctionName << TextureFormatModuleName << TEXTVIEW("Texture");
	if (int32 Index = UE::String::FindFirst(FunctionName, TEXTVIEW("TextureFormat")); Index != INDEX_NONE)
	{
		FunctionName.RemoveAt(Index, TEXTVIEW("TextureFormat").Len());
	}

	FTCHARToUTF8 FunctionNameUtf8(FunctionName);

	if (!GetBuild().GetFunctionRegistry().FindFunctionVersion(FunctionNameUtf8).IsValid())
	{
		return {};
	}

	FWriteScopeLock Lock(GTextureBuildFunctionLock);
	UE::FUtf8SharedString& Function = GTextureBuildFunctionMap.FindOrAdd(TextureFormatName);
	if (Function.IsEmpty())
	{
		Function = FunctionNameUtf8;
	}
	return Function;
}

FCbObject SaveTextureBuildSettings(const UTexture& Texture, const FTextureBuildSettings& BuildSettings, int32 LayerIndex, bool bUseCompositeTexture)
{
	const ITextureFormat* TextureFormat = nullptr;
	if (ITextureFormatManagerModule* TFM = GetTextureFormatManager())
	{
		FName TextureFormatModuleName;
		ITextureFormatModule* TextureFormatModule = nullptr;
		TextureFormat = TFM->FindTextureFormatAndModule(BuildSettings.TextureFormatName, TextureFormatModuleName, TextureFormatModule);
	}
	if (TextureFormat == nullptr)
	{
		return FCbObject();
	}

	FCbWriter Writer;
	Writer.BeginObject();

	Writer.AddUuid("BuildVersion", GetTextureDerivedDataVersion());

	if (BuildSettings.bAffectedBySharedLinearEncoding)
	{
		Writer.AddUuid("SharedLinearEncodingVersion", GetTextureSLEDerivedDataVersion());
	}
	
	if (Texture.CompressionCacheId.IsValid())
	{
		// Not actually read by the worker - just used to make a different key
		Writer.AddUuid("CompressionCacheId", Texture.CompressionCacheId);
	}

	if (uint16 TextureFormatVersion = TextureFormat->GetVersion(BuildSettings.TextureFormatName, &BuildSettings))
	{
		Writer.AddInteger("FormatVersion", TextureFormatVersion);
	}

	Writer.SetName("Build");
	WriteBuildSettings(Writer, BuildSettings, TextureFormat);

	Writer.SetName("Source");
	WriteSource(Writer, Texture, LayerIndex, BuildSettings);

	if (bUseCompositeTexture && Texture.GetCompositeTexture())
	{
		check( Texture.GetCompositeTexture()->Source.IsValid() ); // should have been checked to set bUseCompositeTexture

		Writer.SetName("CompositeSource");
		WriteSource(Writer, *Texture.GetCompositeTexture(), LayerIndex, BuildSettings);
	}

	Writer.EndObject();
	return Writer.Save().AsObject();
}

#endif // WITH_EDITOR
