// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreTypes.h"
#include "Containers/StringFwd.h"

class FCbObject;
class FCbWriter;
class FName;
class ITargetPlatform;
class UTexture;
struct FTextureBuildSettings;

UE::FUtf8SharedString FindTextureBuildFunction(FName TextureFormatName);
FCbObject SaveTextureBuildSettings(const UTexture& Texture, const FTextureBuildSettings& BuildSettings, int32 LayerIndex, bool bUseCompositeTexture);

#if WITH_EDITOR
namespace UE::TextureBuildUtilities
{

bool TryWriteCookDeterminismDiagnostics(FCbWriter& Writer, UTexture* Texture, const ITargetPlatform* TargetPlatform);

} // namespace UE::TextureBuildUtilities
#endif

#endif // WITH_EDITOR
