// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureDerivedDataTask.cpp: Tasks to update texture DDC.
=============================================================================*/

#include "TextureDerivedDataTask.h"
#include "IImageWrapperModule.h"
#include "Misc/ScopedSlowTask.h"
#include "TextureResource.h"
#include "Engine/Texture2DArray.h"

#if WITH_EDITOR

#include "Algo/Accumulate.h"
#include "Algo/AnyOf.h"
#include "ChildTextureFormat.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildInputResolver.h"
#include "DerivedDataBuildInputs.h"
#include "DerivedDataBuildOutput.h"
#include "DerivedDataBuildSession.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataThreadPoolTask.h"
#include "Engine/VolumeTexture.h"
#include "EngineLogs.h"
#include "ImageCoreUtils.h"
#include "Interfaces/ITextureFormat.h"
#include "Serialization/BulkDataRegistry.h"
#include "Serialization/MemoryReader.h"
#include "TextureBuildUtilities.h"
#include "TextureCompiler.h"
#include "TextureDerivedDataBuildUtils.h"
#include "TextureFormatManager.h"
#include "VT/VirtualTextureChunkDDCCache.h"
#include "VT/VirtualTextureDataBuilder.h"
#include "ProfilingDebugging/CookStats.h"

#if ENABLE_COOK_STATS
namespace TextureCookStats
{
	static FCookStats::FDDCResourceUsageStats TaskUsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterTaskCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		TextureCookStats::TaskUsageStats.LogStats(AddStat, TEXT("Texture.Usage"), TEXT("Task"));
	});
}
#endif

static TAutoConsoleVariable<int32> CVarVTValidateCompressionOnLoad(
	TEXT("r.VT.ValidateCompressionOnLoad"),
	0,
	TEXT("Validates that VT data contains no compression errors when loading from DDC")
	TEXT("This is slow, but allows debugging corrupt VT data (and allows recovering from bad DDC)")
);

static TAutoConsoleVariable<int32> CVarVTValidateCompressionOnSave(
	TEXT("r.VT.ValidateCompressionOnSave"),
	0,
	TEXT("Validates that VT data contains no compression errors before saving to DDC")
	TEXT("This is slow, but allows debugging corrupt VT data")
);

static TAutoConsoleVariable<int32> CVarForceRetileTextures(
	TEXT("r.ForceRetileTextures"),
	0,
	TEXT("If Shared Linear Texture Encoding is enabled in project settings, this will force the tiling build step to rebuild,")
	TEXT("however the linear texture is allowed to fetch from cache.")
);


void GetTextureDerivedDataKeyFromSuffix(const FString& KeySuffix, FString& OutKey);
static void PackTextureBuildMetadataInPlatformData(FTexturePlatformData* PlatformData, const UE::TextureBuildUtilities::FTextureBuildMetadata& BuildMetadata)
{
	PlatformData->PreEncodeMipsHash = BuildMetadata.PreEncodeMipsHash;
}

static FTextureEngineParameters GenerateTextureEngineParameters()
{
	FTextureEngineParameters EngineParameters;
	EngineParameters.bEngineSupportsTexture2DArrayStreaming = GSupportsTexture2DArrayStreaming;
	EngineParameters.bEngineSupportsVolumeTextureStreaming = GSupportsVolumeTextureStreaming;
	EngineParameters.NumInlineDerivedMips = NUM_INLINE_DERIVED_MIPS;
	return EngineParameters;
}

class FTextureStatusMessageContext : public FScopedSlowTask
{
public:
	explicit FTextureStatusMessageContext(const FText& InMessage)
		: FScopedSlowTask(0, InMessage, IsInGameThread())
	{
		UE_LOGF(LogTexture,Display,"%ls",*InMessage.ToString());
	}
};


static FText ComposeTextureBuildText(ETextureClass TextureClass,const FString& TexturePathName, int32 SizeX, int32 SizeY, int32 NumSlices, int32 NumBlocks, int32 NumLayers, const FTextureBuildSettings& BuildSettings, ETextureEncodeSpeed InEncodeSpeed, int64 RequiredMemoryEstimate, bool bIsVT)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("TextureClass"), FText::FromString(StaticEnum<ETextureClass>()->GetNameStringByValue((int64)TextureClass)));
	Args.Add(TEXT("TextureName"), FText::FromString(TexturePathName));
	Args.Add(TEXT("TextureFormatName"), FText::FromString(BuildSettings.TextureFormatName.GetPlainNameString()));
	Args.Add(TEXT("IsVT"), FText::FromString( FString( bIsVT ? TEXT(" VT") : TEXT("") ) ) );
	Args.Add(TEXT("TextureResolutionX"), FText::FromString(FString::FromInt(SizeX)));
	Args.Add(TEXT("TextureResolutionY"), FText::FromString(FString::FromInt(SizeY)));
	Args.Add(TEXT("NumBlocks"), FText::FromString(FString::FromInt(NumBlocks)));
	Args.Add(TEXT("NumLayers"), FText::FromString(FString::FromInt(NumLayers)));
	Args.Add(TEXT("NumSlices"), FText::FromString(FString::FromInt(NumSlices)));
	Args.Add(TEXT("EstimatedMemory"), FText::FromString(FString::SanitizeFloat(double(RequiredMemoryEstimate) / (1024.0*1024.0), 3)));
	
	const TCHAR* SpeedText = TEXT("");
	switch (InEncodeSpeed)
	{
	case ETextureEncodeSpeed::Final: SpeedText = TEXT("Final"); break;
	case ETextureEncodeSpeed::Fast: SpeedText = TEXT("Fast"); break;
	case ETextureEncodeSpeed::FinalIfAvailable: SpeedText = TEXT("FinalIfAvailable"); break;
	}

	Args.Add(TEXT("Speed"), FText::FromString(FString(SpeedText)));

	return FText::Format(
		NSLOCTEXT("Engine", "BuildTextureStatus", "Building texture {TextureClass}: {TextureName} ({TextureFormatName}{IsVT}, {TextureResolutionX}x{TextureResolutionY} x{NumSlices}x{NumLayers}x{NumBlocks}) (Required Memory Estimate: {EstimatedMemory} MB), EncodeSpeed: {Speed}"), 
		Args
	);
}

static FText ComposeTextureBuildText(ETextureClass TextureClass, const FString& TexturePathName, const FTextureSourceData& TextureData, const FTextureBuildSettings& BuildSettings, ETextureEncodeSpeed InEncodeSpeed, int64 RequiredMemoryEstimate, bool bIsVT)
{
	const FImage & MipImage = TextureData.Blocks[0].MipsPerLayer[0][0];
	return ComposeTextureBuildText(TextureClass, TexturePathName, MipImage.SizeX, MipImage.SizeY, MipImage.NumSlices, TextureData.Blocks.Num(), TextureData.Layers.Num(), BuildSettings, InEncodeSpeed, RequiredMemoryEstimate, bIsVT);
}

static FText ComposeTextureBuildText(const UTexture& Texture, const FTextureBuildSettings& BuildSettings, ETextureEncodeSpeed InEncodeSpeed, int64 RequiredMemoryEstimate, bool bIsVT)
{
	return ComposeTextureBuildText(Texture.GetTextureClass(), Texture.GetPathName(), Texture.Source.GetSizeX(), Texture.Source.GetSizeY(), Texture.Source.GetNumSlices(), Texture.Source.GetNumBlocks(), Texture.Source.GetNumLayers(), BuildSettings, InEncodeSpeed, RequiredMemoryEstimate, bIsVT);
}

static bool ValidateTexture2DPlatformData(const FTexturePlatformData& TextureData, const UTexture2D& Texture, bool bFromDDC)
{
	// Temporarily disable as the size check reports false negatives on some platforms
#if 0
	bool bValid = true;
	for (int32 MipIndex = 0; MipIndex < TextureData.Mips.Num(); ++MipIndex)
	{
		const FTexture2DMipMap& MipMap = TextureData.Mips[MipIndex];
		const int64 BulkDataSize = MipMap.BulkData.GetBulkDataSize();
		if (BulkDataSize > 0)
		{
			const int64 ExpectedMipSize = CalcTextureMipMapSize(TextureData.SizeX, TextureData.SizeY, TextureData.PixelFormat, MipIndex);
			if (BulkDataSize != ExpectedMipSize)
			{
				//UE_LOGF(LogTexture,Warning,"Invalid mip data. Texture will be rebuilt. MipIndex %d [%dx%d], Expected size %lld, BulkData size %lld, PixelFormat %ls, LoadedFromDDC %d, Texture %ls", 
				//	MipIndex, 
				//	MipMap.SizeX, 
				//	MipMap.SizeY, 
				//	ExpectedMipSize, 
				//	BulkDataSize, 
				//	GPixelFormats[TextureData.PixelFormat].Name, 
				//	bFromDDC ? 1 : 0,
				//	*Texture.GetFullName());
				
				bValid = false;
			}
		}
	}

	return bValid;
#else
	return true;
#endif
}

void FTextureSourceData::InitAsPlaceholder()
{
	ReleaseMemory();

	// This needs to be a tiny texture that can encode on all hardware. It's job is to
	// take up as little memory as possible for textures where we'd rather they not create
	// hw resources at all, but we don't want to hack in a ton of redirects/tests all over
	// the rendering codebase.

	// So we make a 4x4 black RGBA8 texture.
	FTextureSourceBlockData& Block = Blocks.AddDefaulted_GetRef();
	{
		Block.NumMips = 1;
		TArray<FImage>& MipsPerLayer = Block.MipsPerLayer.AddDefaulted_GetRef();
		FImage& Mip = MipsPerLayer.AddDefaulted_GetRef();
		UE::TextureBuildUtilities::GetPlaceholderTextureImage(&Mip);

		Block.NumSlices = Mip.NumSlices;
		Block.SizeX = Mip.SizeX;
		Block.SizeY = Mip.SizeY;
	}

	FTextureSourceLayerData& Layer = Layers.AddDefaulted_GetRef();
	{
		Layer.ImageFormat = ERawImageFormat::BGRA8;
		Layer.SourceGammaSpace = EGammaSpace::Linear;
	}

	bValid = true;
}

void FTextureSourceData::Init(UTexture& InTexture, TextureMipGenSettings InMipGenSettings, bool bInCubeMap, bool bInTextureArray, bool bInVolumeTexture, ETexturePowerOfTwoSetting::Type InPow2Setting, int32 InResizeDuringBuildX, int32 InResizeDuringBuildY, bool bAllowAsyncLoading)
{
	check( bValid == false ); // we set to true at the end, acts as our return value
	
	if ( ! InTexture.Source.IsValid() )
	{
		UE_LOGF(LogTexture, Warning, "FTextureSourceData::Init on Invalid texture: %ls", *InTexture.GetPathName());
		return;
	}

	const int32 NumBlocks = InTexture.Source.GetNumBlocks();
	const int32 NumLayers = InTexture.Source.GetNumLayers();
	if (NumBlocks < 1 || NumLayers < 1)
	{
		UE_LOGF(LogTexture, Warning, "Texture has no source data: %ls", *InTexture.GetPathName());
		return;
	}
	
	// Copy the channel min/max if we have it already
	// if Texture Source did not already have SourceLayerColorInfo, we will update it in GetSourceMips (when we have decompressed data)
	
	TArray<FTextureSourceLayerColorInfo> SourceLayerColorInfo;
	InTexture.Source.GetLayerColorInfo(SourceLayerColorInfo);

	check( SourceLayerColorInfo.Num() == 0 || SourceLayerColorInfo.Num() == NumLayers );
	LayerChannelMinMax.SetNum( SourceLayerColorInfo.Num() );
	for(int32 i=0;i<SourceLayerColorInfo.Num();i++)
	{
		LayerChannelMinMax[i].Key   = SourceLayerColorInfo[i].ColorMin;
		LayerChannelMinMax[i].Value = SourceLayerColorInfo[i].ColorMax;
	}

	Layers.Reserve(NumLayers);
	for (int LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FTextureSourceLayerData& LayerData = Layers.AddDefaulted_GetRef();

		LayerData.ImageFormat = FImageCoreUtils::ConvertToRawImageFormat( InTexture.Source.GetFormat(LayerIndex) );

		LayerData.SourceGammaSpace = InTexture.Source.GetGammaSpace(LayerIndex);
	}

	Blocks.Reserve(NumBlocks);
	SizeInBlocksX = SizeInBlocksY = 0;
	BlockSizeX = BlockSizeY = 0;
	for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
	{
		FTextureSourceBlock SourceBlock;
		InTexture.Source.GetBlock(BlockIndex, SourceBlock);

		if (SourceBlock.NumMips > 0 && SourceBlock.NumSlices > 0)
		{
			FTextureSourceBlockData& BlockData = Blocks.AddDefaulted_GetRef();
			BlockData.BlockX = SourceBlock.BlockX;
			BlockData.BlockY = SourceBlock.BlockY;
			BlockData.SizeX = SourceBlock.SizeX;
			BlockData.SizeY = SourceBlock.SizeY;
			BlockData.NumMips = SourceBlock.NumMips;
			BlockData.NumSlices = SourceBlock.NumSlices;

			if (InMipGenSettings != TMGS_LeaveExistingMips)
			{
				BlockData.NumMips = 1;
			}

			if (!bInCubeMap && !bInTextureArray && !bInVolumeTexture)
			{
				BlockData.NumSlices = 1;
			}

			BlockData.MipsPerLayer.SetNum(NumLayers);

			SizeInBlocksX = FMath::Max(SizeInBlocksX, SourceBlock.BlockX + 1);
			SizeInBlocksY = FMath::Max(SizeInBlocksY, SourceBlock.BlockY + 1);
			BlockSizeX = FMath::Max(BlockSizeX, SourceBlock.SizeX);
			BlockSizeY = FMath::Max(BlockSizeY, SourceBlock.SizeY);
		}
	}

	if ( Blocks.Num() == 0 )
	{
		UE_LOGF(LogTexture, Error, "No valid source blocks [%ls]", *InTexture.GetPathName());
		check( bValid == false );
		return;
	}

	if ( Blocks.Num() > 1 )
	{
		int32 BlockSizeZ=1;
		UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(
			BlockSizeX, BlockSizeY, 1,
			false,
			InPow2Setting, InResizeDuringBuildX, InResizeDuringBuildY,
			BlockSizeX, BlockSizeY, BlockSizeZ);

		for (FTextureSourceBlockData& Block : Blocks)
		{
			int32 AdjustedSizeX, AdjustedSizeY, AdjustedSizeZ;
			UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(
				Block.SizeX, Block.SizeY, 1,
				false,
				InPow2Setting, InResizeDuringBuildX, InResizeDuringBuildY,
				AdjustedSizeX, AdjustedSizeY, AdjustedSizeZ);

			// for the common case of NumBlocks == 1, BlockSizeX == Block.SizeX, MipBiasX/Y will both be zero
			const int32 MipBiasX = FMath::CeilLogTwo(BlockSizeX / AdjustedSizeX);
			const int32 MipBiasY = FMath::CeilLogTwo(BlockSizeY / AdjustedSizeY);
			if (MipBiasX != MipBiasY)
			{
				// @todo Oodle: this is failing even if "pad to pow2 square" is set, can we allow it through in that case?
				// @@!! is this fixed now?
				UE_LOGF(LogTexture, Error, "VT has blocks with mismatched aspect ratios, cannot build. [%ls]", *InTexture.GetPathName());
				check( bValid == false );
				return;
			}

			Block.MipBias = MipBiasX;
		}
	}

	TextureFullName = InTexture.GetFullName();

	bValid = true;
}


void FTextureSourceData::GetSourceMips(FTextureSource& Source, IImageWrapperModule* InImageWrapper)
{
	if (bValid)
	{
		const int32 NumBlocks = Source.GetNumBlocks();
		const int32 NumLayers = Source.GetNumLayers();
		
		// these arrays were sized in Init but not fully filled out :
		check( Blocks.Num() == NumBlocks );
		check( Layers.Num() == NumLayers );

		check( NumBlocks > 0 && NumLayers > 0 )
		
		if ( Blocks[0].MipsPerLayer[0].Num() > 0 )
		{
			// If we already got valid data, nothing to do. (GetSourceMips was called before now it's being called again)
			// @@ is this ever hit ? how?
 			return;
		}

		if (!Source.HasPayloadData())
		{	// don't do any work we can't reload this
			UE_LOGF(LogTexture, Warning, "Unable to get texture source mips because its bulk data has no payload. This may happen if it was duplicated from cooked data. %ls", *TextureFullName);
			ReleaseMemory();
			bValid = false;
			return;
		}

		// Grab a copy of ALL the mip data, we'll get views in to this later.
		FTextureSource::FMipData ScopedMipData = Source.GetMipData(InImageWrapper);
		if (!ScopedMipData.IsValid())
		{
			UE_LOGF(LogTexture, Warning, "Cannot retrieve source data for mips of %ls", *TextureFullName);
			ReleaseMemory();
			bValid = false;
			return;
		}
		

		// If we didn't get ChannelMinMax from the texture source, then compute it now. As time goes on this will get hit less and less.
		if (LayerChannelMinMax.Num() != NumLayers)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureSourceData::GetSourceMips_ChannelMinMax);

			// Update MipMax if it wasn't found before
			// do this after GetMipData() so we do it on decompressed data
			
			if ( Source.UpdateChannelMinMaxFromIncomingTextureData(ScopedMipData.GetData().GetView()) )
			{
				TArray<FTextureSourceLayerColorInfo> SourceLayerColorInfo;
				Source.GetLayerColorInfo(SourceLayerColorInfo);

				check( SourceLayerColorInfo.Num() == 0 || SourceLayerColorInfo.Num() == NumLayers );
				LayerChannelMinMax.SetNum( SourceLayerColorInfo.Num() );
				for(int32 i=0;i<SourceLayerColorInfo.Num();i++)
				{
					LayerChannelMinMax[i].Key   = SourceLayerColorInfo[i].ColorMin;
					LayerChannelMinMax[i].Value = SourceLayerColorInfo[i].ColorMax;
				}
			}
			else
			{
				UE_LOGF(LogTexture, Warning, "Unexpected failure in UpdateChannelMinMaxFromIncomingTextureData on %ls", *TextureFullName);
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureSourceData::GetSourceMips_CopyMips);

			for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
			{
				FTextureSourceBlock SourceBlock;
				Source.GetBlock(BlockIndex, SourceBlock);

				FTextureSourceBlockData& BlockData = Blocks[BlockIndex];
				check( BlockData.MipsPerLayer.Num() == NumLayers );

				for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
				{
					const FTextureSourceLayerData& LayerData = Layers[LayerIndex];

					check( BlockData.MipsPerLayer[LayerIndex].Num() == 0 );
					//if Source had existing mips but TMGS was not LeaveExisting, then BlockData.NumMips is set to 1
					check( BlockData.NumMips > 0 && ( BlockData.NumMips == SourceBlock.NumMips || ( BlockData.NumMips == 1 && SourceBlock.NumMips > 1 ) ) );

					BlockData.MipsPerLayer[LayerIndex].SetNum( BlockData.NumMips );

					for (int32 MipIndex = 0; MipIndex < BlockData.NumMips; ++MipIndex)
					{
						FImageView MipView = ScopedMipData.GetMipDataImageView(BlockIndex, LayerIndex, MipIndex);
							
						check(MipView.GammaSpace == LayerData.SourceGammaSpace);
						check(MipView.Format == LayerData.ImageFormat);

						// allocates the destination FImage and copies into it :
						MipView.CopyTo( BlockData.MipsPerLayer[LayerIndex][MipIndex] );

						// CB notes 04/02/2024 :

						// this copy takes a while, and potentially we could just instead point at the FSharedBuffer from the ScopedMipData
						//   (like MipView here does)
						// at the moment that's not easy because all the code around TextureCompressorModule/Formats expects FImage, not FImageView
						//	perhaps ideally we'd have an FImage variant that's COW

						// fundamentally, this whole alloc and copy is totally unecessary, so it would be great to get rid of it
						//	but practically we need a better way to have FImage point at FSharedBuffer
						// one issue is there's no way to MoveTemp into a TArray from Shared/Unique buffer.
						// also at the moment the TextureCompressorModule/Formats assume the FImage is mutable, that would have to be cleaned up
					}
				}
			}
		}

		{
			//ScopedMipData destructor runs now, which frees the FSharedBuffer, which is slow
			TRACE_CPUPROFILER_EVENT_SCOPE(FTextureSourceData::GetSourceMips_Free);
			ScopedMipData.ResetData();
		}
	}
	
	#if 0
	{
		//we have got the source mip data and made a copy of it
		//	no longer need the BulkData to be in memory
		// note this is different than ScopedMipData.ResetData ; that frees the decompressed copy
		//	this frees the compressed copy
		// (note that BulkData does not cache decompressed payloads so this is usually a nop)
		TRACE_CPUPROFILER_EVENT_SCOPE(FTextureSourceData::GetSourceMips_FreeBulkData);
		Source.ReleaseBulkDataCachedMemory();
	}
	#endif
}

// When texture streaming is disabled, all of the mips are packed into a single FBulkData/FDerivedData
// and "inlined", meaning they are saved and loaded as part of the serialized asset data.
static bool GetBuildSettingsDisablesStreaming(const FTextureBuildSettings& InBuildSettings, const FTextureEngineParameters& InEngineParameters)
{
	if (InBuildSettings.bVirtualStreamable)
	{
		// Only basic 2d textures can be virtual streamable.
		return InBuildSettings.bCubemap || InBuildSettings.bVolume || InBuildSettings.bTextureArray;
	}
	else
	{
		return GetStreamingDisabledForNonVirtualTextureProperties(InBuildSettings.bCubemap, InBuildSettings.bVolume, InBuildSettings.bTextureArray, InEngineParameters);
	}
}

// Dumps the output messages that were created during the given build.
static void PrintIBuildOutputMessages(const UE::DerivedData::FBuildOutput& InBuildOutput)
{
	using namespace UE;
	using namespace UE::DerivedData;

	const FSharedString& Name = InBuildOutput.GetName();
	const FUtf8SharedString& Function = InBuildOutput.GetFunction();

	for (const FBuildOutputMessage& Message : InBuildOutput.GetMessages())
	{
		switch (Message.Level)
		{
		case EBuildOutputMessageLevel::Error:
			// We drop errors to warnings so that they don't stop e.g. a cook from occurring as the cook is likely still
			// usable
			UE_LOGF(LogTexture, Warning, "[Error] %ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		case EBuildOutputMessageLevel::Warning:
			UE_LOGF(LogTexture, Warning, "%ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		case EBuildOutputMessageLevel::Display:
			UE_LOGF(LogTexture, Display, "%ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	for (const FBuildOutputLog& Log : InBuildOutput.GetLogs())
	{
		switch (Log.Level)
		{
		case EBuildOutputLogLevel::Error:
			// We drop errors to warnings so that they don't stop e.g. a cook from occurring as the cook is likely still
			// usable
			UE_LOGF(LogTexture, Warning, "[Error] %ls: %ls (Build of '%ls' by %ls.)",
				*WriteToString<64>(Log.Category), *WriteToString<256>(Log.Message),
				*Name, *WriteToString<32>(Function));
			break;
		case EBuildOutputLogLevel::Warning:
			UE_LOGF(LogTexture, Warning, "%ls: %ls (Build of '%ls' by %ls.)",
				*WriteToString<64>(Log.Category), *WriteToString<256>(Log.Message),
				*Name, *WriteToString<32>(Function));
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	if (InBuildOutput.HasError())
	{
		UE_LOGF(LogTexture, Warning, "Failed to build derived data for build of '%ls' by %ls.",
			*Name, *WriteToString<32>(Function));
		return;
	}
}

namespace UE::TextureDerivedData
{

using namespace UE::DerivedData;
using FBuildInputMetadataArray = TArray<UE::DerivedData::FBuildInputMetaByKey, TInlineAllocator<8>>;
using FBuildInputDataArray = TArray<UE::DerivedData::FBuildInputDataByKey, TInlineAllocator<8>>;

//
// Something to just drop in when you need to pipe the outputs of a previous build to
// the inputs to your build.
//
struct FParentBuildPlumbing
{
	UE::DerivedData::FBuildSession& Session;
	UE::DerivedData::FBuildDefinition Definition;
	UE::DerivedData::FBuildPolicy Policy;

	UE::DerivedData::FOptionalBuildOutput Output;
	UE::DerivedData::EStatus FinalStatus = UE::DerivedData::EStatus::Error;

	FParentBuildPlumbing(UE::DerivedData::FBuildSession& InSession, UE::DerivedData::FBuildDefinition& InDefinition, UE::DerivedData::FBuildPolicy& InPolicy) :
		Session(InSession),
		Definition(InDefinition),
		Policy(InPolicy)
	{
	}

	//
	// We can't actually do anything with our build until we have all the 
	// parent builds' outputs - so step one is to get those. We kick them all off
	// and set it up so that we fire our resolved callback once we're done with them all.
	//
	static void ResolveParentInputMetadata(
		FParentBuildPlumbing& InParentBuild,
		const UE::DerivedData::FBuildDefinition& InChildDefinition,
		UE::DerivedData::IRequestOwner& InRequestOwner,
		FOnBuildInputMetaResolved&& InResolvedCallback
		)
	{
		auto ParentBuildCompleted = [&InParentBuild, InChildDefinition, InResolvedCallback = MoveTemp(InResolvedCallback)](UE::DerivedData::FBuildCompleteParams&& InCompleteParams)
		{
			FBuildInputMetadataArray ChildInputMetadata;

			PrintIBuildOutputMessages(InCompleteParams.Output);

			UE::DerivedData::EStatus Status = InCompleteParams.Status;
			if (Status == UE::DerivedData::EStatus::Ok)
			{
				// We have to save the output itself so we can supply the data later during _our_ build.
				InParentBuild.Output = MoveTemp(InCompleteParams.Output);

				// Find everything we want from this build and pipe them over.
				InChildDefinition.IterateInputBuilds([&Status, &InParentBuild, &ChildInputMetadata](FUtf8StringView InOurKey, const UE::DerivedData::FBuildValueKey& InBuildValueKey)
				{
					//UE_LOGF(LogTexture, Warning, "Input Build parent=%ls key=%ls", *WriteToString<128>(InParentBuild.Definition.GetKey()), *WriteToString<128>(InOurKey));

					// Filter to things _this_ build produces as the child could be pulling values from different parents.
					if (InBuildValueKey.BuildKey == InParentBuild.Definition.GetKey())
					{
						const UE::DerivedData::FValueWithId& ParentBuildValue = InParentBuild.Output.Get().GetValue(InBuildValueKey.Id);
						if (ParentBuildValue.IsNull())
						{
							UE_LOGF(LogTexture, Warning, "Failed to resolve texture build parent input metadata for key: %ls", *WriteToString<128>(InOurKey));
							Status = UE::DerivedData::EStatus::Error;
							return;
						}

						ChildInputMetadata.Add({ InOurKey, ParentBuildValue.GetRawHash(), ParentBuildValue.GetRawSize() });
					}
				});
			}

			if (Status != UE::DerivedData::EStatus::Ok)
			{
				ChildInputMetadata.Reset();
				InParentBuild.FinalStatus = Status;
			}

			InResolvedCallback({ ChildInputMetadata, Status });
			return;
		};

		// Start the build.
		InParentBuild.Session.Build(InParentBuild.Definition, {}, InParentBuild.Policy, InRequestOwner, MoveTemp(ParentBuildCompleted));
	} // end ResolvedInputMeta

	
	static void ResolveParentInputData(
		FParentBuildPlumbing& InParentBuild,
		const FBuildDefinition& InChildDefinition,
		FBuildInputFilter& InInputFilter,
		FOnBuildInputDataResolved&& InResolvedCallback
		)
	{
		// We already have the parent build output from resolving the metadata so
		// we just have to find the values.
		if (!InParentBuild.Output.IsValid())
		{
			return;
		}

		FBuildInputDataArray ChildInputData;
		UE::DerivedData::EStatus Status = UE::DerivedData::EStatus::Ok;
		InChildDefinition.IterateInputBuilds([&InParentBuild, &InInputFilter, &ChildInputData, &Status](FUtf8StringView InOurKey, const UE::DerivedData::FBuildValueKey& InBuildValueKey)
		{
			if (InInputFilter && InInputFilter(InOurKey) == false)
			{
				return;
			}

			const UE::DerivedData::FValueWithId& ParentBuildValue = InParentBuild.Output.Get().GetValue(InBuildValueKey.Id);
			if (!ParentBuildValue || !ParentBuildValue.HasData())
			{
				UE_LOGF(LogTexture, Warning, "Missing parent input data for key: %ls / %ls -- valid %d hasdata %d", *WriteToString<128>(InOurKey), *WriteToString<128>(InBuildValueKey.Id), ParentBuildValue.IsValid(), ParentBuildValue.HasData());
				Status = UE::DerivedData::EStatus::Error;
				return;
			}

			ChildInputData.Add({ InOurKey, ParentBuildValue.GetData() });
		});

		if (Status != UE::DerivedData::EStatus::Ok)
		{
			ChildInputData.Reset();
		}

		InResolvedCallback({ChildInputData, Status});
	}
};

class FTextureGenericBuildInputResolver final : public IBuildInputResolver
{
public:
	UTexture* Texture = nullptr;
	IBuildInputResolver* GlobalResolver = nullptr;

	TMap<UE::DerivedData::FBuildKey, FParentBuildPlumbing> ChildBuilds;

	// Only used if we don't have the global resolver. Since the texture source doesn't deliver as a compressed
	// buffer, we on-demand compress it when the metadata resolves sio we can deliver it in the data resolution.
	// We don't want to load the bulk data unless we need it because the resolver gets constructed whether or not
	// we do a build.
	FCompressedBuffer CompositeSourceBuffer, SourceBuffer;

	const FCompressedBuffer* FindSource(bool bInComposite, const FGuid& BulkDataId)
	{
		FTextureSource* Source = &Texture->Source;
		FCompressedBuffer* Buffer = &SourceBuffer;
		if (bInComposite)
		{
			if (!Texture->GetCompositeTexture())
			{
				return nullptr;
			}
			Source = &Texture->GetCompositeTexture()->Source;
			Buffer = &CompositeSourceBuffer;
		}

		if (Source->GetPersistentId() != BulkDataId)
		{
			return nullptr;
		}

		if (Buffer->IsNull())
		{
			Source->OperateOnLoadedBulkData([&Buffer](const FSharedBuffer& BulkDataBuffer)
			{
				*Buffer = FCompressedBuffer::Compress(BulkDataBuffer);
			});
		}
		return Buffer;
	}

	// Convert from named keys to hash/size pairs. There is no expectation that the results are
	// ready when this function returns - InResolvedCallback is called when the results arrive.
	void ResolveInputMeta(
		const FBuildDefinition& InDefinition,
		IRequestOwner& InRequestOwner,
		FOnBuildInputMetaResolved&& InResolvedCallback) final
	{
		//
		// If we have a global resolver, it needs to handle ALL bulk data resolution. Otherwise,
		// we resolve it against our textures.
		// 
		// The issue is that the global resolver CANT handle anything else and will log errors
		// if it gets anything else requested.
		//
		// We also can't partially resolve - we either handle everything, or the global resolver
		// has to handle everything.
		//

		// If we are build that just consumes inputs from the parent, do that.
		FParentBuildPlumbing* ParentBuild = ChildBuilds.Find(InDefinition.GetKey());
		if (ParentBuild)
		{
			FParentBuildPlumbing::ResolveParentInputMetadata({*ParentBuild}, InDefinition, InRequestOwner, MoveTemp(InResolvedCallback));
			return;
		}

		// Pass through to the global resolver if we have one.
		if (GlobalResolver)
		{
			GlobalResolver->ResolveInputMeta(InDefinition, InRequestOwner, MoveTemp(InResolvedCallback));
			return;
		}

		// No global resolver - try and resolve bulk data against our textures.
		if (Texture)
		{
			FBuildInputMetadataArray Inputs;
			UE::DerivedData::EStatus Status = UE::DerivedData::EStatus::Ok;
			InDefinition.IterateInputBulkData([this, &Status, &Inputs](FUtf8StringView Key, const FGuid& BulkDataId)
			{
				const FCompressedBuffer* Buffer = this->FindSource(Key != UTF8TEXTVIEW("Source"), BulkDataId);
				if (Buffer)
				{
					Inputs.Add({ Key, Buffer->GetRawHash(), Buffer->GetRawSize() });
				}
				else
				{
					UE_LOGF(LogTexture, Warning, "Failed to resolve texture build metadata for key: %ls", *WriteToString<128>(Key));
					Status = EStatus::Error;
				}
			});

			if (Status != UE::DerivedData::EStatus::Ok)
			{
				Inputs.Empty();
			}

			InResolvedCallback({ Inputs, Status });
			return;
		}
	} // end ResolvedInputMeta


	void ResolveInputData(
		const FBuildDefinition& InDefinition,
		IRequestOwner& InRequestOwner,
		FOnBuildInputDataResolved&& InResolvedCallback,
		FBuildInputFilter&& InFilter) final
	{
		FParentBuildPlumbing* ParentBuild = ChildBuilds.Find(InDefinition.GetKey());
		if (ParentBuild)
		{
			FParentBuildPlumbing::ResolveParentInputData(*ParentBuild, InDefinition, InFilter, MoveTemp(InResolvedCallback));
			return;
		}

		// Pass through to the global resolver if we have one.
		if (GlobalResolver)
		{
			GlobalResolver->ResolveInputData(InDefinition, InRequestOwner, MoveTemp(InResolvedCallback), MoveTemp(InFilter));
			return;
		}

		if (Texture)
		{
			EStatus Status = EStatus::Ok;
			TArray<FBuildInputDataByKey> Inputs;
			InDefinition.IterateInputBulkData([this, &InFilter, &Status, &Inputs](FUtf8StringView Key, const FGuid& BulkDataId)
			{
				if (!InFilter || InFilter(Key))
				{
					const FCompressedBuffer* Buffer = this->FindSource(Key != UTF8TEXTVIEW("Source"), BulkDataId);

					if (Buffer)
					{
						Inputs.Add({ Key, *Buffer });
					}
					else
					{
						Status = EStatus::Error;
					}
				}
			});
			InResolvedCallback({ Inputs, Status });
			return;
		}
	}

};

} // UE::TextureDerivedData

static void DDC1_StoreClassicTextureInDerivedData(
	TArray<FCompressedImage2D>& CompressedMips, FTexturePlatformData* DerivedData, bool bVolume, bool bTextureArray, bool bCubemap, uint32 NumMipsInTail,
	uint32 ExtData, bool bReplaceExistingDDC, const FString& TexturePathName, const FString& KeySuffix, int64& BytesCached
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.DDC1_StoreClassicTextureInDerivedData);

	const int32 MipCount = CompressedMips.Num();

	// VT can be bigger than (1<<(MAX_TEXTURE_MIP_COUNT-1)) , but doesn't actually make all those mips
	// bForVirtualTextureStreamingBuild is false in this branch
	// MipCount here can actually be more than MAX_TEXTURE_MIP_COUNT and that's okay if LODBias will drop those mips
	//	because of the delightful way LODBias works, we have to actually build the too-big mips and they will be dropped later
	//	(it's very unusual to reach this case, typically MaxTextureSize should have been set to 16384 preventing that)
	//check(MipCount <= MAX_TEXTURE_MIP_COUNT);

	for (int32 MipIndex = 0; MipIndex < MipCount; ++MipIndex)
	{
		const FCompressedImage2D& CompressedImage = CompressedMips[MipIndex];
		FTexture2DMipMap* NewMip = new FTexture2DMipMap(CompressedImage.SizeX, CompressedImage.SizeY, CompressedImage.GetRHIStyleSizeZ(bTextureArray, bVolume));
		DerivedData->Mips.Add(NewMip);
		NewMip->FileRegionType = FFileRegion::SelectType(EPixelFormat(CompressedImage.PixelFormat));
		check(NewMip->SizeZ == 1 || bVolume || bTextureArray); // Only volume & arrays can have SizeZ != 1

		check(CompressedImage.RawData.GetTypeSize() == 1);
		int64 CompressedDataSize = CompressedImage.RawData.Num();

		// CompressedDataSize can exceed int32 ; eg. 16k x 16k x RGBA16F == 2 GB
		// DDC1 should be 64-bit safe now

		NewMip->BulkData.Lock(LOCK_READ_WRITE);
		void* NewMipData = NewMip->BulkData.Realloc(CompressedDataSize);
		FMemory::Memcpy(NewMipData, CompressedImage.RawData.GetData(), CompressedDataSize);
		NewMip->BulkData.Unlock();

		if (MipIndex == 0)
		{
			DerivedData->SizeX = CompressedImage.SizeX;
			DerivedData->SizeY = CompressedImage.SizeY;
			DerivedData->PixelFormat = (EPixelFormat)CompressedImage.PixelFormat;
			DerivedData->SetNumSlices(CompressedImage.NumSlicesWithDepth);
			DerivedData->SetIsCubemap(bCubemap);
		}
		else
		{
			check(CompressedImage.PixelFormat == DerivedData->PixelFormat);
		}
	}

	FOptTexturePlatformData OptData;
	OptData.NumMipsInTail = NumMipsInTail;
	OptData.ExtData = ExtData;
	DerivedData->SetOptData(OptData);

	// Store it in the cache.
	// @todo: This will remove the streaming bulk data, which we immediately reload below!
	// Should ideally avoid this redundant work, but it only happens when we actually have 
	// to build the texture, which should only ever be once.
	BytesCached = PutDerivedDataInCache(DerivedData, KeySuffix, TexturePathName, bCubemap || (bVolume && !GSupportsVolumeTextureStreaming) || (bTextureArray && !GSupportsTexture2DArrayStreaming), bReplaceExistingDDC);
}

static bool DDC1_DecodeImageIfNeeded(FName BaseTextureFormatName, bool bSRGB, int32 InLODBias, TArray<FCompressedImage2D>& CompressedMips, const FString& TexturePathName)
{
	// Only decompress if we need to in order to view the format in the editor.
	bool bNeedsDecode = IsASTCBlockCompressedTextureFormat(CompressedMips[0].PixelFormat) || IsETCBlockCompressedPixelFormat(CompressedMips[0].PixelFormat);
	if (IsBlockCompressedFormat(CompressedMips[0].PixelFormat)) // checks for BCn
	{
		// On DX we must have at least 4 px and have the top mip be %4=0
		if (InLODBias >= CompressedMips.Num())
		{
			UE_LOGF(LogTexture, Error, "LODBias in DecodeImageIfNeeded exceeds mip count! %d vs %d", InLODBias, CompressedMips.Num());
			return false;
		}

		if (CompressedMips[InLODBias].SizeX % 4 ||
			CompressedMips[InLODBias].SizeY % 4)
		{
			UE_LOGF(LogTexture, Verbose, "Texture %ls needs decoding because of DX block dimension restriction: LODBias %d, Size %dx%d", *BaseTextureFormatName.ToString(), InLODBias, CompressedMips[InLODBias].SizeX, CompressedMips[InLODBias].SizeY);
			bNeedsDecode = true;
		}
	}

	if (!bNeedsDecode)
	{
		return true;
	}

	const ITextureFormat* BaseTextureFormat = GetTextureFormatManager()->FindTextureFormat(BaseTextureFormatName);
	if (!BaseTextureFormat->CanDecodeFormat(CompressedMips[0].PixelFormat))
	{
		UE_LOGF(LogTexture, Error, "Unable to decode texture format %ls / pixel format %ls for PC - texture %ls", *BaseTextureFormatName.ToString(), GetPixelFormatString(CompressedMips[0].PixelFormat), *TexturePathName);
		return false;
	}

	for (FCompressedImage2D& Mip : CompressedMips)
	{
		FSharedBuffer MipData = MakeSharedBufferFromArray(MoveTemp(Mip.RawData));
		FImage DecodedImage;
		if (!BaseTextureFormat->DecodeImage(Mip.SizeX, Mip.SizeY, Mip.NumSlicesWithDepth, Mip.PixelFormat, bSRGB, BaseTextureFormatName, MipData, DecodedImage, TexturePathName))
		{
			UE_LOGF(LogTexture, Error, "DecodeImage failed for format %ls / pixel format %ls - texture %ls", *BaseTextureFormatName.ToString(), GetPixelFormatString(CompressedMips[0].PixelFormat), *TexturePathName);
			return false;
		}

		ERawImageFormat::Type NeededConversion;
		Mip.PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(DecodedImage.Format, &NeededConversion);
		if (NeededConversion != DecodedImage.Format)
		{
			FImage ConvertedImage;
			DecodedImage.CopyTo(ConvertedImage, NeededConversion, DecodedImage.GammaSpace);
			Mip.RawData = MoveTemp(ConvertedImage.RawData);
		}
		else
		{
			Mip.RawData = MoveTemp(DecodedImage.RawData);
		}
	}

	return true;
}


// Synchronous DDC1 texture build function
static void DDC1_BuildTexture(
	ITextureCompressorModule* Compressor,
	IImageWrapperModule* ImageWrapper,
	const UTexture& InTexture,
	const FString& TexturePathName,
	ETextureCacheFlags CacheFlags,
	FTextureSourceData& TextureData,
	FTextureSourceData& CompositeTextureData,
	const TArrayView<FTextureBuildSettings>& InBuildSettingsPerLayer,
	const FTexturePlatformData::FTextureEncodeResultMetadata& InBuildResultMetadata,

	const FString& KeySuffix,
	bool bReplaceExistingDDC,
	int64 RequiredMemoryEstimate,

	FTexturePlatformData* DerivedData,
	int64& BytesCached,
	bool& bSucceeded
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCacheDerivedDataWorker::BuildTexture);

	const bool bHasValidMip0 = TextureData.Blocks.Num() && TextureData.Blocks[0].MipsPerLayer.Num() && TextureData.Blocks[0].MipsPerLayer[0].Num();
	const bool bForVirtualTextureStreamingBuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForVirtualTextureStreamingBuild);

	check( bSucceeded == false ); // we set it to true if we succeed

	if (!ensure(Compressor))
	{
		UE_LOGF(LogTexture, Warning, "Missing Compressor required to build texture %ls", *TexturePathName);
		return;
	}

	if (!bHasValidMip0)
	{
		return;
	}

	ETextureClass TextureClass = InTexture.GetTextureClass();

	// this logs the "Building textures: " message :
	FTextureStatusMessageContext StatusMessage(
		ComposeTextureBuildText(TextureClass, TexturePathName, TextureData, InBuildSettingsPerLayer[0], (ETextureEncodeSpeed)InBuildSettingsPerLayer[0].RepresentsEncodeSpeedNoSend, RequiredMemoryEstimate, bForVirtualTextureStreamingBuild)
		);

	DerivedData->Reset();

	if (bForVirtualTextureStreamingBuild)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT);

		if (DerivedData->VTData == nullptr)
		{
			DerivedData->VTData = new FVirtualTextureBuiltData();
		}
		
		FVirtualTextureBuilderDerivedInfo PredictedInfo;
		if ( ! PredictedInfo.InitializeFromBuildSettings(TextureData, InBuildSettingsPerLayer.GetData()) )
		{
			UE_LOGF(LogTexture, Warning, "VT InitializeFromBuildSettings failed: %ls", *TexturePathName);
			delete DerivedData->VTData;
			DerivedData->VTData = nullptr;
			bSucceeded = false;
			return;		
		}

		FVirtualTextureDataBuilder Builder(*DerivedData->VTData, TexturePathName, Compressor, ImageWrapper);
		if ( ! Builder.Build(TextureData, CompositeTextureData, &InBuildSettingsPerLayer[0], true) )
		{
			delete DerivedData->VTData;
			DerivedData->VTData = nullptr;
			bSucceeded = false;

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return;
			}

			UE_LOGF(LogTexture, Warning, "VT Build failed: %ls", *TexturePathName);
			return;		
		}

		// TextureData was freed by Build (FTextureSourceData.ReleaseMemory), don't use it from here down

		DerivedData->SizeX = DerivedData->VTData->Width;
		DerivedData->SizeY = DerivedData->VTData->Height;
		DerivedData->PixelFormat = DerivedData->VTData->LayerTypes[0];
		DerivedData->SetNumSlices(1);
		DerivedData->ResultMetadata = InBuildResultMetadata;

		// Verify our predicted count matches.
		check(PredictedInfo.NumMips == DerivedData->VTData->GetNumMips());

		bool bCompressionValid = true;
		if (CVarVTValidateCompressionOnSave.GetValueOnAnyThread())
		{
			bCompressionValid = DerivedData->VTData->ValidateData(TexturePathName, true);
		}

		if (ensureMsgf(bCompressionValid, TEXT("Corrupt Virtual Texture compression for %s, can't store to DDC"), *TexturePathName))
		{
			// Store it in the cache.
			// @todo: This will remove the streaming bulk data, which we immediately reload below!
			// Should ideally avoid this redundant work, but it only happens when we actually have 
			// to build the texture, which should only ever be once.
			BytesCached = PutDerivedDataInCache(DerivedData, KeySuffix, TexturePathName, InBuildSettingsPerLayer[0].bCubemap || InBuildSettingsPerLayer[0].bVolume || InBuildSettingsPerLayer[0].bTextureArray, bReplaceExistingDDC);

			if (DerivedData->VTData->Chunks.Num())
			{
				const bool bInlineMips = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::InlineMips);
				bSucceeded = !bInlineMips || DerivedData->TryInlineMipData(InBuildSettingsPerLayer[0].LODBiasWithCinematicMips, TexturePathName);
				if (!bSucceeded)
				{
					UE_LOGF(LogTexture, Display, "Failed to put and then read back mipmap data from DDC for %ls", *TexturePathName);
				}
			}
			else
			{
				UE_LOGF(LogTexture, Warning, "Failed to build %ls derived data for %ls", *InBuildSettingsPerLayer[0].TextureFormatName.GetPlainNameString(), *TexturePathName);
			}
		}
	}
	else
	{
		// Only support single Block/Layer here (Blocks and Layers are intended for VT support)
		if (TextureData.Blocks.Num() > 1)
		{
			// This can happen if user attempts to import a UDIM without VT enabled
			UE_LOGF(LogTexture, Log, "Texture %ls was imported as UDIM with %d blocks but VirtualTexturing is not enabled, only the first block will be available",
				*TexturePathName, TextureData.Blocks.Num());
		}
		if (TextureData.Layers.Num() > 1)
		{
			// This can happen if user attempts to use lightmaps or other layered VT without VT enabled
			UE_LOGF(LogTexture, Log, "Texture %ls has %d layers but VirtualTexturing is not enabled, only the first layer will be available",
				*TexturePathName, TextureData.Layers.Num());
		}

		if (InBuildSettingsPerLayer[0].bCPUAccessible)
		{
			// Copy out the unaltered top mip for cpu access.
			FSharedImage* CPUCopy = new FSharedImage();
			TextureData.Blocks[0].MipsPerLayer[0][0].CopyTo(*CPUCopy);

			DerivedData->CPUCopy = FSharedImageConstRef(CPUCopy);
			DerivedData->SetHasCpuCopy(true);
			
			// Divert the texture source data to a tiny placeholder texture.
			TextureData.InitAsPlaceholder();
		}

		uint32 NumMipsInTail;
		uint32 ExtData;
		
		TArray<FImage> EmptyImageArray;
		TArray<FImage> & CompositeImageArray = (InTexture.GetCompositeTexture() != nullptr && CompositeTextureData.Blocks.Num() && CompositeTextureData.Blocks[0].MipsPerLayer.Num()) ? CompositeTextureData.Blocks[0].MipsPerLayer[0] : EmptyImageArray;

		// Compress the texture by calling texture compressor directly.
		TArray<FCompressedImage2D> CompressedMips;

		bSucceeded = Compressor->BuildTexture(TextureData.Blocks[0].MipsPerLayer[0],
			CompositeImageArray,
			InBuildSettingsPerLayer[0],
			TexturePathName,
			CompressedMips,
			NumMipsInTail,
			ExtData,
			nullptr); // OutMetadata
			
		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return;
		}

		if ( bSucceeded )
		{
			// BuildTexture can free the source images passed to it
			//	so TextureData is invalid after this call
			TextureData.ReleaseMemory();
			CompositeTextureData.ReleaseMemory();;

			if (InBuildSettingsPerLayer[0].bDecodeForPCUsage)
			{
				// If we have shared linear on, then we handle detiling and decoding elsewhere.
				if (!InBuildSettingsPerLayer[0].Tiler)
				{
					UE_LOGF(LogTexture, Display, "Decoding for PC...");

					// The tiler knows how to detile - if there's no tiler ever, then even if it is tiled we don't
					// know what to do about it.
					const ITextureTiler* Tiler = InBuildSettingsPerLayer[0].TilerEvenIfNotSharedLinear;
					if (Tiler)
					{
						FEncodedTextureDescription TextureDescription;
						InBuildSettingsPerLayer[0].GetEncodedTextureDescriptionWithPixelFormat(
							&TextureDescription, CompressedMips[0].PixelFormat, CompressedMips[0].SizeX, CompressedMips[0].SizeY, CompressedMips[0].NumSlicesWithDepth, CompressedMips.Num());

						FEncodedTextureExtendedData ExtendedData = Tiler->GetExtendedDataForTexture(TextureDescription, InBuildSettingsPerLayer[0].LODBias);

						// massaging data representations, sigh.
						FEncodedTextureDescription::FSharedBufferMipChain TiledMips;
						for (FCompressedImage2D& Image : CompressedMips)
						{
							TiledMips.Add(MakeSharedBufferFromArray(MoveTemp(Image.RawData)));
						}

						FEncodedTextureDescription::FUniqueBufferMipChain LinearMips;
						if (!Tiler->DetileMipChain(LinearMips, TiledMips, TextureDescription, ExtendedData, TexturePathName))
						{
							bSucceeded = false;
						}

						if (bSucceeded &&
							LinearMips.Num() == CompressedMips.Num())
						{
							for (int32 Mip = 0; Mip < CompressedMips.Num(); Mip++)
							{
								// No way to move from unique buffer to array
								CompressedMips[Mip].RawData.SetNumUninitialized(LinearMips[Mip].GetSize());
								FMemory::Memcpy(CompressedMips[Mip].RawData.GetData(), LinearMips[Mip].GetData(), LinearMips[Mip].GetSize());
								LinearMips[Mip].Reset();
							}
						}
						else
						{
							bSucceeded = false;
						}
					}

					// If the format can't be viewed on a PC we need to decode it to something that can.
					if (bSucceeded)
					{
						bSucceeded = DDC1_DecodeImageIfNeeded(InBuildSettingsPerLayer[0].BaseTextureFormatName, InBuildSettingsPerLayer[0].bSRGB, InBuildSettingsPerLayer[0].LODBias, CompressedMips, TexturePathName);
					}
				} // end if need to detile here.
			} // end if decoding for PC

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return;
			}

			if (bSucceeded)
			{
				check(CompressedMips.Num());

				DDC1_StoreClassicTextureInDerivedData(
					CompressedMips, DerivedData, InBuildSettingsPerLayer[0].bVolume, InBuildSettingsPerLayer[0].bTextureArray, InBuildSettingsPerLayer[0].bCubemap, 
					NumMipsInTail, ExtData, bReplaceExistingDDC, TexturePathName, KeySuffix, BytesCached);

				DerivedData->ResultMetadata = InBuildResultMetadata;

				const bool bInlineMips = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::InlineMips);
				if (bInlineMips) // Note that mips are inlined when cooking.
				{
					bSucceeded = DerivedData->TryInlineMipData(InBuildSettingsPerLayer[0].LODBiasWithCinematicMips, TexturePathName);
					if (bSucceeded == false)
					{
						// This should only ever happen with DDC issues - it can technically be a transient issue if you lose connection
						// in the middle of a build, but with a stable connection it's probably a ddc bug.
						UE_LOGF(LogTexture, Warning, "Failed to put and then read back mipmap data from DDC for %ls", *TexturePathName);
					}
				}
			}
		}
		else
		{
			// BuildTexture failed
			// will log below
			check( DerivedData->Mips.Num() == 0 );
			DerivedData->Mips.Empty();

			if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
			{
				return;
			}

			UE_LOGF(LogTexture, Warning, "BuildTexture failed to build %ls derived data for %ls", *InBuildSettingsPerLayer[0].TextureFormatName.GetPlainNameString(), *TexturePathName);
		}
	}
}

static int64 GetBuildRequiredMemoryEstimate(UTexture* InTexture, const FTextureBuildSettings* InBuildSettingsPerLayer)
{
	// Thunk to our computation functions that don't rely on Texture.h.

	const FTextureSource & Source = InTexture->Source;

	if (InBuildSettingsPerLayer[0].bVirtualStreamable)
	{
		TArray<ERawImageFormat::Type, TInlineAllocator<1>> LayerFormats;
		LayerFormats.AddZeroed(Source.GetNumLayers());
		for (int32 LayerIndex = 0; LayerIndex < Source.GetNumLayers(); LayerIndex++)
		{
			LayerFormats[LayerIndex] = FImageCoreUtils::ConvertToRawImageFormat(Source.GetFormat(LayerIndex));
		}

		TArray<UE::TextureBuildUtilities::FVirtualTextureSourceBlockInfo, TInlineAllocator<4>> Blocks;
		Blocks.AddDefaulted(Source.GetNumBlocks());
		for (int32 BlockIndex = 0; BlockIndex < Source.GetNumBlocks(); BlockIndex++)
		{
			FTextureSourceBlock Block;
			Source.GetBlock(BlockIndex, Block);

			Blocks[BlockIndex].BlockX = Block.BlockX;
			Blocks[BlockIndex].BlockY = Block.BlockY;
			Blocks[BlockIndex].SizeX = Block.SizeX;
			Blocks[BlockIndex].SizeY = Block.SizeY;
			Blocks[BlockIndex].NumSlices = Block.NumSlices;
			Blocks[BlockIndex].NumMips = Block.NumMips;
		}

		return UE::TextureBuildUtilities::GetVirtualTextureRequiredMemoryEstimate(InBuildSettingsPerLayer, LayerFormats, Blocks);
	}
	else
	{
		// non VT
		FImageInfo Mip0Info;
		Source.GetMipImageInfo(Mip0Info, 0, 0, 0);
		return UE::TextureBuildUtilities::GetPhysicalTextureBuildMemoryEstimate(InBuildSettingsPerLayer, Mip0Info, Source.GetNumMips());
	}
}

FTextureCacheDerivedDataWorker::FTextureCacheDerivedDataWorker(
	ITextureCompressorModule* InCompressor,
	FTexturePlatformData* InDerivedData,
	UTexture* InTexture,
	const FTextureBuildSettings* InSettingsPerLayerFetchFirst,
	const FTextureBuildSettings* InSettingsPerLayerFetchOrBuild,
	const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchFirstMetadata, // can be nullptr if not needed
	const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchOrBuildMetadata, // can be nullptr if not needed
	ETextureCacheFlags InCacheFlags
	)
	: Compressor(InCompressor)
	, ImageWrapper(nullptr)
	, DerivedData(InDerivedData)
	, Texture(*InTexture)
	, TexturePathName(InTexture->GetPathName())
	, CacheFlags(InCacheFlags)
	, bSucceeded(false)
{
	check(DerivedData);
	
	// if ! InTexture->Source.IsValid() -> fail now ?

	RequiredMemoryEstimate = GetBuildRequiredMemoryEstimate(InTexture,InSettingsPerLayerFetchOrBuild);

	if (InSettingsPerLayerFetchFirst)
	{
		BuildSettingsPerLayerFetchFirst.SetNum(InTexture->Source.GetNumLayers());
		for (int32 LayerIndex = 0; LayerIndex < BuildSettingsPerLayerFetchFirst.Num(); ++LayerIndex)
		{
			BuildSettingsPerLayerFetchFirst[LayerIndex] = InSettingsPerLayerFetchFirst[LayerIndex];
		}
		if (InFetchFirstMetadata)
		{
			FetchFirstMetadata = *InFetchFirstMetadata;
		}
	}
	
	BuildSettingsPerLayerFetchOrBuild.SetNum(InTexture->Source.GetNumLayers());
	for (int32 LayerIndex = 0; LayerIndex < BuildSettingsPerLayerFetchOrBuild.Num(); ++LayerIndex)
	{
		BuildSettingsPerLayerFetchOrBuild[LayerIndex] = InSettingsPerLayerFetchOrBuild[LayerIndex];
	}
	if (InFetchOrBuildMetadata)
	{
		FetchOrBuildMetadata = *InFetchOrBuildMetadata;
	}

	// Keys need to be assigned on the create thread.
	{
		FString LocalKeySuffix;
		GetTextureDerivedDataKeySuffix(Texture, BuildSettingsPerLayerFetchOrBuild.GetData(), LocalKeySuffix);
		FString DDK;
		GetTextureDerivedDataKeyFromSuffix(LocalKeySuffix, DDK);
		InDerivedData->FetchOrBuildDerivedDataKey.Emplace<FString>(DDK);
	}
	if (BuildSettingsPerLayerFetchFirst.Num())
	{
		FString LocalKeySuffix;
		GetTextureDerivedDataKeySuffix(Texture, BuildSettingsPerLayerFetchFirst.GetData(), LocalKeySuffix);
		FString DDK;
		GetTextureDerivedDataKeyFromSuffix(LocalKeySuffix, DDK);
		InDerivedData->FetchFirstDerivedDataKey.Emplace<FString>(DDK);
	}

	// At this point, the texture *MUST* have a valid GUID.
	if (!Texture.Source.GetId().IsValid())
	{
		UE_LOGF(LogTexture, Warning, "Building texture with an invalid GUID: %ls", *TexturePathName);
		Texture.Source.ForceGenerateGuid();
	}
	check(Texture.Source.GetId().IsValid());

	// Dump any existing mips.
	DerivedData->Reset();
	UTexture::GetPixelFormatEnum();
		
	const bool bAllowAsyncLoading = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::AllowAsyncLoading);
	const bool bForVirtualTextureStreamingBuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForVirtualTextureStreamingBuild);

	// FVirtualTextureDataBuilder always wants to load ImageWrapper module
	// This is not strictly necessary, used only for debug output, but seems simpler to just always load this here, doesn't seem like it should be too expensive
	if (bAllowAsyncLoading || bForVirtualTextureStreamingBuild)
	{
		ImageWrapper = &FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );
	}

	// All of these settings are fixed across build settings and are derived directly from the texture.
	// So we can just use layer 0 of whatever we have.
	const FTextureBuildSettings& BuildSettings = BuildSettingsPerLayerFetchOrBuild[0];
	TextureData.Init(Texture,
		(TextureMipGenSettings)BuildSettings.MipGenSettings,
		BuildSettings.bCubemap,
		BuildSettings.bTextureArray,
		BuildSettings.bVolume,
		(ETexturePowerOfTwoSetting::Type)BuildSettings.PowerOfTwoMode,
		BuildSettings.ResizeDuringBuildX,
		BuildSettings.ResizeDuringBuildY,
		bAllowAsyncLoading);

	bool bNeedsCompositeData = Texture.GetCompositeTexture() && Texture.CompositeTextureMode != CTM_Disabled && Texture.GetCompositeTexture()->Source.IsValid();
	if (BuildSettings.bCPUAccessible)
	{
		// CPU accessible textures don't run image processing and thus don't need the composite data.
		bNeedsCompositeData = false;
	}

	if (bNeedsCompositeData)
	{
		bool bMatchingBlocks = Texture.GetCompositeTexture()->Source.GetNumBlocks() == Texture.Source.GetNumBlocks();
		
		if (!bMatchingBlocks)
		{
			UE_LOGF(LogTexture, Warning, "Issue while building %ls : Composite texture UDIM Block counts do not match. Composite texture will be ignored", *TexturePathName);
			// note: does not fail, fill not warn again
		}

		if ( bMatchingBlocks )
		{
			CompositeTextureData.Init(*Texture.GetCompositeTexture(),
				(TextureMipGenSettings)BuildSettings.MipGenSettings,
				BuildSettings.bCubemap,
				BuildSettings.bTextureArray,
				BuildSettings.bVolume,
				(ETexturePowerOfTwoSetting::Type)BuildSettings.PowerOfTwoMode,
				BuildSettings.ResizeDuringBuildX,
				BuildSettings.ResizeDuringBuildY,
				bAllowAsyncLoading);
		}
	}
}

// Currently only used for prefetching (pulling data down from shared ddc to local ddc).
static bool TryCacheStreamingMips(const FString& TexturePathName, int32 FirstMipToLoad, int32 FirstMipToPrefetch, FTexturePlatformData* DerivedData)
{
	using namespace UE;
	using namespace UE::DerivedData;
	check(DerivedData->DerivedDataKey.IsType<FString>());

	TArray<FCacheGetValueRequest, TInlineAllocator<16>> MipRequests;

	const int32 LowestMipIndexToPrefetchOrLoad = FMath::Min(FirstMipToPrefetch, FirstMipToLoad);
	const int32 NumMips = DerivedData->Mips.Num();
	const FSharedString Name(WriteToString<256>(TexturePathName, TEXTVIEW(" [Prefetch]")));
	for (int32 MipIndex = LowestMipIndexToPrefetchOrLoad; MipIndex < NumMips; ++MipIndex)
	{
		const FTexture2DMipMap& Mip = DerivedData->Mips[MipIndex];
		if (Mip.IsPagedToDerivedData())
		{
			const FCacheKey MipKey = ConvertLegacyCacheKey(DerivedData->GetDerivedDataMipKeyString(MipIndex, Mip));
			const ECachePolicy Policy
				= (MipIndex >= FirstMipToLoad) ? ECachePolicy::Default
				: (MipIndex >= FirstMipToPrefetch) ? ECachePolicy::Default | ECachePolicy::SkipData
				: ECachePolicy::Query | ECachePolicy::SkipData;
			MipRequests.Add({Name, MipKey, Policy, uint64(MipIndex)});
		}
	}

	if (MipRequests.IsEmpty())
	{
		return true;
	}

	bool bOk = true;
	FRequestOwner BlockingOwner(EPriority::Blocking);
	GetCache().GetValue(MipRequests, BlockingOwner, [DerivedData, &bOk](FCacheGetValueResponse&& Response)
	{
		bOk &= Response.Status == EStatus::Ok;
		if (const FSharedBuffer MipBuffer = Response.Value.GetData().Decompress())
		{
			FTexture2DMipMap& Mip = DerivedData->Mips[int32(Response.UserData)];
			Mip.BulkData.Lock(LOCK_READ_WRITE);
			void* MipData = Mip.BulkData.Realloc(int64(MipBuffer.GetSize()));
			FMemory::Memcpy(MipData, MipBuffer.GetData(), MipBuffer.GetSize());
			Mip.BulkData.Unlock();
		}
	});
	BlockingOwner.Wait();
	return bOk;
}

static void DDC1_FetchAndFillDerivedData(
	/* inputs */
	const UTexture& Texture,
	const FString& TexturePathName,
	ETextureCacheFlags CacheFlags,
	const TArrayView<FTextureBuildSettings>& BuildSettingsPerLayerFetchFirst,
	const FTexturePlatformData::FTextureEncodeResultMetadata& FetchFirstMetadata,

	const TArrayView<FTextureBuildSettings>& BuildSettingsPerLayerFetchOrBuild,
	const FTexturePlatformData::FTextureEncodeResultMetadata& FetchOrBuildMetadata,

	/* outputs */
	FTexturePlatformData* DerivedData,
	FString& KeySuffix,
	bool& bSucceeded,
	bool& bInvalidVirtualTextureCompression,
	int64& BytesCached
	)
{
	using namespace UE;
	using namespace UE::DerivedData;

	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.DDC1_FetchAndFillDerivedData);

	bool bForceRebuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForceRebuild);
	FString FetchOrBuildKeySuffix;
	GetTextureDerivedDataKeySuffix(Texture, BuildSettingsPerLayerFetchOrBuild.GetData(), FetchOrBuildKeySuffix);

	if (bForceRebuild)
	{
		// If we know we are rebuilding, don't touch the cache.
		bSucceeded = false;
		bInvalidVirtualTextureCompression = false;
		KeySuffix = MoveTemp(FetchOrBuildKeySuffix);

		FString FetchOrBuildKey;
		GetTextureDerivedDataKeyFromSuffix(KeySuffix, FetchOrBuildKey);
		DerivedData->DerivedDataKey.Emplace<FString>(MoveTemp(FetchOrBuildKey));
		DerivedData->ResultMetadata = FetchOrBuildMetadata;
		return;
	}
		
	bool bForVirtualTextureStreamingBuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForVirtualTextureStreamingBuild);

	FSharedBuffer RawDerivedData;
	const FSharedString SharedTexturePathName(TexturePathName);
	const FSharedString SharedTextureFastPathName(WriteToString<256>(TexturePathName, TEXTVIEW(" [Fast]")));

	FString LocalDerivedDataKeySuffix;
	FString LocalDerivedDataKey;

	bool bGotDDCData = false;
	bool bUsedFetchFirst = false;
	if (BuildSettingsPerLayerFetchFirst.Num() && !bForceRebuild)
	{
		FString FetchFirstKeySuffix;
		GetTextureDerivedDataKeySuffix(Texture, BuildSettingsPerLayerFetchFirst.GetData(), FetchFirstKeySuffix);

		// If the suffixes are the same, then use fetchorbuild to avoid a get()
		if (FetchFirstKeySuffix != FetchOrBuildKeySuffix)
		{
			FString FetchFirstKey;
			GetTextureDerivedDataKeyFromSuffix(FetchFirstKeySuffix, FetchFirstKey);

			TArray<FCacheGetValueRequest, TInlineAllocator<1>> Requests;
			const FSharedString& TexturePathRequestName = (FetchFirstMetadata.EncodeSpeed == (uint8) ETextureEncodeSpeed::Fast) ? SharedTextureFastPathName : SharedTexturePathName;
			Requests.Add({ TexturePathRequestName, ConvertLegacyCacheKey(FetchFirstKey), ECachePolicy::Default, 0 /* UserData */});

			FRequestOwner BlockingOwner(EPriority::Blocking);

			GetCache().GetValue(Requests, BlockingOwner, [&RawDerivedData](FCacheGetValueResponse&& Response)
			{
				if (Response.UserData == 0)
				{
					RawDerivedData = Response.Value.GetData().Decompress();
				}
			});
			BlockingOwner.Wait();

			bGotDDCData = !RawDerivedData.IsNull();
			if (bGotDDCData)
			{
				bUsedFetchFirst = true;
				LocalDerivedDataKey = MoveTemp(FetchFirstKey);
				LocalDerivedDataKeySuffix = MoveTemp(FetchFirstKeySuffix);
			}
		}
	}

	if (bGotDDCData == false)
	{
		// Didn't get the initial fetch, so we're using fetch/build.
		LocalDerivedDataKeySuffix = MoveTemp(FetchOrBuildKeySuffix);
		GetTextureDerivedDataKeyFromSuffix(LocalDerivedDataKeySuffix, LocalDerivedDataKey);

		TArray<FCacheGetValueRequest, TInlineAllocator<1>> Requests;
		const FSharedString& TexturePathRequestName = (FetchOrBuildMetadata.EncodeSpeed == (uint8) ETextureEncodeSpeed::Fast) ? SharedTextureFastPathName : SharedTexturePathName;
		Requests.Add({ TexturePathRequestName, ConvertLegacyCacheKey(LocalDerivedDataKey), ECachePolicy::Default, 0 /* UserData */ });

		FRequestOwner BlockingOwner(EPriority::Blocking);

		GetCache().GetValue(Requests, BlockingOwner, [&RawDerivedData](FCacheGetValueResponse&& Response)
		{
			if (Response.UserData == 0)
			{
				RawDerivedData = Response.Value.GetData().Decompress();
			}
		});
		BlockingOwner.Wait();

		bGotDDCData = !RawDerivedData.IsNull();
	}

	KeySuffix = LocalDerivedDataKeySuffix;
	DerivedData->DerivedDataKey.Emplace<FString>(LocalDerivedDataKey);
	DerivedData->ResultMetadata = bUsedFetchFirst ? FetchFirstMetadata : FetchOrBuildMetadata;

	if (bGotDDCData)
	{
		const bool bInlineMips = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::InlineMips);
		const bool bForDDC = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForDDCBuild);
		int32 FirstResidentMipIndex = 0;

		BytesCached = RawDerivedData.GetSize();
		FMemoryReaderView Ar(RawDerivedData.GetView(), /*bIsPersistent=*/ true);
		DerivedData->Serialize(Ar, NULL);
		bSucceeded = true;

		if (bForVirtualTextureStreamingBuild)
		{
			if (DerivedData->VTData && DerivedData->VTData->IsInitialized())
			{
				const FSharedString Name(TexturePathName);
				for (FVirtualTextureDataChunk& Chunk : DerivedData->VTData->Chunks)
				{
					if (!Chunk.DerivedDataKey.IsEmpty())
					{
						Chunk.DerivedData = FDerivedData(Name, ConvertLegacyCacheKey(Chunk.DerivedDataKey));
					}
				}
			}
		}
		else
		{
			if (Algo::AnyOf(DerivedData->Mips, [](const FTexture2DMipMap& Mip) { return !Mip.BulkData.IsBulkDataLoaded(); }))
			{
				int32 MipIndex = 0;
				FirstResidentMipIndex = DerivedData->Mips.Num();
				const FSharedString Name(TexturePathName);
				for (FTexture2DMipMap& Mip : DerivedData->Mips)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS;
					const bool bPagedToDerivedData = Mip.bPagedToDerivedData;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS;
					if (bPagedToDerivedData)
					{
						Mip.DerivedData = FDerivedData(Name, ConvertLegacyCacheKey(DerivedData->GetDerivedDataMipKeyString(MipIndex, Mip)));
					}
					else
					{
						FirstResidentMipIndex = FMath::Min(FirstResidentMipIndex, MipIndex);
					}
					++MipIndex;
				}
			}
		}

		// Load any streaming (not inline) mips that are necessary for our platform.
		if (bForDDC)
		{
			bSucceeded = DerivedData->TryLoadMips(0, nullptr, TexturePathName);

			if (bForVirtualTextureStreamingBuild)
			{
				if (DerivedData->VTData != nullptr &&
					DerivedData->VTData->IsInitialized())
				{
					FCacheGetValueRequest Request;
					Request.Name = TexturePathName;
					Request.Policy = ECachePolicy::Default | ECachePolicy::SkipData;

					TArray<FCacheGetValueRequest, TInlineAllocator<16>> ChunkKeys;
					for (const FVirtualTextureDataChunk& Chunk : DerivedData->VTData->Chunks)
					{
						if (!Chunk.DerivedDataKey.IsEmpty())
						{
							ChunkKeys.Add_GetRef(Request).Key = ConvertLegacyCacheKey(Chunk.DerivedDataKey);
						}
					}

					FRequestOwner BlockingOwner(EPriority::Blocking);
					GetCache().GetValue(ChunkKeys, BlockingOwner, [](FCacheGetValueResponse&&){});
					BlockingOwner.Wait();
				}
			}

			if (!bSucceeded)
			{
				UE_LOGF(LogTexture, Display, "Texture %ls is missing mips. The texture will be rebuilt.", *TexturePathName);
			}
		}
		else if (bInlineMips)
		{
			bSucceeded = DerivedData->TryInlineMipData(BuildSettingsPerLayerFetchOrBuild[0].LODBiasWithCinematicMips, TexturePathName);

			if (!bSucceeded)
			{
				UE_LOGF(LogTexture, Display, "Texture %ls is missing streaming mips when loading for an inline request. The texture will be rebuilt.", *TexturePathName);
			}
			else
			{
				// The mips we didn't inline could be under different keys. Until we migrate over to ddc cache records, we have to manually
				// touch them otherwise they could get GC'd in the ddc after a while. We expect to fail this some times for mips that have
				// already been GC'd, and we do NOT want to force a rebuild with them at this time as that will surface any latent 
				// determinism
				if (BuildSettingsPerLayerFetchOrBuild[0].LODBiasWithCinematicMips)
				{
					DerivedData->TouchStreamingMipDDCData(BuildSettingsPerLayerFetchOrBuild[0].LODBiasWithCinematicMips, TexturePathName);
				}
			}
		}
		else
		{
			if (bForVirtualTextureStreamingBuild)
			{
				bSucceeded = DerivedData->VTData != nullptr &&
					DerivedData->VTData->IsInitialized() &&
					DerivedData->AreDerivedVTChunksAvailable(TexturePathName);

				if (!bSucceeded)
				{
					UE_LOGF(LogTexture, Display, "Texture %ls is missing VT Chunks. The texture will be rebuilt.", *TexturePathName);
				}
			}
			else
			{
				const bool bDisableStreaming = ! Texture.IsPossibleToStream();
				const int32 FirstMipToLoad = FirstResidentMipIndex;
				const int32 FirstNonStreamingMipIndex = DerivedData->Mips.Num() - DerivedData->GetNumNonStreamingMips(!bDisableStreaming);
				const int32 FirstMipToPrefetch = IsInGameThread() ? FirstMipToLoad : bDisableStreaming ? 0 : FirstNonStreamingMipIndex;
				bSucceeded = TryCacheStreamingMips(TexturePathName, FirstMipToLoad, FirstMipToPrefetch, DerivedData);
				if (!bSucceeded)
				{
					UE_LOGF(LogTexture, Display, "Texture %ls is missing derived mips. The texture will be rebuilt.", *TexturePathName);
				}
			}
		}

		if (bSucceeded && bForVirtualTextureStreamingBuild && CVarVTValidateCompressionOnLoad.GetValueOnAnyThread())
		{
			check(DerivedData->VTData);
			bSucceeded = DerivedData->VTData->ValidateData(TexturePathName, false);
			if (!bSucceeded)
			{
				UE_LOGF(LogTexture, Display, "Texture %ls has invalid cached VT data. The texture will be rebuilt.", *TexturePathName);
				bInvalidVirtualTextureCompression = true;
			}
		}
		
		// Reset everything derived data so that we can do a clean load from the source data
		if (!bSucceeded)
		{
			DerivedData->Mips.Empty();
			if (DerivedData->VTData)
			{
				delete DerivedData->VTData;
				DerivedData->VTData = nullptr;
			}
		}
	}

}

static bool DDC1_IsTextureDataValid(const FTextureSourceData& TextureData, const FTextureSourceData& CompositeTextureData)
{
	bool bTextureDataValid = TextureData.Blocks.Num() && TextureData.Blocks[0].MipsPerLayer.Num() && TextureData.Blocks[0].MipsPerLayer[0].Num();
	if (CompositeTextureData.IsValid()) // Says IsValid, but means whether or not we _need_ composite texture data.
	{
		// here we know we _need_ composite texture data, so we actually check if the stuff we loaded is valid.
		bool bCompositeDataValid = CompositeTextureData.Blocks.Num() && CompositeTextureData.Blocks[0].MipsPerLayer.Num() && CompositeTextureData.Blocks[0].MipsPerLayer[0].Num();
		return bTextureDataValid && bCompositeDataValid;
	}
	return bTextureDataValid;
}

// Tries to get the source texture data resident for building the texture.
static bool DDC1_LoadAndValidateTextureData(
	UTexture& Texture,
	FTextureSourceData& TextureData,
	FTextureSourceData& CompositeTextureData,
	IImageWrapperModule* ImageWrapper
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.DDC1_LoadAndValidateTextureData);

	// There can be a stall here waiting on the BulkData mutex if it is serializing to the undo buffer on the main thread.

	bool bNeedsGetSourceMips = TextureData.IsValid() && Texture.Source.HasPayloadData();

	if ( bNeedsGetSourceMips )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetSourceMips);
		TextureData.GetSourceMips(Texture.Source, ImageWrapper);
	}

	if (CompositeTextureData.IsValid() && Texture.GetCompositeTexture() && Texture.GetCompositeTexture()->Source.HasPayloadData())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetCompositeSourceMips);
		check( Texture.GetCompositeTexture()->Source.IsValid() );
		CompositeTextureData.GetSourceMips(Texture.GetCompositeTexture()->Source, ImageWrapper);
	}

	return DDC1_IsTextureDataValid(TextureData, CompositeTextureData);
}


bool DDC1_BuildTiledClassicTexture(
	ITextureCompressorModule* Compressor,
	IImageWrapperModule* ImageWrapper,
	UTexture& Texture,
	const FString& TexturePathName,
	const TArrayView<FTextureBuildSettings> BuildSettingsPerLayerFetchFirst,
	const TArrayView<FTextureBuildSettings> BuildSettingsPerLayerFetchOrBuild,
	const FTexturePlatformData::FTextureEncodeResultMetadata& FetchFirstMetadata,
	const FTexturePlatformData::FTextureEncodeResultMetadata& FetchOrBuildMetadata,
	FTextureSourceData& TextureData,
	FTextureSourceData& CompositeTextureData,
	ETextureCacheFlags CacheFlags,
	int32 RequiredMemoryEstimate,
	const FString& KeySuffix,
	// outputs
	FTexturePlatformData* DerivedData,
	int64& BytesCached
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCacheDerivedDataWorker::BuildTiledClassicTexture);

	// we know we are a child format if we have a tiler.
	const ITextureTiler* Tiler = BuildSettingsPerLayerFetchOrBuild[0].Tiler;
	const FChildTextureFormat* ChildFormat = GetTextureFormatManager()->FindTextureFormat(BuildSettingsPerLayerFetchOrBuild[0].TextureFormatName)->GetChildFormat();

	// NonVT textures only have one layer.
	// We need to get the linear texture, which means we have to create the settings for it.
	TArray<FTextureBuildSettings, TInlineAllocator<1>> LinearSettingsPerLayerFetchFirst;
	if (BuildSettingsPerLayerFetchFirst.Num())
	{
		LinearSettingsPerLayerFetchFirst.Add(ChildFormat->GetBaseTextureBuildSettings(BuildSettingsPerLayerFetchFirst[0]));
	}

	TArray<FTextureBuildSettings, TInlineAllocator<1>> LinearSettingsPerLayerFetchOrBuild;
	LinearSettingsPerLayerFetchOrBuild.Add(ChildFormat->GetBaseTextureBuildSettings(BuildSettingsPerLayerFetchOrBuild[0]));

	// Now try and fetch.
	FTexturePlatformData LinearDerivedData;
	FString LinearKeySuffix;
	int64 LinearBytesCached = 0;
	bool bLinearDDCCorrupted = false;
	bool bLinearSucceeded = false;
	DDC1_FetchAndFillDerivedData(
		Texture, TexturePathName, CacheFlags,
		LinearSettingsPerLayerFetchFirst, FetchFirstMetadata, 
		LinearSettingsPerLayerFetchOrBuild, FetchOrBuildMetadata,
		&LinearDerivedData, LinearKeySuffix, bLinearSucceeded, bLinearDDCCorrupted, LinearBytesCached);

	BytesCached = LinearBytesCached;
	bool bHasLinearDerivedData = bLinearSucceeded;

	void* LinearMipData[MAX_TEXTURE_MIP_COUNT] = {};
	int64 LinearMipSizes[MAX_TEXTURE_MIP_COUNT];
	if (bHasLinearDerivedData)
	{
		// The linear bits are built - need to fetch
		if (LinearDerivedData.TryLoadMipsWithSizes(0, LinearMipData, LinearMipSizes, TexturePathName) == false)
		{
			// This can technically happen with a DDC failure and there is an expectation that we can recover and regenerate in such situations.
			// However, it should be very rare and most likely indicated a backend bug, so we still warn.
			UE_LOGF(LogTexture, Warning, "Tiling texture build was unable to load the linear texture mips after fetching, will try to build: %ls", *TexturePathName);
			bHasLinearDerivedData = false;
		}

	}

	if (bHasLinearDerivedData == false)
	{
		// Linear data didn't exist, need to build it.
		bool bGotSourceTextureData = DDC1_LoadAndValidateTextureData(Texture, TextureData, CompositeTextureData, ImageWrapper);
		if (bGotSourceTextureData)
		{
			// We know we want all the mips for tiling, so force inline
			ETextureCacheFlags LinearCacheFlags = CacheFlags;
			EnumAddFlags(LinearCacheFlags, ETextureCacheFlags::InlineMips);

			// Note that this will update the DDC with the linear texture if we end up building _before_ the linear platforms!
			DDC1_BuildTexture(
				Compressor,
				ImageWrapper,
				Texture,
				TexturePathName,
				CacheFlags,
				TextureData,
				CompositeTextureData,
				LinearSettingsPerLayerFetchOrBuild,
				FetchOrBuildMetadata,
				LinearKeySuffix,
				bLinearDDCCorrupted,
				RequiredMemoryEstimate,
				&LinearDerivedData,
				LinearBytesCached,
				bHasLinearDerivedData);

			// TextureData can be freed by Build, don't use it anymore :
			TextureData.ReleaseMemory();
			CompositeTextureData.ReleaseMemory();

			// This should succeed because we asked for inline mips if the build succeeded
			if (bHasLinearDerivedData && 
				LinearDerivedData.TryLoadMipsWithSizes(0, LinearMipData, LinearMipSizes, TexturePathName) == false)
			{
				UE_LOGF(LogTexture, Warning, "Tiling texture build was unable to load the linear texture mips after a successful build, bad bug!: %ls", *TexturePathName);
				return false;
			}
		}
	}

	if (bHasLinearDerivedData == false)
	{
		UE_LOGF(LogTexture, Warning, "Tiling texture build was unable to fetch or build the linear texture source: %ls", *TexturePathName);
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCacheDerivedDataWorker::TileTexture);

	check(LinearDerivedData.GetNumMipsInTail() == 0);

	// Have all the data - do some sanity checks as we convert to the metadata format the tiler expects.
	TArray<FMemoryView, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> InputTextureMipViews;
	FEncodedTextureDescription TextureDescription;
	FEncodedTextureExtendedData TextureExtendedData;
	int32 OutputTextureNumStreamingMips;
	{
		LinearSettingsPerLayerFetchOrBuild[0].GetEncodedTextureDescriptionWithPixelFormat(
			&TextureDescription,
			LinearDerivedData.PixelFormat, LinearDerivedData.Mips[0].SizeX, LinearDerivedData.Mips[0].SizeY, LinearDerivedData.GetNumSlices(), LinearDerivedData.Mips.Num());

		for (int32 MipIndex = 0; MipIndex < TextureDescription.NumMips; MipIndex++)
		{
			check(LinearMipSizes[MipIndex] == TextureDescription.GetMipSizeInBytes(MipIndex));
		}

		TextureExtendedData = Tiler->GetExtendedDataForTexture(TextureDescription, LinearSettingsPerLayerFetchOrBuild[0].LODBias);
		OutputTextureNumStreamingMips = TextureDescription.GetNumStreamingMips(&TextureExtendedData, GenerateTextureEngineParameters());

		for (int32 MipIndex = 0; MipIndex < TextureDescription.NumMips; MipIndex++)
		{
			InputTextureMipViews.Add(FMemoryView(LinearMipData[MipIndex], LinearMipSizes[MipIndex]));
		}
	}

	TArray<FCompressedImage2D> TiledMips;
	TiledMips.AddDefaulted(TextureDescription.NumMips);

	// If the platform packs mip tails, we need to pass all the relevant mip buffers at once.
	int32 MipTailIndex, MipsInTail;
	TextureDescription.GetEncodedMipIterators(&TextureExtendedData, MipTailIndex, MipsInTail);

	UE_LOGF(LogTexture, Display, "Tiling %ls", *TexturePathName);

	// Do the actual tiling.
	for (int32 EncodedMipIndex = 0; EncodedMipIndex < MipTailIndex + 1; EncodedMipIndex++)
	{
		int32 MipsRepresentedThisIndex = EncodedMipIndex == MipTailIndex ? MipsInTail : 1;

		TArrayView<FMemoryView> MipsForLevel = MakeArrayView(InputTextureMipViews.GetData() + EncodedMipIndex, MipsRepresentedThisIndex);

		FSharedBuffer MipData = Tiler->ProcessMipLevel(TextureDescription, TextureExtendedData, MipsForLevel, EncodedMipIndex);
		FIntVector3 MipDims = TextureDescription.GetMipDimensions(EncodedMipIndex);

		// Make sure we got the size we advertised prior to the build. If this ever fires then we
		// have a critical mismatch!
		check(TextureExtendedData.MipSizesInBytes[EncodedMipIndex] == MipData.GetSize());

		FCompressedImage2D& TiledMip = TiledMips[EncodedMipIndex];
		TiledMip.PixelFormat = LinearDerivedData.PixelFormat;
		TiledMip.SizeX = MipDims.X;
		TiledMip.SizeY = MipDims.Y;
		TiledMip.NumSlicesWithDepth = TextureDescription.GetNumSlices_WithDepth(EncodedMipIndex);

		// \todo try and Move this data rather than copying? We use FSharedBuffer as that's the future way,
		// but we're interacting with older systems that didn't have it, and we can't Move() from an FSharedBuffer.
		TiledMip.RawData.AddUninitialized(MipData.GetSize());
		FMemory::Memcpy(TiledMip.RawData.GetData(), MipData.GetData(), MipData.GetSize());
	} // end for each mip

	for (int32 MipIndex = 0; MipIndex < TextureDescription.NumMips; ++MipIndex)
	{
		FMemory::Free(LinearMipData[MipIndex]);
	}

	// The derived data expects to have mips (with no data) for the packed tail, if there is one
	for (int32 MipIndex = MipTailIndex + 1; MipIndex < TextureDescription.NumMips; ++MipIndex)
	{
		FCompressedImage2D& PrevMip = TiledMips[MipIndex - 1];
		FCompressedImage2D& DestMip = TiledMips[MipIndex];
		DestMip.SizeX = FMath::Max(1, PrevMip.SizeX >> 1);
		DestMip.SizeY = FMath::Max(1, PrevMip.SizeY >> 1);
		DestMip.NumSlicesWithDepth = TextureDescription.bVolumeTexture ? FMath::Max(1, PrevMip.NumSlicesWithDepth >> 1) : PrevMip.NumSlicesWithDepth;
		DestMip.PixelFormat = PrevMip.PixelFormat;
	}

	if (LinearSettingsPerLayerFetchOrBuild[0].bDecodeForPCUsage)
	{
		UE_LOGF(LogTexture, Display, "Decoding for PC...");

		FEncodedTextureDescription::FSharedBufferMipChain TiledMipBuffers;
		for (FCompressedImage2D& Image : TiledMips)
		{
			TiledMipBuffers.Add(MakeSharedBufferFromArray(MoveTemp(Image.RawData)));
		}

		FEncodedTextureDescription::FUniqueBufferMipChain LinearMips;
		if (!Tiler->DetileMipChain(LinearMips, TiledMipBuffers, TextureDescription, TextureExtendedData, TexturePathName))
		{
			return false;
		}

		if (LinearMips.Num() != TiledMips.Num())
		{
			return false;
		}

		for (int32 Mip = 0; Mip < TiledMips.Num(); Mip++)
		{
			// No way to move from unique buffer to array
			TiledMips[Mip].RawData.SetNumUninitialized(LinearMips[Mip].GetSize());
			FMemory::Memcpy(TiledMips[Mip].RawData.GetData(), LinearMips[Mip].GetData(), LinearMips[Mip].GetSize());
			LinearMips[Mip].Reset();
		}
		
		// When we detile our extended data no longer applies.
		TextureExtendedData = FEncodedTextureExtendedData();

		if (!DDC1_DecodeImageIfNeeded(LinearSettingsPerLayerFetchOrBuild[0].BaseTextureFormatName, 
			LinearSettingsPerLayerFetchOrBuild[0].bSRGB, LinearSettingsPerLayerFetchOrBuild[0].LODBias,
			TiledMips, TexturePathName))
		{
			return false;
		}
	}

	// We now have the final (tiled) data, and need to fill out the actual build output
	int64 TiledBytesCached;
	DDC1_StoreClassicTextureInDerivedData(TiledMips, DerivedData, TextureDescription.bVolumeTexture, TextureDescription.bTextureArray, TextureDescription.bCubeMap,
		TextureExtendedData.NumMipsInTail, TextureExtendedData.ExtData, false, TexturePathName, KeySuffix, TiledBytesCached);
	
	BytesCached += TiledBytesCached;

	// Do we need to reload streaming mips (evicted during DDC store)
	if (EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::InlineMips))
	{
		if (DerivedData->TryInlineMipData(LinearSettingsPerLayerFetchOrBuild[0].LODBiasWithCinematicMips, TexturePathName) == false)
		{
			UE_LOGF(LogTexture, Display, "Tiled texture build failed to put and then read back tiled mipmap data from DDC for %ls", *TexturePathName);
		}
	}

	return true;
}


// DDC1 primary fetch/build work function
void FTextureCacheDerivedDataWorker::DoWork()
{
	using namespace UE;
	using namespace UE::DerivedData;

	if (CancellationToken.IsCanceled())
	{
		return;
	}

	UE::Tasks::FCancellationTokenScope CancellationScope(CancellationToken);

	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCacheDerivedDataWorker::DoWork);
	COOK_STAT(auto Timer = TextureCookStats::TaskUsageStats.TimeSyncWork());
	COOK_STAT(Timer.AddHit(0));	//	Register as a DDC hit, but no bytes processed. Purely tracking processing overhead, not DDC transformation work

	const bool bAllowAsyncBuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::AllowAsyncBuild);
	const bool bAllowAsyncLoading = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::AllowAsyncLoading);
	const bool bForVirtualTextureStreamingBuild = EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForVirtualTextureStreamingBuild);
	bool bInvalidVirtualTextureCompression = false;

	bSucceeded = false;
	bLoadedFromDDC = false;

	DDC1_FetchAndFillDerivedData(
		/* inputs */ Texture, TexturePathName, CacheFlags, BuildSettingsPerLayerFetchFirst, FetchFirstMetadata, BuildSettingsPerLayerFetchOrBuild, FetchOrBuildMetadata, 
		/* outputs */ DerivedData, KeySuffix, bSucceeded, bInvalidVirtualTextureCompression, BytesCached);
	if (bSucceeded)
	{
		bLoadedFromDDC = true;
	}

	if (CancellationToken.IsCanceled())
	{
		return;
	}

	if (BuildSettingsPerLayerFetchOrBuild[0].Tiler && !bForVirtualTextureStreamingBuild)
	{
		if (CVarForceRetileTextures.GetValueOnAnyThread())
		{
			// We do this after the fetch so it can fill out the metadata and key suffix that gets used.
			bSucceeded = false;
			bLoadedFromDDC = false;

			DerivedData->Mips.Empty();
			delete DerivedData->VTData;
			DerivedData->VTData = nullptr;
		}
	}

	check( ! bTriedAndFailed );

	if (!bSucceeded && bAllowAsyncBuild)
	{
		if (DDC1_LoadAndValidateTextureData(Texture, TextureData, CompositeTextureData, ImageWrapper))
		{
			if (CancellationToken.IsCanceled())
			{
				return;
			}

			for (int32 LayerIndex = 0; LayerIndex < BuildSettingsPerLayerFetchOrBuild.Num(); LayerIndex++)
			{
				if (LayerIndex < TextureData.LayerChannelMinMax.Num())
				{
					BuildSettingsPerLayerFetchOrBuild[LayerIndex].bKnowAlphaTransparency = Compressor->DetermineAlphaChannelTransparency(
						BuildSettingsPerLayerFetchOrBuild[LayerIndex], 
						TextureData.LayerChannelMinMax[LayerIndex].Key,
						TextureData.LayerChannelMinMax[LayerIndex].Value,
						BuildSettingsPerLayerFetchOrBuild[LayerIndex].bHasTransparentAlpha);
				}
			}

			// Replace any existing DDC data, if corrupt compression was detected
			const bool bReplaceExistingDDC = bInvalidVirtualTextureCompression;

			if (BuildSettingsPerLayerFetchOrBuild[0].Tiler && !bForVirtualTextureStreamingBuild)
			{
				bSucceeded = DDC1_BuildTiledClassicTexture(
					Compressor, ImageWrapper, Texture, TexturePathName, BuildSettingsPerLayerFetchFirst, BuildSettingsPerLayerFetchOrBuild, FetchFirstMetadata, FetchOrBuildMetadata,
					TextureData, CompositeTextureData, CacheFlags, RequiredMemoryEstimate, KeySuffix, DerivedData, BytesCached);
			}
			else
			{
				DDC1_BuildTexture(
					Compressor, ImageWrapper, Texture, TexturePathName, CacheFlags, TextureData, CompositeTextureData, BuildSettingsPerLayerFetchOrBuild, FetchOrBuildMetadata, 
					KeySuffix, bReplaceExistingDDC, RequiredMemoryEstimate, DerivedData, BytesCached, bSucceeded);
			}

			if (CancellationToken.IsCanceled())
			{
				return;
			}

			// TextureData may have been freed by Build, don't use it anymore

			if (bInvalidVirtualTextureCompression && DerivedData->VTData)
			{
				// If we loaded data that turned out to be corrupt, flag it here so we can also recreate the VT data cached to local /DerivedDataCache/VT/ directory
				for (FVirtualTextureDataChunk& Chunk : DerivedData->VTData->Chunks)
				{
					Chunk.bCorruptDataLoadedFromDDC = true;
				}
			}

			if ( ! bSucceeded )
			{
				bTriedAndFailed = true;
			}
		}
		else
		{
			bSucceeded = false;

			// Excess logging to try and nail down a spurious failure.
			UE_LOGF(LogTexture, Display, "Texture was not found in DDC and couldn't build as the texture source was unable to load or validate (%ls)", *TexturePathName);
			int32 TextureDataBlocks = TextureData.Blocks.Num();
			int32 TextureDataBlocksLayers = TextureDataBlocks > 0 ? TextureData.Blocks[0].MipsPerLayer.Num() : -1;
			int32 TextureDataBlocksLayerMips = TextureDataBlocksLayers > 0 ? TextureData.Blocks[0].MipsPerLayer[0].Num() : -1;

			UE_LOGF(LogTexture, Display, "Texture Data Blocks: %d Layers: %d Mips: %d", TextureDataBlocks, TextureDataBlocksLayers, TextureDataBlocksLayerMips);
			if (CompositeTextureData.IsValid()) // Says IsValid, but means whether or not we _need_ composite texture data.
			{
				int32 CompositeTextureDataBlocks = CompositeTextureData.Blocks.Num();
				int32 CompositeTextureDataBlocksLayers = CompositeTextureDataBlocks > 0 ? CompositeTextureData.Blocks[0].MipsPerLayer.Num() : -1;
				int32 CompositeTextureDataBlocksLayerMips = CompositeTextureDataBlocksLayers > 0 ? CompositeTextureData.Blocks[0].MipsPerLayer[0].Num() : -1;

				UE_LOGF(LogTexture, Display, "Composite Texture Data Blocks: %d Layers: %d Mips: %d", CompositeTextureDataBlocks, CompositeTextureDataBlocksLayers, CompositeTextureDataBlocksLayerMips);
			}

			// bTriedAndFailed = true; // no retry in Finalize ?  @todo ?
		}
	}

	// there are actually 3 states to bSucceeded
	//	tried & succeeded, tried & failed, not tried yet
	// we may try the build again in Finalize (eg. if !bAllowAsyncBuild)

	if (bSucceeded || bTriedAndFailed)
	{
		TextureData.ReleaseMemory();
		CompositeTextureData.ReleaseMemory();
	}

	if (CancellationToken.IsCanceled())
	{
		return;
	}

	if ( bSucceeded )
	{
		// Populate the VT DDC Cache now if we're asynchronously loading to avoid too many high prio/synchronous request on the render thread
		if (!IsInGameThread() && DerivedData->VTData && !DerivedData->VTData->Chunks.Last().DerivedDataKey.IsEmpty())
		{
			GetVirtualTextureChunkDDCCache()->MakeChunkAvailable_Concurrent(&DerivedData->VTData->Chunks.Last());
		}
	}
}

void FTextureCacheDerivedDataWorker::Finalize()
{
	// Building happens here whenever the ddc is missed and async builds aren't allowed.
	// This generally doesn't happen, but does in a few cases:
	// --	always happens with a ForceRebuildPlatformData, which is called whenever mip data is requested
	//		in the editor and is missing for some reason.
	// --	always with a lighting build, as the async light/shadowmap tasks will disallow async builds. 
	// --	if the texture compiler cvar disallows async texture compilation  "Editor.AsyncTextureCompilation 0"

	TRACE_CPUPROFILER_EVENT_SCOPE(FTextureCacheDerivedDataWorker::Finalize);

	if (CancellationToken.IsCanceled())
	{
		return;
	}

	COOK_STAT(auto Timer = TextureCookStats::TaskUsageStats.TimeSyncWork());
	COOK_STAT(Timer.TrackCyclesOnly());

	if ( bTriedAndFailed )
	{
		UE_LOGF(LogTexture, Warning, "Texture build failed for %ls.  Will not retry in Finalize.", *TexturePathName);
		return;
	}

	if (!bSucceeded)
	{
		if ( !Texture.Source.HasPayloadData() )
		{
			UE_LOGF(LogTexture, Warning, "Unable to build texture source data, no available payload for %ls. This may happen if it was duplicated from cooked data.", *TexturePathName);
			return;
		}

		// note: GetSourceMips will not even try if TextureData.bValid was set to false
		TextureData.GetSourceMips(Texture.Source, ImageWrapper);
		if (Texture.GetCompositeTexture() && Texture.GetCompositeTexture()->Source.IsValid())
		{
			CompositeTextureData.GetSourceMips(Texture.GetCompositeTexture()->Source, ImageWrapper);
		}

		if (DDC1_IsTextureDataValid(TextureData, CompositeTextureData) == false)
		{
			UE_LOGF(LogTexture, Warning, "Unable to get texture source data for synchronous build of %ls", *TexturePathName);
		}
		else
		{
			if (BuildSettingsPerLayerFetchOrBuild[0].Tiler && !EnumHasAnyFlags(CacheFlags, ETextureCacheFlags::ForVirtualTextureStreamingBuild))
			{
				bSucceeded = DDC1_BuildTiledClassicTexture(
					Compressor,
					ImageWrapper,
					Texture,
					TexturePathName,
					BuildSettingsPerLayerFetchFirst,
					BuildSettingsPerLayerFetchOrBuild,
					FetchFirstMetadata,
					FetchOrBuildMetadata,
					TextureData,
					CompositeTextureData,
					CacheFlags,
					RequiredMemoryEstimate,
					KeySuffix,
					DerivedData,
					BytesCached);
			}
			else
			{
				DDC1_BuildTexture(Compressor, ImageWrapper, Texture, TexturePathName, CacheFlags,
					TextureData, CompositeTextureData, BuildSettingsPerLayerFetchOrBuild, FetchOrBuildMetadata, KeySuffix, false /* currently corrupt vt data is not routed out of DoWork() */,
					RequiredMemoryEstimate, DerivedData, BytesCached, bSucceeded);
			}

			if ( ! bSucceeded )
			{
				bTriedAndFailed = true;
			}
		}
	}
		
	if (bSucceeded && BuildSettingsPerLayerFetchOrBuild[0].bVirtualStreamable) // Texture.VirtualTextureStreaming is more a hint that might be overruled by the buildsettings
	{
		check((DerivedData->VTData != nullptr) == Texture.VirtualTextureStreaming); 
	}
}



struct FBuildResultOptions
{
	bool bLoadStreamingMips;
	int32 FirstStreamingMipToLoad;
};

static bool UnpackPlatformDataFromBuild(FTexturePlatformData& OutPlatformData, UE::DerivedData::FBuildCompleteParams&& InBuildCompleteParams, FBuildResultOptions InBuildResultOptions)
{
	using namespace UE;
	using namespace UE::DerivedData;
	UE::DerivedData::FBuildOutput& BuildOutput = InBuildCompleteParams.Output;

	bool bHasCPUCopy = false;
	{
		// CPUCopy might not exist if the build didn't request it, but we pipe it through child builds,
		// so it might be present but zero size.
		const FValueWithId& MetadataValue = BuildOutput.GetValue(FValueId::FromName(ANSITEXTVIEW("CPUCopyImageInfo")));
		if (MetadataValue.IsValid() && MetadataValue.GetRawSize())
		{
			FSharedImageRef CPUCopy = new FSharedImage();
			if (CPUCopy->ImageInfoFromCompactBinary(FCbObject(MetadataValue.GetData().Decompress())) == false)
			{
				UE_LOGF(LogTexture, Error, "Invalid CPUCopyImageInfo in build output '%ls' by %ls.", *BuildOutput.GetName(), *WriteToString<32>(BuildOutput.GetFunction()));
				return false;
			}

			const FValueWithId& DataValue = BuildOutput.GetValue(FValueId::FromName(ANSITEXTVIEW("CPUCopyRawData")));
			if (DataValue.IsValid() == false)
			{
				UE_LOGF(LogTexture, Error, "Missing CPUCopyRawData in build output '%ls' by %ls.", *BuildOutput.GetName(), *WriteToString<32>(BuildOutput.GetFunction()));
				return false;
			}

			FSharedBuffer Data = DataValue.GetData().Decompress();
			CPUCopy->RawData.AddUninitialized(Data.GetSize());
			FMemory::Memcpy(CPUCopy->RawData.GetData(), Data.GetData(), Data.GetSize());
			OutPlatformData.CPUCopy = CPUCopy;
			bHasCPUCopy = true;
		}
	}

	// this will get pulled from the build metadata in a later cl...
	//{
	//	const FValueWithId& Value = BuildOutput.GetValue(FValueId::FromName(UTF8TEXTVIEW("TextureBuildMetadata")));
	//	PackTextureBuildMetadataInPlatformData(&OutPlatformData, FCbObject(Value.GetData().Decompress()));
	//}

	// We take this as a build output, however in ideal (future) situations, this is generated prior to build launch and
	// just routed through the build. Since we currently handle several varying situations, we just always consume it from
	// the build no matter where it came from. (Both TextureDescription and ExtendedData)
	FEncodedTextureDescription EncodedTextureDescription;
	{
		const FValueWithId& Value = BuildOutput.GetValue(FValueId::FromName(ANSITEXTVIEW("EncodedTextureDescription")));
		UE::TextureBuildUtilities::EncodedTextureDescription::FromCompactBinary(EncodedTextureDescription, FCbObject(Value.GetData().Decompress()));
	}

	FEncodedTextureExtendedData EncodedTextureExtendedData;
	{
		const FValueWithId& Value = BuildOutput.GetValue(FValueId::FromName(ANSITEXTVIEW("EncodedTextureExtendedData")));
		UE::TextureBuildUtilities::EncodedTextureExtendedData::FromCompactBinary(EncodedTextureExtendedData, FCbObject(Value.GetData().Decompress()));
	}

	// consider putting this in the build output so that it's only ever polled in one place.
	const FTextureEngineParameters EngineParameters = GenerateTextureEngineParameters();
	int32 NumStreamingMips = EncodedTextureDescription.GetNumStreamingMips(&EncodedTextureExtendedData, EngineParameters);
	int32 NumEncodedMips = EncodedTextureDescription.GetNumEncodedMips(&EncodedTextureExtendedData);
	check(NumEncodedMips >= NumStreamingMips);

	//
	//
	// We have all the metadata we need, we can grab the data
	//
	//
	OutPlatformData.PixelFormat = EncodedTextureDescription.PixelFormat;
	OutPlatformData.SizeX = EncodedTextureDescription.TopMipSizeX;
	OutPlatformData.SizeY = EncodedTextureDescription.TopMipSizeY;
	OutPlatformData.OptData.NumMipsInTail = EncodedTextureExtendedData.NumMipsInTail;
	OutPlatformData.OptData.ExtData = EncodedTextureExtendedData.ExtData;
	{
		const bool bHasOptData = (EncodedTextureExtendedData.NumMipsInTail != 0) || (EncodedTextureExtendedData.ExtData != 0);
		OutPlatformData.SetPackedData(EncodedTextureDescription.GetNumSlices_WithDepth(0), bHasOptData, EncodedTextureDescription.bCubeMap, bHasCPUCopy);
	}
	OutPlatformData.Mips.Empty(EncodedTextureDescription.NumMips);
	EFileRegionType FileRegion = FFileRegion::SelectType(EncodedTextureDescription.PixelFormat);


	FSharedBuffer MipTailData;
	if (EncodedTextureDescription.NumMips > NumStreamingMips)
	{
		const FValueWithId& MipTailValue = BuildOutput.GetValue(FValueId::FromName(ANSITEXTVIEW("MipTail")));
		if (!MipTailValue)
		{
			UE_LOGF(LogTexture, Error, "Missing texture mip tail for build of '%ls' by %ls.", *BuildOutput.GetName(), *WriteToString<32>(BuildOutput.GetFunction()));
			return false;
		}
		MipTailData = MipTailValue.GetData().Decompress();
	}



	//
	// Mips are split up:
	//	Streaming mips are all stored independently under value FTexturePlatformData::MakeMipId(MipIndex);
	//	Nonstreaming ("inlined") mips are stored in one buffer under value "MipTail". To disentangle the separate mips
	//	we need their size.
	//

	uint64 CurrentMipTailOffset = 0;
	for (int32 MipIndex = 0; MipIndex < EncodedTextureDescription.NumMips; MipIndex++)
	{
		const FIntVector3 MipDims = EncodedTextureDescription.GetMipDimensions(MipIndex);
		FTexture2DMipMap* NewMip = new FTexture2DMipMap(MipDims.X, MipDims.Y, MipDims.Z);
		OutPlatformData.Mips.Add(NewMip);

		NewMip->FileRegionType = FileRegion;
		NewMip->SizeZ = EncodedTextureDescription.GetRHIStyleSizeZ(MipIndex);

		if (MipIndex >= NumEncodedMips)
		{
			// Packed mip tail data is inside the outermost mip for the pack, so we don't have
			// any bulk data to pull out.
			continue;
		}

		if (MipIndex >= NumStreamingMips)
		{			
			// This mip is packed inside a single buffer. This is distinct from a "packed mip tail", but might
			// coincidentally be the same. All mips past NumStreamingMips need to be copied into the bulk data
			// and are always resident in memory with the texture.

			uint64 MipSizeInBytes = EncodedTextureExtendedData.MipSizesInBytes[MipIndex];
			FMemoryView MipView = MipTailData.GetView().Mid(CurrentMipTailOffset, MipSizeInBytes);
			CurrentMipTailOffset += MipSizeInBytes;

			NewMip->BulkData.Lock(LOCK_READ_WRITE);
			void* MipAllocData = NewMip->BulkData.Realloc(int64(MipSizeInBytes));
			MakeMemoryView(MipAllocData, MipSizeInBytes).CopyFrom(MipView);
			NewMip->BulkData.Unlock();
		}
		else
		{
			const FValueId MipId = FTexturePlatformData::MakeMipId(MipIndex);
			const FValueWithId& MipValue = BuildOutput.GetValue(MipId);
			if (!MipValue)
			{
				UE_LOGF(LogTexture, Error, "Missing streaming texture mip %d for build of '%ls' by %ls.", MipIndex, *BuildOutput.GetName(), *WriteToString<32>(BuildOutput.GetFunction()));
				return false;
			}

			// Did whoever launched the build want the streaming mips in memory?
			if (InBuildResultOptions.bLoadStreamingMips && (MipIndex >= InBuildResultOptions.FirstStreamingMipToLoad))
			{
				NewMip->BulkData.Lock(LOCK_READ_WRITE);
				const uint64 MipSize = MipValue.GetRawSize();
				void* MipData = NewMip->BulkData.Realloc(IntCastChecked<int64>(MipSize));
				ON_SCOPE_EXIT{ NewMip->BulkData.Unlock(); };
				if (!MipValue.GetData().TryDecompressTo(MakeMemoryView(MipData, MipSize)))
				{
					UE_LOGF(LogTexture, Error, "Failed to decompress streaming texture mip %d for build of '%ls' by %ls.", MipIndex, *BuildOutput.GetName(), *WriteToString<32>(BuildOutput.GetFunction()));
					return false;
				}
			}

			FSharedString MipName(WriteToString<256>(BuildOutput.GetName(), TEXT(" [MIP "), MipIndex, TEXT("]")));
			NewMip->DerivedData = UE::FDerivedData(MoveTemp(MipName), InBuildCompleteParams.CacheKey, MipId);
			PRAGMA_DISABLE_DEPRECATION_WARNINGS;
			NewMip->bPagedToDerivedData = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		}
	}
	return true;
}

static void HandleBuildOutputThenUnpack(FTexturePlatformData& OutPlatformData, UE::DerivedData::FBuildCompleteParams&& InBuildCompleteParams, FBuildResultOptions InBuildResultOptions)
{
	using namespace UE::DerivedData;

	PrintIBuildOutputMessages(InBuildCompleteParams.Output);

	if (InBuildCompleteParams.Output.HasError())
	{
		return;
	}

	UnpackPlatformDataFromBuild(OutPlatformData, MoveTemp(InBuildCompleteParams), InBuildResultOptions);
}

struct FBuildResults
{
	FTexturePlatformData& PlatformData;
	bool bCacheHit = false;
	uint64 BuildOutputSize = 0;

	FBuildResults(FTexturePlatformData& InPlatformData) : 
		PlatformData(InPlatformData)
	{}

};

static void GetBuildResultsFromCompleteParams(
	FBuildResults& OutBuildResults, 
	FBuildResultOptions InBuildResultOptions, 
	UE::DerivedData::FBuildCompleteParams&& InBuildCompleteParams
)
{
	using namespace UE::DerivedData;
	OutBuildResults.PlatformData.DerivedDataKey.Emplace<FCacheKeyProxy>(InBuildCompleteParams.CacheKey);

	// This is false if any build in the chain missses.
	OutBuildResults.bCacheHit = EnumHasAnyFlags(InBuildCompleteParams.BuildStatus, EBuildStatus::CacheQueryHit);

	OutBuildResults.BuildOutputSize = Algo::TransformAccumulate(InBuildCompleteParams.Output.GetValues(),
		[](const FValue& Value) { return Value.GetData().GetRawSize(); }, uint64(0));
	if (InBuildCompleteParams.Status != EStatus::Canceled) // this branch also handles printing errors.
	{
		HandleBuildOutputThenUnpack(OutBuildResults.PlatformData, MoveTemp(InBuildCompleteParams), InBuildResultOptions);
	}
}


struct FBuildInfo
{
	UE::DerivedData::FBuildSession& BuildSession;
	UE::DerivedData::FBuildDefinition BuildDefinition;
	UE::DerivedData::FBuildPolicy BuildPolicy;
	FTexturePlatformData::FStructuredDerivedDataKey Key;
	TOptional<FTexturePlatformData::FTextureEncodeResultMetadata> ResultMetadata;

	explicit FBuildInfo(
		UE::DerivedData::FBuildSession& InBuildSession, 
		UE::DerivedData::FBuildDefinition InBuildDefinition, 
		UE::DerivedData::FBuildPolicy InBuildPolicy, 
		FTexturePlatformData::FStructuredDerivedDataKey&& InKey,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InResultMetadata)
		: BuildSession(InBuildSession)
		, BuildDefinition(InBuildDefinition)
		, BuildPolicy(InBuildPolicy)
		, Key(MoveTemp(InKey))
	{
		if (InResultMetadata)
		{
			ResultMetadata.Emplace(*InResultMetadata);
		}
	}
};

static void LaunchBuildWithFallback(
	FBuildResults& OutBuildResults,
	FBuildResultOptions BuildResultOptions,
	FBuildInfo&& InInitialBuild,
	TOptional<FBuildInfo> InFallbackBuild,
	UE::DerivedData::FRequestOwner& InRequestOwner // Owner must be valid for the duration of the build.
)
{
	using namespace UE::DerivedData;

	if (InInitialBuild.ResultMetadata.IsSet())
	{
		OutBuildResults.PlatformData.ResultMetadata = *InInitialBuild.ResultMetadata;
	}

	// TODO: This has no memory estimate!
	LaunchTaskInThreadPool(0, TEXT("TextureBuild"), InRequestOwner, FTextureCompilingManager::Get().GetThreadPool(),
		[
			FallbackBuild = MoveTemp(InFallbackBuild),
			RequestOwner = &InRequestOwner,
			OutBuildResults = &OutBuildResults,
			BuildResultOptions,
			PrimaryBuild = MoveTemp(InInitialBuild)
		]() mutable
	{
		PrimaryBuild.BuildSession.Build(PrimaryBuild.BuildDefinition, {}, PrimaryBuild.BuildPolicy, *RequestOwner,
			[
				FallbackBuild = MoveTemp(FallbackBuild),
				RequestOwner,
				OutBuildResults,
				BuildResultOptions
			](FBuildCompleteParams&& Params) mutable
		{
			if (Params.Status == EStatus::Error &&
				FallbackBuild.IsSet())
			{
				if (FallbackBuild->ResultMetadata.IsSet())
				{
					OutBuildResults->PlatformData.ResultMetadata = *FallbackBuild->ResultMetadata;
				}
				FallbackBuild->BuildSession.Build(FallbackBuild->BuildDefinition, {}, FallbackBuild->BuildPolicy, *RequestOwner,
					[
						OutBuildResults = OutBuildResults,
						BuildResultOptions
					](FBuildCompleteParams&& Params) mutable
				{
					GetBuildResultsFromCompleteParams(*OutBuildResults, BuildResultOptions, MoveTemp(Params));
				});
			}
			else
			{
				GetBuildResultsFromCompleteParams(*OutBuildResults, BuildResultOptions, MoveTemp(Params));
			}
		});
	});
};


//
// DDC2 texture fetch/build task.
//
class FTextureBuildTask final : public FTextureAsyncCacheDerivedDataTask
{
public:

	FBuildInfo CreateBuildForSettings(
		UE::DerivedData::IBuild& InBuild,
		UE::FSharedString& TexturePath,
		UTexture* InTexture,
		bool bUseCompositeTexture,
		const UE::FUtf8SharedString& FunctionName,
		const UE::FUtf8SharedString& TilingFunctionName,
		const FTextureBuildSettings& InBuildSettings,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InResultMetadata,
		UE::DerivedData::FBuildPolicy FinalBuildPolicy,
		UE::DerivedData::FBuildPolicy ParentBuildPolicy
		)
	{
		using namespace UE::DerivedData;

		FBuildDefinition BaseDefinition = CreateDefinition(InBuild, *InTexture, TexturePath, FunctionName, InBuildSettings, bUseCompositeTexture);
		FBuildDefinition* RunDefinition = &BaseDefinition;

		// If we have a build chain, then the next build determines what the output is as the data they
		// need must be available. For us, we just always forward all data to child builds, then we set the
		// actual policy the build requester wants at the end.

		// Since we want to be able to control the policy for tiling, which is passed to the build thats _next_
		// we need to track what we give to the next build.
		FBuildPolicy NextBuildPolicy = ParentBuildPolicy;

		TOptional<FBuildDefinition> TilingDefinition;
		if (TilingFunctionName.Len())
		{
			UE::TextureDerivedData::FParentBuildPlumbing Parent(BuildSession.Get(), *RunDefinition, NextBuildPolicy);

			TilingDefinition.Emplace(CreateTilingDefinition(InBuild, InTexture, InBuildSettings, nullptr, nullptr, *RunDefinition, TexturePath, TilingFunctionName));

			InputResolver.ChildBuilds.Add(TilingDefinition.GetPtrOrNull()->GetKey(), MoveTemp(Parent));

			RunDefinition = TilingDefinition.GetPtrOrNull();
			NextBuildPolicy = CVarForceRetileTextures.GetValueOnAnyThread() ? EBuildPolicy::Build : ParentBuildPolicy;
		}

		TOptional<FBuildDefinition> DetileDefinition;
		TOptional<FBuildDefinition> DecodeDefinition;
		if (InBuildSettings.bDecodeForPCUsage)
		{
			if (InBuildSettings.TilerEvenIfNotSharedLinear)
			{
				UE::TextureDerivedData::FParentBuildPlumbing Parent(BuildSession.Get(), *RunDefinition, NextBuildPolicy);

				DetileDefinition.Emplace(CreateDetileDefinition(InBuild, InTexture, InBuildSettings, *RunDefinition, TexturePath));

				InputResolver.ChildBuilds.Add(DetileDefinition.GetPtrOrNull()->GetKey(), MoveTemp(Parent));

				RunDefinition = DetileDefinition.GetPtrOrNull();
				NextBuildPolicy = ParentBuildPolicy;
			}

			FEncodedTextureDescription TextureDescription;
			InBuildSettings.GetEncodedTextureDescriptionFromSourceMips(&TextureDescription, InBuildSettings.BaseTextureFormat,
				InTexture->Source.GetSizeX(), InTexture->Source.GetSizeY(), InTexture->Source.GetNumSlices(), InTexture->Source.GetNumMips(),
				true);

			// We use LODBias=0 here because the editor doesn't strip the top mips - so we could need them to view even if they aren't deployed.
			if (UE::TextureBuildUtilities::TextureNeedsDecodeForPC(TextureDescription.PixelFormat, TextureDescription.GetMipWidth(0), TextureDescription.GetMipHeight(0)))
			{
				UE::TextureDerivedData::FParentBuildPlumbing Parent(BuildSession.Get(), *RunDefinition, NextBuildPolicy);
			
				DecodeDefinition.Emplace(CreateDecodeDefinition(InBuild, InTexture, InBuildSettings, *RunDefinition, TexturePath));

				InputResolver.ChildBuilds.Add(DecodeDefinition.GetPtrOrNull()->GetKey(), MoveTemp(Parent));

				RunDefinition = DecodeDefinition.GetPtrOrNull();
				NextBuildPolicy = ParentBuildPolicy;
			}
		}

		FTexturePlatformData::FStructuredDerivedDataKey Key = GetKey(
			BaseDefinition,
			TilingDefinition.GetPtrOrNull(),
			DetileDefinition.GetPtrOrNull(),
			DecodeDefinition.GetPtrOrNull(),
			*InTexture,
			bUseCompositeTexture);

		return FBuildInfo(BuildSession.Get(), *RunDefinition, FinalBuildPolicy, MoveTemp(Key), InResultMetadata);
	}


	FTextureBuildTask(
		UTexture& Texture,
		FTexturePlatformData& InDerivedData,
		const UE::FUtf8SharedString& FunctionName,
		const UE::FUtf8SharedString& TilingFunctionName,
		const FTextureBuildSettings* InSettingsFetchFirst, // can be nullptr
		const FTextureBuildSettings& InSettingsFetchOrBuild,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchFirstMetadata, // can be nullptr
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchOrBuildMetadata, // can be nullptr
		EQueuedWorkPriority Priority,
		ETextureCacheFlags Flags)
		: BuildResults(InDerivedData)
	{
		static bool bLoadedModules = LoadModules();

		BuildResultOptions.bLoadStreamingMips = EnumHasAnyFlags(Flags, ETextureCacheFlags::InlineMips);
		BuildResultOptions.FirstStreamingMipToLoad = InSettingsFetchOrBuild.LODBiasWithCinematicMips;

		// Can't fetch first if we are rebuilding.
		if (InSettingsFetchFirst &&
			EnumHasAnyFlags(Flags, ETextureCacheFlags::ForceRebuild))
		{
			InSettingsFetchFirst = nullptr;
			InFetchFirstMetadata = nullptr;
		}

		// Dump any existing data.
		InDerivedData.Reset();

		using namespace UE;
		using namespace UE::DerivedData;

		EPriority OwnerPriority = EnumHasAnyFlags(Flags, ETextureCacheFlags::Async) ? ConvertFromQueuedWorkPriority(Priority) : EPriority::Blocking;
		Owner.Emplace(OwnerPriority);

		bool bUseCompositeTexture = false;
		if (!IsTextureValidForBuilding(Texture, Flags, InSettingsFetchOrBuild.bCPUAccessible, bUseCompositeTexture))
		{
			return;
		}
		
		// we don't support VT layers here (no SettingsPerLayer)
		check( Texture.Source.GetNumLayers() == 1 );

		// Debug string.
		FSharedString TexturePath;
		{
			TStringBuilder<256> TexturePathBuilder;
			Texture.GetPathName(nullptr, TexturePathBuilder);
			TexturePath = TexturePathBuilder.ToView();
		}

		TOptional<FTextureStatusMessageContext> StatusMessage;
		if (IsInGameThread() && OwnerPriority == EPriority::Blocking)
		{
			// this gets set whether or not we are building the texture, and is a rare edge case for UI feedback.
			// We don't actually know whether we're using fetchfirst or actually building, so if we have two keys,
			// we just assume we're FinalIfAvailable.
			ETextureEncodeSpeed EncodeSpeed = (ETextureEncodeSpeed)InSettingsFetchOrBuild.RepresentsEncodeSpeedNoSend;
			if (InSettingsFetchFirst)
			{
				EncodeSpeed = ETextureEncodeSpeed::FinalIfAvailable;
			}

			StatusMessage.Emplace(ComposeTextureBuildText(Texture, InSettingsFetchOrBuild, EncodeSpeed, GetBuildRequiredMemoryEstimate(&Texture, &InSettingsFetchOrBuild), EnumHasAnyFlags(Flags, ETextureCacheFlags::ForVirtualTextureStreamingBuild)));
		}
		

		TOptional<FTexturePlatformData::FTextureEncodeResultMetadata> FetchFirstResultMetadata;
		if (InFetchFirstMetadata)
		{
			FetchFirstResultMetadata = *InFetchFirstMetadata;
		}

		TOptional<FTexturePlatformData::FTextureEncodeResultMetadata> FetchOrBuildResultMetadata;
		if (InFetchOrBuildMetadata)
		{
			FetchOrBuildResultMetadata = *InFetchOrBuildMetadata;
		}

		// Description and MipTail should always cache. Everything else (i.e. Mip# i.e. streaming mips) should skip data
		// when we are not inlining.
		FBuildPolicy FetchFirstBuildPolicy = FetchFirst_CreateBuildPolicy(BuildResultOptions);
		FBuildPolicy FetchOrBuildPolicy = FetchOrBuild_CreateBuildPolicy(Flags, BuildResultOptions);
		FBuildPolicy ParentBuildPolicy = EnumHasAnyFlags(Flags, ETextureCacheFlags::ForceRebuild) ? (EBuildPolicy::Default & ~EBuildPolicy::CacheQuery) : EBuildPolicy::Default;

		//
		// Set up the build
		//
		IBuild& Build = GetBuild();

		InputResolver.GlobalResolver = GetGlobalBuildInputResolver();
		InputResolver.Texture = &Texture;

		BuildSession = Build.CreateSession(TexturePath, &InputResolver);

		FBuildInfo FetchOrBuildInfo = CreateBuildForSettings(
			Build,
			TexturePath,
			&Texture,
			bUseCompositeTexture,
			FunctionName,
			TilingFunctionName,
			InSettingsFetchOrBuild,
			FetchOrBuildResultMetadata.GetPtrOrNull(),
			FetchOrBuildPolicy,
			ParentBuildPolicy
			);

		BuildResults.PlatformData.FetchOrBuildDerivedDataKey.Emplace<FTexturePlatformData::FStructuredDerivedDataKey>(FetchOrBuildInfo.Key);

		bool bLaunchedBuild = false;
		if (InSettingsFetchFirst)
		{
			FBuildInfo FetchFirstInfo = CreateBuildForSettings(
				Build,
				TexturePath,
				&Texture,
				bUseCompositeTexture,
				FunctionName,
				TilingFunctionName,
				*InSettingsFetchFirst,
				FetchFirstResultMetadata.GetPtrOrNull(),
				FetchFirstBuildPolicy,
				ParentBuildPolicy
				);

			BuildResults.PlatformData.FetchFirstDerivedDataKey.Emplace<FTexturePlatformData::FStructuredDerivedDataKey>(FetchFirstInfo.Key);

			// Only launch fetch first if it's a distinct build.
			if (FetchFirstInfo.Key != FetchOrBuildInfo.Key)
			{
				bLaunchedBuild = true;
				LaunchBuildWithFallback(BuildResults, BuildResultOptions, MoveTemp(FetchFirstInfo), FetchOrBuildInfo, *Owner);
			}
		}

		if (!bLaunchedBuild)
		{
			LaunchBuildWithFallback(BuildResults, BuildResultOptions, MoveTemp(FetchOrBuildInfo), {}, *Owner);
		}

		
		if (StatusMessage.IsSet())
		{
			Owner->Wait();
		}
	}

	static UE::DerivedData::FBuildDefinition CreateDefinition(
		UE::DerivedData::IBuild& Build,
		UTexture& Texture,
		const UE::FSharedString& TexturePath,
		const UE::FUtf8SharedString& FunctionName,
		const FTextureBuildSettings& Settings,
		const bool bUseCompositeTexture)
	{
		UE::DerivedData::FBuildDefinitionBuilder DefinitionBuilder = Build.CreateDefinition(TexturePath, FunctionName);
		DefinitionBuilder.AddConstant(UTF8TEXTVIEW("EngineParameters"), UE::TextureBuildUtilities::TextureEngineParameters::ToCompactBinaryWithDefaults(GenerateTextureEngineParameters()));
		DefinitionBuilder.AddConstant(UTF8TEXTVIEW("Settings"), SaveTextureBuildSettings(Texture, Settings, 0, bUseCompositeTexture));

		// Texture.Source must be uncompressed for TextureBuildFunction
		Texture.Source.RemoveCompression();
		check( ! Texture.Source.IsSourceCompressed() );
		DefinitionBuilder.AddInputBulkData(UTF8TEXTVIEW("Source"), Texture.Source.GetPersistentId());

		if (Texture.GetCompositeTexture() && bUseCompositeTexture)
		{
			FTextureSource & CompositeSource = Texture.GetCompositeTexture()->Source;
			CompositeSource.RemoveCompression();
			check( ! CompositeSource.IsSourceCompressed() );
			DefinitionBuilder.AddInputBulkData(UTF8TEXTVIEW("CompositeSource"), CompositeSource.GetPersistentId());
		}
		return DefinitionBuilder.Build();
	}

private:

	static constexpr FAnsiStringView NonStreamingMipOutputValueNames[] = 
	{
		ANSITEXTVIEW("EncodedTextureDescription"),
		ANSITEXTVIEW("EncodedTextureExtendedData"),
		ANSITEXTVIEW("MipTail"),
		ANSITEXTVIEW("CPUCopyImageInfo"),
		ANSITEXTVIEW("CPUCopyRawData")
	};


	static UE::DerivedData::FBuildPolicy FetchFirst_CreateBuildPolicy(FBuildResultOptions InBuildResultOptions)
	{
		using namespace UE::DerivedData;

		if (InBuildResultOptions.bLoadStreamingMips)
		{
			// We want all of the output values.
			return EBuildPolicy::Cache;
		}
		else
		{
			// Cache everything except the streaming mips.
			FBuildPolicyBuilder FetchFirstBuildPolicyBuilder(EBuildPolicy::CacheQuery | EBuildPolicy::SkipData);
			for (FAnsiStringView NonStreamingValue : NonStreamingMipOutputValueNames)
			{
				FetchFirstBuildPolicyBuilder.AddValuePolicy(FValueId::FromName(NonStreamingValue), EBuildPolicy::Cache);
			}
			return FetchFirstBuildPolicyBuilder.Build();
		}		
	}

	static UE::DerivedData::FBuildPolicy FetchOrBuild_CreateBuildPolicy(ETextureCacheFlags Flags, FBuildResultOptions InBuildResultOptions)
	{
		using namespace UE::DerivedData;

		if (EnumHasAnyFlags(Flags, ETextureCacheFlags::ForceRebuild))
		{
			return EBuildPolicy::Default & ~EBuildPolicy::CacheQuery;
		}
		else if (InBuildResultOptions.bLoadStreamingMips)
		{
			return EBuildPolicy::Default;
		}
		else
		{
			FBuildPolicyBuilder BuildPolicyBuilder(EBuildPolicy::Build | EBuildPolicy::CacheQuery | EBuildPolicy::CacheStoreOnBuild | EBuildPolicy::SkipData);
			for (FAnsiStringView NonStreamingValue : NonStreamingMipOutputValueNames)
			{
				BuildPolicyBuilder.AddValuePolicy(FValueId::FromName(NonStreamingValue), EBuildPolicy::Cache);
			}
			return BuildPolicyBuilder.Build();
		}
	}

	void Finalize(bool& bOutFoundInCache, uint64& OutProcessedByteCount) final
	{
		bOutFoundInCache = BuildResults.bCacheHit;
		OutProcessedByteCount = BuildResults.BuildOutputSize;
	}
public:

	EQueuedWorkPriority GetPriority() const final
	{
		using namespace UE::DerivedData;
		return ConvertToQueuedWorkPriority(Owner->GetPriority());
	}

	bool SetPriority(EQueuedWorkPriority QueuedWorkPriority) final
	{
		using namespace UE::DerivedData;
		Owner->SetPriority(ConvertFromQueuedWorkPriority(QueuedWorkPriority));
		return true;
	}

	bool Cancel() final
	{
		Owner->Cancel();
		return true;
	}

	void Wait() final
	{
		Owner->Wait();
	}

	bool WaitWithTimeout(float TimeLimitSeconds) final
	{
		const double TimeLimit = FPlatformTime::Seconds() + TimeLimitSeconds;
		if (Poll())
		{
			return true;
		}
		do
		{
			FPlatformProcess::Sleep(0.005f);
			if (Poll())
			{
				return true;
			}
		}
		while (FPlatformTime::Seconds() < TimeLimit);
		return false;
	}

	bool Poll() const final
	{
		return Owner->Poll();
	}

	static bool IsTextureValidForBuilding(UTexture& Texture, ETextureCacheFlags Flags, bool bInCPUAccessible, bool& bOutUseCompositeTexture)
	{
		bOutUseCompositeTexture = false;

		const int32 NumBlocks = Texture.Source.GetNumBlocks();
		const int32 NumLayers = Texture.Source.GetNumLayers();
		if (NumBlocks < 1 || NumLayers < 1)
		{
			UE_LOGF(LogTexture, Error, "Texture has no source data: %ls", *Texture.GetPathName());
			return false;
		}

		for (int LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			ETextureSourceFormat TSF = Texture.Source.GetFormat(LayerIndex);
			ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(TSF);

			if ( RawFormat == ERawImageFormat::Invalid )
			{
				UE_LOGF(LogTexture, Error, "Texture %ls has source art in an invalid format.", *Texture.GetPathName());
				return false;
			}

			// valid TSF should round-trip :
			check( FImageCoreUtils::ConvertToTextureSourceFormat( RawFormat ) == TSF );
		}

		int32 BlockSizeX = 0;
		int32 BlockSizeY = 0;
		TArray<FIntPoint> BlockSizes;
		BlockSizes.Reserve(NumBlocks);
		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			FTextureSourceBlock SourceBlock;
			Texture.Source.GetBlock(BlockIndex, SourceBlock);
			if (SourceBlock.NumMips > 0 && SourceBlock.NumSlices > 0)
			{
				BlockSizes.Emplace(SourceBlock.SizeX, SourceBlock.SizeY);
				BlockSizeX = FMath::Max(BlockSizeX, SourceBlock.SizeX);
				BlockSizeY = FMath::Max(BlockSizeY, SourceBlock.SizeY);
			}
		}

		for (int32 BlockIndex = 0; BlockIndex < BlockSizes.Num(); ++BlockIndex)
		{
			const int32 MipBiasX = FMath::CeilLogTwo(BlockSizeX / BlockSizes[BlockIndex].X);
			const int32 MipBiasY = FMath::CeilLogTwo(BlockSizeY / BlockSizes[BlockIndex].Y);
			if (MipBiasX != MipBiasY)
			{
				UE_LOGF(LogTexture, Error, "Texture %ls has blocks with mismatched aspect ratios", *Texture.GetPathName());
				return false;
			}
		}
		
		bool bCompositeTextureViable = Texture.GetCompositeTexture() && Texture.CompositeTextureMode != CTM_Disabled && Texture.GetCompositeTexture()->Source.IsValid();
		if (bInCPUAccessible)
		{
			bCompositeTextureViable = false;
		}
		bool bMatchingBlocks = bCompositeTextureViable && (Texture.GetCompositeTexture()->Source.GetNumBlocks() == Texture.Source.GetNumBlocks());
		
		if (bCompositeTextureViable)
		{
			if (!bMatchingBlocks)
			{
				UE_LOGF(LogTexture, Warning, "Issue while building %ls : Composite texture UDIM block counts do not match. Composite texture will be ignored", *Texture.GetPathName());
			}
		}

		bOutUseCompositeTexture = bMatchingBlocks;

		// TODO: Add validation equivalent to that found in FTextureCacheDerivedDataWorker::BuildTexture for virtual textures
		//		 if virtual texture support is added for this code path.
		if (!EnumHasAnyFlags(Flags, ETextureCacheFlags::ForVirtualTextureStreamingBuild))
		{
			// Only support single Block/Layer here (Blocks and Layers are intended for VT support)
			if (NumBlocks > 1)
			{
				// This can happen if user attempts to import a UDIM without VT enabled
				UE_LOGF(LogTexture, Log, "Texture %ls was imported as UDIM with %d blocks but VirtualTexturing is not enabled, only the first block will be available",
					*Texture.GetPathName(), NumBlocks);
			}
			if (NumLayers > 1)
			{
				// This can happen if user attempts to use lightmaps or other layered VT without VT enabled
				UE_LOGF(LogTexture, Log, "Texture %ls has %d layers but VirtualTexturing is not enabled, only the first layer will be available",
					*Texture.GetPathName(), NumLayers);
			}
		}

		return true;
	}

	static FTexturePlatformData::FStructuredDerivedDataKey GetKey(
		const UE::DerivedData::FBuildDefinition& BuildDefinition, 
		const UE::DerivedData::FBuildDefinition* TilingBuildDefinitionKey,
		const UE::DerivedData::FBuildDefinition* DeTilingBuildDefinitionKey,
		const UE::DerivedData::FBuildDefinition* DecodeBuildDefinitionKey,
		const UTexture& Texture, 
		bool bUseCompositeTexture)
	{
		// DDC2 Key SerializeForKey is here!
		FTexturePlatformData::FStructuredDerivedDataKey Key;
		if (TilingBuildDefinitionKey != nullptr)
		{
			Key.TilingBuildDefinitionKey = TilingBuildDefinitionKey->GetKey().Hash;
		}
		if (DeTilingBuildDefinitionKey != nullptr)
		{
			Key.DeTilingBuildDefinitionKey = DeTilingBuildDefinitionKey->GetKey().Hash;
		}
		if (DecodeBuildDefinitionKey)
		{
			Key.DecodeBuildDefinitionKey = DecodeBuildDefinitionKey->GetKey().Hash;
		}
		Key.BuildDefinitionKey = BuildDefinition.GetKey().Hash;
		Key.SourceGuid = Texture.Source.GetId();
		if (bUseCompositeTexture && Texture.GetCompositeTexture())
		{
			Key.CompositeSourceGuid = Texture.GetCompositeTexture()->Source.GetId();
		}
		//UE_LOGF(LogTexture, Display, "GetKey[%ls] -> %ls / %ls / %ls / %ls", *Texture.GetPathName(), *LexToString(Key.BuildDefinitionKey), *LexToString(Key.TilingBuildDefinitionKey), *LexToString(Key.DeTilingBuildDefinitionKey), *LexToString(Key.DecodeBuildDefinitionKey));
		return Key;
	}

	static void AddParentBuildOutputsAsInputs(UE::DerivedData::FBuildDefinitionBuilder& InDefinitionBuilder, const UE::DerivedData::FBuildKey& InParentBuildKey, 
		const FGuid& InCompressionCacheId, int32 InNumMips, int32 InNumStreamingMips)
	{
		if (InCompressionCacheId.IsValid())
		{
			// Not actually read by the worker - just used to make a different key - and we want to rebuild when they do!
			FCbWriter Writer;
			Writer.BeginObject();
			Writer.AddUuid("CompressionCacheId", InCompressionCacheId);
			Writer.EndObject();
			InDefinitionBuilder.AddConstant("CompressionCacheId", Writer.Save().AsObject());
		}

		InDefinitionBuilder.AddInputBuild(UTF8TEXTVIEW("EncodedTextureDescription"), { InParentBuildKey, UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("EncodedTextureDescription")) });
		InDefinitionBuilder.AddInputBuild(UTF8TEXTVIEW("EncodedTextureExtendedData"), { InParentBuildKey, UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("EncodedTextureExtendedData")) });

		//
		// NOTE! We define all streaming mips as inputs here, which depending on what our parent build is
		// might not actually exist due to packed mip tails. However, we require that the parent build emit
		// the streaming mip as an empty buffer so we don't have to know what the packed mip setup is ahead of
		// time.
		//
		if (InNumMips > InNumStreamingMips)
		{
			InDefinitionBuilder.AddInputBuild(UTF8TEXTVIEW("MipTail"), { InParentBuildKey, UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("MipTail")) });
		}

		for (int32 MipIndex = 0; MipIndex < InNumStreamingMips; MipIndex++)
		{
			TUtf8StringBuilder<10> MipName;
			MipName << "Mip" << MipIndex;
			InDefinitionBuilder.AddInputBuild(MipName, { InParentBuildKey, UE::DerivedData::FValueId::FromName(MipName) });
		}

		//
		// Any CPU texture stuff needs to get passed through even though we don't touch it.
		//
		InDefinitionBuilder.AddInputBuild(UTF8TEXTVIEW("CPUCopyImageInfo"), { InParentBuildKey, UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("CPUCopyImageInfo")) });
		InDefinitionBuilder.AddInputBuild(UTF8TEXTVIEW("CPUCopyRawData"), { InParentBuildKey, UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("CPUCopyRawData")) });
	}

	static UE::DerivedData::FBuildDefinition CreateTilingDefinition(
		UE::DerivedData::IBuild& InBuild,
		UTexture* InTexture,
		const FTextureBuildSettings& InBuildSettings,
		FEncodedTextureDescription* InTextureDescription, // only valid if our textures can generate this pre build
		FEncodedTextureExtendedData* InTextureExtendedData, // only valid if our textures can generate this pre build
		const UE::DerivedData::FBuildDefinition& InParentBuildDefinition,
		const UE::FSharedString& InDefinitionDebugName,
		const UE::FUtf8SharedString& InBuildFunctionName
	)
	{			
		const FTextureEngineParameters EngineParameters = GenerateTextureEngineParameters();

		//
		// We always consume an unpacked texture (i.e. extended data == nullptr)
		//
		const FTextureSource& Source = InTexture->Source;
		int32 InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices;
		int32 InputTextureNumMips = TextureCompressorModule->GetMipCountForBuildSettings(Source.GetSizeX(), Source.GetSizeY(), Source.GetNumSlices(), Source.GetNumMips(), InBuildSettings, InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices);
		int32 InputTextureNumStreamingMips = GetNumStreamingMipsDirect(InputTextureNumMips, InBuildSettings.bCubemap, InBuildSettings.bVolume, InBuildSettings.bTextureArray, nullptr, EngineParameters);

		//
		// A child definition consumes a parent definition and swizzles it. However it
		// needs to know ahead of time the total mip count and the streaming mip count
		//
		const UE::DerivedData::FBuildKey InParentBuildKey = InParentBuildDefinition.GetKey();

		UE::DerivedData::FBuildDefinitionBuilder DefinitionBuilder = InBuild.CreateDefinition(InDefinitionDebugName, InBuildFunctionName);

		AddParentBuildOutputsAsInputs(DefinitionBuilder, InParentBuildKey, InTexture->CompressionCacheId, InputTextureNumMips, InputTextureNumStreamingMips);

		// The tiling build generates the extended data - however it needs the LODBias to do so.
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddInteger("LODBias", InBuildSettings.LODBias);
		Writer.EndObject();

		DefinitionBuilder.AddConstant(UTF8TEXTVIEW("LODBias"), Writer.Save().AsObject());

		return DefinitionBuilder.Build();
	}


	static UE::DerivedData::FBuildDefinition CreateDetileDefinition(
		UE::DerivedData::IBuild& InBuild,
		UTexture* InTexture,
		const FTextureBuildSettings& InBuildSettings,
		const UE::DerivedData::FBuildDefinition& InParentBuildDefinition,
		const UE::FSharedString& InDefinitionDebugName
	)
	{
		//
		// This consumes a tiled texture and converts back to a linear representation.
		//

		// We have the same outputs as inputs, and everything comes from the build

		const FTextureEngineParameters EngineParameters = GenerateTextureEngineParameters();
		const FTextureSource& Source = InTexture->Source;
		int32 InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices;
		int32 InputTextureNumMips = TextureCompressorModule->GetMipCountForBuildSettings(Source.GetSizeX(), Source.GetSizeY(), Source.GetNumSlices(), Source.GetNumMips(), InBuildSettings, InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices);
		int32 InputTextureNumStreamingMips = GetNumStreamingMipsDirect(InputTextureNumMips, InBuildSettings.bCubemap, InBuildSettings.bVolume, InBuildSettings.bTextureArray, nullptr, EngineParameters);

		UE::DerivedData::FBuildDefinitionBuilder DefinitionBuilder = InBuild.CreateDefinition(InDefinitionDebugName, InBuildSettings.TilerEvenIfNotSharedLinear->GetDetileBuildFunctionName());

		const UE::DerivedData::FBuildKey InParentBuildKey = InParentBuildDefinition.GetKey();

		AddParentBuildOutputsAsInputs(DefinitionBuilder, InParentBuildKey, InTexture->CompressionCacheId, InputTextureNumMips, InputTextureNumStreamingMips);

		return DefinitionBuilder.Build();
	}

	static UE::DerivedData::FBuildDefinition CreateDecodeDefinition(
		UE::DerivedData::IBuild& InBuild,
		UTexture* InTexture,
		const FTextureBuildSettings& InBuildSettings,
		const UE::DerivedData::FBuildDefinition& InParentBuildDefinition,
		const UE::FSharedString& InDefinitionDebugName
	)
	{
		//
		// This consumes an encoded texture and converts it back to RGBA8/RGBA16F
		//
		const FTextureEngineParameters EngineParameters = GenerateTextureEngineParameters();
		const FTextureSource& Source = InTexture->Source;
		int32 InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices;
		int32 InputTextureNumMips = TextureCompressorModule->GetMipCountForBuildSettings(Source.GetSizeX(), Source.GetSizeY(), Source.GetNumSlices(), Source.GetNumMips(), InBuildSettings, InputTextureMip0SizeX, InputTextureMip0SizeY, InputTextureMip0NumSlices);
		int32 InputTextureNumStreamingMips = GetNumStreamingMipsDirect(InputTextureNumMips, InBuildSettings.bCubemap, InBuildSettings.bVolume, InBuildSettings.bTextureArray, nullptr, EngineParameters);

		const ITextureFormat* BaseTextureFormat = GetTextureFormatManager()->FindTextureFormat(InBuildSettings.BaseTextureFormatName);

		UE::DerivedData::FBuildDefinitionBuilder DefinitionBuilder = InBuild.CreateDefinition(InDefinitionDebugName, BaseTextureFormat->GetDecodeBuildFunctionName());

		const UE::DerivedData::FBuildKey InParentBuildKey = InParentBuildDefinition.GetKey();
		AddParentBuildOutputsAsInputs(DefinitionBuilder, InParentBuildKey, InTexture->CompressionCacheId, InputTextureNumMips, InputTextureNumStreamingMips);


		{
			FCbWriter Writer;
			Writer.BeginObject();
			Writer.AddString("BaseFormatName", InBuildSettings.BaseTextureFormatName.ToString());
			Writer.AddInteger("BaseFormatVersion", BaseTextureFormat->GetVersion(InBuildSettings.BaseTextureFormatName));
			Writer.AddInteger("LODBias", InBuildSettings.LODBias);
			Writer.AddInteger("bSRGB", InBuildSettings.bSRGB);
			Writer.EndObject();
			DefinitionBuilder.AddConstant(UTF8TEXTVIEW("TextureInfo"), Writer.Save().AsObject());
		}

		return DefinitionBuilder.Build();
	}

	static ITextureCompressorModule* TextureCompressorModule;
	static bool LoadModules()
	{
		FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		TextureCompressorModule = &FModuleManager::LoadModuleChecked<ITextureCompressorModule>(TEXTURE_COMPRESSOR_MODULENAME);
		return true;
	}
	
	// Stuff that we get as a result of the build.
	FBuildResults BuildResults;

	// Controls for what optional build outputs we want.
	FBuildResultOptions BuildResultOptions;

	// Build bureaucracy
	TOptional<UE::DerivedData::FRequestOwner> Owner;

	UE::DerivedData::FOptionalBuildSession BuildSession;
	UE::TextureDerivedData::FTextureGenericBuildInputResolver InputResolver;

	FRWLock Lock;
}; // end DDC2 fetch/build task (FTextureBuildTask)

/* static */ ITextureCompressorModule* FTextureBuildTask::TextureCompressorModule = 0;

FTextureAsyncCacheDerivedDataTask* CreateTextureBuildTask(
	UTexture& Texture,
	FTexturePlatformData& DerivedData,
	const FTextureBuildSettings* SettingsFetch,
	const FTextureBuildSettings& SettingsFetchOrBuild,
	const FTexturePlatformData::FTextureEncodeResultMetadata* FetchMetadata,
	const FTexturePlatformData::FTextureEncodeResultMetadata* FetchOrBuildMetadata,
	EQueuedWorkPriority Priority,
	ETextureCacheFlags Flags)
{
	using namespace UE;
	using namespace UE::DerivedData;

	// If we are tiling, we need to alter the build settings to act as though it's 
	// for the linear base format for the build function - the tiling itself will be
	// a separate build function that will consume the output of that.
	// We have to do this here because if we do it where build settings are created, then
	// the DDC key that is externally visible won't know anything about the tiling and
	// the de-dupe code in BeginCacheForCookedPlatformData will delete the tiling build.
	TOptional<FTextureBuildSettings> BaseSettingsFetch;
	TOptional<FTextureBuildSettings> BaseSettingsFetchOrBuild;
	FUtf8SharedString TilingFunctionName;
	const FTextureBuildSettings* UseSettingsFetch = SettingsFetch;
	const FTextureBuildSettings* UseSettingsFetchOrBuild = &SettingsFetchOrBuild;
	if (SettingsFetchOrBuild.Tiler)
	{
		TilingFunctionName = SettingsFetchOrBuild.Tiler->GetBuildFunctionName();

		if (SettingsFetch)
		{
			BaseSettingsFetch = *SettingsFetch;
			BaseSettingsFetch->TextureFormatName = BaseSettingsFetch->BaseTextureFormatName;
			BaseSettingsFetch->Tiler = nullptr;
			UseSettingsFetch = BaseSettingsFetch.GetPtrOrNull();
		}
		BaseSettingsFetchOrBuild = SettingsFetchOrBuild;
		BaseSettingsFetchOrBuild->TextureFormatName = BaseSettingsFetchOrBuild->BaseTextureFormatName;
		UseSettingsFetchOrBuild = BaseSettingsFetchOrBuild.GetPtrOrNull();
	}
	
	if (FUtf8SharedString FunctionName = FindTextureBuildFunction(UseSettingsFetchOrBuild->TextureFormatName); !FunctionName.IsEmpty())
	{
		return new FTextureBuildTask(Texture, DerivedData, FunctionName, TilingFunctionName, UseSettingsFetch, *UseSettingsFetchOrBuild, FetchMetadata, FetchOrBuildMetadata, Priority, Flags);

	}
	return nullptr;
}

FTexturePlatformData::FStructuredDerivedDataKey CreateTextureDerivedDataKey(
	UTexture& Texture,
	ETextureCacheFlags CacheFlags,
	const FTextureBuildSettings& Settings)
{
	using namespace UE;
	using namespace UE::DerivedData;

	TOptional<FTextureBuildSettings> BaseSettings;
	const FTextureBuildSettings* UseSettings = &Settings;
	FUtf8SharedString TilingFunctionName;
	if (Settings.Tiler)
	{
		TilingFunctionName = Settings.Tiler->GetBuildFunctionName();

		BaseSettings = Settings;
		BaseSettings->TextureFormatName = BaseSettings->BaseTextureFormatName;
		UseSettings = BaseSettings.GetPtrOrNull();
	}

	if (FUtf8SharedString FunctionName = FindTextureBuildFunction(UseSettings->TextureFormatName); !FunctionName.IsEmpty())
	{
		IBuild& Build = GetBuild();

		TStringBuilder<256> TexturePath;
		Texture.GetPathName(nullptr, TexturePath);

		bool bUseCompositeTexture = false;
		if (FTextureBuildTask::IsTextureValidForBuilding(Texture, CacheFlags, Settings.bCPUAccessible, bUseCompositeTexture))
		{
			check( Texture.Source.GetNumLayers() == 1 ); // no SettingsPerLayer here
			FBuildDefinition Definition = FTextureBuildTask::CreateDefinition(Build, Texture, TexturePath.ToView(), FunctionName, *UseSettings, bUseCompositeTexture);
			FBuildDefinition* ParentDefinition = &Definition;
			TOptional<FBuildDefinition> TilingDefinition;
			if (TilingFunctionName.IsEmpty() == false)
			{
				TilingDefinition.Emplace(FTextureBuildTask::CreateTilingDefinition(Build, &Texture, *UseSettings, nullptr, nullptr, *ParentDefinition, TexturePath.ToView(), TilingFunctionName));
				ParentDefinition = TilingDefinition.GetPtrOrNull();
			}

			TOptional<FBuildDefinition> DetileDefinition;
			TOptional<FBuildDefinition> DecodeDefinition;
			if (Settings.bDecodeForPCUsage)
			{
				// If the format emits a tiler, we might need to detile:
				if (Settings.TilerEvenIfNotSharedLinear)
				{
					DetileDefinition.Emplace(FTextureBuildTask::CreateDetileDefinition(Build, &Texture, *UseSettings, *ParentDefinition, TexturePath.ToView()));
					ParentDefinition = DetileDefinition.GetPtrOrNull();
				}

				// Get the texture description with alpha - for our purposes (detecting needs decode) alpha present/no doesn't matter so we can get it all beforehand.
				FEncodedTextureDescription TextureDescription;
				UseSettings->GetEncodedTextureDescriptionFromSourceMips(&TextureDescription, UseSettings->BaseTextureFormat, 
					Texture.Source.GetSizeX(), Texture.Source.GetSizeY(), Texture.Source.GetNumSlices(), Texture.Source.GetNumMips(),
					true);

				// We use LODBias=0 here because the editor doesn't strip the top mips - so we could need them to view even if they aren't deployed.
				if (UE::TextureBuildUtilities::TextureNeedsDecodeForPC(TextureDescription.PixelFormat, TextureDescription.GetMipWidth(0), TextureDescription.GetMipHeight(0)))
				{
					DecodeDefinition.Emplace(FTextureBuildTask::CreateDecodeDefinition(Build, &Texture, *UseSettings, *ParentDefinition, TexturePath.ToView()));
					ParentDefinition = DecodeDefinition.GetPtrOrNull();
				}
			}

			return FTextureBuildTask::GetKey(Definition, TilingDefinition.GetPtrOrNull(), DetileDefinition.GetPtrOrNull(), DecodeDefinition.GetPtrOrNull(), Texture, bUseCompositeTexture);
		}
	}
	return FTexturePlatformData::FStructuredDerivedDataKey();
}

#endif // WITH_EDITOR
