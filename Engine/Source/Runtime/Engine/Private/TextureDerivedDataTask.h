// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	TextureDerivedDataTask.h: Tasks to update texture DDC.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

#include "Async/AsyncWork.h"
#include "DerivedDataCacheKeyProxy.h"
#include "Engine/Texture2D.h"
#include "IImageWrapperModule.h"
#include "ImageCore.h"
#include "TextureCompiler.h"
#include "Misc/EnumClassFlags.h"
#include "TextureCompressorModule.h"
#include "TextureEncodingSettings.h"

#endif // WITH_EDITOR


enum
{
	/**
	*	The number of mips to store "inline". This is the number of mips, counting from the smallest, that are
	*	always loaded and never streamed.
	*/
	NUM_INLINE_DERIVED_MIPS = 7,
};

#if WITH_EDITOR

namespace UE::DerivedData { class FBuildOutput; }

void GetTextureDerivedDataKeySuffix(const UTexture& Texture, const FTextureBuildSettings* BuildSettingsPerLayer, FString& OutKeySuffix);
int64 PutDerivedDataInCache(FTexturePlatformData* DerivedData, const FString& DerivedDataKeySuffix, const FStringView& TextureName, bool bForceAllMipsToBeInlined, bool bReplaceExistingDDC);

enum class ETextureCacheFlags : uint32
{
	None			= 0x00,
	Async			= 0x01,
	ForceRebuild	= 0x02,
	
	/** 
	* If specified, all mips will be loaded in to BulkData at the end of the Cache() operation, prior to return. Without this, 
	* only "Inline" mips will be in BulkData (for streaming textures, typically the smallest NUM_INLINE_DERIVED_MIPS mips).
	*/
	InlineMips		= 0x08,	
	AllowAsyncBuild	= 0x10,
	ForDDCBuild		= 0x20,
	RemoveSourceMipDataAfterCache = 0x40,
	AllowAsyncLoading = 0x80,
	ForVirtualTextureStreamingBuild = 0x100,
};

ENUM_CLASS_FLAGS(ETextureCacheFlags);

// Everything required to get the texture source data.
struct FTextureSourceLayerData
{
	ERawImageFormat::Type ImageFormat = ERawImageFormat::Invalid;
	EGammaSpace SourceGammaSpace = EGammaSpace::Invalid;
};

struct FTextureSourceBlockData
{
	TArray<TArray<FImage>> MipsPerLayer; // @@ use FImageView instead
	int32 BlockX = 0;
	int32 BlockY = 0;
	int32 SizeInBlocksX = 1; // Normally each blocks covers a 1x1 block area
	int32 SizeInBlocksY = 1;
	int32 SizeX = 0;
	int32 SizeY = 0;
	int32 NumMips = 0;
	int32 NumSlices = 0;
	int32 MipBias = 0;
};

struct FTextureSourceData
{
	FTextureSourceData()
		: SizeInBlocksX(0)
		, SizeInBlocksY(0)
		, BlockSizeX(0)
		, BlockSizeY(0)
		, bValid(false)
	{}

	// Clear the current source data and make it into a placeholder texture.
	void InitAsPlaceholder();

	void Init(UTexture& InTexture, TextureMipGenSettings InMipGenSettings, bool bInCubeMap, bool bInTextureArray, bool bInVolumeTexture, ETexturePowerOfTwoSetting::Type InPow2Setting, int32 InResizeDuringBuildX, int32 InResizeDuringBuildY, bool bAllowAsyncLoading);
	bool IsValid() const { return bValid; }

	// ImageWrapperModule is not used
	void GetSourceMips(FTextureSource& Source, IImageWrapperModule* InImageWrapper = nullptr);

	void ReleaseMemory()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Texture.ReleaseMemory);

		Blocks.Empty();
		Layers.Empty();
		bValid = false;
	}

	TArray<TPair<FLinearColor, FLinearColor>> LayerChannelMinMax; // Key == Min, Value == Max

	FString TextureFullName;
	TArray<FTextureSourceLayerData> Layers;
	TArray<FTextureSourceBlockData> Blocks;
	int32 SizeInBlocksX;
	int32 SizeInBlocksY;
	int32 BlockSizeX;
	int32 BlockSizeY;
	bool bValid;
};

/**
 * DDC1 texture fetch/build class.
 * 
 * This class is only used directly when we _aren't_ async, however the async
 * class will still route through this for doing the actual fetch/build.
 */
class FTextureCacheDerivedDataWorker : public FNonAbandonableTask
{
	/** Texture compressor module, must be loaded in the game thread. see FModuleManager::WarnIfItWasntSafeToLoadHere() */
	ITextureCompressorModule* Compressor;
	/** Image wrapper module, must be loaded in the game thread. see FModuleManager::WarnIfItWasntSafeToLoadHere() */
	IImageWrapperModule* ImageWrapper;
	/** Where to store derived data. */
	FTexturePlatformData* DerivedData;
	/** The texture for which derived data is being cached. */
	UTexture& Texture;

	/** The name of the texture we are building. Here to avoid GetPathName calls off the main thread. */
	FString TexturePathName;

	/** Compression settings. We need two for when we are in the fallback case. We have
	*	to do this out here so that we can generate keys before knowing which one we'll use */
	TArray<FTextureBuildSettings> BuildSettingsPerLayerFetchFirst;
	TArray<FTextureBuildSettings> BuildSettingsPerLayerFetchOrBuild;

	// Metadata for the different fetches we could do. Stored to DerivedData once we know.
	FTexturePlatformData::FTextureEncodeResultMetadata FetchFirstMetadata;
	FTexturePlatformData::FTextureEncodeResultMetadata FetchOrBuildMetadata;

	/** Derived data key suffix that we ended up using. */
	FString KeySuffix;

	/** Source mip images. */
	FTextureSourceData TextureData;
	/** Source mip images of the composite texture (e.g. normal map for compute roughness). Not necessarily in RGBA32F, usually only top mip as other mips need to be generated first */
	FTextureSourceData CompositeTextureData;
	/** Texture cache flags. */
	ETextureCacheFlags CacheFlags;
	/** Have many bytes were loaded from DDC or built (for telemetry) */
	int64 BytesCached = 0;
	/** Estimate of the peak amount of memory required to complete this task. */
	int64 RequiredMemoryEstimate = -1;

	/** true if caching has succeeded. */
	bool bSucceeded = false;
	/** true if caching was tried and failed; if bSucceeded & bTriedAndFailed are both false, it has not been tried yet */
	bool bTriedAndFailed = false;
	/** true if the derived data was pulled from DDC */
	bool bLoadedFromDDC = false;
	/** Will be set to true if we should abort compilation */
	UE::Tasks::FCancellationToken CancellationToken;

public:

	/** Initialization constructor. */
	FTextureCacheDerivedDataWorker(
		ITextureCompressorModule* InCompressor,
		FTexturePlatformData* InDerivedData,
		UTexture* InTexture,
		const FTextureBuildSettings* InSettingsPerLayerFetchFirst, // can be nullptr
		const FTextureBuildSettings* InSettingsPerLayerFetchOrBuild,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchFirstMetadata, // can be nullptr
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchOrBuildMetadata, // can be nullptr
		ETextureCacheFlags InCacheFlags);

	/** Does the work to cache derived data. Safe to call from any thread. */
	void DoWork();

	/** Finalize work. Must be called ONLY by the game thread! */
	void Finalize();

	/** Expose bytes cached for telemetry. */
	int64 GetBytesCached() const
	{
		return BytesCached;
	}

	/** Estimate of the peak amount of memory required to complete this task. */
	int64 GetRequiredMemoryEstimate() const
	{
		return RequiredMemoryEstimate;
	}

	/** Expose how the resource was returned for telemetry. */
	bool WasLoadedFromDDC() const
	{
		return bLoadedFromDDC;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTextureCacheDerivedDataWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void MarkAsCanceled()
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationCancelable())
		{
			CancellationToken.Cancel();
		}
	}
};

struct FTextureAsyncCacheDerivedDataTask
{
	virtual ~FTextureAsyncCacheDerivedDataTask() = default;
	virtual void Finalize(bool& bOutFoundInCache, uint64& OutProcessedByteCount) = 0;
	virtual EQueuedWorkPriority GetPriority() const = 0;
	virtual bool SetPriority(EQueuedWorkPriority QueuedWorkPriority) = 0;
	virtual bool Cancel() = 0;
	virtual void Wait() = 0;
	virtual bool WaitWithTimeout(float TimeLimitSeconds) = 0;
	virtual bool Poll() const = 0;
};

//
// DDC1 async texture fetch/build task.
//
class FTextureAsyncCacheDerivedDataWorkerTask final : public FTextureAsyncCacheDerivedDataTask, public FAsyncTask<FTextureCacheDerivedDataWorker>
{
public:
	FTextureAsyncCacheDerivedDataWorkerTask(
		FQueuedThreadPool* InQueuedPool,
		ITextureCompressorModule* InCompressor,
		FTexturePlatformData* InDerivedData,
		UTexture* InTexture,
		const FTextureBuildSettings* InSettingsPerLayerFetchFirst,
		const FTextureBuildSettings* InSettingsPerLayerFetchOrBuild,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchFirstMetadata,
		const FTexturePlatformData::FTextureEncodeResultMetadata* InFetchOrBuildMetadata,
		ETextureCacheFlags InCacheFlags
		)
		: FAsyncTask<FTextureCacheDerivedDataWorker>(
			InCompressor,
			InDerivedData,
			InTexture,
			InSettingsPerLayerFetchFirst,
			InSettingsPerLayerFetchOrBuild,
			InFetchFirstMetadata,
			InFetchOrBuildMetadata,
			InCacheFlags
			)
		, QueuedPool(InQueuedPool)
	{
	}

	void Finalize(bool& bOutFoundInCache, uint64& OutProcessedByteCount) final
	{
		GetTask().Finalize();
		bOutFoundInCache = GetTask().WasLoadedFromDDC();
		OutProcessedByteCount = GetTask().GetBytesCached();
	}

	EQueuedWorkPriority GetPriority() const final
	{
		return FAsyncTask<FTextureCacheDerivedDataWorker>::GetPriority();
	}

	bool SetPriority(EQueuedWorkPriority QueuedWorkPriority) final
	{
		return FAsyncTask<FTextureCacheDerivedDataWorker>::Reschedule(QueuedPool, QueuedWorkPriority);
	}

	bool Cancel() final
	{
		return FAsyncTask<FTextureCacheDerivedDataWorker>::IsDone() || FAsyncTask<FTextureCacheDerivedDataWorker>::Cancel();
	}

	void Wait() final
	{
		FAsyncTask<FTextureCacheDerivedDataWorker>::EnsureCompletion();
	}

	bool WaitWithTimeout(float TimeLimitSeconds) final
	{
		return FAsyncTask<FTextureCacheDerivedDataWorker>::WaitCompletionWithTimeout(TimeLimitSeconds);
	}

	bool Poll() const final
	{
		return FAsyncTask<FTextureCacheDerivedDataWorker>::IsWorkDone();
	}

private:
	FQueuedThreadPool* QueuedPool;
};

// Creates the DDC2 texture fetch/build task.
FTextureAsyncCacheDerivedDataTask* CreateTextureBuildTask(
	UTexture& Texture,
	FTexturePlatformData& DerivedData,
	const FTextureBuildSettings* SettingsFetchFirst, // can be nullptr
	const FTextureBuildSettings& SettingsFetchOrBuild,
	const FTexturePlatformData::FTextureEncodeResultMetadata* FetchMetadata,
	const FTexturePlatformData::FTextureEncodeResultMetadata* FetchOrBuildMetadata,
	EQueuedWorkPriority Priority,
	ETextureCacheFlags Flags);

// This needs to match the key generated by the build, otherwise the build will get continually kicked
// by CachePlatformData().
FTexturePlatformData::FStructuredDerivedDataKey CreateTextureDerivedDataKey(
	UTexture& Texture,
	ETextureCacheFlags CacheFlags,
	const FTextureBuildSettings& Settings);

#endif // WITH_EDITOR
