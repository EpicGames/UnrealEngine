// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureEncodingSettings.h"
#include "TextureEncodingSettingsPrivate.h"
#include "Engine/TextureDefines.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureEncodingSettings)

DEFINE_LOG_CATEGORY_STATIC(LogTextureEncodingSettings, Log, All);

UTextureEncodingProjectSettings::UTextureEncodingProjectSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	bSharedLinearTextureEncoding(false),
	bFinalUsesRDO(false),
	FinalRDOLambda(30), /* OodleTex_RDOLagrangeLambda_Default */
	FinalEffortLevel(ETextureEncodeEffort::Normal),
	FinalUniversalTiling(ETextureUniversalTiling::Disabled),
	bFastUsesRDO(false),
	FastRDOLambda(30), /* OodleTex_RDOLagrangeLambda_Default */
	FastEffortLevel(ETextureEncodeEffort::Normal),
	FastUniversalTiling(ETextureUniversalTiling::Disabled),
	CookUsesSpeed(ETextureEncodeSpeed::Final),
	EditorUsesSpeed(ETextureEncodeSpeed::FinalIfAvailable)
{
}

UTextureEncodingUserSettings::UTextureEncodingUserSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	ForceEncodeSpeed(ETextureEncodeSpeedOverride::Disabled)
{
}

enum class ReadyMask : uint32
{
	Query = 0,
	Project = 0x1,
	User = 0x2,
	Everyone = 0x3
};

#if WITH_EDITORONLY_DATA
static void ConstructResolvedSettings(ReadyMask InReadyMask, UDeveloperSettings* InSettings, FResolvedTextureEncodingSettings* OutResolvedSettings)
{
	// Settings is the last one to get initialized - the others have all had PostInitProperties
	// called, so we can safely retrieve their CDO. This is all so we don't cause a cycle
	// calling GetDefault() - we only call it for classes we know are complete.
	UTextureEncodingProjectSettings const* ProjectSettings = nullptr;
	UTextureEncodingUserSettings const* UserSettings = nullptr;

	if (InReadyMask == ReadyMask::Project)
	{
		ProjectSettings = (UTextureEncodingProjectSettings*)InSettings;
	}
	else if (InReadyMask == ReadyMask::User)
	{
		UserSettings = (UTextureEncodingUserSettings*)InSettings;
	}

	if (ProjectSettings == nullptr)
	{
		ProjectSettings = GetDefault<UTextureEncodingProjectSettings>();
	}
	if (UserSettings == nullptr)
	{
		UserSettings = GetDefault<UTextureEncodingUserSettings>();
	}

	OutResolvedSettings->User.ForceEncodeSpeed = UserSettings->ForceEncodeSpeed;

	OutResolvedSettings->Project.bSharedLinearTextureEncoding = ProjectSettings->bSharedLinearTextureEncoding;
	{
		bool CmdLineSLE;
		if (FParse::Bool(FCommandLine::Get(), TEXT("-ForceSharedLinearTextureEncoding="), CmdLineSLE))
		{
			OutResolvedSettings->Project.bSharedLinearTextureEncoding = CmdLineSLE;
			UE_LOGF(LogTextureEncodingSettings, Display, "Shared linear texture encoding forced to %ls via command line.", *LexToString(CmdLineSLE));
		}
	}
	OutResolvedSettings->Project.bFinalUsesRDO = ProjectSettings->bFinalUsesRDO;
	OutResolvedSettings->Project.bFastUsesRDO = ProjectSettings->bFastUsesRDO;
	OutResolvedSettings->Project.FinalRDOLambda = ProjectSettings->FinalRDOLambda;
	OutResolvedSettings->Project.FastRDOLambda = ProjectSettings->FastRDOLambda;
	OutResolvedSettings->Project.FinalEffortLevel = ProjectSettings->FinalEffortLevel;
	OutResolvedSettings->Project.FinalUniversalTiling = ProjectSettings->FinalUniversalTiling;
	OutResolvedSettings->Project.FastEffortLevel = ProjectSettings->FastEffortLevel;
	OutResolvedSettings->Project.FastUniversalTiling = ProjectSettings->FastUniversalTiling;
	OutResolvedSettings->Project.CookUsesSpeed = ProjectSettings->CookUsesSpeed;
	OutResolvedSettings->Project.EditorUsesSpeed = ProjectSettings->EditorUsesSpeed;

	// Determine what encode speed to use
	{
		const UEnum* EncodeSpeedEnum = StaticEnum<ETextureEncodeSpeed>();

		// Overridden by command line?
		FString CmdLineSpeed;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ForceTextureEncodeSpeed="), CmdLineSpeed))
		{
			int64 Value = EncodeSpeedEnum->GetValueByNameString(CmdLineSpeed);
			if (Value == INDEX_NONE)
			{
				UE_LOGF(LogTextureEncodingSettings, Error, "Invalid value for ForceTextureEncodeSpeed, ignoring. Valid values are the ETextureEncodeSpeed enum (Final, FinalIfAvailable, Fast)");
			}
			else
			{
				OutResolvedSettings->EncodeSpeed = (ETextureEncodeSpeed)Value;
				UE_LOGF(LogTextureEncodingSettings, Display, "Texture Encode Speed forced to %ls via command line.", *EncodeSpeedEnum->GetNameStringByValue(Value));
				return;
			}
		}

		// Overridden by user settings?
		if (OutResolvedSettings->User.ForceEncodeSpeed != ETextureEncodeSpeedOverride::Disabled)
		{
			// enums have same values for payload.
			OutResolvedSettings->EncodeSpeed = (ETextureEncodeSpeed)(OutResolvedSettings->User.ForceEncodeSpeed);
			UE_LOGF(LogTextureEncodingSettings, Display, "Texture Encode Speed forced to %ls via user settings.", *EncodeSpeedEnum->GetNameStringByValue((int64)OutResolvedSettings->EncodeSpeed));
			return;
		}

		// Use project settings
		if (GIsEditor && !IsRunningCommandlet())
		{
			// Interactive editor
			OutResolvedSettings->EncodeSpeed = OutResolvedSettings->Project.EditorUsesSpeed;
			UE_LOGF(LogTextureEncodingSettings, Display, "Texture Encode Speed: %ls (editor).", *EncodeSpeedEnum->GetNameStringByValue((int64)OutResolvedSettings->EncodeSpeed));
		}
		else
		{
			OutResolvedSettings->EncodeSpeed = OutResolvedSettings->Project.CookUsesSpeed;
			UE_LOGF(LogTextureEncodingSettings, Display, "Texture Encode Speed: %ls (cook).", *EncodeSpeedEnum->GetNameStringByValue((int64)OutResolvedSettings->EncodeSpeed));
		}
	}

	// Log encode speeds
	{
		UEnum* EncodeEffortEnum = StaticEnum<ETextureEncodeEffort>();

		UEnum* UniversalTilingEnum = StaticEnum<ETextureUniversalTiling>();

		FString FastRDOString;
		if (OutResolvedSettings->Project.bFastUsesRDO)
		{
			FastRDOString = FString("On");
			if (OutResolvedSettings->Project.FastUniversalTiling != ETextureUniversalTiling::Disabled)
			{
				FastRDOString += TEXT(" UT=");
				FastRDOString += UniversalTilingEnum->GetNameStringByValue((int64)OutResolvedSettings->Project.FastUniversalTiling);
			}
		}
		else
		{
			FastRDOString = FString("Off");
		}

		FString FinalRDOString;
		if (OutResolvedSettings->Project.bFinalUsesRDO)
		{
			FinalRDOString = FString("On");
			if (OutResolvedSettings->Project.FinalUniversalTiling != ETextureUniversalTiling::Disabled)
			{
				FinalRDOString += TEXT(" UT=");
				FinalRDOString += UniversalTilingEnum->GetNameStringByValue((int64)OutResolvedSettings->Project.FinalUniversalTiling);
			}
		}
		else
		{
			FinalRDOString = FString("Off");
		}

		UE_LOGF(LogTextureEncodingSettings, Display, "Oodle Texture Encode Speed settings: Fast: RDO %ls Lambda=%d, Effort=%ls Final: RDO %ls Lambda=%d, Effort=%ls", \
			*FastRDOString, OutResolvedSettings->Project.bFastUsesRDO ? OutResolvedSettings->Project.FastRDOLambda : 0, *(EncodeEffortEnum->GetNameStringByValue((int64)OutResolvedSettings->Project.FastEffortLevel)), \
			*FinalRDOString, OutResolvedSettings->Project.bFinalUsesRDO ? OutResolvedSettings->Project.FinalRDOLambda : 0, *(EncodeEffortEnum->GetNameStringByValue((int64)OutResolvedSettings->Project.FinalEffortLevel)));
	}

	UE_LOGF(LogTextureEncodingSettings, Display, "Shared linear texture encoding: %ls", OutResolvedSettings->Project.bSharedLinearTextureEncoding ? TEXT("Enabled") : TEXT("Disabled"));
}

static void TryToResolveSettings(ReadyMask InReadyMask, UDeveloperSettings* InSettings, FResolvedTextureEncodingSettings** OutResolvedSettings)
{
	static FResolvedTextureEncodingSettings ResolvedSettings;


	// If we are called with anything other than ::Query, then we are in PIP for the settings classes,
	// which a) only happens from the main thread during preinit and b) is guaranteed to happen. We don't know the order,
	// so we just track who we've seen and copy everything once we've got everyone.
	// 
	// Once we've completed engine init, then we could get hit from other threads, however WhoIsReady
	// at that point is guaranteed to be Everyone, so we don't need to worry about atomicity of read-modify-write.
	static uint32 WhoIsReady = 0;
	if (InReadyMask != ReadyMask::Query)
	{		
		WhoIsReady |= (uint32)InReadyMask;
		if (WhoIsReady == (uint32)ReadyMask::Everyone)
		{
			ConstructResolvedSettings(InReadyMask, InSettings, &ResolvedSettings);
		}
	}
	else
	{
		// Two cases:
		// 1 -- We are requesting the properties before engine class init has completed,
		//		meaning we are game thread, so WhoIsReady is NOT everyone, so we need to
		//		do initialization by requesting our CDOs.
		// 2 -- We are requesting the properties _after_ engine class init has completed,
		//		meaning WhoIsReady is Everyone and we can safely return the value. Since
		//		we _know_ WhoIsReady will be Everyone before we go async, we can safely
		//		query it.
		if (WhoIsReady != (uint32)ReadyMask::Everyone)
		{
			GetDefault<UTextureEncodingUserSettings>();
			GetDefault<UTextureEncodingProjectSettings>();
			check(WhoIsReady == (uint32)ReadyMask::Everyone);
		}
	}

	if (OutResolvedSettings)
	{
		*OutResolvedSettings = &ResolvedSettings;
	}
}

#endif // WITH_EDITORONLY_DATA

FResolvedTextureEncodingSettings const& FResolvedTextureEncodingSettings::Get()
{
#if WITH_EDITORONLY_DATA
	FResolvedTextureEncodingSettings* ResolvedSettings = nullptr;
	TryToResolveSettings(ReadyMask::Query, nullptr, &ResolvedSettings);
	return *ResolvedSettings;
#else
	checkNoEntry();
	static FResolvedTextureEncodingSettings DefaultSettings;
	return DefaultSettings;
#endif
}

void UTextureEncodingUserSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITORONLY_DATA
	TryToResolveSettings(ReadyMask::User, this, nullptr);
#endif
}

void UTextureEncodingProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITORONLY_DATA
	TryToResolveSettings(ReadyMask::Project, this, nullptr);
#endif
}
