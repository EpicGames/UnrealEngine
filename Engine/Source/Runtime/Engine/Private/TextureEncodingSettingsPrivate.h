// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TextureEncodingSettings.h"
#include "TextureEncodingSettingsPrivate.generated.h"

// 
// Encoding can either use the "Final" or "Fast" speeds, for supported encoders (e.g. Oodle)
// Encode speed settings have no effect on encoders that don't support encode speed, currently limited to Oodle.
//
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Texture Encoding"), MinimalAPI)
class UTextureEncodingProjectSettings : public UDeveloperSettings
{
	//
	// Anything added here should be added to FResolvedTextureEncodingSettings!
	//
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void PostInitProperties() override;

	// If true, platforms that want to take a linearly encoded texture and then tile them
	// will try to reuse the linear texture rather than encode it for every platform. This can result in
	// massive speedups for texture building as tiling is very fast compared to encoding. So instead of:
	//
	//	Host Platform: Linear encode
	//	Console 1: Linear encode + platform specific tile
	//	Console 2: Linear encode + platform specific tile
	//
	// you instead get:
	//	Host platform: Linear encode
	//	Console 1: fetch linear + platform specific tile
	//	Console 2: fetch linear + platform specific tile
	//
	// Note that this has no effect on cook time, only build time - once the texture is in the DDC this has no
	// effect.
	UPROPERTY(EditAnywhere, config, Category=EncodeSettings, meta = (ConfigRestartRequired = true))
	uint32 bSharedLinearTextureEncoding : 1;

	// If true, Final encode speed enables rate-distortion optimization on supported encoders to
	// decrease *on disc* size of textures in compressed package files.
	// This rate-distortion tradeoff is controlled via "Lambda". The "LossyCompressionAmount" parameter on
	// textures is used to control it. Specific LossyCompressionAmount values correspond to
	// to RDO lambdas of:
	// 
	//	None - Disable RDO for this texture.
	//	Lowest - 1 (Least distortion)
	//	Low - 10
	//	Medium - 20
	//	High - 30
	//	Highest - 40
	// 
	// If set to Default, then the LossyCompressionAmount in the LODGroup for the texture is
	// used. If that is also Default, then the RDOLambda in these settings is used.
	//
	// Note that any distortion introduced is on top of, and likely less than, any introduced by the format itself.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	uint32 bFinalUsesRDO : 1;

	// Ignored if UsesRDO is false. This value is used if a given texture is using "Default" LossyCompressionAmount.
	// Otherwise, the value of LossyCompressionAmount is translated in to a fixed lambda (see UsesRDO tooltip).
	// 
	// Low values (1) represent highest quality (least distortion) results.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (DisplayName = "Final RDO Lambda", UIMin = 1, UIMax = 100, ClampMin = 1, ClampMax = 100, EditCondition = bFinalUsesRDO, ConfigRestartRequired = true))
	int8 FinalRDOLambda;

	// Specifies how much time to take trying for better encoding results.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	ETextureEncodeEffort FinalEffortLevel;	

	// Specifies how to assume textures are laid out on disc. This only applies to Oodle with RDO
	// enabled. 256 KB is a good middle ground. Enabling this will decrease the on-disc
	// sizes of textures for platforms with exposed texture tiling (i.e. consoles), but will slightly increase
	// sizes of textures for platforms with opaque tiling (i.e. desktop).
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	ETextureUniversalTiling FinalUniversalTiling;

	// If true, Final encode speed enables rate-distortion optimization on supported encoders to
	// decrease *on disc* size of textures in compressed package files.
	// This rate-distortion tradeoff is controlled via "Lambda". The "LossyCompressionAmount" parameter on
	// textures is used to control it. Specific LossyCompressionAmount values correspond to
	// to RDO lambdas of:
	// 
	//	None - Disable RDO for this texture.
	//	Lowest - 1 (Least distortion)
	//	Low - 10
	//	Medium - 20
	//	High - 30
	//	Highest - 40
	// 
	// If set to Default, then the LossyCompressionAmount in the LODGroup for the texture is
	// used. If that is also Default, then the RDOLambda in these settings is used.
	//
	// Note that any distortion introduced is on top of, and likely less than, any introduced by the format itself.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	uint32 bFastUsesRDO : 1;

	// Ignored if UsesRDO is false. This value is used if a given texture is using "Default" LossyCompressionAmount.
	// Otherwise, the value of LossyCompressionAmount is translated in to a fixed lambda (see UsesRDO tooltip).
	// 
	// Low values (1) represent highest quality (least distortion) results.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (DisplayName = "Fast RDO Lambda", UIMin = 1, UIMax = 100, ClampMin = 1, ClampMax = 100, EditCondition = bFastUsesRDO, ConfigRestartRequired = true))
	int8 FastRDOLambda;

	// Specifies how much time to take trying for better encode results.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	ETextureEncodeEffort FastEffortLevel;

	// Specifies how to assume textures are laid out on disc. This only applies to Oodle with RDO
	// enabled. 256 KB is a good middle ground. Enabling this will decrease the on-disc
	// sizes of textures for platforms with exposed texture tiling (i.e. consoles), but will slightly increase
	// sizes of textures for platforms with opaque tiling (i.e. desktop).
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeedSettings, meta = (ConfigRestartRequired = true))
	ETextureUniversalTiling FastUniversalTiling;

	// Which encode speed non interactive editor sessions will use (i.e. commandlets)
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeeds, meta = (ConfigRestartRequired = true))
	ETextureEncodeSpeed CookUsesSpeed;

	// Which encode speed everything else uses.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeeds, meta = (ConfigRestartRequired = true))
	ETextureEncodeSpeed EditorUsesSpeed;
};

UCLASS(config = EditorPerProjectUserSettings, defaultconfig, meta = (DisplayName = "Texture Encoding"), MinimalAPI)
class UTextureEncodingUserSettings : public UDeveloperSettings
{
	//
	// Anything added here should be added to FResolvedTextureEncodingSettings!
	//
	GENERATED_UCLASS_BODY()

	ENGINE_API virtual void PostInitProperties() override;

	// Local machine/project setting to force an encode speed, if desired.
	// See the Engine "Texture Encoding" section for details.
	UPROPERTY(config, EditAnywhere, Category = EncodeSpeeds, meta = (ConfigRestartRequired = true))
	ETextureEncodeSpeedOverride ForceEncodeSpeed;
};
