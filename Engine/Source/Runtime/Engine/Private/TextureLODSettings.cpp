// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TextureLODSettings.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTarget.h"
#include "HAL/IConsoleManager.h"
#include "ImageCoreUtils.h"
#include "Interfaces/ITargetPlatform.h"
#include "VirtualTextureEnum.h"

#if ALLOW_OTHER_PLATFORM_CONFIG
#include "UObject/UObjectIterator.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureLODSettings)

int32 GUITextureLODBias = 0;
FAutoConsoleVariableRef CVarUITextureLODBias(
	TEXT("r.UITextureLODBias"),
	GUITextureLODBias,
	TEXT("Extra LOD bias to apply to UI textures. (default=0)"),
	ECVF_Scalability
);

// GlobalTextureLODBias similar to UI Texture LODBias but affects all textures types
// Intended for marketing/cinematics/screencaps, not normal cook usage
//	(eg. set to "-15" to get all possible mip levels in built textures)
// !Note that changing this in an interactive editor may not affect already loaded textures!
//	it is best to set this at startup, via command line or ini config
// (?? could use ECVF_ReadOnly)
int32 GlobalTextureLODBias = 0;
FAutoConsoleVariableRef CVarGlobalTextureLODBias(
	TEXT("r.GlobalTextureLODBias"),
	GlobalTextureLODBias,
	TEXT("Extra LOD bias to apply to all textures.  Can be negative. (default=0)"),
	ECVF_Scalability
);

int32 TextureGroupMaxQualityLevel = -1;
FAutoConsoleVariableRef CVarTextureGroupMaxQualityLevel(
	TEXT("r.TextureGroup.MaxQualityLevel"),
	TextureGroupMaxQualityLevel,
	TEXT("Used to apply texture quality bias to all textures and to limit TextureGroupQualityLevel. (default=-1)"),
	ECVF_Default
);

int32 TextureGroupQualityLevel = -1;
FAutoConsoleVariableRef CVarTextureGroupQualityLevel(
	TEXT("r.TextureGroup.QualityLevel"),
	TextureGroupQualityLevel,
	TEXT("Used to apply texture quality bias to streamable textures at runtime. Clamped by r.TextureGroup.MaxQualityLevel. (default=-1)"),
	ECVF_Default
);

void FTextureLODGroup::SetupGroup()
{
	// editor would never want to use smaller mips based on memory (could affect cooking, etc!)
	if (!GIsEditor)
	{
		// setup 
		switch (FPlatformMemory::GetMemorySizeBucket())
		{
			case EPlatformMemorySizeBucket::Smallest:
				// use Smallest values, if they exist, or Smaller, if not
				if (LODBias_Smallest > 0)
				{
					LODBias = LODBias_Smallest;
				}
				else if (LODBias_Smaller > 0)
				{
					LODBias = LODBias_Smaller;
				}
				if (MaxLODSize_Smallest > 0)
				{
					MaxLODSize = MaxLODSize_Smallest;
				}
				else if (MaxLODSize_Smaller > 0)
				{
					MaxLODSize = MaxLODSize_Smaller;
				}
				break;
			case EPlatformMemorySizeBucket::Smaller:
				// use Smaller values if they exist
				if (LODBias_Smaller > 0)
				{
					LODBias = LODBias_Smaller;
				}
				if (MaxLODSize_Smaller > 0)
				{
					MaxLODSize = MaxLODSize_Smaller;
				}
				break;
			default:
				break;
		}

		if (TextureGroupMaxQualityLevel >= 0 && TextureGroupMaxQualityLevel < 4)
		{
			int32 MaxSizes[4];
			ComputeMaxSizesForQualityLevels(MaxSizes);
			if (MaxSizes[TextureGroupMaxQualityLevel] != -1)
			{
				MaxLODSize = MaxSizes[TextureGroupMaxQualityLevel];
			}
		}
	}

	MaxLODMipCount = FMath::CeilLogTwo(MaxLODSize);

	// Linear filtering
	if (MinMagFilter == NAME_Linear)
	{
		if (MipFilter == NAME_Point)
		{
			Filter = ETextureSamplerFilter::Bilinear;
		}
		else
		{
			Filter = ETextureSamplerFilter::Trilinear;
		}
	}
	// Point. Don't even care about mip filter.
	else if (MinMagFilter == NAME_Point)
	{
		Filter = ETextureSamplerFilter::Point;
	}
	// Aniso or unknown.
	else
	{
		if (MipFilter == NAME_Point)
		{
			Filter = ETextureSamplerFilter::AnisotropicPoint;
		}
		else
		{
			Filter = ETextureSamplerFilter::AnisotropicLinear;
		}
	}
}

bool FTextureLODGroup::operator==(const FTextureLODGroup& Other) const
{
	// Do a UPROPERTY compare to avoid easily broken manual checks
	UScriptStruct* ScriptStruct = FTextureLODGroup::StaticStruct();
	
	return ScriptStruct->CompareScriptStruct(this, &Other, 0);
}

void FTextureLODGroup::ComputeMaxSizesForQualityLevels(int32 OutMaxSizes[4]) const
{
	// We don't want to implicitly generate quality if no quality was set for this group
	if (MaxLODSize_Low == -1 &&
		MaxLODSize_Medium == -1 &&
		MaxLODSize_High == -1 &&
		MaxLODSize_Epic == -1)
	{
		OutMaxSizes[0] = OutMaxSizes[1] = OutMaxSizes[2] = OutMaxSizes[3] = -1;
		return;
	}

	// Copy or compute fallbacks for sizes
	OutMaxSizes[0] = MaxLODSize_Low == -1 ? OptionalMaxLODSize : MaxLODSize_Low;
	// In case none of the values are set
	if (OutMaxSizes[0] == -1)
	{
		OutMaxSizes[0] = 128;
	}
	OutMaxSizes[1] = MaxLODSize_Medium == -1 ? OutMaxSizes[0] << 1 : MaxLODSize_Medium;
	OutMaxSizes[2] = MaxLODSize_High == -1 ? OutMaxSizes[1] << 1 : MaxLODSize_High;
	OutMaxSizes[3] = MaxLODSize_Epic == -1 ? MaxLODSize : FMath::Min(MaxLODSize_Epic, MaxLODSize);

	// Ensure size[n] is not higher than size[n + 1]
	OutMaxSizes[2] = FMath::Min(OutMaxSizes[2], OutMaxSizes[3]);
	OutMaxSizes[1] = FMath::Min(OutMaxSizes[1], OutMaxSizes[2]);
	OutMaxSizes[0] = FMath::Min(OutMaxSizes[0], OutMaxSizes[1]);
}

UTextureLODSettings::UTextureLODSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

/**
 * Returns the texture group names, sorted like enum.
 *
 * @return array of texture group names
 */
TArray<FString> UTextureLODSettings::GetTextureGroupNames()
{
	// no need for this to be FString, could be TCHAR *
	TArray<FString> TextureGroupNames;

	// TEXTUREGROUP_MAX is not actually Max, it's count
	TextureGroupNames.Reserve(TEXTUREGROUP_MAX);

#define GROUPNAMES(g) TextureGroupNames.Emplace(TEXT(#g));
	FOREACH_ENUM_TEXTUREGROUP(GROUPNAMES)
#undef GROUPNAMES

	check( TextureGroupNames.Num() == TEXTUREGROUP_MAX );

	return TextureGroupNames;
}

void UTextureLODSettings::SetupLODGroup(int32 GroupId)
{
	TextureLODGroups[GroupId].SetupGroup();
}

int32 UTextureLODSettings::CalculateLODBias(const UTexture* Texture, bool bIncCinematicMips) const
{	
	check( Texture );
	TextureMipGenSettings MipGenSetting = TMGS_MAX;
	int32 TextureMinSize = 0;
	int32 TextureMaxSize = 0;

#if WITH_EDITORONLY_DATA
	MipGenSetting = Texture->MipGenSettings;
	TextureMinSize = Texture->GetMinTextureSize();
	TextureMaxSize = Texture->GetMaxTextureSize();
#endif // #if WITH_EDITORONLY_DATA

	// note: these are the Platform texture size and can return 0 during build!
	float Width = Texture->GetSurfaceWidth();
	float Height = Texture->GetSurfaceHeight();

	return CalculateLODBias( FMath::RoundToInt(Width), FMath::RoundToInt(Height), TextureMinSize, TextureMaxSize, Texture->LODGroup, Texture->LODBias, bIncCinematicMips ? Texture->NumCinematicMipLevels : 0, MipGenSetting, Texture->IsCurrentlyVirtualTextured(), Texture->IsStreamable());
}

int32 UTextureLODSettings::CalculateLODBias(int32 Width, int32 Height, int32 MinSize, int32 MaxSize, int32 LODGroup, int32 LODBias, int32 NumCinematicMipLevels, TextureMipGenSettings InMipGenSetting, bool bVirtualTexture, bool bCurrentlyStreamableTexture) const
{	
	checkf(LODGroup < TextureLODGroups.Num(), TEXT("A texture had passed a bad LODGroup to UTextureLODSettings::CalculateLODBias (%d, out of %d groups). This code does not have the texture name. The LODSettings object is '%s'"), LODGroup, TextureLODGroups.Num(), *GetPathName());

	// Find LOD group.
	const FTextureLODGroup& LODGroupInfo = TextureLODGroups[LODGroup];

	// Note: MaxLODSize sort of acts like a max texture size limit
	//	but it isn't really a good way to limit texture size
	// because there are various ways in which it can be ignored
	// eg. on textures set to NoMipmaps

	// Test to see if we have no mip generation as in which case the LOD bias will be ignored
	// VTs don't respect NoMipmaps, mips are required for VTs
	const TextureMipGenSettings FinalMipGenSetting = (InMipGenSetting == TMGS_FromTextureGroup) ? (TextureMipGenSettings)LODGroupInfo.MipGenSettings : InMipGenSetting;
	if ( !bVirtualTexture && FinalMipGenSetting == TMGS_NoMipmaps )
	{
		return 0;
	}

	// Calculate maximum number of miplevels.
	if (MaxSize > 0)
	{
		Width = FMath::Min(Width, MaxSize);
		Height = FMath::Min(Height, MaxSize);
	}

	// Don't let the LOD remove ALL mips
	int32 TextureMaxLOD = FImageCoreUtils::GetMipCountFromDimensions(Width, Height, 0, false) - 1;


	// Calculate LOD bias.
	int32 UsedLODBias = 0;
	if (!bVirtualTexture)
	{
		// VT doesn't support cinematic mips
		UsedLODBias += NumCinematicMipLevels;
	}
	if (!FPlatformProperties::RequiresCookedData())
	{
		// When cooking, LODBias and LODGroupInfo.LODBias are taken into account to strip the top mips.
		// Considering them again here would apply them twice.
		UsedLODBias	+= LODBias;
		UsedLODBias += bVirtualTexture ? LODGroupInfo.LODBias_VT : LODGroupInfo.LODBias;

		// GlobalTextureLODBias from cvar r.GlobalTextureLODBias
		//	only applied on non-cooked platforms (will apply when cooking)
		UsedLODBias += GlobalTextureLODBias;
	}
	
	if (LODGroup == TEXTUREGROUP_UI)
	{
		UsedLODBias += GUITextureLODBias;  
		// @todo Oodle : GUITextureLODBias is applied at both cook time & run time , which is screwy
		// -> this looks broken, probably should be inside the !RequiresCookedData
	}

	// Clamp the accumulated UsedLodBias to account for MinSize.
	if (MinSize > 0)
	{
		const int32 MaxLODBias = FMath::Max((int32)FMath::CeilLogTwo(FMath::Min(Width, Height)) - (int32)FMath::CeilLogTwo(MinSize), 0);
		UsedLODBias = FMath::Min(UsedLODBias, MaxLODBias);
	}

	// Min/Max LOD Size clamps the *LODBias*, not the actual texture LODs. Meaning:
	//	MaxLODSize caps the largest mip size.
	//	MinLODSize prevents LODBias from making the largest mip smaller than the given value. Does *not*
	//	affect the size of the smallest mip. Almost never used.
	int32 MaxLODSize = LODGroupInfo.MaxLODSize;
	if (!GIsEditor)
	{
		// Non-streamable textures effectively bake in the bias once created and don't respond to changes, which means we can't apply bias that can change at runtime
		if (bCurrentlyStreamableTexture && !bVirtualTexture && TextureGroupQualityLevel >= 0 && TextureGroupQualityLevel < 4)
		{
			int32 QualityLevel = TextureGroupQualityLevel;
			if (TextureGroupMaxQualityLevel >= 0)
			{
				QualityLevel = FMath::Min(QualityLevel, TextureGroupMaxQualityLevel);
			}

			int32 MaxSizes[4];
			LODGroupInfo.ComputeMaxSizesForQualityLevels(MaxSizes);
			if (MaxSizes[QualityLevel] != -1)
			{
				MaxLODSize = MaxSizes[QualityLevel];
			}
		}
	}

	int32 MinLOD = FMath::CeilLogTwo(LODGroupInfo.MinLODSize);
	int32 MaxLOD = FMath::CeilLogTwo(MaxLODSize);

	if (bVirtualTexture)
	{
		// Virtual textures have no MinLOD and a specific MaxLOD setting.
		MinLOD = 0;
		MaxLOD = LODGroupInfo.MaxLODSize_VT > 0 ? FMath::CeilLogTwo(LODGroupInfo.MaxLODSize_VT) : TextureMaxLOD;
	}

	int32 WantedMaxLOD	= FMath::Clamp( TextureMaxLOD - UsedLODBias, MinLOD, MaxLOD );
	WantedMaxLOD		= FMath::Clamp( WantedMaxLOD, 0, TextureMaxLOD );
	UsedLODBias			= TextureMaxLOD - WantedMaxLOD;

	return UsedLODBias;
}

int32 UTextureLODSettings::CalculateNumOptionalMips(int32 LODGroup, const int32 Width, const int32 Height, const int32 NumMips, const int32 MinMipToInline, TextureMipGenSettings InMipGenSetting) const
{
	// shouldn't need to call this client side, this is calculated at save texture time
	check( FPlatformProperties::RequiresCookedData() == false);

	const FTextureLODGroup& LODGroupInfo = TextureLODGroups[LODGroup];

	const TextureMipGenSettings& FinalMipGenSetting = (InMipGenSetting == TMGS_FromTextureGroup) ? (TextureMipGenSettings)LODGroupInfo.MipGenSettings : InMipGenSetting;
	if ( FinalMipGenSetting == TMGS_NoMipmaps)
	{
		return 0;
	}

	int32 OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
	if (TextureGroupOptionalQualityLevel >= 0 && TextureGroupOptionalQualityLevel < 4)
	{
		int32 MaxSizes[4];
		LODGroupInfo.ComputeMaxSizesForQualityLevels(MaxSizes);
		if (MaxSizes[TextureGroupOptionalQualityLevel] != -1)
		{
			OptionalMaxLODSize = MaxSizes[TextureGroupOptionalQualityLevel];
		}
	}

	int32 OptionalLOD = FMath::Min<int32>(FMath::CeilLogTwo(OptionalMaxLODSize) + 1, NumMips);

	// "MinMipToInline" actually comes from NumNonStreaming
	//  this ensures that optional mips are always streaming mips
	//  also streaming mips are always compression block sized aligned, therefore optional mips are too

	int32 NumOptionalMips = FMath::Min(NumMips - (OptionalLOD - LODGroupInfo.OptionalLODBias), MinMipToInline);
	return NumOptionalMips;
}

/**
* TextureLODGroups access with bounds check
*
* @param   GroupIndex      usually from Texture.LODGroup
* @return                  A handle to the indexed LOD group. 
*/
FTextureLODGroup& UTextureLODSettings::GetTextureLODGroup(TextureGroup GroupIndex)
{
	check(GroupIndex >= 0 && GroupIndex < TEXTUREGROUP_MAX);
	return TextureLODGroups[GroupIndex];
}

/**
* TextureLODGroups access with bounds check
*
* @param   GroupIndex      usually from Texture.LODGroup
* @return                  A handle to the indexed LOD group.
*/
const FTextureLODGroup& UTextureLODSettings::GetTextureLODGroup(TextureGroup GroupIndex) const
{
	check(GroupIndex >= 0 && GroupIndex < TEXTUREGROUP_MAX);
	return TextureLODGroups[GroupIndex];
}

#if WITH_EDITORONLY_DATA
void UTextureLODSettings::GetMipGenSettings(const UTexture& Texture, TextureMipGenSettings& OutMipGenSettings, float& OutSharpen, uint32& OutKernelSize, bool& bOutDownsampleWithAverage, bool& bOutSharpenWithoutColorShift, bool &bOutBorderColorBlack) const
{
	TextureMipGenSettings Setting = (TextureMipGenSettings)Texture.MipGenSettings;

	bOutBorderColorBlack = false;

	// avoiding the color shift assumes we deal with colors which is not true for normalmaps and masks
	// or we blur where it's good to blur the color as well
	bOutSharpenWithoutColorShift = ( !Texture.IsNormalMap() && Texture.CompressionSettings != TC_Masks );

	bOutDownsampleWithAverage = true;

	// inherit from texture group
	if(Setting == TMGS_FromTextureGroup)
	{
		const FTextureLODGroup& LODGroup = TextureLODGroups[Texture.LODGroup];

		Setting = LODGroup.MipGenSettings;
	}

	// angular filtering only applies to cubemaps
	// note: currently intentionally NOT allowed on cubearrays
	if (Setting == TMGS_Angular && Texture.GetTextureClass() != ETextureClass::Cube )
	{
		Setting = TMGS_SimpleAverage;
	}

	OutMipGenSettings = Setting;

	// ------------

	// default to 2x2 SimpleAverage :
	// if you generate mips when TMGS was set to NoMipMaps or LeaveExisting, etc.
	//	it will use these defaults
	OutSharpen = 0;
	OutKernelSize = 2;

	if(Setting >= TMGS_Sharpen0 && Setting <= TMGS_Sharpen10)
	{
		// 0 .. 2.0f
		OutSharpen = ((int32)Setting - (int32)TMGS_Sharpen0) * 0.2f;
		OutKernelSize = 8;
	}
	else if(Setting >= TMGS_Blur1 && Setting <= TMGS_Blur5)
	{
		int32 BlurFactor = ((int32)Setting + 1 - (int32)TMGS_Blur1);
		check( BlurFactor > 0 );
		OutSharpen = -BlurFactor * 2;
		OutKernelSize = 2 + 2 * BlurFactor;
		bOutDownsampleWithAverage = false;
		bOutSharpenWithoutColorShift = false;
		bOutBorderColorBlack = true;
	}
}

void UTextureLODSettings::GetDownscaleOptions(const UTexture& Texture, const ITargetPlatform& CurrentPlatform, float& Downscale, ETextureDownscaleOptions& DownscaleOptions) const
{
	float GroupDownscale = FMath::Clamp(TextureLODGroups[Texture.LODGroup].Downscale, 1.0f, 8.0f);
	ETextureDownscaleOptions GroupDownscaleOptions = TextureLODGroups[Texture.LODGroup].DownscaleOptions;
	if (GroupDownscaleOptions == ETextureDownscaleOptions::Default)
	{
		if ( Texture.bUseNewMipFilter )
		{
			// leave Default alone
		}
		else
		{
			// legacy behavior, change Default to SimpleAverage
			GroupDownscaleOptions = ETextureDownscaleOptions::SimpleAverage;
		}
	}
		
	Downscale = Texture.Downscale.GetValueForPlatform(*CurrentPlatform.IniPlatformName());
	if (Downscale < 1.f)
	{
		// value is not overriden for this texture
		Downscale = GroupDownscale;
	}

	DownscaleOptions = Texture.DownscaleOptions;
	if (DownscaleOptions == ETextureDownscaleOptions::Default)
	{
		DownscaleOptions = GroupDownscaleOptions;
	}
}

#endif // #if WITH_EDITORONLY_DATA



/**
 * Returns the LODGroup mip gen settings
 *
 * @param	InLODGroup		The LOD Group ID 
 * @return	TextureMipGenSettings for lod group
 */
const TextureMipGenSettings UTextureLODSettings::GetTextureMipGenSettings(int32 InLODGroup) const
{
	return TextureLODGroups[InLODGroup].MipGenSettings; 
}



/**
 * Returns the filter state that should be used for the passed in texture, taking
 * into account other system settings.
 *
 * @param	Texture		Texture to retrieve filter state for, must not be 0
 * @return	Filter sampler state for passed in texture
 */
ETextureSamplerFilter UTextureLODSettings::GetSamplerFilter(const UTexture* Texture) const
{
	// Default to point filtering.
	ETextureSamplerFilter Filter = ETextureSamplerFilter::Point;

	switch(Texture->Filter)
	{
		case TF_Nearest: 
			Filter = ETextureSamplerFilter::Point; 
			break;
		case TF_Bilinear: 
			Filter = ETextureSamplerFilter::Bilinear; 
			break;
		case TF_Trilinear: 
			Filter = ETextureSamplerFilter::Trilinear; 
			break;

		// TF_Default
		default:
			// Use LOD group value to find proper filter setting.
			Filter = TextureLODGroups[Texture->LODGroup].Filter;
	}

	return Filter;
}

ETextureSamplerFilter UTextureLODSettings::GetSamplerFilter(int32 InLODGroup) const
{
	return TextureLODGroups[InLODGroup].Filter;
}


ETextureMipLoadOptions UTextureLODSettings::GetMipLoadOptions(const UTexture* Texture) const
{
	check(Texture);
	if (Texture->MipLoadOptions != ETextureMipLoadOptions::Default)
	{
		return Texture->MipLoadOptions;
	}
	else
	{
		return TextureLODGroups[Texture->LODGroup].MipLoadOptions;
	}
}
