// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureLayout.cpp: Texture space allocation.
=============================================================================*/

#include "TextureLayout.h"

bool FBinnedTextureLayout::AddElement(FIntPoint ElementSize, FIntPoint& OutElementMin)
{
	FBin* MatchingBin = nullptr;

	for (FBin& Bin : Bins)
	{
		if (Bin.ElementSize == ElementSize)
		{
			MatchingBin = &Bin;
			break;
		}
	}

	if (!MatchingBin)
	{
		Bins.Add(FBin(ElementSize));
		MatchingBin = &Bins.Last();
	}

	if (MatchingBin->bOutOfSpace)
	{
		return false;
	}

	for (FBinAllocation& BinAllocation : MatchingBin->BinAllocations)
	{
		if (BinAllocation.FreeList.Num() > 0)
		{
			const int32 FreeX = BinAllocation.FreeList.Pop();
			OutElementMin = BinAllocation.LayoutAllocation.Min + FIntPoint(FreeX * ElementSize.X, 0);
			return true;
		}
	}

	{
		int32 EffectiveBinSize = 2 * FMath::Clamp(BinSizeTexels / (ElementSize.X * ElementSize.Y), 1, MaxSize.X / ElementSize.X);
		FBinAllocation NewBinAllocation;
		bool bAllocated = false;
		
		do 
		{
			EffectiveBinSize /= 2;
			bAllocated = Layout.AddElement((uint32&)NewBinAllocation.LayoutAllocation.Min.X, (uint32&)NewBinAllocation.LayoutAllocation.Min.Y, EffectiveBinSize * ElementSize.X, ElementSize.Y);
		} 
		while (!bAllocated && EffectiveBinSize > 1);

		if (!bAllocated)
		{
			MatchingBin->bOutOfSpace = true;
			return false; 
		}

		NewBinAllocation.LayoutAllocation.Max = NewBinAllocation.LayoutAllocation.Min + FIntPoint(EffectiveBinSize * ElementSize.X, ElementSize.Y);
		OutElementMin = NewBinAllocation.LayoutAllocation.Min;

		NewBinAllocation.FreeList.Reserve(EffectiveBinSize);

		for (int32 i = EffectiveBinSize - 1; i >= 1; i--)
		{
			NewBinAllocation.FreeList.Add(i);
		}

		MatchingBin->BinAllocations.Add(MoveTemp(NewBinAllocation));

		return true;
	}
}

void FBinnedTextureLayout::RemoveElement(FIntRect Element)
{
	FBin* MatchingBin = nullptr;

	for (FBin& Bin : Bins)
	{
		if (Bin.ElementSize == Element.Size())
		{
			MatchingBin = &Bin;
			break;
		}
	}

	check(MatchingBin);
	check(MatchingBin->BinAllocations.Num() > 0);

	bool bResetOutOfSpace = false;

	for (int32 AllocationIndex = 0; AllocationIndex < MatchingBin->BinAllocations.Num(); AllocationIndex++)
	{
		FBinAllocation& BinAllocation = MatchingBin->BinAllocations[AllocationIndex];

		if (BinAllocation.LayoutAllocation.Intersect(Element))
		{
			BinAllocation.FreeList.Add((Element.Min.X - BinAllocation.LayoutAllocation.Min.X) / Element.Width());

			if (BinAllocation.FreeList.Num() == BinAllocation.LayoutAllocation.Width() / Element.Width())
			{
				verify(Layout.RemoveElement(BinAllocation.LayoutAllocation.Min.X, BinAllocation.LayoutAllocation.Min.Y, BinAllocation.LayoutAllocation.Width(), BinAllocation.LayoutAllocation.Height()));
				MatchingBin->BinAllocations.RemoveAt(AllocationIndex);
				bResetOutOfSpace = true;
			}

			break;
		}
	}

	if (bResetOutOfSpace)
	{
		for (FBin& Bin : Bins)
		{
			Bin.bOutOfSpace = false;
		}
	}
}
