// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureLightProfile.cpp: Implementation of UTextureLightProfile.
=============================================================================*/

#include "Engine/TextureLightProfile.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureLightProfile)


/*-----------------------------------------------------------------------------
	UTextureLightProfile
-----------------------------------------------------------------------------*/
UTextureLightProfile::UTextureLightProfile(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NeverStream = true;
	Brightness = -1;
}

#if WITH_EDITOR
void UTextureLightProfile::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (VirtualTextureStreaming)
	{
		UE_LOGF(LogTexture, Warning, "VirtualTextureStreaming not supported for IES textures");
		VirtualTextureStreaming = false;
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

