// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureRenderTarget.cpp: UTextureRenderTarget implementation
=============================================================================*/

#include "Engine/TextureRenderTarget.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "TextureResource.h"
#include "SceneRenderTargetParameters.h"
#include "EngineModule.h"
#include "EngineLogs.h"
#include "RenderUtils.h"
#include "Logging/MessageLog.h"
#include "Hash/xxhash.h"
#include "ImageCoreUtils.h"
#include "RendererInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTarget)

/*-----------------------------------------------------------------------------
UTextureRenderTarget
-----------------------------------------------------------------------------*/

UTextureRenderTarget::UTextureRenderTarget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TargetGamma = 0.f; // 0 means inherit from resource which is 2.2
	NeverStream = true;
	SRGB = true; // <- odd, not usually what you want; often replaced with IsSRGB()
	LODGroup = TEXTUREGROUP_RenderTarget;	
	bNeedsTwoCopies = false;
	bCanCreateUAV = false;
	bTargetArraySlicesIndependently = false;
#if WITH_EDITORONLY_DATA
	CompressionNone = true;
#endif // #if WITH_EDITORONLY_DATA
}


FTextureRenderTargetResource* UTextureRenderTarget::GetRenderTargetResource()
{
	check(IsInRenderingThread() || 
		(IsInParallelRenderingThread() && (!GetResource() || GetResource()->IsInitialized()))); // we allow this in parallel, but only if the resource is initialized...otherwise it might be a race on intialization
	FTextureRenderTargetResource* Result = NULL;
	if (GetResource() &&
		GetResource()->IsInitialized() )
	{
		Result = static_cast<FTextureRenderTargetResource*>(GetResource());
	}
	return Result;
}


FTextureRenderTargetResource* UTextureRenderTarget::GameThread_GetRenderTargetResource()
{
	check( IsInGameThread() );
	return static_cast< FTextureRenderTargetResource*>(GetResource());
}


FTextureResource* UTextureRenderTarget::CreateResource()
{
	return NULL;
}


EMaterialValueType UTextureRenderTarget::GetMaterialType() const
{
	return MCT_Texture;
}

bool UTextureRenderTarget::CanConvertToTexture(FText* OutErrorMessage) const
{
	ETextureSourceFormat TextureSourceFormat;
	EPixelFormat PixelFormat;
	return CanConvertToTexture(TextureSourceFormat, PixelFormat, OutErrorMessage);
}

ETextureSourceFormat UTextureRenderTarget::ValidateTextureFormatForConversionToTextureInternal(EPixelFormat InFormat, const TArrayView<const EPixelFormat>& InCompatibleFormats, FText* OutErrorMessage) const
{
	// InCompatibleFormats can be empty, meaning anything works
	if ( InCompatibleFormats.Num() != 0 && !InCompatibleFormats.Contains(InFormat))
	{
		if (OutErrorMessage != nullptr)
		{
			TArray<const TCHAR*> CompatibleFormatStrings;
			Algo::Transform(InCompatibleFormats, CompatibleFormatStrings, [](EPixelFormat InCompatiblePixelFormat) { return GetPixelFormatString(InCompatiblePixelFormat); });

			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget", "UnsupportedFormatForConversionToTexture", "Unsupported format ({0}) for converting {1} to {2}. Supported formats are: {3}"),
				FText::FromString(FString(GetPixelFormatString(InFormat))),
				FText::FromString(GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetName()),
				FText::FromString(FString::Join(CompatibleFormatStrings, TEXT(","))));
		}
		return TSF_Invalid;
	}

	// Return what ETextureSourceFormat corresponds to this EPixelFormat (must match the conversion capabilities of UTextureRenderTarget::UpdateTexture) : 
	ERawImageFormat::Type RawFormat = FImageCoreUtils::GetRawImageFormatForPixelFormat(InFormat);
	ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	return TextureFormat;
}

// there are three different RHI APIs to read pixels from render targets
// they correspond to FColor, FLinearColor, and FFloat16
// choose which of the three to use for this RT
ERawImageFormat::Type UTextureRenderTarget::GetReadPixelsFormat(EPixelFormat PF,bool bIsVolume)
{
	if ( bIsVolume )
	{
		// volumes are different, must always use 16F path
		return ERawImageFormat::RGBA16F;
	}
	else if ( PF == PF_FloatRGBA )
	{
		// for non-volumes, only exact match 16F is supported
		return ERawImageFormat::RGBA16F;
	}
	else
	{
		// either 8-bit FColor or 32F FLinearColor path

		ERawImageFormat::Type RF = FImageCoreUtils::GetRawImageFormatForPixelFormat(PF);

		if ( RF == ERawImageFormat::BGRA8 || RF == ERawImageFormat::G8 )
		{
			return ERawImageFormat::BGRA8; // use FColor
		}
		else
		{
			return ERawImageFormat::RGBA32F; // use FLinearColor
		}
	}
}

#if WITH_EDITOR

bool UTextureRenderTarget::UpdateTexture(UTexture* InTexture, EConstructTextureFlags InFlags, const TArray<uint8>* InAlphaOverride, FOnTextureChangingDelegate InOnTextureChangingDelegate, FText* OutErrorMessage)
{
	// InTexture will be filled from the RT
	// InTexture must already be set up as the right type of texture

	if (InTexture == nullptr)
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = NSLOCTEXT("TextureRenderTarget", "NullTexture", "Cannot update texture : null texture.");
		}
		return false;
	}

	UClass* TextureUClass = GetTextureUClass();
	check(TextureUClass != nullptr);
	if (!InTexture->GetClass()->IsChildOf(TextureUClass))
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget", "InvalidTextureClass", "Cannot update texture {0} : its class is {1} while a {2} is expected."), 
				FText::FromString(InTexture->GetName()),
				FText::FromString(InTexture->GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetClass()->GetName()));
		}
		return false;
	}

	ETextureSourceFormat OutTextureSourceFormat = TSF_Invalid;
	EPixelFormat PixelFormat = PF_Unknown; 
	if (!CanConvertToTexture(OutTextureSourceFormat, PixelFormat, OutErrorMessage))
	{
		return false;
	}
	check( OutTextureSourceFormat != TSF_Invalid );

	// Note : use UTexture's GetTextureClass() here, as URenderTarget's GetTextureClass() unconveniently returns ETextureClass::RenderTarget :
	ETextureClass TextureClass = InTexture->GetTextureClass();
	const bool bIsVolume = (TextureClass == ETextureClass::Volume);
	const bool bIsArray = (TextureClass == ETextureClass::Array) || (TextureClass == ETextureClass::CubeArray);
	const bool bIsCube = (TextureClass == ETextureClass::Cube) || (TextureClass == ETextureClass::CubeArray);
	const int32 SizeX = GetSurfaceWidth();
	const int32 SizeY = GetSurfaceHeight();
	const int32 SizeZ = bIsVolume ? GetSurfaceDepth() : 1;
	const int32 NumSurfaces = bIsVolume ? SizeZ : (bIsArray || bIsCube) ? GetSurfaceArraySize() : 1;

	FTextureRenderTargetResource* RenderTarget = GameThread_GetRenderTargetResource();
	if (RenderTarget == nullptr)
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget", "NullRenderTargetResourceForUpdate", "Cannot update texture : render target {0} has been released."), FText::FromString(GetPathNameSafe(this)));
		}
		return false;
	}

	TextureCompressionSettings CompressionSettingsForTexture = IsHDR(PixelFormat) ? TC_HDR : TC_Default;
	if ( PixelFormat == PF_R16F || PixelFormat == PF_R32_FLOAT )
	{
		// single 16F instead of TC_HDR which is RGBA16F
		CompressionSettingsForTexture = TC_HalfFloat;
	}
	#if 0
	// @todo Oodle : this seems to work fine but not sure it's desirable because of the way format affects usage in Unreal
	else if ( PixelFormat == PF_G8 || PixelFormat == PF_G16 )
	{
		CompressionSettingsForTexture = TC_Grayscale;
	}
	#endif

	const int32 NumMips = 1; // There's only support for mip 0 ATM

	bool bTextureChanging = false;
	bTextureChanging |= InTexture->Source.GetSizeX() != SizeX;
	bTextureChanging |= InTexture->Source.GetSizeY() != SizeY;
	bTextureChanging |= InTexture->Source.GetVolumeSizeZ() != SizeZ;
	// Note : for FTextureSource, NumSlices means NumSurfaces (e.g. 6 * GetNumSlices() for a cube map array) : 
	bTextureChanging |= InTexture->Source.GetNumSlices() != NumSurfaces;
	bTextureChanging |= InTexture->Source.GetFormat() != OutTextureSourceFormat;
	bTextureChanging |= InTexture->CompressionSettings != CompressionSettingsForTexture;

	FXxHash64 OldDataHash;

	// Hashing the content is only really useful if none of the other texture settings have changed.
	const bool bHashContent = !bTextureChanging;
	if (bHashContent)
	{
		const uint8* OldData = InTexture->Source.LockMipReadOnly(0);
		const int32 OldDataSize = InTexture->Source.CalcMipSize(0);
		OldDataHash = FXxHash64::HashBuffer(OldData, OldDataSize);
		InTexture->Source.UnlockMip(0);
	}

	// Temp storage that will be used for the texture init.
	ERawImageFormat::Type ReadFormat = GetReadPixelsFormat(PixelFormat,bIsVolume);

	check( ReadFormat == ERawImageFormat::BGRA8 ||
		ReadFormat == ERawImageFormat::RGBA32F ||
		ReadFormat == ERawImageFormat::RGBA16F );

	EGammaSpace ReadGammaSpace = EGammaSpace::Linear;
	bool bSRGB = IsSRGB();
	if ( ERawImageFormat::GetFormatNeedsGammaSpace(ReadFormat) && bSRGB )
	{
		ReadGammaSpace = EGammaSpace::sRGB;
	}

	FImage ReadImage;
	ReadImage.Init(SizeX,SizeY,NumSurfaces,ReadFormat,ReadGammaSpace);

	const int32 NumPixelsPerSurface = SizeX * SizeY;
	const int32 NumBytesPerPixel = ReadImage.GetBytesPerPixel();
	const int64 DestinationSurfaceMipSize = ReadImage.GetSliceSizeBytes();
	const int64 DestinationSize = ReadImage.GetImageSizeBytes();

	check(NumBytesPerPixel > 0);
	checkf((InAlphaOverride == nullptr) || (NumPixelsPerSurface == InAlphaOverride->Num()), TEXT("If InAlphaOverride is specified, it must have the same number of pixels as a single slice (currently, render target has %d pixels per slice, AlphaOverride: %d). It is also expected to be 1-byte encoded (1 element per pixel)."),
		NumPixelsPerSurface, InAlphaOverride->Num());

	TArray<uint8> NewData;
	NewData.SetNumUninitialized(DestinationSize);

	// Read the data surface by surface (i.e. for a 2D array or a volume: slice by slice, for a cubemap: face by face and for a cubemap array: face by face, slice by slice
	for (int32 SurfaceIndex = 0; SurfaceIndex < NumSurfaces; ++SurfaceIndex)
	{
		FReadSurfaceDataFlags ReadSurfaceDataFlags(RCM_UNorm, bIsCube ? static_cast<ECubeFace>(SurfaceIndex % 6) : CubeFace_MAX);
		ReadSurfaceDataFlags.SetArrayIndex(bIsCube ? SurfaceIndex / 6 : SurfaceIndex);

		void * ReadIntoSlice = ReadImage.GetPixelPointer(0,0,SurfaceIndex);

		switch (ReadFormat)
		{
		case ERawImageFormat::BGRA8: // FColor
		{
			TArray<FColor> NewDataColor;
			RenderTarget->ReadPixels(NewDataColor, ReadSurfaceDataFlags);
			check(NewDataColor.Num() == NumPixelsPerSurface);

			const int32 NumBytes = NewDataColor.Num() * NewDataColor.GetTypeSize();
			check(NumBytes == DestinationSurfaceMipSize);
			FMemory::Memcpy(ReadIntoSlice, NewDataColor.GetData(), NumBytes);

			// override the alpha if desired
			TArrayView<FColor> SurfaceColorData = MakeArrayView(reinterpret_cast<FColor*>(ReadIntoSlice), NumPixelsPerSurface);
			if (InAlphaOverride != nullptr)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = (*InAlphaOverride)[PixelIndex];
				}
			}
			else if (InFlags & CTF_RemapAlphaAsMasked)
			{
				// if the target was rendered with a masked texture, then the depth will probably have been written instead of 0/255 for the
				// alpha, and the depth when unwritten will be 255, so remap 255 to 0 (masked out area) and anything else as 255 (written to area)
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = (SurfaceColorData[PixelIndex].A == 255) ? 0 : 255;
				}
			}
			else if (InFlags & CTF_ForceOpaque)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = 255;
				}
			}
		}
		break;
		
		case ERawImageFormat::RGBA32F: // FLinearColor
		{
			TArray<FLinearColor> NewDataColor;
			RenderTarget->ReadLinearColorPixels(NewDataColor, ReadSurfaceDataFlags);
			check(NewDataColor.Num() == NumPixelsPerSurface);

			const int32 NumBytes = NewDataColor.Num() * NewDataColor.GetTypeSize();
			check(NumBytes == DestinationSurfaceMipSize);
			FMemory::Memcpy(ReadIntoSlice, NewDataColor.GetData(), NumBytes);

			// override the alpha if desired
			TArrayView<FLinearColor> SurfaceColorData = MakeArrayView(reinterpret_cast<FLinearColor*>(ReadIntoSlice), NumPixelsPerSurface);
			if (InAlphaOverride != nullptr)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = (*InAlphaOverride)[PixelIndex] * (1.f/255.f);
				}
			}
			else if (InFlags & CTF_RemapAlphaAsMasked)
			{
				// if the target was rendered with a masked texture, then the depth will probably have been written instead of 0/255 for the
				// alpha, and the depth when unwritten will be 255, so remap 255 to 0 (masked out area) and anything else as 255 (written to area)
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = (SurfaceColorData[PixelIndex].A >= 1.f) ? 0.f : 1.f;
				}
			}
			else if (InFlags & CTF_ForceOpaque)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceColorData[PixelIndex].A = 1.f;
				}
			}
		}
		break;

		case ERawImageFormat::RGBA16F:
		{
			TArray<FFloat16Color> NewDataFloat16Color;
			check(NumBytesPerPixel == sizeof(FFloat16Color));

			// ReadFloat16Pixels is only allowed if PF is exactly FloatRGBA
			RenderTarget->ReadFloat16Pixels(NewDataFloat16Color, ReadSurfaceDataFlags);

			check(NewDataFloat16Color.Num() == NumPixelsPerSurface);
			const int32 NumBytes = NewDataFloat16Color.Num() * NewDataFloat16Color.GetTypeSize();
			check(NumBytes == DestinationSurfaceMipSize);
			FMemory::Memcpy(ReadIntoSlice, NewDataFloat16Color.GetData(), NumBytes);

			// override the alpha if desired
			TArrayView<FFloat16Color> SurfaceFloat16ColorData = MakeArrayView(reinterpret_cast<FFloat16Color*>(ReadIntoSlice), NumPixelsPerSurface);
			if (InAlphaOverride != nullptr)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceFloat16ColorData[PixelIndex].A = (*InAlphaOverride)[PixelIndex] * (1.f/255.f);
				}
			}
			else if (InFlags & CTF_RemapAlphaAsMasked)
			{
				// ?? unclear this path makes any sense on Float16
				FFloat16 F16Zero; F16Zero.SetZero();
				FFloat16 F16One; F16One.SetOne();

				// if the target was rendered with a masked texture, then the depth will probably have been written instead of 0/255 for the
				// alpha, and the depth when unwritten will be 255, so remap 255 to 0 (masked out area) and anything else as 255 (written to area)
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceFloat16ColorData[PixelIndex].A = (SurfaceFloat16ColorData[PixelIndex].A >= 1.f) ? F16Zero : F16One;
				}
			}
			else if (InFlags & CTF_ForceOpaque)
			{
				for (int32 PixelIndex = 0; PixelIndex < NumPixelsPerSurface; ++PixelIndex)
				{
					SurfaceFloat16ColorData[PixelIndex].A.SetOne();
				}
			}
		}
		break;

		default:
			checkf(false, TEXT("Unexpected ReadFormat"));
		}
	}

	// ReadFormat is one of the three primary color types
	// OutTextureSourceFormat is the closest TSF to our PF
	if ( OutTextureSourceFormat != FImageCoreUtils::ConvertToTextureSourceFormat(ReadFormat) )
	{
		// convert image to OutTextureSourceFormat
		ReadFormat = FImageCoreUtils::ConvertToRawImageFormat(OutTextureSourceFormat);
		
		ReadGammaSpace = EGammaSpace::Linear;
		if ( ERawImageFormat::GetFormatNeedsGammaSpace(ReadFormat) && bSRGB )
		{
			ReadGammaSpace = EGammaSpace::sRGB;
		}
		ReadImage.ChangeFormat(ReadFormat,ReadGammaSpace);
	}

	if (bHashContent)
	{
		FXxHash64 NewDataHash = FXxHash64::HashBuffer(ReadImage.RawData.GetData(), ReadImage.RawData.Num());
		bTextureChanging = OldDataHash != NewDataHash;
	}

	if (bTextureChanging)
	{
		InOnTextureChangingDelegate(InTexture);

		InTexture->PreEditChange(nullptr);

		// init to the same size as the render target
		InTexture->Source.Init(ReadImage);
		InTexture->CompressionSettings = CompressionSettingsForTexture;
		InTexture->SRGB = ReadImage.GammaSpace == EGammaSpace::sRGB;

		InTexture->PostEditChange();
	}

	return true;
}

UTexture* UTextureRenderTarget::ConstructTexture(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, EConstructTextureFlags InFlags, const TArray<uint8>* InAlphaOverride, FText* OutErrorMessage)
{
	// InFlags = CTF_Default = CTF_Compress | CTF_SRGB,

	FTextureRenderTargetResource* RenderTarget = GameThread_GetRenderTargetResource();
	if (RenderTarget == nullptr)
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget", "NullRenderTargetResourceForConstruct", "Cannot construct texture : render target {0} has been released."), FText::FromString(GetPathNameSafe(this)));
		}
		return nullptr;
	}

	// Make sure the conversion is possible between this render target and this texture. After this point, the conversion shouldn't be able to ever fail : 
	if (!CanConvertToTexture(OutErrorMessage))
	{
		return nullptr;
	}

	// Create the texture object
	UClass* TextureUClass = GetTextureUClass();
	check(TextureUClass != nullptr);
	UTexture* Result = NewObject<UTexture>(InOuter, TextureUClass, FName(*InNewTextureName), InObjectFlags);

	verify(UpdateTexture(Result, InFlags, InAlphaOverride, /*InOnTextureChangingDelegate = */[](UTexture*) {}, OutErrorMessage));

	// if render target gamma used was 1.0 then disable SRGB for the static texture
	// note: UTextureRenderTarget2D also has an explicit SRGB flag in the UTexture parent class
	//	  these are NOT correctly kept in sync
	//		I see SRGB = 1 but Gamma = 1.0
	//	 see also IsSRGB() which is yet another query that has different ideas
	//	furthermore, float formats do not support anything but Linear gamma
	//
	// InFlags typically starts as Default with CTF_SRGB on, and then it's turned off here :
	float Gamma = RenderTarget->GetDisplayGamma();
	if (FMath::Abs(Gamma - 1.0f) < UE_KINDA_SMALL_NUMBER)
	{
		InFlags = static_cast<EConstructTextureFlags>(InFlags & ~CTF_SRGB);
	}

	Result->SRGB = (InFlags & CTF_SRGB) != 0;

	Result->MipGenSettings = TMGS_FromTextureGroup;

	if ((InFlags & CTF_AllowMips) == 0)
	{
		Result->MipGenSettings = TMGS_NoMipmaps;
	}

	if (InFlags & CTF_Compress)
	{
		// Set compression options.
		Result->DeferCompression = (InFlags & CTF_DeferCompression) ? true : false;
	}
	else
	{
		// Disable compression
		Result->CompressionNone = true;
		Result->DeferCompression = false;
	}

	Result->SetModernSettingsForNewOrChangedTexture();

	if ((InFlags & CTF_SkipPostEdit) == 0)
	{
		Result->PostEditChange();
	}

	return Result;
}

#endif // WITH_EDITOR

/*-----------------------------------------------------------------------------
	FTextureRenderTargetResource
-----------------------------------------------------------------------------*/

/** 
 * Return true if a render target of the given format is allowed
 * for creation
 */
bool FTextureRenderTargetResource::IsSupportedFormat( EPixelFormat Format )
{
	// this should at least support all the formats in GetPixelFormatFromRenderTargetFormat

	switch( Format )
	{
	case PF_G8:
	case PF_R8G8:
	case PF_B8G8R8A8:
	case PF_R8G8B8A8:
	case PF_A16B16G16R16:
	case PF_R16F:
	case PF_G16R16F:
	case PF_R32_FLOAT:
	case PF_G32R32F:
	case PF_A32B32G32R32F:
	case PF_FloatRGB:
	case PF_FloatRGBA: // for exporting materials to .obj/.mtl
	case PF_FloatR11G11B10: //Pixel inspector for Reading HDR Color
	case PF_A2B10G10R10: //Pixel inspector for normal buffer
	case PF_DepthStencil: //Pixel inspector for depth and stencil buffer
	case PF_G16:// for heightmaps
		return true;
	default:
		return false;
	}
}

const FTextureRHIRef& FTextureRenderTargetResource::GetShaderResourceTexture() const
{
	// Override GetShaderResourceTexture() because FTextureRenderTargetResource is both a FRenderTarget (RenderTargetTextureRHI) and a FTexture (TextureRHI)
	//  but in some implementations (e.g. cubemaps), those are different. We need to return the one that's used as a shader resource here :
	return TextureRHI;
}


/*static*/ float UTextureRenderTarget::GetDefaultDisplayGamma()
{
	// TextureRenderTarget default gamma does not respond to Engine->DisplayGamma setting
	//	?? was that intentional or a bug ?
	//return FRenderTarget::GetEngineDisplayGamma();

	// return hard-coded gamma 2.2 which corresponds to SRGB
	return 2.2f;
}

/** 
* Render target resource should be sampled in linear color space
*
* @return display gamma expected for rendering to this render target 
*/
float FTextureRenderTargetResource::GetDisplayGamma() const
{
	// we'd like to just do this, but Owner is in the derived classes
	//return Owner->GetDisplayGamma();

	return UTextureRenderTarget::GetDefaultDisplayGamma();
}

/*-----------------------------------------------------------------------------
FDeferredUpdateResource
-----------------------------------------------------------------------------*/

/** 
* if true then FDeferredUpdateResource::UpdateResources needs to be called 
* (should only be set on the rendering thread)
*/
bool FDeferredUpdateResource::bNeedsUpdate = true;

/** 
* Resources can be added to this list if they need a deferred update during scene rendering.
* @return global list of resource that need to be updated. 
*/
TLinkedList<FDeferredUpdateResource*>*& FDeferredUpdateResource::GetUpdateList()
{		
	static TLinkedList<FDeferredUpdateResource*>* FirstUpdateLink = NULL;
	return FirstUpdateLink;
}

/**
* Iterate over the global list of resources that need to
* be updated and call UpdateResource on each one.
*/
void FDeferredUpdateResource::UpdateResources(FRHICommandListImmediate& RHICmdList)
{
	if( bNeedsUpdate )
	{
		TLinkedList<FDeferredUpdateResource*>*& UpdateList = FDeferredUpdateResource::GetUpdateList();
		for( TLinkedList<FDeferredUpdateResource*>::TIterator ResourceIt(UpdateList);ResourceIt; )
		{
			FDeferredUpdateResource* RTResource = *ResourceIt;
			// iterate to next resource before removing an entry
			ResourceIt.Next();

			if( RTResource )
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FlushDeferredResourceUpdate);
				RTResource->FlushDeferredResourceUpdate(RHICmdList);
			}
		}
		// since the updates should only occur once globally
		// then we need to reset this before rendering any viewports
		bNeedsUpdate = false;
	}
}

/**
 * Performs a deferred resource update on this resource if it exists in the UpdateList.
 */
void FDeferredUpdateResource::FlushDeferredResourceUpdate( FRHICommandListImmediate& RHICmdList )
{
	if( UpdateListLink.IsLinked() )
	{
		checkf(bNeedsUpdate, TEXT("The update list does not need to be updated at this point"));

		UpdateDeferredResource(RHICmdList);
		if( bOnlyUpdateOnce )
		{
			// Remove from list if only a single update was requested
			RemoveFromDeferredUpdateList();
		}
	}
}

/**
* Add this resource to deferred update list
* @param OnlyUpdateOnce - flag this resource for a single update if true
*/
void FDeferredUpdateResource::AddToDeferredUpdateList( bool OnlyUpdateOnce )
{
	bool bExists=false;
	TLinkedList<FDeferredUpdateResource*>*& UpdateList = FDeferredUpdateResource::GetUpdateList();
	for( TLinkedList<FDeferredUpdateResource*>::TIterator ResourceIt(UpdateList);ResourceIt;ResourceIt.Next() )
	{
		if( (*ResourceIt) == this )
		{
			bExists=true;
			break;
		}
	}
	if( !bExists )
	{
		UpdateListLink = TLinkedList<FDeferredUpdateResource*>(this);
		UpdateListLink.LinkHead(UpdateList);
		bNeedsUpdate = true;
	}
	bOnlyUpdateOnce=OnlyUpdateOnce;
}

/**
* Remove this resource from deferred update list
*/
void FDeferredUpdateResource::RemoveFromDeferredUpdateList()
{
	UpdateListLink.Unlink();
}

void FDeferredUpdateResource::ResetSceneTextureExtentsHistory()
{
	GetRendererModule().ResetSceneTextureExtentHistory();
}
