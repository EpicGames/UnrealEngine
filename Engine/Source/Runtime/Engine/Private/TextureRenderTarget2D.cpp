// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureRenderTarget2D.cpp: UTextureRenderTarget2D implementation
=============================================================================*/

#include "Engine/TextureRenderTarget2D.h"
#include "HAL/LowLevelMemStats.h"
#include "Misc/MessageDialog.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "Engine/Texture2D.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "EngineLogs.h"
#include "GenerateMips.h"
#include "RenderGraphUtils.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#if WITH_EDITOR
#include "Components/SceneCaptureComponent2D.h"
#include "TextureCompiler.h"
#include "UObject/UObjectIterator.h"
#endif

int32 GTextureRenderTarget2DMaxSizeX = 999999999;
int32 GTextureRenderTarget2DMaxSizeY = 999999999;

/*-----------------------------------------------------------------------------
	UTextureRenderTarget2D
-----------------------------------------------------------------------------*/

UTextureRenderTarget2D::UTextureRenderTarget2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SizeX = 1;
	SizeY = 1;
	bHDR_DEPRECATED = true;
	RenderTargetFormat = RTF_RGBA16f;
	bSupportsUAV = false;
	bAutoGenerateMips = false;
	NumMips = 0;
	ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	OverrideFormat = PF_Unknown;
	bForceLinearGamma = true; // <<-- if you set RTF_RGBA8_SRGB, this is turned off
	bNoFastClear = false;
	MipsSamplerFilter = Filter;
	MipsAddressU = TA_Clamp;
	MipsAddressV = TA_Clamp;

	// note UTextureRenderTarget::UTextureRenderTarget set SRGB = true
	// later SRGB may be set = IsSRGB();

	// see also UTextureRenderTarget::TargetGamma
}

EPixelFormat UTextureRenderTarget2D::GetFormat() const
{
	if (OverrideFormat == PF_Unknown)
	{
		return GetPixelFormatFromRenderTargetFormat(RenderTargetFormat);
	}
	else
	{
		return OverrideFormat;
	}
}

bool UTextureRenderTarget2D::IsSRGB() const
{
	// in theory you'd like the "bool SRGB" variable to == this, but it does not

	// ?? note: UTextureRenderTarget::TargetGamma is ignored here
	// ?? note: GetDisplayGamma forces linear for some float formats, but this doesn't

	if (OverrideFormat == PF_Unknown)
	{
		return RenderTargetFormat == RTF_RGBA8_SRGB;
	}
	else
	{
		return !bForceLinearGamma;
	}
}

FTextureResource* UTextureRenderTarget2D::CreateResource()
{
	if (bSupportsUAV)
	{
		bCanCreateUAV = 1;
	}

	if (bAutoGenerateMips)
	{
		NumMips = FMath::FloorLog2(FMath::Max(SizeX, SizeY)) + 1;
	}
	else
	{
		NumMips = 1;
	}
 
	FTextureRenderTarget2DResource* Result = new FTextureRenderTarget2DResource(this);
	return Result;
}

uint32 UTextureRenderTarget2D::CalcTextureMemorySizeEnum(ETextureMipCount Enum) const
{
	// Calculate size based on format.  All mips are resident on render targets so we always return the same value.
	EPixelFormat Format = GetFormat();
	int32 BlockSizeX = GPixelFormats[Format].BlockSizeX;
	int32 BlockSizeY = GPixelFormats[Format].BlockSizeY;
	int32 BlockBytes = GPixelFormats[Format].BlockBytes;
	int32 NumBlocksX = (SizeX + BlockSizeX - 1) / BlockSizeX;
	int32 NumBlocksY = (SizeY + BlockSizeY - 1) / BlockSizeY;
	int32 NumBytes = NumBlocksX * NumBlocksY * BlockBytes;
	return NumBytes;
}

EMaterialValueType UTextureRenderTarget2D::GetMaterialType() const
{
	return MCT_Texture2D;
}

void UTextureRenderTarget2D::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	int32 NumBytes = CalcTextureMemorySizeEnum(TMC_AllMips);

	CumulativeResourceSize.AddUnknownMemoryBytes(NumBytes);
}

void UTextureRenderTarget2D::InitCustomFormat( uint32 InSizeX, uint32 InSizeY, EPixelFormat InOverrideFormat, bool bInForceLinearGamma )
{
	check(InSizeX > 0 && InSizeY > 0);
	check(FTextureRenderTargetResource::IsSupportedFormat(InOverrideFormat));

	// set required size/format
	SizeX = InSizeX;
	SizeY = InSizeY;
	OverrideFormat = InOverrideFormat;
	bForceLinearGamma = bInForceLinearGamma;

	if (!ensureMsgf(SizeX >= 0 && SizeX <= 65536, TEXT("Invalid SizeX=%u for RenderTarget %s"), SizeX, *GetName()))
	{
		SizeX = 1;
	}

	if (!ensureMsgf(SizeY >= 0 && SizeY <= 65536, TEXT("Invalid SizeY=%u for RenderTarget %s"), SizeY, *GetName()))
	{
		SizeY = 1;
	}

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTarget2D::InitAutoFormat(uint32 InSizeX, uint32 InSizeY)
{
	check(InSizeX > 0 && InSizeY > 0);

	// ?? missing ?
	//OverrideFormat = PF_Unknown;
	//bForceLinearGamma = true;

	// set required size
	SizeX = InSizeX;
	SizeY = InSizeY;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTarget2D::ResizeTarget(uint32 InSizeX, uint32 InSizeY)
{
	if (SizeX != InSizeX || SizeY != InSizeY)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		if (bAutoGenerateMips)
		{
			NumMips = FMath::FloorLog2(FMath::Max(SizeX, SizeY)) + 1;
		}

		if (GetResource())
		{
			FTextureRenderTarget2DResource* InResource = static_cast<FTextureRenderTarget2DResource*>(GetResource());
			int32 NewSizeX = SizeX;
			int32 NewSizeY = SizeY;
			ENQUEUE_RENDER_COMMAND(ResizeRenderTarget)(
				[InResource, NewSizeX, NewSizeY, LocalNumMips = NumMips](FRHICommandListImmediate& RHICmdList)
				{
					InResource->Resize(NewSizeX, NewSizeY, LocalNumMips);
					InResource->UpdateDeferredResource(RHICmdList, true);
				}
			);
		}
		else
		{
			UpdateResource();
		}
	}
}

void UTextureRenderTarget2D::UpdateResourceImmediate(bool bClearRenderTarget/*=true*/)
{
	if (GetResource())
	{
		FTextureRenderTarget2DResource* InResource = static_cast<FTextureRenderTarget2DResource*>(GetResource());
		ENQUEUE_RENDER_COMMAND(UpdateResourceImmediate)(
			[InResource, bClearRenderTarget](FRHICommandListImmediate& RHICmdList)
			{
				InResource->UpdateDeferredResource(RHICmdList, bClearRenderTarget);
			}
		);
	}
}

#if WITH_EDITOR
void UTextureRenderTarget2D::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	EPixelFormat Format = GetFormat();

	const int32 WarnSize = 2048; 

	if (SizeX > WarnSize || SizeY > WarnSize)
	{
		const float MemoryMb = SizeX * SizeY * GPixelFormats[Format].BlockBytes / 1024.0f / 1024.0f;
		FNumberFormattingOptions FloatFormat;
		FloatFormat.SetMaximumFractionalDigits(1);
		FText Message = FText::Format( NSLOCTEXT("TextureRenderTarget2D", "LargeTextureRenderTarget2DWarning", "A TextureRenderTarget2D of size {0}x{1} will use {2}Mb, which may result in extremely poor performance or an Out Of Video Memory crash.\nAre you sure?"), FText::AsNumber(SizeX), FText::AsNumber(SizeY), FText::AsNumber(MemoryMb, &FloatFormat));
		const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::YesNo, Message);
	
		if (Choice == EAppReturnType::No)
		{
			SizeX = FMath::Clamp<int32>(SizeX,1,WarnSize);
			SizeY = FMath::Clamp<int32>(SizeY,1,WarnSize);
		}
	}

	const int32 MaxSize = 16384;
	
	SizeX = FMath::Clamp<int32>(SizeX - (SizeX % GPixelFormats[Format].BlockSizeX),1,MaxSize);
	SizeY = FMath::Clamp<int32>(SizeY - (SizeY % GPixelFormats[Format].BlockSizeY),1,MaxSize);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTextureRenderTarget2D, RenderTargetFormat))
	{
		if (RenderTargetFormat == RTF_RGBA8_SRGB)
		{
			bForceLinearGamma = false;
		}
		else
		{
			bForceLinearGamma = true;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);

    // SRGB may have been changed by Super, reset it since we prefer to honor explicit user choice
	SRGB = IsSRGB();

	// Notify any scene capture components that point to this texture that they may need to refresh
	static const FName SizeXName = GET_MEMBER_NAME_CHECKED(UTextureRenderTarget2D, SizeX);
	static const FName SizeYName = GET_MEMBER_NAME_CHECKED(UTextureRenderTarget2D, SizeY);

	if ((PropertyChangedEvent.GetPropertyName() == SizeXName || PropertyChangedEvent.GetPropertyName() == SizeYName))
	{
		for (TObjectIterator<USceneCaptureComponent2D> It; It; ++It)
		{
			USceneCaptureComponent2D* SceneCaptureComponent = *It;
			if (SceneCaptureComponent->TextureTarget == this)
			{
				// During interactive edits, time is paused, so the Tick function which normally handles capturing isn't called, and we
				// need a manual refresh.  We also need a refresh if the capture doesn't happen automatically every frame.
				if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) || !SceneCaptureComponent->bCaptureEveryFrame)
				{
					SceneCaptureComponent->CaptureSceneDeferred();
				}
			}
		}
	}
}
#endif // WITH_EDITOR

void UTextureRenderTarget2D::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::AddedTextureRenderTargetFormats)
	{
		RenderTargetFormat = bHDR_DEPRECATED ? RTF_RGBA16f : RTF_RGBA8;
	}

	if (Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::ExplicitSRGBSetting)
	{
		float DisplayGamme = 2.2f;
		EPixelFormat Format = GetFormat();

		if (TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
		{
			DisplayGamme = TargetGamma;
		}
		else if (Format == PF_FloatRGB || Format == PF_FloatRGBA || bForceLinearGamma)
		{
			DisplayGamme = 1.0f;
		}

		// This is odd behavior to apply the sRGB gamma correction when target gamma is not 1.0f, but this
		// is to maintain old behavior and users won't have to change content.
		if (RenderTargetFormat == RTF_RGBA8 && FMath::Abs(DisplayGamme - 1.0f) > UE_KINDA_SMALL_NUMBER)
		{
			RenderTargetFormat = RTF_RGBA8_SRGB;
			SRGB = true;
		}
	}
}

void UTextureRenderTarget2D::PostLoad()
{
	float OriginalSizeX = SizeX;
	float OriginalSizeY = SizeY;

	SizeX = FMath::Min<int32>(SizeX, GTextureRenderTarget2DMaxSizeX);
	SizeY = FMath::Min<int32>(SizeY, GTextureRenderTarget2DMaxSizeY);

	// Maintain aspect ratio if clamped
	if( SizeX != OriginalSizeX || SizeY != OriginalSizeY )
	{
		float ScaleX = SizeX / OriginalSizeX;
		float ScaleY = SizeY / OriginalSizeY;
		
		if( ScaleX < ScaleY )
		{
			SizeY = OriginalSizeY * ScaleX;
		}
		else
		{
			SizeX = OriginalSizeX * ScaleY;
		}
	}
	
	Super::PostLoad();
}


FString UTextureRenderTarget2D::GetDesc()	
{
	// size and format string
	return FString::Printf( TEXT("Render to Texture %dx%d[%s]"), SizeX, SizeY, GPixelFormats[GetFormat()].Name );
}

ETextureRenderTargetSampleCount UTextureRenderTarget2D::GetSampleCount() const
{
	// Note: MSAA is currently only supported in UCanvasRenderTarget2D
	return ETextureRenderTargetSampleCount::RTSC_1;
}

UTexture2D* UTextureRenderTarget2D::ConstructTexture2D(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags, TArray<uint8>* InAlphaOverride)
{
	UTexture2D* Result = nullptr;

#if WITH_EDITOR
	FText ErrorMessage;
	Result = Cast<UTexture2D>(ConstructTexture(InOuter, InNewTextureName, InObjectFlags, static_cast<EConstructTextureFlags>(InFlags), InAlphaOverride, &ErrorMessage));
	if (Result == nullptr)
	{ 
		UE_LOGF(LogTexture, Error, "Couldn't construct texture : %ls", *ErrorMessage.ToString());
	}
#endif // WITH_EDITOR

	return Result;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
ETextureSourceFormat UTextureRenderTarget2D::GetTextureFormatForConversionToTexture2D() const
{
	ETextureSourceFormat TextureSourceFormat = TSF_Invalid;
	EPixelFormat PixelFormat = PF_Unknown;
	if (!CanConvertToTexture(TextureSourceFormat, PixelFormat, /*OutErrorMessage = */nullptr))
	{
		return TSF_Invalid;
	}

	return TextureSourceFormat;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSubclassOf<UTexture> UTextureRenderTarget2D::GetTextureUClass() const
{
	return UTexture2D::StaticClass();
}

bool UTextureRenderTarget2D::CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const
{
	const EPixelFormat LocalFormat = GetFormat();

	// empty array means all formats supported
	const ETextureSourceFormat TextureSourceFormat = ValidateTextureFormatForConversionToTextureInternal(LocalFormat, { }, OutErrorMessage);
	if (TextureSourceFormat == TSF_Invalid)
	{
		return false;
	}

	if ((SizeX <= 0) || (SizeY <= 0))
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget2D", "InvalidSizeForConversionToTexture", "Invalid size ({0},{1}) for converting {2} to {3}"),
				FText::AsNumber(SizeX),
				FText::AsNumber(SizeY),
				FText::FromString(GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetName()));
		}
		return false;
	}

	OutPixelFormat = LocalFormat;
	OutTextureSourceFormat = TextureSourceFormat;
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UTextureRenderTarget2D::UpdateTexture2D(UTexture2D* InTexture2D, ETextureSourceFormat InTextureFormat, uint32 Flags, TArray<uint8>* AlphaOverride)
{
	UpdateTexture2D(InTexture2D, InTextureFormat, Flags, AlphaOverride, FTextureChangingDelegate());
}

void UTextureRenderTarget2D::UpdateTexture2D(UTexture2D* InTexture2D, ETextureSourceFormat InTextureFormat, uint32 Flags, TArray<uint8>* AlphaOverride, FTextureChangingDelegate TextureChangingDelegate)
{
#if WITH_EDITOR
	// Simply forward the internal delegate to the external one (the only difference being the specialized type of the texture) :
	auto OnTextureChangingDelegate = [TextureChangingDelegate](UTexture* InTexture) { TextureChangingDelegate.ExecuteIfBound(CastChecked<UTexture2D>(InTexture)); };
	FText ErrorMessage;
	bool bSuccess = UpdateTexture(InTexture2D, static_cast<EConstructTextureFlags>(Flags), AlphaOverride, OnTextureChangingDelegate, &ErrorMessage);
	if (!bSuccess)
	{
		UE_LOGF(LogTexture, Error, "Cannot update texture : %ls", *ErrorMessage.ToString());
	}
#endif // WITH_EDITOR
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/*-----------------------------------------------------------------------------
	FTextureRenderTarget2DResource
-----------------------------------------------------------------------------*/

FTextureRenderTarget2DResource::FTextureRenderTarget2DResource(const class UTextureRenderTarget2D* InOwner)
	:	Owner(InOwner)
	,	ClearColor(InOwner->ClearColor)
	,	Format(InOwner->GetFormat())
	,	TargetSizeX(Owner->SizeX)
	,	TargetSizeY(Owner->SizeY)
	,	TargetNumMips(Owner->GetNumMips())
{
	// note: Resource has a bSRGB field which is not set or checked in the RenderTarget code
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FTextureRenderTarget2DResource::~FTextureRenderTarget2DResource() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Clamp size of the render target resource to max values
 *
 * @param MaxSizeX max allowed width
 * @param MaxSizeY max allowed height
 */
void FTextureRenderTarget2DResource::ClampSize(int32 MaxSizeX,int32 MaxSizeY)
{
	// upsize to go back to original or downsize to clamp to max
 	int32 NewSizeX = FMath::Min<int32>(Owner->SizeX,MaxSizeX);
 	int32 NewSizeY = FMath::Min<int32>(Owner->SizeY,MaxSizeY);
	if (NewSizeX != TargetSizeX || NewSizeY != TargetSizeY)
	{
		TargetSizeX = NewSizeX;
		TargetSizeY = NewSizeY;
		// reinit the resource with new TargetSizeX,TargetSizeY
		check(TargetSizeX >= 0 && TargetSizeY >= 0);
		UpdateRHI(FRHICommandListImmediate::Get());
	}	
}

ETextureCreateFlags FTextureRenderTarget2DResource::GetCreateFlags()
{
	// Create the RHI texture. Only one mip is used and the texture is targetable for resolve.
	ETextureCreateFlags TexCreateFlags = Owner->IsSRGB() ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None;
	TexCreateFlags |= Owner->bGPUSharedFlag ? ETextureCreateFlags::Shared : ETextureCreateFlags::None;
	
	if (Owner->bAutoGenerateMips)
	{
		if (FGenerateMips::WillFormatSupportCompute(Format))
		{
			TexCreateFlags |= ETextureCreateFlags::UAV;
		}
	}

	if (Owner->bCanCreateUAV)
	{
		TexCreateFlags |= ETextureCreateFlags::UAV;
	}

	if (Owner->bNoFastClear)
	{
		TexCreateFlags |= ETextureCreateFlags::NoFastClear;
	}

	return TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource;
}

/**
 * Initializes the RHI render target resources used by this resource.
 * Called when the resource is initialized, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	if( TargetSizeX > 0 && TargetSizeY > 0 )
	{
		const static FLazyName ClassName(TEXT("FTextureRenderTarget2DResource"));

		FString RHIResourceName = Owner->GetName();
		ETextureCreateFlags TexCreateFlags = GetCreateFlags();
		const int32 NumSamples = GetNumFromRenderTargetSampleCount(Owner->GetSampleCount());

		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(*RHIResourceName)
			.SetExtent(TargetSizeX, TargetSizeY)
			.SetFormat(Format)
			.SetNumMips(TargetNumMips)
			.SetNumSamples(NumSamples)
			.SetFlags(TexCreateFlags)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetClearValue(FClearValueBinding(ClearColor))
			.SetClassName(ClassName)
			.SetOwnerName(GetOwnerName());

		TextureRHI = RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);

		if (NumSamples > 1)
		{
			ETextureCreateFlags ResolveTexCreateFlags = TexCreateFlags;
			EnumRemoveFlags(ResolveTexCreateFlags, ETextureCreateFlags::RenderTargetable);
			EnumAddFlags(ResolveTexCreateFlags, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::ResolveTargetable);

			Desc.SetFlags(ResolveTexCreateFlags);
			Desc.SetNumSamples(1);

			TextureRHI = RHICmdList.CreateTexture(Desc);
		}
		else if (Owner->bNeedsTwoCopies)
		{
			Desc.SetFlags(TexCreateFlags | ETextureCreateFlags::ShaderResource);

			TextureRHI = RHICmdList.CreateTexture(Desc);
		}

		if (EnumHasAnyFlags(TexCreateFlags, ETextureCreateFlags::UAV))
		{
			UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(RenderTargetTextureRHI, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(RenderTargetTextureRHI));
		}

		SetGPUMask(FRHIGPUMask::All());
		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		AddToDeferredUpdateList(true);
	}

	// Create the sampler state RHI resource.
	const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		(ESamplerFilter)TextureLODSettings->GetSamplerFilter( Owner ),
		Owner->AddressX == TA_Wrap ? AM_Wrap : (Owner->AddressX == TA_Clamp ? AM_Clamp : AM_Mirror),
		Owner->AddressY == TA_Wrap ? AM_Wrap : (Owner->AddressY == TA_Clamp ? AM_Clamp : AM_Mirror),
		AM_Wrap,
		0,
		TextureLODSettings->GetTextureLODGroup(Owner->LODGroup).MaxAniso
	);
	SamplerStateRHI = GetOrCreateSamplerState( SamplerStateInitializer );
}

/**
 * Release the RHI render target resources used by this resource.
 * Called when the resource is released, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DResource::ReleaseRHI()
{
	// release the FTexture RHI resources here as well
	FTexture::ReleaseRHI();

	RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	RenderTargetTextureRHI.SafeRelease();
	MipGenerationCache.SafeRelease();

	// remove grom global list of deferred clears
	RemoveFromDeferredUpdateList();
}


#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTarget2D)

/**
 * Updates (resolves) the render target texture.
 * Optionally clears the contents of the render target to the clear color.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DResource::UpdateDeferredResource( FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget/*=true*/ )
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	SCOPED_DRAW_EVENT(RHICmdList, GPUResourceUpdate)
	RemoveFromDeferredUpdateList();

	// Skip executing an empty graph.
	if (TextureRHI == RenderTargetTextureRHI && !bClearRenderTarget && !Owner->bAutoGenerateMips)
	{
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	CacheRenderTarget(RenderTargetTextureRHI, TEXT("MipGeneration"), MipGenerationCache);
	FRDGTextureRef RenderTargetTextureRDG = GraphBuilder.RegisterExternalTexture(MipGenerationCache);
	FRDGTextureRef TextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TextureRHI, TEXT("TextureRenderTarget2DResource")));

 	// clear the target surface to the clear color
	if (bClearRenderTarget)
	{
		ensure(RenderTargetTextureRHI.IsValid() && (RenderTargetTextureRHI->GetClearColor() == ClearColor));
		AddClearRenderTargetPass(GraphBuilder, RenderTargetTextureRDG, ClearColor);
	}
 
	if (Owner->bAutoGenerateMips)
	{
		/**Convert the input values from the editor to a compatible format for FSamplerStateInitializerRHI. 
			Ensure default sampler is Bilinear clamp*/
		FGenerateMips::Execute(GraphBuilder, GetFeatureLevel(), RenderTargetTextureRDG, FGenerateMipsParams{
			Owner->MipsSamplerFilter == TF_Nearest ? SF_Point : (Owner->MipsSamplerFilter == TF_Trilinear ? SF_Trilinear : SF_Bilinear),
			Owner->MipsAddressU == TA_Wrap ? AM_Wrap : (Owner->MipsAddressU == TA_Mirror ? AM_Mirror : AM_Clamp),
			Owner->MipsAddressV == TA_Wrap ? AM_Wrap : (Owner->MipsAddressV == TA_Mirror ? AM_Mirror : AM_Clamp)});
	}

	AddCopyTexturePass(GraphBuilder, RenderTargetTextureRDG, TextureRDG, FRHICopyTextureInfo());

	GraphBuilder.Execute();
}

void FTextureRenderTarget2DResource::Resize(int32 NewSizeX, int32 NewSizeY, int32 NewNumMips)
{
	if (TargetSizeX != NewSizeX || TargetSizeY != NewSizeY || TargetNumMips != NewNumMips)
	{
		TargetSizeX = NewSizeX;
		TargetSizeY = NewSizeY;
		TargetNumMips = NewNumMips;
		UpdateRHI(FRHICommandListImmediate::Get());
	}
}

/** 
 * @return width of target
 */
uint32 FTextureRenderTarget2DResource::GetSizeX() const
{
	return TargetSizeX;
}

/** 
 * @return height of target
 */
uint32 FTextureRenderTarget2DResource::GetSizeY() const
{
	return TargetSizeY;
}

/** 
 * @return dimensions of target surface
 */
FIntPoint FTextureRenderTarget2DResource::GetSizeXY() const
{ 
	return FIntPoint(TargetSizeX, TargetSizeY); 
}

/** 
* Render target resource should be sampled in linear color space
*
* @return display gamma expected for rendering to this render target 
*/
float UTextureRenderTarget2D::GetDisplayGamma() const
{
	// if TargetGamma is set (not zero), it overrides everything else
	if (TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
	{
		return TargetGamma;
	}

	// ?? special casing just two of the float PixelFormats to force 1.0 gamma here is inconsistent
	//		(there are lots of other float formats)
	// ignores Owner->IsSRGB() ? it's similar but not quite the same
	EPixelFormat Format = GetFormat();
	if (Format == PF_FloatRGB || Format == PF_FloatRGBA || bForceLinearGamma )
	{
		return 1.0f;
	}

	return UTextureRenderTarget::GetDefaultDisplayGamma(); // hard-coded 2.2 , actually means SRGB
}

float FTextureRenderTarget2DResource::GetDisplayGamma() const
{
	return Owner->GetDisplayGamma();
}