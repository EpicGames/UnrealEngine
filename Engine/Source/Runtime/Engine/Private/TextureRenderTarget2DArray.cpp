// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureRenderTarget2DArray.cpp: UTextureRenderTarget2DArray implementation
=============================================================================*/

#include "Engine/TextureRenderTarget2DArray.h"
#include "HAL/LowLevelMemStats.h"
#include "RenderingThread.h"
#include "TextureRenderTarget2DArrayResource.h"
#include "UnrealEngine.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/Texture2DArray.h"
#include "RHIUtilities.h"
#include "UObject/Package.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "RenderGraphUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTarget2DArray)

/*-----------------------------------------------------------------------------
	UTextureRenderTarget2DArray
-----------------------------------------------------------------------------*/

UTextureRenderTarget2DArray::UTextureRenderTarget2DArray(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHDR = true;
	ClearColor = FLinearColor(0.0f, 1.0f, 0.0f, 1.0f);
	OverrideFormat = PF_Unknown;
	bSupportsUAV = false;
	bForceLinearGamma = true;
}

EPixelFormat UTextureRenderTarget2DArray::GetFormat() const
{
	if (OverrideFormat == PF_Unknown)
	{
		return bHDR ? PF_FloatRGBA : PF_B8G8R8A8;
	}
	else
	{
		return OverrideFormat;
	}
}

bool UTextureRenderTarget2DArray::IsSRGB() const
{
	bool bIsSRGB = true;

	// if render target gamma used was 1.0 then disable SRGB for the static texture
	if (FMath::Abs(GetDisplayGamma() - 1.0f) < UE_KINDA_SMALL_NUMBER)
	{
		bIsSRGB = false;
	}

	return bIsSRGB;
}

void UTextureRenderTarget2DArray::Init(uint32 InSizeX, uint32 InSizeY, uint32 InSlices, EPixelFormat InFormat)
{
	check((InSizeX > 0) && (InSizeY > 0) && (InSlices > 0));
	check(!(InSizeX % GPixelFormats[InFormat].BlockSizeX));
	check(!(InSizeY % GPixelFormats[InFormat].BlockSizeY));
	//check(FTextureRenderTargetResource::IsSupportedFormat(InFormat));

	// set required size/format
	SizeX = InSizeX;
	SizeY = InSizeY;
	Slices = InSlices;
	OverrideFormat = InFormat;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTarget2DArray::InitAutoFormat(uint32 InSizeX, uint32 InSizeY, uint32 InSlices)
{
	check((InSizeX > 0) && (InSizeY > 0) && (InSlices > 0));
	check(!(InSizeX % GPixelFormats[GetFormat()].BlockSizeX));
	check(!(InSizeY % GPixelFormats[GetFormat()].BlockSizeY));
	//check(FTextureRenderTargetResource::IsSupportedFormat(GetFormat()));

	// set required size/format
	SizeX = InSizeX;
	SizeY = InSizeY;
	Slices = InSlices;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTarget2DArray::UpdateResourceImmediate(bool bClearRenderTarget/*=true*/)
{
	if (GetResource())
	{
		FTextureRenderTarget2DArrayResource* InResource = static_cast<FTextureRenderTarget2DArrayResource*>(GetResource());
		ENQUEUE_RENDER_COMMAND(UpdateResourceImmediate)(
			[InResource, bClearRenderTarget](FRHICommandListImmediate& RHICmdList)
		{
			InResource->UpdateDeferredResource(RHICmdList, bClearRenderTarget);
		});
	}
}
void UTextureRenderTarget2DArray::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Calculate size based on format.
	const EPixelFormat Format = GetFormat();
	const int64 BlockSizeX = GPixelFormats[Format].BlockSizeX;
	const int64 BlockSizeY = GPixelFormats[Format].BlockSizeY;
	const int64 BlockBytes = GPixelFormats[Format].BlockBytes;
	const int64 NumBlocksX = (SizeX + BlockSizeX - 1) / BlockSizeX;
	const int64 NumBlocksY = (SizeY + BlockSizeY - 1) / BlockSizeY;
	const int64 NumBytes = NumBlocksX * NumBlocksY * Slices * BlockBytes;

	CumulativeResourceSize.AddUnknownMemoryBytes(NumBytes);
}

FTextureResource* UTextureRenderTarget2DArray::CreateResource()
{
	if (bSupportsUAV)
	{
		bCanCreateUAV = 1;
	}

	return new FTextureRenderTarget2DArrayResource(this);
}

EMaterialValueType UTextureRenderTarget2DArray::GetMaterialType() const
{
	return MCT_Texture2DArray;
}

#if WITH_EDITOR
void UTextureRenderTarget2DArray::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	constexpr int32 MaxSize = 2048;

	EPixelFormat Format = GetFormat();
	SizeX = FMath::Clamp<int32>(SizeX - (SizeX % GPixelFormats[Format].BlockSizeX), 1, MaxSize);
	SizeY = FMath::Clamp<int32>(SizeY - (SizeY % GPixelFormats[Format].BlockSizeY), 1, MaxSize);
	Slices = FMath::Clamp<int32>(Slices, 1, MaxSize);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UTextureRenderTarget2DArray::PostLoad()
{
	Super::PostLoad();

	if (!FPlatformProperties::SupportsWindowedMode())
	{
		// clamp the render target size in order to avoid reallocating the scene render targets
		SizeX = FMath::Min<int32>(SizeX, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
		SizeY = FMath::Min<int32>(SizeY, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
		Slices = FMath::Min<int32>(Slices, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
	}
}

FString UTextureRenderTarget2DArray::GetDesc()
{
	return FString::Printf(TEXT("Render to Texture 2DArray %dx%d[%s]"), SizeX, SizeX, GPixelFormats[GetFormat()].Name);
}

TSubclassOf<UTexture> UTextureRenderTarget2DArray::GetTextureUClass() const
{
	return UTexture2DArray::StaticClass();
}

bool UTextureRenderTarget2DArray::CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const
{
	const EPixelFormat LocalFormat = GetFormat();
	// These are the formats currently available for conversion to texture for UTextureRenderTarget2DArray : 
	const ETextureSourceFormat TextureSourceFormat = ValidateTextureFormatForConversionToTextureInternal(GetFormat(), { PF_G8, PF_R8G8, PF_B8G8R8A8, PF_FloatRGBA }, OutErrorMessage);
	if (TextureSourceFormat == TSF_Invalid)
	{
		return false;
	}

	if ((SizeX <= 0) || (SizeY <= 0) || (Slices <= 0))
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTarget2DArray", "InvalidSizeForConversionToTexture", "Invalid size (({0},{1}), {2} slices) for converting {3} to {4}"),
				FText::AsNumber(SizeX),
				FText::AsNumber(SizeY),
				FText::AsNumber(Slices),
				FText::FromString(GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetName()));
		}
		return false;
	}

	OutPixelFormat = LocalFormat;
	OutTextureSourceFormat = TextureSourceFormat;
	return true;
}

UTexture2DArray* UTextureRenderTarget2DArray::ConstructTexture2DArray(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags, TArray<uint8>* InAlphaOverride)
{
	UTexture2DArray* Result = nullptr;

#if WITH_EDITOR
	FText ErrorMessage;
	Result = Cast<UTexture2DArray>(ConstructTexture(InOuter, InNewTextureName, InObjectFlags, static_cast<EConstructTextureFlags>(InFlags), InAlphaOverride, &ErrorMessage));
	if (Result == nullptr)
	{
		UE_LOGF(LogTexture, Error, "Couldn't construct texture : %ls", *ErrorMessage.ToString());
	}
#endif // #if WITH_EDITOR

	return Result;
}

/*-----------------------------------------------------------------------------
	FTextureRenderTarget2DArrayResource
-----------------------------------------------------------------------------*/

/**
 * Initializes the RHI render target resources used by this resource.
 * Called when the resource is initialized, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DArrayResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	if ((Owner->SizeX > 0) && (Owner->SizeY > 0) && (Owner->Slices > 0))
	{
		// Create the RHI texture. Only one mip is used and the texture is targetable for resolve.
		ETextureCreateFlags TexCreateFlags = Owner->IsSRGB() ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None;
		if (Owner->bCanCreateUAV)
		{
			TexCreateFlags |= ETextureCreateFlags::UAV;
		}

		if (Owner->bTargetArraySlicesIndependently)
		{
			TexCreateFlags |= ETextureCreateFlags::TargetArraySlicesIndependently;
		}

		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2DArray(TEXT("FTextureRenderTarget2DArrayResource"))
				.SetExtent(Owner->SizeX, Owner->SizeY)
				.SetArraySize(Owner->Slices)
				.SetFormat(Owner->GetFormat())
				.SetNumMips(Owner->GetNumMips())
				.SetFlags(TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetClearValue(FClearValueBinding(Owner->ClearColor))
				.SetInitialState(ERHIAccess::SRVMask);

			TextureRHI = RHICmdList.CreateTexture(Desc);
		}

		if (EnumHasAnyFlags(TexCreateFlags, ETextureCreateFlags::UAV))
		{
			UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(TextureRHI, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureRHI));
		}

		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);
		RenderTargetTextureRHI = TextureRHI;

		AddToDeferredUpdateList(true);
	}

	// Create the sampler state RHI resource.
	const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		(ESamplerFilter)TextureLODSettings->GetSamplerFilter(Owner),
		AM_Wrap,
		AM_Wrap,
		AM_Wrap,
		0,
		TextureLODSettings->GetTextureLODGroup(Owner->LODGroup).MaxAniso
	);
	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
}

/**
 * Release the RHI render target resources used by this resource.
 * Called when the resource is released, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DArrayResource::ReleaseRHI()
{
	// release the FTexture RHI resources here as well
	FTexture::ReleaseRHI();

	RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	RenderTargetTextureRHI.SafeRelease();

	// remove from global list of deferred clears
	RemoveFromDeferredUpdateList();
}

/**
 * Updates (resolves) the render target texture.
 * Optionally clears each face of the render target to the clear color.
 * This is only called by the rendering thread.
 */
void FTextureRenderTarget2DArrayResource::UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget/*=true*/)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	RemoveFromDeferredUpdateList();

	if (!bClearRenderTarget)
	{
		return;
	}

	// If the render target has the TargetArraySlicesIndependently flag, we can do the right thing and clear all slices :
	if (EnumHasAnyFlags(TextureRHI->GetDesc().Flags, ETextureCreateFlags::TargetArraySlicesIndependently))
	{
		FRDGTextureClearInfo TextureClearInfo;
		TextureClearInfo.NumSlices = TextureRHI->GetDesc().ArraySize;
		FRDGBuilder GraphBuilder(RHICmdList);
		FRDGTextureRef TextureRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TextureRHI, TEXT("TextureRenderTarget2DArrayResource")));
		AddClearRenderTargetPass(GraphBuilder, TextureRDG, TextureClearInfo);
		GraphBuilder.Execute();
	}
	else
	{
		// Otherwise, just (probably erroneously) clear the first slice for backwards compatibility :
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
		ClearRenderTarget(RHICmdList, TextureRHI);
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}
}

/**
 * @return width of target
 */
uint32 FTextureRenderTarget2DArrayResource::GetSizeX() const
{
	return Owner->SizeX;
}

/**
 * @return height of target
 */
uint32 FTextureRenderTarget2DArrayResource::GetSizeY() const
{
	return Owner->SizeY;
}

/**
 * @return dimensions of target surface
 */
FIntPoint FTextureRenderTarget2DArrayResource::GetSizeXY() const
{
	return FIntPoint(Owner->SizeX, Owner->SizeY);
}

float UTextureRenderTarget2DArray::GetDisplayGamma() const
{
	if (TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
	{
		return TargetGamma;
	}
	EPixelFormat Format = GetFormat();
	if (Format == PF_FloatRGB || Format == PF_FloatRGBA || bForceLinearGamma)
	{
		return 1.0f;
	}
	return UTextureRenderTarget::GetDefaultDisplayGamma();
}

float FTextureRenderTarget2DArrayResource::GetDisplayGamma() const
{
	return Owner->GetDisplayGamma();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FTextureRenderTarget2DArrayResource::ReadPixels(TArray<FColor>& OutImageData, int32 InSlice, FIntRect InRect)
{
	FReadSurfaceDataFlags Flags;
	Flags.SetArrayIndex(InSlice);
	return FRenderTarget::ReadPixels(OutImageData, Flags, InRect);
}

bool FTextureRenderTarget2DArrayResource::ReadPixels(TArray<FFloat16Color>& OutImageData, int32 InSlice, FIntRect InRect)
{
	FReadSurfaceDataFlags Flags;
	Flags.SetArrayIndex(InSlice);
	return FRenderTarget::ReadFloat16Pixels(OutImageData, Flags, InRect);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
