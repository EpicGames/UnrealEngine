// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureRenderTargetCube.cpp: UTextureRenderTargetCube implementation
=============================================================================*/

#include "Engine/TextureRenderTargetCube.h"
#include "HAL/LowLevelMemStats.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UnrealEngine.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/TextureCube.h"
#include "RHIUtilities.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "GenerateMips.h"

#if WITH_EDITOR
#include "Components/SceneCaptureComponentCube.h"
#include "UObject/UObjectIterator.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTargetCube)

/*-----------------------------------------------------------------------------
	UTextureRenderTargetCube
-----------------------------------------------------------------------------*/

UTextureRenderTargetCube::UTextureRenderTargetCube(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHDR = true;
	ClearColor = FLinearColor(0.0f, 1.0f, 0.0f, 1.0f);
	OverrideFormat = PF_Unknown;
	bSupportsUAV = false;
	bForceLinearGamma = true;
	bAutoGenerateMips = false;
	MipsSamplerFilter = Filter;
	NumMips = 0;
	// note bool SRGB not set
}

EPixelFormat UTextureRenderTargetCube::GetFormat() const
{
	if(OverrideFormat == PF_Unknown)
	{
		return bHDR ? PF_FloatRGBA : PF_B8G8R8A8;
	}
	else
	{
		return OverrideFormat;
	}
}

bool UTextureRenderTargetCube::IsSRGB() const
{
	// compare with IsSRGB() call on RenderTarget2D which uses different logic
	//	also "bool SRGB" should be set = bIsSRGB, but isn't

	bool bIsSRGB = true;
	// if render target gamma used was 1.0 then disable SRGB for the static texture
	if(FMath::Abs(GetDisplayGamma() - 1.0f) < UE_KINDA_SMALL_NUMBER)
	{
		bIsSRGB = false;
	}

	return bIsSRGB;
}

void UTextureRenderTargetCube::Init(uint32 InSizeX, EPixelFormat InFormat)
{
	check(InSizeX > 0);
	check(!(InSizeX % GPixelFormats[InFormat].BlockSizeX));
	check(FTextureRenderTargetResource::IsSupportedFormat(InFormat));

	// set required size/format
	SizeX = InSizeX;
	OverrideFormat = InFormat;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTargetCube::InitAutoFormat(uint32 InSizeX)
{
	// ?? looks like this is missing :
	//OverrideFormat = PF_Unknown;

	check(InSizeX > 0);
	check(!(InSizeX % GPixelFormats[GetFormat()].BlockSizeX));
	check(FTextureRenderTargetResource::IsSupportedFormat(GetFormat()));

	// set required size/format
	SizeX = InSizeX;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTargetCube::UpdateResourceImmediate(bool bClearRenderTarget/*=true*/)
{
	if (GetResource())
	{
		FTextureRenderTargetCubeResource* InResource = static_cast<FTextureRenderTargetCubeResource*>(GetResource());
		ENQUEUE_RENDER_COMMAND(UpdateResourceImmediate)(
			[InResource, bClearRenderTarget](FRHICommandListImmediate& RHICmdList)
			{
				InResource->UpdateDeferredResource(RHICmdList, bClearRenderTarget);
			}
		);
	}
}

void UTextureRenderTargetCube::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Calculate size based on format.
	EPixelFormat Format = GetFormat();
	int32 BlockSizeX	= GPixelFormats[Format].BlockSizeX;
	int32 BlockSizeY	= GPixelFormats[Format].BlockSizeY;
	int32 BlockBytes	= GPixelFormats[Format].BlockBytes;
	int32 NumBlocksX	= (SizeX + BlockSizeX - 1) / BlockSizeX;
	int32 NumBlocksY	= (SizeX + BlockSizeY - 1) / BlockSizeY;
	int32 NumBytes	= NumBlocksX * NumBlocksY * BlockBytes * 6;

	CumulativeResourceSize.AddUnknownMemoryBytes(NumBytes);
}


FTextureResource* UTextureRenderTargetCube::CreateResource()
{
	if (bSupportsUAV)
	{
		bCanCreateUAV = 1;
	}

	if (bAutoGenerateMips)
	{
		NumMips = FMath::FloorLog2(SizeX) + 1;
	}
	else
	{
		NumMips = 1;
	}

	return new FTextureRenderTargetCubeResource(this);
}


EMaterialValueType UTextureRenderTargetCube::GetMaterialType() const
{
	return MCT_TextureCube;
}

#if WITH_EDITOR
void UTextureRenderTargetCube::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	constexpr int32 MaxSize = 2048;

	EPixelFormat Format = GetFormat();
	SizeX = FMath::Clamp<int32>(SizeX - (SizeX % GPixelFormats[Format].BlockSizeX),1,MaxSize);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Notify any scene capture components that point to this texture that they may need to refresh
	static const FName SizeXName = GET_MEMBER_NAME_CHECKED(UTextureRenderTargetCube, SizeX);
	static const FName AutoGenerateMipsName = GET_MEMBER_NAME_CHECKED(UTextureRenderTargetCube, bAutoGenerateMips);

	if (PropertyChangedEvent.GetPropertyName() == SizeXName || PropertyChangedEvent.GetPropertyName() == AutoGenerateMipsName)
	{
		for (TObjectIterator<USceneCaptureComponentCube> It; It; ++It)
		{
			USceneCaptureComponentCube* SceneCaptureComponent = *It;
			if (SceneCaptureComponent->TextureTarget == this)
			{
				// During interactive edits, time is paused, so the Tick function which normally handles capturing isn't called, and we
				// need a manual refresh.  We also need a refresh if the capture doesn't happen automatically every frame.
				if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) || !SceneCaptureComponent->bCaptureEveryFrame)
				{
					SceneCaptureComponent->CaptureSceneDeferred();
				}
			}
		}
	}
}
#endif // WITH_EDITOR


void UTextureRenderTargetCube::PostLoad()
{
	Super::PostLoad();

	if (!FPlatformProperties::SupportsWindowedMode())
	{
		// clamp the render target size in order to avoid reallocating the scene render targets
		SizeX = FMath::Min<int32>(SizeX,FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
	}
}


FString UTextureRenderTargetCube::GetDesc()
{
	return FString::Printf( TEXT("Render to Texture Cube %dx%d[%s]"), SizeX, SizeX, GPixelFormats[GetFormat()].Name);
}

TSubclassOf<UTexture> UTextureRenderTargetCube::GetTextureUClass() const
{
	return UTextureCube::StaticClass();
}

bool UTextureRenderTargetCube::CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const
{
	const EPixelFormat LocalFormat = GetFormat();

	const ETextureSourceFormat TextureSourceFormat = ValidateTextureFormatForConversionToTextureInternal(LocalFormat, { }, OutErrorMessage);
	if (TextureSourceFormat == TSF_Invalid)
	{
		return false;
	}

	if ((SizeX <= 0) || (SizeX & (SizeX - 1)))
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTargetCube", "InvalidSizeForConversionToTexture", "Invalid size ({0},{0}) for converting {1} to {2}. Needs to be a power of 2."),
				FText::AsNumber(SizeX),
				FText::FromString(GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetName()));
		}
		return false;
	}

	OutPixelFormat = LocalFormat;
	OutTextureSourceFormat = TextureSourceFormat;
	return true;
}

UTextureCube* UTextureRenderTargetCube::ConstructTextureCube(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags, TArray<uint8>* InAlphaOverride)
{
	UTextureCube* Result = NULL;

#if WITH_EDITOR
	FText ErrorMessage;
	Result = Cast<UTextureCube>(ConstructTexture(InOuter, InNewTextureName, InObjectFlags, static_cast<EConstructTextureFlags>(InFlags), InAlphaOverride, &ErrorMessage));
	if (Result == nullptr)
	{
		UE_LOGF(LogTexture, Error, "Couldn't construct texture : %ls", *ErrorMessage.ToString());
	}
#endif // #if WITH_EDITOR

	return Result;
}

/*-----------------------------------------------------------------------------
	FTextureRenderTargetCubeResource
-----------------------------------------------------------------------------*/

/**
 * Initializes the RHI render target resources used by this resource.
 * Called when the resource is initialized, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetCubeResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	if(Owner->SizeX > 0)
	{
		// note Resource has a bSRGB and Owner has a bool SRGB , neither are set right
		//	instead call Owner->IsSRGB()

		// Create the RHI texture. Only one mip is used and the texture is targetable for resolve.
		ETextureCreateFlags TexCreateFlags = Owner->IsSRGB() ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None;
		if (Owner->bCanCreateUAV)
		{
			TexCreateFlags |= ETextureCreateFlags::UAV;
		}
		
		if (Owner->bAutoGenerateMips)
		{
			if (FGenerateMips::WillFormatSupportCompute(Owner->GetFormat()))
			{
				TexCreateFlags |= ETextureCreateFlags::UAV;
			}
			else
			{
				// Required in FGenerateMips::ExecuteRaster for FRenderTargetBinding to work on individual slice
				TexCreateFlags |= ETextureCreateFlags::TargetArraySlicesIndependently;
			}
		}

		if (Owner->bTargetArraySlicesIndependently)
		{
			TexCreateFlags |= ETextureCreateFlags::TargetArraySlicesIndependently;
		}

		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::CreateCube(TEXT("FTextureRenderTargetCubeResource"))
				.SetExtent(Owner->SizeX)
				.SetFormat(Owner->GetFormat())
				.SetNumMips(Owner->GetNumMips())
				.SetFlags(TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetClearValue(FClearValueBinding(Owner->ClearColor))
				.SetInitialState(ERHIAccess::SRVMask);

			TextureRHI = RHICmdList.CreateTexture(Desc);
		}

		SetGPUMask(FRHIGPUMask::All());

		if (EnumHasAnyFlags(TexCreateFlags, ETextureCreateFlags::UAV))
		{
			UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(TextureRHI, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureRHI));
		}

		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		// Create the RHI target surface used for rendering to
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FTextureRenderTargetCubeResource"))
				.SetExtent(Owner->SizeX, Owner->SizeX)
				.SetFormat(Owner->GetFormat())
				.SetNumMips(Owner->GetNumMips())
				.SetFlags(TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetClearValue(FClearValueBinding(Owner->ClearColor))
				.SetInitialState(ERHIAccess::SRVMask);

			RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);
		}

		AddToDeferredUpdateList(true);
	}

	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(Owner),
		AM_Clamp,
		AM_Clamp,
		AM_Clamp
	);
	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
}

/**
 * Release the RHI render target resources used by this resource.
 * Called when the resource is released, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetCubeResource::ReleaseRHI()
{
	// release the FTexture RHI resources here as well
	FTexture::ReleaseRHI();

	RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	RenderTargetTextureRHI.SafeRelease();

	// remove from global list of deferred clears
	RemoveFromDeferredUpdateList();
}

/**
 * Updates (resolves) the render target texture.
 * Optionally clears each face of the render target to the clear color.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetCubeResource::UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget/*=true*/)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	RemoveFromDeferredUpdateList();

	if (!bClearRenderTarget)
	{
		return;
	}

	RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));

	ClearRenderTarget(RHICmdList, RenderTargetTextureRHI);

	RHICmdList.Transition({
		FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::CopySrc),
		FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::CopyDest)
	});

	FRHICopyTextureInfo CopyInfo;

	for(int32 FaceIdx = CubeFace_PosX; FaceIdx < CubeFace_MAX; FaceIdx++)
	{
		CopyInfo.DestSliceIndex = FaceIdx;
		RHICmdList.CopyTexture(RenderTargetTextureRHI, TextureRHI, CopyInfo);
	}

	RHICmdList.Transition({
		FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::CopySrc, ERHIAccess::SRVMask),
		FRHITransitionInfo(TextureRHI, ERHIAccess::CopyDest, ERHIAccess::SRVMask)
	});
}

/** 
 * @return width of target
 */
uint32 FTextureRenderTargetCubeResource::GetSizeX() const
{
	return Owner->SizeX;
}

/** 
 * @return height of target
 */
uint32 FTextureRenderTargetCubeResource::GetSizeY() const
{
	return Owner->SizeX; // there is no "SizeY" , cubes must be square
}

/** 
 * @return dimensions of target surface
 */
FIntPoint FTextureRenderTargetCubeResource::GetSizeXY() const
{
	return FIntPoint(Owner->SizeX, Owner->SizeX);
}

float UTextureRenderTargetCube::GetDisplayGamma() const
{
	// code dupe of RenderTarget2D

	if(TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
	{
		return TargetGamma;
	}
	EPixelFormat Format = GetFormat();
	if(Format == PF_FloatRGB || Format == PF_FloatRGBA || bForceLinearGamma)
	{
		return 1.0f;
	}

	return UTextureRenderTarget::GetDefaultDisplayGamma(); // eg. 2.2
}

float FTextureRenderTargetCubeResource::GetDisplayGamma() const
{
	return Owner->GetDisplayGamma();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
* Copy the texels of a single face of a cube texture into an array.
* @param OutImageData - RGBA8 values will be stored in this array.
* @param InFlags - read flags. ensure cubeface member has been set.
* @param InRect - Rectangle of texels to copy. Empty InRect (0,0,0,0) defaults to the whole surface size.
* @return True if the read succeeded.
*/
bool FTextureRenderTargetCubeResource::ReadPixels(TArray< FColor >& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InRect)
{
	return FRenderTarget::ReadPixels(OutImageData, InFlags, InRect);
}

/**
* Copy the texels of a single face of a cube texture into an array.
* @param OutImageData - float16 values will be stored in this array.
* @param InFlags - read flags. ensure cubeface member has been set.
* @param InRect - Rectangle of texels to copy. Empty InRect (0,0,0,0) defaults to the whole surface size.
* @return True if the read succeeded.
*/
bool FTextureRenderTargetCubeResource::ReadPixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InRect)
{
	return FRenderTarget::ReadFloat16Pixels(OutImageData, InFlags, InRect);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
