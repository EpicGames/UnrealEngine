// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureRenderTargetVolume.cpp: UTextureRenderTargetVolume implementation
=============================================================================*/

#include "Engine/TextureRenderTargetVolume.h"
#include "RenderingThread.h"
#include "TextureRenderTargetVolumeResource.h"
#include "UnrealEngine.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/VolumeTexture.h"
#include "HAL/LowLevelMemStats.h"
#include "RHIUtilities.h"
#include "UObject/Package.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureRenderTargetVolume)

/*-----------------------------------------------------------------------------
	UTextureRenderTargetVolume
-----------------------------------------------------------------------------*/

UTextureRenderTargetVolume::UTextureRenderTargetVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHDR = true;
	ClearColor = FLinearColor(0.0f, 1.0f, 0.0f, 1.0f);
	OverrideFormat = PF_Unknown;
	bSupportsUAV = false;
	bForceLinearGamma = true;
}

EPixelFormat UTextureRenderTargetVolume::GetFormat() const
{
	if(OverrideFormat == PF_Unknown)
	{
		return bHDR ? PF_FloatRGBA : PF_B8G8R8A8;
	}
	else
	{
		return OverrideFormat;
	}
}

bool UTextureRenderTargetVolume::IsSRGB() const
{
	bool bIsSRGB = true;
	// if render target gamma used was 1.0 then disable SRGB for the static texture
	if(FMath::Abs(GetDisplayGamma() - 1.0f) < UE_KINDA_SMALL_NUMBER)
	{
		bIsSRGB = false;
	}
	
	return bIsSRGB;
}

void UTextureRenderTargetVolume::Init(uint32 InSizeX, uint32 InSizeY, uint32 InSizeZ, EPixelFormat InFormat)
{
	check((InSizeX > 0) && (InSizeY > 0) && (InSizeZ > 0));
	check(!(InSizeX % GPixelFormats[InFormat].BlockSizeX));
	check(!(InSizeY % GPixelFormats[InFormat].BlockSizeY));
	check(!(InSizeZ % GPixelFormats[InFormat].BlockSizeZ));
	//check(FTextureRenderTargetResource::IsSupportedFormat(InFormat));

	// set required size/format
	SizeX = InSizeX;
	SizeY = InSizeY;
	SizeZ = InSizeZ;
	OverrideFormat = InFormat;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTargetVolume::InitAutoFormat(uint32 InSizeX, uint32 InSizeY, uint32 InSizeZ)
{
	check((InSizeX > 0) && (InSizeY > 0) && (InSizeZ > 0));
	check(!(InSizeX % GPixelFormats[GetFormat()].BlockSizeX));
	check(!(InSizeY % GPixelFormats[GetFormat()].BlockSizeY));
	check(!(InSizeZ % GPixelFormats[GetFormat()].BlockSizeZ));
	//check(FTextureRenderTargetResource::IsSupportedFormat(GetFormat()));

	// set required size/format
	SizeX = InSizeX;
	SizeY = InSizeY;
	SizeZ = InSizeZ;

	// Recreate the texture's resource.
	UpdateResource();
}

void UTextureRenderTargetVolume::UpdateResourceImmediate(bool bClearRenderTarget/*=true*/)
{
	if (GetResource())
	{
		FTextureRenderTargetVolumeResource* InResource = static_cast<FTextureRenderTargetVolumeResource*>(GetResource());
		ENQUEUE_RENDER_COMMAND(UpdateResourceImmediate)(
			[InResource, bClearRenderTarget](FRHICommandListImmediate& RHICmdList)
			{
				InResource->UpdateDeferredResource(RHICmdList, bClearRenderTarget);
			}
		);
	}
}
void UTextureRenderTargetVolume::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Calculate size based on format.
	const EPixelFormat Format = GetFormat();
	const int32 BlockSizeX	= GPixelFormats[Format].BlockSizeX;
	const int32 BlockSizeY	= GPixelFormats[Format].BlockSizeY;
	const int32 BlockSizeZ	= GPixelFormats[Format].BlockSizeZ;
	const int32 BlockBytes	= GPixelFormats[Format].BlockBytes;
	const int32 NumBlocksX	= (SizeX + BlockSizeX - 1) / BlockSizeX;
	const int32 NumBlocksY	= (SizeY + BlockSizeY - 1) / BlockSizeY;
	const int32 NumBlocksZ	= (SizeZ + BlockSizeZ - 1) / BlockSizeZ;
	const int32 NumBytes	= NumBlocksX * NumBlocksY * NumBlocksZ * BlockBytes;

	CumulativeResourceSize.AddUnknownMemoryBytes(NumBytes);
}

FTextureResource* UTextureRenderTargetVolume::CreateResource()
{
	if (bSupportsUAV)
	{
		bCanCreateUAV = 1;
	}

	return new FTextureRenderTargetVolumeResource(this);
}

EMaterialValueType UTextureRenderTargetVolume::GetMaterialType() const
{
	return MCT_VolumeTexture;
}

#if WITH_EDITOR
void UTextureRenderTargetVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	constexpr int32 MaxSize = 2048;

	EPixelFormat Format = GetFormat();
	SizeX = FMath::Clamp<int32>(SizeX - (SizeX % GPixelFormats[Format].BlockSizeX), 1, MaxSize);
	SizeY = FMath::Clamp<int32>(SizeY - (SizeY % GPixelFormats[Format].BlockSizeY), 1, MaxSize);
	SizeZ = FMath::Clamp<int32>(SizeZ - (SizeZ % GPixelFormats[Format].BlockSizeZ), 1, MaxSize);

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UTextureRenderTargetVolume::PostLoad()
{
	Super::PostLoad();

	if (!FPlatformProperties::SupportsWindowedMode())
	{
		// clamp the render target size in order to avoid reallocating the scene render targets
		SizeX = FMath::Min<int32>(SizeX, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
		SizeY = FMath::Min<int32>(SizeY, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
		SizeZ = FMath::Min<int32>(SizeZ, FMath::Min<int32>(GSystemResolution.ResX, GSystemResolution.ResY));
	}
}

FString UTextureRenderTargetVolume::GetDesc()
{
	return FString::Printf( TEXT("Render to Texture Volume %dx%dx%d[%s]"), SizeX, SizeY, SizeZ, GPixelFormats[GetFormat()].Name);
}

TSubclassOf<UTexture> UTextureRenderTargetVolume::GetTextureUClass() const
{
	return UVolumeTexture::StaticClass();
}

bool UTextureRenderTargetVolume::CanConvertToTexture(ETextureSourceFormat& OutTextureSourceFormat, EPixelFormat& OutPixelFormat, FText* OutErrorMessage) const
{
	const EPixelFormat LocalFormat = GetFormat();

	// Formats limited by support in Read3DSurfaceFloatData (see FTextureRenderTargetVolumeResource::ReadFloat16Pixels) :
	ETextureSourceFormat TextureSourceFormat = ValidateTextureFormatForConversionToTextureInternal(LocalFormat, 
		{ PF_R32_FLOAT, PF_FloatRGBA, PF_R16F }, OutErrorMessage);
	if (TextureSourceFormat == TSF_Invalid)
	{
		return false;
	}

	// this is not actually required, TextureSourceFormat is a free choice
	// volumes will always use the 16F read path, that comes from GetReadPixelsFormat()
	TextureSourceFormat = TSF_RGBA16F;

	if ((SizeX <= 0) || (SizeY <= 0) || (SizeZ <= 0))
	{
		if (OutErrorMessage != nullptr)
		{
			*OutErrorMessage = FText::Format(NSLOCTEXT("TextureRenderTargetVolume", "InvalidSizeForConversionToTexture", "Invalid size ({0},{1},{2}) for converting {3} to {4}"),
				FText::AsNumber(SizeX),
				FText::AsNumber(SizeY),
				FText::AsNumber(SizeZ),
				FText::FromString(GetClass()->GetName()),
				FText::FromString(GetTextureUClass()->GetName()));
		}
		return false;
	}

	OutPixelFormat = LocalFormat;
	OutTextureSourceFormat = TextureSourceFormat;
	return true;
}

UVolumeTexture* UTextureRenderTargetVolume::ConstructTextureVolume(UObject* InOuter, const FString& InNewTextureName, EObjectFlags InObjectFlags, uint32 InFlags, TArray<uint8>* InAlphaOverride)
{
	UVolumeTexture* Result = nullptr;

#if WITH_EDITOR
	FText ErrorMessage;
	Result = Cast<UVolumeTexture>(ConstructTexture(InOuter, InNewTextureName, InObjectFlags, static_cast<EConstructTextureFlags>(InFlags), InAlphaOverride, &ErrorMessage));
	if (Result == nullptr)
	{
		UE_LOGF(LogTexture, Error, "Couldn't construct texture : %ls", *ErrorMessage.ToString());
	}
#endif // #if WITH_EDITOR

	return Result;
}

/*-----------------------------------------------------------------------------
	FTextureRenderTargetVolumeResource
-----------------------------------------------------------------------------*/

/**
 * Initializes the RHI render target resources used by this resource.
 * Called when the resource is initialized, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetVolumeResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	if((Owner->SizeX > 0) && (Owner->SizeY > 0) && (Owner->SizeZ > 0))
	{
		// Create the RHI texture. Only one mip is used and the texture is targetable for resolve.
		ETextureCreateFlags TexCreateFlags = Owner->IsSRGB() ? TexCreate_SRGB : TexCreate_None;

		if (Owner->bCanCreateUAV)
		{
			TexCreateFlags |= TexCreate_UAV;
		}

		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("FTextureRenderTargetVolumeResource"))
				.SetExtent(Owner->SizeX, Owner->SizeY)
				.SetDepth(Owner->SizeZ)
				.SetFormat(Owner->GetFormat())
				.SetNumMips(Owner->GetNumMips())
				.SetFlags(TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetClearValue(FClearValueBinding(Owner->ClearColor));

			TextureRHI = RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);
		}

		if (EnumHasAnyFlags(TexCreateFlags, TexCreate_UAV))
		{
			UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(TextureRHI, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(TextureRHI));
		}

		RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, TextureRHI);

		AddToDeferredUpdateList(true);
	}

	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(Owner),
		AM_Clamp,
		AM_Clamp,
		AM_Clamp
	);
	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
}

/**
 * Release the RHI render target resources used by this resource.
 * Called when the resource is released, or when reseting all RHI resources.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetVolumeResource::ReleaseRHI()
{
	// release the FTexture RHI resources here as well
	FTexture::ReleaseRHI();

	RHIClearTextureReference(Owner->TextureReference.TextureReferenceRHI);
	RenderTargetTextureRHI.SafeRelease();

	// remove from global list of deferred clears
	RemoveFromDeferredUpdateList();
}

/**
 * Updates (resolves) the render target texture.
 * Optionally clears each face of the render target to the clear color.
 * This is only called by the rendering thread.
 */
void FTextureRenderTargetVolumeResource::UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget/*=true*/)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Owner->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(FRHITextureCreateDesc().GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, FRHITextureCreateDesc().GetTraceClassName(), Owner->GetPackage()->GetFName());

	RemoveFromDeferredUpdateList();

	if (bClearRenderTarget)
	{
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
		ClearRenderTarget(RHICmdList, TextureRHI);
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}
}

uint32 FTextureRenderTargetVolumeResource::GetSizeX() const
{
	return Owner->SizeX;
}

uint32 FTextureRenderTargetVolumeResource::GetSizeY() const
{
	return Owner->SizeY;
}

uint32 FTextureRenderTargetVolumeResource::GetSizeZ() const
{
	return Owner->SizeZ;
}

FIntPoint FTextureRenderTargetVolumeResource::GetSizeXY() const
{
	return FIntPoint(Owner->SizeX, Owner->SizeY);
}

float UTextureRenderTargetVolume::GetDisplayGamma() const
{
	// code dupe ; move this up to the top level, it's duped everywhere

	if(TargetGamma > UE_KINDA_SMALL_NUMBER * 10.0f)
	{
		return TargetGamma;
	}
	EPixelFormat Format = GetFormat();
	// ?? hard-coding a few formats but not others, likely wrong
	if(Format == PF_R32_FLOAT || Format == PF_FloatRGBA || bForceLinearGamma)
	{
		return 1.0f;
	}

	return UTextureRenderTarget::GetDefaultDisplayGamma();
}

float FTextureRenderTargetVolumeResource::GetDisplayGamma() const
{
	return Owner->GetDisplayGamma();
}

bool FTextureRenderTargetVolumeResource::ReadPixels(TArray<FColor>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	checkf(false, TEXT("Not implemented : volume textures only support float16 ATM. Use ReadFloat16Pixels and convert to the desired format"));
	return false;
}

bool FTextureRenderTargetVolumeResource::ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	if (InSrcRect == FIntRect(0, 0, 0, 0))
	{
		InSrcRect = FIntRect(0, 0, GetSizeXY().X, GetSizeXY().Y);
	}
	
	check( Owner->GetFormat() == GetRenderTargetTexture()->GetFormat() );
	EPixelFormat PF = GetRenderTargetTexture()->GetFormat();

	if ( PF != PF_FloatRGBA && PF != PF_R16F && PF != PF_R32_FLOAT )
	{
		// limitation of RHIRead3DSurfaceFloatData

		return false;
	}

	OutImageData.Reset();

	// Read the render target surface data back.	
	ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)
	(
		[RenderTarget_RT = this, SrcRect_RT = InSrcRect, OutData_RT = &OutImageData, Flags_RT = InFlags](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.Read3DSurfaceFloatData(RenderTarget_RT->GetRenderTargetTexture(), SrcRect_RT, FIntPoint(Flags_RT.GetArrayIndex(), Flags_RT.GetArrayIndex() + 1), *OutData_RT);
		}
	);
	FlushRenderingCommands();

	return true;
}

bool FTextureRenderTargetVolumeResource::ReadLinearColorPixels(TArray<FLinearColor>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	checkf(false, TEXT("Not implemented : volume textures only support float16 ATM. Use ReadFloat16Pixels and convert to the desired format"));
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FTextureRenderTargetVolumeResource::ReadPixels(TArray<FColor>& OutImageData, int32 InDepthSlice, FIntRect InRect)
{
	FReadSurfaceDataFlags Flags;
	Flags.SetArrayIndex(InDepthSlice);
	TArray<FFloat16Color> TempData;
	if (FRenderTarget::ReadFloat16Pixels(TempData, Flags, InRect))
	{
		for (const FFloat16Color& SrcColor : TempData)
		{
			OutImageData.Emplace(FLinearColor(SrcColor).ToFColor(bSRGB));
		}
	}

	return true;
}

bool FTextureRenderTargetVolumeResource::ReadPixels(TArray<FFloat16Color>& OutImageData, int32 InDepthSlice, FIntRect InRect)
{
	FReadSurfaceDataFlags Flags;
	Flags.SetArrayIndex(InDepthSlice);
	return FRenderTarget::ReadFloat16Pixels(OutImageData, Flags, InRect);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
