// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TickTaskManager.cpp: Manager for ticking tasks
=============================================================================*/

#include "Engine/Level.h"
#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/StatsTrace.h"
#include "TickTaskManagerInterface.h"
#include "Async/ParallelFor.h"
#include "Misc/Fork.h"
#include "Misc/TimeGuard.h"
#include "Templates/UniquePtr.h"
#include "UObject/RemoteExecutor.h"
#include "TaskSyncManager.h"

using namespace UE::Tasks;
using namespace UE::Tick;

DEFINE_LOG_CATEGORY_STATIC(LogTick, Log, All);

DECLARE_CYCLE_STAT(TEXT("Queue Ticks"),STAT_QueueTicks,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Queue Ticks Wait"),STAT_QueueTicksWait,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Queue Tick Task"),STAT_QueueTickTask,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Post Queue Tick Task"),STAT_PostTickTask,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Finalize Parallel Queue"),STAT_FinalizeParallelQueue,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Do Deferred Removes"),STAT_DoDeferredRemoves,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("Schedule cooldowns"), STAT_ScheduleCooldowns,STATGROUP_Game);
DECLARE_DWORD_COUNTER_STAT(TEXT("Ticks Queued"),STAT_TicksQueued,STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("TG_NewlySpawned"), STAT_TG_NewlySpawned, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("ReleaseTickGroup"), STAT_ReleaseTickGroup, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("ReleaseTickGroup Block"), STAT_ReleaseTickGroup_Block, STATGROUP_TickGroups);
DECLARE_CYCLE_STAT(TEXT("CleanupTasksWait"), STAT_CleanupTasksWait, STATGROUP_TickGroups);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

static TAutoConsoleVariable<float> CVarStallStartFrame(
	TEXT("CriticalPathStall.TickStartFrame"),
	0.0f,
	TEXT("Sleep for the given time in start frame. Time is given in ms. This is a debug option used for critical path analysis and forcing a change in the critical path."));

static TAutoConsoleVariable<int32> CVarLogTicks(
	TEXT("tick.LogTicks"),
	0,
	TEXT("Spew ticks for debugging."));

static TAutoConsoleVariable<int32> CVarLogTicksShowPrerequistes(
	TEXT("tick.ShowPrerequistes"),
	1,
	TEXT("When logging ticks, show the prerequistes; debugging."));

static TAutoConsoleVariable<int32> CVarAllowAsyncComponentTicks(
	TEXT("tick.AllowAsyncComponentTicks"),
	1,
	TEXT("If true, components (and other tick functions) with bRunOnAnyThread set will run in parallel with other ticks."));

static TAutoConsoleVariable<int32> CVarAllowBatchedTicks(
	TEXT("tick.AllowBatchedTicks"),
	0,
	TEXT("If true, tick functions with bAllowTickBatching will be automatically combined into a single tick task."));

static TAutoConsoleVariable<int32> CVarAllowBatchedTicksUnordered(
	TEXT("tick.AllowBatchedTicksUnordered"),
	0,
	TEXT("If true, tick functions within a batch can be run out of order."));

static TAutoConsoleVariable<int32> CVarAllowOptimizedPrerequisites(
	TEXT("tick.AllowOptimizedPrerequisites"),
	1,
	TEXT("If true, the code will ignore prerequisites that are not relevant due to guaranteed order of tick groups"));

// This was disabled by default in UE 5.5
static TAutoConsoleVariable<int32> CVarAllowConcurrentQueue(
	TEXT("tick.AllowConcurrentTickQueue"),
	0,
	TEXT("If true, queue ticks concurrently using multiple threads at once. This may be faster on platforms with many cores but can change the order of ticking."));

static TAutoConsoleVariable<int32> CVarAllowAsyncTickDispatch(
	TEXT("tick.AllowAsyncTickDispatch"),
	0,
	TEXT("If true, ticks are dispatched in a task thread."));

static TAutoConsoleVariable<int32> CVarAllowAsyncTickCleanup(
	TEXT("tick.AllowAsyncTickCleanup"),
	0,
	TEXT("If true, ticks are cleaned up in a task thread."));

static float GTimeguardThresholdMS = 0.0f;
static FAutoConsoleVariableRef CVarLightweightTimeguardThresholdMS(
	TEXT("tick.LightweightTimeguardThresholdMS"), 
	GTimeguardThresholdMS, 
	TEXT("Threshold in milliseconds for the tick timeguard"),
	ECVF_Default);

static float GIdleTaskWorkMS = 0.0f;
static FAutoConsoleVariableRef CVarIdleTaskWorkMS(
	TEXT("tick.IdleTaskWorkMS"),
	GIdleTaskWorkMS,
	TEXT("If > 0, when the game thread is idle waiting for other threads to complete tasks it will try to spend this much time in milliseconds processing other tasks.\n")
	TEXT("If this is 0, it will keep processing game thread ticks until blocked and then wait.\n")
	TEXT("If < 0 this will use the UE 5.5 behavior to skip idle task processing and stall the game thread entirely."),
	ECVF_Default);

FAutoConsoleTaskPriority CPrio_DispatchTaskPriority(
	TEXT("TaskGraph.TaskPriorities.TickDispatchTaskPriority"),
	TEXT("Task and thread priority for tick tasks dispatch."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

FAutoConsoleTaskPriority CPrio_CleanupTaskPriority(
	TEXT("TaskGraph.TaskPriorities.TickCleanupTaskPriority"),
	TEXT("Task and thread priority for tick cleanup."),
	ENamedThreads::NormalThreadPriority, 
	ENamedThreads::NormalTaskPriority	
	);

FAutoConsoleTaskPriority CPrio_NormalAsyncTickTaskPriority(
	TEXT("TaskGraph.TaskPriorities.NormalAsyncTickTaskPriority"),
	TEXT("Task and thread priority for async ticks that are not high priority."),
	ENamedThreads::NormalThreadPriority,
	ENamedThreads::NormalTaskPriority
	);

FAutoConsoleTaskPriority CPrio_HiPriAsyncTickTaskPriority(
	TEXT("TaskGraph.TaskPriorities.HiPriAsyncTickTaskPriority"),
	TEXT("Task and thread priority for async ticks that are high priority."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

FORCEINLINE bool CanDemoteIntoTickGroup(ETickingGroup TickGroup)
{
	switch (TickGroup)
	{
		case TG_StartPhysics:
		case TG_EndPhysics:
			return false;
	}
	return true;
}

template<typename InElementType, typename InAllocator = FDefaultAllocator>
class TArrayWithThreadsafeAdd : public TArray<InElementType, InAllocator>
{
public:
	typedef InElementType ElementType;
	typedef InAllocator   Allocator;

	template <typename... ArgsType>
	int32 EmplaceThreadsafe(ArgsType&&... Args)
	{
		const int32 Index = AddUninitializedThreadsafe(1);
		new(this->GetData() + Index) ElementType(Forward<ArgsType>(Args)...);
		return Index;
	}


	/**
	 * Adds a given number of uninitialized elements into the array using an atomic increment on the array num
	 *
	 * Caution, the array must have sufficient slack or this will assert/crash. You must presize the array.
	 *
	 * Caution, AddUninitialized() will create elements without calling
	 * the constructor and this is not appropriate for element types that
	 * require a constructor to function properly.
	 *
	 * @param Count Number of elements to add.
	 *
	 * @returns Number of elements in array before addition.
	 */
	int32 AddUninitializedThreadsafe(int32 Count = 1)
	{
		checkSlow(Count >= 0);
		const int32 OldNum = FPlatformAtomics::InterlockedAdd(&this->ArrayNum, Count);
		check(OldNum + Count <= this->ArrayMax);
		return OldNum;
	}

	/**
	 * Adds a new item to the end of the array, using atomics to update the current size of the array
	 *
	 * Caution, the array must have sufficient slack or this will assert/crash. You must presize the array.
	 *
	 * @param Item	The item to add
	 * @return		Index to the new item
	 */
	FORCEINLINE int32 AddThreadsafe(const ElementType& Item)
	{
		return EmplaceThreadsafe(Item);
	}

};

/** Description of how the tick function should execute */
struct FTickContext
{
	/** The world in which the object being ticked is contained */
	UWorld*					World;
	/** Delta time to tick */
	float					DeltaSeconds;
	/** Current or desired thread */
	ENamedThreads::Type		Thread;
	/** Tick type such as gameplay */
	TEnumAsByte<ELevelTick>	TickType;
	/** Tick group this was started in */
	TEnumAsByte<ETickingGroup> TickGroup;
	/** If true, log each tick */
	bool					bLogTick;
	/** If true, log prereqs */
	bool					bLogTicksShowPrerequistes;
	
	
	FTickContext(float InDeltaSeconds = 0.0f, ELevelTick InTickType = LEVELTICK_All, ETickingGroup InTickGroup = TG_PrePhysics, ENamedThreads::Type InThread = ENamedThreads::GameThread)
		: World(nullptr)
		, DeltaSeconds(InDeltaSeconds)
		, Thread(InThread)
		, TickType(InTickType)
		, TickGroup(InTickGroup)
		, bLogTick(false)
		, bLogTicksShowPrerequistes(false)
	{
	}

	FTickContext(const FTickContext& In) = default;
	FTickContext& operator=(const FTickContext& In) = default;
};


/** This is an integer that represents the conditions for which ticks can be grouped together */
struct FTickGroupCondition
{
	union
	{
		uint32 IntVersion;
		struct {
			TEnumAsByte<ETickingGroup> StartGroup;
			TEnumAsByte<ETickingGroup> EndGroup;
			bool bHighPriority;
			bool bRunOnAnyThread;
		};
	};
	
	FTickGroupCondition()
		: IntVersion(0)
	{
	}

	FTickGroupCondition(const FTickFunction* TickFunction)
		: StartGroup(TickFunction->GetActualTickGroup())
		, EndGroup(TickFunction->GetActualEndTickGroup())
		, bHighPriority(TickFunction->bHighPriority)
		, bRunOnAnyThread(TickFunction->bRunOnAnyThread)
	{
	}

	FORCEINLINE bool operator==(const FTickGroupCondition& Other) const
	{
		return IntVersion == Other.IntVersion;
	}

	FORCEINLINE FTickGroupCondition& operator=(const FTickGroupCondition& Other)
	{
		IntVersion = Other.IntVersion;
		return *this;
	}

	FORCEINLINE friend uint32 GetTypeHash(const FTickGroupCondition& Condition)
	{
		return Condition.IntVersion; 
	}
};

static_assert(sizeof(FTickGroupCondition) == 4, "Tick group condition must be an integer");

/** Task for a single tick function */
class FTickFunctionTask
{
	/** Functions to tick */
	FTickFunction* Target;
	/** Tick context with the desired execution thread */
	FTickContext Context;

public:
	FORCEINLINE FTickFunctionTask(FTickFunction* InTarget, const FTickContext* InContext)
		: Target(InTarget)
		, Context(*InContext)
	{
	}
	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FTickFunctionTask, STATGROUP_TaskGraphTasks);
	}
	/** Return the desired execution thread for this task */
	FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return Context.Thread;
	}
	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}
	/**
	 * Actually execute the tick.
	 * @param	CurrentThread; the thread we are running on
	 * @param	MyCompletionGraphEvent; my completion event. Not always useful since at the end of DoWork, you can assume you are done and hence further tasks do not need you as a prerequisite.
	 * However, MyCompletionGraphEvent can be useful for passing to other routines or when it is handy to set up subsequents before you actually do work.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (Context.bLogTick)
		{
			Target->LogTickFunction(CurrentThread, Context.bLogTicksShowPrerequistes);
		}
		if (Target->IsTickFunctionEnabled())
		{
#if DO_TIMEGUARD
			FTimerNameDelegate NameFunction = FTimerNameDelegate::CreateLambda( [&]{ return FString::Printf(TEXT("Slowtick %s "), *Target->DiagnosticMessage()); } );
			SCOPE_TIME_GUARD_DELEGATE_MS(NameFunction, 4);
#endif
			LIGHTWEIGHT_TIME_GUARD_BEGIN(FTickFunctionTask, GTimeguardThresholdMS);

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
			auto ExecuteTickWork = [this, CurrentThread, &MyCompletionGraphEvent]()
			{
				// !IsCompletionHandleValid is an indication we had previously been ticked this frame and then migrated back
				if (Target->IsCompletionHandleValid() && Target->IsTickFunctionEnabled())
				{
#endif
				Target->ExecuteTick(Target->CalculateDeltaTime(Context.DeltaSeconds, Context.World), Context.TickType, CurrentThread, MyCompletionGraphEvent);
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
				}
			};

			if (Target->bRunTransactionally)
			{
				UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(UEnum::GetValueAsName(Context.TickGroup));
				UE::RemoteExecutor::ExecuteTransactional(Target->CachedDiagnosticContext, ExecuteTickWork);
				UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(Target->CachedDiagnosticMessage);
			}
			else
			{
				ExecuteTickWork();
			}
#endif

			LIGHTWEIGHT_TIME_GUARD_END(FTickFunctionTask, Target->DiagnosticMessage());
		}
		Target->ClearTaskInformation();  // This is stale and a good time to clear it for safety
	}
};

// Declare the internal async task that is used for scheduling tick tasks with the task graph backend
using FTickGraphTask = FBaseGraphTask;

/** Info used to execute a batch tick */
struct FTickBatchInfo
{
	/** Actual task assigned to this batch */
	FTickGraphTask* TickTask = nullptr;
	/** Prerequisites that are important */
	TArray<FTickFunction*> TickPrerequisites;
	/** Functions to tick, defaults to a single one */
	TArray<FTickFunction*> TickFunctions;

	FORCEINLINE void Reset()
	{
		TickTask = nullptr;
		// Maintain sizes because the order will probably be the same next frame
		TickPrerequisites.Reset();
		TickFunctions.Reset();
	}

};

/** Task for executing multiple functions at once */
class FBatchTickFunctionTask
{
	/** Batch to execute */
	FTickBatchInfo* TickBatch;
	/** Tick context with the desired execution thread */
	FTickContext Context;

public:
	FORCEINLINE FBatchTickFunctionTask(FTickBatchInfo* InTickBatch, const FTickContext* InContext)
		: TickBatch(InTickBatch)
		, Context(*InContext)
	{
		check(TickBatch);
	}
	FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return Context.Thread;
	}
	/**
	 * Actually execute the tick.
	 * @param	CurrentThread; the thread we are running on
	 * @param	MyCompletionGraphEvent; my completion event. Not always useful since at the end of DoWork, you can assume you are done and hence further tasks do not need you as a prerequisite.
	 * However, MyCompletionGraphEvent can be useful for passing to other routines or when it is handy to set up subsequents before you actually do work.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		check(TickBatch && TickBatch->TickFunctions.Num() > 0);
		for (FTickFunction* Target : TickBatch->TickFunctions)
		{
			if (Context.bLogTick)
			{
				Target->LogTickFunction(CurrentThread, Context.bLogTicksShowPrerequistes);
			}
			if (Target->IsTickFunctionEnabled())
			{
#if UE_WITH_REMOTE_OBJECT_HANDLE
				auto ExecuteTickWork = [this, Target, CurrentThread, &MyCompletionGraphEvent]()
				{
#endif
#if DO_TIMEGUARD
				FTimerNameDelegate NameFunction = FTimerNameDelegate::CreateLambda( [&]{ return FString::Printf(TEXT("Slowtick %s "), *Target->DiagnosticMessage()); } );
				SCOPE_TIME_GUARD_DELEGATE_MS(NameFunction, 4);
#endif
				LIGHTWEIGHT_TIME_GUARD_BEGIN(FBatchTickFunctionTask, GTimeguardThresholdMS);
				Target->ExecuteTick(Target->CalculateDeltaTime(Context.DeltaSeconds, Context.World), Context.TickType, CurrentThread, MyCompletionGraphEvent);
				LIGHTWEIGHT_TIME_GUARD_END(FBatchTickFunctionTask, Target->DiagnosticMessage());

				Target->ClearTaskInformation();  // This is stale and a good time to clear it for safety
#if UE_WITH_REMOTE_OBJECT_HANDLE
				};

				if (Target->bRunTransactionally)
				{
					UE::RemoteExecutor::EnqueueWork(Target->CachedDiagnosticContext, true, ExecuteTickWork);
				}
				else
				{
					ExecuteTickWork();
				}
#endif
			}
			else
			{
				Target->ClearTaskInformation();  // This is stale and a good time to clear it for safety
			}
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(UEnum::GetValueAsName(Context.TickGroup));
		UE::RemoteExecutor::ExecutePendingWork();
		UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT("BatchTick"));
#endif
	}
};


/**
 * Class that handles the actual tick tasks and starting and completing tick groups
 */
class FTickTaskSequencer
{
	/**
	 * Class that handles dispatching a tick group
	 */
	class FDipatchTickGroupTask
	{
		/** Sequencer to proxy to **/
		FTickTaskSequencer &TTS;
		/** Tick group to dispatch **/
		ETickingGroup WorldTickGroup;
	public:
		/** Constructor
			* @param InTarget - Function to tick
			* @param InContext - context to tick in, here thread is desired execution thread
		**/
		FORCEINLINE FDipatchTickGroupTask(FTickTaskSequencer &InTTS, ETickingGroup InWorldTickGroup)
			: TTS(InTTS)
			, WorldTickGroup(InWorldTickGroup)
		{
		}
		static FORCEINLINE TStatId GetStatId()
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FDipatchTickGroupTask, STATGROUP_TaskGraphTasks);
		}
		static FORCEINLINE ENamedThreads::Type GetDesiredThread()
		{
			return CPrio_DispatchTaskPriority.Get();
		}
		static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
		{
			return ESubsequentsMode::TrackSubsequents;
		}
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
		{
			TTS.DispatchTickGroup(CurrentThread, WorldTickGroup);
		}
	};
	/**
	 * Class that handles Reset a tick group
	 */
	class FResetTickGroupTask
	{
		/** Sequencer to proxy to **/
		FTickTaskSequencer &TTS;
		/** Tick group to dispatch **/
		ETickingGroup WorldTickGroup;
	public:
		/** Constructor
			* @param InTarget - Function to tick
			* @param InContext - context to tick in, here thread is desired execution thread
		**/
		FORCEINLINE FResetTickGroupTask(FTickTaskSequencer &InTTS, ETickingGroup InWorldTickGroup)
			: TTS(InTTS)
			, WorldTickGroup(InWorldTickGroup)
		{
		}
		static FORCEINLINE TStatId GetStatId()
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FResetTickGroupTask, STATGROUP_TaskGraphTasks);
		}
		static FORCEINLINE ENamedThreads::Type GetDesiredThread()
		{
			return CPrio_CleanupTaskPriority.Get();
		}
		FORCEINLINE static ESubsequentsMode::Type GetSubsequentsMode()
		{
			return ESubsequentsMode::TrackSubsequents;
		}
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
		{
			TTS.ResetTickGroup(WorldTickGroup);
		}
	};

	/** List of batched ticks, these stay allocated between frames but are cleared */
	TArray< TPair<FTickGroupCondition, TUniquePtr<FTickBatchInfo> > > TickBatches;
	int32 TickBatchesNum = 0;

	/** Completion handles for each phase of ticks, this is compatible with FGraphEventArray */
	TArrayWithThreadsafeAdd<FGraphEventRef, TInlineAllocator<4> > TickCompletionEvents[TG_MAX];

	/** Event-triggered tick functions indexed by end group, these functions must be triggered manually before the group ends */
	TArrayWithThreadsafeAdd<FTickFunction*> ManualDispatchTicks[TG_MAX];

	/** HiPri Held tasks for each tick group. */
	TArrayWithThreadsafeAdd<FTickGraphTask*> HiPriTickTasks[TG_MAX][TG_MAX];

	/** LowPri Held tasks for each tick group. */
	TArrayWithThreadsafeAdd<FTickGraphTask*> TickTasks[TG_MAX][TG_MAX];

	/** These are waited for at the end of the frame; they are not on the critical path, but they have to be done before we leave the frame. */
	FGraphEventArray CleanupTasks;

	/** we keep track of the last TG we have blocked for so when we do block, we know which TG's to wait for . */
	ETickingGroup WaitForTickGroup;

	/** If true, ticks can execute on other threads */
	bool				bAllowConcurrentTicks;
	/** If true, multiple ticks can be combined into a single task */
	bool				bAllowBatchedTicksForFrame;
	/** If true, ticks within a batch can run out of order */
	bool				bAllowBatchedTicksUnorderedForFrame;
	/** If true, some prerequisites will be ignored */
	bool				bAllowOptimizedPrerequisites;

	/** If true, log each tick */
	bool				bLogTicks;
	/** If true, log tick prerequisites when logging ticks */
	bool				bLogTicksShowPrerequistes;
	/** If true, tick everything from the main game thread */
	bool				bSingleThreadMode = false;

public:

	/** Retrieve the global tick task sequencer shared for all ticks */
	static FTickTaskSequencer& Get()
	{
		static FTickTaskSequencer SingletonInstance;
		return SingletonInstance;
	}

	/** Return true if we should be running in single threaded mode, ala dedicated server */
	static bool SingleThreadedMode()
	{
#if DEFAULT_FORK_PROCESS_MULTITHREAD
		// In a Forked multithread process ignore FPlatformProcess::SupportsMultithreading because the parent process never supports multithreading and the function will always return false
		if (FForkProcessHelper::SupportsMultithreadingPostFork())
		{
			// Stay in single-thread mode until the forked process decides it's safe to run in multithread
			return FForkProcessHelper::IsForkedMultithreadInstance() == false;
		}
		else
#endif
		{
			auto IsSingleThreadOnce = []()
			{
				// Are we a dedicated server that supports multithreading
				if (FApp::IsMultithreadServer() && FPlatformProcess::SupportsMultithreading())
				{
					return false;
				}
				else if (!FApp::ShouldUseThreadingForPerformance() || IsRunningDedicatedServer() || FPlatformMisc::NumberOfCores() < 3 || !FPlatformProcess::SupportsMultithreading())
				{
					return true;
				}

				return false;
			};

			static const bool bIsSingleThread = IsSingleThreadOnce();
			return bIsSingleThread;
		}
	};

	/** Accessor that will return a valid task pointer or null if not actually scheduled */
	FORCEINLINE static FTickGraphTask* GetGraphTask(const FTickFunction* TickFunction)
	{
		return (FTickGraphTask*)TickFunction->GetTaskPointer(FTickFunction::ETickTaskState::HasTask);
	}

	/** Non-threadsafe accessor for checking if this has been visited for queuing this frame, only valid on registered functiosn */
	FORCEINLINE static bool HasBeenVisited(const FTickFunction* TickFunction, uint32 CurrentFrameCounter)
	{
		return (TickFunction->InternalData->TickVisitedGFrameCounter.load(std::memory_order_relaxed) == CurrentFrameCounter);
	}

	/** Sets up TickContext for a tick function that will possibly tick */
	FORCEINLINE FTickContext SetupTickContext(FTickFunction* TickFunction, const FTickContext& TickContext)
	{
		checkSlow(TickFunction->InternalData);
		checkSlow(TickFunction->InternalData->ActualStartTickGroup >=0 && TickFunction->InternalData->ActualStartTickGroup < TG_MAX);

		FTickContext UseContext = TickContext;
		UseContext.TickGroup = TickFunction->GetActualTickGroup();
		UseContext.bLogTick = bLogTicks;
		UseContext.bLogTicksShowPrerequistes = bLogTicksShowPrerequistes;

		bool bIsOriginalTickGroup = (TickFunction->InternalData->ActualStartTickGroup == TickFunction->TickGroup);

		if (TickFunction->bRunOnAnyThread && bAllowConcurrentTicks && bIsOriginalTickGroup)
		{
			if (TickFunction->bHighPriority)
			{
				UseContext.Thread = CPrio_HiPriAsyncTickTaskPriority.Get();
			}
			else
			{
				UseContext.Thread = CPrio_NormalAsyncTickTaskPriority.Get();
			}
		}
		else
		{
			UseContext.Thread = ENamedThreads::SetTaskPriority(ENamedThreads::GameThread, TickFunction->bHighPriority ? ENamedThreads::HighTaskPriority : ENamedThreads::NormalTaskPriority);
		}

		return UseContext;
	}

	/**
	 * Start a task for a single function
	 *
	 * @param	InPrerequisites - prerequisites that must be completed before this tick can begin
	 * @param	TickFunction - the tick function to queue
	 * @param	Context - tick context to tick in, that has been setup by SetupTickContext
	 */
	FORCEINLINE void StartTickTask(const FGraphEventArray* Prerequisites, FTickFunction* TickFunction, const FTickContext& TickContext)
	{
		
	}

	/** Add a completion handle to a tick group **/
	FORCEINLINE void AddTickTaskCompletion(ETickingGroup StartTickGroup, ETickingGroup EndTickGroup, FTickGraphTask* Task, bool bHiPri)
	{
		checkSlow(StartTickGroup >=0 && StartTickGroup < TG_MAX && EndTickGroup >=0 && EndTickGroup < TG_MAX && StartTickGroup <= EndTickGroup);
		if (bHiPri)
		{
			HiPriTickTasks[StartTickGroup][EndTickGroup].Add(Task);
		}
		else
		{
			TickTasks[StartTickGroup][EndTickGroup].Add(Task);
		}
		TickCompletionEvents[EndTickGroup].Add(Task->GetCompletionEvent());
	}
	/** Add a completion handle to a tick group, parallel version **/
	FORCEINLINE void AddTickTaskCompletionParallel(ETickingGroup StartTickGroup, ETickingGroup EndTickGroup, FTickGraphTask* Task, bool bHiPri)
	{
		check(StartTickGroup >= 0 && StartTickGroup < TG_NewlySpawned && EndTickGroup >= 0 && EndTickGroup < TG_NewlySpawned && StartTickGroup <= EndTickGroup);
		if (bHiPri)
		{
			HiPriTickTasks[StartTickGroup][EndTickGroup].AddThreadsafe(Task);
		}
		else
		{
			TickTasks[StartTickGroup][EndTickGroup].AddThreadsafe(Task);
		}
		TickCompletionEvents[EndTickGroup].AddThreadsafe(Task->GetCompletionEvent());
	}

	/** Return true if this prerequisite should be tracked in the event graph */
	FORCEINLINE bool ShouldConsiderPrerequisite(FTickFunction* TickFunction, FTickFunction* Prereq)
	{
		// Ignore prereqs that are guaranteed to finish in a previous group
		// This can be called before TickFunction has it's final group set, but Prereq will always be correct
		// There is no hard wait for DuringPhysics so always consider those
		TEnumAsByte<enum ETickingGroup> PrereqEnd = Prereq->GetActualEndTickGroup();
		return (!bAllowOptimizedPrerequisites || PrereqEnd >= TickFunction->TickGroup || PrereqEnd == TG_DuringPhysics);
	}

	/** Return true if this tick condition is safe to batch */
	FORCEINLINE bool CanBatchCondition(FTickGroupCondition Condition)
	{
		// Don't batch high priority ticks or ones that last more than a single tick group
		return !Condition.bRunOnAnyThread && !Condition.bHighPriority && Condition.StartGroup == Condition.EndGroup;
	}

	/** Call before doing any batched ticks */
	void SetupBatchedTicks(int32 NumTicks)
	{
		// This is cleared at the end of tick
		ensure(TickBatchesNum == 0);
	}

	/** Finished batching ticks for the frame */
	void FinishBatchedTicks(const FTickContext& InContext)
	{
		if (bAllowBatchedTicksForFrame)
		{
			// Disable batching for the rest of the frame as we don't want to batch QueueNewlySpawned
			bAllowBatchedTicksForFrame = false;
		}
	}

	/** Set up the lists for AddTickTaskCompletionParallel, since we are using AddThreadsafe, we need to presize the arrays **/
	void SetupAddTickTaskCompletionParallel(int32 NumTicks)
	{
		for (int32 TickGroup = 0; TickGroup < TG_MAX; TickGroup++)
		{
			for (int32 EndTickGroup = 0; EndTickGroup < TG_MAX; EndTickGroup++)
			{
				HiPriTickTasks[TickGroup][EndTickGroup].Reserve(NumTicks);
				TickTasks[TickGroup][EndTickGroup].Reserve(NumTicks);
			}
			TickCompletionEvents[TickGroup].Reserve(NumTicks);
			ManualDispatchTicks[TickGroup].Reserve(NumTicks);
		}
	}

	/** This will add to an existing batch, create a new batch, or just spawn a single task and return null */
	FTickBatchInfo* QueueOrBatchTickTask(TArray<FTickFunction*, TInlineAllocator<2>>& Prerequisites, FTickFunction* TickFunction, const FTickContext& TickContext)
	{
		if (bAllowBatchedTicksForFrame && TickFunction->bAllowTickBatching)
		{
			FTickGroupCondition Condition = FTickGroupCondition(TickFunction);

			if (CanBatchCondition(Condition))
			{
				// Look for an appropriate batch
				FTickBatchInfo* BatchInfo = nullptr;
				for (int32 BatchIndex = 0; BatchIndex < TickBatchesNum; BatchIndex++)
				{
					if (Condition == TickBatches[BatchIndex].Key)
					{
						FTickBatchInfo* PossibleBatch = TickBatches[BatchIndex].Value.Get();
						bool bPrerequisitesMatch = true;

						for (FTickFunction* Prereq : Prerequisites)
						{
							const bool bBatchRequiresPrerequisite = PossibleBatch->TickPrerequisites.Contains(Prereq);
							const bool bBatchExecutesPrerequisite = Prereq->GetTaskPointer(FTickFunction::ETickTaskState::HasTask) == PossibleBatch->TickTask;

							if (!bAllowBatchedTicksUnorderedForFrame)
							{
								// if batches must execute in-order (default), then we can accept this batch if either the batch
								// requires the same prerequisite, or if the prerequisite is already in this batch
								if (!bBatchRequiresPrerequisite && !bBatchExecutesPrerequisite)
								{
									bPrerequisitesMatch = false;
									break;
								}
							}
							else
							{
								// if batches may execute their functions out of order, then we can't accept a batch
								// that itself executes the prerequisite, we can only accept batches that have the prerequisite
								// as a prerequisite of the batch itself
								if (!bBatchRequiresPrerequisite)
								{
									bPrerequisitesMatch = false;
									break;
								}
							}
						}
						if (bPrerequisitesMatch)
						{
							BatchInfo = PossibleBatch;
							break;
						}
					}
				}

				if (!BatchInfo)
				{
					// Create a new batch, resizing array if needed
					check(TickBatchesNum <= TickBatches.Num());
					if (TickBatchesNum == TickBatches.Num())
					{
						TickBatches.Emplace(FTickGroupCondition(), MakeUnique<FTickBatchInfo>());
						check(TickBatches.IsValidIndex(TickBatchesNum));
					}
				
					TickBatches[TickBatchesNum].Key = Condition;
					BatchInfo = TickBatches[TickBatchesNum].Value.Get();
					TickBatchesNum++;

					check(BatchInfo->TickTask == nullptr);
					
					FTickContext UseContext = SetupTickContext(TickFunction, TickContext);
					if (Prerequisites.Num() > 0)
					{
						BatchInfo->TickPrerequisites = Prerequisites;
						FGraphEventArray PrerequisiteEvents;
						for (FTickFunction* Prereq : Prerequisites)
						{
							PrerequisiteEvents.Add(Prereq->GetCompletionHandle());
						}
						BatchInfo->TickTask = TGraphTask<FBatchTickFunctionTask>::CreateTask(&PrerequisiteEvents, ENamedThreads::GameThread).ConstructAndHold(BatchInfo, &UseContext);
					}
					else
					{
						BatchInfo->TickTask = TGraphTask<FBatchTickFunctionTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(BatchInfo, &UseContext);
					}
					

					AddTickTaskCompletion(Condition.StartGroup, Condition.EndGroup, BatchInfo->TickTask, Condition.bHighPriority);
				}

				// Add this tick function to batch, which could be the first one
				BatchInfo->TickFunctions.Add(TickFunction);
				TickFunction->SetTaskPointer(FTickFunction::ETickTaskState::HasTask, BatchInfo->TickTask);

				return BatchInfo;
			}
		}

		// No batching, create a single task
		if (Prerequisites.Num() > 0)
		{
			FGraphEventArray PrerequisiteEvents;
			for (FTickFunction* Prereq : Prerequisites)
			{
				PrerequisiteEvents.Add(Prereq->GetCompletionHandle());
			}

			QueueTickTask(&PrerequisiteEvents, TickFunction, TickContext);
		}
		else
		{
			QueueTickTask(nullptr, TickFunction, TickContext);
		}

		return nullptr;
	}

	/**
	 * Start a tick task and add the completion handle
	 *
	 * @param	InPrerequisites - prerequisites that must be completed before this tick can begin
	 * @param	TickFunction - the tick function to queue
	 * @param	Context - tick context to tick in. Thread here is the current thread.
	 */
	FORCEINLINE void QueueTickTask(const FGraphEventArray* Prerequisites, FTickFunction* TickFunction, const FTickContext& TickContext)
	{
		FTickContext UseContext = SetupTickContext(TickFunction, TickContext);
		FTickGraphTask* Task = TGraphTask<FTickFunctionTask>::CreateTask(Prerequisites, ENamedThreads::GameThread).ConstructAndHold(TickFunction, &UseContext);
		TickFunction->SetTaskPointer(FTickFunction::ETickTaskState::HasTask, Task);

		if (TickFunction->bDispatchManually)
		{
			const ETickingGroup TickGroup = TickFunction->InternalData->ActualEndTickGroup;
			ManualDispatchTicks[TickGroup].Add(TickFunction);
			TickCompletionEvents[TickGroup].Add(Task->GetCompletionEvent());
			TickFunction->bWasDispatchedManually = false;
		}
		else
		{
			AddTickTaskCompletion(TickFunction->InternalData->ActualStartTickGroup, TickFunction->InternalData->ActualEndTickGroup, Task, TickFunction->bHighPriority);
		}
	}

	/**
	 * Start a tick task and add the completion handle, for parallel queuing
	 *
	 * @param	InPrerequisites - prerequisites that must be completed before this tick can begin
	 * @param	TickFunction - the tick function to queue
	 * @param	Context - tick context to tick in. Thread here is the current thread.
	 */
	FORCEINLINE void QueueTickTaskParallel(const FGraphEventArray* Prerequisites, FTickFunction* TickFunction, const FTickContext& TickContext)
	{
		FTickContext UseContext = SetupTickContext(TickFunction, TickContext);
		FTickGraphTask* Task = TGraphTask<FTickFunctionTask>::CreateTask(Prerequisites, ENamedThreads::GameThread).ConstructAndHold(TickFunction, &UseContext);
		TickFunction->SetTaskPointer(FTickFunction::ETickTaskState::HasTask, Task);

		if (TickFunction->bDispatchManually)
		{
			const ETickingGroup TickGroup = TickFunction->InternalData->ActualEndTickGroup;
			ManualDispatchTicks[TickGroup].AddThreadsafe(TickFunction);
			TickCompletionEvents[TickGroup].AddThreadsafe(Task->GetCompletionEvent());
			TickFunction->bWasDispatchedManually = false;
		}
		else
		{
			AddTickTaskCompletionParallel(TickFunction->InternalData->ActualStartTickGroup, TickFunction->InternalData->ActualEndTickGroup, Task, TickFunction->bHighPriority);
		}
	}

	/** Make sure all manual tick dispatches have happened to avoid deadlocks */
	void VerifyManualDispatch(ETickingGroup WorldTickGroup)
	{
		for (FTickFunction* ManualDispatchTick : ManualDispatchTicks[WorldTickGroup])
		{
			if (ManualDispatchTick->CanDispatchManually())
			{
				// TODO We may want to warn if this task has prerequisites as that would lead to confusing behavior
				
				// This could return false if it was just dispatched on another thread
				ManualDispatchTick->DispatchManually();
			}
		}
	}

	/**
	 * Release the queued ticks for a given tick group and process them.
	 * @param WorldTickGroup - tick group to release
	 * @param bBlockTillComplete - if true, do not return until all ticks are complete
	 * @param TickFunctionsToManualDispatch - dispatch any manual tick functions in this lister after the normal ones
	**/
	void ReleaseTickGroup(ETickingGroup WorldTickGroup, bool bBlockTillComplete, TArray<FTickFunction*>& TicksToManualDispatch)
	{
		if (bLogTicks)
		{
			UE_LOGF(LogTick, Log, "tick %6llu ---------------------------------------- Release tick group %d",(uint64)GFrameCounter, (int32)WorldTickGroup);
		}
		checkSlow(WorldTickGroup >= 0 && WorldTickGroup < TG_MAX);

		{
			SCOPE_CYCLE_COUNTER(STAT_ReleaseTickGroup);
			if (bSingleThreadMode || CVarAllowAsyncTickDispatch.GetValueOnGameThread() == 0)
			{
				DispatchTickGroup(ENamedThreads::GameThread, WorldTickGroup);
			}
			else
			{
				// dispatch the tick group on another thread, that way, the game thread can be processing ticks while ticks are being queued by another thread
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(
					TGraphTask<FDipatchTickGroupTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(*this, WorldTickGroup));
			}
		}

		for (FTickFunction* TickToManualDispatch : TicksToManualDispatch)
		{
			// These must be safe to dispatch
			ensure(TickToManualDispatch->DispatchManually());
		}
		TicksToManualDispatch.Reset();

		if (bBlockTillComplete || bSingleThreadMode)
		{
			SCOPE_CYCLE_COUNTER(STAT_ReleaseTickGroup_Block);
			for (ETickingGroup Block = WaitForTickGroup; Block <= WorldTickGroup; Block = ETickingGroup(Block + 1))
			{
				CA_SUPPRESS(6385);
				if (TickCompletionEvents[Block].Num())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(TickCompletionEvents);

					if (GIdleTaskWorkMS >= 0.0)
					{
						bool bDoIdleWork = (GIdleTaskWorkMS > 0.0);
						bool bDoDeadlockCheck = ManualDispatchTicks[Block].Num() > 0;
						uint64 EndIdleWork = 0;
						int32 PreviousTasksRemaining = TickCompletionEvents[Block].Num();
						int32 IdleCount = 0;

						auto IdleWorkUpdate = [this, &bDoIdleWork, &bDoDeadlockCheck, &EndIdleWork, &PreviousTasksRemaining, &IdleCount, Block]
							(int32 TasksRemaining) -> FTaskGraphInterface::EProcessTasksOperation
						{
							if (bDoIdleWork)
							{
								// Compute cycles for ending idle work if necessary, this is called after the first pass of processing
								if (EndIdleWork == 0)
								{
									const double TaskWorkSeconds = (GIdleTaskWorkMS / 1000.0);
									EndIdleWork = FPlatformTime::Cycles64() + FMath::TruncToInt64(TaskWorkSeconds / FPlatformTime::GetSecondsPerCycle64());
								}
								else if (FPlatformTime::Cycles64() > EndIdleWork)
								{
									bDoIdleWork = false;
								}
							}

							if (bDoDeadlockCheck)
							{
								if (TasksRemaining == PreviousTasksRemaining)
								{
									// No tasks were completed during last attempt
									constexpr int32 IdleCountToBeDeadlocked = 10;
									if (++IdleCount > IdleCountToBeDeadlocked)
									{
										// Nothing is changing so this could be deadlocked, make sure our manual dispatches have happened
										this->VerifyManualDispatch(Block);

										bDoDeadlockCheck = false;
									}
								}
								else
								{
									PreviousTasksRemaining = TasksRemaining;
									IdleCount = 0;
								}
							}
							
							if (bDoIdleWork)
							{
								// Run a worker thread task, this could also run other game thread tasks if we add support for that
								return FTaskGraphInterface::EProcessTasksOperation::ProcessOneOtherTask;
							}
							else if (bDoDeadlockCheck)
							{
								// Still need to check for deadlock so process named thread tasks if there are any
								return FTaskGraphInterface::EProcessTasksOperation::ProcessNamedThreadTasks;
							}
							else
							{
								// We don't need to do the idle or deadlock checks so wait forever
								return FTaskGraphInterface::EProcessTasksOperation::WaitUntilComplete;
							}
						};

						FTaskGraphInterface::Get().ProcessUntilTasksComplete(TickCompletionEvents[Block], ENamedThreads::GameThread, IdleWorkUpdate);
					}
					else
					{
						// Old behavior of waiting for all of them
						FTaskGraphInterface::Get().WaitUntilTasksComplete(TickCompletionEvents[Block], ENamedThreads::GameThread);
					}

					if (bSingleThreadMode || Block == TG_NewlySpawned || CVarAllowAsyncTickCleanup.GetValueOnGameThread() == 0 || TickCompletionEvents[Block].Num() < 50)
					{
						ResetTickGroup(Block);
					}
					else
					{
						DECLARE_CYCLE_STAT(TEXT("FDelegateGraphTask.ResetTickGroup"), STAT_FDelegateGraphTask_ResetTickGroup, STATGROUP_TaskGraphTasks);
						CleanupTasks.Add(TGraphTask<FResetTickGroupTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(*this, Block));
					}
				}
			}
			WaitForTickGroup = ETickingGroup(WorldTickGroup + (WorldTickGroup == TG_NewlySpawned ? 0 : 1)); // don't advance for newly spawned
		}
		else
		{
			// since this is used to soak up some async time for another task (physics), we should process whatever we have now
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			check(WorldTickGroup + 1 < TG_MAX && WorldTickGroup != TG_NewlySpawned); // you must block on the last tick group! And we must block on newly spawned
		}
	}

	/**
	 * Resets the internal state of the object at the start of a frame
	 */
	void StartFrame()
	{
		bLogTicks = !!CVarLogTicks.GetValueOnGameThread();
		bLogTicksShowPrerequistes = !!CVarLogTicksShowPrerequistes.GetValueOnGameThread();

		// Always cache the setting at the start of the tick process because in some rare cases (forking) the process can switch from single-thread to multi-thread mid-tick
		bSingleThreadMode = SingleThreadedMode();

		if (bLogTicks)
		{
			UE_LOGF(LogTick, Log, "tick %6llu ---------------------------------------- Start Frame",(uint64)GFrameCounter);
		}

		if (bSingleThreadMode)
		{
			bAllowConcurrentTicks = false;
		}
		else
		{
			bAllowConcurrentTicks = !!CVarAllowAsyncComponentTicks.GetValueOnGameThread();
		}

		bAllowBatchedTicksForFrame = !!CVarAllowBatchedTicks.GetValueOnGameThread();
		bAllowBatchedTicksUnorderedForFrame = !!CVarAllowBatchedTicksUnordered.GetValueOnGameThread();
		bAllowOptimizedPrerequisites = !!CVarAllowOptimizedPrerequisites.GetValueOnGameThread();

		WaitForCleanup();

		for (int32 Index = 0; Index < TG_MAX; Index++)
		{
			check(!TickCompletionEvents[Index].Num());  // we should not be adding to these outside of a ticking proper and they were already cleared after they were ticked
			TickCompletionEvents[Index].Reset();
			ManualDispatchTicks[Index].Reset();
			for (int32 IndexInner = 0; IndexInner < TG_MAX; IndexInner++)
			{
				check(!TickTasks[Index][IndexInner].Num() && !HiPriTickTasks[Index][IndexInner].Num());  // we should not be adding to these outside of a ticking proper and they were already cleared after they were ticked
				TickTasks[Index][IndexInner].Reset();
				HiPriTickTasks[Index][IndexInner].Reset();
			}
		}
		WaitForTickGroup = (ETickingGroup)0;
	}
	/**
	 * Checks that everything is clean at the end of a frame
	 */
	void EndFrame()
	{
		if (bLogTicks)
		{
			UE_LOGF(LogTick, Log, "tick %6llu ---------------------------------------- End Frame",(uint64)GFrameCounter);
		}

		// Clear out batched tick data but leave allocated for next frame
		for (TPair<FTickGroupCondition, TUniquePtr<FTickBatchInfo> >& Pair : TickBatches)
		{
			Pair.Key.IntVersion = 0;
			Pair.Value->Reset();
		}
		TickBatchesNum = 0;

	}
private:

	FTickTaskSequencer()
		: bAllowConcurrentTicks(false)
		, bAllowBatchedTicksForFrame(false)
		, bAllowOptimizedPrerequisites(false)
		, bLogTicks(false)
		, bLogTicksShowPrerequistes(false)
	{
		TFunction<void()> ShutdownCallback([this](){WaitForCleanup();});
		FTaskGraphInterface::Get().AddShutdownCallback(ShutdownCallback);
	}

	~FTickTaskSequencer()
	{
		// Need to clean up oustanding tasks before we destroy this data structure.
		// Typically it is already gone because the task graph shutdown first.
		WaitForCleanup();
	}

	void WaitForCleanup()
	{
		if (CleanupTasks.Num() > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_CleanupTasksWait);
			FTaskGraphInterface::Get().WaitUntilTasksComplete(CleanupTasks, ENamedThreads::GameThread);
			CleanupTasks.Reset();
		}
	}

	void ResetTickGroup(ETickingGroup WorldTickGroup)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ResetTickGroup);
		TickCompletionEvents[WorldTickGroup].Reset();
		ManualDispatchTicks[WorldTickGroup].Reset();
	}

	void DispatchTickGroup(ENamedThreads::Type CurrentThread, ETickingGroup WorldTickGroup)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_DispatchTickGroup);
		for (int32 IndexInner = 0; IndexInner < TG_MAX; IndexInner++)
		{
			TArray<FTickGraphTask*>& TickArray = HiPriTickTasks[WorldTickGroup][IndexInner]; //-V781
			if (IndexInner < WorldTickGroup)
			{
				check(TickArray.Num() == 0); // makes no sense to have and end TG before the start TG
			}
			else
			{
				for (int32 Index = 0; Index < TickArray.Num(); Index++)
				{
					TickArray[Index]->Unlock(CurrentThread);
				}
			}
			TickArray.Reset();
		}
		for (int32 IndexInner = 0; IndexInner < TG_MAX; IndexInner++)
		{
			TArray<FTickGraphTask*>& TickArray = TickTasks[WorldTickGroup][IndexInner]; //-V781
			if (IndexInner < WorldTickGroup)
			{
				check(TickArray.Num() == 0); // makes no sense to have and end TG before the start TG
			}
			else
			{
				for (int32 Index = 0; Index < TickArray.Num(); Index++)
				{
					TickArray[Index]->Unlock(CurrentThread);
				}
			}
			TickArray.Reset();
		}
	}

};


class FTickTaskLevel
{
public:
	/** Constructor, grabs, the sequencer singleton **/
	FTickTaskLevel()
		: TickTaskSequencer(FTickTaskSequencer::Get())
		, bTickNewlySpawned(false)
	{
	}
	~FTickTaskLevel()
	{
		for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
		{
			(*It)->InternalData->bRegistered = false;
		}
		for (TSet<FTickFunction*>::TIterator It(AllDisabledTickFunctions); It; ++It)
		{
			(*It)->InternalData->bRegistered = false;
		}
		FTickFunction* CoolingDownNode = AllCoolingDownTickFunctions.Head;
		while (CoolingDownNode)
		{
			CoolingDownNode->InternalData->bRegistered = false;
			CoolingDownNode = CoolingDownNode->InternalData->Next;
		}
		for (FTickScheduleDetails& TickDetails : TickFunctionsToReschedule)
		{
			TickDetails.TickFunction->InternalData->bRegistered = false;
		}
	}

	/**
	 * Queues the ticks for this level
	 * @param InContext - information about the tick
	 * @return the total number of ticks we will be ticking
	 */
	int32 StartFrame(const FTickContext& InContext)
	{
		check(!NewlySpawnedTickFunctions.Num()); // There shouldn't be any in here at this point in the frame
		Context.TickGroup = ETickingGroup(0); // reset this to the start tick group
		Context.DeltaSeconds = InContext.DeltaSeconds;
		Context.TickType = InContext.TickType;
		Context.Thread = ENamedThreads::GameThread;
		Context.World = InContext.World;
		bTickNewlySpawned = true;

		int32 CooldownTicksEnabled = 0;
		{
			// Make sure all scheduled Tick Functions that are ready are put into the cooling down state
			ScheduleTickFunctionCooldowns();

			// Determine which cooled down ticks will be enabled this frame
			float CumulativeCooldown = 0.f;
			FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
			while (TickFunction)
			{
				if (CumulativeCooldown + TickFunction->InternalData->RelativeTickCooldown >= Context.DeltaSeconds)
				{
					TickFunction->InternalData->RelativeTickCooldown -= (Context.DeltaSeconds - CumulativeCooldown);
					break;
				}
				CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;

				TickFunction->TickState = FTickFunction::ETickState::Enabled;
				TickFunction = TickFunction->InternalData->Next;
				++CooldownTicksEnabled;
			}
		}

		return AllEnabledTickFunctions.Num() + CooldownTicksEnabled;
	}

	/**
	 * Queues the ticks for this level
	 * @param InContext - information about the tick
	 */
	void StartFrameParallel(const FTickContext& InContext, TArray<FTickFunction*>& AllTickFunctions)
	{
		check(!NewlySpawnedTickFunctions.Num()); // There shouldn't be any in here at this point in the frame
		Context.TickGroup = ETickingGroup(0); // reset this to the start tick group
		Context.DeltaSeconds = InContext.DeltaSeconds;
		Context.TickType = InContext.TickType;
		Context.Thread = ENamedThreads::GameThread;
		Context.World = InContext.World;
		bTickNewlySpawned = true;

		for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
		{
			FTickFunction* TickFunction = *It;
			AllTickFunctions.Add(TickFunction);
		}
		
		{
			// Make sure all scheduled Tick Functions that are ready are put into the cooling down state
			ScheduleTickFunctionCooldowns();

			// Determine which cooled down ticks will be enabled this frame
			float CumulativeCooldown = 0.f;
			FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
			while (TickFunction)
			{
				if (CumulativeCooldown + TickFunction->InternalData->RelativeTickCooldown >= Context.DeltaSeconds)
				{
					TickFunction->InternalData->RelativeTickCooldown -= (Context.DeltaSeconds - CumulativeCooldown);
					break;
				}
				CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;

				TickFunction->TickState = FTickFunction::ETickState::Enabled;
				AllTickFunctions.Add(TickFunction);

				RescheduleForInterval(TickFunction, TickFunction->TickInterval - (Context.DeltaSeconds - CumulativeCooldown)); // Give credit for any overrun

				AllCoolingDownTickFunctions.Head = TickFunction->InternalData->Next;
				TickFunction = TickFunction->InternalData->Next;
			}
		}
	}

	struct FTickScheduleDetails
	{
		FTickScheduleDetails(FTickFunction* InTickFunction, const float InCooldown, bool bInDeferredRemove = false)
			: TickFunction(InTickFunction)
			, Cooldown(InCooldown)
			, bDeferredRemove(bInDeferredRemove)
		{}

		FTickFunction* TickFunction;
		float Cooldown;
		bool bDeferredRemove;
	};

	/** Returns true if found in reschedule list and interval was updated */
	bool UpdateRescheduleInterval(FTickFunction* TickFunction, float InInterval)
	{
		auto FindTickFunctionInRescheduleList = [TickFunction](const FTickScheduleDetails& TSD)
		{
			return (TSD.TickFunction == TickFunction);
		};
		FTickScheduleDetails* TickDetails = TickFunctionsToReschedule.FindByPredicate(FindTickFunctionInRescheduleList);
		if (TickDetails)
		{
			TickDetails->Cooldown = InInterval;
			return true;
		}
		return false;
	}

	void RescheduleForInterval(FTickFunction* TickFunction, float InInterval)
	{
		TickFunction->InternalData->bWasInterval = true;
		TickFunctionsToReschedule.Add(FTickScheduleDetails(TickFunction, InInterval));
	}

	void RescheduleForIntervalParallel(FTickFunction* TickFunction)
	{
		// note we do the remove later!
		TickFunction->InternalData->bWasInterval = true;
		TickFunctionsToReschedule.AddThreadsafe(FTickScheduleDetails(TickFunction, TickFunction->TickInterval, true));
	}
	/* Helper to presize reschedule array */
	void ReserveTickFunctionCooldowns(int32 NumToReserve)
	{
		TickFunctionsToReschedule.Reserve(TickFunctionsToReschedule.Num() + NumToReserve);
	}
	/* Do deferred removes */
	void DoDeferredRemoves()
	{
		if (TickFunctionsToReschedule.Num() > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_DoDeferredRemoves);

			for (FTickScheduleDetails& TickDetails : TickFunctionsToReschedule)
			{
				if (TickDetails.bDeferredRemove && TickDetails.TickFunction->TickState != FTickFunction::ETickState::Disabled)
				{
					verify(AllEnabledTickFunctions.Remove(TickDetails.TickFunction) == 1);
				}
			}
		}
	}

	/* Puts a TickFunction in to the cooldown state*/
	void ScheduleTickFunctionCooldowns()
	{
		if (TickFunctionsToReschedule.Num() > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_ScheduleCooldowns);

			TickFunctionsToReschedule.Sort([](const FTickScheduleDetails& A, const FTickScheduleDetails& B)
			{
				return A.Cooldown < B.Cooldown;
			});

			int32 RescheduleIndex = 0;
			float CumulativeCooldown = 0.f;
			FTickFunction* PrevComparisonTickFunction = nullptr;
			FTickFunction* ComparisonTickFunction = AllCoolingDownTickFunctions.Head;
			while (ComparisonTickFunction && RescheduleIndex < TickFunctionsToReschedule.Num())
			{
				const float CooldownTime = TickFunctionsToReschedule[RescheduleIndex].Cooldown;
				if ((CumulativeCooldown + ComparisonTickFunction->InternalData->RelativeTickCooldown) > CooldownTime)
				{
					FTickFunction* TickFunction = TickFunctionsToReschedule[RescheduleIndex].TickFunction;
					check(TickFunction->InternalData->bWasInterval);
					if (TickFunction->TickState != FTickFunction::ETickState::Disabled)
					{
						TickFunction->TickState = FTickFunction::ETickState::CoolingDown;
						TickFunction->InternalData->RelativeTickCooldown = CooldownTime - CumulativeCooldown;

						if (PrevComparisonTickFunction)
						{
							PrevComparisonTickFunction->InternalData->Next = TickFunction;
						}
						else
						{
							check(ComparisonTickFunction == AllCoolingDownTickFunctions.Head);
							AllCoolingDownTickFunctions.Head = TickFunction;
						}
						TickFunction->InternalData->Next = ComparisonTickFunction;
						PrevComparisonTickFunction = TickFunction;
						ComparisonTickFunction->InternalData->RelativeTickCooldown -= TickFunction->InternalData->RelativeTickCooldown;
						CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;
					}
					++RescheduleIndex;
				}
				else
				{
					CumulativeCooldown += ComparisonTickFunction->InternalData->RelativeTickCooldown;
					PrevComparisonTickFunction = ComparisonTickFunction;
					ComparisonTickFunction = ComparisonTickFunction->InternalData->Next;
				}
			}
			for ( ; RescheduleIndex < TickFunctionsToReschedule.Num(); ++RescheduleIndex)
			{
				FTickFunction* TickFunction = TickFunctionsToReschedule[RescheduleIndex].TickFunction;
				checkSlow(TickFunction);
				if (TickFunction->TickState != FTickFunction::ETickState::Disabled)
				{
					const float CooldownTime = TickFunctionsToReschedule[RescheduleIndex].Cooldown;

					TickFunction->TickState = FTickFunction::ETickState::CoolingDown;
					TickFunction->InternalData->RelativeTickCooldown = CooldownTime - CumulativeCooldown;

					TickFunction->InternalData->Next = nullptr;
					if (PrevComparisonTickFunction)
					{
						PrevComparisonTickFunction->InternalData->Next = TickFunction;
					}
					else
					{
						check(ComparisonTickFunction == AllCoolingDownTickFunctions.Head);
						AllCoolingDownTickFunctions.Head = TickFunction;
					}
					PrevComparisonTickFunction = TickFunction;

					CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;
				}
			}
			TickFunctionsToReschedule.Reset();
		}
	}

	/* Queue all tick functions for execution */
	void QueueAllTicks()
	{
		FTickTaskSequencer& TTS = FTickTaskSequencer::Get();

		// Only use the lower 32 bits of the frame counter
		uint32 CurrentFrameCounter = (uint32)GFrameCounter;
		
		for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
		{
			FTickFunction* TickFunction = *It;
			if (!TTS.HasBeenVisited(TickFunction, CurrentFrameCounter))
			{
				TickFunction->QueueTickFunction(TTS, Context);
			}

			if (TickFunction->TickInterval > 0.f)
			{
				It.RemoveCurrent();
				RescheduleForInterval(TickFunction, TickFunction->TickInterval);
			}
		}
		int32 EnabledCooldownTicks = 0;
		float CumulativeCooldown = 0.f;
		while (FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head)
		{
			if (TickFunction->TickState == FTickFunction::ETickState::Enabled)
			{
				CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;
				if (!TTS.HasBeenVisited(TickFunction, CurrentFrameCounter))
				{
					TickFunction->QueueTickFunction(TTS, Context);
				}
				RescheduleForInterval(TickFunction, TickFunction->TickInterval - (Context.DeltaSeconds - CumulativeCooldown)); // Give credit for any overrun
				AllCoolingDownTickFunctions.Head = TickFunction->InternalData->Next;
			}
			else
			{
				break;
			}
		}
	}
	/**
	 * Queues the newly spawned ticks for this level
	 * @return - the number of items
	 */
	int32 QueueNewlySpawned(ETickingGroup CurrentTickGroup)
	{
		Context.TickGroup = CurrentTickGroup;
		int32 Num = 0;

		// Constructing set iterators is not trivial, so avoid the following block if it will have no effect
		if (NewlySpawnedTickFunctions.Num() != 0)
		{
			uint32 CurrentFrameCounter = (uint32)GFrameCounter;
			FTickTaskSequencer& TTS = FTickTaskSequencer::Get();
			for (TSet<FTickFunction*>::TIterator It(NewlySpawnedTickFunctions); It; ++It)
			{
				FTickFunction* TickFunction = *It;
				if (!TTS.HasBeenVisited(TickFunction, CurrentFrameCounter))
				{
					TickFunction->QueueTickFunction(TTS, Context);
				}
				Num++;

				if (TickFunction->TickInterval > 0.f)
				{
					AllEnabledTickFunctions.Remove(TickFunction);
					RescheduleForInterval(TickFunction, TickFunction->TickInterval);
				}
			}
			NewlySpawnedTickFunctions.Empty();
		}
		return Num;
	}
	/**
	 * If there is infinite recursive spawning, log that and discard them
	 */
	void LogAndDiscardRunawayNewlySpawned(ETickingGroup CurrentTickGroup)
	{
		Context.TickGroup = CurrentTickGroup;
		FTickTaskSequencer& TTS = FTickTaskSequencer::Get();
		for (TSet<FTickFunction*>::TIterator It(NewlySpawnedTickFunctions); It; ++It)
		{
			FTickFunction* TickFunction = *It;
			UE_LOGF(LogTick, Error, "Could not tick newly spawned in 100 iterations; runaway recursive spawing. Tick is %ls.", *TickFunction->DiagnosticMessage());

			if (TickFunction->TickInterval > 0.f)
			{
				AllEnabledTickFunctions.Remove(TickFunction);
				RescheduleForInterval(TickFunction, TickFunction->TickInterval);
			}
		}
		NewlySpawnedTickFunctions.Empty();
	}

	/**
	 * Run all of the ticks for a pause frame synchronously on the game thread.
	 * The capability of pause ticks are very limited. There are no dependencies or ordering or tick groups.
	 * @param InContext - information about the tick
	 */
	void RunPauseFrame(const FTickContext& InContext)
	{
		check(!NewlySpawnedTickFunctions.Num()); // There shouldn't be any in here at this point in the frame

		TArray<FTickFunction*> ExecuteTickFunctions;

		float CumulativeCooldown = 0.f;
		FTickFunction* PrevTickFunction = nullptr;
		FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
		while (TickFunction)
		{
			CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;
			if (TickFunction->bTickEvenWhenPaused)
			{
				bool bExecuteTick = false;
				TickFunction->SetTaskPointer(FTickFunction::ETickTaskState::NotQueued, nullptr); // this is stale, clear it out now
				if (CumulativeCooldown < InContext.DeltaSeconds)
				{
					// Queue up the tick function for later and do the reschedule before as it is in the normal ticking logic
					ExecuteTickFunctions.Add(TickFunction);
					RescheduleForInterval(TickFunction, TickFunction->TickInterval - (InContext.DeltaSeconds - CumulativeCooldown)); // Give credit for any overrun
				}
				else
				{
					RescheduleForInterval(TickFunction, CumulativeCooldown - InContext.DeltaSeconds);
				}
				if (PrevTickFunction)
				{
					PrevTickFunction->InternalData->Next = TickFunction->InternalData->Next;
				}
				else
				{
					check(TickFunction == AllCoolingDownTickFunctions.Head);
					AllCoolingDownTickFunctions.Head = TickFunction->InternalData->Next;
				}
				if (TickFunction->InternalData->Next)
				{
					TickFunction->InternalData->Next->InternalData->RelativeTickCooldown += TickFunction->InternalData->RelativeTickCooldown;
					CumulativeCooldown -= TickFunction->InternalData->RelativeTickCooldown; // Since the next object in the list will have this cooldown included take it back out of the cumulative
				}
			}
			else
			{
				PrevTickFunction = TickFunction;
			}
			TickFunction = TickFunction->InternalData->Next;
		}

		for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
		{
			TickFunction = *It;
			TickFunction->SetTaskPointer(FTickFunction::ETickTaskState::NotQueued, nullptr); // this is stale, clear it out now
			if (TickFunction->bTickEvenWhenPaused && TickFunction->TickState == FTickFunction::ETickState::Enabled)
			{
				// Queue up the tick function for later and do the reschedule before as it is in the normal ticking logic
				ExecuteTickFunctions.Add(TickFunction);

				if (TickFunction->TickInterval > 0.f)
				{
					It.RemoveCurrent();
					RescheduleForInterval(TickFunction, TickFunction->TickInterval);
				}
			}
		}

		check(!NewlySpawnedTickFunctions.Num()); // We don't support new spawns during pause ticks

		for (FTickFunction* TickFunctionToExecute : ExecuteTickFunctions)
		{
			// Only use the lower 32 bits of the frame counter
			uint32 CurrentFrameCounter = (uint32)GFrameCounter;
			TickFunctionToExecute->InternalData->TickVisitedGFrameCounter.store(CurrentFrameCounter, std::memory_order_relaxed);
			TickFunctionToExecute->InternalData->TickQueuedGFrameCounter.store(CurrentFrameCounter, std::memory_order_relaxed);
			TickFunctionToExecute->ExecuteTick(TickFunctionToExecute->CalculateDeltaTime(InContext.DeltaSeconds, InContext.World), InContext.TickType, ENamedThreads::GameThread, FGraphEventRef());
		}
	}

	/** End a tick frame **/
	void EndFrame()
	{
		ScheduleTickFunctionCooldowns();

		bTickNewlySpawned = false;
#if DO_CHECK
		// hmmm, this might be ok, but basically anything that was added this late cannot be ticked until the next frame
		if (NewlySpawnedTickFunctions.Num())
		{
			for (TSet<FTickFunction*>::TIterator It(NewlySpawnedTickFunctions); It; ++It)
			{
				FTickFunction* TickFunction = *It;
				UE_LOGF(LogTick, Error, "Newly spawned tick function was added after tick groups have been run. Tick is %ls.", *TickFunction->DiagnosticMessage());
			}

			ensureMsgf(false, TEXT("Newly spawned tick functions were added after tick groups have been run. See log for details."));
			NewlySpawnedTickFunctions.Empty();
		}
#endif
	}
	// Interface that is private to FTickFunction

	/** Return true if this tick function is in the primary list **/
	bool HasTickFunction(FTickFunction* TickFunction)
	{
		return AllEnabledTickFunctions.Contains(TickFunction) || AllDisabledTickFunctions.Contains(TickFunction) || AllCoolingDownTickFunctions.Contains(TickFunction);
	}

	/** Add the tick function to the primary list **/
	void AddTickFunction(FTickFunction* TickFunction)
	{
		check(!HasTickFunction(TickFunction));
		if (TickFunction->TickState == FTickFunction::ETickState::Enabled)
		{
			AllEnabledTickFunctions.Add(TickFunction);
			if (bTickNewlySpawned)
			{
				NewlySpawnedTickFunctions.Add(TickFunction);
			}
		}
		else
		{
			check(TickFunction->TickState == FTickFunction::ETickState::Disabled);
			AllDisabledTickFunctions.Add(TickFunction);
		}
	}

	/** Dumps info about a tick function to output device. */
	void DumpTickFunction(FOutputDevice& Ar, FTickFunction* Function, UEnum* TickGroupEnum, const float RemainingCooldown = 0.f)
	{
		// Info about the function.
		Ar.Logf(TEXT("%s, %s, ActualStartTickGroup: %s, Prerequesities: %d"),
			*Function->DiagnosticMessage(),
			Function->IsTickFunctionEnabled() ? (RemainingCooldown > 0.f ? *FString::Printf(TEXT("Cooling Down for %.4g seconds"),RemainingCooldown) : TEXT("Enabled")) : TEXT("Disabled"),
			*TickGroupEnum->GetNameStringByValue(Function->GetActualTickGroup()),
			Function->Prerequisites.Num());

		// List all prerequisities
		for (int32 Index = 0; Index < Function->Prerequisites.Num(); ++Index)
		{
			const FTickPrerequisite& Prerequisite = Function->Prerequisites[Index];
			if (Prerequisite.PrerequisiteObject.IsValid())
			{
				Ar.Logf(TEXT("    %s, %s"), *Prerequisite.PrerequisiteObject->GetFullName(), *Prerequisite.PrerequisiteTickFunction->DiagnosticMessage());
			}
			else
			{
				Ar.Logf(TEXT("    Invalid Prerequisite"));
			}
		}

		// Handle any children if they exist
		Function->ForEachNestedTick([this, &Ar, TickGroupEnum, RemainingCooldown](FTickFunction& NestedTick)
		{
			DumpTickFunction(Ar, &NestedTick, TickGroupEnum, RemainingCooldown);
		});
	}

	/** Dumps all tick functions to output device. */
	void DumpAllTickFunctions(FOutputDevice& Ar, int32& EnabledCount, int32& DisabledCount, bool bEnabled, bool bDisabled)
	{
		UEnum* TickGroupEnum = CastChecked<UEnum>(StaticFindObject(UEnum::StaticClass(), nullptr, TEXT("/Script/Engine.ETickingGroup"), EFindObjectFlags::ExactClass));
		if (bEnabled)
		{
			for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
			{
				DumpTickFunction(Ar, *It, TickGroupEnum);
			}
			float CumulativeCooldown = 0.f;
			FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
			while (TickFunction)
			{
				CumulativeCooldown += TickFunction->InternalData->RelativeTickCooldown;
				DumpTickFunction(Ar, TickFunction, TickGroupEnum, CumulativeCooldown);
				TickFunction = TickFunction->InternalData->Next;
				++EnabledCount;
			}
		}
		else
		{
			FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
			while (TickFunction)
			{
				TickFunction = TickFunction->InternalData->Next;
				++EnabledCount;
			}
		}
		EnabledCount += AllEnabledTickFunctions.Num();
		if (bDisabled)
		{
			for (TSet<FTickFunction*>::TIterator It(AllDisabledTickFunctions); It; ++It)
			{
				DumpTickFunction(Ar, *It, TickGroupEnum);
			}
		}
		DisabledCount += AllDisabledTickFunctions.Num();
	}

	FORCEINLINE void AddTickFunctionToMap(TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess>& ClassNameToCountMap, FTickFunction* Function, bool bDetailed)
	{
		FName ContextName = Function->DiagnosticContext(bDetailed);
		// Find entry for this context (or add it if not present)
		int32& CurrentCount = ClassNameToCountMap.FindOrAdd(ContextName);
		// Increment count
		CurrentCount++; 
	}

	void AddTickFunctionsToMap(TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess>& ClassNameToCountMap, int32& EnabledCount, bool bDetailed, bool bFilterCoolingDown, float CurrentTime, float CurrentUnpausedTime)
	{
		// Add ticks from AllEnabledTickFunctions
		for (TSet<FTickFunction*>::TIterator It(AllEnabledTickFunctions); It; ++It)
		{
			AddTickFunctionToMap(ClassNameToCountMap, *It, bDetailed);
		}
		EnabledCount += AllEnabledTickFunctions.Num();

		// Add ticks that are cooling down
		float CumulativeCooldown = 0.f;
		FTickFunction* TickFunction = AllCoolingDownTickFunctions.Head;
		while (TickFunction)
		{
			// Note: Timestamp check assumes TickFunction->TickGroup has been evaluated this frame
			if (bFilterCoolingDown && TickFunction->GetLastIntervalTickGameTime() != (TickFunction->bTickEvenWhenPaused ? CurrentUnpausedTime : CurrentTime))
			{				
				TickFunction = TickFunction->InternalData->Next;
				continue;
			}

			AddTickFunctionToMap(ClassNameToCountMap, TickFunction, bDetailed);
			TickFunction = TickFunction->InternalData->Next;
			++EnabledCount;
		}
	}

	/** Remove the tick function from the primary list **/
	void RemoveTickFunction(FTickFunction* TickFunction)
	{
		switch(TickFunction->TickState)
		{
		case FTickFunction::ETickState::Enabled:
			if (TickFunction->InternalData->bWasInterval)
			{
				// An enabled function with a tick interval could be in either the enabled or cooling down list
				if (AllEnabledTickFunctions.Remove(TickFunction) == 0)
				{
					auto FindTickFunctionInRescheduleList = [TickFunction](const FTickScheduleDetails& TSD)
					{
						return (TSD.TickFunction == TickFunction);
					};
					int32 Index = TickFunctionsToReschedule.IndexOfByPredicate(FindTickFunctionInRescheduleList);
					bool bFound = Index != INDEX_NONE;
					if (bFound)
					{
						TickFunctionsToReschedule.RemoveAtSwap(Index);
					}
					FTickFunction* PrevComparisionFunction = nullptr;
					FTickFunction* ComparisonFunction = AllCoolingDownTickFunctions.Head;
					while (ComparisonFunction && !bFound)
					{
						if (ComparisonFunction == TickFunction)
						{
							bFound = true;
							if (PrevComparisionFunction)
							{
								PrevComparisionFunction->InternalData->Next = TickFunction->InternalData->Next;
							}
							else
							{
								check(TickFunction == AllCoolingDownTickFunctions.Head);
								AllCoolingDownTickFunctions.Head = TickFunction->InternalData->Next;
							}
							TickFunction->InternalData->Next = nullptr;
						}
						else
						{
							PrevComparisionFunction = ComparisonFunction;
							ComparisonFunction = ComparisonFunction->InternalData->Next;
						}
					}
					check(bFound); // otherwise you changed TickState while the tick function was registered. Call SetTickFunctionEnable instead.
				}
			}
			else
			{
				verify(AllEnabledTickFunctions.Remove(TickFunction) == 1); // otherwise you changed TickState while the tick function was registered. Call SetTickFunctionEnable instead.
			}
			break;

		case FTickFunction::ETickState::Disabled:
			verify(AllDisabledTickFunctions.Remove(TickFunction) == 1); // otherwise you changed TickState while the tick function was registered. Call SetTickFunctionEnable instead.
			break;

		case FTickFunction::ETickState::CoolingDown:
			auto FindTickFunctionInRescheduleList = [TickFunction](const FTickScheduleDetails& TSD)
			{
				return (TSD.TickFunction == TickFunction);
			};
			int32 Index = TickFunctionsToReschedule.IndexOfByPredicate(FindTickFunctionInRescheduleList);
			bool bFound = Index != INDEX_NONE;
			if (bFound)
			{
				TickFunctionsToReschedule.RemoveAtSwap(Index);
			}
			FTickFunction* PrevComparisonFunction = nullptr;
			FTickFunction* ComparisonFunction = AllCoolingDownTickFunctions.Head;
			while (ComparisonFunction && !bFound)
			{
				if (ComparisonFunction == TickFunction)
				{
					bFound = true;
					if (PrevComparisonFunction)
					{
						PrevComparisonFunction->InternalData->Next = TickFunction->InternalData->Next;
					}
					else
					{
						check(TickFunction == AllCoolingDownTickFunctions.Head);
						AllCoolingDownTickFunctions.Head = TickFunction->InternalData->Next;
					}
					if (TickFunction->InternalData->Next)
					{
						TickFunction->InternalData->Next->InternalData->RelativeTickCooldown += TickFunction->InternalData->RelativeTickCooldown;
						TickFunction->InternalData->Next = nullptr;
					}
				}
				else
				{
					PrevComparisonFunction = ComparisonFunction;
					ComparisonFunction = ComparisonFunction->InternalData->Next;
				}
			}
			check(bFound); // otherwise you changed TickState while the tick function was registered. Call SetTickFunctionEnable instead.
			break;
		}
		if (bTickNewlySpawned)
		{
			NewlySpawnedTickFunctions.Remove(TickFunction);
		}
	}

private:

	struct FCoolingDownTickFunctionList
	{
		FCoolingDownTickFunctionList()
			: Head(nullptr)
		{
		}

		bool Contains(FTickFunction* TickFunction) const
		{
			FTickFunction* Node = Head;
			while (Node)
			{
				if (Node == TickFunction)
				{
					return true;
				}
				Node = Node->InternalData->Next;
			}
			return false;
		}

		FTickFunction* Head;
	};

	/** Global Sequencer														*/
	FTickTaskSequencer&							TickTaskSequencer;
	/** Primary list of enabled tick functions **/
	TSet<FTickFunction*>						AllEnabledTickFunctions;
	/** Primary list of enabled tick functions **/
	FCoolingDownTickFunctionList				AllCoolingDownTickFunctions;
	/** Primary list of disabled tick functions **/
	TSet<FTickFunction*>						AllDisabledTickFunctions;
	/** Utility array to avoid memory reallocations when collecting functions to reschedule **/
	TArrayWithThreadsafeAdd<FTickScheduleDetails>				TickFunctionsToReschedule;
	/** List of tick functions added during a tick phase; these items are also duplicated in AllLiveTickFunctions for future frames **/
	TSet<FTickFunction*>						NewlySpawnedTickFunctions;
	/** tick context **/
	FTickContext								Context;
	/** true during the tick phase, when true, tick function adds also go to the newly spawned list. **/
	bool										bTickNewlySpawned;
};

/** Helper struct to hold completion items from parallel task. They are moved into a separate place for cache coherency **/
struct FTickGroupCompletionItem
{
	/** Task created **/
	FTickGraphTask* Task;
	/** Tick group to complete with **/
	TEnumAsByte<ETickingGroup>	ActualStartTickGroup;
	/** True if this was a misplaced interval tick that we need to deal with **/
	bool bNeedsToBeRemovedFromTickListsAndRescheduled;
	/** True if this is hi pri **/
	bool bHiPri;
};

/** Class that aggregates the individual levels and deals with parallel tick setup **/
class FTickTaskManager : public FTickTaskManagerInterface
{
public:
	/**
	 * Singleton to retrieve the global tick task manager
	 * @return Reference to the global tick task manager
	**/
	static FTickTaskManager& Get()
	{
		static FTickTaskManager SingletonInstance;
		return SingletonInstance;
	}

	/** Allocate a new ticking structure for a ULevel **/
	virtual FTickTaskLevel* AllocateTickTaskLevel() override
	{
		return new FTickTaskLevel;
	}

	/** Free a ticking structure for a ULevel **/
	virtual void FreeTickTaskLevel(FTickTaskLevel* TickTaskLevel) override
	{
		check(!LevelList.Contains(TickTaskLevel));
		delete TickTaskLevel;
	}

	/**
	 * Ticks the dynamic actors in the given levels based upon their tick group. This function
	 * is called once for each ticking group
	 *
	 * @param World	- World currently ticking
	 * @param DeltaSeconds - time in seconds since last tick
	 * @param TickType - type of tick (viewports only, time only, etc)
	 * @param LevelsToTick - the levels to tick, may be a subset of InWorld->Levels
	 */
	virtual void StartFrame(UWorld* InWorld, float InDeltaSeconds, ELevelTick InTickType, const TArray<ULevel*>& LevelsToTick) override
	{
		SCOPE_CYCLE_COUNTER(STAT_QueueTicks);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(QueueTicks);

#if !UE_BUILD_SHIPPING
		if (CVarStallStartFrame.GetValueOnGameThread() > 0.0f)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Tick_Intentional_Stall);
			FPlatformProcess::Sleep(CVarStallStartFrame.GetValueOnGameThread() / 1000.0f);
		}
#endif
		if (FTaskSyncManager* SyncManager = FTaskSyncManager::Get())
		{
			// This can create tick functions
			SyncManager->StartFrame(InWorld, InDeltaSeconds, InTickType);
		}

		Context.TickGroup = ETickingGroup(0); // reset this to the start tick group
		Context.DeltaSeconds = InDeltaSeconds;
		Context.TickType = InTickType;
		Context.Thread = ENamedThreads::GameThread;
		Context.World = InWorld;

		bTickNewlySpawned = true;
		TickTaskSequencer.StartFrame();
		FillLevelList(LevelsToTick);

		int32 NumWorkerThread = 0;
		bool bConcurrentQueue = false;

		if (!FTickTaskSequencer::SingleThreadedMode())
		{
			// Concurrent tick may be faster in some situations but can change the order of ticking
			bConcurrentQueue = !!CVarAllowConcurrentQueue.GetValueOnGameThread();
		}

		if (!bConcurrentQueue)
		{
			int32 TotalTickFunctions = 0;
			for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
			{
				TotalTickFunctions += LevelList[LevelIndex]->StartFrame(Context);
			}
			INC_DWORD_STAT_BY(STAT_TicksQueued, TotalTickFunctions);
			CSV_CUSTOM_STAT(Basic, TicksQueued, TotalTickFunctions, ECsvCustomStatOp::Accumulate);
			TickTaskSequencer.SetupBatchedTicks(TotalTickFunctions);
			for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
			{
				LevelList[LevelIndex]->QueueAllTicks();
			}
			TickTaskSequencer.FinishBatchedTicks(Context);
		}
		else
		{
			ensureMsgf(!CVarAllowBatchedTicks.GetValueOnGameThread(), TEXT("Concurrent queuing is not compatible with batched ticks!"));

			for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
			{
				LevelList[LevelIndex]->StartFrameParallel(Context, AllTickFunctions);
			}
			INC_DWORD_STAT_BY(STAT_TicksQueued, AllTickFunctions.Num());
			CSV_CUSTOM_STAT(Basic, TicksQueued, AllTickFunctions.Num(), ECsvCustomStatOp::Accumulate);
			TickTaskSequencer.SetupAddTickTaskCompletionParallel(AllTickFunctions.Num());
			for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
			{
				LevelList[LevelIndex]->ReserveTickFunctionCooldowns(AllTickFunctions.Num());
			}
			ParallelFor(AllTickFunctions.Num(),
				[this](int32 Index)
				{
					FTickFunction* TickFunction = AllTickFunctions[Index];

					TArray<FTickFunction*, TInlineAllocator<8> > StackForCycleDetection;
					TickFunction->QueueTickFunctionParallel(Context, StackForCycleDetection);
				}
			);
			AllTickFunctions.Reset();

			for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
			{
				LevelList[LevelIndex]->DoDeferredRemoves();
			}
		}
	}

	/**
	 * Run all of the ticks for a pause frame synchronously on the game thread.
	 * The capability of pause ticks are very limited. There are no dependencies or ordering or tick groups.
	 * @param World	- World currently ticking
	 * @param DeltaSeconds - time in seconds since last tick
	 * @param TickType - type of tick (viewports only, time only, etc)
	 */
	virtual void RunPauseFrame(UWorld* InWorld, float InDeltaSeconds, ELevelTick InTickType, const TArray<ULevel*>& LevelsToTick) override
	{
		bTickNewlySpawned = true; // we don't support new spawns, but lets at least catch them.
		Context.TickGroup = ETickingGroup(0); // reset this to the start tick group
		Context.DeltaSeconds = InDeltaSeconds;
		Context.TickType = InTickType;
		Context.Thread = ENamedThreads::GameThread;
		Context.World = InWorld;
		FillLevelList(LevelsToTick);
		for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
		{
			LevelList[LevelIndex]->RunPauseFrame(Context);
		}
		Context.World = nullptr;
		bTickNewlySpawned = false;
		LevelList.Reset();
	}
	/**
		* Run a tick group, ticking all actors and components
		* @param Group - Ticking group to run
		* @param bBlockTillComplete - if true, do not return until all ticks are complete
	*/
	virtual void RunTickGroup(ETickingGroup Group, bool bBlockTillComplete ) override
	{
		check(Context.TickGroup == Group); // this should already be at the correct value, but we want to make sure things are happening in the right order
		check(bTickNewlySpawned); // we should be in the middle of ticking

		TArray<FTickFunction*> TicksToManualDispatch;
		FTaskSyncManager* SyncManager = FTaskSyncManager::Get();

		if (SyncManager)
		{
			SyncManager->StartTickGroup(Context.World, Group, TicksToManualDispatch);
		}

		TickTaskSequencer.ReleaseTickGroup(Group, bBlockTillComplete, TicksToManualDispatch);
		Context.TickGroup = ETickingGroup(Context.TickGroup + 1); // new actors go into the next tick group because this one is already gone
		if (bBlockTillComplete) // we don't deal with newly spawned ticks within the async tick group, they wait until after the async stuff
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_TickTask_RunTickGroup_BlockTillComplete);

			bool bFinished = false;
			for (int32 Iterations = 0;Iterations < 101; Iterations++)
			{
				int32 Num = 0;
				for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
				{
					Num += LevelList[LevelIndex]->QueueNewlySpawned(Context.TickGroup);
				}
				if (Num && Context.TickGroup == TG_NewlySpawned)
				{
					SCOPE_CYCLE_COUNTER(STAT_TG_NewlySpawned);
					TickTaskSequencer.ReleaseTickGroup(TG_NewlySpawned, true, TicksToManualDispatch);
				}
				else
				{
					bFinished = true;
					break;
				}
			}
			if (!bFinished)
			{
				// this is runaway recursive spawning.
				for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
				{
					LevelList[LevelIndex]->LogAndDiscardRunawayNewlySpawned(Context.TickGroup);
				}
			}
		}

		if (SyncManager)
		{
			SyncManager->EndTickGroup(Context.World, Group);
		}
	}

	/** Finish a frame of ticks **/
	virtual void EndFrame() override
	{
		TickTaskSequencer.EndFrame();
		bTickNewlySpawned = false;
		for( int32 LevelIndex = 0; LevelIndex < LevelList.Num(); LevelIndex++ )
		{
			LevelList[LevelIndex]->EndFrame();
		}

		FTaskSyncManager* SyncManager = FTaskSyncManager::Get();
		if (SyncManager)
		{
			SyncManager->EndFrame(Context.World);
		}

		Context.World = nullptr;
		LevelList.Reset();
	}

	// Interface that is private to FTickFunction

	/** Return true if this tick function is in the primary list **/
	bool HasTickFunction(ULevel* InLevel, FTickFunction* TickFunction)
	{
		FTickTaskLevel* Level = TickTaskLevelForLevel(InLevel, false);
		return Level && Level->HasTickFunction(TickFunction);
	}
	/** Add the tick function to the primary list **/
	void AddTickFunction(ULevel* InLevel, FTickFunction* TickFunction)
	{
		check(TickFunction->TickGroup >= 0 && TickFunction->TickGroup < TG_NewlySpawned); // You may not schedule a tick in the newly spawned group...they can only end up there if they are spawned late in a frame.
		FTickTaskLevel* Level = TickTaskLevelForLevel(InLevel);
		Level->AddTickFunction(TickFunction);
		TickFunction->InternalData->TickTaskLevel = Level;
	}
	/** Remove the tick function from the primary list **/
	void RemoveTickFunction(FTickFunction* TickFunction)
	{
		check(TickFunction->InternalData);
		FTickTaskLevel* Level = TickFunction->InternalData->TickTaskLevel;
		check(Level);
		Level->RemoveTickFunction(TickFunction);
	}

private:
	/** Default constructor **/
	FTickTaskManager()
		: TickTaskSequencer(FTickTaskSequencer::Get())
		, bTickNewlySpawned(false)
	{
		IConsoleManager::Get().RegisterConsoleCommand(TEXT("dumpticks"), TEXT("Dumps all tick functions registered with FTickTaskManager to log."));
		
		// Create the task sync manager if it is needed later
		FTaskSyncManager* CreatedSyncManager = FTaskSyncManager::Get();
	}

	/** Fill the level list, only with levels that can actually tick */
	void FillLevelList(const TArray<ULevel*>& Levels)
	{
		check(!LevelList.Num());
		for( int32 LevelIndex = 0; LevelIndex < Levels.Num(); LevelIndex++ )
		{
			ULevel* Level = Levels[LevelIndex];
			if (Level && Level->bIsVisible && Level->TickTaskLevel)
			{
				LevelList.Add(Level->TickTaskLevel);
			}
		}
	}

	/** Find the tick level for this actor **/
	FTickTaskLevel* TickTaskLevelForLevel(ULevel* Level, bool bCreateIfNeeded = true)
	{
		check(Level);

		if (bCreateIfNeeded && Level->TickTaskLevel == nullptr)
		{
			Level->TickTaskLevel = AllocateTickTaskLevel();
		}

		check(Level->TickTaskLevel);
		return Level->TickTaskLevel;
	}



	/** Dumps all tick functions to output device */
	virtual void DumpAllTickFunctions(FOutputDevice& Ar, UWorld* InWorld, bool bEnabled, bool bDisabled, bool bGrouped) override
	{
		int32 EnabledCount = 0, DisabledCount = 0;

		Ar.Logf(TEXT(""));
		Ar.Logf(TEXT("============================ Tick Functions (%s) ============================"), bGrouped ? TEXT("GROUPED") : ((bEnabled && bDisabled) ? TEXT("All") : (bEnabled ? TEXT("Enabled") : TEXT("Disabled"))));

		check(InWorld);

		if (bGrouped)
		{
			TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess> TickContextToCountMap;
			GetEnabledTickFunctionCounts(InWorld, TickContextToCountMap, EnabledCount, true);

			// Build sorted array of tick context by count
			struct FSortedTickContextGroup
			{
				FName Context;
				int32 Count;
			};

			TArray<FSortedTickContextGroup> SortedTickContexts;
			SortedTickContexts.AddZeroed(TickContextToCountMap.Num());
			int32 TickNum = 0;
			for (auto It = TickContextToCountMap.CreateConstIterator(); It; ++It)
			{
				SortedTickContexts[TickNum].Context = It->Key;
				SortedTickContexts[TickNum].Count = It->Value;
				TickNum++;
			}

			SortedTickContexts.Sort([](const FSortedTickContextGroup& A, const FSortedTickContextGroup& B) { return A.Count > B.Count; });

			// Now print it
			for (int32 TickIdx = 0; TickIdx < SortedTickContexts.Num(); TickIdx++)
			{
				Ar.Logf(TEXT("%s, %d"), *SortedTickContexts[TickIdx].Context.ToString(), SortedTickContexts[TickIdx].Count);
			}

			Ar.Logf(TEXT(""));
			Ar.Logf(TEXT("Total enabled tick functions: %d."), EnabledCount);
			Ar.Logf(TEXT(""));
		}
		else
		{
			for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
			{
				ULevel* Level = InWorld->GetLevel(LevelIndex);
				if (Level->bIsVisible && Level->TickTaskLevel)
				{
					Level->TickTaskLevel->DumpAllTickFunctions(Ar, EnabledCount, DisabledCount, bEnabled, bDisabled);
				}
			}

			Ar.Logf(TEXT(""));
			Ar.Logf(TEXT("Total registered tick functions: %d, enabled: %d, disabled: %d."), EnabledCount + DisabledCount, EnabledCount, DisabledCount);
			Ar.Logf(TEXT(""));
		}
	}


	virtual void GetEnabledTickFunctionCounts(UWorld* InWorld, TSortedMap<FName, int32, FDefaultAllocator, FNameFastLess>& TickContextToCountMap, int32& EnabledCount, bool bDetailed, bool bFilterCoolingDown=false)
	{
		check(InWorld);

		if (bFilterCoolingDown && InWorld->TickGroup >= 0 && InWorld->TickGroup < TG_NewlySpawned)
		{
			UE_LOGF(LogTick, Warning, "GetEnabledTickFunctionCounts is filtering results before TickGroups have finished. TickFunctions with a cooldown interval may be missing.");
		}

		const float WorldTimeSeconds = InWorld->GetTimeSeconds();
		const float WorldUnpausedTimeSeconds = InWorld->GetUnpausedTimeSeconds();


		for (int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = InWorld->GetLevel(LevelIndex);
			if (Level->bIsVisible && Level->TickTaskLevel)
			{
				Level->TickTaskLevel->AddTickFunctionsToMap(TickContextToCountMap, EnabledCount, bDetailed, bFilterCoolingDown, WorldTimeSeconds, WorldUnpausedTimeSeconds);
			}
		}
	}

	/** Global Sequencer														*/
	FTickTaskSequencer&							TickTaskSequencer;
	/** List of current levels **/
	TArray<FTickTaskLevel*>						LevelList;
	/** tick context **/
	FTickContext								Context;
	/** true during the tick phase, when true, tick function adds also go to the newly spawned list. **/
	bool										bTickNewlySpawned;

	/** true during the tick phase, when true, tick function adds also go to the newly spawned list. **/
	TArray<FTickFunction*> AllTickFunctions;
};


/** Default constructor, intitalizes to reasonable defaults **/
FTickFunction::FTickFunction()
	: TickGroup(TG_PrePhysics)
	, EndTickGroup(TG_PrePhysics)
	, bTickEvenWhenPaused(false)
	, bCanEverTick(false)
	, bStartWithTickEnabled(false)
	, bAllowTickOnDedicatedServer(true)
	, bAllowTickBatching(false)
	, bHighPriority(false)
	, bRunOnAnyThread(false)
	, bRunTransactionally(false)
	, bDispatchManually(false)
	, bWasDispatchedManually(false)
	, TickState(ETickState::Enabled)
	, TickInterval(0.f)
{
}

FTickFunction::FInternalData::FInternalData()
	: bRegistered(false)
	, bWasInterval(false)
	, TaskState(ETickTaskState::NotQueued)
	, ActualStartTickGroup(TG_PrePhysics)
	, ActualEndTickGroup(TG_PrePhysics)
	, TickVisitedGFrameCounter(0)
	, TickQueuedGFrameCounter(0)
	, TaskPointer(nullptr)
	, Next(nullptr)
	, RelativeTickCooldown(0.f)
	, LastIntervalTickSeconds(-1.0)
	, TickTaskLevel(nullptr)
{
}

/** Destructor, unregisters the tick function **/
FTickFunction::~FTickFunction()
{
	UnRegisterTickFunction();
}


/**
* Adds the tick function to the primary list of tick functions.
* @param Level - level to place this tick function in
**/
void FTickFunction::RegisterTickFunction(ULevel* Level)
{
	if (!IsTickFunctionRegistered())
	{
		// Only allow registration of tick if we are are allowed on dedicated server, or we are not a dedicated server
		const UWorld* World = Level ? Level->GetWorld() : nullptr;
		if(bAllowTickOnDedicatedServer || !(World && World->IsNetMode(NM_DedicatedServer)))
		{
			if (InternalData == nullptr)
			{
				InternalData.Reset(new FInternalData());
			}
			FTickTaskManager::Get().AddTickFunction(Level, this);
			InternalData->bRegistered = true;
		}
	}
	else
	{
		check(FTickTaskManager::Get().HasTickFunction(Level, this));
	}
}

/** Removes the tick function from the primary list of tick functions. **/
void FTickFunction::UnRegisterTickFunction()
{
	if (IsTickFunctionRegistered())
	{
		FTickTaskManager::Get().RemoveTickFunction(this);
		InternalData->bRegistered = false;
	}
}

/** Enables or disables this tick function. **/
void FTickFunction::SetTickFunctionEnable(bool bInEnabled)
{
	if (IsTickFunctionRegistered())
	{
		if (bInEnabled == (TickState == ETickState::Disabled))
		{
			FTickTaskLevel* TickTaskLevel = InternalData->TickTaskLevel;
			check(TickTaskLevel);
			TickTaskLevel->RemoveTickFunction(this);
			TickState = (bInEnabled ? ETickState::Enabled : ETickState::Disabled);
			TickTaskLevel->AddTickFunction(this);
		}

		if (TickState == ETickState::Disabled)
		{
			InternalData->LastIntervalTickSeconds = -1.0;
		}
	}
	else
	{
		TickState = (bInEnabled ? ETickState::Enabled : ETickState::Disabled);
	}
}

void FTickFunction::UpdateTickIntervalAndCoolDown(float NewTickInterval)
{
	TickInterval = NewTickInterval;
	if(IsTickFunctionRegistered() && TickState != ETickState::Disabled && InternalData->bWasInterval)
	{
		FTickTaskLevel* TickTaskLevel = InternalData->TickTaskLevel;
		check(TickTaskLevel);

		// Try to update the interval from the reschedule list
		if (!TickTaskLevel->UpdateRescheduleInterval(this, TickInterval))
		{
			// If is was not in the reschedule list means it needs to be removed from the cooldown list and rescheduled.
			TickTaskLevel->RemoveTickFunction(this);
			TickTaskLevel->RescheduleForInterval(this, TickInterval);
		}
	}
}

void FTickFunction::AddPrerequisite(UObject* TargetObject, struct FTickFunction& TargetTickFunction)
{
	const bool bThisCanTick = (bCanEverTick || IsTickFunctionRegistered());
	const bool bTargetCanTick = (TargetTickFunction.bCanEverTick || TargetTickFunction.IsTickFunctionRegistered());

	if (bThisCanTick && bTargetCanTick)
	{
		Prerequisites.AddUnique(FTickPrerequisite(TargetObject, TargetTickFunction));
	}
}

void FTickFunction::RemovePrerequisite(UObject* TargetObject, struct FTickFunction& TargetTickFunction)
{
	Prerequisites.RemoveSwap(FTickPrerequisite(TargetObject, TargetTickFunction));
}

void FTickFunction::SetPriorityIncludingPrerequisites(bool bInHighPriority)
{
	if (bHighPriority != bInHighPriority)
	{
		bHighPriority = bInHighPriority;
		for (auto& Prereq : Prerequisites)
		{
			if (Prereq.PrerequisiteObject.Get() && Prereq.PrerequisiteTickFunction && Prereq.PrerequisiteTickFunction->bHighPriority != bInHighPriority)
			{
				Prereq.PrerequisiteTickFunction->SetPriorityIncludingPrerequisites(bInHighPriority);
			}
		}
	}
}

void FTickFunction::LogTickFunction(ENamedThreads::Type CurrentThread, bool bLogPrerequisites, int32 Indent)
{
	UE_LOGF(LogTick, Log, "%lstick %ls [%1d, %1d] %6llu %2d %ls", FCString::Spc(Indent * 2), bHighPriority ? TEXT("*") : TEXT(" "), (int32)GetActualTickGroup(), (int32)GetActualEndTickGroup(), (uint64)GFrameCounter, (int32)CurrentThread, *DiagnosticMessage());
	if (bLogPrerequisites)
	{
		ShowPrerequistes();
	}
	
	// Handle nested ticks
	ForEachNestedTick([CurrentThread, bLogPrerequisites, Indent](FTickFunction& NestedTick)
	{
		NestedTick.LogTickFunction(CurrentThread, bLogPrerequisites, Indent + 1);
	});
}

void FTickFunction::ShowPrerequistes(int32 Indent)
{
	for (auto& Prereq : Prerequisites)
	{
		if (Prereq.PrerequisiteTickFunction)
		{
			UE_LOGF(LogTick, Log, "%ls prereq %ls", FCString::Spc(Indent * 2), *Prereq.PrerequisiteTickFunction->DiagnosticMessage());
			Prereq.PrerequisiteTickFunction->ShowPrerequistes(Indent + 1);
		}
	}
}

bool FTickFunction::IsCompletionHandleValid() const
{
	return InternalData && (InternalData->TaskState == ETickTaskState::HasTask || InternalData->TaskState == ETickTaskState::HasCompletionEvent);
}

FGraphEventRef FTickFunction::GetCompletionHandle() const
{
	check(InternalData);
	if (InternalData->TaskState == ETickTaskState::HasCompletionEvent)
	{
		// This will increment the reference count on the event
		return *((FGraphEventRef*)InternalData->TaskPointer);
	}

	FTickGraphTask* Task = FTickTaskSequencer::GetGraphTask(this);
	check(Task);
	return Task->GetCompletionEvent();
}

bool FTickFunction::CanDispatchManually() const
{
	if (bDispatchManually && !bWasDispatchedManually && InternalData && InternalData->TaskState == ETickTaskState::HasTask)
	{
		return true;
	}
	return false;
}

bool FTickFunction::DispatchManually()
{
	if (ensure(bDispatchManually) && CanDispatchManually())
	{
		FTickGraphTask* Task = FTickTaskSequencer::GetGraphTask(this);
		check(Task);

		bWasDispatchedManually = true;
				
		// This could run the tick function and invalidate anything
		Task->Unlock();

		return true;
	}

	return false;
}

void FTickFunction::ExecuteNestedTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	// Not valid to call on a real scheduled function
	check(GetTaskPointer(ETickTaskState::HasTask) == nullptr);

	// This does not increment the reference counter
	if (MyCompletionGraphEvent.IsValid())
	{
		SetTaskPointer(ETickTaskState::HasCompletionEvent, (void*)&MyCompletionGraphEvent);
	}
	ExecuteTick(DeltaTime, TickType, CurrentThread, MyCompletionGraphEvent);
	ClearTaskInformation();
}

void FTickFunction::ClearTaskInformation()
{
	SetTaskPointer(ETickTaskState::NotQueued, nullptr);
}

void FTickFunction::SetTaskPointer(ETickTaskState NewState, void* InTaskPointer)
{
	// If this is called on a tick function that was never registered, we don't want to allocate memory or crash
	if (InternalData)
	{
		if (NewState == ETickTaskState::NotQueued)
		{
			InternalData->TaskState = ETickTaskState::NotQueued;
			InternalData->TaskPointer = nullptr;
		}
		else
		{
			InternalData->TaskState = NewState;
			InternalData->TaskPointer = InTaskPointer;
		}
	}
}

void FTickFunction::QueueTickFunction(FTickTaskSequencer& TTS, const struct FTickContext& TickContext)
{
	// Only compare the 32bit part of the frame counter
	uint32 CurrentFrameCounter = (uint32)GFrameCounter;

	checkSlow(TickContext.Thread == ENamedThreads::GameThread); // we assume same thread here
	check(IsTickFunctionRegistered() && !TTS.HasBeenVisited(this, CurrentFrameCounter));

	// Mark visited at start of function
	InternalData->TickVisitedGFrameCounter.store(CurrentFrameCounter, std::memory_order_relaxed);
	if (TickState != FTickFunction::ETickState::Disabled)
	{
		ETickingGroup MaxStartTickGroup = ETickingGroup(0);
		ETickingGroup MaxEndTickGroup = ETickingGroup(0);

		TArray<FTickFunction*, TInlineAllocator<2>> RawPrerequisites;
		for (int32 PrereqIndex = 0; PrereqIndex < Prerequisites.Num(); PrereqIndex++)
		{
			FTickFunction* Prereq = Prerequisites[PrereqIndex].PrerequisiteTickFunction;
			if (!Prerequisites[PrereqIndex].PrerequisiteObject.IsValid(true))
			{
				// stale prereq, delete it
				Prerequisites.RemoveAtSwap(PrereqIndex--);
			}
			else if (Prereq->IsTickFunctionRegistered())
			{
				// If the prerequisite object is valid Prereq can't be null, and if it is registered InternalData can't be null either
				if (!TTS.HasBeenVisited(Prereq, CurrentFrameCounter))
				{
					// If the prerequisite hasn't been visited, queue it now
					Prereq->QueueTickFunction(TTS, TickContext);
				}

				if (Prereq->InternalData->TickQueuedGFrameCounter.load(std::memory_order_relaxed) != CurrentFrameCounter)
				{
					// This can only happen if the prerequisite is is partially queued in the current stack
					UE_LOGF(LogTick, Warning, "While processing prerequisites for %ls, could not use %ls because it would form a cycle.", *DiagnosticMessage(), *Prereq->DiagnosticMessage());
				}
				else if (Prereq->InternalData->TaskState == ETickTaskState::NotQueued)
				{
					// Ignore disabled dependencies, this means that intermediate scene components will break the automatic dependency setting
				}
				else if (TTS.ShouldConsiderPrerequisite(this, Prereq))
				{
					MaxStartTickGroup = FMath::Max<ETickingGroup>(MaxStartTickGroup, Prereq->InternalData->ActualStartTickGroup.GetValue());
					MaxEndTickGroup = FMath::Max<ETickingGroup>(MaxEndTickGroup, Prereq->InternalData->ActualEndTickGroup.GetValue());
					RawPrerequisites.Add(Prereq);
				}
			}
		}

		// tick group is the max of the prerequisites, the current tick group, and the desired tick group
		ETickingGroup MyActualTickGroup = FMath::Max<ETickingGroup>(MaxStartTickGroup, FMath::Max<ETickingGroup>(TickGroup.GetValue(), TickContext.TickGroup));
		if (MyActualTickGroup != TickGroup)
		{
			// if the tick was "demoted", make sure it ends up in an ordinary tick group.
			while (!CanDemoteIntoTickGroup(MyActualTickGroup))
			{
				MyActualTickGroup = ETickingGroup(MyActualTickGroup + 1);
			}
		}
		InternalData->ActualStartTickGroup = MyActualTickGroup;
		InternalData->ActualEndTickGroup = MyActualTickGroup;

		// Also check to see if the end tick group needs to be extended separately
		ETickingGroup MyActualEndTickGroup = FMath::Max<ETickingGroup>(MaxEndTickGroup, FMath::Max<ETickingGroup>(EndTickGroup.GetValue(), MyActualTickGroup));

		if (MyActualEndTickGroup > MyActualTickGroup)
		{
			check(MyActualEndTickGroup <= TG_NewlySpawned);
			ETickingGroup TestTickGroup = ETickingGroup(MyActualTickGroup + 1);
			while (TestTickGroup <= MyActualEndTickGroup)
			{
				if (CanDemoteIntoTickGroup(TestTickGroup))
				{
					InternalData->ActualEndTickGroup = TestTickGroup;
				}
				TestTickGroup = ETickingGroup(TestTickGroup + 1);
			}
		}

		if (TickState == FTickFunction::ETickState::Enabled)
		{
			TTS.QueueOrBatchTickTask(RawPrerequisites, this, TickContext);
		}
	}

	// Mark as queued (finished visiting), even if it was not turned into a real task
	InternalData->TickQueuedGFrameCounter.store(CurrentFrameCounter, std::memory_order_relaxed);
}

void FTickFunction::QueueTickFunctionParallel(const struct FTickContext& TickContext, TArray<FTickFunction*, TInlineAllocator<8> >& StackForCycleDetection)
{
	// Only compare the 32bit part of the frame counter
	uint32 CurrentFrameCounter = (uint32)GFrameCounter;
	uint32 OldValue = InternalData->TickVisitedGFrameCounter.load(std::memory_order_relaxed);
	if (OldValue != CurrentFrameCounter)
	{
		// Modify the visited frame if another thread has not already done so
		InternalData->TickVisitedGFrameCounter.compare_exchange_strong(OldValue, CurrentFrameCounter);
	}

	if (OldValue != CurrentFrameCounter)
	{
		check(IsTickFunctionRegistered());
		if (TickState != FTickFunction::ETickState::Disabled)
		{
			ETickingGroup MaxStartTickGroup = ETickingGroup(0);
			ETickingGroup MaxEndTickGroup = ETickingGroup(0);

			FGraphEventArray TaskPrerequisites;
			if (Prerequisites.Num())
			{
				StackForCycleDetection.Push(this);
				for (int32 PrereqIndex = 0; PrereqIndex < Prerequisites.Num(); PrereqIndex++)
				{
					FTickFunction* Prereq = Prerequisites[PrereqIndex].Get();

					if (!Prereq)
					{
						// stale prereq, delete it
						Prerequisites.RemoveAtSwap(PrereqIndex--);
					}
                    else if (StackForCycleDetection.Contains(Prereq))
                    {
                        UE_LOGF(LogTick, Warning, "While processing prerequisites for %ls, could use %ls because it would form a cycle.", *DiagnosticMessage(), *Prereq->DiagnosticMessage());
                    }
					else if (Prereq->IsTickFunctionRegistered())
					{
						// recursive call to make sure my prerequisite is set up so I can use its completion handle
						Prereq->QueueTickFunctionParallel(TickContext, StackForCycleDetection);
						if (!Prereq->IsCompletionHandleValid())
						{
							//ok UE_LOGF(LogTick, Warning, "While processing prerequisites for %ls, could use %ls because it is disabled.",*DiagnosticMessage(), *Prereq->DiagnosticMessage());
						}
						else
						{
							MaxStartTickGroup = FMath::Max<ETickingGroup>(MaxStartTickGroup, Prereq->InternalData->ActualStartTickGroup.GetValue());
							MaxEndTickGroup = FMath::Max<ETickingGroup>(MaxEndTickGroup, Prereq->InternalData->ActualEndTickGroup.GetValue());
							TaskPrerequisites.Add(Prereq->GetCompletionHandle());
						}
					}
				}
				StackForCycleDetection.Pop();
			}

			// tick group is the max of the prerequisites, the current tick group, and the desired tick group
			ETickingGroup MyActualTickGroup =  FMath::Max<ETickingGroup>(MaxStartTickGroup, FMath::Max<ETickingGroup>(TickGroup,TickContext.TickGroup));
			if (MyActualTickGroup != TickGroup)
			{
				// if the tick was "demoted", make sure it ends up in an ordinary tick group.
				while (!CanDemoteIntoTickGroup(MyActualTickGroup))
				{
					MyActualTickGroup = ETickingGroup(MyActualTickGroup + 1);
				}
			}
			InternalData->ActualStartTickGroup = MyActualTickGroup;
			InternalData->ActualEndTickGroup = MyActualTickGroup;
		
			// Also check to see if the end tick group needs to be extended separately
			ETickingGroup MyActualEndTickGroup = FMath::Max<ETickingGroup>(MaxEndTickGroup, FMath::Max<ETickingGroup>(EndTickGroup.GetValue(), MyActualTickGroup));

			if (MyActualEndTickGroup > MyActualTickGroup)
			{
				check(MyActualEndTickGroup <= TG_NewlySpawned);
				ETickingGroup TestTickGroup = ETickingGroup(MyActualTickGroup + 1);
				while (TestTickGroup <= MyActualEndTickGroup)
				{
					if (CanDemoteIntoTickGroup(TestTickGroup))
					{
						InternalData->ActualEndTickGroup = TestTickGroup;
					}
					TestTickGroup = ETickingGroup(TestTickGroup + 1);
				}
			}

			if (TickState == FTickFunction::ETickState::Enabled)
			{
				FTickTaskSequencer::Get().QueueTickTaskParallel(&TaskPrerequisites, this, TickContext);
				if (!InternalData->bWasInterval && TickInterval > 0.f)
				{
					InternalData->TickTaskLevel->RescheduleForIntervalParallel(this);
				}
			}
		}
		
		InternalData->TickQueuedGFrameCounter = CurrentFrameCounter;
	}
	else
	{
		// if we are not going to process it, we need to at least wait until the other thread finishes it
		std::atomic<uint32>& TickQueuedGFrameCounter = InternalData->TickQueuedGFrameCounter;
		if (TickQueuedGFrameCounter != CurrentFrameCounter)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FTickFunction_QueueTickFunctionParallel_Spin);
			while (TickQueuedGFrameCounter != CurrentFrameCounter)
			{
				FPlatformProcess::YieldThread();
			}
		}
	}
}

float FTickFunction::CalculateDeltaTime(float DeltaTime, const UWorld* TickingWorld)
{
	if (!InternalData->bWasInterval)
	{
		// No tick interval. Return the world delta seconds, and make sure to mark that
		// we're not tracking last-tick-time for this object.
		InternalData->LastIntervalTickSeconds = -1.0;
	}
	else
	{
		// We've got a tick interval. Mark last-tick-time. If we already had last-tick-time, return
		// the time since then; otherwise, return the world delta seconds.
		const double CurrentWorldTime = (bTickEvenWhenPaused ? TickingWorld->GetUnpausedTimeSeconds() : TickingWorld->GetTimeSeconds());
		if (InternalData->LastIntervalTickSeconds >= 0.0)
		{
			DeltaTime = CurrentWorldTime - InternalData->LastIntervalTickSeconds;
		}
		InternalData->LastIntervalTickSeconds = CurrentWorldTime;
	}

	return DeltaTime;
}

/**
 * Singleton to retrieve the global tick task manager
 * @return Reference to the global tick task manager
**/
FTickTaskManagerInterface& FTickTaskManagerInterface::Get()
{
	return FTickTaskManager::Get();
}


struct FTestTickFunction : public FTickFunction
{
	FTestTickFunction()
	{
		TickGroup = TG_PrePhysics;
		bTickEvenWhenPaused = true;
	}
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TestStatOverhead_FTestTickFunction);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TestStatOverhead_FTestTickFunction_Inner);
	}
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph **/
	virtual FString DiagnosticMessage() override
	{
		return FString("test");
	}

	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override
	{
		return FName("test");
	}
};

template<>
struct TStructOpsTypeTraits<FTestTickFunction> : public TStructOpsTypeTraitsBase2<FTestTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

static const int32 NumTestTickFunctions = 10000;
static TArray<FTestTickFunction> TestTickFunctions;
static TArray<FTestTickFunction*> IndirectTestTickFunctions;

static void RemoveTestTickFunctions(const TArray<FString>& Args)
{
	if (TestTickFunctions.Num() || IndirectTestTickFunctions.Num())
	{
		UE_LOGF(LogConsoleResponse, Display, "Removing Test Tick Functions.");
		TestTickFunctions.Empty(NumTestTickFunctions);
		for (int32 Index = 0; Index < IndirectTestTickFunctions.Num(); Index++)
		{
			delete IndirectTestTickFunctions[Index];
		}
		IndirectTestTickFunctions.Empty(NumTestTickFunctions);
	}
}

static void AddTestTickFunctions(const TArray<FString>& Args, UWorld* InWorld)
{
	RemoveTestTickFunctions(Args);
	ULevel* Level = InWorld->GetCurrentLevel();
	UE_LOGF(LogConsoleResponse, Display, "Adding 1000 ticks in a cache coherent fashion.");


	TestTickFunctions.Reserve(NumTestTickFunctions);
	for (int32 Index = 0; Index < NumTestTickFunctions; Index++)
	{
		TestTickFunctions.AddDefaulted_GetRef().RegisterTickFunction(Level);
	}
}

static void AddIndirectTestTickFunctions(const TArray<FString>& Args, UWorld* InWorld)
{
	RemoveTestTickFunctions(Args);
	ULevel* Level = InWorld->GetCurrentLevel();
	UE_LOGF(LogConsoleResponse, Display, "Adding 1000 ticks in a cache coherent fashion.");
	TArray<FTestTickFunction*> Junk;
	for (int32 Index = 0; Index < NumTestTickFunctions; Index++)
	{
		for (int32 JunkIndex = 0; JunkIndex < 8; JunkIndex++)
		{
			Junk.Add(new FTestTickFunction); // don't give the allocator an easy ride
		}
		FTestTickFunction* NewTick = new FTestTickFunction;
		NewTick->RegisterTickFunction(Level);
		IndirectTestTickFunctions.Add(NewTick);
	}
	for (int32 JunkIndex = 0; JunkIndex < 8; JunkIndex++)
	{
		delete Junk[JunkIndex];
	}
}

static FAutoConsoleCommand RemoveTestTickFunctionsCmd(
	TEXT("tick.RemoveTestTickFunctions"),
	TEXT("Remove no-op ticks to test performance of ticking infrastructure."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&RemoveTestTickFunctions)
	);

static FAutoConsoleCommandWithWorldAndArgs AddTestTickFunctionsCmd(
	TEXT("tick.AddTestTickFunctions"),
	TEXT("Add no-op ticks to test performance of ticking infrastructure."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&AddTestTickFunctions)
	);

static FAutoConsoleCommandWithWorldAndArgs AddIndirectTestTickFunctionsCmd(
	TEXT("tick.AddIndirectTestTickFunctions"),
	TEXT("Add no-op ticks to test performance of ticking infrastructure."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&AddIndirectTestTickFunctions)
	);


