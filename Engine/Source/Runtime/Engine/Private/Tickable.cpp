// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tickable.h"
#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_CYCLE_STAT(TEXT("TickableGameObjects Time"), STAT_TickableGameObjectsTime, STATGROUP_Game);

void FTickableObjectBase::FTickableStatics::QueueTickableObjectForAdd(FTickableObjectBase* InTickable, ETickableTickType NewTickType)
{
	// Objects set to never tick at creation will not be registered
	check(NewTickType != ETickableTickType::Never);

	// This only needs to lock the new object queue
	UE::TScopeLock NewTickableObjectsLock(NewTickableObjectsCritical);
	NewTickableObjects.Add(InTickable, NewTickType);
}

void FTickableObjectBase::FTickableStatics::SetTickTypeForTickableObject(FTickableObjectBase* InTickable, ETickableTickType NewTickType)
{
	UE::TScopeLock TickableObjectsLock(TickableObjectsCritical);
	UE::TScopeLock NewTickableObjectsLock(NewTickableObjectsCritical);
	
	const int32 Pos = TickableObjects.IndexOfByKey(InTickable);
	if (NewTickType == ETickableTickType::Never)
	{
		// Remove from pending list if it hasn't been registered yet
		NewTickableObjects.Remove(InTickable);

		// The item may be missing depending on destruction order during shutdown
		if (Pos != INDEX_NONE)
		{
			if (bIsTickingObjects)
			{
				// During ticking it is not safe to modify the array so null and mark for later
				TickableObjects[Pos].TickableObject = nullptr;
				bNeedsCleanup = true;
			}
			else
			{
				TickableObjects.RemoveAt(Pos);
			}
		}
	}
	else
	{
		if (Pos != INDEX_NONE)
		{
			// If this is registered, it was removed from the new list in BeginTicking
			check(!NewTickableObjects.Contains(InTickable));

			// Existing entries should never be set to new object
			if (ensure(NewTickType != ETickableTickType::NewObject))
			{
				// This will modify behavior for the current frame if it has not ticked yet
				TickableObjects[Pos].TickType = NewTickType;
			}
		}
		else
		{
			// Add to the pending list (which could override previous request), this will apply it next frame
			NewTickableObjects.Add(InTickable, NewTickType);
		}
	}
}

void FTickableObjectBase::FTickableStatics::StartTicking()
{
	check(!bIsTickingObjects);

	UE::TScopeLock NewTickableObjectsLock(NewTickableObjectsCritical);

	for (TPair<FTickableObjectBase*, ETickableTickType> Pair : NewTickableObjects)
	{
		FTickableObjectBase* NewTickableObject = Pair.Key;
		ETickableTickType TickType = Pair.Value;

		// SetTickTypeForTickableObject will not add to new list if it already exists in TickableObjects
		check(!TickableObjects.Contains(NewTickableObject));

		if (TickType == ETickableTickType::NewObject)
		{
			// Query object if unknown
			TickType = NewTickableObject->GetTickableTickType();
		}

		if (TickType != ETickableTickType::Never)
		{
			TickableObjects.Add({ NewTickableObject, TickType });
		}
	}
	NewTickableObjects.Empty();

	bIsTickingObjects = true;
}

void FTickableObjectBase::FTickableStatics::FinishTicking()
{
	check(bIsTickingObjects);
	if (bNeedsCleanup)
	{
		TickableObjects.RemoveAll([](const FTickableObjectEntry& Entry) { return Entry.TickableObject == nullptr; });
		bNeedsCleanup = false;
	}

	bIsTickingObjects = false;
}


void FTickableObjectBase::SimpleTickObjects(FTickableStatics& Statics, TFunctionRef<void(FTickableObjectBase*)> TickFunc)
{
	UE::TScopeLock LockTickableObjects(Statics.TickableObjectsCritical);

	Statics.StartTicking();

	for (const FTickableObjectEntry& TickableEntry : Statics.TickableObjects)
	{
		if (FTickableObjectBase* TickableObject = TickableEntry.TickableObject)
		{
			// NOTE: This deliberately does not call IsAllowedToTick as it is deprecated and was not called in code this is replacing
			if ((TickableEntry.TickType == ETickableTickType::Always) || TickableObject->IsTickable())
			{
				TickFunc(TickableObject);
			}
		}
	}

	Statics.FinishTicking();
}


// FTickableGameObject implementation
FTickableGameObject::FTickableGameObject(ETickableTickType StartingTickType)
{
	if (StartingTickType != ETickableTickType::Never)
	{
		// It is only safe to create tickable game objects on the game thread, as otherwise there is a race condition between object initialize and the game thread tick
		// If you hit this ensure, change the constructor to use FTickableGameObject(ETickableTickType::Never) and call SetTickableTickType after initialization
		ensure(IsInGameThread());

		// Queue for creation, this can get called very early in startup
		FTickableStatics& Statics = GetStatics();
		Statics.QueueTickableObjectForAdd(this, StartingTickType);
	}
}

FTickableGameObject::~FTickableGameObject()
{
	// This won't do anything if it was already set to never tick
	// Ignore on engine exit as the singleton may be in the middle of destruction
	if (!IsEngineExitRequested())
	{
		SetTickableTickType(ETickableTickType::Never);
	}
}

void FTickableGameObject::SetTickableTickType(ETickableTickType NewTickType)
{
	FTickableStatics& Statics = GetStatics();
	Statics.SetTickTypeForTickableObject(this, NewTickType);
}

FTickableObjectBase::FTickableStatics& FTickableGameObject::GetStatics()
{
	static FTickableStatics Singleton;
	return Singleton;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FTickableGameObject::TickObjects(UWorld* World, const ELevelTick LevelTickType, const bool bIsPaused, const float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_TickableGameObjectsTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Tickables);

	FTickableStatics& Statics = GetStatics();

	check(IsInGameThread());

	// Don't tick game objects after exit because too many things could be destroyed
	if (!IsEngineExitRequested())
	{
		// It's a long lock but it's ok, the only thing we can block here is the GC worker thread that destroys UObjects
		UE::TScopeLock LockTickableObjects(Statics.TickableObjectsCritical);

		Statics.StartTicking();

		for (const FTickableObjectEntry& TickableEntry : Statics.TickableObjects)
		{
			if (FTickableGameObject* TickableObject = static_cast<FTickableGameObject*>(TickableEntry.TickableObject))
			{
				// If it is tickable and in this world
				if (TickableObject->IsAllowedToTick()
					&& ((TickableEntry.TickType == ETickableTickType::Always) || TickableObject->IsTickable())
					&& (TickableObject->GetTickableGameObjectWorld() == World))
				{
					// If tick type is All because at least one game world ticked, this will treat the null world as a game world
					const bool bIsGameWorld = LevelTickType == LEVELTICK_All || (World && World->IsGameWorld());

					// If we are in editor and it is editor tickable, always tick
					// If this is a game world then tick if we are not doing a time only (paused) update and we are not paused or the object is tickable when paused
					if ((GIsEditor && TickableObject->IsTickableInEditor()) ||
						(bIsGameWorld && ((!bIsPaused && LevelTickType != LEVELTICK_TimeOnly) || (bIsPaused && TickableObject->IsTickableWhenPaused()))))
					{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
						UE::RemoteExecutor::EnqueueWork(TickableObject->GetStatId().GetName(), true, [TickableObject, DeltaSeconds]() {
#endif
						
						SCOPE_CYCLE_COUNTER_STATID(TickableObject->GetStatId());
						TickableObject->Tick(DeltaSeconds);
						
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
						});
#endif
					}
				}
			}
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
		UE::RemoteExecutor::ExecutePendingWork();
#endif

		Statics.FinishTicking();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS