// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TileRendering.cpp: Tile rendering implementation.
=============================================================================*/

#include "CanvasRender.h"
#include "CanvasRendererItem.h"
#include "PrimitiveUniformShaderParameters.h"
#include "RHIStaticStates.h"
#include "EngineModule.h"
#include "MeshPassProcessor.h"
#include "RenderGraphUtils.h"
#include "RenderUtils.h"
#include "SceneView.h"
#include "UnrealClient.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(CanvasDrawTiles, TEXT("CanvasDrawTiles"));

static const uint32 CanvasTileVertexCount = 4;
static const uint32 CanvasTileIndexCount = 6;

FCanvasTileRendererItem::FTileVertexFactory::FTileVertexFactory(
	const FStaticMeshVertexBuffers* InVertexBuffers,
	ERHIFeatureLevel::Type InFeatureLevel)
	: FLocalVertexFactory(InFeatureLevel, "FTileVertexFactory")
	, VertexBuffers(InVertexBuffers)
{}

void FCanvasTileRendererItem::FTileVertexFactory::InitResource(FRHICommandListBase& RHICmdList)
{
	FLocalVertexFactory::FDataType VertexData;
	VertexBuffers->PositionVertexBuffer.BindPositionVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindTangentVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindLightMapVertexBuffer(this, VertexData, 0);
	VertexBuffers->ColorVertexBuffer.BindColorVertexBuffer(this, VertexData);
	SetData(RHICmdList, VertexData);

	FLocalVertexFactory::InitResource(RHICmdList);
}

FMeshBatch* FCanvasTileRendererItem::FRenderData::AllocTileMeshBatch(FCanvasRenderContext& InRenderContext, FHitProxyId InHitProxyId)
{
	FMeshBatch* MeshBatch = InRenderContext.Alloc<FMeshBatch>();

	MeshBatch->VertexFactory = &VertexFactory;
	MeshBatch->MaterialRenderProxy = MaterialRenderProxy;
	MeshBatch->ReverseCulling = false;
	MeshBatch->Type = PT_TriangleList;
	MeshBatch->DepthPriorityGroup = SDPG_Foreground;
	MeshBatch->BatchHitProxyId = InHitProxyId;

	FMeshBatchElement& MeshBatchElement = MeshBatch->Elements[0];
	MeshBatchElement.IndexBuffer = &IndexBuffer;
	MeshBatchElement.FirstIndex = 0;
	MeshBatchElement.NumPrimitives = 0;
	MeshBatchElement.MinVertexIndex = 0;
	MeshBatchElement.MaxVertexIndex = GetNumVertices() - 1;
	MeshBatchElement.PrimitiveUniformBufferResource = &GIdentityPrimitiveUniformBuffer;

	return MeshBatch;
}

FCanvasTileRendererItem::FRenderData::FRenderData(
	ERHIFeatureLevel::Type InFeatureLevel,
	const FMaterialRenderProxy* InMaterialRenderProxy,
	const FCanvas::FTransformEntry& InTransform)
	: MaterialRenderProxy(InMaterialRenderProxy)
	, Transform(InTransform)
	, VertexFactory(&StaticMeshVertexBuffers, InFeatureLevel)
{}

FCanvasTileRendererItem::FRenderData::~FRenderData()
{
	ReleaseTileMesh();
}

uint32 FCanvasTileRendererItem::FRenderData::GetNumVertices() const
{
	return Tiles.Num() * CanvasTileVertexCount;
}

uint32 FCanvasTileRendererItem::FRenderData::GetNumIndices() const
{
	return Tiles.Num() * CanvasTileIndexCount;
}

void FCanvasTileRendererItem::FRenderData::InitTileMesh(FRHICommandListBase& RHICmdList, const FSceneView& View)
{
	static_assert(CanvasTileVertexCount == 4, "Invalid tile tri-list size.");
	static_assert(CanvasTileIndexCount == 6, "Invalid tile tri-list size.");

	const uint32 TotalVertexCount = GetNumVertices();
	const uint32 TotalIndexCount = GetNumIndices();

	StaticMeshVertexBuffers.PositionVertexBuffer.Init(TotalVertexCount);
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(TotalVertexCount, 1);
	StaticMeshVertexBuffers.ColorVertexBuffer.Init(TotalVertexCount);
	
	IndexBuffer.Indices.SetNum(TotalIndexCount);
	// Make sure the index buffer is using the appropriate size :
	IndexBuffer.ForceUse32Bit(TotalVertexCount > MAX_uint16);

	for (int32 i = 0; i < Tiles.Num(); i++)
	{
		const FTileInst& Tile = Tiles[i];
		const uint32 FirstIndex = i * CanvasTileIndexCount;
		const uint32 FirstVertex = i * CanvasTileVertexCount;

		IndexBuffer.Indices[FirstIndex + 0] = FirstVertex + 0;
		IndexBuffer.Indices[FirstIndex + 1] = FirstVertex + 1;
		IndexBuffer.Indices[FirstIndex + 2] = FirstVertex + 2;
		IndexBuffer.Indices[FirstIndex + 3] = FirstVertex + 2;
		IndexBuffer.Indices[FirstIndex + 4] = FirstVertex + 1;
		IndexBuffer.Indices[FirstIndex + 5] = FirstVertex + 3;

		const float X = Tile.X;
		const float Y = Tile.Y;
		const float U = Tile.U;
		const float V = Tile.V;
		const float SizeX = Tile.SizeX;
		const float SizeY = Tile.SizeY;
		const float SizeU = Tile.SizeU;
		const float SizeV = Tile.SizeV;

		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(FirstVertex + 0) = FVector3f(X + SizeX, Y, 0.0f);
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(FirstVertex + 1) = FVector3f(X, Y, 0.0f);
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(FirstVertex + 2) = FVector3f(X + SizeX, Y + SizeY, 0.0f);
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(FirstVertex + 3) = FVector3f(X, Y + SizeY, 0.0f);

		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(FirstVertex + 0, 0, FVector2f(U + SizeU, V));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(FirstVertex + 1, 0, FVector2f(U, V));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(FirstVertex + 2, 0, FVector2f(U + SizeU, V + SizeV));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(FirstVertex + 3, 0, FVector2f(U, V + SizeV));

		for (int j = 0; j < CanvasTileVertexCount; j++)
		{
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(FirstVertex + j, FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f));
			StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(FirstVertex + j) = Tile.InColor;
		}
	}

	StaticMeshVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
	StaticMeshVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
	VertexFactory.InitResource(RHICmdList);
}

void FCanvasTileRendererItem::FRenderData::ReleaseTileMesh()
{
	VertexFactory.ReleaseResource();
	IndexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.PositionVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.ColorVertexBuffer.ReleaseResource();
}

void FCanvasTileRendererItem::FRenderData::RenderTiles(
	FCanvasRenderContext& RenderContext,
	FMeshPassProcessorRenderState& DrawRenderState,
	const FSceneView& View,
	bool bIsHitTesting,
	bool bUse128bitRT)
{
	check(IsInRenderingThread());

	if (Tiles.Num() == 0)
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(RenderContext.GraphBuilder, CanvasDrawTiles, "%s", *MaterialRenderProxy->GetIncompleteMaterialWithFallback(GMaxRHIFeatureLevel).GetFriendlyName());

	TRACE_CPUPROFILER_EVENT_SCOPE(CanvasDrawTiles);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CanvasDrawTiles)

	IRendererModule& RendererModule = GetRendererModule();

	InitTileMesh(RenderContext.GraphBuilder.RHICmdList, View);

	// We know we have at least 1 tile so prep up a new batch right away : 
	FMeshBatch* CurrentMeshBatch = AllocTileMeshBatch(RenderContext, Tiles[0].HitProxyId);
	check(CurrentMeshBatch->Elements[0].FirstIndex == 0); // The first batch should always start at the first index 

	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); TileIdx++)
	{
		const FTileInst& Tile = Tiles[TileIdx];

		// We only need a new batch when the hit proxy id changes : 
		if (CurrentMeshBatch->BatchHitProxyId != Tile.HitProxyId)
		{
			// Flush the current batch before allocating a new one: 
			GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, *CurrentMeshBatch, bIsHitTesting, CurrentMeshBatch->BatchHitProxyId, bUse128bitRT);

			CurrentMeshBatch = AllocTileMeshBatch(RenderContext, Tile.HitProxyId);
			CurrentMeshBatch->Elements[0].FirstIndex = CanvasTileIndexCount * TileIdx;
		}

		// Add 2 triangles to the batch per tile : 
		CurrentMeshBatch->Elements[0].NumPrimitives += 2;
	}

	// Flush the final batch: 
	check(CurrentMeshBatch != nullptr);
	GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, *CurrentMeshBatch, bIsHitTesting, CurrentMeshBatch->BatchHitProxyId, bUse128bitRT);
}

bool FCanvasTileRendererItem::Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas)
{
	FGameTime Time;
	if (!bFreezeTime)
	{
		Time = Canvas->GetTime();
	}

	checkSlow(Data);

	const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();

	const FSceneViewFamily& ViewFamily = *RenderContext.Alloc<FSceneViewFamily>(FSceneViewFamily::ConstructionValues(
		CanvasRenderTarget,
		nullptr,
		FEngineShowFlags(ESFIM_Game))
		.SetTime(Time));

	const FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	const FSceneView& View = *RenderContext.Alloc<const FSceneView>(ViewInitOptions);

	Data->RenderTiles(RenderContext, DrawRenderState, View, Canvas->IsHitTesting());

	if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
	{
		RenderContext.DeferredRelease(MoveTemp(Data));
		Data = nullptr;
	}

	return true;
}

bool FCanvasTileRendererItem::Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope)
{
	FGameTime Time;
	if (!bFreezeTime)
	{
		Time = Canvas->GetTime();
	}

	checkSlow(Data);

	const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
	if (ensure(CanvasRenderTarget))
	{
		const FSceneViewFamily* ViewFamily = new FSceneViewFamily(FSceneViewFamily::ConstructionValues(
			CanvasRenderTarget,
			Canvas->GetScene(),
			FEngineShowFlags(ESFIM_Game))
			.SetTime(Time));

		const FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

		// make a temporary view
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverlayColor = FLinearColor::White;

		const FSceneView* View = new FSceneView(ViewInitOptions);

		const bool bIsHitTesting = Canvas->IsHitTesting();
		const bool bDeleteOnRender = Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender;

		bool bRequiresExplicit128bitRT = false;

		FTextureRHIRef CanvasRTTexture = CanvasRenderTarget->GetRenderTargetTexture();
		if (CanvasRTTexture)
		{
			bRequiresExplicit128bitRT = PlatformRequires128bitRT(CanvasRTTexture->GetFormat());
		}

		RenderScope.EnqueueRenderCommand(
			[LocalData = Data, View, bIsHitTesting, bRequiresExplicit128bitRT]
			(FCanvasRenderContext& RenderContext) mutable
		{
			FMeshPassProcessorRenderState DrawRenderState;

			// disable depth test & writes
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

			LocalData->RenderTiles(RenderContext, DrawRenderState, *View, bIsHitTesting, bRequiresExplicit128bitRT);

			RenderContext.DeferredRelease(MoveTemp(LocalData));
			RenderContext.DeferredDelete(View->Family);
			RenderContext.DeferredDelete(View);
		});

		if (bDeleteOnRender)
		{
			Data = nullptr;
		}

		return true;
	}
		
	return false;
}
