// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Timeline.cpp
=============================================================================*/

#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"
#include "GameFramework/WorldSettings.h"
#include "Net/UnrealNetwork.h"
#include "Components/TimelineComponent.h"
#include "Engine/World.h"
#include "Misc/App.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

DEFINE_LOG_CATEGORY_STATIC(LogTimeline, Log, All);

DECLARE_CYCLE_STAT(TEXT("TimelineComp Tick"), STAT_TimelineCompTick, STATGROUP_Default);

//////////////////////////////////////////////////////////////////////////
// FTimeline

UEnum* FTimeline::GetTimelineDirectionEnum()
{
	static UEnum* TimelineDirectionEnum = nullptr;
	if (nullptr == TimelineDirectionEnum)
	{
		FTopLevelAssetPath TimelineDirectionEnumEnumPath(TEXT("/Script/Engine"), TEXT("ETimelineDirection"));
		TimelineDirectionEnum = FindObject<UEnum>(TimelineDirectionEnumEnumPath);
		check(TimelineDirectionEnum);
	}
	return TimelineDirectionEnum;
}

void FTimeline::Play()
{
	bReversePlayback = false;

	ChangeMarkPlayingState(/*bIsPlaying=*/ true);
}

void FTimeline::PlayFromStart()
{
	SetPlaybackPosition(0.f, false);
	Play();
}

void FTimeline::Reverse()
{
	bReversePlayback = true;
	
	ChangeMarkPlayingState(/*bIsPlaying=*/ true);
}

void FTimeline::ReverseFromEnd()
{
	SetPlaybackPosition(GetTimelineLength(), false);
	Reverse();
}

void FTimeline::Stop()
{
	ChangeMarkPlayingState(/*bIsPlaying=*/false);
}

bool FTimeline::IsPlaying() const
{
	// Consider us playing if the first bit is set to 1
	return PlayingStateTracker & 0x01;
}

void FTimeline::AddEvent(float Time, FOnTimelineEvent Event)
{
	FTimelineEventEntry NewEntry;
	NewEntry.Time = Time;
	NewEntry.EventFunc = Event;

	Events.Add(NewEntry);
}

void FTimeline::AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVector InterpFunc, FName PropertyName, FName TrackName)
{
	if (VectorCurve)
	{
		FTimelineVectorTrack NewEntry;
		NewEntry.VectorCurve = VectorCurve;
		NewEntry.InterpFunc = InterpFunc;
		NewEntry.TrackName = TrackName;
		NewEntry.VectorPropertyName = PropertyName;

		InterpVectors.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpVector: VectorCurve is null so it will not be added!");
	}
}

void FTimeline::AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVectorStatic InterpFunc)
{
	if (VectorCurve)
	{
		FTimelineVectorTrack NewEntry;
		NewEntry.VectorCurve = VectorCurve;
		NewEntry.InterpFuncStatic = InterpFunc;

		InterpVectors.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpVector: VectorCurve is null so it will not be added!");
	}
}

void FTimeline::AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloat InterpFunc, FName PropertyName, FName TrackName)
{
	if (FloatCurve)
	{
		FTimelineFloatTrack NewEntry;
		NewEntry.FloatCurve = FloatCurve;
		NewEntry.InterpFunc = InterpFunc;
		NewEntry.TrackName = TrackName;
		NewEntry.FloatPropertyName = PropertyName;

		InterpFloats.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpFloat: FloatCurve is null so it will not be added!");
	}
}

void FTimeline::AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloatStatic InterpFunc)
{
	if (FloatCurve)
	{
		FTimelineFloatTrack NewEntry;
		NewEntry.FloatCurve = FloatCurve;
		NewEntry.InterpFuncStatic = InterpFunc;

		InterpFloats.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpFloat: FloatCurve is null so it will not be added!");
	}
}

void FTimeline::AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColor InterpFunc, FName PropertyName, FName TrackName)
{
	if (LinearColorCurve)
	{
		FTimelineLinearColorTrack NewEntry;
		NewEntry.LinearColorCurve = LinearColorCurve;
		NewEntry.InterpFunc = InterpFunc;
		NewEntry.TrackName = TrackName;
		NewEntry.LinearColorPropertyName = PropertyName;

		InterpLinearColors.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpLinearColor: LinearColorCurve is null so it will not be added!");
	}
}

void FTimeline::AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColorStatic InterpFunc)
{
	if (LinearColorCurve)
	{
		FTimelineLinearColorTrack NewEntry;
		NewEntry.LinearColorCurve = LinearColorCurve;
		NewEntry.InterpFuncStatic = InterpFunc;

		InterpLinearColors.Add(NewEntry);
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::AddInterpLinearColor: LinearColorCurve is null so it will not be added!");
	}
}

void FTimeline::SetFloatCurve(UCurveFloat* NewFloatCurve, FName FloatTrackName)
{
	if (NewFloatCurve)
	{
		bool bFoundTrack = false;
		if (FloatTrackName != NAME_None)
		{
			for (FTimelineFloatTrack& FloatTrack : InterpFloats)
			{
				if (FloatTrack.TrackName == FloatTrackName)
				{
					FloatTrack.FloatCurve = NewFloatCurve;
					bFoundTrack = true;
					break;
				}
			}
		}

		if(!bFoundTrack)
		{
			UE_LOGF(LogTimeline, Log, "SetFloatCurve: No float track with name %ls!", *FloatTrackName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::SetFloatCurve: NewFloatCurve is null so it will not be set!");
	}
}

void FTimeline::SetVectorCurve(UCurveVector* NewVectorCurve, FName VectorTrackName)
{
	if (NewVectorCurve)
	{
		bool bFoundTrack = false;
		if (VectorTrackName != NAME_None)
		{
			for (FTimelineVectorTrack& VectorTrack : InterpVectors)
			{
				if (VectorTrack.TrackName == VectorTrackName)
				{
					VectorTrack.VectorCurve = NewVectorCurve;
					bFoundTrack = true;
					break;
				}
			}
		}

		if (!bFoundTrack)
		{
			UE_LOGF(LogTimeline, Log, "SetVectorCurve: No vector track with name %ls!", *VectorTrackName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::SetVectorCurve: NewVectorCurve is null so it will not be set!");
	}
}

void FTimeline::SetLinearColorCurve(UCurveLinearColor* NewLinearColorCurve, FName LinearColorTrackName)
{
	if (NewLinearColorCurve)
	{
		bool bFoundTrack = false;
		if (LinearColorTrackName != NAME_None)
		{
			for (FTimelineLinearColorTrack& ColorTrack : InterpLinearColors)
			{
				if (ColorTrack.TrackName == LinearColorTrackName)
				{
					ColorTrack.LinearColorCurve = NewLinearColorCurve;
					bFoundTrack = true;
					break;
				}

			}
		}

		if (!bFoundTrack)
		{
			UE_LOGF(LogTimeline, Log, "SetLinearColorCurve: No color track with name %ls!", *LinearColorTrackName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogTimeline, Warning, "FTimeline::SetLinearColorCurve: NewLinearColorCurve is null so it will not be set!");
	}
}

void FTimeline::SetPlaybackPosition(float NewPosition, bool bFireEvents, bool bFireUpdate)
{
	float OldPosition = Position;
	Position = NewPosition;

	UObject* PropSetObject = PropertySetObject.Get();

	// Iterate over each vector interpolation
	for(int32 InterpIdx=0; InterpIdx<InterpVectors.Num(); InterpIdx++)
	{
		FTimelineVectorTrack& VecEntry = InterpVectors[InterpIdx];
		if ( VecEntry.VectorCurve && (VecEntry.InterpFunc.IsBound() || VecEntry.VectorPropertyName != NAME_None || VecEntry.InterpFuncStatic.IsBound()) )
		{
			// Get vector from curve
			FVector const Vec = VecEntry.VectorCurve->GetVectorValue(Position);

			// Pass vec to specified function
			VecEntry.InterpFunc.ExecuteIfBound(Vec);

			// Set vector property
			if (PropSetObject)
			{
				if (VecEntry.VectorProperty == NULL)
				{
					VecEntry.VectorProperty = FindFProperty<FStructProperty>(PropSetObject->GetClass(), VecEntry.VectorPropertyName);
					if(VecEntry.VectorProperty == NULL)
					{
						UE_LOGF(LogTimeline, Log, "SetPlaybackPosition: No vector property '%ls' in '%ls'", *VecEntry.VectorPropertyName.ToString(), *PropSetObject->GetName());
					}
				}
				if (VecEntry.VectorProperty)
				{
					*VecEntry.VectorProperty->ContainerPtrToValuePtr<FVector>(PropSetObject) = Vec;
				}
			}

			// Pass vec to non-dynamic version of the specified function
			VecEntry.InterpFuncStatic.ExecuteIfBound(Vec);
		}
	}

	// Iterate over each float interpolation
	for(int32 InterpIdx=0; InterpIdx<InterpFloats.Num(); InterpIdx++)
	{
		FTimelineFloatTrack& FloatEntry = InterpFloats[InterpIdx];
		if( FloatEntry.FloatCurve && (FloatEntry.InterpFunc.IsBound() || FloatEntry.FloatPropertyName != NAME_None || FloatEntry.InterpFuncStatic.IsBound()) )
		{
			// Get float from func
			const float Val = FloatEntry.FloatCurve->GetFloatValue(Position);

			// Pass float to specified function
			FloatEntry.InterpFunc.ExecuteIfBound(Val);

			// Set float property
			if (PropSetObject)
			{
				if (FloatEntry.FloatProperty == NULL)
				{
					FloatEntry.FloatProperty = FindFProperty<FFloatProperty>(PropSetObject->GetClass(), FloatEntry.FloatPropertyName);
					if(FloatEntry.FloatProperty == NULL)
					{
						UE_LOGF(LogTimeline, Log, "SetPlaybackPosition: No float property '%ls' in '%ls'", *FloatEntry.FloatPropertyName.ToString(), *PropSetObject->GetName());
					}
				}
				if (FloatEntry.FloatProperty)
				{
					FloatEntry.FloatProperty->SetPropertyValue_InContainer(PropSetObject, Val);
				}
			}
			
			// Pass float to non-dynamic version of the specified function
			FloatEntry.InterpFuncStatic.ExecuteIfBound(Val);
		}
	}

	// Iterate over each color interpolation
	for(int32 InterpIdx=0; InterpIdx<InterpLinearColors.Num(); InterpIdx++)
	{
		FTimelineLinearColorTrack& ColorEntry = InterpLinearColors[InterpIdx];
		if ( ColorEntry.LinearColorCurve && (ColorEntry.InterpFunc.IsBound() || ColorEntry.LinearColorPropertyName != NAME_None || ColorEntry.InterpFuncStatic.IsBound()) )
		{
			// Get vector from curve
			const FLinearColor Color = ColorEntry.LinearColorCurve->GetLinearColorValue(Position);

			// Pass vec to specified function
			ColorEntry.InterpFunc.ExecuteIfBound(Color);

			// Set vector property
			if (PropSetObject)
			{
				if (ColorEntry.LinearColorProperty == NULL)
				{
					ColorEntry.LinearColorProperty = FindFProperty<FStructProperty>(PropSetObject->GetClass(), ColorEntry.LinearColorPropertyName);
					if(ColorEntry.LinearColorProperty == NULL)
					{
						UE_LOGF(LogTimeline, Log, "SetPlaybackPosition: No linear color property '%ls' in '%ls'", *ColorEntry.LinearColorPropertyName.ToString(), *PropSetObject->GetName());
					}
				}
				if (ColorEntry.LinearColorProperty)
				{
					*ColorEntry.LinearColorProperty->ContainerPtrToValuePtr<FLinearColor>(PropSetObject) = Color;
				}
			}

			// Pass vec to non-dynamic version of the specified function
			ColorEntry.InterpFuncStatic.ExecuteIfBound(Color);
		}
	}

	if(DirectionPropertyName != NAME_None)
	{
		if (PropSetObject)
		{
			if (DirectionProperty == nullptr)
			{
				DirectionProperty = FindFProperty<FByteProperty>(PropSetObject->GetClass(), DirectionPropertyName);
				if (DirectionProperty == nullptr)
				{
					DirectionProperty = FindFProperty<FEnumProperty>(PropSetObject->GetClass(), DirectionPropertyName);
				}

				if (DirectionProperty == nullptr)
				{
					UE_LOGF(LogTimeline, Log, "SetPlaybackPosition: No direction property '%ls' in '%ls'", *DirectionPropertyName.ToString(), *PropSetObject->GetName());
				}
			}
			if (DirectionProperty)
			{
				const ETimelineDirection::Type CurrentDirection = bReversePlayback ? ETimelineDirection::Backward : ETimelineDirection::Forward;
				TEnumAsByte<ETimelineDirection::Type> ValueAsByte(CurrentDirection);
				if (FByteProperty* ByteDirection = CastField<FByteProperty>(DirectionProperty))
				{
					ByteDirection->SetPropertyValue_InContainer(PropSetObject, ValueAsByte);
				}
				else
				{
					FEnumProperty* EnumProp = CastFieldChecked<FEnumProperty>(DirectionProperty);
					void* PropAddr = EnumProp->ContainerPtrToValuePtr<void>(PropSetObject);
					FNumericProperty* UnderlyingProp = EnumProp->GetUnderlyingProperty();
					UnderlyingProp->SetIntPropertyValue(PropAddr, (int64)ValueAsByte);
				}
			}
		}
	}


	// If we should be firing events for this track...
	if (bFireEvents)
	{
		// If playing sequence forwards.
		float MinTime, MaxTime;
		if (!bReversePlayback)
		{
			MinTime = OldPosition;
			MaxTime = Position;

			// Slight hack here.. if playing forwards and reaching the end of the sequence, force it over a little to ensure we fire events actually on the end of the sequence.
			if (MaxTime == GetTimelineLength())
			{
				MaxTime += (float)UE_KINDA_SMALL_NUMBER;
			}
		}
		// If playing sequence backwards.
		else
		{
			MinTime = Position;
			MaxTime = OldPosition;

			// Same small hack as above for backwards case.
			if (MinTime == 0.f)
			{
				MinTime -= (float)UE_KINDA_SMALL_NUMBER;
			}
		}

		// See which events fall into traversed region.
		for (int32 i = 0; i < Events.Num(); i++)
		{
			float EventTime = Events[i].Time;

			// Need to be slightly careful here and make behavior for firing events symmetric when playing forwards of backwards.
			bool bFireThisEvent = false;
			if (!bReversePlayback)
			{
				if (EventTime >= MinTime && EventTime < MaxTime)
				{
					bFireThisEvent = true;
				}
			}
			else
			{
				if (EventTime > MinTime && EventTime <= MaxTime)
				{
					bFireThisEvent = true;
				}
			}

			if (bFireThisEvent)
			{
				Events[i].EventFunc.ExecuteIfBound();
			}
		}
	}

	// Execute the delegate to say that all properties are updated
	if (bFireUpdate)
	{
		TimelinePostUpdateFunc.ExecuteIfBound();
	}
}


float FTimeline::GetPlaybackPosition() const
{
	return Position;
}

void FTimeline::SetLooping(bool bNewLooping)
{
	bLooping = bNewLooping;
}

bool FTimeline::IsLooping() const
{
	return bLooping;
}

bool FTimeline::IsReversing() const
{
	return IsPlaying() && bReversePlayback;
}

void FTimeline::SetPlayRate(float NewRate)
{
	PlayRate = NewRate;
}

float FTimeline::GetPlayRate() const
{
	return PlayRate;
}

void FTimeline::TickTimeline(float DeltaTime)
{
	bool bIsFinished = false;

	if (IsPlaying())
	{
		const float TimelineLength = GetTimelineLength();
		float EffectiveDeltaTime = DeltaTime * (bReversePlayback ? (-PlayRate) : (PlayRate));

		float NewPosition = Position + EffectiveDeltaTime;

		if(EffectiveDeltaTime > 0.0f)
		{
			if(NewPosition > TimelineLength)
			{
				// If looping, play to end, jump to start, and set target to somewhere near the beginning.
				if(bLooping)
				{
					SetPlaybackPosition(TimelineLength, true, false);
					SetPlaybackPosition(0.f, false, false);

					if( TimelineLength > 0.f )
					{
						while(NewPosition > TimelineLength)
						{
							NewPosition -= TimelineLength;
						}
					}
					else
					{
						NewPosition = 0.f;
					}
				}
				// If not looping, snap to end and stop playing.
				else
				{
					NewPosition = TimelineLength;
					Stop();
					bIsFinished = true;
				}
			}
		}
		else
		{
			if(NewPosition < 0.f)
			{
				// If looping, play to start, jump to end, and set target to somewhere near the end.
				if(bLooping)
				{
					SetPlaybackPosition(0.f, true, false);
					SetPlaybackPosition(TimelineLength, false, false);

					if( TimelineLength > 0.f )
					{
						while(NewPosition < 0.f)
						{
							NewPosition += TimelineLength;
						}
					}
					else
					{
						NewPosition = 0.f;
					}
				}
				// If not looping, snap to start and stop playing.
				else
				{
					NewPosition = 0.f;
					Stop();
					bIsFinished = true;
				}
			}
		}

		SetPlaybackPosition(NewPosition, true);
	}

	// Notify user that timeline finished
	if (bIsFinished) 
	{
		TimelineFinishedFunc.ExecuteIfBound();
		TimelineFinishFuncStatic.ExecuteIfBound();
	}
}

void FTimeline::SetNewTime(float NewTime)
{
	// Ensure value is sensible
	NewTime = FMath::Clamp<float>(NewTime, 0.0f, Length);
	SetPlaybackPosition(NewTime, false);
}

float FTimeline::GetTimelineLength() const
{
	switch (LengthMode)
	{
	case TL_TimelineLength:
		return Length;
	case TL_LastKeyFrame:
		return GetLastKeyframeTime();
	default:
		UE_LOGF(LogTimeline, Error, "Invalid timeline length mode on timeline!");
		return 0.f;
	}
}

float FTimeline::GetScaledTimelineLength() const
{
	const float CurrentPlayRate = GetPlayRate();
	if(CurrentPlayRate != 0.0f)
	{
		return GetTimelineLength() / CurrentPlayRate;	
	}
	else
	{
		UE_LOGF(LogTimeline, Error, "Invalid timeline PlayRate!");
		return 0.0f;
	}
}

/** Sets the timeline length mode */
void FTimeline::SetTimelineLengthMode(ETimelineLengthMode NewMode)
{
	LengthMode = NewMode;
}

void FTimeline::SetTimelineLength(float NewLength)
{
	Length = NewLength;
	if(Position > NewLength)
	{
		SetNewTime(NewLength-UE_KINDA_SMALL_NUMBER);
	}
}

void FTimeline::SetPropertySetObject(UObject* NewPropertySetObject)
{
	PropertySetObject = NewPropertySetObject;
}

void FTimeline::SetTimelinePostUpdateFunc(FOnTimelineEvent NewTimelinePostUpdateFunc)
{
	TimelinePostUpdateFunc = NewTimelinePostUpdateFunc;
}

void FTimeline::SetTimelineFinishedFunc(FOnTimelineEvent NewTimelineFinishedFunc)
{
	TimelineFinishedFunc = NewTimelineFinishedFunc;
}

void FTimeline::SetTimelineFinishedFunc(FOnTimelineEventStatic NewTimelineFinishedFunc)
{
	TimelineFinishFuncStatic = NewTimelineFinishedFunc;
}

void FTimeline::SetDirectionPropertyName(FName InDirectionPropertyName)
{
	DirectionPropertyName = InDirectionPropertyName;
}

float FTimeline::GetLastKeyframeTime() const
{
	float MaxTime = 0.f;

	// Check the various tracks for the max time specified
	for( auto it = Events.CreateConstIterator(); it; ++it )
	{
		const FTimelineEventEntry& CurrentEvent = *it;
		MaxTime = FMath::Max(CurrentEvent.Time, MaxTime);
	}

	for( auto it = InterpVectors.CreateConstIterator(); it; ++it )
	{
		const FTimelineVectorTrack& CurrentTrack = *it;
		float MinVal, MaxVal;
		CurrentTrack.VectorCurve->GetTimeRange(MinVal, MaxVal);
		MaxTime = FMath::Max(MaxVal, MaxTime);
	}

	for( auto it = InterpFloats.CreateConstIterator(); it; ++it )
	{
		const FTimelineFloatTrack& CurrentTrack = *it;
		float MinVal, MaxVal;
		CurrentTrack.FloatCurve->GetTimeRange(MinVal, MaxVal);
		MaxTime = FMath::Max(MaxVal, MaxTime);
	}

	for( auto it = InterpLinearColors.CreateConstIterator(); it; ++it )
	{
		const FTimelineLinearColorTrack& CurrentTrack = *it;
		float MinVal, MaxVal;
		CurrentTrack.LinearColorCurve->GetTimeRange( MinVal, MaxVal );
		MaxTime = FMath::Max( MaxVal, MaxTime );
	}

	return MaxTime;
}

void FTimeline::ChangeMarkPlayingState(const bool bIsPlaying)
{
	// We will use the upper bits of the PlayingStateTracker as "dirty" bits to ensure that the
	// value gets replicated properly in case you call "Play" on the same frame as the timeline
	// finishes.
	if (bIsPlaying)
	{
		// Increment by 2 to "dirty" the number without touching the first bit (2 is 0010)
        // wrapping it to 255 (max uint8 - 1)
        PlayingStateTracker = FMath::WrapExclusive(PlayingStateTracker + 2, 2, 255);
    
        // Make sure that we always set the 1 bit because we are playing.
        PlayingStateTracker |= 0x01;
	}
	else
	{
		// To flag the state as stopped, reset the first bit to zero
		PlayingStateTracker &= ~0x01;
	}
}

void FTimeline::GetAllCurves(TSet<class UCurveBase*>& InOutCurves) const
{
	for (auto& Track : InterpVectors)
	{
		InOutCurves.Add(Track.VectorCurve);
	}
	for (auto& Track : InterpFloats)
	{
		InOutCurves.Add(Track.FloatCurve);
	}
	for (auto& Track : InterpLinearColors)
	{
		InOutCurves.Add(Track.LinearColorCurve);
	}
}

bool FTimeline::Serialize(FArchive& Ar)
{
	// Need a custom version to handle the bPlaying deprecation
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	// Returning false will just run the normal serialization
	return false;
}

void FTimeline::PostSerialize(const FArchive& Ar)
{
	// If we are loading code from before the serialization change...
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::TimelinePlayingStateTrackerDeprecation)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS

		// ...populate the new PlayingStateTracker with the value of the
		// old bPlaying flag. 
		ChangeMarkPlayingState(bPlaying_DEPRECATED);

PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

//////////////////////////////////////////////////////////////////////////
// UTimelineComponent

UTimelineComponent::UTimelineComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
}

void UTimelineComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(TimelineComponent);
	SCOPE_CYCLE_COUNTER(STAT_TimelineCompTick);

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bIgnoreTimeDilation)
	{
		// Get the raw, undilated delta time.
		DeltaTime = FApp::GetDeltaTime();
		const UWorld* World = GetWorld();
		if (const AWorldSettings* WorldSettings = World ? World->GetWorldSettings() : nullptr)
		{
			// Clamp DeltaTime in the same way as before.
			// UWorld::Tick called AWorldSettings::FixupDeltaSeconds, which clamped between Min and MaxUndilatedFrameTime.
			DeltaTime = FMath::Clamp(DeltaTime, WorldSettings->MinUndilatedFrameTime, WorldSettings->MaxUndilatedFrameTime);
		}
	}

	TheTimeline.TickTimeline(DeltaTime);

	if (!IsNetSimulating())
	{
		// Do not deactivate if we are done, since bActive is a replicated property and we shouldn't have simulating
		// clients touch replicated variables.
		if (!TheTimeline.IsPlaying())
		{
			Deactivate();
		}
	}
}

bool UTimelineComponent::IsReadyForOwnerToAutoDestroy() const
{
	return !IsPlaying();
}

void UTimelineComponent::Activate(bool bReset)
{
	Super::Activate(bReset);
	PrimaryComponentTick.SetTickFunctionEnable(true);
}

void UTimelineComponent::Deactivate()
{
	Super::Deactivate();
	PrimaryComponentTick.SetTickFunctionEnable(false);
}

void UTimelineComponent::Play()
{
	Activate();
	TheTimeline.Play();
}

void UTimelineComponent::PlayFromStart()
{
	Activate();
	TheTimeline.PlayFromStart();
}

void UTimelineComponent::Reverse()
{
	Activate();
	TheTimeline.Reverse();
}

void UTimelineComponent::ReverseFromEnd()
{
	Activate();
	TheTimeline.ReverseFromEnd();
}

void UTimelineComponent::Stop()
{
	TheTimeline.Stop();
}

bool UTimelineComponent::IsPlaying() const
{
	return TheTimeline.IsPlaying();
}

void UTimelineComponent::AddEvent(float Time, FOnTimelineEvent Event)
{
	TheTimeline.AddEvent(Time, Event);
}

void UTimelineComponent::AddInterpVector(UCurveVector* VectorCurve, FOnTimelineVector InterpFunc, FName PropertyName, FName TrackName)
{
	TheTimeline.AddInterpVector(VectorCurve, InterpFunc, PropertyName, TrackName);
}

void UTimelineComponent::AddInterpVector(UCurveVector* VectorCurve, const FOnTimelineVectorStatic InterpFunc)
{
	TheTimeline.AddInterpVector(VectorCurve, InterpFunc);
}

void UTimelineComponent::AddInterpFloat(UCurveFloat* FloatCurve, FOnTimelineFloat InterpFunc, FName PropertyName, FName TrackName)
{
	TheTimeline.AddInterpFloat(FloatCurve, InterpFunc, PropertyName, TrackName);
}

void UTimelineComponent::AddInterpFloat(UCurveFloat* FloatCurve, const FOnTimelineFloatStatic InterpFunc)
{
	TheTimeline.AddInterpFloat(FloatCurve, InterpFunc);
}

void UTimelineComponent::AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, FOnTimelineLinearColor InterpFunc, FName PropertyName, FName TrackName)
{
	TheTimeline.AddInterpLinearColor(LinearColorCurve, InterpFunc, PropertyName, TrackName);
}

void UTimelineComponent::AddInterpLinearColor(UCurveLinearColor* LinearColorCurve, const FOnTimelineLinearColorStatic InterpFunc)
{
	TheTimeline.AddInterpLinearColor(LinearColorCurve, InterpFunc);
}

void UTimelineComponent::SetPlaybackPosition(float NewPosition, bool bFireEvents, bool bFireUpdate)
{
	TheTimeline.SetPlaybackPosition(NewPosition, bFireEvents, bFireUpdate);
}

float UTimelineComponent::GetPlaybackPosition() const
{
	return TheTimeline.GetPlaybackPosition();
}

void UTimelineComponent::SetLooping(bool bNewLooping)
{
	TheTimeline.SetLooping(bNewLooping);
}

bool UTimelineComponent::IsLooping() const
{
	return TheTimeline.IsLooping();
}

bool UTimelineComponent::IsReversing() const
{
	return TheTimeline.IsReversing();
}

void UTimelineComponent::SetPlayRate(float NewRate)
{
	TheTimeline.SetPlayRate(NewRate);
}

float UTimelineComponent::GetPlayRate() const
{
	return TheTimeline.GetPlayRate();
}

void UTimelineComponent::SetNewTime(float NewTime)
{
	TheTimeline.SetNewTime(NewTime);
}

float UTimelineComponent::GetTimelineLength() const
{
	return TheTimeline.GetTimelineLength();
}

float UTimelineComponent::GetScaledTimelineLength() const
{
	return TheTimeline.GetScaledTimelineLength();
}

void UTimelineComponent::SetTimelineLength(float NewLength)
{
	return TheTimeline.SetTimelineLength(NewLength);
}

void UTimelineComponent::SetTimelineLengthMode(ETimelineLengthMode NewLengthMode)
{
	TheTimeline.SetTimelineLengthMode(NewLengthMode);
}

void UTimelineComponent::SetIgnoreTimeDilation(bool bNewIgnoreTimeDilation)
{
	bIgnoreTimeDilation = bNewIgnoreTimeDilation;
}

bool UTimelineComponent::GetIgnoreTimeDilation() const
{
	return bIgnoreTimeDilation;
}

void UTimelineComponent::SetFloatCurve(UCurveFloat* NewFloatCurve, FName FloatTrackName)
{
	TheTimeline.SetFloatCurve(NewFloatCurve, FloatTrackName);
}

void UTimelineComponent::SetVectorCurve(UCurveVector* NewVectorCurve, FName VectorTrackName)
{
	TheTimeline.SetVectorCurve(NewVectorCurve, VectorTrackName);
}

void UTimelineComponent::SetLinearColorCurve(UCurveLinearColor* NewLinearColorCurve, FName LinearColorTrackName)
{
	TheTimeline.SetLinearColorCurve(NewLinearColorCurve, LinearColorTrackName);
}

void UTimelineComponent::SetPropertySetObject(UObject* NewPropertySetObject)
{
	TheTimeline.SetPropertySetObject(NewPropertySetObject);
}

void UTimelineComponent::SetTimelinePostUpdateFunc(FOnTimelineEvent NewTimelinePostUpdateFunc)
{
	TheTimeline.SetTimelinePostUpdateFunc(NewTimelinePostUpdateFunc);
}

void UTimelineComponent::SetTimelineFinishedFunc(FOnTimelineEvent NewTimelineFinishedFunc)
{
	TheTimeline.SetTimelineFinishedFunc(NewTimelineFinishedFunc);
}

void UTimelineComponent::SetTimelineFinishedFunc(FOnTimelineEventStatic NewTimelineFinishedFunc)
{
	TheTimeline.SetTimelineFinishedFunc(NewTimelineFinishedFunc);
}

ETimelineSigType UTimelineComponent::GetTimelineSignatureForFunction(const UFunction* Func)
{
	if(Func != NULL)
	{
		if(Func->IsSignatureCompatibleWith(GetTimelineEventSignature()))
		{
			return ETS_EventSignature;
		}
		else if(Func->IsSignatureCompatibleWith(GetTimelineFloatSignature()))
		{
			return ETS_FloatSignature;
		}
		else if(Func->IsSignatureCompatibleWith(GetTimelineVectorSignature()))
		{
			return ETS_VectorSignature;
		}
		else if(Func->IsSignatureCompatibleWith(GetTimelineLinearColorSignature()))
		{
			return ETS_LinearColorSignature;
		}
	}

	return ETS_InvalidSignature;
}

UFunction* UTimelineComponent::GetTimelineEventSignature()
{
	UFunction* TimelineEventSig = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/Engine")), TEXT("OnTimelineEvent__DelegateSignature"));
	check(TimelineEventSig != NULL);
	return TimelineEventSig;
}


UFunction* UTimelineComponent::GetTimelineFloatSignature()
{
	UFunction* TimelineFloatSig = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/Engine")), TEXT("OnTimelineFloat__DelegateSignature"));
	check(TimelineFloatSig != NULL);
	return TimelineFloatSig;
}


UFunction* UTimelineComponent::GetTimelineVectorSignature()
{
	UFunction* TimelineVectorSig = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/Engine")), TEXT("OnTimelineVector__DelegateSignature"));
	check(TimelineVectorSig != NULL);
	return TimelineVectorSig;
}


UFunction* UTimelineComponent::GetTimelineLinearColorSignature()
{
	UFunction* TimelineVectorSig = FindObject<UFunction>(FindPackage(nullptr, TEXT("/Script/Engine")), TEXT("OnTimelineLinearColor__DelegateSignature"));
	check(TimelineVectorSig != NULL);
	return TimelineVectorSig;
}


void UTimelineComponent::SetDirectionPropertyName(FName DirectionPropertyName)
{
	TheTimeline.SetDirectionPropertyName(DirectionPropertyName);
}

void UTimelineComponent::OnRep_Timeline(FTimeline& OldTimeline)
{
	if (!TheTimeline.IsPlaying() && OldTimeline.GetPlaybackPosition() != TheTimeline.GetPlaybackPosition())
	{
		// make sure a final update call occurs on the client for the final position
		TheTimeline.SetPlaybackPosition(TheTimeline.GetPlaybackPosition(), false, true);
	}
}

/// @cond DOXYGEN_WARNINGS

void UTimelineComponent::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( UTimelineComponent, TheTimeline );
}

/// @endcond

void UTimelineComponent::GetAllCurves(TSet<class UCurveBase*>& InOutCurves) const
{
	TheTimeline.GetAllCurves(InOutCurves);
}


bool UTimelineComponent::IsPostLoadThreadSafe() const
{
	return true;
}
