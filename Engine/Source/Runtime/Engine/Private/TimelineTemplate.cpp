// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TimelineTemplate.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/Package.h"
#include "EngineLogs.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TimelineTemplate)

namespace
{
	void SanitizePropertyName(FString& PropertyName)
	{
		// Sanitize the name
		for (int32 i = 0; i < PropertyName.Len(); ++i)
		{
			TCHAR& C = PropertyName[i];

			const bool bGoodChar =
				((C >= 'A') && (C <= 'Z')) || ((C >= 'a') && (C <= 'z')) ||		// A-Z (upper and lowercase) anytime
				(C == '_') ||													// _ anytime
				((i > 0) && (C >= '0') && (C <= '9'));							// 0-9 after the first character

			if (!bGoodChar)
			{
				C = '_';
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// UTimelineTemplate

UTimelineTemplate::UTimelineTemplate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TimelineLength = 5.0f;
	bReplicated = false;
}

void UTimelineTemplate::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) == false)
	{
		TimelineGuid = FGuid::NewGuid();
		UpdateCachedNames();
	}
}

const FString UTimelineTemplate::TemplatePostfix(TEXT("_Template"));

void UTimelineTemplate::UpdateCachedNames()
{
	FString TimelineName = GetName();
	TimelineName.RemoveFromEnd(TemplatePostfix);

	VariableName = *TimelineName;

	FString DirectionPropertyNameStr = FString::Printf(TEXT("%s__Direction_%s"), *TimelineName, *TimelineGuid.ToString());
	SanitizePropertyName(DirectionPropertyNameStr);
	DirectionPropertyName = *DirectionPropertyNameStr;

	UpdateFunctionName = *FString::Printf(TEXT("%s__UpdateFunc"), *TimelineName);
	FinishedFunctionName = *FString::Printf(TEXT("%s__FinishedFunc"), *TimelineName);

	for (FTTEventTrack& EventTrack : EventTracks)
	{
		EventTrack.SetTrackName(EventTrack.GetTrackName(),this);
	}

	for (FTTFloatTrack& FloatTrack : FloatTracks)
	{
		FloatTrack.SetTrackName(FloatTrack.GetTrackName(), this);
	}

	for (FTTVectorTrack& VectorTrack : VectorTracks)
	{
		VectorTrack.SetTrackName(VectorTrack.GetTrackName(), this);
	}

	for (FTTLinearColorTrack& LinearColorTrack : LinearColorTracks)
	{
		LinearColorTrack.SetTrackName(LinearColorTrack.GetTrackName(), this);
	}
}

int32 UTimelineTemplate::FindFloatTrackIndex(const FName FloatTrackName) const
{
	for(int32 i=0; i<FloatTracks.Num(); i++)
	{
		if(FloatTracks[i].GetTrackName() == FloatTrackName)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

int32 UTimelineTemplate::FindVectorTrackIndex(const FName VectorTrackName) const
{
	for(int32 i=0; i<VectorTracks.Num(); i++)
	{
		if(VectorTracks[i].GetTrackName() == VectorTrackName)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

int32 UTimelineTemplate::FindEventTrackIndex(const FName EventTrackName) const
{
	for(int32 i=0; i<EventTracks.Num(); i++)
	{
		if(EventTracks[i].GetTrackName() == EventTrackName)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

int32 UTimelineTemplate::FindLinearColorTrackIndex(const FName ColorTrackName) const
{
	for(int32 i=0; i<LinearColorTracks.Num(); i++)
	{
		if(LinearColorTracks[i].GetTrackName() == ColorTrackName)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

bool UTimelineTemplate::IsNewTrackNameValid(const FName NewTrackName) const
{
	// can't be NAME_None
	if (NewTrackName == NAME_None)
	{
		return false;
	}

	// Check each type of track to see if it already exists
	return	FindFloatTrackIndex(NewTrackName) == INDEX_NONE && 
			FindVectorTrackIndex(NewTrackName) == INDEX_NONE &&
			FindEventTrackIndex(NewTrackName) == INDEX_NONE &&
			FindLinearColorTrackIndex(NewTrackName) == INDEX_NONE;
}

int32 UTimelineTemplate::FindMetaDataEntryIndexForKey(const FName Key) const
{
	for(int32 i=0; i<MetaDataArray.Num(); i++)
	{
		if(MetaDataArray[i].DataKey == Key)
		{
			return i;
		}
	}
	return INDEX_NONE;
}

const FString& UTimelineTemplate::GetMetaData(const FName Key) const
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	check(EntryIndex != INDEX_NONE);
	return MetaDataArray[EntryIndex].DataValue;
}

void UTimelineTemplate::SetMetaData(const FName Key, FString Value)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray[EntryIndex].DataValue = MoveTemp(Value);
	}
	else
	{
		MetaDataArray.Emplace( FBPVariableMetaDataEntry(Key, MoveTemp(Value)) );
	}
}

void UTimelineTemplate::RemoveMetaData(const FName Key)
{
	int32 EntryIndex = FindMetaDataEntryIndexForKey(Key);
	if(EntryIndex != INDEX_NONE)
	{
		MetaDataArray.RemoveAt(EntryIndex);
	}
}

FString UTimelineTemplate::MakeUniqueCurveName(UObject* Obj, UObject* InOuter)
{
	FString OriginalName = Obj->GetName();
	FName TestName(*OriginalName);
	while(StaticFindObjectFast(NULL, InOuter, TestName))
	{
		TestName = FName(*OriginalName, TestName.GetNumber()+1);
	}
	return TestName.ToString();
}

FString UTimelineTemplate::TimelineVariableNameToTemplateName(FName Name)
{
	return Name.ToString() + TEXT("_Template");
}

void UTimelineTemplate::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	//can't be done in post-load, as the UK2Node_Timeline::AllocateDefaultPins, just does Preload.
	//if we've never used this feature, init with the defaults
	if (TrackDisplayOrder.Num() == 0)
	{
		for (int32 EventIndex = 0; EventIndex < EventTracks.Num(); ++EventIndex)
		{
			FTTTrackId TrackId(0 /* TT_Event */, EventIndex);
			TrackDisplayOrder.Add(TrackId);
		}
		for (int32 FloatIndex = 0; FloatIndex < FloatTracks.Num(); ++FloatIndex)
		{
			FTTTrackId TrackId(1 /* TT_FloatInterp*/, FloatIndex);
			TrackDisplayOrder.Add(TrackId);
		}
		for (int32 VectorIndex = 0; VectorIndex < VectorTracks.Num(); ++VectorIndex)
		{
			FTTTrackId TrackId(2 /* TT_VectorInterp*/, VectorIndex);
			TrackDisplayOrder.Add(TrackId);
		}
		for (int32 ColorIndex = 0; ColorIndex < LinearColorTracks.Num(); ++ColorIndex)
		{
			FTTTrackId TrackId(3 /* TT_LinearColorInterp 7*/, ColorIndex);
			TrackDisplayOrder.Add(TrackId);
		}
	}
#endif
}

void UTimelineTemplate::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::StoreTimelineNamesInTemplate)
	{
		UpdateCachedNames();
	}
}

bool UTimelineTemplate::Rename(const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags)
{
	const FName CurrentName = GetFName();

	bool bSuccess = Super::Rename(InName, NewOuter, Flags);

	if (CurrentName != GetFName())
	{
		UpdateCachedNames();
	}

	return bSuccess;
}

void UTimelineTemplate::PostEditImport()
{
	Super::PostEditImport();

	UpdateCachedNames();
}

void UTimelineTemplate::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	UObject* NewCurveOuter = GetOuter();

	const bool bTransientPackage = GetOutermost() == GetTransientPackage();
	// Prevent curves being duplicated during blueprint reinstancing
	const bool bDuplicateCurves = !( bTransientPackage || GIsDuplicatingClassForReinstancing );

	for(TArray<struct FTTFloatTrack>::TIterator It = FloatTracks.CreateIterator();It;++It)
	{
		FTTFloatTrack& Track = *It;
		if( Track.CurveFloat != NULL )
		{
			if( bDuplicateCurves && !Track.bIsExternalCurve )
			{
				if(!Track.CurveFloat->GetOuter()->IsA(UPackage::StaticClass()))
				{
					Track.CurveFloat = DuplicateObject<UCurveFloat>(Track.CurveFloat, NewCurveOuter, *MakeUniqueCurveName(Track.CurveFloat, NewCurveOuter));
				}
			}
		}
		else
		{
			UE_LOGF(LogBlueprint, Warning, "Timeline %ls Track %ls in %ls has an invalid curve.  Please fix!", *GetVariableName().ToString(), *Track.GetTrackName().ToString(), *GetPathNameSafe(GetOuter()));
		}
	}

	for(TArray<struct FTTEventTrack>::TIterator It = EventTracks.CreateIterator();It;++It)
	{
		FTTEventTrack& Track = *It;
		if( Track.CurveKeys != NULL )
		{
			if( bDuplicateCurves && !Track.bIsExternalCurve )
			{
				if(!Track.CurveKeys->GetOuter()->IsA(UPackage::StaticClass()))
				{
					Track.CurveKeys = DuplicateObject<UCurveFloat>(Track.CurveKeys, NewCurveOuter, *MakeUniqueCurveName(Track.CurveKeys, NewCurveOuter));
				}
			}
		}
		else
		{
			UE_LOGF(LogBlueprint, Warning, "Timeline %ls Track %ls in %ls has an invalid curve.  Please fix!", *GetVariableName().ToString(), *Track.GetTrackName().ToString(), *GetPathNameSafe(GetOuter()));
		}
	}

	for(TArray<struct FTTVectorTrack>::TIterator It = VectorTracks.CreateIterator();It;++It)
	{
		FTTVectorTrack& Track = *It;
		if( Track.CurveVector != NULL )
		{
			if( bDuplicateCurves && !Track.bIsExternalCurve )
			{
				if(!Track.CurveVector->GetOuter()->IsA(UPackage::StaticClass()))
				{
					Track.CurveVector = DuplicateObject<UCurveVector>(Track.CurveVector, NewCurveOuter, *MakeUniqueCurveName(Track.CurveVector, NewCurveOuter));
				}
			}
		}
		else
		{
			UE_LOGF(LogBlueprint, Warning, "Timeline %ls Track %ls in %ls has an invalid curve.  Please fix!", *GetVariableName().ToString(), *Track.GetTrackName().ToString(), *GetPathNameSafe(GetOuter()));
		}
	}

	for(TArray<struct FTTLinearColorTrack>::TIterator It = LinearColorTracks.CreateIterator();It;++It)
	{
		FTTLinearColorTrack& Track = *It;
		if( Track.CurveLinearColor != NULL )
		{
			if( bDuplicateCurves && !Track.bIsExternalCurve )
			{
				if(!Track.CurveLinearColor->GetOuter()->IsA(UPackage::StaticClass()))
				{
					Track.CurveLinearColor = DuplicateObject<UCurveLinearColor>(Track.CurveLinearColor, NewCurveOuter, *MakeUniqueCurveName(Track.CurveLinearColor, NewCurveOuter));
				}
			}
		}
		else
		{
			UE_LOGF(LogBlueprint, Warning, "Timeline %ls Track %ls in %ls has an invalid curve.  Please fix!", *GetVariableName().ToString(), *Track.GetTrackName().ToString(), *GetPathNameSafe(GetOuter()));
		}
	}

	TimelineGuid = FGuid::NewGuid();

	UpdateCachedNames();
}

void UTimelineTemplate::GetAllCurves(TSet<UCurveBase*>& InOutCurves) const
{
	for (const FTTEventTrack& Track : EventTracks)
	{
		InOutCurves.Add(Track.CurveKeys);
	}
	for (const FTTFloatTrack& Track : FloatTracks)
	{
		InOutCurves.Add(Track.CurveFloat);
	}
	for (const FTTVectorTrack& Track : VectorTracks)
	{
		InOutCurves.Add(Track.CurveVector);
	}
	for (const FTTLinearColorTrack& Track : LinearColorTracks)
	{
		InOutCurves.Add(Track.CurveLinearColor);
	}
}

FTTTrackId UTimelineTemplate::GetDisplayTrackId(int32 DisplayTrackIndex)
{
#if WITH_EDITORONLY_DATA
	//based on refresh of the list, we can't guarantee the display track index is still valid after a delete
	if (TrackDisplayOrder.IsValidIndex(DisplayTrackIndex))
	{
		return TrackDisplayOrder[DisplayTrackIndex];
	}
#endif
	return FTTTrackId(-1, -1);
}

int32 UTimelineTemplate::GetNumDisplayTracks() const
{
#if WITH_EDITORONLY_DATA
	return TrackDisplayOrder.Num();
#else
	return 0;
#endif
}

void UTimelineTemplate::RemoveDisplayTrack(int32 DisplayTrackIndex)
{
#if WITH_EDITORONLY_DATA
	check(TrackDisplayOrder.IsValidIndex(DisplayTrackIndex));

	FTTTrackId TrackToRemove = TrackDisplayOrder[DisplayTrackIndex];
	TrackDisplayOrder.RemoveAt(DisplayTrackIndex);

	//Adjust all other Tracks of the same type!!!
	for (int32 i = 0; i < TrackDisplayOrder.Num(); ++i)
	{
		//if this is the same type and was AFTER the removed track, adjust to the new index
		if ((TrackDisplayOrder[i].TrackType == TrackToRemove.TrackType) && (TrackDisplayOrder[i].TrackIndex > TrackToRemove.TrackIndex))
		{
			TrackDisplayOrder[i].TrackIndex--;
		}
	}
#endif
}

void UTimelineTemplate::MoveDisplayTrack(int32 DisplayTrackIndex, int32 DirectionDelta)
{
#if WITH_EDITORONLY_DATA
	check(TrackDisplayOrder.IsValidIndex(DisplayTrackIndex));
	check(TrackDisplayOrder.IsValidIndex(DisplayTrackIndex + DirectionDelta));
	TrackDisplayOrder.Swap(DisplayTrackIndex, DisplayTrackIndex + DirectionDelta);
#endif
}

void UTimelineTemplate::AddDisplayTrack(FTTTrackId NewTrackId)
{
#if WITH_EDITORONLY_DATA
	TrackDisplayOrder.Add(NewTrackId);
#endif
}



void FTTTrackBase::SetTrackName(const FName NewTrackName, UTimelineTemplate* OwningTimeline)
{
	TrackName = NewTrackName;
}

void FTTEventTrack::SetTrackName(const FName NewTrackName, UTimelineTemplate* OwningTimeline)
{
	FTTTrackBase::SetTrackName(NewTrackName, OwningTimeline);

	FunctionName = *FString::Printf(TEXT("%s__%s__EventFunc"), *OwningTimeline->GetVariableName().ToString(), *GetTrackName().ToString());
}

void FTTPropertyTrack::SetTrackName(const FName NewTrackName, UTimelineTemplate* OwningTimeline)
{
	FTTTrackBase::SetTrackName(NewTrackName, OwningTimeline);

	FString PropertyNameStr = FString::Printf(TEXT("%s_%s_%s"), *OwningTimeline->GetVariableName().ToString(), *GetTrackName().ToString(), *OwningTimeline->TimelineGuid.ToString());
	SanitizePropertyName(PropertyNameStr);
	PropertyName = *PropertyNameStr;
}

bool FTTTrackBase::operator==( const FTTTrackBase& T2 ) const
{
	return (TrackName == T2.TrackName) &&
#if WITH_EDITORONLY_DATA
		(bIsExpanded == T2.bIsExpanded) &&
		(bIsCurveViewSynchronized == T2.bIsCurveViewSynchronized) &&
#endif
		(bIsExternalCurve == T2.bIsExternalCurve);
}

bool FTTEventTrack::operator==( const FTTEventTrack& T2 ) const
{
	bool bKeyCurvesEqual = (CurveKeys == T2.CurveKeys);
	if (!bKeyCurvesEqual && CurveKeys && T2.CurveKeys)
	{
		bKeyCurvesEqual = (*CurveKeys == *T2.CurveKeys);
	}
	return bKeyCurvesEqual && FTTTrackBase::operator==(T2);
}

bool FTTFloatTrack::operator==( const FTTFloatTrack& T2 ) const
{
	bool bFloatCurvesEqual = (CurveFloat == T2.CurveFloat);
	if (!bFloatCurvesEqual && CurveFloat && T2.CurveFloat)
	{
		bFloatCurvesEqual = (*CurveFloat == *T2.CurveFloat);
	}
	return bFloatCurvesEqual && FTTTrackBase::operator==(T2);
}

bool FTTVectorTrack::operator==( const FTTVectorTrack& T2 ) const
{
	bool bVectorCurvesEqual = (CurveVector == T2.CurveVector);
	if (!bVectorCurvesEqual && CurveVector && T2.CurveVector)
	{
		bVectorCurvesEqual = (*CurveVector == *T2.CurveVector);
	}
	return FTTTrackBase::operator==(T2) && bVectorCurvesEqual;
}

bool FTTLinearColorTrack::operator==( const FTTLinearColorTrack& T2 ) const
{
	bool bColorCurvesEqual = (CurveLinearColor == T2.CurveLinearColor);
	if (!bColorCurvesEqual && CurveLinearColor && T2.CurveLinearColor)
	{
		bColorCurvesEqual = (*CurveLinearColor == *T2.CurveLinearColor);
	}
	return bColorCurvesEqual && FTTTrackBase::operator==(T2);
}


