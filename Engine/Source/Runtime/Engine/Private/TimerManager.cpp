// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TickTaskManager.cpp: Manager for ticking tasks
=============================================================================*/

#include "TimerManager.h"
#include "AutoRTFM.h"
#include "Containers/StringConv.h"
#include "Engine/GameInstance.h"
#include "Engine/TimerHandle.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "Misc/StringOutputDevice.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/StatsTrace.h"
#include "Templates/UnrealTemplate.h"
#include "UnrealEngine.h"
#include "Misc/TimeGuard.h"
#include "HAL/PlatformStackWalk.h"

DECLARE_CYCLE_STAT(TEXT("SetTimer"), STAT_SetTimer, STATGROUP_Engine);
DECLARE_CYCLE_STAT(TEXT("SetTimeForNextTick"), STAT_SetTimerForNextTick, STATGROUP_Engine);
DECLARE_CYCLE_STAT(TEXT("ClearTimer"), STAT_ClearTimer, STATGROUP_Engine);
DECLARE_CYCLE_STAT(TEXT("ClearAllTimers"), STAT_ClearAllTimers, STATGROUP_Engine);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);

/** Track the last assigned handle globally */
uint64 FTimerManager::LastAssignedSerialNumber = 0;

static float DumpTimerLogsThreshold = 0.f;
static FAutoConsoleVariableRef CVarDumpTimerLogsThreshold(
	TEXT("TimerManager.DumpTimerLogsThreshold"), DumpTimerLogsThreshold,
	TEXT("Threshold (in milliseconds) after which we log timer info to try and help track down spikes in the timer code. Disabled when set to 0"),
	ECVF_Default);

static int32 DumpTimerLogResolveVirtualFunctions = 1;
static FAutoConsoleVariableRef CVarDumpTimerLogResolveVirtualFunctions(
	TEXT("TimerManager.DumpTimerLogResolveVirtualFunctions"), DumpTimerLogResolveVirtualFunctions,
	TEXT("When logging timer info virtual functions will be resolved, if possible."),
	ECVF_Default);

static int32 DumpTimerLogSymbolNames = 1;
static FAutoConsoleVariableRef CVarDumpTimerLogSymbolNames(
	TEXT("TimerManager.DumpTimerLogSymbolNames"), DumpTimerLogSymbolNames,
	TEXT("When logging timer info, symbol names will be included if set to 1."),
	ECVF_Default);

static int32 MaxExpiredTimersToLog = 30;
static FAutoConsoleVariableRef CVarMaxExpiredTimersToLog(
	TEXT("TimerManager.MaxExpiredTimersToLog"), 
	MaxExpiredTimersToLog,
	TEXT("Maximum number of TimerData exceeding the threshold to log in a single frame."));

static int32 GuaranteeEngineTickDelay = 0;
static FAutoConsoleVariableRef CVarTimerManagerGuaranteeEngineTickDelay(
	TEXT("TimerManager.GuaranteeEngineTickDelay"),
	GuaranteeEngineTickDelay,
	TEXT("If true, timers delayed until next tick will guarantee the engine tick has advanced. If false, these could run during the same engine tick (default behavior prior to 5.5)."));

static bool bEnableTransactionalTimers = true;
static FAutoConsoleVariableRef CVarTimerManagerEnableTransactionalTimers(
	TEXT("TimerManager.EnableTransactionalTimers"),
	bEnableTransactionalTimers,
	TEXT("If false, the AutoRTFM transactional timer path will be disabled."));

#ifndef UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD
#define UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD !UE_BUILD_SHIPPING
#endif

// Verbose debug logging - especially useful for debugging transactional behavior.
#define UE_TIMER_MANAGER_ENABLE_VERBOSE_LOGGING 0
#if UE_TIMER_MANAGER_ENABLE_VERBOSE_LOGGING
static_assert(!UE_BUILD_SHIPPING);
static unsigned GTimerManagerDebugNesting = 0;
static const char * GTimerManagerDebugNestingIndentation = "                                ";
#define UE_TIMER_MANAGER_DEBUG_LOG(MSG, ...) \
	UE_LOG(LogEngine, Log, TEXT("%hs:%-4d %hs %hs " MSG), __FILE__, __LINE__, \
		AutoRTFM::IsClosed() ? "[closed]" : "[open]  ", \
		GTimerManagerDebugNestingIndentation + (32 - 2 * GTimerManagerDebugNesting), \
		##__VA_ARGS__)
#define UE_TIMER_MANAGER_DEBUG_SCOPE(MSG, ...) \
	UE_TIMER_MANAGER_DEBUG_LOG(MSG " -- begin", ##__VA_ARGS__) \
	GTimerManagerDebugNesting++; \
	ON_SCOPE_EXIT { GTimerManagerDebugNesting--; UE_TIMER_MANAGER_DEBUG_LOG(MSG " -- end", ##__VA_ARGS__); }
#else
#define UE_TIMER_MANAGER_DEBUG_LOG(MSG, ...)
#define UE_TIMER_MANAGER_DEBUG_SCOPE(MSG, ...)
#endif

#if UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD
static int32 DumpAllTimerLogsThreshold = -1;
static FAutoConsoleVariableRef CVarDumpAllTimerLogsThreshold(
	TEXT("TimerManager.DumpAllTimerLogsThreshold"),
	DumpAllTimerLogsThreshold,
	TEXT("Threshold (in count of active timers) at which to dump info about all active timers to logs. -1 means this is disabled. NOTE: This will only be dumped once per process launch."));
#endif // #if UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD


#if UE_ENABLE_TRACKING_TIMER_SOURCES
static int32 GBuildTimerSourceList = 0;
static FAutoConsoleVariableRef CVarGBuildTimerSourceList(
	TEXT("TimerManager.BuildTimerSourceList"), GBuildTimerSourceList,
	TEXT("When non-zero, tracks which timers expire each frame, dumping them during shutdown or when the flag is changed back to 0.")
	TEXT("\n  0: Off")
	TEXT("\n  1: On - Group timers by class (useful to focus on entire systems of things, especially bad spikey frames where we care about aggregates)")
	TEXT("\n  2: On - Do not group timers by class (useful if individual instances are problematic)"),
	ECVF_Default);

// Information about a single timer or timer source being tracked
struct FTimerSourceEntry
{
	uint64 TotalCount = 0;
	double SumApproxRate = 0.0;
	uint32 LastFrameID = 0;
	uint32 NumThisFrame = 0;
	uint32 MaxPerFrame = 0;

	void UpdateEntry(float Rate, uint32 FrameID, int32 CallCount)
	{
		TotalCount += CallCount;
		SumApproxRate += Rate;

		if (LastFrameID != FrameID)
		{
			LastFrameID = FrameID;
			NumThisFrame = 0;
		}
		NumThisFrame += CallCount;

		MaxPerFrame = FMath::Max(NumThisFrame, MaxPerFrame);
	}
};

// List of information about all timers that have expired during a tracking window
struct FTimerSourceList
{
	TMap<FString, FTimerSourceEntry> Entries;
	UGameInstance* OwningGameInstance = nullptr;

	// This is similar to FTimerUnifiedDelegate::ToString() but it tries to find the base class / exclude vptr printout info so that timers are collapsed/aggregated better
	static FString GetPartialDeduplicateDelegateToString(const FTimerUnifiedDelegate& Delegate)
	{
		FString FunctionNameStr = "NotBound!";
		FString ObjectNameStr = "NotBound!";

		if (const FTimerDelegate* FuncDelegate = Delegate.VariantDelegate.TryGet<FTimerDelegate>())
		{
			if (FuncDelegate->IsBound())
			{
				ObjectNameStr = TEXT("NonDynamicDelegate");
				FName FunctionName;
#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME
				FunctionName = FuncDelegate->TryGetBoundFunctionName();
#endif
				if (FunctionName.IsNone())
				{
					uint64 ProgramCounter = FuncDelegate->GetBoundProgramCounterForTimerManager();
					if (ProgramCounter != 0)
					{
						// Add the function address
						if (DumpTimerLogSymbolNames)
						{
							// Try to resolve the function address to a symbol
							FProgramCounterSymbolInfo SymbolInfo;
							FPlatformStackWalk::ProgramCounterToSymbolInfo(ProgramCounter, /*out*/ SymbolInfo);
							FunctionNameStr = FString::Printf(TEXT("%s [%s:%d]"), ANSI_TO_TCHAR(SymbolInfo.FunctionName), ANSI_TO_TCHAR(SymbolInfo.Filename), SymbolInfo.LineNumber);
						}
						else
						{
							FunctionNameStr = FString::Printf(TEXT("func: 0x%llx"), ProgramCounter);
						}
					}
					else
					{
						FunctionNameStr = TEXT("0x0");
					}
				}
				else
				{
					FunctionNameStr = FunctionName.ToString();
				}
			}
		}
		else if (const FTimerDynamicDelegate* FuncDynDelegate = Delegate.VariantDelegate.TryGet<FTimerDynamicDelegate>())
		{
			if (FuncDynDelegate->IsBound())
			{
				const FName FuncFName = FuncDynDelegate->GetFunctionName();
				FunctionNameStr = FuncFName.ToString();

				UClass* SourceClass = nullptr;
				if (const UObject* Object = FuncDynDelegate->GetUObject())
				{
					SourceClass = Object->GetClass();
					if (UFunction* Func = SourceClass->FindFunctionByName(FuncFName))
					{
						SourceClass = Func->GetOwnerClass();
					}
				}

				ObjectNameStr = GetPathNameSafe(SourceClass);
			}
		}
		else if (const FTimerFunction* TimerFunction = Delegate.VariantDelegate.TryGet<FTimerFunction>())
		{
			if (*TimerFunction)
			{
				FunctionNameStr = TEXT("TFunction");
				ObjectNameStr = TEXT("");	
			}
		}

		return FString::Printf(TEXT("%s,\"%s\""), *ObjectNameStr, *FunctionNameStr);
	}

	void AddEntry(const FTimerData& Data, int32 CallCount)
	{
		Entries.FindOrAdd((GBuildTimerSourceList == 2) ? Data.TimerDelegate.ToString() : GetPartialDeduplicateDelegateToString(Data.TimerDelegate)).UpdateEntry(Data.Rate, GFrameCounter, CallCount);
	}

	void DumpEntries()
	{
		FString AdditionalInfoAboutGameInstance;
		if (OwningGameInstance != nullptr)
		{
			if (FWorldContext* WorldContext = OwningGameInstance->GetWorldContext())
			{
				if (WorldContext->RunAsDedicated)
				{
					AdditionalInfoAboutGameInstance = TEXT(" (dedicated server)");
				}
			}
		}

		UE_LOGF(LogEngine, Log, "-----------------------");
		UE_LOGF(LogEngine, Log, "Listing Expired Timer Stats for OwningGameInstance=%ls%ls", *GetPathNameSafe(OwningGameInstance), *AdditionalInfoAboutGameInstance);
		
		UE_LOGF(LogEngine, Log, "TotalExpires,RateAvg,PeakInFrame,ObjectOrClass,FunctionName");
		Entries.ValueSort([](const FTimerSourceEntry& A, const FTimerSourceEntry& B) { return A.TotalCount > B.TotalCount; });
		for (const auto& KVP : Entries)
		{
			UE_LOGF(LogEngine, Log, "%llu,%.2f,%u,%ls", KVP.Value.TotalCount, KVP.Value.SumApproxRate / KVP.Value.TotalCount, KVP.Value.MaxPerFrame, *KVP.Key);
		}

		UE_LOGF(LogEngine, Log, "-----------------------");
	}

	~FTimerSourceList()
	{
		DumpEntries();
	}
};
#endif // UE_ENABLE_TRACKING_TIMER_SOURCES

namespace
{
	void DescribeFTimerDataSafely(FOutputDevice& Ar, const FTimerData& Data)
	{
		Ar.Logf(
			TEXT("TimerData %p : bLoop=%s, bRequiresDelegate=%s, Status=%d, Rate=%f, ExpireTime=%f, Delegate=%s"),
			&Data,
			Data.bLoop ? TEXT("true") : TEXT("false"),
			Data.bRequiresDelegate ? TEXT("true") : TEXT("false"),
			static_cast<int32>(Data.Status),
			Data.Rate,
			Data.ExpireTime,
			*Data.TimerDelegate.ToString()
		);
	}

	FString GetFTimerDataSafely(const FTimerData& Data)
	{
		FStringOutputDevice Output;
		DescribeFTimerDataSafely(Output, Data);
		return MoveTemp(Output);
	}

	#if UE_WITH_REMOTE_OBJECT_HANDLE
	/**
	* Returns true if the TimerData target is locally resident. When a target is locally resident, you can
	* execute the TimerDelegate without causing (an initial) migration. The delegate itself may cause other migrations.
	*/
	bool IsTimerTargetResident(const FTimerUnifiedDelegate& TimerDelegate)
	{
		using namespace UE::RemoteObject::Handle;

		FRemoteObjectId RemoteObjectId = UE::RemoteObject::Handle::GetRemoteObjectId([&TimerDelegate]()
		{
			const UObject* TargetObject;
			switch (TimerDelegate.VariantDelegate.GetIndex())
			{
				case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
					TargetObject = TimerDelegate.VariantDelegate.Get<FTimerDelegate>().GetUObject();
					break;

				case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
					TargetObject = TimerDelegate.VariantDelegate.Get<FTimerDynamicDelegate>().GetUObject();
					break;

				case FTimerDelegateVariant::IndexOfType<FTimerFunction>():
					// fallthrough -- TFunction does not know about an underlying UObject, so we always consider it local.
				default:
					TargetObject = nullptr;
					break;
			}

			return TargetObject;
		});

		return GetResidence(RemoteObjectId) == EResidence::Local;
	}
	#endif
}

struct FTimerHeapOrder
{
	explicit FTimerHeapOrder(const TSparseArray<FTimerData>& InTimers)
		: Timers(InTimers)
		, NumTimers(InTimers.Num())
	{
	}

	bool operator()(FTimerHandle LhsHandle, FTimerHandle RhsHandle) const
	{
		int32 LhsIndex = LhsHandle.GetIndex();
		int32 RhsIndex = RhsHandle.GetIndex();

		const FTimerData& LhsData = Timers[LhsIndex];
		const FTimerData& RhsData = Timers[RhsIndex];

		return LhsData.ExpireTime < RhsData.ExpireTime;
	}

	const TSparseArray<FTimerData>& Timers;
	int32 NumTimers;
};

FTimerManager::FTimerManager(UGameInstance* GameInstance)
	: InternalTime(0.0)
	, LastTickedFrame(static_cast<uint64>(-1))
	, OwningGameInstance(nullptr)
{
	if (IsRunningDedicatedServer())
	{
		// Off by default, reenable if needed
		//FCoreDelegates::OnHandleSystemError.AddRaw(this, &FTimerManager::OnCrash);
	}

	if (GameInstance)
	{
		SetGameInstance(GameInstance);
	}

	FModuleManager::Get().OnLibraryUnmount().AddRaw(this, &FTimerManager::RemoveExpiredTimers);
}

FTimerManager::~FTimerManager()
{
	FModuleManager::Get().OnLibraryUnmount().RemoveAll(this);

	if (IsRunningDedicatedServer())
	{
		FCoreDelegates::OnHandleSystemError.RemoveAll(this);
	}
}

void FTimerManager::OnCrash()
{
	UE_LOGF(LogEngine, Warning, "TimerManager %p on crashing delegate called, dumping extra information", this);

	UE_LOGF(LogEngine, Log, "------- %d Active Timers (including expired) -------", ActiveTimerHeap.Num());
	int32 ExpiredActiveTimerCount = 0;
	for (FTimerHandle Handle : ActiveTimerHeap)
	{
		const FTimerData& Timer = GetTimer(Handle);
		if (Timer.Status == ETimerStatus::ActivePendingRemoval)
		{
			++ExpiredActiveTimerCount;
		}
		else
		{
			DescribeFTimerDataSafely(*GLog, Timer);
		}
	}
	UE_LOGF(LogEngine, Log, "------- %d Expired Active Timers -------", ExpiredActiveTimerCount);

	UE_LOGF(LogEngine, Log, "------- %d Paused Timers -------", PausedTimerSet.Num());
	for (FTimerHandle Handle : PausedTimerSet)
	{
		const FTimerData& Timer = GetTimer(Handle);
		DescribeFTimerDataSafely(*GLog, Timer);
	}

	UE_LOGF(LogEngine, Log, "------- %d Pending Timers -------", PendingTimerSet.Num());
	for (FTimerHandle Handle : PendingTimerSet)
	{
		const FTimerData& Timer = GetTimer(Handle);
		DescribeFTimerDataSafely(*GLog, Timer);
	}

	UE_LOGF(LogEngine, Log, "------- %d Total Timers -------", PendingTimerSet.Num() + PausedTimerSet.Num() + ActiveTimerHeap.Num() - ExpiredActiveTimerCount);

	UE_LOGF(LogEngine, Warning, "TimerManager %p dump ended", this);
}

void FTimerUnifiedDelegate::Execute() const
{
	switch (VariantDelegate.GetIndex())
	{
	case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
		{
			const FTimerDelegate& FuncDelegate = VariantDelegate.Get<FTimerDelegate>();
			if (FuncDelegate.IsBound())
			{
				FScopeCycleCounterUObject Context(FuncDelegate.GetUObject());
				FuncDelegate.Execute();
			}
			break;
		}
	case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
		{
			const FTimerDynamicDelegate& FuncDynDelegate = VariantDelegate.Get<FTimerDynamicDelegate>();
			if (FuncDynDelegate.IsBound())
			{
				// stat scope is handled by UObject::ProcessEvent for the UFunction.
				FuncDynDelegate.ProcessDelegate<UObject>(nullptr);
			}
			break;
		}
	case FTimerDelegateVariant::IndexOfType<FTimerFunction>():
		{
			if (const FTimerFunction& TimerFunction = VariantDelegate.Get<FTimerFunction>())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FTimerUnifiedDelegate_Execute);
				TimerFunction();
			}
			break;
		}
	default:
		break;
	}
}

bool FTimerUnifiedDelegate::IsBound() const
{
	switch (VariantDelegate.GetIndex())
	{
	case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
		return VariantDelegate.Get<FTimerDelegate>().IsBound();
	case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
		return VariantDelegate.Get<FTimerDynamicDelegate>().IsBound();
	case FTimerDelegateVariant::IndexOfType<FTimerFunction>():
		return VariantDelegate.Get<FTimerFunction>()!=nullptr;
	default:
		return false;
	}
}

const void* FTimerUnifiedDelegate::GetBoundObject() const
{
	switch (VariantDelegate.GetIndex())
	{
	case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
		{
			const FTimerDelegate& FuncDelegate = VariantDelegate.Get<FTimerDelegate>();
			if (FuncDelegate.IsBound())
			{
				return FuncDelegate.GetObjectForTimerManager();
			}
			break;
		}
	case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
		{
			const FTimerDynamicDelegate& FuncDynDelegate = VariantDelegate.Get<FTimerDynamicDelegate>();
			if (FuncDynDelegate.IsBound())
			{
				return FuncDynDelegate.GetUObject();
			}
			break;
		}
	default:
		break;
	}
	return nullptr;
}

FString FTimerUnifiedDelegate::ToString() const
{
	switch (VariantDelegate.GetIndex())
	{
	case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
		{
			const FTimerDelegate& FuncDelegate = VariantDelegate.Get<FTimerDelegate>();
			if (FuncDelegate.IsBound())
			{
				FString FunctionNameStr;
				FName FunctionName;
#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME
				FunctionName = FuncDelegate.TryGetBoundFunctionName();
#endif
				if (FunctionName.IsNone())
				{
					void** VtableAddr = nullptr;
#if PLATFORM_COMPILER_CLANG || defined(_MSC_VER)
					// Add the vtable address
					const void* UserObject = FuncDelegate.GetObjectForTimerManager();
					if (UserObject)
					{
						VtableAddr = *(void***)UserObject;
						FunctionNameStr = FString::Printf(TEXT("vtbl: %p"), VtableAddr);
					}
#endif // PLATFORM_COMPILER_CLANG

					uint64 ProgramCounter = FuncDelegate.GetBoundProgramCounterForTimerManager();
					if (ProgramCounter != 0)
					{
						// Add the function address

#if PLATFORM_COMPILER_CLANG
						// See if this is a virtual function. Heuristic is that real function addresses are higher than some value, and vtable offsets are lower
						const uint64 MaxVTableAddressOffset = 32768;
						if (DumpTimerLogResolveVirtualFunctions && VtableAddr && ProgramCounter > 0 && ProgramCounter < MaxVTableAddressOffset)
						{
							// If the ProgramCounter is just an offset to the vtable (virtual member function) then resolve the actual ProgramCounter here.
							ProgramCounter = (uint64)VtableAddr[ProgramCounter / sizeof(void*)];
						}
#endif // PLATFORM_COMPILER_CLANG

						FunctionNameStr += FString::Printf(TEXT(" func: 0x%llx"), ProgramCounter);

						if (DumpTimerLogSymbolNames)
						{
							// Try to resolve the function address to a symbol
							FProgramCounterSymbolInfo SymbolInfo;
							SymbolInfo.FunctionName[0] = 0;
							SymbolInfo.Filename[0] = 0;
							SymbolInfo.LineNumber = 0;
							FPlatformStackWalk::ProgramCounterToSymbolInfo(ProgramCounter, SymbolInfo);
							FunctionNameStr += FString::Printf(TEXT(" %s [%s:%d]"), ANSI_TO_TCHAR(SymbolInfo.FunctionName), ANSI_TO_TCHAR(SymbolInfo.Filename), SymbolInfo.LineNumber);
						}
					}
					else
					{
						FunctionNameStr = TEXT(" 0x0");
					}
				}

				const UObject* const Object = FuncDelegate.GetUObject();
				return FString::Printf(TEXT("DELEGATE,%s,%s"), Object == nullptr ? TEXT("NO OBJ") : *Object->GetPathName(), *FunctionNameStr);
			}
			return TEXT("UNBOUND DELEGATE");
		}
	case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
		{
			const FTimerDynamicDelegate& FuncDynDelegate = VariantDelegate.Get<FTimerDynamicDelegate>();
			if (FuncDynDelegate.IsBound())
			{
				const UObject* Object = FuncDynDelegate.GetUObject();
				return FString::Printf(TEXT("DYN DELEGATE,%s,%s"), Object == nullptr ? TEXT("NO OBJ") : *Object->GetPathName(), *FuncDynDelegate.GetFunctionName().ToString());
			}
			return TEXT("UNBOUND DYN DELEGATE");
		}
	case FTimerDelegateVariant::IndexOfType<FTimerFunction>():
		{
			if (const FTimerFunction& TimerFunction = VariantDelegate.Get<FTimerFunction>())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FTimerUnifiedDelegate_Execute);
				return TEXT("TFUNCTION");
			}
			return TEXT("UNBOUND TFUNCTION");
		}
	default:
		return TEXT("UNSET");
	}
}

// ---------------------------------
// Private members
// ---------------------------------

FTimerData& FTimerManager::GetTimer(FTimerHandle const& InHandle)
{
	int32 Index = InHandle.GetIndex();
	checkSlow(Index >= 0 && Index < Timers.GetMaxIndex() && Timers.IsAllocated(Index) && Timers[Index].Handle == InHandle);
	FTimerData& Timer = Timers[Index];
	return Timer;
}

FTimerData* FTimerManager::FindTimer(FTimerHandle const& InHandle)
{
	// not currently threadsafe
	check(IsInGameThread());

	if (!InHandle.IsValid())
	{
		return nullptr;
	}

	int32 Index = InHandle.GetIndex();
	if (Index < 0 || Index >= Timers.GetMaxIndex() || !Timers.IsAllocated(Index))
	{
		return nullptr;
	}

	FTimerData& Timer = Timers[Index];

	if (Timer.Handle != InHandle || Timer.Status == ETimerStatus::ActivePendingRemoval)
	{
		return nullptr;
	}

	return &Timer;
}

/** Finds a handle to a dynamic timer bound to a particular pointer and function name. */
FTimerHandle FTimerManager::K2_FindDynamicTimerHandle(FTimerDynamicDelegate InDynamicDelegate) const
{
	FTimerHandle Result;

	if (const void* Obj = InDynamicDelegate.GetUObject())
	{
		if (const TSet<FTimerHandle>* TimersForObject = ObjectToTimers.Find(Obj))
		{
			for (FTimerHandle Handle : *TimersForObject)
			{
				const FTimerData& Data = GetTimer(Handle);
				if (Data.Status == ETimerStatus::ActivePendingRemoval)
				{
					continue;
				}
				const FTimerDynamicDelegate* DynDelegate = Data.TimerDelegate.VariantDelegate.TryGet<FTimerDynamicDelegate>();
				if (DynDelegate && *DynDelegate == InDynamicDelegate)
				{
					Result = Handle;
					break;
				}
			}
		}
	}

	return Result;
}

void FTimerManager::InternalSetTimer(FTimerHandle& InOutHandle, FTimerUnifiedDelegate&& InDelegate, float InRate, bool bInLoop, float InFirstDelay)
{
	InternalSetTimer(InOutHandle, MoveTemp(InDelegate), InRate, FTimerManagerTimerParameters{ .bLoop = bInLoop, .FirstDelay = InFirstDelay });
}

void FTimerManager::InternalSetTimer(FTimerHandle& InOutHandle, FTimerUnifiedDelegate&& InDelegate, float InRate, const FTimerManagerTimerParameters& InTimerParameters)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("InternalSetTimer(InOutHandle: %s, InDelegate: %s)", *InOutHandle.ToString(), *InDelegate.ToString());

	SCOPE_CYCLE_COUNTER(STAT_SetTimer);

	// not currently threadsafe
	check(IsInGameThread());

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	ensureAlwaysMsgf(!InDelegate.VariantDelegate.IsType<FTimerFunction>(), TEXT("Non-UObject Delegates (TFunctions) are not safe with DSTM. Use FTimerDelegate::CreateUObject and don't capture raw pointers."));
#endif

	if (FindTimer(InOutHandle))
	{
		// if the timer is already set, just clear it and we'll re-add it, since 
		// there's no data to maintain.
		InternalClearTimer(InOutHandle);
	}

	if (InRate > 0.f)
	{
		// set up the new timer
		FTimerData NewTimerData;
		NewTimerData.TimerDelegate = MoveTemp(InDelegate);

		NewTimerData.Rate = InRate;
		NewTimerData.bLoop = InTimerParameters.bLoop;
		NewTimerData.bMaxOncePerFrame = InTimerParameters.bMaxOncePerFrame;
		NewTimerData.bRequiresDelegate = NewTimerData.TimerDelegate.IsBound();

		// Set level collection
		const UWorld* const OwningWorld = OwningGameInstance ? OwningGameInstance->GetWorld() : nullptr;
		if (OwningWorld && OwningWorld->GetActiveLevelCollection())
		{
			NewTimerData.LevelCollection = OwningWorld->GetActiveLevelCollection()->GetType();
		}

		const float FirstDelay = (InTimerParameters.FirstDelay >= 0.f) ? InTimerParameters.FirstDelay : InRate;

		FTimerHandle NewTimerHandle;
#if UE_AUTORTFM
		if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
		{
			// Reserve a slot in Timers for this transactional timer.
			NewTimerHandle = GenerateReservedHandleForTransaction();

			// Add the timer to TransactionalTimers.
			NewTimerData.ExpireTime = FirstDelay;
			NewTimerData.Handle = NewTimerHandle;
			TransactionalTimers.Add(NewTimerHandle, FTransactionalTimer
				{
					.NewData = MoveTemp(NewTimerData),
					.Status = HasBeenTickedThisFrame() ? FTransactionalTimer::Active : FTransactionalTimer::Pending,
				});
			MaybeRegisterTransactionalCallbacks();
		}
		else
#endif  // UE_AUTORTFM
		if (HasBeenTickedThisFrame())
		{
			NewTimerData.ExpireTime = InternalTime + FirstDelay;
			NewTimerData.Status = ETimerStatus::Active;
			NewTimerHandle = AddTimer(MoveTemp(NewTimerData));
			ActiveTimerHeap.HeapPush(NewTimerHandle, FTimerHeapOrder(Timers));
		}
		else
		{
			// Store time remaining in ExpireTime while pending
			NewTimerData.ExpireTime = FirstDelay;
			NewTimerData.Status = ETimerStatus::Pending;
			NewTimerHandle = AddTimer(MoveTemp(NewTimerData));
			PendingTimerSet.Add(NewTimerHandle);
		}

		InOutHandle = NewTimerHandle;
	}
	else
	{
		InOutHandle.Invalidate();
	}
}

FTimerHandle FTimerManager::InternalSetTimerForNextTick(FTimerUnifiedDelegate&& InDelegate)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("InternalSetTimerForNextTick(InDelegate: %s)", *InDelegate.ToString());

	SCOPE_CYCLE_COUNTER(STAT_SetTimerForNextTick);

	// not currently threadsafe
	check(IsInGameThread());

	FTimerData NewTimerData;
	NewTimerData.Rate = 0.f;
	NewTimerData.bLoop = false;
	NewTimerData.bRequiresDelegate = true;
	NewTimerData.TimerDelegate = MoveTemp(InDelegate);

	// Set level collection
	const UWorld* const OwningWorld = OwningGameInstance ? OwningGameInstance->GetWorld() : nullptr;
	if (OwningWorld && OwningWorld->GetActiveLevelCollection())
	{
		NewTimerData.LevelCollection = OwningWorld->GetActiveLevelCollection()->GetType();
	}

	// Add to pending timer heap to guarantee it happens next frame if we want to delay
	const bool bQueueForCurrentFrame = (GuaranteeEngineTickDelay == 0) || HasBeenTickedThisFrame();
	
	FTimerHandle NewTimerHandle;
	
#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
	{
		// Reserve a slot in Timers for this transactional timer.
		NewTimerHandle = GenerateReservedHandleForTransaction();
		NewTimerData.Handle = NewTimerHandle;

		// Add the timer to TransactionalTimers.
		TransactionalTimers.FindOrAdd(NewTimerHandle, FTransactionalTimer
			{
				.NewData = MoveTemp(NewTimerData),
				.Status = bQueueForCurrentFrame ? FTransactionalTimer::Active : FTransactionalTimer::Pending,
			});
		MaybeRegisterTransactionalCallbacks();
	}
	else
#endif  // UE_AUTORTFM
	if (bQueueForCurrentFrame)
	{
		NewTimerData.Status = ETimerStatus::Active;
		NewTimerData.ExpireTime = InternalTime;
		NewTimerHandle = AddTimer(MoveTemp(NewTimerData));
		ActiveTimerHeap.HeapPush(NewTimerHandle, FTimerHeapOrder(Timers));
	}
	else
	{
		NewTimerData.Status = ETimerStatus::Pending;
		NewTimerHandle = AddTimer(MoveTemp(NewTimerData));
		PendingTimerSet.Add(NewTimerHandle);
	}

	UE_TIMER_MANAGER_DEBUG_LOG("NewTimerHandle: %s", *NewTimerHandle.ToString());
	return NewTimerHandle;
}

void FTimerManager::InternalClearTimer(FTimerHandle InHandle)
{
	UE_TIMER_MANAGER_DEBUG_LOG("InternalClearTimer(InHandle: %s)", *InHandle.ToString());

	SCOPE_CYCLE_COUNTER(STAT_ClearTimer);

	// not currently threadsafe
	check(IsInGameThread());

#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
	{
		// Add or replace the transactional timer with the canceled status.
		FTransactionalTimer& Timer = TransactionalTimers.FindOrAdd(InHandle);
		Timer.NewData.Reset();
		Timer.Status = FTransactionalTimer::Canceled;
		MaybeRegisterTransactionalCallbacks();
		return;
	}
#endif  // UE_AUTORTFM

	FTimerData& Data = GetTimer(InHandle);
	UE_TIMER_MANAGER_DEBUG_LOG("Status: %d", static_cast<int>(Data.Status));
	switch (Data.Status)
	{
		case ETimerStatus::Pending:
			{
				int32 NumRemoved = PendingTimerSet.Remove(InHandle);
				check(NumRemoved == 1);
				RemoveTimer(InHandle);
			}
			break;

		case ETimerStatus::Active:
			Data.Status = ETimerStatus::ActivePendingRemoval;
			break;

		case ETimerStatus::ActivePendingRemoval:
			// Already removed
			break;

		case ETimerStatus::Paused:
			{
				int32 NumRemoved = PausedTimerSet.Remove(InHandle);
				check(NumRemoved == 1);
				RemoveTimer(InHandle);
			}
			break;

		case ETimerStatus::Executing:
			check(CurrentlyExecutingTimer == InHandle);

			// Edge case. We're currently handling this timer when it got cleared.  Clear it to prevent it firing again
			// in case it was scheduled to fire multiple times.
			CurrentlyExecutingTimer.Invalidate();
			RemoveTimer(InHandle);
			break;

		case ETimerStatus::ReservedForTransaction:
			// Only the on-commit or on-abort handler may remove a
			// transactionally-reserved timer.
			check(AutoRTFM::IsCommitting() || AutoRTFM::IsAborting());
			RemoveTimer(InHandle);
			break;
			
		default:
			check(false);
	}
}

void FTimerManager::InternalClearAllTimers(void const* Object)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("InternalClearAllTimers(Object: %p)", Object);

	SCOPE_CYCLE_COUNTER(STAT_ClearAllTimers);

	if (!Object)
	{
		return;
	}

#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
	{
		for (TTuple<FTimerHandle, FTransactionalTimer>& It : TransactionalTimers)
		{
			if (TOptional<FTimerData>& NewData = It.Value.NewData; NewData.IsSet())
			{
				if (NewData->TimerDelegate.GetBoundObject() == Object)
				{
					It.Value.NewData.Reset();
					It.Value.Status = FTransactionalTimer::Canceled;
				}
			}
		}
	}
#endif  // UE_AUTORTFM

	TSet<FTimerHandle>* TimersToRemove = ObjectToTimers.Find(Object);
	if (!TimersToRemove)
	{
		return;
	}

	TSet<FTimerHandle> LocalTimersToRemove = *TimersToRemove;
	for (FTimerHandle TimerToRemove : LocalTimersToRemove)
	{
		InternalClearTimer(TimerToRemove);
	}
}

float FTimerManager::InternalGetTimerRemaining(FTimerData const* const TimerData) const
{
	if (TimerData)
	{
		switch (TimerData->Status)
		{
			case ETimerStatus::Active:
				return TimerData->ExpireTime - InternalTime;

			case ETimerStatus::Executing:
				return 0.0f;

			case ETimerStatus::ReservedForTransaction:
				break;

			default:
				// ExpireTime is time remaining for paused timers
				return TimerData->ExpireTime;
		}
	}

	return -1.f;
}

float FTimerManager::InternalGetTimerElapsed(FTimerData const* const TimerData) const
{
	if (TimerData)
	{
		switch (TimerData->Status)
		{
			case ETimerStatus::Active:
			case ETimerStatus::Executing:
				return (TimerData->Rate - (TimerData->ExpireTime - InternalTime));

			default:
				// ExpireTime is time remaining for paused timers
				return (TimerData->Rate - TimerData->ExpireTime);
		}
	}

	return -1.f;
}

float FTimerManager::InternalGetTimerRate(FTimerData const* const TimerData) const
{
	if (TimerData)
	{
		return TimerData->Rate;
	}
	return -1.f;
}

void FTimerManager::PauseTimer(FTimerHandle InHandle)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("PauseTimer(InHandle: %s)", *InHandle.ToString());

	// not currently threadsafe
	check(IsInGameThread());

#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
	{
		// Add or replace the transactional timer with the paused status (if it
		// didn't exist with a canceled status).
		FTransactionalTimer& Timer = TransactionalTimers.FindOrAdd(InHandle);
		if (Timer.Status != FTransactionalTimer::Canceled)
		{
			Timer.Status = FTransactionalTimer::Paused;
		}
		MaybeRegisterTransactionalCallbacks();
		return;
	}
#endif  // UE_AUTORTFM

	FTimerData* TimerToPause = FindTimer(InHandle);
	if (!TimerToPause || TimerToPause->Status == ETimerStatus::Paused)
	{
		return;
	}

	ETimerStatus PreviousStatus = TimerToPause->Status;
	UE_TIMER_MANAGER_DEBUG_LOG("Status: %d", PreviousStatus);

	// Remove from previous TArray
	switch( PreviousStatus )
	{
		case ETimerStatus::ActivePendingRemoval:
			break;

		case ETimerStatus::Active:
			{
				int32 IndexIndex = ActiveTimerHeap.Find(InHandle);
				check(IndexIndex != INDEX_NONE);
				ActiveTimerHeap.HeapRemoveAt(IndexIndex, FTimerHeapOrder(Timers), EAllowShrinking::No);
			}
			break;

		case ETimerStatus::Pending:
			{
				int32 NumRemoved = PendingTimerSet.Remove(InHandle);
				check(NumRemoved == 1);
			}
			break;

		case ETimerStatus::Executing:
			check(CurrentlyExecutingTimer == InHandle);

			CurrentlyExecutingTimer.Invalidate();
			break;

		default:
			check(false);
	}

	// Don't pause the timer if it's currently executing and isn't going to loop
	if( PreviousStatus == ETimerStatus::Executing && !TimerToPause->bLoop )
	{
		RemoveTimer(InHandle);
	}
	else
	{
		// Add to Paused list
		PausedTimerSet.Add(InHandle);

		// Set new status
		TimerToPause->Status = ETimerStatus::Paused;

		// Store time remaining in ExpireTime while paused. Don't do this if the timer is in the pending list.
		if (PreviousStatus != ETimerStatus::Pending)
		{
			TimerToPause->ExpireTime -= InternalTime;
		}
	}
}

void FTimerManager::UnPauseTimer(FTimerHandle InHandle)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("UnPauseTimer(InHandle: %s)", *InHandle.ToString());

	// not currently threadsafe
	check(IsInGameThread());

#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsClosed())
	{
		// Add or replace the transactional timer with the pending status (if it
		// didn't exist with a canceled status).
		FTransactionalTimer& Timer = TransactionalTimers.FindOrAdd(InHandle);
		if (Timer.Status != FTransactionalTimer::Canceled)
		{
			Timer.Status = HasBeenTickedThisFrame() ? FTransactionalTimer::Active : FTransactionalTimer::Pending;
		}
		MaybeRegisterTransactionalCallbacks();
		return;
	}
#endif  // UE_AUTORTFM
	

	FTimerData* TimerToUnPause = FindTimer(InHandle);
	if (!TimerToUnPause || TimerToUnPause->Status != ETimerStatus::Paused)
	{
		return;
	}

	// Move it out of paused list and into proper TArray
	if( HasBeenTickedThisFrame() )
	{
		// Convert from time remaining back to a valid ExpireTime
		TimerToUnPause->ExpireTime += InternalTime;
		TimerToUnPause->Status = ETimerStatus::Active;
		ActiveTimerHeap.HeapPush(InHandle, FTimerHeapOrder(Timers));
	}
	else
	{
		TimerToUnPause->Status = ETimerStatus::Pending;
		PendingTimerSet.Add(InHandle);
	}

	// remove from paused list
	PausedTimerSet.Remove(InHandle);
}

FTimerData::FTimerData()
	: bLoop(false)
	, bMaxOncePerFrame(false)
	, bRequiresDelegate(false)
	, Status(ETimerStatus::Active)
	, LevelCollection(ELevelCollectionType::DynamicSourceLevels)
	, Rate(0)
	, ExpireTime(0)
{}

// ---------------------------------
// Public members
// ---------------------------------

DECLARE_DWORD_COUNTER_STAT(TEXT("TimerManager Heap Size"),STAT_NumHeapEntries,STATGROUP_Game);

void FTimerManager::Tick(float DeltaTime)
{
	AutoRTFM::UnreachableIfClosed("FTimerManager::Tick cannot be called from the closed");

	SCOPED_NAMED_EVENT(FTimerManager_Tick, FColor::Orange);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(TimerManager);
	UE_TIMER_MANAGER_DEBUG_SCOPE("FTimerManager::Tick");

#if DO_TIMEGUARD && 0
	TArray<FTimerUnifiedDelegate> RunTimerDelegates;
	FTimerNameDelegate NameFunction = FTimerNameDelegate::CreateLambda([&] {
			FString ActiveDelegates;
			for ( const FTimerUnifiedDelegate& Descriptor : RunTimerDelegates )
			{
				ActiveDelegates += FString::Printf(TEXT("Delegate %s, "), *Descriptor.ToString() );
			}
			return FString::Printf(TEXT("UWorld::Tick - TimerManager, %s"), *ActiveDelegates);
		});


	// no delegate should take longer then 5ms to run 
	SCOPE_TIME_GUARD_DELEGATE_MS(NameFunction, 5);
#endif	

	// @todo, might need to handle long-running case
	// (e.g. every X seconds, renormalize to InternalTime = 0)

	INC_DWORD_STAT_BY(STAT_NumHeapEntries, ActiveTimerHeap.Num());

	if (HasBeenTickedThisFrame())
	{
		return;
	}

#if UE_ENABLE_TRACKING_TIMER_SOURCES
	if ((TimerSourceList == nullptr) != (GBuildTimerSourceList == 0))
	{
		if (TimerSourceList == nullptr)
		{
			TimerSourceList.Reset(new FTimerSourceList);
			TimerSourceList->OwningGameInstance = OwningGameInstance;
		}
		else
		{
			TimerSourceList.Reset();
		}
	}
#endif

	const double StartTime = FPlatformTime::Seconds();
	bool bDumpTimerLogsThresholdExceeded = false;
	int32 NbExpiredTimers = 0;

	InternalTime += DeltaTime;

	UWorld* const OwningWorld = OwningGameInstance ? OwningGameInstance->GetWorld() : nullptr;

#if UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD
	// Dump timer info to logs if we have way too many timers active.
	UE_SUPPRESS(LogEngine, Warning,
	{
		if (DumpAllTimerLogsThreshold > 0 && ActiveTimerHeap.Num() > DumpAllTimerLogsThreshold)
		{
			static bool bAlreadyLogged = false;
			if(!bAlreadyLogged)
			{
				bAlreadyLogged = true;
			
				UE_LOGF(LogEngine, Warning, "Number of active Timers (%d) has exceeded DumpAllTimerLogsThreshold (%d)!  Dumping all timer info to log:", ActiveTimerHeap.Num(), DumpAllTimerLogsThreshold);

				TArray<const FTimerData*> ValidActiveTimers;
				ValidActiveTimers.Reserve(ActiveTimerHeap.Num());
				for (FTimerHandle Handle : ActiveTimerHeap)
				{
					if (const FTimerData* Data = FindTimer(Handle))
					{
						ValidActiveTimers.Add(Data);
					}
				}

				for (const FTimerData* Data : ValidActiveTimers)
				{
					DescribeFTimerDataSafely(*GLog, *Data);
				}
			}
		}
	});
#endif // #if UE_ENABLE_DUMPALLTIMERLOGSTHRESHOLD

	while (ActiveTimerHeap.Num() > 0)
	{
		FTimerHandle TopHandle = ActiveTimerHeap.HeapTop();

		// Test for expired timers
		int32 TopIndex = TopHandle.GetIndex();
		FTimerData* Top = &Timers[TopIndex];

		if (Top->Status == ETimerStatus::ActivePendingRemoval)
		{
			UE_TIMER_MANAGER_DEBUG_LOG("Removing top timer (%s) with status ActivePendingRemoval", *TopHandle.ToString());
			ActiveTimerHeap.HeapPop(TopHandle, FTimerHeapOrder(Timers), EAllowShrinking::No);
			RemoveTimer(TopHandle);
			continue;
		}

		if (InternalTime > Top->ExpireTime)
		{
			// Timer has expired! Fire the delegate, then handle potential looping.

			if (bDumpTimerLogsThresholdExceeded)
			{
				++NbExpiredTimers;
				if (NbExpiredTimers <= MaxExpiredTimersToLog)
				{
					DescribeFTimerDataSafely(*GLog, *Top);
				}
			}

			// Set the relevant level context for this timer
			const int32 LevelCollectionIndex = OwningWorld ? OwningWorld->FindCollectionIndexByType(Top->LevelCollection) : INDEX_NONE;
			
			FScopedLevelCollectionContextSwitch LevelContext(LevelCollectionIndex, OwningWorld);

			// Remove it from the heap and store it while we're executing
			ActiveTimerHeap.HeapPop(CurrentlyExecutingTimer, FTimerHeapOrder(Timers), EAllowShrinking::No);
			Top->Status = ETimerStatus::Executing;

			// Determine how many times the timer may have elapsed (e.g. for large DeltaTime on a short looping timer)
			int32 const CallCount = Top->bLoop ? 
				FMath::TruncToInt( (InternalTime - Top->ExpireTime) / Top->Rate ) + 1
				: 1;

#if UE_ENABLE_TRACKING_TIMER_SOURCES
			if (TimerSourceList.IsValid())
			{
				//@TODO: The actual call count may be less, e.g., if the delegate clears itself during the loop below
				TimerSourceList->AddEntry(*Top, CallCount);
			}
#endif

			// Now call the function
			for (int32 CallIdx=0; CallIdx<CallCount; ++CallIdx)
			{ 
#if DO_TIMEGUARD && 0
				FTimerNameDelegate NameFunction = FTimerNameDelegate::CreateLambda([&] { 
						return FString::Printf(TEXT("FTimerManager slowtick from delegate %s "), *Top->TimerDelegate.ToString());
					});
				// no delegate should take longer then 2ms to run 
				SCOPE_TIME_GUARD_DELEGATE_MS(NameFunction, 2);
#endif
#if DO_TIMEGUARD && 0
				RunTimerDelegates.Add(Top->TimerDelegate);
#endif


#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
				bool bExecuteTransactionally = false;
				FName WorkName;
				switch (Top->TimerDelegate.VariantDelegate.GetIndex())
				{
				case FTimerDelegateVariant::IndexOfType<FTimerDelegate>():
					WorkName = FName{ TEXT("TimerManager::Execute FTimerDelegate") };
					bExecuteTransactionally = true;
					break;
				case FTimerDelegateVariant::IndexOfType<FTimerDynamicDelegate>():
					WorkName = FName{ TEXT("TimerManager::Execute FTimerDynamicDelegate") };
					bExecuteTransactionally = true;
					break;
				default:
					break;
				}
				if (bExecuteTransactionally)
				{
					UE_TIMER_MANAGER_DEBUG_SCOPE("ExecuteTransactional(%s, %s)", *CurrentlyExecutingTimer.ToString(), *Top->TimerDelegate.ToString());
					UE::RemoteExecutor::ExecuteTransactional(WorkName, [this, Handle = CurrentlyExecutingTimer]
					{
						checkf(Handle == CurrentlyExecutingTimer,
							TEXT("CurrentlyExecutingTimer changed between retries: Was=%s, Now=%s"),
							*Handle.ToString(), *CurrentlyExecutingTimer.ToString());
						FTimerData* Top = FindTimer(CurrentlyExecutingTimer);
						checkf(Top && !WillRemoveTimerAssert(CurrentlyExecutingTimer), TEXT("RemoveTimer(CurrentlyExecutingTimer) - due to fail before Execute()"));
						Top->TimerDelegate.Execute();
					});
				}
				else
#endif
				{
					UE_TIMER_MANAGER_DEBUG_SCOPE("Execute non-transactionally(%s, %s)", *CurrentlyExecutingTimer.ToString(), *Top->TimerDelegate.ToString());
					checkf(!WillRemoveTimerAssert(CurrentlyExecutingTimer),
						TEXT("RemoveTimer(Top = %p, CurrentlyExecutingTimer = %s) - due to fail before Execute()"),
						Top, *CurrentlyExecutingTimer.ToString());
					Top->TimerDelegate.Execute();
				}

				// Update Top pointer, in case it has been invalidated by the Execute call
				Top = FindTimer(CurrentlyExecutingTimer);
				checkf(!Top || !WillRemoveTimerAssert(CurrentlyExecutingTimer),
					TEXT("RemoveTimer(Top = %p, CurrentlyExecutingTimer = %s) - due to fail after Execute()"),
					Top, *CurrentlyExecutingTimer.ToString());
				if (!Top || Top->Status != ETimerStatus::Executing || Top->bMaxOncePerFrame)
				{
					break;
				}
			}

			if (DumpTimerLogsThreshold > 0.f && !bDumpTimerLogsThresholdExceeded)
			{
				// help us hunt down outliers that cause our timer manager times to spike.  Recommended that users set meaningful DumpTimerLogsThresholds in appropriate ini files if they are seeing spikes in the timer manager.
				const double DeltaT = (FPlatformTime::Seconds() - StartTime) * 1000.f;
				if (DeltaT >= DumpTimerLogsThreshold)
				{
					bDumpTimerLogsThresholdExceeded = true;
                    ++NbExpiredTimers;
					UE_LOGF(LogEngine, Log, "TimerManager's time threshold of %.2fms exceeded with a deltaT of %.4f, dumping current timer data.", DumpTimerLogsThreshold, DeltaT);

					if (Top)
					{
						DescribeFTimerDataSafely(*GLog, *Top);
					}
					else
					{
						UE_LOGF(LogEngine, Log, "There was no timer data for the first timer after exceeding the time threshold!");
					}
				}
			}

			// test to ensure it didn't get cleared during execution
			if (Top)
			{
				#if UE_WITH_REMOTE_OBJECT_HANDLE
				// IsBound can cause a migration (and we are not inside a transaction).
				// If the Delegate's Target is non-Resident, then we should reschedule and only do IsBound() when it's Resident.
				const bool bForceLoopingTimerReschedule = !IsTimerTargetResident(Top->TimerDelegate);
				#else
				constexpr bool bForceLoopingTimerReschedule = false;
				#endif

				// if timer requires a delegate, make sure it's still validly bound (i.e. the delegate's object didn't get deleted or something)
				if (Top->bLoop && (bForceLoopingTimerReschedule || !Top->bRequiresDelegate || Top->TimerDelegate.IsBound()))
				{
					// Put this timer back on the heap
					Top->ExpireTime += CallCount * Top->Rate;
					Top->Status = ETimerStatus::Active;
					ActiveTimerHeap.HeapPush(CurrentlyExecutingTimer, FTimerHeapOrder(Timers));
				}
				else
				{
					UE_TIMER_MANAGER_DEBUG_LOG("Removing timer (%s) after execution", *CurrentlyExecutingTimer.ToString());
					RemoveTimer(CurrentlyExecutingTimer);
				}

				CurrentlyExecutingTimer.Invalidate();
			}
		}
		else
		{
			// no need to go further down the heap, we can be finished
			break;
		}
	}

	if (NbExpiredTimers > MaxExpiredTimersToLog)
	{
		UE_LOGF(LogEngine, Log, "TimerManager's caught %d Timers exceeding the time threshold. Only the first %d were logged.", NbExpiredTimers, MaxExpiredTimersToLog);
	}

	// Timer has been ticked.
	LastTickedFrame = GFrameCounter;

	// If we have any Pending Timers, add them to the Active Queue.
	if( PendingTimerSet.Num() > 0 )
	{
		for (FTimerHandle Handle : PendingTimerSet)
		{
			FTimerData& TimerToActivate = GetTimer(Handle);

			// Convert from time remaining back to a valid ExpireTime
			TimerToActivate.ExpireTime += InternalTime;
			TimerToActivate.Status = ETimerStatus::Active;
			ActiveTimerHeap.HeapPush( Handle, FTimerHeapOrder(Timers) );
		}
		PendingTimerSet.Reset();
	}
}

TStatId FTimerManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FTimerManager, STATGROUP_Tickables);
}

void FTimerManager::SetGameInstance(UGameInstance* InGameInstance)
{
	OwningGameInstance = InGameInstance;

#if UE_ENABLE_TRACKING_TIMER_SOURCES
	if (TimerSourceList.IsValid())
	{
		TimerSourceList->OwningGameInstance = OwningGameInstance;
	}
#endif
}

void FTimerManager::ListTimers() const
{
	// not currently threadsafe
	check(IsInGameThread());

	TArray<const FTimerData*> ValidActiveTimers;
	ValidActiveTimers.Reserve(ActiveTimerHeap.Num());
	for (FTimerHandle Handle : ActiveTimerHeap)
	{
		if (const FTimerData* Data = FindTimer(Handle))
		{
			ValidActiveTimers.Add(Data);
		}
	}

	UE_LOGF(LogEngine, Log, "------- %d Active Timers -------", ValidActiveTimers.Num());
	for (const FTimerData* Data : ValidActiveTimers)
	{
		check(Data);
		DescribeFTimerDataSafely(*GLog, *Data);
	}

	UE_LOGF(LogEngine, Log, "------- %d Paused Timers -------", PausedTimerSet.Num());
	for (FTimerHandle Handle : PausedTimerSet)
	{
		const FTimerData& Data = GetTimer(Handle);
		DescribeFTimerDataSafely(*GLog, Data);
	}

	UE_LOGF(LogEngine, Log, "------- %d Pending Timers -------", PendingTimerSet.Num());
	for (FTimerHandle Handle : PendingTimerSet)
	{
		const FTimerData& Data = GetTimer(Handle);
		DescribeFTimerDataSafely(*GLog, Data);
	}

	UE_LOGF(LogEngine, Log, "------- %d Total Timers -------", PendingTimerSet.Num() + PausedTimerSet.Num() + ValidActiveTimers.Num());
}

FTimerHandle FTimerManager::AddTimer(FTimerData&& TimerData)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("AddTimer()");
	
	const void* TimerIndicesByObjectKey = TimerData.TimerDelegate.GetBoundObject();
	TimerData.TimerIndicesByObjectKey = TimerIndicesByObjectKey;

	int32 NewIndex = Timers.Add(MoveTemp(TimerData));

	FTimerHandle Result = GenerateHandle(NewIndex);
	Timers[NewIndex].Handle = Result;

	if (TimerIndicesByObjectKey)
	{
		AddObjectToTimers(Result, TimerIndicesByObjectKey);
	}

	return Result;
}

void FTimerManager::AddObjectToTimers(FTimerHandle TimerHandle, const void *TimerIndicesByObjectKey)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("AddObjectToTimers(TimerHandle: %s, TimerIndicesByObjectKey: %p)", *TimerHandle.ToString(), TimerIndicesByObjectKey);

	checkSlow(TimerIndicesByObjectKey)
	TSet<FTimerHandle>& HandleSet = ObjectToTimers.FindOrAdd(TimerIndicesByObjectKey);

	bool bAlreadyExists = false;
	HandleSet.Add(TimerHandle, &bAlreadyExists);
	checkf(!bAlreadyExists, TEXT("A timer with this handle and object has already been added! (%s)"), *TimerHandle.ToString());
}

void FTimerManager::RemoveTimer(FTimerHandle Handle)
{
	UE_TIMER_MANAGER_DEBUG_SCOPE("RemoveTimer(Handle: %s)", *Handle.ToString());

	const FTimerData& Data = GetTimer(Handle);

	// Remove TimerIndicesByObject entry if necessary
	if (const void* TimerIndicesByObjectKey = Data.TimerIndicesByObjectKey)
	{
		UE_TIMER_MANAGER_DEBUG_LOG("TimerIndicesByObjectKey: %p", TimerIndicesByObjectKey);

		TSet<FTimerHandle>* TimersForObject = ObjectToTimers.Find(TimerIndicesByObjectKey);
		checkf(TimersForObject, TEXT("Removed timer was bound to an object which is not tracked by ObjectToTimers! (%s)"), *GetFTimerDataSafely(Data));

		int32 NumRemoved = TimersForObject->Remove(Handle);
		checkf(NumRemoved == 1, TEXT("Removed timer was bound to an object which is not tracked by ObjectToTimers! (%s)"), *GetFTimerDataSafely(Data));

		if (TimersForObject->Num() == 0)
		{
			ObjectToTimers.Remove(TimerIndicesByObjectKey);
		}
	}

#if UE_AUTORTFM
	if (bEnableTransactionalTimers && AutoRTFM::IsTransactional() && TransactionalTimers.Find(Handle))
	{
		// Removing a timer in the transactional open that has been modified in
		// the transaction. The timer slot cannot be released until the
		// transaction is complete, so change the status to ReservedForTransaction.
		FTimerData Reserved{};
		Reserved.Handle = Handle;
		Reserved.Status = ETimerStatus::ReservedForTransaction;
		Timers[Handle.GetIndex()] = MoveTemp(Reserved);
	}
	else
#endif  // UE_AUTORTFM
	{
		Timers.RemoveAt(Handle.GetIndex());
	}
}

bool FTimerManager::WillRemoveTimerAssert(FTimerHandle Handle) const
{
	const FTimerData& Data = GetTimer(Handle);

	// Remove TimerIndicesByObject entry if necessary
	if (const void* TimerIndicesByObjectKey = Data.TimerIndicesByObjectKey)
	{
		const TSet<FTimerHandle>* TimersForObject = ObjectToTimers.Find(TimerIndicesByObjectKey);
		if (!TimersForObject)
		{
			return true;
		}

		const FTimerHandle* Found = TimersForObject->Find(Handle);
		if (!Found)
		{
			return true;
		}
	}

	return false;
}

void FTimerManager::RemoveExpiredTimers()
{
	TArray<FTimerHandle> HandlesToClear;

	for (const FTimerData& TimerData : Timers)
	{
		if (!TimerData.TimerDelegate.IsBound())
		{
			HandlesToClear.Add(TimerData.Handle);
		}
	}	

	for (FTimerHandle Handle : HandlesToClear)
	{
		int32 IndexIndex = ActiveTimerHeap.Find(Handle);
		if (IndexIndex != INDEX_NONE)
		{
			ActiveTimerHeap.HeapRemoveAt(IndexIndex, FTimerHeapOrder(Timers), EAllowShrinking::No);
		}

		PendingTimerSet.Remove(Handle);
		PausedTimerSet.Remove(Handle);

		RemoveTimer(Handle);
	}
}

FTimerHandle FTimerManager::GenerateHandle(int32 Index)
{
	uint64 NewSerialNumber = ++LastAssignedSerialNumber;
	if (!ensureMsgf(NewSerialNumber != FTimerHandle::MaxSerialNumber, TEXT("Timer serial number has wrapped around!")))
	{
		NewSerialNumber = (uint64)1;
	}

	FTimerHandle Result;
	Result.SetIndexAndSerialNumber(Index, NewSerialNumber);
	return Result;
}

#if UE_AUTORTFM
void FTimerManager::MaybeRegisterTransactionalCallbacks()
{
	if (bRegisteredTransactionalCallbacks)
	{
		return;
	}

	bRegisteredTransactionalCallbacks = true;

	AutoRTFM::OnCommit([this]
	{
		UE_TIMER_MANAGER_DEBUG_SCOPE("OnCommit()");
		
		for (TTuple<FTimerHandle, FTransactionalTimer>& It : TransactionalTimers)
		{
			const FTimerHandle Handle = It.Key;
			FTransactionalTimer& TransactionalTimer = It.Value;
			UE_TIMER_MANAGER_DEBUG_LOG("TransactionalTimer: {%s, Handle: %s, Status: %d}",
				TransactionalTimer.NewData ? TEXT("with-data") : TEXT("no-data"),
				*Handle.ToString(), (int)TransactionalTimer.Status);

			if (TransactionalTimer.NewData)
			{
				// Timer was created during the transaction.
				// Move the transactional timer to the non-transactional Timers array.
				FTimerData& TimerData = Timers[Handle.GetIndex()];
				check(Handle == TimerData.Handle);
				check(Handle == TransactionalTimer.NewData->Handle);
				TimerData = MoveTemp(*TransactionalTimer.NewData);

				switch (TransactionalTimer.Status)
				{
				case FTransactionalTimer::Active:
					TimerData.ExpireTime += InternalTime;
					TimerData.Status = ETimerStatus::Active;
					ActiveTimerHeap.HeapPush(TimerData.Handle, FTimerHeapOrder(Timers));
					break;

				case FTransactionalTimer::Pending:
					TimerData.Status = ETimerStatus::Pending;
					PendingTimerSet.Add(TimerData.Handle);
					break;

				case FTransactionalTimer::Paused:
					TimerData.Status = ETimerStatus::Paused;
					PausedTimerSet.Add(Handle);
					break;

				case FTransactionalTimer::Canceled:
					checkf(false, TEXT("FTransactionalTimer::NewData should be empty when Status is Canceled"));
					break;
				}

				if (const void* TimerIndicesByObjectKey = TimerData.TimerDelegate.GetBoundObject())
				{
					TimerData.TimerIndicesByObjectKey = TimerIndicesByObjectKey;
					AddObjectToTimers(Handle, TimerIndicesByObjectKey);
				}
			}
			else
			{
				switch (TransactionalTimer.Status)
				{
				case FTransactionalTimer::Active:
				case FTransactionalTimer::Pending:
				{
					// The code in this block mirrors UnPauseTimer() but with
					// the HasBeenTickedThisFrame() conditional replaced with
					// TransactionalTimer.Status
					FTimerData* TimerToUnPause = FindTimer(Handle);
					if (TimerToUnPause && TimerToUnPause->Status == ETimerStatus::Paused)
					{
						// Move it out of paused list and into proper TArray
						if (TransactionalTimer.Status == FTransactionalTimer::Active)
						{
							// Convert from time remaining back to a valid ExpireTime
							TimerToUnPause->ExpireTime += InternalTime;
							TimerToUnPause->Status = ETimerStatus::Active;
							ActiveTimerHeap.HeapPush(Handle, FTimerHeapOrder(Timers));
						}
						else
						{
							TimerToUnPause->Status = ETimerStatus::Pending;
							PendingTimerSet.Add(Handle);
						}

						// remove from paused list
						PausedTimerSet.Remove(Handle);
					}
					break;
				}

				case FTransactionalTimer::Paused:
					PauseTimer(Handle);
					break;

				case FTransactionalTimer::Canceled:
					if (FindTimer(Handle))
					{
						InternalClearTimer(Handle);
					}
					break;
				}
			}
		}
		TransactionalTimers.Empty();
		bRegisteredTransactionalCallbacks = false;
	});

	AutoRTFM::OnPreAbort([this]
	{
		UE_TIMER_MANAGER_DEBUG_SCOPE("OnPreAbort()");
		
		// Remove transactionally reserved timers.
		for (auto& It : TransactionalTimers)
		{
			const FTimerHandle& Handle = It.Key;
			const int32 Index = Handle.GetIndex();
			FTimerData* const Timer = FindTimer(Handle);
			if (Timer && Timer->Status == ETimerStatus::ReservedForTransaction)
			{
				RemoveTimer(Handle);
			}
		}
		// Note: TransactionalTimers & bRegisteredTransactionalCallbacks will be
		// cleared automatically by AutoRTFM memory rollback.
	});
}

FTimerHandle FTimerManager::GenerateReservedHandleForTransaction()
{
	FTimerData Reservation;
	Reservation.Status = ETimerStatus::ReservedForTransaction;
	int32 NewIndex = Timers.Add(MoveTemp(Reservation));
	FTimerHandle Handle = GenerateHandle(NewIndex);
	Timers[NewIndex].Handle = Handle;
	return Handle;
}
#endif  // UE_AUTORTFM

// Handler for ListTimers console command
static void OnListTimers(UWorld* World)
{
	if(World != nullptr)
	{
		World->GetTimerManager().ListTimers();
	}
}

// Register ListTimers console command, needs a World context
FAutoConsoleCommandWithWorld ListTimersConsoleCommand(
	TEXT("ListTimers"),
	TEXT(""),
	FConsoleCommandWithWorldDelegate::CreateStatic(OnListTimers)
	);

#if UE_TIMER_MANAGER_ENABLE_VERBOSE_LOGGING
#undef UE_TIMER_MANAGER_DEBUG_LOG
#undef UE_TIMER_MANAGER_DEBUG_SCOPE
#endif
#undef UE_TIMER_MANAGER_ENABLE_VERBOSE_LOGGING
