// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/World.h"
#include "Misc/AutomationTest.h"
#include "TimerManager.h"
#include "Engine/Engine.h"
#include "Tests/AutomationCommon.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTimerManagerTest, "System.Engine.TimerManager", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#define TIMER_TEST_TEXT( Format, ... ) FString::Printf(TEXT("%s - %d: %s"), TEXT(__FILE__) , __LINE__ , *FString::Printf(TEXT(Format), ##__VA_ARGS__) )

void TimerTest_TickWorld(UWorld* World, float Time = 1.f)
{
	const float Step = 0.1f;
	while(Time > 0.f)
	{
		World->Tick(ELevelTick::LEVELTICK_All, FMath::Min(Time, Step) );
		Time-=Step;

		// This is terrible but required for subticking like this.
		// we could always cache the real GFrameCounter at the start of our tests and restore it when finished.
		GFrameCounter++;
	}
}

struct FDummy
{
	FDummy() { Count = 0; }

	void Callback() { ++Count; }
	void Reset() { Count = 0; }

	uint8 Count;
};

// Make sure that the timer manager works as expected when given invalid delegates and handles
AUTORTFM_DISABLE bool TimerManagerTest_InvalidTimers(UWorld* World, FAutomationTestBase* Test)
{
	FTimerManager& TimerManager = World->GetTimerManager();
	FTimerHandle Handle;

	Test->TestFalse(TIMER_TEST_TEXT("TimerExists called with an invalid handle"), TimerManager.TimerExists(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerActive called with an invalid handle"), TimerManager.IsTimerActive(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerPaused called with an invalid handle"), TimerManager.IsTimerPaused(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRate called with an invalid handle"), (TimerManager.GetTimerRate(Handle) == -1.f));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with an invalid handle"), (TimerManager.GetTimerElapsed(Handle) == -1.f));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with an invalid handle"), (TimerManager.GetTimerRemaining(Handle) == -1.f));

	// these don't return values but we should run them to make sure they don't do something horrible like crash
	TimerManager.PauseTimer(Handle);
	TimerManager.UnPauseTimer(Handle);
	TimerManager.ClearTimer(Handle);
	
	return true;
}

// Make sure that the timer manager works as expected when given delegates and handles that aren't in the timer manager
AUTORTFM_DISABLE bool TimerManagerTest_MissingTimers(UWorld* World, FAutomationTestBase* Test)
{
	FTimerManager& TimerManager = World->GetTimerManager();
	FTimerHandle Handle = TimerManager.GenerateHandle(123);

	Test->TestFalse(TIMER_TEST_TEXT("TimerExists called with an invalid handle"), TimerManager.TimerExists(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerActive called with an invalid handle"), TimerManager.IsTimerActive(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerPaused called with an invalid handle"), TimerManager.IsTimerPaused(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRate called with an invalid handle"), (TimerManager.GetTimerRate(Handle) == -1.f));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with an invalid handle"), (TimerManager.GetTimerElapsed(Handle) == -1.f));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with an invalid handle"), (TimerManager.GetTimerRemaining(Handle) == -1.f));

	// these don't return values but we should run them to make sure they don't do something horrible like crash
	TimerManager.PauseTimer(Handle);
	TimerManager.UnPauseTimer(Handle);
	TimerManager.ClearTimer(Handle);

	return true;
}

AUTORTFM_DISABLE bool TimerManagerTest_ValidTimer_HandleWithDelegate(UWorld* World, FAutomationTestBase* Test)
{
	FTimerManager& TimerManager = World->GetTimerManager();
	FTimerDelegate Delegate;
	const float Rate = 1.5f;

	FDummy Dummy;
	Delegate.BindRaw(&Dummy, &FDummy::Callback);

	FTimerHandle Handle;
	TimerManager.SetTimer(Handle, Delegate, Rate, false);

	Test->TestTrue(TIMER_TEST_TEXT("Handle should be valid after calling SetTimer"), Handle.IsValid());
	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a pending timer"), TimerManager.TimerExists(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerActive called with a pending timer"), TimerManager.IsTimerActive(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerPaused called with a pending timer"), TimerManager.IsTimerPaused(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRate called with a pending timer"), (TimerManager.GetTimerRate(Handle) == Rate));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with a pending timer"), (TimerManager.GetTimerElapsed(Handle) == 0.f));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with a pending timer"), (TimerManager.GetTimerRemaining(Handle) == Rate));

	// small tick to move the timer from the pending list to the active list, the timer will start counting time after this tick
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);

	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a pending timer"), TimerManager.TimerExists(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerActive called with an active timer"), TimerManager.IsTimerActive(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerPaused called with an active timer"), TimerManager.IsTimerPaused(Handle));

	TimerTest_TickWorld(World);

	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with an active timer after one step"), (FMath::IsNearlyEqual(TimerManager.GetTimerElapsed(Handle), 1.f, UE_KINDA_SMALL_NUMBER)));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with an active timer after one step"), (FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), Rate - 1.f, UE_KINDA_SMALL_NUMBER)));

	TimerManager.PauseTimer(Handle);

	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a paused timer"), TimerManager.TimerExists(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerActive called with a paused timer"), TimerManager.IsTimerActive(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerPaused called with a paused timer"), TimerManager.IsTimerPaused(Handle));

	TimerTest_TickWorld(World);

	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a paused timer"), TimerManager.TimerExists(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerActive called with a paused timer"), TimerManager.IsTimerActive(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerPaused called with a paused timer"), TimerManager.IsTimerPaused(Handle));

	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with a paused timer after one step"), (FMath::IsNearlyEqual(TimerManager.GetTimerElapsed(Handle), 1.f, UE_KINDA_SMALL_NUMBER)));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with a paused timer after one step"), (FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), Rate - 1.f, UE_KINDA_SMALL_NUMBER)));

	TimerManager.UnPauseTimer(Handle);

	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a pending timer"), TimerManager.TimerExists(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerActive called with a pending timer"), TimerManager.IsTimerActive(Handle));
	Test->TestFalse(TIMER_TEST_TEXT("IsTimerPaused called with a pending timer"), TimerManager.IsTimerPaused(Handle));

	TimerTest_TickWorld(World);

	Test->TestFalse(TIMER_TEST_TEXT("TimerExists called with a completed timer"), TimerManager.TimerExists(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("Count of callback executions"), Dummy.Count == 1);


	// Test resetting the timer
	TimerManager.SetTimer(Handle, Delegate, Rate, false);
	TimerManager.SetTimer(Handle, 0.f, false);

	Test->TestFalse(TIMER_TEST_TEXT("TimerExists called with a reset timer"), TimerManager.TimerExists(Handle));

	// Test looping timers
	Dummy.Reset();
	TimerManager.SetTimer(Handle, Delegate, Rate, true);
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);

	TimerTest_TickWorld(World, 2.f);

	Test->TestTrue(TIMER_TEST_TEXT("TimerExists called with a looping timer"), TimerManager.TimerExists(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("IsTimerActive called with a looping timer"), TimerManager.IsTimerActive(Handle));

	Test->TestTrue(TIMER_TEST_TEXT("Count of callback executions"), Dummy.Count == 1);
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with a looping timer"), (FMath::IsNearlyEqual(TimerManager.GetTimerElapsed(Handle), 2.f - (Rate * Dummy.Count), UE_KINDA_SMALL_NUMBER)));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with a looping timer"), (FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), Rate * (Dummy.Count + 1) - 2.f, UE_KINDA_SMALL_NUMBER)));

	TimerTest_TickWorld(World, 2.f);

	Test->TestTrue(TIMER_TEST_TEXT("Count of callback executions"), Dummy.Count == 2);
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerElapsed called with a looping timer"), (FMath::IsNearlyEqual(TimerManager.GetTimerElapsed(Handle), 4.f - (Rate * Dummy.Count), UE_KINDA_SMALL_NUMBER)));
	Test->TestTrue(TIMER_TEST_TEXT("GetTimerRemaining called with a looping timer"), (FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), Rate * (Dummy.Count + 1) - 4.f, UE_KINDA_SMALL_NUMBER)));

	TimerManager.SetTimer(Handle, 0.f, false);

	Test->TestFalse(TIMER_TEST_TEXT("TimerExists called with a reset looping timer"), TimerManager.TimerExists(Handle));

	return true;
}

struct FLoopingTestFunc
{
	static FTimerManager* TimerManager;
	static FTimerHandle* Handle;
	static int TimerCalled;
	static float NewTime;
	static void TimerExecute()
	{
		TimerCalled++;
		if (TimerCalled == 1)
		{
			TimerManager->SetTimer(*Handle, FTimerDelegate::CreateStatic(FLoopingTestFunc::TimerExecute), NewTime, true);
		}
		else
		{
			TimerManager->ClearTimer(*Handle);
		}
	}
};

FTimerManager* FLoopingTestFunc::TimerManager = nullptr;
FTimerHandle* FLoopingTestFunc::Handle = nullptr;
int FLoopingTestFunc::TimerCalled = 0;
float FLoopingTestFunc::NewTime = 1.0f;

AUTORTFM_DISABLE bool TimerManagerTest_ValidTimer_HandleLoopingSetDuringExecute(UWorld* World, FAutomationTestBase* Test)
{
	FTimerManager& TimerManager = World->GetTimerManager();
	FTimerHandle Handle;
	const float Rate = 3.f;

	FLoopingTestFunc::TimerManager = &TimerManager;
	FLoopingTestFunc::Handle = &Handle;
	FLoopingTestFunc::TimerCalled = 0;

	Test->TestTrue(TIMER_TEST_TEXT("Timer called count starts at 0"), FLoopingTestFunc::TimerCalled == 0); //-V547

	TimerManager.SetTimer(Handle, FTimerDelegate::CreateStatic(FLoopingTestFunc::TimerExecute), Rate, true);

	// small tick to move the timer from the pending list to the active list, the timer will start counting time after this tick
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);

	TimerTest_TickWorld(World, 3.0f);
	Test->TestTrue(TIMER_TEST_TEXT("Timer was called first time"), FLoopingTestFunc::TimerCalled == 1); //-V547
	Test->TestTrue(TIMER_TEST_TEXT("Timer was readded"), TimerManager.IsTimerActive(Handle));
	Test->TestTrue(TIMER_TEST_TEXT("Timer was readded with correct time"), FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), FLoopingTestFunc::NewTime, 1e-2f));

	TimerTest_TickWorld(World, 1.1f);
	Test->TestTrue(TIMER_TEST_TEXT("Timer was called second time"), FLoopingTestFunc::TimerCalled == 2); //-V547
	Test->TestFalse(TIMER_TEST_TEXT("Timer handle no longer active"), TimerManager.IsTimerActive(Handle));

	return true;
}

AUTORTFM_DISABLE bool TimerManagerTest_LoopingTimers_DifferentHandles(UWorld* World, FAutomationTestBase* Test)
{
	FTimerManager& TimerManager = World->GetTimerManager();
	FTimerHandle HandleOne, HandleTwo;

	int32 CallCount = 0;
	auto Func = [](int* InCallCount){ (*InCallCount)++; };

	FTimerDelegate Delegate = FTimerDelegate::CreateStatic(Func, &CallCount);

	FTimerHandle Handle;
	TimerManager.SetTimer(Handle, Delegate, 1.0f, false);
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);

	Test->TestTrue(TIMER_TEST_TEXT("First delegate time remaining is 1.0f"),
		FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), 1.0f, UE_KINDA_SMALL_NUMBER));
	
	TimerManager.SetTimer(Handle, Delegate, 5.0f, false);
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);
	Test->TestTrue(TIMER_TEST_TEXT("Reset delegate time remaining is 5.0f"),
		FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(Handle), 5.0f, UE_KINDA_SMALL_NUMBER));

	TimerManager.SetTimer(HandleOne, Delegate, 1.0f, true);
	TimerManager.SetTimer(HandleTwo, Delegate, 1.5f, true);
	TimerTest_TickWorld(World, UE_KINDA_SMALL_NUMBER);

	Test->TestTrue(TIMER_TEST_TEXT("Handle One is active"), TimerManager.IsTimerActive(HandleOne));
	Test->TestTrue(TIMER_TEST_TEXT("Handle Two is active"), TimerManager.IsTimerActive(HandleTwo));

	TimerTest_TickWorld(World, 1.0f);

	Test->TestTrue(TIMER_TEST_TEXT("Handle One is active after tick"), TimerManager.IsTimerActive(HandleOne));
	Test->TestTrue(TIMER_TEST_TEXT("Handle Two is active after tick"), TimerManager.IsTimerActive(HandleTwo));

	Test->TestTrue(TIMER_TEST_TEXT("Handle One has 0 seconds remaining after tick"), 
		FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(HandleOne), 0.0f, 1e-2f) );
	Test->TestTrue(TIMER_TEST_TEXT("Handle Two has 0.5 seconds remaining after tick"),
		FMath::IsNearlyEqual(TimerManager.GetTimerRemaining(HandleTwo), 0.5f, 1e-2f));

	return true;
}

bool FTimerManagerTest::RunTest(const FString& Parameters)
{
	// This will get cleaned up when it leaves scope
	FTestWorldWrapper WorldWrapper;
	WorldWrapper.CreateTestWorld(EWorldType::Game);
	UWorld* World = WorldWrapper.GetTestWorld();

	if (World)
	{
		WorldWrapper.BeginPlayInTestWorld();
		TimerManagerTest_InvalidTimers(World, this);
		TimerManagerTest_MissingTimers(World, this);
		TimerManagerTest_ValidTimer_HandleWithDelegate(World, this);
		TimerManagerTest_ValidTimer_HandleLoopingSetDuringExecute(World, this);
		TimerManagerTest_LoopingTimers_DifferentHandles(World, this);
		WorldWrapper.ForwardErrorMessages(this);

		return !HasAnyErrors();
	}

	return false;
}


