// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceFilter.h"

#if TRACE_FILTERING_ENABLED

#include "TraceFilters.h"

#include "ObjectTrace.h"

#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"

struct ENGINE_API FTraceFilterObjectAnnotation
{
	FTraceFilterObjectAnnotation()
		: bIsTraceable(true)
	{}

	bool bIsTraceable;

	/** Determine if this annotation is default - required for annotations */
	FORCEINLINE bool IsDefault() const
	{
		return bIsTraceable;
	}
};

DEFINE_LOG_CATEGORY_STATIC(TraceFiltering, Display, Display);

/** Modified version of FUObjectAnnotationSparse, allowing for direct (non-const) access to the AnnotationMap. And being able to manually (Un)Lock allowing for batch changes */
class FTraceUObjectAnnotation : public FUObjectArray::FUObjectDeleteListener
{
public:
	virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override
	{
		RemoveAnnotation(Object);
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	FTraceUObjectAnnotation()
	{
		// default constructor is required to be default annotation
		check(AnnotationCacheValue.IsDefault());
	}

	virtual ~FTraceUObjectAnnotation()
	{
		RemoveAllAnnotations();
	}

private:
	void AddAnnotationInternal(const UObjectBase* Object, const FTraceFilterObjectAnnotation& Annotation)
	{
		check(Object);
		UE::TScopeLock ScopeLock(AnnotationMapCritical);
		AnnotationCacheKey = Object;
		AnnotationCacheValue = Annotation;
		if (AnnotationCacheValue.IsDefault())
		{
			RemoveAnnotation(Object); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			if (AnnotationMap.Num() == 0)
			{
				// we are adding the first one, so if we are auto removing or verifying removal, register now
				GUObjectArray.AddUObjectDeleteListener(this);
			}
			AnnotationMap.Add(AnnotationCacheKey, AnnotationCacheValue);
		}
	}

public:

	void AddAnnotation(const UObjectBase* Object, FTraceFilterObjectAnnotation&& Annotation)
	{
		AddAnnotationInternal(Object, MoveTemp(Annotation));
	}

	void AddAnnotation(const UObjectBase* Object, const FTraceFilterObjectAnnotation& Annotation)
	{
		AddAnnotationInternal(Object, Annotation);
	}

	void RemoveAnnotation(const UObjectBase* Object)
	{
		UE::TScopeLock ScopeLock(AnnotationMapCritical);
		check(Object);
		
		AnnotationCacheKey = Object;
		AnnotationCacheValue = FTraceFilterObjectAnnotation();
		const bool bHadElements = (AnnotationMap.Num() > 0);
		AnnotationMap.Remove(AnnotationCacheKey);
		if (bHadElements && AnnotationMap.Num() == 0)
		{
			// we are removing the last one, so if we are auto removing or verifying removal, unregister now
			GUObjectArray.RemoveUObjectDeleteListener(this);
		}
	}

	void RemoveAllAnnotations()
	{
		UE::TScopeLock ScopeLock(AnnotationMapCritical);

		AnnotationCacheKey = nullptr;
		AnnotationCacheValue = FTraceFilterObjectAnnotation();
		const bool bHadElements = (AnnotationMap.Num() > 0);
		AnnotationMap.Empty();
		if (bHadElements)
		{
			// we are removing the last one, so if we are auto removing or verifying removal, unregister now
			GUObjectArray.RemoveUObjectDeleteListener(this);
		}
	}

	FORCEINLINE FTraceFilterObjectAnnotation GetAnnotation(const UObjectBase* Object)
	{
		UE::TScopeLock ScopeLock(AnnotationMapCritical);

		check(Object);
		
		if (Object != AnnotationCacheKey)
		{
			AnnotationCacheKey = Object;
			FTraceFilterObjectAnnotation* Entry = AnnotationMap.Find(AnnotationCacheKey);
			if (Entry)
			{
				AnnotationCacheValue = *Entry;
			}
			else
			{
				AnnotationCacheValue = FTraceFilterObjectAnnotation();
			}
		}
		return AnnotationCacheValue;
	}

	TMap<const UObjectBase*, FTraceFilterObjectAnnotation>& GetAnnotationMap()
	{
		return AnnotationMap;
	}

	void Lock()
	{
		AnnotationMapScopeLock.Emplace(AnnotationMapCritical);
	}

	void Unlock()
	{		
		AnnotationMapScopeLock.Reset();
	}

	bool IsLocked()
	{
		return AnnotationMapScopeLock.IsSet();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return AnnotationMap.GetAllocatedSize();
	}

private:
	TMap<const UObjectBase*, FTraceFilterObjectAnnotation> AnnotationMap;
	FTransactionallySafeCriticalSection AnnotationMapCritical;

	TOptional<UE::TScopeLock<FTransactionallySafeCriticalSection>> AnnotationMapScopeLock;

	const UObjectBase* AnnotationCacheKey = nullptr;
	FTraceFilterObjectAnnotation AnnotationCacheValue;
};

FTraceUObjectAnnotation GObjectFilterAnnotations;

/** Console command allowing to debug the current state of GObjectFilterAnnotations, to see which objects are Traceable*/
FAutoConsoleCommand FlushFilterStateCommand(TEXT("TraceFilter.FlushState"), TEXT("Flushes the current trace filtering state to the output log."),
		FConsoleCommandDelegate::CreateLambda([]()
		{
			TMap<const UWorld*, TArray<const AActor*>> WorldToActorMap;
			TMap<const AActor*, TArray<const UActorComponent*>> ActorToComponentMap;
			TSet<const UObject*> Objects;

			/** Collect object type information */
			const TMap<const UObjectBase*, FTraceFilterObjectAnnotation>& Annotations = GObjectFilterAnnotations.GetAnnotationMap();

			/** Retrieve all annotated objects */
			TArray<const UObjectBase*> BaseObjects;
			Annotations.GenerateKeyArray(BaseObjects);

			/** Filter each object according to its type */
			for (const UObjectBase* ObjectBase : BaseObjects)
			{
				const UObject* Object = (const UObject*)ObjectBase;
				if (const UWorld* World = Cast<UWorld>(Object))
				{
					WorldToActorMap.Add(World);
				}
			}

			for (const UObjectBase* ObjectBase : BaseObjects)
			{
				const UObject* Object = (const UObject*)ObjectBase;
				if (const AActor* Actor = Cast<AActor>(Object))
				{
					WorldToActorMap.FindChecked(Actor->GetWorld()).Add(Actor);
					ActorToComponentMap.Add(Actor);
				}
			}

			for (const UObjectBase* ObjectBase : BaseObjects)
			{
				const UObject* Object = (const UObject*)ObjectBase;
				if (const UActorComponent* Component = Cast<UActorComponent>(Object))
				{
					ActorToComponentMap.FindChecked(Component->GetOwner()).Add(Component);
				}
			}

			Algo::TransformIf(BaseObjects, Objects, 
				[](const UObjectBase* ObjectBase)
				{
					const UObject* Object = (const UObject*)ObjectBase;
					return !Cast<UWorld>(Object) && !Cast<AActor>(Object) && !Cast<UActorComponent>(Object);
				},
				[](const UObjectBase* ObjectBase)
				{
					const UObject* Object = (const UObject*)ObjectBase;
					return Object;
				}
			);

			/** Output collated data */
			FString OutputString;

			/** For each UWorld, output all AActors */
			for (const TPair<const UWorld*, TArray<const AActor*>>& WorldToActor : WorldToActorMap)
			{	
				OutputString += TEXT("\n");
				OutputString += WorldToActor.Key->GetName();
				OutputString += TEXT(" [UWorld]\n");

				/** For each AActor, output all UActorComponent */
				for (const AActor* Actor : WorldToActor.Value)
				{
					OutputString += TEXT("\t- ");
					OutputString += Actor->GetName();
					OutputString += TEXT(" [Actor]\n");

					TArray<const UActorComponent*> ActorComponents = ActorToComponentMap.FindChecked(Actor);
					for (const UActorComponent* Component : ActorComponents)
					{
						OutputString += TEXT("\t\t* ");
						OutputString += Component->GetName();
						OutputString += TEXT(" [Component]\n");
					}			
				}

				OutputString += TEXT("----------------------------------------------------\n");
			}			

			for (const UObject* Object : Objects)
			{
				OutputString += Object->GetName();
				OutputString += TEXT(" [Object]\n");
			}

			UE_LOGF(TraceFiltering, Display, "%ls", *OutputString);
		})
	);

template<bool bForceThreadSafe>
bool FTraceFilter::IsObjectTraceable(const UObject* InObject)
{
	if constexpr (!bForceThreadSafe)
	{
		check(GObjectFilterAnnotations.IsLocked());
	}

	// Object not found in the AnnotationMap means that it is at the default value, which is bIsTraceable == true
	return GObjectFilterAnnotations.GetAnnotationMap().Find(InObject) == nullptr;
}

template bool FTraceFilter::IsObjectTraceable</*bForceThreadSafe = */ false>(const UObject* InObject);
template bool FTraceFilter::IsObjectTraceable</*bForceThreadSafe = */ true>(const UObject* InObject);

template<bool bForceThreadSafe>
void FTraceFilter::SetObjectIsTraceable(const UObject* InObject, bool bIsTraceable)
{
	ensure(InObject);

	if constexpr (bForceThreadSafe)
	{
		FTraceFilterObjectAnnotation Annotation;
		Annotation.bIsTraceable = bIsTraceable;
		GObjectFilterAnnotations.AddAnnotation(InObject, Annotation);

		if (bIsTraceable)
		{
			TRACE_OBJECT(InObject);
		}
	}
	else
	{
		check(GObjectFilterAnnotations.IsLocked());
		TMap<const UObjectBase*, FTraceFilterObjectAnnotation>& AnnotationMap = GObjectFilterAnnotations.GetAnnotationMap();
		if (!bIsTraceable)
		{
			AnnotationMap.FindOrAdd(InObject).bIsTraceable = false;
		}
		else
		{
			AnnotationMap.Remove(InObject);
			TRACE_OBJECT(InObject);
		}
	}
}

template void FTraceFilter::SetObjectIsTraceable</*bForceThreadSafe = */ true>(const UObject* InObject, bool bIsTraceable);
template void FTraceFilter::SetObjectIsTraceable</*bForceThreadSafe = */ false>(const UObject* InObject, bool bIsTraceable);

template<bool bForceThreadSafe>
void FTraceFilter::MarkObjectTraceable(const UObject* InObject)
{
	SetObjectIsTraceable<bForceThreadSafe>(InObject, true);
}

template void FTraceFilter::MarkObjectTraceable</*bForceThreadSafe = */ true>(const UObject* InObject);
template void FTraceFilter::MarkObjectTraceable</*bForceThreadSafe = */ false>(const UObject* InObject);

void FTraceFilter::Init()
{
	FTraceActorFilter::Initialize();
	FTraceWorldFilter::Initialize();
}

void FTraceFilter::Destroy()
{
	GObjectFilterAnnotations.RemoveAllAnnotations();
	FTraceActorFilter::Destroy();
	FTraceWorldFilter::Destroy();
}

void FTraceFilter::Lock()
{
	GObjectFilterAnnotations.Lock();
}

void FTraceFilter::Unlock()
{	
	GObjectFilterAnnotations.Unlock();
}

#endif // TRACE_FILTERING_ENABLED

