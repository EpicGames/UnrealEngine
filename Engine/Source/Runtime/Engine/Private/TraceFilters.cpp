// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceFilters.h"

#if TRACE_FILTERING_ENABLED

#include "EngineUtils.h"

FDelegateHandle FTraceActorFilter::WorldPreInitHandle;
FDelegateHandle FTraceActorFilter::WorldPostInitHandle;
FDelegateHandle FTraceActorFilter::OnActorSpawnedHandle;

void FTraceActorFilter::Initialize()
{
	WorldPreInitHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&FTraceActorFilter::OnPreWorldInit);
	WorldPostInitHandle = FWorldDelegates::OnPostWorldInitialization.AddStatic(&FTraceActorFilter::OnPostWorldInit);
}

void FTraceActorFilter::Destroy()
{
	FWorldDelegates::OnPreWorldInitialization.Remove(WorldPreInitHandle);
	FWorldDelegates::OnPostWorldInitialization.Remove(WorldPostInitHandle);
}

void FTraceActorFilter::OnPreWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	World->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateStatic(&FTraceActorFilter::OnActorSpawned));
}

void FTraceActorFilter::OnPostWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (CAN_TRACE_OBJECT(World))
	{
		// Handle actors that were duplicated over from a previous UWorld instance (PIE)
		for (TActorIterator<AActor> It(World); It; ++It)
		{
			if (AActor* Actor = *It)
			{
				OnActorSpawned(Actor);
			}
		}
	}
}

void FTraceActorFilter::OnActorSpawned(AActor* Actor)
{
	/** Mark traceable if UWorld is marked as such */
	if (CAN_TRACE_OBJECT(Actor->GetWorld()))
	{
		MARK_OBJECT_TRACEABLE(Actor);
	}
}

TAutoConsoleVariable<int32> CVarRecordAllWorldTypes(
	TEXT("Insights.RecordAllWorldTypes"),
	0,
	TEXT("Gameplay Insights recording by default only records Game and PIE worlds.")
	TEXT("Toggle this value to 1 to record other world types."));

FDelegateHandle FTraceWorldFilter::WorldInitHandle;

void FTraceWorldFilter::Initialize()
{
	WorldInitHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&FTraceWorldFilter::OnWorldInit);	
}

void FTraceWorldFilter::Destroy()
{
	FWorldDelegates::OnPreWorldInitialization.Remove(WorldInitHandle);
}

void FTraceWorldFilter::OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (CVarRecordAllWorldTypes.GetValueOnAnyThread() != 0 ||
			World == nullptr ||
			World->WorldType == EWorldType::Game ||
			World->WorldType == EWorldType::PIE)
	{
		MARK_OBJECT_TRACEABLE(World);
	}
}

#endif // TRACE_FILTERING_ENABLED
