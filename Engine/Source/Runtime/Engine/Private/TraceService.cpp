// Copyright Epic Games, Inc. All Rights Reserved.
#include "TraceService.h"

#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "ProfilingDebugging/TraceScreenshot.h"
#include "Runtime/EngineMessages/Public/TraceControlMessages.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/MemoryArchive.h"
#include "Trace/Trace.h"
#include "TraceControlMessages.h"

FTraceService::FTraceService()
	: FTraceService(IMessagingModule::Get().GetDefaultBus())
{
}

FTraceService::FTraceService(const TSharedPtr<IMessageBus>& InBus)
{
	SessionId = FApp::GetSessionId();
	InstanceId = FApp::GetInstanceId();

	if (InBus.IsValid())
	{
		MessageEndpoint = FMessageEndpoint::Builder("FTraceService", InBus.ToSharedRef())
			.Handling<FTraceControlDiscoveryPing>(this, &FTraceService::OnDiscoveryPing)
			.Handling<FTraceControlChannelsSet>(this, &FTraceService::OnChannelSet)
			.Handling<FTraceControlStop>(this, &FTraceService::OnStop)
			.Handling<FTraceControlSend>(this, &FTraceService::OnSend)
			.Handling<FTraceControlFile>(this, &FTraceService::OnFile)
			.Handling<FTraceControlSnapshotSend>(this, &FTraceService::OnSnapshotSend)
			.Handling<FTraceControlSnapshotFile>(this, &FTraceService::OnSnapshotFile)
			.Handling<FTraceControlPause>(this, &FTraceService::OnPause)
			.Handling<FTraceControlResume>(this, &FTraceService::OnResume)
			.Handling<FTraceControlBookmark>(this, &FTraceService::OnBookmark)
#if UE_SCREENSHOT_TRACE_ENABLED
			.Handling<FTraceControlScreenshot>(this, &FTraceService::OnScreenshot)
#endif
#if UE_TRACE_ALLOW_SECURE_TRACING 
			.Handling<FTraceControlSecure>(this, &FTraceService::OnSecure)
#endif
			.Handling<FTraceControlSetStatNamedEvents>(this, &FTraceService::OnSetStatNamedEvents)
			.Handling<FTraceControlStatusPing>(this, &FTraceService::OnStatusPing)
			.Handling<FTraceControlSettingsPing>(this, &FTraceService::OnSettingsPing)
			.Handling<FTraceControlChannelsPing>(this, &FTraceService::OnChannelsPing);

		if (!MessageEndpoint.IsValid())
		{
			return;
		}

		MessageEndpoint->Subscribe<FTraceControlStatusPing>();
		MessageEndpoint->Subscribe<FTraceControlSettingsPing>();
		MessageEndpoint->Subscribe<FTraceControlDiscoveryPing>();
		MessageEndpoint->Subscribe<FTraceControlChannelsPing>();
		// MessageEndpoint->Subscribe<FTraceControlStop>();
		// MessageEndpoint->Subscribe<FTraceControlSend>();
		// MessageEndpoint->Subscribe<FTraceControlChannelsSet>();
		// MessageEndpoint->Subscribe<FTraceControlFile>();
		// MessageEndpoint->Subscribe<FTraceControlSnapshotSend>();
		// MessageEndpoint->Subscribe<FTraceControlSnapshotSend>();
		// MessageEndpoint->Subscribe<FTraceControlPause>();
		// MessageEndpoint->Subscribe<FTraceControlResume>();
		// MessageEndpoint->Subscribe<FTraceControlBookmark>();
#if UE_SCREENSHOT_TRACE_ENABLED
		MessageEndpoint->Subscribe<FTraceControlScreenshot>();
#endif
		MessageEndpoint->Subscribe<FTraceControlSetStatNamedEvents>();
#if UE_TRACE_ALLOW_SECURE_TRACING 
		// MessageEndpoint->Subscribe<FTraceControlSecure>();
#endif
	}
}

void FTraceService::FillTraceStatusMessage(FTraceControlStatus* Message)
{
	// Get the current endpoint and ids
	Message->Endpoint = FTraceAuxiliary::GetTraceDestinationString();
	Message->bIsTracing = FTraceAuxiliary::IsConnected(Message->SessionGuid, Message->TraceGuid);
	
	// For stats we can query TraceLog directly.
	UE::Trace::FStatistics Stats;
	UE::Trace::GetStatistics(Stats);
	Message->BytesSent = Stats.BytesSent;
	Message->BytesTraced = Stats.BytesTraced;
	Message->MemoryUsed = Stats.MemoryUsed;
	Message->CacheAllocated = Stats.CacheAllocated;
	Message->CacheUsed = Stats.CacheUsed;
	Message->CacheWaste = Stats.CacheWaste;
	Message->bAreStatNamedEventsEnabled = GCycleStatsShouldEmitNamedEvents > 0;
	Message->bIsPaused = FTraceAuxiliary::IsPaused();
	Message->StatusTimestamp = FDateTime::Now();
	Message->TraceSystemStatus = static_cast<uint8>(FTraceAuxiliary::GetTraceSystemStatus());
}

void FTraceService::OnChannelSet(const FTraceControlChannelsSet& Message, const TSharedRef<IMessageContext>& Context)
{
	TMap<uint32, FString> Errors;
	FTraceAuxiliary::EnableChannels(Message.ChannelIdsToEnable, &Errors);
	FTraceAuxiliary::DisableChannels(Message.ChannelIdsToDisable, &Errors);
	if (!Errors.IsEmpty())
	{
		auto Response = MessageEndpoint->MakeMessage<FTraceControlChannelsSetError>();
		Response->Errors = MoveTemp(Errors);
		MessageEndpoint->Send(Response, Context->GetSender());
	}
}

void FTraceService::OnStop(const FTraceControlStop& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::Stop();
}

void FTraceService::OnSend(const FTraceControlSend& Message, const TSharedRef<IMessageContext>& Context)
{
	HandleSendUri(Message);
}

void FTraceService::OnFile(const FTraceControlFile& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::FOptions Options;
	Options.bTruncateFile = Message.bTruncateFile;
	Options.bExcludeTail = Message.bExcludeTail;
	
	FTraceAuxiliary::Start(
		FTraceAuxiliary::EConnectionType::File,
		*Message.File,
		*Message.Channels,
		&Options
	);
}

void FTraceService::OnSnapshotSend(const FTraceControlSnapshotSend& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::SendSnapshot(*Message.Host);
}

void FTraceService::OnSnapshotFile(const FTraceControlSnapshotFile& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::WriteSnapshot(*Message.File);
}

void FTraceService::OnPause(const FTraceControlPause& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::Pause();
}

void FTraceService::OnResume(const FTraceControlResume& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceAuxiliary::Resume();
}

void FTraceService::OnBookmark(const FTraceControlBookmark& Message, const TSharedRef<IMessageContext>& Context)
{
	TRACE_BOOKMARK(TEXT("%s"), *Message.Label);
}

#if UE_SCREENSHOT_TRACE_ENABLED
void FTraceService::OnScreenshot(const FTraceControlScreenshot& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceScreenshot::RequestScreenshot(Message.Name, Message.bShowUI);
}
#endif // UE_SCREENSHOT_TRACE_ENABLED

void FTraceService::OnSetStatNamedEvents(const FTraceControlSetStatNamedEvents& Message, const TSharedRef<IMessageContext>& Context)
{
	if (Message.bEnabled && GCycleStatsShouldEmitNamedEvents == 0)
	{
		++GCycleStatsShouldEmitNamedEvents;
	}
	if (!Message.bEnabled && GCycleStatsShouldEmitNamedEvents > 0)
	{
		GCycleStatsShouldEmitNamedEvents = 0;
	}
}

void FTraceService::HandleSendUri(const FTraceControlSend& Message)
{
	FTraceAuxiliary::FOptions Options;
	Options.bExcludeTail = Message.bExcludeTail;
	
	FTraceAuxiliary::Start(
		FTraceAuxiliary::EConnectionType::Network,
		*Message.Host,
		*Message.Channels,
		&Options
	);
}

void FTraceService::OnStatusPing(const FTraceControlStatusPing& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceControlStatus* Response = FMessageEndpoint::MakeMessage<FTraceControlStatus>();
	FillTraceStatusMessage(Response);
	
	MessageEndpoint->Send(Response, Context->GetSender());
}

void FTraceService::OnChannelsPing(const FTraceControlChannelsPing& Message, const TSharedRef<IMessageContext>& Context)
{
	
	struct FEnumerateUserData
	{
		TArray<FString> Channels;
		TArray<FString> Descriptions;
		TArray<uint32> Ids;
		TArray<uint32> ReadOnlyIds;
		TArray<uint32> EnabledIds;
	} UserData;
	
	UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& ChannelInfo, void* User)
	{
		FEnumerateUserData* UserData = static_cast<FEnumerateUserData*>(User);
		FAnsiStringView NameView = FAnsiStringView(ChannelInfo.Name).LeftChop(7); // Remove "Channel" suffix
		const uint32 ChannelId = ChannelInfo.Id;
		UserData->Channels.Emplace(NameView);
		UserData->Ids.Emplace(ChannelId);
		UserData->Descriptions.Emplace(ChannelInfo.Desc);
		if (ChannelInfo.bIsReadOnly)
		{
			UserData->ReadOnlyIds.Add(ChannelId);
		}
		if (ChannelInfo.bIsEnabled)
		{
			UserData->EnabledIds.Add(ChannelId);
		}
		return true;
	}, &UserData);

	// Only send channel description message if the number of channels has changed.
	if (Message.KnownChannelCount < uint32(UserData.Channels.Num()))
	{
		FTraceControlChannelsDesc* DescResponse = FMessageEndpoint::MakeMessage<FTraceControlChannelsDesc>();
		DescResponse->Channels = MoveTemp(UserData.Channels);
		DescResponse->Ids = MoveTemp(UserData.Ids);
		DescResponse->Descriptions = MoveTemp(UserData.Descriptions);
		DescResponse->ReadOnlyIds = MoveTemp(UserData.ReadOnlyIds);
		MessageEndpoint->Send(DescResponse, Context->GetSender());
	}

	// Always send status response
	FTraceControlChannelsStatus* StatusResponse = FMessageEndpoint::MakeMessage<FTraceControlChannelsStatus>();
	StatusResponse->EnabledIds = MoveTemp(UserData.EnabledIds);
	MessageEndpoint->Send(StatusResponse, Context->GetSender());
}

void FTraceService::OnSettingsPing(const FTraceControlSettingsPing& Message, const TSharedRef<IMessageContext>& Context)
{
	FTraceControlSettings* Response = FMessageEndpoint::MakeMessage<FTraceControlSettings>();
	UE::Trace::FInitializeDesc const* InitDesc = FTraceAuxiliary::GetInitializeDesc();

	if (InitDesc)
	{
		Response->bUseImportantCache = InitDesc->bUseImportantCache;
		Response->bUseWorkerThread = InitDesc->bUseWorkerThread;
		Response->TailSizeBytes = InitDesc->TailSizeBytes;
	}

	auto AddPreset = [&Response](const FTraceAuxiliary::FChannelPreset& Preset)
	{
		FTraceChannelPreset TracePreset;
		TracePreset.Name = Preset.Name;
		TracePreset.ChannelList = Preset.ChannelList;
		TracePreset.bIsReadOnly = Preset.bIsReadOnly;

		Response->ChannelPresets.Add(TracePreset);
		return FTraceAuxiliary::EEnumerateResult::Continue;
	};

	FTraceAuxiliary::EnumerateFixedChannelPresets(AddPreset);
	FTraceAuxiliary::EnumerateChannelPresetsFromSettings(AddPreset);

#if UE_TRACE_ALLOW_SECURE_TRACING 
	Response->Capabilities.Add(FTraceCapability{ TEXT("SecureConnection") });
#endif

	MessageEndpoint->Send(Response, Context->GetSender());
}

void FTraceService::OnDiscoveryPing(const FTraceControlDiscoveryPing& Message, const TSharedRef<IMessageContext>& Context)
{
	if ((!Message.SessionId.IsValid() && !Message.InstanceId.IsValid()) || (Message.InstanceId == FApp::GetInstanceId() || Message.SessionId == FApp::GetSessionId()))
	{
		const auto Response = FMessageEndpoint::MakeMessage<FTraceControlDiscovery>();
		Response->SessionId = FApp::GetSessionId();
		Response->InstanceId = FApp::GetInstanceId();

		FillTraceStatusMessage(Response);

		MessageEndpoint->Send(
			Response,
			Context->GetSender()
		);
	}
}


void FTraceService::OnSecure(const FTraceControlSecure& Message, const TSharedRef<IMessageContext>& Context)
{
#if UE_TRACE_ALLOW_SECURE_TRACING 

	// Generate a random authorization token
	FGuid AuthToken = FGuid::NewGuid();
	FArrayWriter AuthData;
	AuthData << AuthToken;
	uint32 AuthTokenSize = AuthData.Num();
	
	// Setup the security options
	FString Identity = FGuid::NewGuid().ToString(); // Random identity
	FTraceAuxiliary::FSecureOptions Options;
	Options.AuthorizationToken = AuthData;
	Options.TimeoutSeconds = 30.0f;
	Options.Identity = TCHAR_TO_ANSI(*Identity);

	const bool bSuccess = FTraceAuxiliary::StartSecure(Options);

	if (!bSuccess)
	{
		return;
	}

	const auto Response = FMessageEndpoint::MakeMessage<FTraceControlSecureResponse>();
	Response->Host = FTraceAuxiliary::GetSecureHost();
	Response->Port = FTraceAuxiliary::GetSecurePort();
	Response->AuthorizationToken = AuthData;
	Response->CertificateAuthority = FTraceAuxiliary::GetSecureCertificate();
	if (!Response->CertificateAuthority.IsEmpty())
	{
		Response->Mode = ETraceSecureMode::SelfSignedCertAndAuthToken;
	}
	else
	{
		Response->Mode = ETraceSecureMode::AuthToken;
		Response->Identity = Identity;
	}

	MessageEndpoint->Send(Response, Context->GetSender());
#endif
}

