// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TriangleRendering.cpp: Simple triangle rendering implementation.
=============================================================================*/

#include "CanvasRendererItem.h"
#include "CanvasRender.h"
#include "PrimitiveUniformShaderParameters.h"
#include "RHIStaticStates.h"
#include "EngineModule.h"
#include "MeshPassProcessor.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "UnrealClient.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(CanvasDrawTriangles, TEXT("CanvasDrawTriangles"));

FCanvasTriangleRendererItem::FTriangleVertexFactory::FTriangleVertexFactory(
	const FStaticMeshVertexBuffers* InVertexBuffers,
	ERHIFeatureLevel::Type InFeatureLevel)
	: FLocalVertexFactory(InFeatureLevel, "FTriangleVertexFactory")
	, VertexBuffers(InVertexBuffers)
{}

void FCanvasTriangleRendererItem::FTriangleVertexFactory::InitResource(FRHICommandListBase& RHICmdList)
{
	FLocalVertexFactory::FDataType VertexData;
	VertexBuffers->PositionVertexBuffer.BindPositionVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindTangentVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(this, VertexData);
	VertexBuffers->StaticMeshVertexBuffer.BindLightMapVertexBuffer(this, VertexData, 0);
	VertexBuffers->ColorVertexBuffer.BindColorVertexBuffer(this, VertexData);
	SetData(RHICmdList, VertexData);

	FLocalVertexFactory::InitResource(RHICmdList);
}

FCanvasTriangleRendererItem::FRenderData::~FRenderData()
{
	ReleaseTriangleMesh();
}

FMeshBatch* FCanvasTriangleRendererItem::FRenderData::AllocTriangleMeshBatch(FCanvasRenderContext& InRenderContext, FHitProxyId InHitProxyId)
{
	FMeshBatch* MeshBatch = InRenderContext.Alloc<FMeshBatch>();

	MeshBatch->VertexFactory = &VertexFactory;
	MeshBatch->MaterialRenderProxy = MaterialRenderProxy;
	MeshBatch->ReverseCulling = false;
	MeshBatch->bDisableBackfaceCulling = true;
	MeshBatch->Type = PT_TriangleList;
	MeshBatch->DepthPriorityGroup = SDPG_Foreground;
	MeshBatch->BatchHitProxyId = InHitProxyId;

	FMeshBatchElement& MeshBatchElement = MeshBatch->Elements[0];
	MeshBatchElement.IndexBuffer = &IndexBuffer;
	MeshBatchElement.FirstIndex = 0;
	MeshBatchElement.NumPrimitives = 0;
	MeshBatchElement.MinVertexIndex = 0;
	MeshBatchElement.MaxVertexIndex = GetNumVertices() - 1;
	MeshBatchElement.PrimitiveUniformBufferResource = &GIdentityPrimitiveUniformBuffer;

	return MeshBatch;
}

uint32 FCanvasTriangleRendererItem::FRenderData::GetNumVertices() const
{
	return Triangles.Num() * 3;
}

uint32 FCanvasTriangleRendererItem::FRenderData::GetNumIndices() const
{
	return Triangles.Num() * 3;
}

void FCanvasTriangleRendererItem::FRenderData::InitTriangleMesh(FRHICommandListBase& RHICmdList, const FSceneView& View)
{
	const uint32 NumIndices = GetNumIndices();
	const uint32 NumVertices = GetNumIndices();
	StaticMeshVertexBuffers.PositionVertexBuffer.Init(NumVertices);
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(NumVertices, 1);
	StaticMeshVertexBuffers.ColorVertexBuffer.Init(NumVertices);

	IndexBuffer.Indices.SetNum(NumIndices);
	// Make sure the index buffer is using the appropriate size :
	IndexBuffer.ForceUse32Bit(NumVertices > MAX_uint16);

	for (int32 i = 0; i < Triangles.Num(); i++)
	{
		const uint32 StartIndex = i * 3;

		/** The use of an index buffer here is actually necessary to workaround an issue with BaseVertexIndex, DrawPrimitive, and manual vertex fetch.
		 *  In short, DrawIndexedPrimitive with StartIndex map SV_VertexId to the correct location, but DrawPrimitive with BaseVertexIndex will not.
		 */
		IndexBuffer.Indices[StartIndex + 0] = StartIndex + 0;
		IndexBuffer.Indices[StartIndex + 1] = StartIndex + 1;
		IndexBuffer.Indices[StartIndex + 2] = StartIndex + 2;

		const FCanvasUVTri& Tri = Triangles[i].Tri;

		// create verts. Notice the order is (1, 0, 2)
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(StartIndex + 0) = FVector3f(Tri.V1_Pos.X, Tri.V1_Pos.Y, 0.0f);
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(StartIndex + 1) = FVector3f(Tri.V0_Pos.X, Tri.V0_Pos.Y, 0.0f);
		StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(StartIndex + 2) = FVector3f(Tri.V2_Pos.X, Tri.V2_Pos.Y, 0.0f);

		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(StartIndex + 0, FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(StartIndex + 1, FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(StartIndex + 2, FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f));

		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(StartIndex + 0, 0, FVector2f(Tri.V1_UV.X, Tri.V1_UV.Y));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(StartIndex + 1, 0, FVector2f(Tri.V0_UV.X, Tri.V0_UV.Y));
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(StartIndex + 2, 0, FVector2f(Tri.V2_UV.X, Tri.V2_UV.Y));

		StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(StartIndex + 0) = Tri.V1_Color.ToFColor(true);
		StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(StartIndex + 1) = Tri.V0_Color.ToFColor(true);
		StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(StartIndex + 2) = Tri.V2_Color.ToFColor(true);
	}

	StaticMeshVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
	StaticMeshVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
	VertexFactory.InitResource(RHICmdList);
};

void FCanvasTriangleRendererItem::FRenderData::ReleaseTriangleMesh()
{
	VertexFactory.ReleaseResource();
	IndexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.PositionVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffers.ColorVertexBuffer.ReleaseResource();
}

void FCanvasTriangleRendererItem::FRenderData::RenderTriangles(
	FCanvasRenderContext& RenderContext,
	FMeshPassProcessorRenderState& DrawRenderState,
	const FSceneView& View,
	bool bIsHitTesting)
{
	check(IsInRenderingThread());

	if (Triangles.Num() == 0)
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(RenderContext.GraphBuilder, CanvasDrawTriangles, "%s", *MaterialRenderProxy->GetIncompleteMaterialWithFallback(GMaxRHIFeatureLevel).GetFriendlyName());

	TRACE_CPUPROFILER_EVENT_SCOPE(CanvasDrawTriangles);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CanvasDrawTriangles)

	IRendererModule& RendererModule = GetRendererModule();

	InitTriangleMesh(RenderContext.GraphBuilder.RHICmdList, View);


	// We know we have at least 1 triangle so prep up a new batch right away : 
	FMeshBatch* CurrentMeshBatch = AllocTriangleMeshBatch(RenderContext, Triangles[0].HitProxyId);
	check (CurrentMeshBatch->Elements[0].FirstIndex == 0); // The first batch should always start at the first index 

	for (int32 TriIdx = 0; TriIdx < Triangles.Num(); TriIdx++)
	{
		const FTriangleInst& Tri = Triangles[TriIdx];

		// We only need a new batch when the hit proxy id changes : 
		if (CurrentMeshBatch->BatchHitProxyId != Tri.HitProxyId)
		{
			// Flush the current batch before allocating a new one: 
			GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, *CurrentMeshBatch, bIsHitTesting, CurrentMeshBatch->BatchHitProxyId);

			CurrentMeshBatch = AllocTriangleMeshBatch(RenderContext, Tri.HitProxyId);
			CurrentMeshBatch->Elements[0].FirstIndex = 3 * TriIdx;
		}

		// Add 1 triangle to the batch :
		++CurrentMeshBatch->Elements[0].NumPrimitives;
	}

	// Flush the final batch: 
	check(CurrentMeshBatch != nullptr);
	GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, *CurrentMeshBatch, bIsHitTesting, CurrentMeshBatch->BatchHitProxyId);
}

bool FCanvasTriangleRendererItem::Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas)
{
	FGameTime Time;
	if (!bFreezeTime)
	{
		Time = Canvas->GetTime();
	}

	checkSlow(Data);

	const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();

	const FSceneViewFamily& ViewFamily = *RenderContext.Alloc<const FSceneViewFamily>(FSceneViewFamily::ConstructionValues(
		CanvasRenderTarget,
		nullptr,
		FEngineShowFlags(ESFIM_Game))
		.SetTime(Time));

	const FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	const FSceneView& View = *RenderContext.Alloc<const FSceneView>(ViewInitOptions);

	Data->RenderTriangles(RenderContext, DrawRenderState, View, Canvas->IsHitTesting());

	if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
	{
		RenderContext.DeferredRelease(MoveTemp(Data));
		Data = nullptr;
	}

	return true;
}

bool FCanvasTriangleRendererItem::Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope)
{
	FGameTime Time;
	if (!bFreezeTime)
	{
		Time = Canvas->GetTime();
	}

	checkSlow(Data);

	const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();

	const FSceneViewFamily* ViewFamily = new FSceneViewFamily(FSceneViewFamily::ConstructionValues(
		CanvasRenderTarget,
		Canvas->GetScene(),
		FEngineShowFlags(ESFIM_Game))
		.SetTime(Time));

	const FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	const FSceneView* View = new FSceneView(ViewInitOptions);

	const bool bIsHitTesting = Canvas->IsHitTesting();
	const bool bDeleteOnRender = Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender;

	RenderScope.EnqueueRenderCommand(
		[LocalData = Data, View, bIsHitTesting](FCanvasRenderContext& RenderContext) mutable
	{
		FMeshPassProcessorRenderState DrawRenderState;

		// disable depth test & writes
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

		LocalData->RenderTriangles(RenderContext, DrawRenderState, *View, bIsHitTesting);

		RenderContext.DeferredRelease(MoveTemp(LocalData));
		RenderContext.DeferredDelete(View->Family);
		RenderContext.DeferredDelete(View);
	});

	if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
	{
		Data = nullptr;
	}

	return true;
}
