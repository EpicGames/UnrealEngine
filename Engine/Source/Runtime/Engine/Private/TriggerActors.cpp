// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TriggerActors.cpp: Trigger implementation
=============================================================================*/

#include "Async/TaskGraphInterfaces.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/TriggerBase.h"
#include "Engine/TriggerBox.h"
#include "Engine/TriggerCapsule.h"
#include "Engine/TriggerSphere.h"
#include "Components/BoxComponent.h"
#include "Components/SphereComponent.h"
#include "Components/CapsuleComponent.h"
#include "UObject/ConstructorHelpers.h"
namespace
{
	static const FColor TriggerBaseColor(100, 255, 100, 255);
	static const FName TriggerCollisionProfileName("Trigger");
}
ATriggerCapsule::ATriggerCapsule(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UCapsuleComponent>(TEXT("CollisionComp")))
{
	UCapsuleComponent* CapsuleCollisionComponent = CastChecked<UCapsuleComponent>(GetCollisionComponent());
	CapsuleCollisionComponent->ShapeColor = TriggerBaseColor;
	CapsuleCollisionComponent->InitCapsuleSize(+40.0f, +80.0f);
	CapsuleCollisionComponent->SetCollisionProfileName(TriggerCollisionProfileName);

	bCollideWhenPlacing = true;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;
#if WITH_EDITORONLY_DATA
	if (UBillboardComponent* TriggerSpriteComponent = GetSpriteComponent())
	{
		TriggerSpriteComponent->SetupAttachment(CapsuleCollisionComponent);
	}

	if (GetSpriteComponent())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TriggerTextureObject;
			FName ID_Triggers;
			FText NAME_Triggers;
			FConstructorStatics()
				: TriggerTextureObject(TEXT("/Engine/EditorResources/S_TriggerCapsule"))
				, ID_Triggers(TEXT("Triggers"))
				, NAME_Triggers(NSLOCTEXT("SpriteCategory", "Triggers", "Triggers"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GetSpriteComponent()->Sprite = ConstructorStatics.TriggerTextureObject.Get();
	}
#endif
}

#if WITH_EDITOR
void ATriggerCapsule::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 500.0f : 5.0f );

	UCapsuleComponent * CapsuleComponent = CastChecked<UCapsuleComponent>(GetRootComponent());
	if ( bCtrlDown )
	{
		// CTRL+Scaling modifies trigger collision height.  This is for convenience, so that height
		// can be changed without having to use the non-uniform scaling widget (which is
		// inaccessable with spacebar widget cycling).
		const float CapsuleRadius = CapsuleComponent->GetUnscaledCapsuleRadius();
		float CapsuleHalfHeight = CapsuleComponent->GetUnscaledCapsuleHalfHeight();
		
		CapsuleHalfHeight += ModifiedScale.X;
		CapsuleHalfHeight = FMath::Max( 0.0f, CapsuleHalfHeight );
		CapsuleComponent->SetCapsuleSize(CapsuleRadius, CapsuleHalfHeight);
	}
	else
	{
		float CapsuleRadius = CapsuleComponent->GetUnscaledCapsuleRadius();
		float CapsuleHalfHeight = CapsuleComponent->GetUnscaledCapsuleHalfHeight();
		
		CapsuleRadius += ModifiedScale.X;
		CapsuleRadius = FMath::Max( 0.0f, CapsuleRadius );

		// If non-uniformly scaling, Z scale affects height and Y can affect radius too.
		if ( !ModifiedScale.AllComponentsEqual() )
		{
			// *2 to keep the capsule more capsule shaped during scaling
			CapsuleHalfHeight+= ModifiedScale.Z * 2.0f;
			CapsuleHalfHeight = FMath::Max( 0.0f, CapsuleHalfHeight );

			CapsuleRadius += ModifiedScale.Y;
			CapsuleRadius = FMath::Max( 0.0f, CapsuleRadius );
		}
		else
		{
			// uniform scaling, so apply to height as well

			// *2 to keep the capsule more capsule shaped during scaling
			CapsuleHalfHeight += ModifiedScale.Z * 2.0f;
			CapsuleHalfHeight = FMath::Max( 0.0f, CapsuleHalfHeight );
		}

		CapsuleComponent->SetCapsuleSize(CapsuleRadius, CapsuleHalfHeight);
	}
}
#endif	//WITH_EDITOR

ATriggerBox::ATriggerBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UBoxComponent>(TEXT("CollisionComp")))
{
	UBoxComponent* BoxCollisionComponent = CastChecked<UBoxComponent>(GetCollisionComponent());

	BoxCollisionComponent->ShapeColor = TriggerBaseColor;
	BoxCollisionComponent->InitBoxExtent(FVector(40.0f, 40.0f, 40.0f));
	BoxCollisionComponent->SetCollisionProfileName(TriggerCollisionProfileName);
#if WITH_EDITORONLY_DATA
	if (UBillboardComponent* TriggerSpriteComponent = GetSpriteComponent())
	{
		TriggerSpriteComponent->SetupAttachment(BoxCollisionComponent);
	}

	if (GetSpriteComponent())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TriggerTextureObject;
			FName ID_Triggers;
			FText NAME_Triggers;
			FConstructorStatics()
				: TriggerTextureObject(TEXT("/Engine/EditorResources/S_TriggerBox"))
					, ID_Triggers(TEXT("Triggers"))
					, NAME_Triggers(NSLOCTEXT("SpriteCategory", "Triggers", "Triggers"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GetSpriteComponent()->Sprite = ConstructorStatics.TriggerTextureObject.Get();
	}
#endif
}

#if WITH_EDITOR
void ATriggerBox::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 500.0f : 5.0f );

	UBoxComponent * BoxComponent = CastChecked<UBoxComponent>(GetRootComponent());
	if ( bCtrlDown )
	{
		// CTRL+Scaling modifies trigger collision height.  This is for convenience, so that height
		// can be changed without having to use the non-uniform scaling widget (which is
		// inaccessable with spacebar widget cycling).
		FVector Extent = BoxComponent->GetUnscaledBoxExtent() + FVector(0, 0, ModifiedScale.X);
		Extent.Z = FMath::Max<FVector::FReal>(0, Extent.Z);
		BoxComponent->SetBoxExtent(Extent);
	}
	else
	{
		FVector Extent = BoxComponent->GetUnscaledBoxExtent() + FVector(ModifiedScale.X, ModifiedScale.Y, ModifiedScale.Z);
		Extent.X = FMath::Max<FVector::FReal>(0, Extent.X);
		Extent.Y = FMath::Max<FVector::FReal>(0, Extent.Y);
		Extent.Z = FMath::Max<FVector::FReal>(0, Extent.Z);
		BoxComponent->SetBoxExtent(Extent);
	}
}
#endif	//WITH_EDITOR

ATriggerSphere::ATriggerSphere(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<USphereComponent>(TEXT("CollisionComp")))
{
	USphereComponent* SphereCollisionComponent = CastChecked<USphereComponent>(GetCollisionComponent());

	SphereCollisionComponent->ShapeColor = TriggerBaseColor;
	SphereCollisionComponent->InitSphereRadius(+40.0f);
	SphereCollisionComponent->SetCollisionProfileName(TriggerCollisionProfileName);
#if WITH_EDITORONLY_DATA
	if (UBillboardComponent* TriggerSpriteComponent = GetSpriteComponent())
	{
		TriggerSpriteComponent->SetupAttachment(SphereCollisionComponent);
	}

	if (GetSpriteComponent())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TriggerTextureObject;
			FName ID_Triggers;
			FText NAME_Triggers;
			FConstructorStatics()
				: TriggerTextureObject(TEXT("/Engine/EditorResources/S_TriggerSphere"))
				, ID_Triggers(TEXT("Triggers"))
				, NAME_Triggers(NSLOCTEXT("SpriteCategory", "Triggers", "Triggers"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GetSpriteComponent()->Sprite = ConstructorStatics.TriggerTextureObject.Get();
	}
#endif
}

#if WITH_EDITOR
void ATriggerSphere::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector ModifiedScale = DeltaScale * ( AActor::bUsePercentageBasedScaling ? 500.0f : 5.0f );

	float ScaleToApply = ModifiedScale.GetMax();
	const float MinScale = ModifiedScale.GetMin();

	// Use scale with the greater length
	if (FMath::Abs(MinScale) > ScaleToApply)
	{
		ScaleToApply = MinScale;
	}

	USphereComponent * SphereComponent = CastChecked<USphereComponent>(GetRootComponent());
	SphereComponent->SetSphereRadius(FMath::Max<float>(0.0f, SphereComponent->GetUnscaledSphereRadius() + ScaleToApply));
}
#endif	//WITH_EDITOR
