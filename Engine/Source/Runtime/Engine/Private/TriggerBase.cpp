// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TriggerBase.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Components/ShapeComponent.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TriggerBase)

ATriggerBase::ATriggerBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetHidden(true);
	SetCanBeDamaged(false);

	// ATriggerBase is requesting UShapeComponent which is abstract, however it is responsibility
	// of a derived class to override this type with ObjectInitializer.SetDefaultSubobjectClass.
	CollisionComponent = CreateDefaultSubobject<UShapeComponent>(TEXT("CollisionComp"));
	if (CollisionComponent)
	{
		RootComponent = CollisionComponent;
		CollisionComponent->bHiddenInGame = false;
	}

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> TriggerTextureObject;
			FName ID_Triggers;
			FText NAME_Triggers;
			FConstructorStatics()
				: TriggerTextureObject(TEXT("/Engine/EditorResources/S_Trigger"))
				, ID_Triggers(TEXT("Triggers"))
				, NAME_Triggers(NSLOCTEXT( "SpriteCategory", "Triggers", "Triggers" ))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.TriggerTextureObject.Get();
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = false;
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Triggers;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Triggers;
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif
}

