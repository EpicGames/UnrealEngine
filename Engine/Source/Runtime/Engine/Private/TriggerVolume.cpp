// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/TriggerVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TriggerVolume)

ATriggerVolume::ATriggerVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static FName CollisionProfileName("Trigger");
	GetBrushComponent()->SetCollisionProfileName(CollisionProfileName);

	bColored = true;
	BrushColor.R = 100;
	BrushColor.G = 255;
	BrushColor.B = 100;
	BrushColor.A = 255;

}

#if WITH_EDITOR

void ATriggerVolume::LoadedFromAnotherClass(const FName& OldClassName)
{
	Super::LoadedFromAnotherClass(OldClassName);

	if(GetLinkerUEVersion() < VER_UE4_REMOVE_DYNAMIC_VOLUME_CLASSES)
	{
		static FName DynamicTriggerVolume_NAME("DynamicTriggerVolume");

		if(OldClassName == DynamicTriggerVolume_NAME)
		{
			GetBrushComponent()->Mobility = EComponentMobility::Movable;
		}
	}
}

#endif

