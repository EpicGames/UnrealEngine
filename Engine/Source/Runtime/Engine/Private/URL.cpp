// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	URL.cpp: Various file-management functions.
=============================================================================*/

#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "GameMapsSettings.h"
#include "Engine/World.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

/*-----------------------------------------------------------------------------
	FURL Statics.
-----------------------------------------------------------------------------*/

// Variables.
FUrlConfig FURL::UrlConfig;
bool FURL::bDefaultsInitialized=false;

// Static init.
void FURL::StaticInit()
{
	UrlConfig.Init();
	bDefaultsInitialized = true;
}
void FURL::StaticExit()
{
	UrlConfig.Reset();
	bDefaultsInitialized = false;
}

FArchive& operator<<( FArchive& Ar, FURL& U )
{
	Ar << U.Protocol << U.Host << U.Map << U.Portal << U.Op << U.Port << U.Valid;
	return Ar;
}

/*-----------------------------------------------------------------------------
	Internal.
-----------------------------------------------------------------------------*/

static bool ValidNetChar( const TCHAR* c )
{
	// NOTE: We purposely allow for SPACE characters inside URL strings, since we need to support player aliases
	//   on the URL that potentially have spaces in them.

	// @todo: Support true URL character encode/decode (e.g. %20 for spaces), so that we can be compliant with
	//   URL protocol specifications

	// NOTE: EQUALS characters (=) are not checked here because they're valid within fragments, but incoming
	//   option data should always be filtered of equals signs

	// / Is now allowed because absolute paths are required in various places

	if( FCString::Strchr( c, '?' ) || FCString::Strchr( c, '#' ) )		// ? and # delimit fragments
	{		
		return false;
	}

	return true;
}

/*-----------------------------------------------------------------------------
	Constructors.
-----------------------------------------------------------------------------*/

FURL::FURL( const TCHAR* LocalFilename )
:	Protocol	( UrlConfig.DefaultProtocol )
,	Host		( UrlConfig.DefaultHost )
,	Port		( UrlConfig.DefaultPort )
,	Valid		( 1 )
,	Op			()
,	Portal		( UrlConfig.DefaultPortal )
{
	// strip off any extension from map name
	if (LocalFilename)
	{
		if (FPackageName::IsValidLongPackageName(LocalFilename))
		{
			Map = LocalFilename;
		}
		else
		{
			Map = FPaths::GetBaseFilename(LocalFilename);
		}
	}
	else
	{
		Map = UGameMapsSettings::GetGameDefaultMap();
	}
}

//
// Helper function.
//
static inline TCHAR* HelperStrchr( TCHAR* Src, TCHAR A, TCHAR B )
{
	TCHAR* AA = FCString::Strchr( Src, A );
	TCHAR* BB = FCString::Strchr( Src, B );
	return (AA && (!BB || AA<BB)) ? AA : BB;
}


/**
 * Static: Removes any special URL characters from the specified string
 *
 * @param Str String to be filtered
 */
void FURL::FilterURLString( FString& Str )
{
	FString NewString;
	for( int32 CurCharIndex = 0; CurCharIndex < Str.Len(); ++CurCharIndex )
	{
		const TCHAR CurChar = Str[ CurCharIndex ];

		if( CurChar != ':' && CurChar != '?' && CurChar != '#' && CurChar != '=' )
		{
			NewString.AppendChar( Str[ CurCharIndex ] );
		}
	}

	Str = NewString;
}




FURL::FURL( FURL* Base, const TCHAR* TextURL, ETravelType Type )
:	Protocol	( UrlConfig.DefaultProtocol )
,	Host		( UrlConfig.DefaultHost )
,	Port		( UrlConfig.DefaultPort )
,	Valid		( 1 )
,	Map			( UGameMapsSettings::GetGameDefaultMap() )
,	Op			()
,	Portal		( UrlConfig.DefaultPortal )
{
	check(TextURL);

	if( !bDefaultsInitialized )
	{
		FURL::StaticInit();
		Protocol = UrlConfig.DefaultProtocol;
		Host = UrlConfig.DefaultHost;
		Port = UrlConfig.DefaultPort;
		Portal = UrlConfig.DefaultPortal;
	}

	// Make a copy.
	const int32 URLLength = FCString::Strlen(TextURL);
	TCHAR* TempURL = new TCHAR[URLLength+1];
	TCHAR* URL = TempURL;

	FCString::Strncpy( TempURL, TextURL, URLLength + 1 );

	// Copy Base.
	if( Type==TRAVEL_Relative )
	{
		check(Base);
		Protocol = Base->Protocol;
		Host     = Base->Host;
		Map      = Base->Map;
		Portal   = Base->Portal;
		Port     = Base->Port;
	}
	if( Type==TRAVEL_Relative || Type==TRAVEL_Partial )
	{
		check(Base);
		for( int32 i=0; i<Base->Op.Num(); i++ )
		{
			Op.Add(Base->Op[i]);
		}
	}

	// Skip leading blanks.
	while( *URL == ' ' )
	{
		URL++;
	}

	// Options.
	TCHAR* s = HelperStrchr(URL,'?','#');
	if( s )
	{
		TCHAR OptionChar=*s, NextOptionChar=0;
		*(s++) = 0;
		do
		{
			TCHAR* t = HelperStrchr(s,'?','#');
			if( t )
			{
				NextOptionChar = *t;
				*(t++) = 0;
			}
			if( !ValidNetChar( s ) )
			{
				*this = FURL();
				Valid = 0;
				break;
			}
			if( OptionChar=='?' )
			{
				if (s && s[0] == '-')
				{
					// Remove an option if it starts with -
					s++;
					RemoveOption( s );
				}
				else
				{
					AddOption( s );
				}
			}
			else
			{
				Portal = s;
			}
			s = t;
			OptionChar = NextOptionChar;
		} while( s );
	}


	auto IsAnySlash = [](TCHAR Char) -> bool
		{
			return Char == '\\' || Char == '/';
		};

	auto FindProtocolDelim = [&IsAnySlash](TCHAR* InStr) -> TCHAR*
		{
			for (TCHAR* CurChar=InStr; *CurChar != '\0'; CurChar++)
			{
				if (CurChar[0] == ':' && IsAnySlash(CurChar[1]) && FCString::Strlen(CurChar) > 2 && IsAnySlash(CurChar[2]))
				{
					return CurChar;
				}
			}

			return nullptr;
		};

	auto ValidateProtocol = [](const TCHAR* InStr) -> bool
		{
			bool bReturnVal = FChar::IsAlpha(*InStr);

			if (bReturnVal)
			{
				for (const TCHAR* CurChar=InStr; *CurChar != '\0'; CurChar++)
				{
					if (!(FChar::IsAlnum(*CurChar) || *CurChar == '-' || *CurChar == '+' || *CurChar == '.'))
					{
						bReturnVal = false;
						break;
					}
				}
			}

			return bReturnVal;
		};

	// Pre-parse protocols in the form unreal://, to allow proper validation
	bool bParsedProtocol = false;

	if (Valid == 1 && URL != nullptr)
	{
		TCHAR* ProtocolDelim = FindProtocolDelim(URL);

		if (ProtocolDelim != nullptr)
		{
			*ProtocolDelim = '\0';

			TCHAR* CurProtocol = URL;

			Valid = ValidateProtocol(CurProtocol);

			if (Valid == 1)
			{
				Protocol = CurProtocol;
				URL = ProtocolDelim + 3;
				bParsedProtocol = true;
			}
		}
	}


	// Don't allow double slashes/backslashes, in any combination (nor "\?")
	if (Valid == 1 && URL != nullptr)
	{
		for (const TCHAR* CurChar=URL; *CurChar != '\0'; CurChar++)
		{
			if (IsAnySlash(CurChar[0]) && (IsAnySlash(CurChar[1]) || CurChar[1] == '?'))
			{
				Valid = 0;
				break;
			}
		}
	}


	FString URLStr(Valid == 1 ? URL : TEXT(""));

	if (Valid == 1)
	{
		// Handle pure filenames & Posix paths.
		bool FarHost=0;
		bool FarMap=0;

		if (URLStr.Len() > 2 && ((URLStr[0] != '[' && URLStr[0] != ':' && URLStr[1]==':') || (URLStr[0]=='/' && !FPackageName::IsValidLongPackageName(URLStr, true))) )
		{
			// Pure filename.

			if (!bParsedProtocol)
			{
				Protocol = UrlConfig.DefaultProtocol;
			}

			Map = URLStr;
			Portal = UrlConfig.DefaultPortal;
			URLStr = TEXT("");
			FarHost = 1;
			FarMap = 1;
			Host = TEXT("");
		}
		else
		{
			// Determine location of the first opening square bracket. Square brackets enclose an IPv6 address if they have a port.
			bool bPotentialIPv6Address = false;
			bool bMultipleColons = false;

			auto FindCharIdx = [](const FString& Str, const TCHAR Char) -> int32
			{
				int32 ReturnVal;

				Str.FindChar(Char, ReturnVal);

				return ReturnVal;
			};

			{
				int32 SquareBracketIdx;
				int32 FirstColonIdx = FindCharIdx(URLStr, ':');
				int32 LastColonIdx;
				int32 DotIdx = FindCharIdx(URLStr, '.');

				bPotentialIPv6Address = URLStr.FindChar('[', SquareBracketIdx);
				URLStr.FindLastChar(':', LastColonIdx);

				int32 SecondColonIdx = URLStr.Find(TEXT(":"), ESearchCase::CaseSensitive, ESearchDir::FromStart, FirstColonIdx + 1);
				bMultipleColons = FirstColonIdx != INDEX_NONE && LastColonIdx != INDEX_NONE && FirstColonIdx != LastColonIdx && SecondColonIdx != LastColonIdx;

				// Parse protocol. Don't consider colons that occur after the opening square brace,
				// because they are valid characters in an IPv6 address.
				if (!bParsedProtocol && FirstColonIdx != INDEX_NONE && FirstColonIdx > 1 &&
					((SquareBracketIdx == INDEX_NONE && !bMultipleColons) || (FirstColonIdx < SquareBracketIdx) || (bMultipleColons && URLStr[FirstColonIdx+1] == '/')) &&
					(DotIdx == INDEX_NONE || FirstColonIdx < DotIdx))
				{
					Protocol = URLStr.Left(FirstColonIdx);
					URLStr = URLStr.Mid(FirstColonIdx + 1);
				}
			}

			// Parse optional leading double-slashes.
			if (URLStr.Len() > 1 && (URLStr[0] == '/' && URLStr[1] =='/'))
			{
				URLStr = URLStr.Mid(2);
				FarHost = 1;
				Host = TEXT("");
			}

			// Parse optional host name and port.
			int32 DotIdx = FindCharIdx(URLStr, '.');
			int32 ColonIdx = FindCharIdx(URLStr, ':');
			int32 LastColonIdx;
			int32 MapExtIdx = URLStr.Find(FPackageName::GetMapPackageExtension());
			int32 MapExtLen = FPackageName::GetMapPackageExtension().Len();
			int32 SaveExtIdx = URLStr.Find(UrlConfig.DefaultSaveExt);
			int32 SaveExtLen = UrlConfig.DefaultSaveExt.Len();
			int32 DemoExtIdx = URLStr.Find(TEXT("demo"));
			int32 DemoExtLen = 4;

			URLStr.FindLastChar(':', LastColonIdx);

			auto IsAlnumChecked = [](const FString& Str, int32 Idx) -> bool
				{
					return Str.IsValidIndex(Idx) && FChar::IsAlnum(Str[Idx]);
				};

			bool bIsHostnameWithDot = DotIdx > 0 &&
				// MapExtIdx includes the dot - .umap - other extensions don't
				(DotIdx != MapExtIdx || IsAlnumChecked(URLStr, DotIdx+MapExtLen)) &&
				((DotIdx + 1) != SaveExtIdx || IsAlnumChecked(URLStr, DotIdx+SaveExtLen+1)) &&
				((DotIdx + 1) != DemoExtIdx || IsAlnumChecked(URLStr, DotIdx+4+1));


			// Square bracket indicates an IPv6 address, but IPv6 addresses can contain dots also
			// They also typically have multiple colons in them as well.
			if (bIsHostnameWithDot || bPotentialIPv6Address || (ColonIdx != INDEX_NONE && ColonIdx == LastColonIdx) || bMultipleColons)
			{
				int32 MapIdx = FindCharIdx(URLStr, '/');
				FString HostAndPort = (MapIdx != INDEX_NONE ? URLStr.Left(MapIdx) : URLStr);

				URLStr = (MapIdx != INDEX_NONE) ? URLStr.Mid(MapIdx) : TEXT("");


				// Don't override the Base Host:Port, for relative travel (not rigorously enforced in this function, selective regression fix)
				if (Type != TRAVEL_Relative || Host.IsEmpty())
				{
					// Skip past all the ':' characters in the IPv6 address to get to the port.
					int32 ClosingBracketIdx = FindCharIdx(HostAndPort, ']');
					int32 PortSearchIdx = HostAndPort.Find(":", ESearchCase::CaseSensitive, ESearchDir::FromStart, FMath::Max(ClosingBracketIdx, 0));
					int32 NextColonIdx = HostAndPort.Find(":", ESearchCase::CaseSensitive, ESearchDir::FromStart, PortSearchIdx + 1);

					// If we don't have a square bracket, check for another colon. This will tell us if we're dealing with an IPv6 address
					// if there isn't one, then we're dealing with a port.
					if (PortSearchIdx != INDEX_NONE && (ClosingBracketIdx != INDEX_NONE || NextColonIdx == INDEX_NONE))
					{
						Port = FCString::Atoi(*HostAndPort.Mid(PortSearchIdx+1));
						HostAndPort = HostAndPort.Left(PortSearchIdx);
					}

					int32 OpeningBracketIdx = FindCharIdx(HostAndPort, '[');

					// If the input was an IPv6 address with a port, we need to remove the brackets then.
					if (OpeningBracketIdx != INDEX_NONE && ClosingBracketIdx != INDEX_NONE)
					{
						Host = HostAndPort.Mid(OpeningBracketIdx + 1, (ClosingBracketIdx - OpeningBracketIdx) - 1);
					}
					else
					{
						// Otherwise, leave the address as is.
						Host = HostAndPort;
					}
				}

				if( FCString::Stricmp(*Protocol,*UrlConfig.DefaultProtocol)==0 )
				{
					Map = UGameMapsSettings::GetGameDefaultMap();
				}
				else
				{
					Map = TEXT("");
				}

				FarHost = 1;
			}
		}
	}

	// Parse optional map
	if (Valid == 1 && URLStr.Len() > 0)
	{
		// Map.
		if (URLStr[0] != '/')
		{
			// find full pathname from short map name
			FString MapFullName;
			FText MapNameError;
			bool bFoundMap = false;

			if (FPaths::FileExists(URLStr) && FPackageName::TryConvertFilenameToLongPackageName(URLStr, MapFullName))
			{
				Map = MapFullName;
				bFoundMap = true;
			}
			else if (!FPackageName::DoesPackageNameContainInvalidCharacters(URLStr, &MapNameError))
			{
				// First try to use the asset registry if it is available and finished scanning
				if (FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
				{
					IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

					if (!AssetRegistry.IsLoadingAssets())
					{
						TArray<FAssetData> MapList;

						FARFilter Filter;
						Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
						Filter.WithoutPackageFlags = PKG_CookGenerated | PKG_PlayInEditor;

						if (AssetRegistry.GetAssets(Filter, /*out*/ MapList))
						{
							FName TargetTestName(*URLStr);
							for (const FAssetData& MapAsset : MapList)
							{
								if (MapAsset.AssetName == TargetTestName)
								{
									Map = MapAsset.PackageName.ToString();
									bFoundMap = true;
									break;
								}
							}
						}
					}
				}

				if (!bFoundMap)
				{
					// Fall back to a slow AssetRegistry scan for the package
					IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
					FName ExistingPackageName = AssetRegistry.GetFirstPackageByName(URLStr);
					if (!ExistingPackageName.IsNone())
					{
						ExistingPackageName.ToString(Map);
						bFoundMap = true;
					}
				}
			}

			if (!bFoundMap)
			{
				// can't find file, invalidate and bail
				UE_LOGF(LogLongPackageNames, Warning, "Can't Find URL: %ls: %ls. Invalidating and reverting to Default URL.", *URLStr, *MapNameError.ToString());
				*this = FURL();
				Valid = 0;
			}
		}
		else
		{
			// already a full pathname
			Map = URLStr;
		}

	}
	
	// Validate everything.
	// The FarHost check does not serve any purpose I can see, and will just cause valid URLs to fail (URLs with no options, why does a URL
	// need an option to be valid?)
	if (Valid == 1 && (!ValidateProtocol(*Protocol) || !ValidNetChar(*Host) /*|| !ValidNetChar(*Map)*/ || !ValidNetChar(*Portal) /*|| (!FarHost && !FarMap && !Op.Num())*/))
	{
		*this = FURL();
		Valid = 0;
	}

	// If Valid == 1, success.

	delete[] TempURL;
}

/*-----------------------------------------------------------------------------
	Conversion to text.
-----------------------------------------------------------------------------*/

//
// Convert this URL to text.
//
FString FURL::ToString(bool FullyQualified) const
{
	FString Result;

	// Emit protocol.
	if ((Protocol != UrlConfig.DefaultProtocol) || FullyQualified)
	{
		Result += Protocol;
		Result += TEXT(":");

		if (Host != UrlConfig.DefaultHost)
		{
			Result += TEXT("//");
		}
	}

	// Emit host and port
	if ((Host != UrlConfig.DefaultHost) || (Port != UrlConfig.DefaultPort))
	{
		Result += GetHostPortString();

		if (!Map.StartsWith(TEXT("/")) && !Map.StartsWith(TEXT("\\")))
		{
			Result += TEXT("/");
		}
	}

	// Emit map.
	if (Map.Len() > 0)
	{
		Result += Map;
	}

	// Emit options.
	for (int32 i = 0; i < Op.Num(); i++)
	{
		Result += TEXT("?");
		Result += Op[i];
	}

	// Emit portal.
	if (Portal.Len() > 0)
	{
		Result += TEXT("#");
		Result += Portal;
	}

	return Result;
}

//
// Convert the host and port values of this URL into a string that's safe for serialization
//
FString FURL::GetHostPortString() const
{
	FString Result;
	int32 FirstColonIndex, LastColonIndex;
	bool bNotUsingDefaultPort = (Port != UrlConfig.DefaultPort);

	// If this is an IPv6 address (determined if there's more than one colon) 
	// and we're going to be adding the port, we need to put in the brackets
	// This is done because there's no sane way to serialize the address otherwise
	if (Host.FindChar(':', FirstColonIndex) && Host.FindLastChar(':', LastColonIndex)
		&& FirstColonIndex != LastColonIndex && bNotUsingDefaultPort)
	{
		Result += FString::Printf(TEXT("[%s]"), *Host);
	}
	else
	{
		// Otherwise print the IPv6/IPv4 address as is
		Result += Host;
	}


	if (bNotUsingDefaultPort)
	{
		Result += TEXT(":");
		Result += FString::Printf(TEXT("%i"), Port);
	}

	return Result;
}

/*-----------------------------------------------------------------------------
	Informational.
-----------------------------------------------------------------------------*/

bool FURL::IsInternal() const
{
	return Protocol == UrlConfig.DefaultProtocol;
}

bool FURL::IsLocalInternal() const
{
	return IsInternal() && Host.Len()==0;
}

void FURL::AddOption( const TCHAR* Str )
{
	int32 Match = FCString::Strchr(Str, '=') ? (FCString::Strchr(Str, '=') - Str) : FCString::Strlen(Str);
	int32 i;

	for (i = 0; i < Op.Num(); i++)
	{
		if (FCString::Strnicmp(*Op[i], Str, Match) == 0 && ((*Op[i])[Match] == '=' || (*Op[i])[Match] == '\0'))
		{
			break;
		}
	}

	if (i == Op.Num())
	{
		Op.Emplace(Str);
	}
	else
	{
		Op[i] = Str;
	}
}


void FURL::RemoveOption( const TCHAR* Key, const TCHAR* Section, const FString& Filename )
{
	if ( !Key || *Key == '\0')
		return;

	for ( int32 i = Op.Num() - 1; i >= 0; i-- )
	{
		if ( Op[i].Left(FCString::Strlen(Key)) == Key )
		{
			if (GConfig->RemoveKeyFromSection(Section ? Section : TEXT("DefaultPlayer"), Key, Filename))
			{
				GConfig->Flush( 0, Filename );
			}

			Op.RemoveAt(i);
		}
	}
}

void FURL::LoadURLConfig( const TCHAR* Section, const FString& Filename )
{
	TArray<FString> Options;
	GConfig->GetSection( Section, Options, Filename );
	for( int32 i=0;i<Options.Num();i++ )
	{
		AddOption( *Options[i] );
	}
}

void FURL::SaveURLConfig( const TCHAR* Section, const TCHAR* Item, const FString& Filename ) const
{
	for( int32 i=0; i<Op.Num(); i++ )
	{
		TCHAR Temp[1024];
		FCString::Strcpy( Temp, *Op[i] );
		TCHAR* Value = FCString::Strchr(Temp,'=');
		if( Value )
		{
			*Value++ = 0;
			if( FCString::Stricmp(Temp,Item)==0 )
				GConfig->SetString( Section, Temp, Value, Filename );
		}
	}
}

bool FURL::HasOption( const TCHAR* Test ) const
{
	return GetOption( Test, NULL ) != NULL;
}

const TCHAR* FURL::GetOption( const TCHAR* Match, const TCHAR* Default ) const
{
	const int32 Len = FCString::Strlen(Match);
	
	if( Len > 0 )
	{
		for( int32 i = 0; i < Op.Num(); i++ ) 
		{
			const TCHAR* s = *Op[i];
			if( FCString::Strnicmp( s, Match, Len ) == 0 ) 
			{
				if (s[Len-1] == '=' || s[Len] == '=' || s[Len] == '\0') 
				{
					return s + Len;
				}
			}
		}
	}

	return Default;
}

/*-----------------------------------------------------------------------------
	Comparing.
-----------------------------------------------------------------------------*/

bool FURL::operator==( const FURL& Other ) const
{
	if
	(	Protocol	!= Other.Protocol
	||	Host		!= Other.Host
	||	Map			!= Other.Map
	||	Port		!= Other.Port
	||  Op.Num()    != Other.Op.Num() )
		return 0;

	for( int32 i=0; i<Op.Num(); i++ )
		if( Op[i] != Other.Op[i] )
			return 0;

	return 1;
}

/*-----------------------------------------------------------------------------
	Testing
-----------------------------------------------------------------------------*/
#if WITH_DEV_AUTOMATION_TESTS && !UE_BUILD_SHIPPING

struct FURLTestCase
{
	FURLTestCase(const FString& InQueryString, const FString& InHost, const FString InProtocol=TEXT(""), int32 InPort = -1) :
		QueryString(InQueryString),
		Protocol(InProtocol),
		Host(InHost),
		Port(InPort)
	{

	}
	FString QueryString;
	FString Protocol;
	FString Host;
	int32 Port;
};

static bool URLSerializationTests(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	TArray<FURLTestCase> TestCases;
	if (FParse::Command(&Cmd, TEXT("URLSERIALIZATION")))
	{
		TestCases.Push(FURLTestCase(TEXT("2001:0db8:85a3:0000:0000:8a2e:0370:7334"), TEXT("2001:0db8:85a3:0000:0000:8a2e:0370:7334")));
		TestCases.Push(FURLTestCase(TEXT("[2001:0db8:85a3:0000:0000:8a2e:0370:7334]:7778"), TEXT("2001:0db8:85a3:0000:0000:8a2e:0370:7334"), TEXT(""), 7778));
		TestCases.Push(FURLTestCase(TEXT("[2001:db8:85a3::8a2e:370:7334]"), TEXT("2001:db8:85a3::8a2e:370:7334")));
		TestCases.Push(FURLTestCase(TEXT("epic://2001:db8:85a3::8a2e:370:7334"), TEXT("2001:db8:85a3::8a2e:370:7334"), TEXT("epic")));
		TestCases.Push(FURLTestCase(TEXT("192.168.0.1:17777"), TEXT("192.168.0.1"), TEXT(""), 17777));
		TestCases.Push(FURLTestCase(TEXT("test://192.168.0.1"), TEXT("192.168.0.1"), TEXT("test")));
		TestCases.Push(FURLTestCase(TEXT("::ffff:192.168.0.1"), TEXT("::ffff:192.168.0.1")));
		TestCases.Push(FURLTestCase(TEXT("[::ffff:192.168.0.1]"), TEXT("::ffff:192.168.0.1")));
		TestCases.Push(FURLTestCase(TEXT("[::ffff:192.168.0.1]:7778"), TEXT("::ffff:192.168.0.1"), TEXT(""), 7778));
		TestCases.Push(FURLTestCase(TEXT("test://::ffff:192.168.0.1"), TEXT("::ffff:192.168.0.1"), TEXT("test")));
		TestCases.Push(FURLTestCase(TEXT("unreal:192.168.0.1:7776"), TEXT("192.168.0.1"), TEXT("unreal"), 7776));
		TestCases.Push(FURLTestCase(TEXT("192.168.0.1"), TEXT("192.168.0.1")));
		TestCases.Push(FURLTestCase(TEXT("http://[::ffff:192.168.0.1]:8080"), TEXT("::ffff:192.168.0.1"), TEXT("http"), 8080));
		TestCases.Push(FURLTestCase(TEXT("https:[2001:db8:85a3::8a2e:370:7334]:443"), TEXT("2001:db8:85a3::8a2e:370:7334"), TEXT("https"), 443));
		TestCases.Push(FURLTestCase(TEXT("steam.76561197993275299:20/"), TEXT("steam.76561197993275299"), TEXT(""), 20));
		TestCases.Push(FURLTestCase(TEXT("unreal::44750/"), TEXT(""), TEXT("unreal"), 44750));

		bool bAllCasesPassed = true;
		for (const auto& TestCase : TestCases)
		{
			for (int32 TestType=0; TestType<2; TestType++)
			{
				bool bConversionTest = TestType == 1;
				FString CurQueryString;

				if (bConversionTest)
				{
					FURL ConversionURL(nullptr, *TestCase.QueryString, TRAVEL_Absolute);

					CurQueryString = ConversionURL.ToString();
				}
				else
				{
					CurQueryString = TestCase.QueryString;
				}

				FURL TestURL(nullptr, *CurQueryString, TRAVEL_Absolute);
				// This is ugly, but we use it to report which cases failed and why.
				const bool bHostMatched = (TestURL.Host == TestCase.Host);
				const bool bPortMatched = (TestCase.Port == -1 || TestCase.Port == TestURL.Port);
				const bool bProtocolMatched = (TestCase.Protocol.IsEmpty() || TestCase.Protocol == TestURL.Protocol);

				if (bHostMatched && bPortMatched &&	bProtocolMatched)
				{
					UE_LOGF(LogCore, Log, "Test %ls (%ls) passed!", *TestCase.QueryString, (bConversionTest ? *CurQueryString : TEXT("")));
				}
				else
				{
					bAllCasesPassed = false;
					UE_LOGF(LogCore, Warning, "Test %ls (%ls) failed! Matching flags: Host[%d] Port[%d] Protocol[%d]", *TestCase.QueryString,
							(bConversionTest ? *CurQueryString : TEXT("")), bHostMatched, bPortMatched, bProtocolMatched);
					if (!bHostMatched)
					{
						UE_LOGF(LogCore, Warning, "URL had host %ls, expected %ls", *TestURL.Host, *TestCase.Host);
					}

					if (!bPortMatched)
					{
						UE_LOGF(LogCore, Warning, "URL had port %d, expected %d", TestURL.Port, TestCase.Port);
					}

					if (!bProtocolMatched)
					{
						UE_LOGF(LogCore, Warning, "URL had protocol %ls, expected %ls", *TestURL.Protocol, *TestCase.Protocol);
					}
				}
			}
		}

		if (bAllCasesPassed)
		{
			UE_LOGF(LogCore, Log, "All URL serialization cases passed.");
		}
		else
		{
			UE_LOGF(LogCore, Warning, "An URL serialization case failed!");
		}

		return true;
	}

	return false;
}

FStaticSelfRegisteringExec FURLTests(URLSerializationTests);
#endif // WITH_DEV_AUTOMATION_TESTS && !UE_BUILD_SHIPPING

