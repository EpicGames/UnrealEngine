// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include "UObject/UnrealNames.h"
#include "UObject/Package.h"
#include "LevelUtils.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "UnrealEngine.h"
#include "Engine/World.h"
#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "Misc/EditorPathHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FActorLocatorFragment> FActorLocatorFragment::FragmentType;
UE::UniversalObjectLocator::TParameterTypeHandle<FActorLocatorFragmentResolveParameter> FActorLocatorFragmentResolveParameter::ParameterType;

ULevel* GetLevelFromContext(const UObject* InContext)
{
	ULevel* Level = Cast<ULevel>(const_cast<UObject*>(InContext));
	if (!Level && InContext)
	{
		Level = InContext->GetTypedOuter<ULevel>();
	}
	return Level;
}

UObject* ResolveActorWithinLevel(const FActorLocatorFragment& Payload, ULevel* Level)
{
	// Default to owning world (to resolve AlwaysLoaded actors not part of a Streaming Level and Disabled Streaming World Partitions)
	UWorld* StreamingWorld = nullptr;

	// Construct the path to the level asset that the streamed level relates to
	ULevelStreaming* LevelStreaming = ULevelStreaming::FindStreamingLevel(Level);
	if (LevelStreaming)
	{
		// If we're loading a world partition runtime cell, we need to find the streaming world that is responsible for resolving those actors
		if (Level->IsWorldPartitionRuntimeCell())
		{
			StreamingWorld = LevelStreaming->GetStreamingWorld();
			check(StreamingWorld);
			LevelStreaming = ULevelStreaming::FindStreamingLevel(StreamingWorld->PersistentLevel);
		}
		else
		{
			StreamingWorld = Level->GetTypedOuter<UWorld>();
		}
	}

	if (LevelStreaming && StreamingWorld)
	{
		// StreamedLevelPackage is a package name of the form /Game/Folder/MapName, not a full asset path
		FName StreamedPackageName = (LevelStreaming->PackageNameToLoad == NAME_None) ? LevelStreaming->GetWorldAssetPackageFName() : LevelStreaming->PackageNameToLoad;

		// @todo: we're only checking package name here - to be 100% correct we should really check the asset name as well,
		//        but that is probably not necessary because multiple level assets in a single package are not supported
		if (Payload.Path.GetAssetPath().GetPackageName() == StreamedPackageName)
		{
			// Payload.Path.GetSubPathString() specifies the path from the package (so includes PersistentLevel.) so we must do a ResolveSubObject from its outer

			UObject* ResolvedObject = nullptr;
			StreamingWorld->ResolveSubobject(*Payload.Path.GetSubPathString(), ResolvedObject, /*bLoadIfExists*/false);
			return ResolvedObject;
		}
	}

	return nullptr;
}

UE::UniversalObjectLocator::FResolveResult FActorLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	const FActorLocatorFragmentResolveParameter* Parameter = Params.FindParameter<FActorLocatorFragmentResolveParameter>();

	// If we have a custom fragment parameter specified and it matches the source asset path of this fragment's payload,
	//   use the streaming world specified by the source asset path.
	if (Parameter && Parameter->StreamingWorld && Parameter->SourceAssetPath == Path.GetAssetPath())
	{
		if (!Parameter->ContainerID.IsMainContainer())
		{
			// Append the ContainerID and lookup
			const FString SubPathString = FWorldPartitionLevelHelper::AddActorContainerIDToSubPathString(
				Parameter->ContainerID, Path.GetSubPathString());

			Parameter->StreamingWorld->ResolveSubobject(*SubPathString, Result, /*bLoadIfExists*/false);
		}
		else
		{
			// Traditional level streaming needs to resolve bindings from the actual world that owns the streamed level
			Parameter->StreamingWorld->ResolveSubobject(*Path.GetSubPathString(), Result, /*bLoadIfExists*/false);
		}
	}

	if (Result)
	{
		return FResolveResultData(Result);
	}

	// Next handle default level streaming and partition worlds behavior
	if (ULevel* Level = GetLevelFromContext(Params.Context))
	{
		Result = ResolveActorWithinLevel(*this, Level);
	}

	if (Result)
	{
		return FResolveResultData(Result);
	}

#if WITH_EDITORONLY_DATA

	const UPackage* ContextPackage = Params.Context ? Params.Context->GetOutermost() : nullptr;
	const int32     PIEInstanceID  = ContextPackage ? ContextPackage->GetPIEInstanceID() : INDEX_NONE;

	// The Actor Fragment is explicit about providing a resolution context for its bindings. We never want to resolve to objects
	// with a different PIE instance ID, even if the current callstack is being executed inside a different GPlayInEditorID
	// scope. Since ResolveObject will always call FixupForPIE in editor based on GPlayInEditorID, we always override the current
	// GPlayInEditorID to be the current PIE instance of the provided context.
	FTemporaryPlayInEditorIDOverride PIEGuard(PIEInstanceID);

	// Finally fallback to just trying to resolve the path directly
	auto ResolvePathWithPIEHandling = [PIEInstanceID](const FSoftObjectPath& PathPtr)
	{
		// If we are resolving within a PIE instance, fixup the PIE instance on the path first
		if (PIEInstanceID != INDEX_NONE)
		{
			FSoftObjectPath PIEPath = PathPtr;
			PIEPath.FixupForPIE(PIEInstanceID);
			return PIEPath.ResolveObject();
		}
		return PathPtr.ResolveObject();
	};

	Result = ResolvePathWithPIEHandling(Path);

	if (!Result)
	{
		// If the path failed to resolve, attempt to fixup redirectors on the path to handle cases
		//     where this path hasn't been saved yet, but references things that have (and have been redirected)
		FSoftObjectPath TempPath = Path;
		TempPath.PreSavePath();

		if (TempPath != Path)
		{
			Result = ResolvePathWithPIEHandling(TempPath);
		}
	}

#else  // WITH_EDITORONLY_DATA

	// By default we just resolve the path directly
	Result = Path.ResolveObject();

#endif // WITH_EDITORONLY_DATA

	return FResolveResultData(Result);
}

void FActorLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FActorLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FActorLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

#if WITH_EDITOR
	if (InParams.Context)
	{
		Path = FEditorPathHelper::GetEditorPathFromReferencer(InParams.Object, InParams.Context);
	}
	else
	{
		Path = FEditorPathHelper::GetEditorPath(InParams.Object);
	}
#else
	Path = InParams.Object;
#endif

	// Fixup PIE prefixes so that re always reference the non-PIE instance object
#if WITH_EDITORONLY_DATA
	UPackage* ObjectPackage = InParams.Object->GetOutermost();
	if (ensure(ObjectPackage))
	{
		// If this is being set from PIE we need to remove the pie prefix and point to the editor object
		if (ObjectPackage->GetPIEInstanceID() != INDEX_NONE)
		{
			TStringBuilder<16> PIEPrefix;
			PIEPrefix.Appendf(PLAYWORLD_PACKAGE_PREFIX TEXT("_%d_"), ObjectPackage->GetPIEInstanceID());

			FString NewPath = Path.ToString();
			NewPath.ReplaceInline(*PIEPrefix, TEXT(""));
			Path.SetPath(NewPath);
		}
	}
#endif

	// Really, actors should be relative to their level in order to support streaming within level instances, but
	//   world partition makes that impossible
	return FInitializeResult::Absolute();
}

uint32 FActorLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// Can only reference actors
	if (ObjectToReference->IsA<AActor>())
	{
		// This locator should always be used over subobject locators in order to ensure they are used even if the context is a level
		return 2000;
	}
	return 0;
}
