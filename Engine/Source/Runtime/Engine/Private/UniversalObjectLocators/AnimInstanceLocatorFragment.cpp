// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"

#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimInstanceLocatorFragment)

#define LOCTEXT_NAMESPACE "AnimInstanceLocatorFragment"

namespace UE::UniversalObjectLocator
{
	static constexpr FStringView DefaultAnimInstance(TEXTVIEW("default"));
	static constexpr FStringView PostProcessAnimInstance(TEXTVIEW("post_process"));
}

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimInstanceLocatorFragment> FAnimInstanceLocatorFragment::FragmentType;

uint32 FAnimInstanceLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	if (ObjectToReference && ObjectToReference->IsA<UAnimInstance>())
	{
		return 1000;
	}
	return 0;
}

UE::UniversalObjectLocator::FResolveResult FAnimInstanceLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	const USkeletalMeshComponent* SkelMeshComp = Cast<const USkeletalMeshComponent>(Params.Context);

	if (SkelMeshComp)
	{
		if (Type == EAnimInstanceLocatorFragmentType::PostProcessAnimInstance)
		{
			return UE::UniversalObjectLocator::FResolveResultData(SkelMeshComp->GetPostProcessInstance());
		}
		else
		{
			return UE::UniversalObjectLocator::FResolveResultData(SkelMeshComp->GetAnimInstance());
		}
	}

	return UE::UniversalObjectLocator::FResolveResult();
}

UE::UniversalObjectLocator::FInitializeResult FAnimInstanceLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	// 3 Different options here:
	// Context is a USkeletalMeshComponent
	// 		Result: Payload with a Type as either AnimInstance or PostProcessAnimInstance depending on what Object is
	// Object is a USkeletalMeshComponent, Context is nullptr
	// 		Result: ParentContext points to the skel mesh component, Type depends on UAnimInstance
	// Object is a UAnimInstance, Context is nullptr
	// 		Result: ParentContext points to the skel mesh component, Type depends on UAnimInstance

	const USkeletalMeshComponent* SkelMeshComp = InParams.GetContextAs<USkeletalMeshComponent>();
	if (SkelMeshComp)
	{
		const UAnimInstance* AnimInstance = InParams.GetObjectAs<const UAnimInstance>();
		if (AnimInstance && AnimInstance == SkelMeshComp->GetPostProcessInstance())
		{
			// Use the post process instance
			Type = EAnimInstanceLocatorFragmentType::PostProcessAnimInstance;
		}
		else
		{
			// Assume main anim instance
			Type = EAnimInstanceLocatorFragmentType::AnimInstance;
		}

		return FInitializeResult::Relative(SkelMeshComp);
	}
	else
	{
		// Context needs to be baked into the locator
		const UAnimInstance* AnimInstance = InParams.GetObjectAs<const UAnimInstance>();
		if (AnimInstance)
		{
			SkelMeshComp = AnimInstance->GetTypedOuter<const USkeletalMeshComponent>();
		}
		else
		{
			SkelMeshComp = InParams.GetObjectAs<const USkeletalMeshComponent>();
		}

		if (!ensureMsgf(SkelMeshComp, TEXT("Unable to create an Anim Instance locator for object %s since there is no valid skeletal mesh."), InParams.Object ? *InParams.Object->GetName() : TEXT("<<None>>")))
		{
			return FInitializeResult::Failure();
		}

		if (AnimInstance && AnimInstance == SkelMeshComp->GetPostProcessInstance())
		{
			// Use the post process instance
			Type = EAnimInstanceLocatorFragmentType::PostProcessAnimInstance;
		}
		{
			// Assume main anim instance
			Type = EAnimInstanceLocatorFragmentType::AnimInstance;
		}

		return FInitializeResult::Relative(SkelMeshComp);
	}
}

void FAnimInstanceLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	using namespace UE::UniversalObjectLocator;

	switch (Type)
	{
	case EAnimInstanceLocatorFragmentType::AnimInstance:
		// Leave blank for default
		// OutStringBuilder.Append(DefaultAnimInstance);
		break;
	case EAnimInstanceLocatorFragmentType::PostProcessAnimInstance:
		OutStringBuilder.Append(PostProcessAnimInstance);
		break;
	}
}

UE::UniversalObjectLocator::FParseStringResult FAnimInstanceLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	using namespace UE::UniversalObjectLocator;

	if (InString.Len() == 0 || InString.Compare(DefaultAnimInstance, ESearchCase::IgnoreCase) == 0)
	{
		Type = EAnimInstanceLocatorFragmentType::AnimInstance;
		return FParseStringResult().Success();
	}
	else if (InString.Compare(PostProcessAnimInstance, ESearchCase::IgnoreCase) == 0)
	{
		Type = EAnimInstanceLocatorFragmentType::AnimInstance;
		return FParseStringResult().Success();
	}

	return FParseStringResult().Failure(
		UE_UOL_PARSE_ERROR(Params,
			FText::Format(
				LOCTEXT("Error_InvalidAnimInstanceType", "Invalid anim instance type '{0}'."),
				FText::FromStringView(InString)
			)
		)
	);
}

#undef LOCTEXT_NAMESPACE
