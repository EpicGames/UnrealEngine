// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocators/AssetLocatorFragment.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "IUniversalObjectLocatorModule.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FAssetLocatorFragment> FAssetLocatorFragment::FragmentType;

FAssetLocatorFragment::FAssetLocatorFragment(const FAssetData& InAssetData)
	: Path(InAssetData.PackageName, InAssetData.AssetName)
{
}

FAssetLocatorFragment::FAssetLocatorFragment(const FTopLevelAssetPath& InPath)
	: Path(InPath)
{
}

UE::UniversalObjectLocator::FResolveResult FAssetLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	FString PathString = Path.ToString();

	if (EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Unload))
	{
		// Assets cannot be explicitly unloaded
		return FResolveResult();
	}

	FResolveResultData Result;

	Result.Object = FindObject<UObject>(nullptr, *PathString);
	if (Result.Object == nullptr && EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Load))
	{
		Result.Object = StaticLoadObject(UObject::StaticClass(), nullptr, *PathString, nullptr, LOAD_None, nullptr, true);
		Result.Flags.bWasLoaded = (Result.Object != nullptr);
	}

#if WITH_EDITOR
	// Look at core redirects if we didn't find the object
	if (!Result.Object)
	{
		FSoftObjectPath FixupObjectPath(MoveTemp(PathString));
		if (FixupObjectPath.FixupCoreRedirects())
		{
			FString FixedUpPathString = FixupObjectPath.ToString();

			Result.Object = FindObject<UObject>(nullptr, *FixedUpPathString);
			if (Result.Object == nullptr && EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Load))
			{
				Result.Object = LoadObject<UObject>(nullptr, *FixedUpPathString);
				Result.Flags.bWasLoaded = (Result.Object != nullptr);
			}
		}
	}
#endif

	while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Result.Object))
	{
		Result.Object = Redirector->DestinationObject;
	}

	return Result;
}

UE::UniversalObjectLocator::FInitializeResult FAssetLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	Path.TrySetPath(InParams.Object);
	return FInitializeResult::Absolute();
}

void FAssetLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FAssetLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

uint32 FAssetLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// Can only reference top level persistent assets
	UObject* Outer = ObjectToReference->GetOuter();
	if (Outer && Outer->IsA<UPackage>() && !ObjectToReference->HasAnyFlags(RF_Transient) && Outer != GetTransientPackage())
	{
		return 1000;
	}

	// We can't use this at all
	return 0;
}
