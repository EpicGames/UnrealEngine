// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocators/UniversalObjectLocatorScriptingExtensions.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniversalObjectLocatorScriptingExtensions)

FUniversalObjectLocator UUniversalObjectLocatorScriptingExtensions::MakeUniversalObjectLocator(UObject* Object, UObject* Context)
{
	return FUniversalObjectLocator(Object, Context);
}

FUniversalObjectLocator UUniversalObjectLocatorScriptingExtensions::UniversalObjectLocatorFromString(const FString& InString)
{
	using namespace::UE::UniversalObjectLocator;

	FParseStringParams Params;
	Params.Flags = EParseStringFlags::ErrorMessaging;

	FUniversalObjectLocator Locator;
	FParseStringResult Result = Locator.TryParseString(InString, Params);
	if (!Result)
	{
		FFrame::KismetExecutionMessage(
			*FText::Format(
				NSLOCTEXT("UOL", "ParseError", "Unable to parse a valid UOL from string '{0}': {1}."),
				FText::FromStringView(InString),
				Result.ErrorMessage
			).ToString()
		, ELogVerbosity::Error);
	}
	return Locator;
}

bool UUniversalObjectLocatorScriptingExtensions::IsEmpty(const FUniversalObjectLocator& Locator)
{
	return Locator.IsEmpty();
}

FString UUniversalObjectLocatorScriptingExtensions::ToString(const FUniversalObjectLocator& Locator)
{
	TStringBuilder<128> String;
	Locator.ToString(String);
	return String.ToString();
}

UObject* UUniversalObjectLocatorScriptingExtensions::SyncFind(const FUniversalObjectLocator& Locator, UObject* Context)
{
	return Locator.SyncFind(Context);
}

UObject* UUniversalObjectLocatorScriptingExtensions::SyncLoad(const FUniversalObjectLocator& Locator, UObject* Context)
{
	return Locator.SyncLoad(Context);
}

void UUniversalObjectLocatorScriptingExtensions::SyncUnload(const FUniversalObjectLocator& Locator, UObject* Context)
{
	Locator.SyncUnload(Context);
}
