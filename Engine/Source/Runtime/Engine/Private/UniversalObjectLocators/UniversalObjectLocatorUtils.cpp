// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocators/UniversalObjectLocatorUtils.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/Level.h"

namespace UE::UniversalObjectLocator
{
	AActor* SpawnActorForLocator(UWorld* World, TSubclassOf<AActor> ActorClass, FName ActorName, AActor* TemplateActor/* = nullptr*/)
	{
		if (World)
		{
			const EObjectFlags ObjectFlags = RF_Transient | RF_Transactional;

			FName SpawnName =
#if WITH_EDITOR

				// Construct the object with the same name that we will set later on the actor to avoid renaming it insSide SetActorLabel
				!ActorName.IsNone() ? MakeUniqueObjectName(World->PersistentLevel.Get(), ActorClass->GetClass(), ActorName) : NAME_None;
#else
			NAME_None;
#endif

			// If there's an object that already exists with the requested name, it needs to be renamed (it's probably pending kill)
			if (!SpawnName.IsNone())
			{
				UObject* ExistingObject = StaticFindObjectFast(nullptr, World->PersistentLevel, SpawnName);
				if (ExistingObject)
				{
					FName DefunctName = MakeUniqueObjectName(World->PersistentLevel.Get(), ExistingObject->GetClass());
					ExistingObject->Rename(*DefunctName.ToString(), nullptr);
				}
			}

			// Spawn the preview actor
			FActorSpawnParameters SpawnInfo;
			{
				SpawnInfo.Name = *SpawnName.ToString().Replace(TEXT(" "), TEXT("_"));
				SpawnInfo.Name = MakeUniqueObjectName(World->PersistentLevel.Get(), ActorClass, SpawnInfo.Name);
				SpawnInfo.ObjectFlags = ObjectFlags;
				SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				// allow pre-construction variables to be set.
				SpawnInfo.bDeferConstruction = true;
				SpawnInfo.OverrideLevel = World->PersistentLevel.Get();
				SpawnInfo.Template = TemplateActor;
			}

			AActor* SpawnedActor = World->SpawnActor<AActor>(ActorClass, SpawnInfo);

#if WITH_EDITOR
			if (GIsEditor)
			{
				// Explicitly set RF_Transactional on spawned actors so we can undo/redo properties on them.
				SpawnedActor->SetFlags(RF_Transactional);

				for (UActorComponent* Component : SpawnedActor->GetComponents())
				{
					if (Component)
					{
						Component->SetFlags(RF_Transactional);
					}
				}
			}
#endif

			const bool bIsDefaultTransform = true;
			SpawnedActor->FinishSpawning(FTransform());

#if WITH_EDITOR
			// Don't set the actor label in PIE as this requires flushing async loading.
			if (World->WorldType == EWorldType::Editor)
			{
				SpawnedActor->SetActorLabel(!ActorName.IsNone() ? ActorName.ToString() : SpawnInfo.Name.ToString());
			}
#endif
			return SpawnedActor;
		}
		return nullptr;
	}

	void DestroyActorForLocator(AActor* Actor)
	{
		if (Actor)
		{
			// Destroy the Actor we made previously
#if WITH_EDITOR
			if (GIsEditor)
			{
				// Explicitly remove RF_Transactional on spawned actors since we don't want to trasact spawn/destroy events
				Actor->ClearFlags(RF_Transactional);
				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (Component)
					{
						Component->ClearFlags(RF_Transactional);
					}
				}
			}
#endif
			Actor->Destroy();
		}
	}


} // namespace UE::UniversalObjectLocator