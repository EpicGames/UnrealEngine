// Copyright Epic Games, Inc. All Rights Reserved.


#include "UnrealClient.h"
#include "BatchedElements.h"
#include "ContentStreaming.h"
#include "Engine/GameViewportClient.h"
#include "Misc/FileHelper.h"
#include "DummyViewport.h"
#include "EngineStats.h"
#include "Input/PopupMethodReply.h"
#include "RenderCaptureInterface.h"
#include "RenderingThread.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "InputKeyEventArgs.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Math/Float16Color.h"
#include "UnrealEngine.h"
#include "Components/PostProcessComponent.h"
#include "HighResScreenshot.h"
#include "HModel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/CoreDelegates.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/PostProcessVolume.h"
#include "Performance/EnginePerformanceTargets.h"
#include "Elements/Framework/TypedElementList.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "RenderCounters.h"
#include "RenderGraphUtils.h"
#include "DynamicResolutionState.h"
#include "Stats/StatsTrace.h"
#include "Stats/ThreadIdleStats.h"
#include "RHIUtilities.h"
#include "Engine/CoreSettings.h"
#include "RenderTimer.h"

DEFINE_LOG_CATEGORY_STATIC(LogClient, Log, All);

UE_IMPLEMENT_STRUCT("/Script/Engine", PostProcessSettings);

static int32 GHitProxyCaptureNextUpdate = 0;
static FAutoConsoleVariableRef CVarHitProxyCaptureEnable(
	TEXT("r.HitProxy.CaptureNextUpdate"),
	GHitProxyCaptureNextUpdate,
	TEXT("Enables GPU capture of hit proxy rendering on the next update."));

static int32 GShowTimeStampsInStatUnit = 0;
static FAutoConsoleVariableRef CVarShowTimeStampsInStatUnit(
	TEXT("stats.UnitTimestamp"),
	GShowTimeStampsInStatUnit,
	TEXT("Enables UTC time and framecounter in stat unit."));

bool FViewport::bIsGameRenderingEnabled = true;
int32 FViewport::PresentAndStopMovieDelay = 0;

bool FRenderTarget::ReadPixels(TArray< FColor >& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	if(InSrcRect == FIntRect(0, 0, 0, 0))
	{
		InSrcRect = FIntRect(0, 0, GetSizeXY().X, GetSizeXY().Y);
	}

	OutImageData.Reset();

	// Read the render target surface data back.	
	ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)(
		[RenderTarget_RT = this, SrcRect_RT = InSrcRect, OutData_RT = &OutImageData, Flags_RT = InFlags](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.ReadSurfaceData(RenderTarget_RT->GetShaderResourceTexture(), SrcRect_RT, *OutData_RT, Flags_RT);
		});
	FlushRenderingCommands();

	return OutImageData.Num() > 0;
}

bool FRenderTarget::ReadPixelsPtr(FColor* OutImageBytes, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	TArray<FColor> SurfaceData;

	bool bResult = ReadPixels( SurfaceData, InFlags, InSrcRect);
	if( bResult )
	{
		FMemory::Memcpy( OutImageBytes, &SurfaceData[ 0 ], SurfaceData.Num() * sizeof(FColor) );
	}

	return bResult;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FRenderTarget::ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, ECubeFace CubeFace)
{
	return ReadFloat16Pixels(OutImageData, FReadSurfaceDataFlags(RCM_UNorm, CubeFace));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FRenderTarget::ReadFloat16Pixels(TArray<FFloat16Color>& OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	// if the RenderTarget is not EXACTLY PF_FloatRGBA , this will check down in the RHI
	// (eg. PF_FloatRGB will fail)
	// this check is correct, but you can't use GetShaderResourceTexture() except from render thread (could be a race)
	//check( GetShaderResourceTexture()->GetDesc().Format == PF_FloatRGBA );

	if (InSrcRect == FIntRect(0, 0, 0, 0))
	{
		InSrcRect = FIntRect(0, 0, GetSizeXY().X, GetSizeXY().Y);
	}

	OutImageData.Reset();

	// Read the render target surface data back.	
	ENQUEUE_RENDER_COMMAND(ReadSurfaceFloatCommand)(
		[RenderTarget_RT = this, SrcRect_RT = InSrcRect, OutData_RT = &OutImageData, Flags_RT = InFlags](FRHICommandListImmediate& RHICmdList)
	{
		FRHITexture* Texture = RenderTarget_RT->GetShaderResourceTexture();
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
		RHICmdList.ReadSurfaceFloatData(Texture, SrcRect_RT, *OutData_RT, Flags_RT);
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	});
	FlushRenderingCommands();

	return OutImageData.Num() > 0;
}

bool FRenderTarget::ReadLinearColorPixels(TArray<FLinearColor> &OutImageData, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	if (InSrcRect == FIntRect(0, 0, 0, 0))
	{
		InSrcRect = FIntRect(0, 0, GetSizeXY().X, GetSizeXY().Y);
	}

	OutImageData.Reset();

	// Read the render target surface data back.	
	ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)(
		[RenderTarget_RT = this, SrcRect_RT = InSrcRect, OutData_RT = &OutImageData, Flags_RT = InFlags](FRHICommandListImmediate& RHICmdList)
	{
		FRHITexture* Texture = RenderTarget_RT->GetShaderResourceTexture();
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::SRVMask, ERHIAccess::CopySrc));
		RHICmdList.ReadSurfaceData(Texture, SrcRect_RT, *OutData_RT, Flags_RT);
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	});
	FlushRenderingCommands();

	return OutImageData.Num() > 0;
}

bool FRenderTarget::ReadLinearColorPixelsPtr(FLinearColor* OutImageBytes, FReadSurfaceDataFlags InFlags, FIntRect InSrcRect)
{
	TArray<FLinearColor> SurfaceData;

	bool bResult = ReadLinearColorPixels(SurfaceData, InFlags, InSrcRect);
	if (bResult)
	{
		check(SurfaceData.Num() != 0);
		FMemory::Memcpy(OutImageBytes, &SurfaceData[0], SurfaceData.Num() * sizeof(FLinearColor));
	}

	return bResult;
}

float FRenderTarget::GetDisplayGamma() const
{
	return GetEngineDisplayGamma();
}

float FRenderTarget::GetEngineDisplayGamma() // static
{
	// when we say we want a 2.2 gamma, what we actually mean is that we want SRGB conversion in most cases

	if (GEngine == NULL)
	{
		return 2.2f;
	}
	else
	{
		if (FMath::Abs(GEngine->DisplayGamma) <= 0.0f)
		{
			UE_LOGF(LogClient, Error, "Invalid DisplayGamma! Resetting to the default of 2.2");
			GEngine->DisplayGamma = 2.2f;
		}
		return GEngine->DisplayGamma;
	}
}

FRenderTarget::FRenderTarget() = default;
FRenderTarget::~FRenderTarget() = default;

const FTextureRHIRef& FRenderTarget::GetRenderTargetTexture() const
{
	return RenderTargetTextureRHI;
}

FRDGTextureRef FRenderTarget::GetRenderTargetTexture(FRDGBuilder& GraphBuilder) const
{
	return RegisterExternalTexture(GraphBuilder, GetRenderTargetTexture(), TEXT("RenderTarget"));
}

FUnorderedAccessViewRHIRef FRenderTarget::GetRenderTargetUAV() const
{
	return FUnorderedAccessViewRHIRef();
}

const FTextureRHIRef& FRenderTarget::GetShaderResourceTexture() const
{
	return GetRenderTargetTexture();
}

void FScreenshotRequest::RequestScreenshot(bool bInShowUI, const bool bInRestrictToGameViewport)
{
	// empty string means we'll later pick the name
	RequestScreenshot(TEXT(""), bInShowUI, /*bAddUniqueSuffix*/ true, /*bHdrScreenshot*/ false, FIntRect(), bInRestrictToGameViewport);
}

void FScreenshotRequest::RequestScreenshot(const FString& InFilename, bool bInShowUI, bool bAddUniqueSuffix, bool bHdrScreenshot, const FIntRect& InViewRect, const bool bInRestrictToGameViewport)
{
	FString GeneratedFilename = InFilename;
	CreateViewportScreenShotFilename(GeneratedFilename);

	const TCHAR* ScreenshotExtension = bHdrScreenshot ? TEXT("exr") : TEXT("png");

	if (bAddUniqueSuffix)
	{
		const bool bRemovePath = false;
		GeneratedFilename = FPaths::GetBaseFilename(GeneratedFilename, bRemovePath);
		if (GetHighResScreenshotConfig().bDateTimeBasedNaming)
		{
			FFileHelper::GenerateDateTimeBasedBitmapFilename(GeneratedFilename, ScreenshotExtension, Filename);
		}
		else
		{
			FFileHelper::GenerateNextBitmapFilename(GeneratedFilename, ScreenshotExtension, Filename);
		}
	}
	else
	{
		Filename = GeneratedFilename;
		if (FPaths::GetExtension(Filename).Len() == 0)
		{
			Filename += TEXT(".");
			Filename += ScreenshotExtension;
		}
	}

	// Register the screenshot
	if (!Filename.IsEmpty())
	{
		bShowUI = bInShowUI;
		bRestrictToGameViewport = bInRestrictToGameViewport;
		ViewRect = InViewRect;
		bIsScreenshotRequested = true;
	}

	GScreenMessagesRestoreState = GAreScreenMessagesEnabled;

	// Disable Screen Messages when the screenshot is requested without UI.
	if (bInShowUI == false)
	{
		GAreScreenMessagesEnabled = false;
	}
}


void FScreenshotRequest::Reset()
{
	bIsScreenshotRequested = false;
	Filename.Empty();
	bShowUI = false;
	bRestrictToGameViewport = false;
	ViewRect = FIntRect();
}

void FScreenshotRequest::CreateViewportScreenShotFilename(FString& InOutFilename)
{
	FString TypeName;

	if(GIsDumpingMovie)
	{
		TypeName = TEXT("MovieFrame");

		if(GIsDumpingMovie > 0)
		{
			// <=0:off (default), <0:remains on, >0:remains on for n frames (n is the number specified)
			--GIsDumpingMovie;
		}
	}
	else if(GIsHighResScreenshot)
	{
		FString FilenameOverride = GetHighResScreenshotConfig().FilenameOverride;
		TypeName = FilenameOverride.IsEmpty() ? TEXT("HighresScreenshot") : FilenameOverride;
	}
	else
	{
		TypeName = InOutFilename.IsEmpty() ? TEXT("ScreenShot") : InOutFilename;
	}
	check(!TypeName.IsEmpty());

	//default to using the path that is given
	InOutFilename = TypeName;
	if (!TypeName.Contains(TEXT("/")) && !TypeName.Contains(TEXT("\\")))
	{
		InOutFilename = GetDefault<UEngine>()->GameScreenshotSaveDirectory.Path / TypeName;
	}
}

TArray<FColor>* FScreenshotRequest::GetHighresScreenshotMaskColorArray()
{
	return &HighresScreenshotMaskColorArray;
}

FIntPoint& FScreenshotRequest::GetHighresScreenshotMaskExtents()
{
	return HighresScreenshotMaskExtents;
}

// @param bAutoType true: automatically choose GB/MB/KB/... false: always use MB for easier comparisons
FString GetMemoryString( const double Value, const bool bAutoType )
{
	if (bAutoType)
	{
		if (Value > 1024.0 * 1024.0 * 1024.0)
		{
			return FString::Printf( TEXT( "%.2f GB" ), float( Value / (1024.0 * 1024.0 * 1024.0) ) );
		}
		if (Value > 1024.0 * 1024.0)
		{
			return FString::Printf( TEXT( "%.2f MB" ), float( Value / (1024.0 * 1024.0) ) );
		}
		if (Value > 1024.0)
		{
			return FString::Printf( TEXT( "%.2f KB" ), float( Value / (1024.0) ) );
		}
		return FString::Printf( TEXT( "%.2f B" ), float( Value ) );
	}
	
	return FString::Printf( TEXT( "%.2f MB" ), float( Value / (1024.0 * 1024.0) ) );
}

FOnScreenshotRequestProcessed FScreenshotRequest::ScreenshotProcessedDelegate;
FOnScreenshotCaptured FScreenshotRequest::ScreenshotCapturedDelegate;
FOnHDRScreenshotCaptured FScreenshotRequest::HDRScreenshotCapturedDelegate;
bool FScreenshotRequest::bIsScreenshotRequested = false;
FString FScreenshotRequest::Filename;
FString FScreenshotRequest::NextScreenshotName;
bool FScreenshotRequest::bShowUI = false;
bool FScreenshotRequest::bRestrictToGameViewport = false;
TArray<FColor> FScreenshotRequest::HighresScreenshotMaskColorArray;
FIntPoint FScreenshotRequest::HighresScreenshotMaskExtents;
FIntRect FScreenshotRequest::ViewRect = FIntRect();

static TAutoConsoleVariable<int32> CVarFullSizeUnitGraph(
	TEXT("FullSizeUnitGraph"),
	0,
	TEXT("If true, the unit graph is the old full size, full brightness version."));


int32 FStatUnitData::DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY)
{
	float DiffTime;
	if (FApp::IsBenchmarking() || FApp::UseFixedTimeStep())
	{
		/** If we're in fixed time step mode, FApp::GetCurrentTime() will be incorrect for benchmarking */
		const double CurrentTime = FPlatformTime::Seconds();
		if (LastTime == 0)
		{
			LastTime = CurrentTime;
		}
		DiffTime = CurrentTime - LastTime;
		LastTime = CurrentTime;
	}
	else
	{
		/** Use the DiffTime we computed last frame, because it correctly handles the end of frame idling and corresponds better to the other unit times. */
		DiffTime = FApp::GetCurrentTime() - FApp::GetLastTime();
	}

	RawFrameTime = DiffTime * 1000.0f;
	FrameTime = 0.9 * FrameTime + 0.1 * RawFrameTime;

	/** Number of milliseconds the gamethread was used last frame. */
	RawGameThreadTime = FPlatformTime::ToMilliseconds(GGameThreadTime);
	GameThreadTime = 0.9 * GameThreadTime + 0.1 * RawGameThreadTime;

	RawGameThreadTimeCriticalPath = FPlatformTime::ToMilliseconds(GGameThreadTimeCriticalPath);
	GameThreadTimeCriticalPath = 0.9 * GameThreadTimeCriticalPath + 0.1 * RawGameThreadTimeCriticalPath;

	/** Number of milliseconds the renderthread was used last frame. */
	RawRenderThreadTime = FPlatformTime::ToMilliseconds(GRenderThreadTime);
	RenderThreadTime = 0.9 * RenderThreadTime + 0.1 * RawRenderThreadTime;

	RawRenderThreadTimeCriticalPath = FPlatformTime::ToMilliseconds(GRenderThreadTimeCriticalPath);
	RenderThreadTimeCriticalPath = 0.9 * RenderThreadTimeCriticalPath + 0.1 * RawRenderThreadTimeCriticalPath;

	RawRHITTime = FPlatformTime::ToMilliseconds(GRHIThreadTime);
	RHITTime = 0.9 * RHITTime + 0.1 * RawRHITTime;

	RawInputLatencyTime = FPlatformTime::ToMilliseconds64(GInputLatencyTime);
	InputLatencyTime = 0.9 * InputLatencyTime + 0.1 * RawInputLatencyTime;

	FDynamicResolutionStateInfos DynamicResolutionStateInfos;
	GEngine->GetDynamicResolutionCurrentStateInfos(/* out */ DynamicResolutionStateInfos);

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		/** Number of milliseconds the GPU was busy last frame. */
		RawGPUFrameTime[GPUIndex] = FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles(GPUIndex));
		GPUFrameTime[GPUIndex] = 0.9 * GPUFrameTime[GPUIndex] + 0.1 * RawGPUFrameTime[GPUIndex];

		if (GRHISupportsGPUUsage)
		{
			FRHIGPUUsageFractions GPUUsageFractions = RHIGetGPUUsage(GPUIndex);

			RawGPUClockFraction[GPUIndex] = GPUUsageFractions.ClockScaling;
			GPUClockFraction[GPUIndex] = 0.9 * GPUClockFraction[GPUIndex] + 0.1 * RawGPUClockFraction[GPUIndex];

			RawGPUUsageFraction[GPUIndex] = GPUUsageFractions.CurrentProcessMHz;
			GPUUsageFraction[GPUIndex] = 0.9 * GPUUsageFraction[GPUIndex] + 0.1 * RawGPUUsageFraction[GPUIndex];

			RawGPUMemoryUsage[GPUIndex] = GPUUsageFractions.CurrentProcessMemoryUsage;
			GPUMemoryUsage[GPUIndex] = 0.9 * GPUMemoryUsage[GPUIndex] + 0.1 * RawGPUMemoryUsage[GPUIndex];

			RawGPUExternalUsageFraction[GPUIndex] = GPUUsageFractions.ExternalProcessesMHz;
			GPUExternalUsageFraction[GPUIndex] = 0.9 * GPUExternalUsageFraction[GPUIndex] + 0.1 * RawGPUExternalUsageFraction[GPUIndex];

			RawGPUExternalMemoryUsage[GPUIndex] = GPUUsageFractions.ExternalProcessMemoryUsage;
			GPUExternalMemoryUsage[GPUIndex] = 0.9 * GPUExternalUsageFraction[GPUIndex] + 0.1 * RawGPUExternalMemoryUsage[GPUIndex];
		}
	}

	SET_FLOAT_STAT(STAT_UnitFrame, FrameTime);
	SET_FLOAT_STAT(STAT_UnitRender, RenderThreadTime);
	SET_FLOAT_STAT(STAT_UnitRHIT, RHITTime);
	SET_FLOAT_STAT(STAT_UnitGame, GameThreadTime);
	SET_FLOAT_STAT(STAT_UnitGPU, GPUFrameTime[0]);
	SET_FLOAT_STAT(STAT_InputLatencyTime, InputLatencyTime);

	GEngine->SetAverageUnitTimes(FrameTime, RenderThreadTime, GameThreadTime, GPUFrameTime[0], RHITTime);

	float Max_RenderThreadTime = 0.0f;
	float Max_GameThreadTime = 0.0f;
	float Max_GPUFrameTime[MAX_NUM_GPUS] = { 0.0f };
	float Max_FrameTime = 0.0f;
	float Max_RHITTime = 0.0f;
	float Max_InputLatencyTime = 0.0f;

	FViewportClient* Client = InViewport->GetClient();
	const bool bShowUnitCriticalPathTimes = Client && Client->IsStatEnabled(TEXT("UnitCriticalPath"));
	const bool bShowUnitMaxTimes = Client && Client->IsStatEnabled(TEXT("UnitMax"));
	const bool bShowTSRStatistics = Client && Client->IsStatEnabled(TEXT("TSR"));
#if !UE_BUILD_SHIPPING
	const bool bShowRawUnitTimes = Client && Client->IsStatEnabled(TEXT("Raw"));
	RenderThreadTimes[CurrentIndex] = bShowRawUnitTimes ? RawRenderThreadTime : RenderThreadTime;
	GameThreadTimes[CurrentIndex] = bShowRawUnitTimes ? RawGameThreadTime : GameThreadTime;
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		GPUFrameTimes[GPUIndex][CurrentIndex] = bShowRawUnitTimes ? RawGPUFrameTime[GPUIndex] : GPUFrameTime[GPUIndex];
	}
	FrameTimes[CurrentIndex] = bShowRawUnitTimes ? RawFrameTime : FrameTime;
	RHITTimes[CurrentIndex] = bShowRawUnitTimes ? RawRHITTime : RHITTime;
	InputLatencyTimes[CurrentIndex] = bShowRawUnitTimes ? RawInputLatencyTime : InputLatencyTime;
	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		ResolutionFractions[Budget][CurrentIndex] = DynamicResolutionStateInfos.ResolutionFractionApproximations[Budget];
	}
	CurrentIndex++;
	if (CurrentIndex == NumberOfSamples)
	{
		CurrentIndex = 0;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bShowUnitMaxTimes)
	{
		for (int32 MaxIndex = 0; MaxIndex < NumberOfSamples; MaxIndex++)
		{
			if (Max_RenderThreadTime < RenderThreadTimes[MaxIndex])
			{
				Max_RenderThreadTime = RenderThreadTimes[MaxIndex];
			}
			if (Max_GameThreadTime < GameThreadTimes[MaxIndex])
			{
				Max_GameThreadTime = GameThreadTimes[MaxIndex];
			}
			for (uint32 GPUIndex : FRHIGPUMask::All())
			{
				if (Max_GPUFrameTime[GPUIndex] < GPUFrameTimes[GPUIndex][MaxIndex])
				{
					Max_GPUFrameTime[GPUIndex] = GPUFrameTimes[GPUIndex][MaxIndex];
				}
			}
			if (Max_FrameTime < FrameTimes[MaxIndex])
			{
				Max_FrameTime = FrameTimes[MaxIndex];
			}
			if (Max_RHITTime < RHITTimes[MaxIndex])
			{
				Max_RHITTime = RHITTimes[MaxIndex];
			}
			if (Max_InputLatencyTime < InputLatencyTimes[MaxIndex])
			{
				Max_InputLatencyTime = InputLatencyTimes[MaxIndex];
			}
		}
	}
#endif // #if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif // #if !UE_BUILD_SHIPPING

	FColor StatRed(233, 109, 99);		// (Salmon) Red that survives video compression
	FColor StatGreen(127, 202, 159);	// (De York) Green that survives video compression
	FColor StatOrange(244, 186, 112);	// Orange that survives video compression
	FColor StatMagenda(204, 153, 204);	// (Deep Pink) Magenta that survives video compression

	// Render CPU thread and GPU frame times.
	const bool bStereoRendering = GEngine->IsStereoscopic3D(InViewport);
	UFont* Font = (!FPlatformProperties::SupportsWindowedMode() && GEngine->GetMediumFont()) ? GEngine->GetMediumFont() : GEngine->GetSmallFont();

	const bool bShowUnitTimeGraph = Client && Client->IsStatEnabled(TEXT("UnitGraph"));
	bool bHaveGPUData[MAX_NUM_GPUS] = { false };
	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		bHaveGPUData[GPUIndex] = RawGPUFrameTime[GPUIndex] > 0;
	}
	const bool bHaveInputLatencyData = InputLatencyTime > 0;

	const float AlertResolutionFraction = 0.70f; // Truncation of sqrt(0.5) for easier remembering.

	// Draw unit.
	{
		const FColor NoUnitGraphColor(220, 220, 220);

		#define STATUNIT_FORMAT_AVGTIME TEXT("%3.2f ms")
		#define STATUNIT_FORMAT_CRITICALPATHTIME TEXT("%3.2f ms")
		#define STATUNIT_FORMAT_MAXTIME TEXT("%4.2f ms")
		#define STATUNIT_FORMAT_PERCEMT TEXT("%3.2f %%")

		const int32 AvgUnitColumnId = 0;
		const int32 CriticalPathUnitColumnId = 1;
		const int32 MaxUnitColumnId = bShowUnitCriticalPathTimes ? 2 : 1;

		const int32 ColumnWidth = Font->GetStringSize(TEXT(" 0000.00 ms "));
		int32 ColumnCount = 1;
		if (bShowUnitCriticalPathTimes)
		{
			++ColumnCount;
		}
		if (bShowUnitMaxTimes)
		{
			++ColumnCount;
		}

		int32 X3 = InX * (bStereoRendering ? 0.5f : 1.0f);
		int32 X2 = X3 - ColumnWidth * (ColumnCount - 1);

		const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);

		/* Draw a cell on raw that have custom number of columns. */
		auto DrawCell = [&](int32 RowId, int32 ColumnId, int32 ColumnCount, const FString& CellText, const FColor& CellColor)
		{
			check(ColumnId < ColumnCount);
			int32 CellCoordX = X3 - ColumnWidth * (ColumnCount - ColumnId - 1);
			int32 CellCoordY = InY + RowId * RowHeight;
			InCanvas->DrawShadowedString(CellCoordX, CellCoordY, *CellText, Font, CellColor);
		};

		/* Draw a row's title that have a custom number of columns. */
		auto DrawRowTitle = [&](int32 RowId, int32 InColumnCount, const TCHAR* Title, const FColor& UnitGraphColor)
		{
			FString FullTitle = FString::Printf(TEXT("%s:   "), Title);
			int32 TitleSize = Font->GetStringSize(*FullTitle);

			int32 TitleCoordX = X3 - ColumnWidth * (InColumnCount - 1);
			int32 TitleCoordY = InY + RowId * RowHeight;
			InCanvas->DrawShadowedString(TitleCoordX - TitleSize, TitleCoordY, *FullTitle, Font, bShowUnitTimeGraph ? UnitGraphColor : FColor::White);
		};

		/* Push drawing of rows toward the bottom of the screen. */
		auto PushRows = [&](int32 RowsCount)
		{
			InY += RowHeight * RowsCount;
		};

		/* Draw a default row's title. */
		auto DrawTitleString = [&](const TCHAR* Title, const FColor& UnitGraphColor)
		{
			DrawRowTitle(/* RowId = */ 0, ColumnCount, Title, UnitGraphColor);
		};

		/* Draw a default row's avg. */
		auto DrawDefaultAvgCell = [&](const FString& CellText, const FColor& UnitGraphColor)
		{
			DrawCell(/* RowId = */ 0, AvgUnitColumnId, ColumnCount, CellText, UnitGraphColor);
		};
		auto DrawDefaultCriticalPathCell = [&](const FString& CellText, const FColor& UnitGraphColor)
		{
			check(bShowUnitCriticalPathTimes);
			DrawCell(/* RowId = */ 0, CriticalPathUnitColumnId, ColumnCount, CellText, UnitGraphColor);
		};
		auto DrawDefaultMaxCell = [&](const FString& CellText, const FColor& UnitGraphColor)
		{
			check(bShowUnitMaxTimes);
			DrawCell(/* RowId = */ 0, MaxUnitColumnId, ColumnCount, CellText, UnitGraphColor);
		};

		if (bShowUnitCriticalPathTimes || bShowUnitMaxTimes)
		{
			// if there are multiple columns, draw the name of each column

			DrawDefaultAvgCell(TEXT("Avg"), FColor::White);
			if (bShowUnitCriticalPathTimes)
			{
				DrawDefaultCriticalPathCell(TEXT("Critical"), FColor::White);
			}
			if (bShowUnitMaxTimes)
			{
				DrawDefaultMaxCell(TEXT("Max"), FColor::White);
			}
			PushRows(/* RowsCount = */ 1);
		}

		{
			DrawTitleString(TEXT("Frame"), /* UnitGraphColor = */ FColor(100, 255, 100));
			{
				const FColor FrameTimeAverageColor = GEngine->GetFrameTimeDisplayColor(FrameTime);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, FrameTime), FrameTimeAverageColor);
			}
			if (bShowUnitMaxTimes)
			{
				const FColor MaxFrameTimeColor = GEngine->GetFrameTimeDisplayColor(Max_FrameTime);
				DrawDefaultMaxCell(FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_FrameTime), MaxFrameTimeColor);
			}
			PushRows(/* RowsCount = */ 1);
		}

		{
			DrawTitleString(TEXT("Game"), /* UnitGraphColor = */ FColor(255, 100, 100));
			{
				const FColor GameThreadAverageColor = GEngine->GetFrameTimeDisplayColor(GameThreadTime);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, GameThreadTime), GameThreadAverageColor);
			}
			if (bShowUnitCriticalPathTimes)
			{
				const FColor CriticalPathFrameTimeColor = GEngine->GetFrameTimeDisplayColor(GameThreadTimeCriticalPath);
				DrawDefaultCriticalPathCell(FString::Printf(STATUNIT_FORMAT_CRITICALPATHTIME, GameThreadTimeCriticalPath), CriticalPathFrameTimeColor);
			}
			if (bShowUnitMaxTimes)
			{
				const FColor GameThreadMaxColor = GEngine->GetFrameTimeDisplayColor(Max_GameThreadTime);
				DrawDefaultMaxCell(FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_GameThreadTime), GameThreadMaxColor);
			}
			PushRows(/* RowsCount = */ 1);
		}

		{
			DrawTitleString(TEXT("Draw"), /* UnitGraphColor = */ FColor(100, 100, 255));
			{
				const FColor RenderThreadAverageColor = GEngine->GetFrameTimeDisplayColor(RenderThreadTime);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, RenderThreadTime), RenderThreadAverageColor);
			}
			if (bShowUnitCriticalPathTimes)
			{
				const FColor CriticalPathFrameTimeColor = GEngine->GetFrameTimeDisplayColor(RenderThreadTimeCriticalPath);
				DrawDefaultCriticalPathCell(FString::Printf(STATUNIT_FORMAT_CRITICALPATHTIME, RenderThreadTimeCriticalPath), CriticalPathFrameTimeColor);
			}
			if (bShowUnitMaxTimes)
			{
				const FColor RenderThreadMaxColor = GEngine->GetFrameTimeDisplayColor(Max_RenderThreadTime);
				DrawDefaultMaxCell(FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_RenderThreadTime), RenderThreadMaxColor);
			}
			PushRows(/* RowsCount = */ 1);
		}

		if (IsRunningRHIInSeparateThread())
		{
			DrawTitleString(TEXT("RHIT"), /* UnitGraphColor = */ FColor(255, 100, 255));
			{
				const FColor RenderThreadAverageColor = GEngine->GetFrameTimeDisplayColor(RHITTime);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, RHITTime), RenderThreadAverageColor);
			}
			if (bShowUnitMaxTimes)
			{
				const FColor RenderThreadMaxColor = GEngine->GetFrameTimeDisplayColor(Max_RHITTime);
				DrawDefaultMaxCell(FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_RHITTime), RenderThreadMaxColor);
			}
			PushRows(/* RowsCount = */ 1);
		}

		// Draw all GPU informations
		{
			const int32 ColumnPerGPU = ColumnCount;
			const bool bDisplayGPUIndexes = GNumExplicitGPUsForRendering > 1 || GVirtualMGPU;
			const int32 GPURowCount = 1 + (GRHISupportsGPUUsage ? 5 : 0) + (bDisplayGPUIndexes ? 1 : 0);
			const int32 GPUColumnCount = ColumnCount * GNumExplicitGPUsForRendering;

			// Draw the different timings:
			{
				int32 GPURowId = bDisplayGPUIndexes ? 1 : 0;
				DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPU Time"), /* UnitGraphColor = */ FColor(255, 255, 100));

				if (GRHISupportsGPUUsage)
				{
					DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPU Clock"), NoUnitGraphColor);
					DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPU Usage"), NoUnitGraphColor);
					DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPU Mem"), NoUnitGraphColor);
					DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPUExt Usage"), NoUnitGraphColor);
					DrawRowTitle(GPURowId++, GPUColumnCount, TEXT("GPUExt Mem"), NoUnitGraphColor);
				}
				check(GPURowId == GPURowCount);
			}

			// Draw each GPU
			for (uint32 GPUIndex : FRHIGPUMask::All())
			{
				int32 GPURowId = 0;
				if (bDisplayGPUIndexes)
				{
					DrawCell(
						GPURowId++, GPUIndex * ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
						GVirtualMGPU ? FString::Printf(TEXT("VGPU%d"), GPUIndex) : FString::Printf(TEXT("GPU%d"), GPUIndex), NoUnitGraphColor);
				}

				if (!bHaveGPUData[GPUIndex])
				{
					continue;
				}

				{
					{
						const FColor GPUAverageColor = GEngine->GetFrameTimeDisplayColor(GPUFrameTime[GPUIndex]);
						DrawCell(
							GPURowId, GPUIndex * ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							FString::Printf(STATUNIT_FORMAT_AVGTIME, GPUFrameTime[GPUIndex]), GPUAverageColor);
					}

					if (bShowUnitMaxTimes)
					{
						const FColor GPUMaxColor = GEngine->GetFrameTimeDisplayColor(Max_GPUFrameTime[GPUIndex]);
						DrawCell(
							GPURowId, GPUIndex* ColumnPerGPU + MaxUnitColumnId, GPUColumnCount,
							FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_GPUFrameTime[GPUIndex]), GPUMaxColor);
					}

					GPURowId++;
				}

				// Only display the GPU usage of one GPU usage when using the -VMGPU 
				if (GRHISupportsGPUUsage && GPUIndex > 0 && GVirtualMGPU)
				{
					GPURowId = GPURowCount;
				}
				else if (GRHISupportsGPUUsage)
				{
					constexpr float GIGABYTE = 1024.0f * 1024.0f * 1024.0f;
					{
						const FColor Color = (GPUClockFraction[GPUIndex] < 0.5f) ? StatRed : ((GPUClockFraction[GPUIndex] < 0.75f) ? StatOrange : StatGreen);
						DrawCell(
							GPURowId++, GPUIndex* ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							FString::Printf(STATUNIT_FORMAT_PERCEMT, 100.0f * GPUClockFraction[GPUIndex]), Color);
					}

					{
						const FColor Color = (GPUUsageFraction[GPUIndex] < 0.5f) ? StatRed : ((GPUUsageFraction[GPUIndex] < 0.75f) ? StatOrange : StatGreen);
						DrawCell(
							GPURowId++, GPUIndex* ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							FString::Printf(STATUNIT_FORMAT_PERCEMT, 100.0f * GPUUsageFraction[GPUIndex]), Color);
					}

					{
						const FColor Color = (GPUMemoryUsage[GPUIndex] > GIGABYTE * 3) ? StatRed : ((GPUMemoryUsage[GPUIndex] > GIGABYTE * 2) ? StatOrange : StatGreen);
						DrawCell(
							GPURowId++, GPUIndex* ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							GetMemoryString((double)GPUMemoryUsage[GPUIndex]), Color);
					}

					{
						const FColor Color = (GPUExternalUsageFraction[GPUIndex] > 0.5f) ? StatRed : ((GPUExternalUsageFraction[GPUIndex] > 0.25f) ? StatOrange : StatGreen);
						DrawCell(
							GPURowId++, GPUIndex* ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							FString::Printf(STATUNIT_FORMAT_PERCEMT, 100.0f * GPUExternalUsageFraction[GPUIndex]), Color);
					}

					{
						const FColor Color = (GPUExternalMemoryUsage[GPUIndex] > GIGABYTE * 3) ? StatRed : ((GPUExternalMemoryUsage[GPUIndex] > GIGABYTE * 2) ? StatOrange : StatGreen);
						DrawCell(
							GPURowId++, GPUIndex* ColumnPerGPU + AvgUnitColumnId, GPUColumnCount,
							GetMemoryString((double)GPUExternalMemoryUsage[GPUIndex]), Color);
					}
				}
				check(GPURowId == GPURowCount);
			}

			PushRows(/* RowsCount = */ GPURowCount);
		}

		if (bHaveInputLatencyData)
		{
			const float ReasonableInputLatencyFactor = 2.5f;
			DrawTitleString(TEXT("Input"), /* UnitGraphColor = */ FColor(255, 255, 100));
			{
				const FColor InputLatencyAverageColor = GEngine->GetFrameTimeDisplayColor(InputLatencyTime / ReasonableInputLatencyFactor);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, InputLatencyTime), InputLatencyAverageColor);
			}
			if (bShowUnitMaxTimes)
			{
				const FColor InputLatencyMaxColor = GEngine->GetFrameTimeDisplayColor(Max_InputLatencyTime / ReasonableInputLatencyFactor);
				DrawDefaultMaxCell(FString::Printf(STATUNIT_FORMAT_MAXTIME, Max_InputLatencyTime), InputLatencyMaxColor);
			}
			PushRows(/* RowsCount = */ 1);
		}

		{
			if (bShowUnitMaxTimes)
			{
				FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

				DrawTitleString(TEXT("Mem"), NoUnitGraphColor);
				DrawDefaultAvgCell(GetMemoryString(Stats.UsedPhysical), StatGreen);
				DrawDefaultMaxCell(GetMemoryString(Stats.PeakUsedPhysical), StatGreen);
				PushRows(/* RowsCount = */ 1);
				
				DrawTitleString(TEXT("VMem"), NoUnitGraphColor);
				DrawDefaultAvgCell(GetMemoryString(Stats.UsedVirtual), StatGreen);
				DrawDefaultMaxCell(GetMemoryString(Stats.PeakUsedVirtual), StatGreen);
				PushRows(/* RowsCount = */ 1);
			}
			else
			{
				uint64 MemoryUsed = FPlatformMemory::GetMemoryUsedFast();
				if (MemoryUsed > 0)
				{
					// print out currently used memory
					DrawTitleString(TEXT("Mem"), NoUnitGraphColor);
					DrawDefaultAvgCell(GetMemoryString(MemoryUsed), StatGreen);
					PushRows(/* RowsCount = */ 1);
				}
			}
		}

#if PLATFORM_DESKTOP
		{
			FRHIMemoryStats RHIMemStats;
			RHIGetMemoryStats(RHIMemStats);

			if (RHIMemStats.IsValid())
			{
				const bool bOverBudget = RHIMemStats.IsOverBudget();
				const FColor Color = bOverBudget ? StatRed : StatGreen;
				DrawTitleString(TEXT("VRam"), NoUnitGraphColor);
				// Show "Used / Budget" format
				FString VRAMText = FString::Printf(TEXT("%s / %s"),
					*GetMemoryString(RHIMemStats.UsedLocal),
					*GetMemoryString(RHIMemStats.BudgetLocal));
				DrawDefaultAvgCell(VRAMText, Color);

				PushRows(/* RowsCount = */ 1);
			}
		}
#endif

		// Dynamic resolution
		if (DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::Enabled || DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::DebugForceEnabled)
		{
			const FIntPoint RenderResolution = GPixelRenderCounters.GetRenderResolution();
			float ResolutionFraction = DynamicResolutionStateInfos.ResolutionFractionApproximations[GDynamicPrimaryResolutionFraction];
			float ScreenPercentage = ResolutionFraction * 100.0f;

			DrawTitleString(TEXT("DynRes"), /* UnitGraphColor = */ FColor(255, 160, 100));
			if (DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::Enabled)
			{
				FColor Color = (ResolutionFraction < AlertResolutionFraction) ? StatRed : ((ResolutionFraction < FMath::Min(ResolutionFraction * 0.97f, 1.0f)) ? StatOrange : StatGreen);
				DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%% (%dx%d)"), ScreenPercentage, RenderResolution.X, RenderResolution.Y), Color);
			}
			else
			{
				if (GEngine->GetDynamicResolutionState() && GEngine->GetDynamicResolutionState()->IsSupported())
				{
					DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%% (%dx%d) (Forced)"), ScreenPercentage, RenderResolution.X, RenderResolution.Y), StatMagenda);
				}
				else
				{
					DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%% (%dx%d) (Forced, Unsupported)"), ScreenPercentage, RenderResolution.X, RenderResolution.Y), StatRed);
				}
			}

			PushRows(/* RowsCount = */ 1);

			// Other dynamic render scalings
			if (DynamicResolutionStateInfos.Status == EDynamicResolutionStatus::Enabled && DynamicRenderScaling::IsSupported())
			{
				for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
				{
					const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
					const DynamicRenderScaling::FHeuristicSettings& HeuristicSettings = Budget.GetSettings();
					if (Budget == GDynamicPrimaryResolutionFraction || !HeuristicSettings.IsEnabled())
					{
						continue;
					}

					FString DisplayName = Budget.GetName();
					DisplayName.ReplaceInline(TEXT("Dynamic"), TEXT("Dyn"));
					DisplayName.ReplaceInline(TEXT("Resolution"), TEXT("Res"));

					ResolutionFraction = DynamicResolutionStateInfos.ResolutionFractionApproximations[Budget];
					ScreenPercentage = ResolutionFraction * 100.0f;

					FColor Color = (ResolutionFraction < AlertResolutionFraction) ? StatRed : ((ResolutionFraction < FMath::Min(ResolutionFraction * 0.97f, 1.0f)) ? StatOrange : StatGreen);

					DrawTitleString(*DisplayName, /* UnitGraphColor = */ FColor::White);
					if (HeuristicSettings.Model == DynamicRenderScaling::EHeuristicModel::Quadratic)
					{
						DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%% x %3.1f%%"), ScreenPercentage, ScreenPercentage), Color);
					}
					else
					{
						DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%%"), ScreenPercentage), Color);
					}
					PushRows(/* RowsCount = */ 1);
				}
			}
		}
		else
		{
			// Render Resolution
			const float ScreenPercentage = GPixelRenderCounters.GetResolutionFraction() * 100.0f;
			const FIntPoint RenderResolution = GPixelRenderCounters.GetRenderResolution();
			DrawTitleString(TEXT("RenderRes"), NoUnitGraphColor);
			DrawDefaultAvgCell(FString::Printf(TEXT("%3.1f%% (%dx%d)"), ScreenPercentage, RenderResolution.X, RenderResolution.Y), StatGreen);
			PushRows(/* RowsCount = */ 1);
		}

		// Draw calls
		{
			// Assume we don't have more than 1 GPU in mobile.
			int32 NumDrawCalls = GNumDrawCallsRHI[0];
			DrawTitleString(TEXT("Draws"), NoUnitGraphColor);
			DrawDefaultAvgCell(FString::Printf(TEXT("%d"), NumDrawCalls), StatGreen);
			PushRows(/* RowsCount = */ 1);
		}
			
		// Primitives
		{
			// Assume we don't have more than 1 GPU in mobile.
			int32 NumPrimitives = GNumPrimitivesDrawnRHI[0];
			DrawTitleString(TEXT("Prims"), NoUnitGraphColor);
			if (NumPrimitives < 10000)
			{
				DrawDefaultAvgCell(FString::Printf(TEXT("%d"), NumPrimitives), StatGreen);
			}
			else
			{
				float NumPrimitivesK = NumPrimitives/1000.f;
				DrawDefaultAvgCell(FString::Printf(TEXT("%.1fK"), NumPrimitivesK), StatGreen);
			}

			PushRows(/* RowsCount = */ 1);
		}

		// Timestamp / Frame count if enabled
		if (GShowTimeStampsInStatUnit)
		{
			DrawTitleString(TEXT("UTC Time"), NoUnitGraphColor);
			DrawDefaultAvgCell(FDateTime::UtcNow().ToString(TEXT("[%H.%M.%S:%s]")), StatGreen);
			PushRows(/* RowsCount = */ 1);

			DrawTitleString(TEXT("FrameNum"), NoUnitGraphColor);
			DrawDefaultAvgCell(FString::Printf(TEXT("%llu"), GFrameCounter), StatGreen);
			PushRows(/* RowsCount = */ 1);
		}

		if (bShowTSRStatistics && GPixelRenderCounters.GetPixelDisplayCount())
		{
			const uint32 PixelRenderCount = GPixelRenderCounters.GetPixelRenderCount();
			const uint32 PixelDisplayCount = GPixelRenderCounters.GetPixelDisplayCount();

			// TSR input feed in pixel/s
			{
				const FColor Color = (PixelRenderCount < 1280 * 720) ? StatRed : ((PixelRenderCount < 1920 * 1080) ? StatOrange : StatGreen);

				float TSRFeed = PixelRenderCount * (1000.0f / FrameTime);

				DrawTitleString(TEXT("TSR feed"), NoUnitGraphColor);
				DrawDefaultAvgCell(FString::Printf(TEXT("%.2f MP/s"), float(TSRFeed) / 1000000.0f), Color);

				PushRows(/* RowsCount = */ 1);
			}

			// TSR history convergence rate
			{
				// Target 1 sample per pixel for the convergence speed measurment.
				const float TargetSamplePerPixel = 1.0f;

				// Ideal TSR uses it to render 1080p -> 4k at 60hz.
				const float IdealConvergenceTime = TargetSamplePerPixel * FMath::Pow(1080.0f / 2160.0f, -2.0f) * (1000.0f / 60.0f);

				// Compute the resolution fraction agregate.
				float ResolutionFraction = FMath::Sqrt(float(PixelRenderCount) / float(PixelDisplayCount));
				float ConvergenceSpeedMultiplier = FMath::Pow(ResolutionFraction, -2.0f);

				// Compute how long it takes for the history to converge to 1spp.
				float ConvergenceFrameCount = ConvergenceSpeedMultiplier * TargetSamplePerPixel;
				float ConvergenceTime = FMath::Max(ConvergenceFrameCount, 1.0f) * FrameTime;

				const FColor Color = (ConvergenceTime <= IdealConvergenceTime) ? StatGreen : ((ConvergenceTime < 2.0f * IdealConvergenceTime) ? StatOrange : StatRed);

				DrawTitleString(TEXT("TSR 1spp"), NoUnitGraphColor);
				DrawDefaultAvgCell(FString::Printf(STATUNIT_FORMAT_AVGTIME, ConvergenceTime), Color);

				PushRows(/* RowsCount = */ 1);
			}
		}
	}

#if !UE_BUILD_SHIPPING
	// Draw simple unit time graph
	if (bShowUnitTimeGraph)
	{
		const bool bSmallGraph = !CVarFullSizeUnitGraph.GetValueOnGameThread();

		UFont* SmallFont = GEngine->GetSmallFont();
		check(SmallFont);
		int32 AlertPrintWidth = SmallFont->GetStringSize(TEXT("000.0"));
		int32 AlertPrintHeight = SmallFont->GetStringHeightSize(TEXT("000.0"));

		// For each type of statistic that we want to graph (0=Render, 1=Game, 2=GPU, 3=Frame)
		enum EGraphStats
		{
			EGS_Render = 0,
			EGS_Game,
			EGS_GPU,
			EGS_Frame,
			EGS_RHIT,
			EGS_UnboundedHighValueCount,

			EGS_DynRes = EGS_UnboundedHighValueCount,
			EGS_Count
		};

		// The vertical axis is time in milliseconds
		// The horizontal axis is the frame number (NOT time!!!)

		// Threshold where graph lines will pulsate for slow frames
		extern TAutoConsoleVariable<float> GTargetFrameTimeThresholdCVar;
		const float TargetTimeMS = GTargetFrameTimeThresholdCVar.GetValueOnGameThread();

		const float AlertTimeMS = TargetTimeMS;

		// Graph layout
		const float GraphHeight = (bSmallGraph ? 120.0f : 350.0f);
#if PLATFORM_ANDROID || PLATFORM_IOS
		const float GraphLeftXPos = 20.0f;
		const float GraphBottomYPos = GraphHeight + 80.0f;
#else
		const float GraphLeftXPos = 80.0f;
		const float GraphBottomYPos = InCanvas->GetRenderTarget()->GetSizeXY().Y / InCanvas->GetDPIScale() - 50.0f;
#endif

		const float GraphBackgroundMarginSize = 8.0f;
		const float GraphHorizPixelsPerFrame = (bSmallGraph ? 1.0f : 2.0f);

		const float TargetTimeMSHeight = GraphHeight * 0.85f;
		const float	MaxDynresTargetTimeMSHeight = TargetTimeMSHeight * 0.75f;

		const float OutOfBudgetMarginHeight = (bSmallGraph ? 1 : 3);

		const float GraphTotalWidth = GraphHorizPixelsPerFrame * NumberOfSamples;
		const float GraphTotalHeight = TargetTimeMSHeight + (OutOfBudgetMarginHeight + (float)EGS_UnboundedHighValueCount) * AlertPrintHeight;

		// Scale MS axis so that TargetTimeMS stays at fixed ordinate.
		const float GraphVerticalPixelsPerMS = TargetTimeMSHeight / TargetTimeMS;

		// Scale dyn res so that RawMaxResolutionFraction is at MaxDynresTargetTimeMSHeight or below.
		const float GraphVerticalPixelsPerResolutionFraction = FMath::Min(100.0f, MaxDynresTargetTimeMSHeight / GDynamicPrimaryResolutionFraction.GetSettings().EstimateCostScale(DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction]));

		// Compute pulse effect for lines above alert threshold
		const float AlertPulseFreq = 8.0f;
		const float AlertPulse = 0.5f + 0.5f * FMath::Sin((0.25f * UE_PI * 2.0) + (FApp::GetCurrentTime() * UE_PI * 2.0) * AlertPulseFreq);

		// Draw background.
		{
			FLinearColor BackgroundColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.7f);
			FCanvasTileItem BackgroundTile(
				FVector2D(GraphLeftXPos - GraphBackgroundMarginSize, GraphBottomYPos - GraphTotalHeight - GraphBackgroundMarginSize),
				FVector2D(GraphTotalWidth + 2 * GraphBackgroundMarginSize, GraphTotalHeight + 2 * GraphBackgroundMarginSize),
				BackgroundColor);

			BackgroundTile.BlendMode = SE_BLEND_AlphaBlend;

			InCanvas->DrawItem(BackgroundTile);
		}


		FBatchedElements* BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Line);
		FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

		// Reserve line vertices (2 border lines, 4 reference lines, then up to the maximum number of graph lines)
		BatchedElements->AddReserveLines(2 + 4 + EGS_Count * NumberOfSamples);

		// Draw timing graph frame.
		{
			const FLinearColor GraphBorderColor(0.1f, 0.1f, 0.1f);

			// Left
			BatchedElements->AddLine(
				FVector(GraphLeftXPos - 1.0f, GraphBottomYPos - GraphTotalHeight - GraphBackgroundMarginSize, 0.0f),
				FVector(GraphLeftXPos - 1.0f, GraphBottomYPos - 1.0f, 0.0f),
				GraphBorderColor,
				HitProxyId);

			// Bottom
			BatchedElements->AddLine(
				FVector(GraphLeftXPos - 1.0f, GraphBottomYPos - 1.0f, 0.0f),
				FVector(GraphLeftXPos + GraphHorizPixelsPerFrame * NumberOfSamples + GraphBackgroundMarginSize, GraphBottomYPos - 1.0f, 0.0f),
				GraphBorderColor,
				HitProxyId);

			InCanvas->DrawShadowedString(
				GraphLeftXPos - GraphBackgroundMarginSize,
				GraphBottomYPos - GraphTotalHeight - GraphBackgroundMarginSize - AlertPrintHeight - 2.0f,
				bShowRawUnitTimes ? TEXT("(Raw timings)") : TEXT("(Filtered timings)"), SmallFont, GraphBorderColor);
		}

		// Timing alert line
		{
			const FLinearColor LineColor(0.2f, 0.06f, 0.06f);
			FVector StartPos(
				GraphLeftXPos - 1.0f,
				GraphBottomYPos - AlertTimeMS * GraphVerticalPixelsPerMS,
				0.0f);
			FVector EndPos(
				GraphLeftXPos + GraphHorizPixelsPerFrame * NumberOfSamples + GraphBackgroundMarginSize,
				StartPos.Y,
				0.0f);

			BatchedElements->AddLine(
				StartPos,
				EndPos,
				LineColor,
				HitProxyId);

			InCanvas->DrawShadowedString(EndPos.X + 4.0f, EndPos.Y - AlertPrintHeight / 2, *FString::Printf(TEXT("%3.1f ms (budget)"), AlertTimeMS), SmallFont, LineColor);
		}

		// Screen percentage upper bound line
		{
			const FLinearColor LineColor(0.2f, 0.1f, 0.02f);
			FVector StartPos(
				GraphLeftXPos - 1.0f,
				GraphBottomYPos - GraphVerticalPixelsPerResolutionFraction * GDynamicPrimaryResolutionFraction.GetSettings().EstimateCostScale(DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction]),
				0.0f);
			FVector EndPos(
				GraphLeftXPos + GraphHorizPixelsPerFrame * NumberOfSamples + GraphBackgroundMarginSize,
				StartPos.Y,
				0.0f);

			BatchedElements->AddLine(
				StartPos,
				EndPos,
				LineColor,
				HitProxyId);

			float MaxScreenPercentage = DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction] * 100.0f;
			InCanvas->DrawShadowedString(
				EndPos.X + 4.0f,
				EndPos.Y - AlertPrintHeight / 2,
				*FString::Printf(TEXT("%3.1f%% x %3.1f%% (max)"), MaxScreenPercentage, MaxScreenPercentage), SmallFont, LineColor);
		}

		// Screen percentage = 100% native line
		if (DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction] > 1.0f)
		{
			const FLinearColor LineColor(0.2f, 0.1f, 0.02f);
			FVector StartPos(
				GraphLeftXPos - 1.0f,
				GraphBottomYPos - GraphVerticalPixelsPerResolutionFraction,
				0.0f);
			FVector EndPos(
				GraphLeftXPos + GraphHorizPixelsPerFrame * NumberOfSamples + GraphBackgroundMarginSize,
				StartPos.Y,
				0.0f);

			BatchedElements->AddLine(
				StartPos,
				EndPos,
				LineColor,
				HitProxyId);

			if (GraphVerticalPixelsPerResolutionFraction * (GDynamicPrimaryResolutionFraction.GetSettings().EstimateCostScale(DynamicResolutionStateInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction]) - 1.0f) >= AlertPrintHeight)
			{
				InCanvas->DrawShadowedString(EndPos.X + 4.0f, EndPos.Y - AlertPrintHeight / 2, TEXT("100.0% x 100.0% (native)"), SmallFont, LineColor);
			}
		}

		// Screen percentage = AlertResolutionFraction * 100 line
		{
			const FLinearColor LineColor(0.2f, 0.1f, 0.02f);
			FVector StartPos(
				GraphLeftXPos - 1.0f,
				GraphBottomYPos - GraphVerticalPixelsPerResolutionFraction * AlertResolutionFraction * AlertResolutionFraction,
				0.0f);
			FVector EndPos(
				GraphLeftXPos + GraphHorizPixelsPerFrame * NumberOfSamples + GraphBackgroundMarginSize,
				StartPos.Y,
				0.0f);

			BatchedElements->AddLine(
				StartPos,
				EndPos,
				LineColor,
				HitProxyId);

			float AlertScreenPercentage = AlertResolutionFraction * 100.0f;
			InCanvas->DrawShadowedString(EndPos.X + 4.0f, EndPos.Y - AlertPrintHeight / 2,
				*FString::Printf(TEXT("%3.1f%% x %3.1f%% (alert)"), AlertScreenPercentage, AlertScreenPercentage), SmallFont, LineColor);
		}

		int32 AlertPrintY = GraphBottomYPos - AlertTimeMS * GraphVerticalPixelsPerMS - OutOfBudgetMarginHeight * AlertPrintHeight;

		const bool bShowFrameTimeInUnitGraph = Client && Client->IsStatEnabled(TEXT("UnitTime"));
		for (int32 StatIndex = 0; StatIndex < EGS_Count; ++StatIndex)
		{
			int32 LastPrintX = 0xFFFFFFFF;
			AlertPrintY -= AlertPrintHeight;

			// If we don't have GPU data to display, then skip this line
			if ((StatIndex == EGS_GPU && !bHaveGPUData[0])
				|| (StatIndex == EGS_Frame && bShowFrameTimeInUnitGraph == false && bHaveGPUData[0])
				|| (StatIndex == EGS_RHIT && !IsRunningRHIInSeparateThread()))
			{
				continue;
			}

			FLinearColor StatColor;
			float* Values = NULL;
			float GraphVerticalPixelPerValue = 1.0f;
			float AbsoluteAlertValueThreshold = -1.0f;
			float RelativeAlertValueThreshold = -1.0f;
			int32 DisplayPow = 1;
			float DisplayMultiplier = 1.0f;
			bool HigherIsBest = false;
			if (StatIndex == EGS_Render)
			{
				AbsoluteAlertValueThreshold = AlertTimeMS;
				Values = RenderThreadTimes.GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerMS;
				StatColor = FLinearColor(0.1f, 0.1f, 1.0f);		// Blue
			}
			else if (StatIndex == EGS_Game)
			{
				AbsoluteAlertValueThreshold = AlertTimeMS;
				Values = GameThreadTimes.GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerMS;
				StatColor = FLinearColor(1.0f, 0.1f, 0.1f);		// Red
			}
			else if (StatIndex == EGS_GPU)
			{
				AbsoluteAlertValueThreshold = AlertTimeMS;
				// Multi-GPU support : We don't support more than 1 GPU in stat unitgraph yet.
				Values = GPUFrameTimes[0].GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerMS;
				StatColor = FLinearColor(1.0f, 1.0f, 0.1f);		// Yellow
			}
			else if (StatIndex == EGS_Frame)
			{
				AbsoluteAlertValueThreshold = AlertTimeMS;
				Values = FrameTimes.GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerMS;
				StatColor = FLinearColor(0.1f, 1.0f, 0.1f);		// Green
			}
			else if (StatIndex == EGS_RHIT)
			{
				AbsoluteAlertValueThreshold = AlertTimeMS;
				Values = RHITTimes.GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerMS;
				StatColor = FLinearColor(1.0f, 0.1f, 1.0f);		// Green
			}
			else if (StatIndex == EGS_DynRes) //-V547
			{
				const DynamicRenderScaling::FBudget& Budget = GDynamicPrimaryResolutionFraction;

				AbsoluteAlertValueThreshold = AlertResolutionFraction;
				RelativeAlertValueThreshold = 0.05f;
				Values = ResolutionFractions[GDynamicPrimaryResolutionFraction].GetData();
				GraphVerticalPixelPerValue = GraphVerticalPixelsPerResolutionFraction;
				StatColor = FLinearColor(1.0f, 0.5f, 0.1f);
				DisplayPow = Budget.GetSettings().Model == DynamicRenderScaling::EHeuristicModel::Quadratic ? 2 : 1;
				DisplayMultiplier = 100.0f;
				HigherIsBest = true;
				AlertPrintY = GraphBottomYPos - AlertResolutionFraction * AlertResolutionFraction * GraphVerticalPixelsPerResolutionFraction + AlertPrintHeight;
			}
			else
			{
				unimplemented();
			}


			// For each sample in our data set
			for (int32 CurFrameIndex = 0; CurFrameIndex < NumberOfSamples; ++CurFrameIndex)
			{
				const int32 PrevFrameIndex = FMath::Max(0, CurFrameIndex - 1);
				const int32 NextFrameIndex = FMath::Min(NumberOfSamples - 1, CurFrameIndex + 1);
				int32 PrevUnitIndex = (CurrentIndex + PrevFrameIndex) % NumberOfSamples;
				int32 CurUnitIndex = (CurrentIndex + CurFrameIndex) % NumberOfSamples;
				int32 NextUnitIndex = (CurrentIndex + NextFrameIndex) % NumberOfSamples;

				const float PrevValue = Values[PrevUnitIndex];
				const float CurValue = Values[CurUnitIndex];
				const float NextValue = Values[NextUnitIndex];

				if (CurValue < 0.0f || PrevValue < 0.0f)
				{
					continue;
				}

				const float MaxClampingY = GraphTotalHeight - 2 * StatIndex;

				const FVector LineStart(
					GraphLeftXPos + (float)PrevFrameIndex * GraphHorizPixelsPerFrame,
					GraphBottomYPos - FMath::Min(PrevValue * (DisplayPow == 2 ? PrevValue : 1.0f) * GraphVerticalPixelPerValue, MaxClampingY),
					0.0f);

				const FVector LineEnd(
					GraphLeftXPos + (float)CurFrameIndex * GraphHorizPixelsPerFrame,
					GraphBottomYPos - FMath::Min(CurValue * (DisplayPow == 2 ? CurValue : 1.0f) * GraphVerticalPixelPerValue, MaxClampingY),
					0.0f);

				BatchedElements->AddLine(LineStart, LineEnd, StatColor, HitProxyId);

				if (AbsoluteAlertValueThreshold < 0.0f)
				{
					continue;
				}

				// Absolute alert detection.
				bool Alert = (
					(!HigherIsBest && CurValue > AbsoluteAlertValueThreshold && (CurFrameIndex == 0 || PrevValue <= AbsoluteAlertValueThreshold)) ||
					(HigherIsBest && CurValue < AbsoluteAlertValueThreshold && (CurFrameIndex == 0 || PrevValue >= AbsoluteAlertValueThreshold)));
				float AlertValue = AbsoluteAlertValueThreshold;

				// If not absolute alert detection, look for relative alert.
				if (!Alert && RelativeAlertValueThreshold > 0.0f)
				{
					AlertValue = PrevValue * (1 - RelativeAlertValueThreshold);
					Alert = NextUnitIndex > 0 && (
						(!HigherIsBest && CurValue > AlertValue && CurValue >= NextValue) ||
						(HigherIsBest && CurValue < AlertValue && CurValue <= NextValue));
				}

				if (Alert)
				{
					const int32 AlertPadding = 1;
					float MaxValue = CurValue;
					int32 MinCheckFrames = FMath::Min<int32>(FPlatformMath::CeilToInt((float)AlertPrintWidth / GraphHorizPixelsPerFrame) + 10, NumberOfSamples);
					int32 CheckIndex = CurUnitIndex + 1;
					for (; CheckIndex < MinCheckFrames; ++CheckIndex)
					{
						MaxValue = HigherIsBest ? FMath::Min(MaxValue, Values[CheckIndex]) : FMath::Max(MaxValue, Values[CheckIndex]);
					}
					for (; CheckIndex < NumberOfSamples; ++CheckIndex)
					{
						if ((!HigherIsBest && Values[CheckIndex] <= AlertValue) ||
							(HigherIsBest && Values[CheckIndex] >= AlertValue))
						{
							break;
						}
						MaxValue = HigherIsBest ? FMath::Min(MaxValue, Values[CheckIndex]) : FMath::Max(MaxValue, Values[CheckIndex]);
					}

					int32 StartX = GraphLeftXPos + (float)PrevFrameIndex * GraphHorizPixelsPerFrame - AlertPrintWidth;
					if (StartX > LastPrintX)
					{

						InCanvas->DrawShadowedString(
							StartX,
							AlertValue != AbsoluteAlertValueThreshold ? LineEnd.Y : AlertPrintY,
							*FString::Printf(TEXT("%3.1f"), CurValue * DisplayMultiplier), SmallFont, StatColor);
						LastPrintX = StartX + AlertPrintWidth + AlertPadding;
					}
				}
			}
		}
	}
#endif	// !UE_BUILD_SHIPPING

	return InY;
}

int32 FStatThermalsData::DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY)
{
	const EDeviceThermalState CurrentThermalState = FPlatformMisc::GetDeviceThermalState();

	FLinearColor StatColor = FColor::White;
	FString SeverityStr;
	switch (CurrentThermalState)
	{
	case EDeviceThermalState::Unsupported:
		SeverityStr = TEXT("Unsupported");
		StatColor = FColor::Blue;
		break;
	case EDeviceThermalState::None:
		SeverityStr = TEXT("None");
		StatColor = FColor::Green;
		break;
	case EDeviceThermalState::Light:
		SeverityStr = TEXT("Light");
		StatColor = FColor::Yellow;
		break;
	case EDeviceThermalState::Moderate:
		SeverityStr = TEXT("Moderate");
		StatColor = FColor::Orange;
		break;
	case  EDeviceThermalState::Severe:
		SeverityStr = TEXT("Severe");
		StatColor = FColor::Red;
		break;
	case  EDeviceThermalState::Critical:
		SeverityStr = TEXT("Critical");
		StatColor = FColor::Red;
		break;
	case  EDeviceThermalState::Emergency:
		SeverityStr = TEXT("Emergency");
		StatColor = FColor::Red;
		break;
	default:
		SeverityStr = TEXT("Unknown");
		StatColor = FColor::Blue;
		break;
	}

	UFont* SmallFont = GEngine->GetSmallFont();
	check(SmallFont);

	const FString LabelText = TEXT("Thermal throttle state:   ");
	const int32 TitleSize = SmallFont->GetStringSize(*LabelText);

	const int32 LabelCoordX = InX - TitleSize;
	InCanvas->DrawShadowedString(
		LabelCoordX,
		InY,
		*LabelText,
		SmallFont,
		FColor::White);

	InCanvas->DrawShadowedString(
		InX,
		InY,
		*SeverityStr,
		SmallFont,
		StatColor);

	return InY;
}

int32 FStatHitchesData::DrawStat(FViewport* InViewport, FCanvas* InCanvas, int32 InX, int32 InY)
{
	const double CurrentTime = FPlatformTime::Seconds();
	if (LastTime > 0)
	{
		const float HitchThresholdSecs = FEnginePerformanceTargets::GetHitchFrameTimeThresholdMS() * 0.001f;

		const float DeltaSeconds = CurrentTime - LastTime;
		if (DeltaSeconds > HitchThresholdSecs)
		{
			Hitches[OverwriteIndex] = DeltaSeconds;
			When[OverwriteIndex] = CurrentTime;
			OverwriteIndex = (OverwriteIndex + 1) % NumHitches;

			UE_LOGF(LogClient, Warning, "HITCH %d              running cnt = %5d", int32(DeltaSeconds * 1000), Count++);
		}

		const int32 MaxY = InCanvas->GetRenderTarget()->GetSizeXY().Y;
		static const double TravelTime = 4.2;
		for (int32 i = 0; i < NumHitches; i++)
		{
			if (When[i] > 0 && When[i] <= CurrentTime && When[i] >= CurrentTime - TravelTime)
			{
				const float MyHitchSecs = Hitches[i];
				const float MyHitchMS = MyHitchSecs * 1000.0f;

				// Scale the time before passing in so that hitches aren't all red
				const FColor MyColor = GEngine->GetFrameTimeDisplayColor(MyHitchMS * 0.25f);

				const int32 MyY = InY + int32(float(MaxY - InY) * float((CurrentTime - When[i]) / TravelTime));
				const FString Hitch = FString::Printf(TEXT("%5d"), int32(MyHitchMS));
				InCanvas->DrawShadowedString(InX, MyY, *Hitch, GEngine->GetSmallFont(), MyColor);
			}
		}
	}
	LastTime = CurrentTime;
	return InY;
}


/*=============================================================================
//
// FViewport implementation.
//
=============================================================================*/

/** Send when a viewport is resized */
FViewport::FOnViewportResized FViewport::ViewportResizedEvent;

FViewport::FViewport(FViewportClient* const InClient)
	: FViewport()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient = InClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (InClient)
	{
		InClient->AddAssociation(*this);

		bRequiresHitProxyStorage = InClient->RequiresHitProxyStorage();
#if !WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (bRequiresHitProxyStorage)
		{
			UE_LOGF(LogClient, Warning, "Consoles don't need hitproxy storage - wasting memory!?");
		}
#endif

		InClient->AddAssociation(*this);
	}
}

FViewport::FViewport()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: ViewportClient(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, WindowMode(IsRunningGame() ? GSystemResolution.WindowMode : EWindowMode::Windowed)
	, bRequiresHitProxyStorage(false)
	, bHitProxiesCached(false)
	, bHasRequestedToggleFreeze(false)
	, bIsSlateViewport(false)
	, bIsHDR(false)
	, bTakeHighResScreenShot(false)
{
	//initialize the hit proxy kernel
	HitProxySize = 5;
	if (GIsEditor) 
	{
		GConfig->GetInt( TEXT("UnrealEd.HitProxy"), TEXT("HitProxySize"), (int32&)HitProxySize, GEditorIni );
		HitProxySize = FMath::Clamp( HitProxySize, (uint32)1, (uint32)MAX_HITPROXYSIZE );
	}

	AppVersionString = FString::Printf( TEXT( "Build: %s" ), FApp::GetBuildVersion() );

	bIsPlayInEditorViewport = false;
}

FViewport::~FViewport()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ViewportClient)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FViewport::SetViewportClient(TStrongPtrVariant<FViewportClient>(nullptr));
	}
}

bool FViewport::TakeHighResScreenShot()
{
	if( GScreenshotResolutionX == 0 && GScreenshotResolutionY == 0 )
	{
		GScreenshotResolutionX = SizeX * GetHighResScreenshotConfig().ResolutionMultiplier;
		GScreenshotResolutionY = SizeY * GetHighResScreenshotConfig().ResolutionMultiplier;
	}

	uint32 MaxTextureDimension = GetMax2DTextureDimension();

	// Check that we can actually create a destination texture of this size
	if (GScreenshotResolutionX > MaxTextureDimension || GScreenshotResolutionY > MaxTextureDimension)
	{
		// Send a notification to tell the user the screenshot has failed
		auto Message = NSLOCTEXT("UnrealClient", "HighResScreenshotTooBig", "The high resolution screenshot multiplier is too large for your system. Please try again with a smaller value!");
		FNotificationInfo Info(Message);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 5.0f;
		Info.bUseSuccessFailIcons = false;
		Info.bUseLargeFont = false;
		FSlateNotificationManager::Get().AddNotification(Info); 

		UE_LOGF(LogClient, Warning, "The specified multiplier for high resolution screenshot is too large for your system (requested size %ux%u, max size %ux%u)! Please try again with a smaller value.", GScreenshotResolutionX, GScreenshotResolutionY, MaxTextureDimension, MaxTextureDimension);

		GIsHighResScreenshot = false;
		return false;
	}
	else
	{
		// Everything is OK. Take the shot.
		bTakeHighResScreenShot = true;

		//Force a redraw.
		Invalidate();	

		return true;
	}
}

static void HighResScreenshotBeginFrame(FDummyViewport* DummyViewport)
{
	GFrameCounter++;
	ENQUEUE_RENDER_COMMAND(BeginFrameCommand, LoopTick)(
		[CurrentFrameCounter = GFrameCounter](FRHICommandListImmediate& RHICmdList)
	{
		GFrameCounterRenderThread = CurrentFrameCounter;
		GFrameNumberRenderThread++;
		FCoreDelegates::OnBeginFrameRT.Broadcast();
	});
}

static void HighResScreenshotEndFrame(FDummyViewport* DummyViewport)
{
	ENQUEUE_RENDER_COMMAND(EndFrameCommand, LoopTick)(
		[](FRHICommandListImmediate& RHICmdList)
	{
		FCoreDelegates::OnEndFrameRT.Broadcast();
		RHICmdList.EndFrame();
	});
}

void FViewport::HighResScreenshot()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* const ViewportClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (!ViewportClientPtr)
	{
		return;
	}

	FEngineShowFlags* const EngineShowFlags = ViewportClientPtr->GetEngineShowFlags();
	if (!EngineShowFlags)
	{
		return;
	}

	// We need to cache this as FScreenshotRequest is a global and the filename is
	// cleared out before we use it below
	const FString CachedScreenshotName = FScreenshotRequest::GetFilename();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TUniquePtr<FDummyViewport> DummyViewport = FDummyViewport::Create(ViewportClientPtr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DummyViewport->SetInitialSize(FIntPoint(
		(GScreenshotResolutionX > 0) ? GScreenshotResolutionX : SizeX,
		(GScreenshotResolutionY > 0) ? GScreenshotResolutionY : SizeY
	));

	DummyViewport->SetupHDR(GetDisplayColorGamut(), GetDisplayOutputFormat(), GetSceneHDREnabled());

	BeginInitResource(DummyViewport.Get());

	bool MaskShowFlagBackup = EngineShowFlags->HighResScreenshotMask;
	const auto MotionBlurShowFlagBackup = EngineShowFlags->MotionBlur;

	EngineShowFlags->SetHighResScreenshotMask(GetHighResScreenshotConfig().bMaskEnabled);
	EngineShowFlags->SetMotionBlur(false);

	// Forcing 128-bit rendering pipeline
	static IConsoleVariable* SceneColorFormatVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SceneColorFormat"));
	static IConsoleVariable* PostColorFormatVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessingColorFormat"));
	static IConsoleVariable* ForceLODVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForceLOD"));

	check(SceneColorFormatVar && PostColorFormatVar);
	const int32 OldSceneColorFormat = SceneColorFormatVar->GetInt();
	const int32 OldPostColorFormat = PostColorFormatVar->GetInt();
	const int32 OldForceLOD = ForceLODVar ? ForceLODVar->GetInt() : -1;

	if (GetHighResScreenshotConfig().bForce128BitRendering)
	{
		SceneColorFormatVar->Set(5, ECVF_SetByCode);
		PostColorFormatVar->Set(1, ECVF_SetByCode);
	}

	if (ForceLODVar)
	{
		// Force highest LOD
		ForceLODVar->Set(0, ECVF_SetByCode);
	}

	// Render the requested number of frames (at least once)
	static const auto HighResScreenshotDelay = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HighResScreenshotDelay"));
	const uint32 DefaultScreenshotDelay = 4;
	uint32 FrameDelay = HighResScreenshotDelay ? FMath::Max(HighResScreenshotDelay->GetValueOnGameThread(), 1) : DefaultScreenshotDelay;

	// End the frame that was started before HighResScreenshot() was called. Pass nullptr for the viewport because there's no need to
	// call EndRenderFrame on the real viewport, as BeginRenderFrame hasn't been called yet.
	HighResScreenshotEndFrame(nullptr);
	GIsHighResScreenshot = false;
	// Perform run-up.
	while (FrameDelay)
	{
		HighResScreenshotBeginFrame(DummyViewport.Get());
		if (FrameDelay == 1)
		{
			GIsHighResScreenshot = true;
		}
		UWorld* const World = ViewportClientPtr->GetWorld();
		FCanvas Canvas(DummyViewport.Get(), nullptr, World, World->GetFeatureLevel());
		{
			ViewportClientPtr->Draw(DummyViewport.Get(), &Canvas);
		}
		Canvas.Flush_GameThread();

		// Draw the debug canvas
		DummyViewport->GetDebugCanvas()->SetAllowedModes(FCanvas::Allow_DeleteOnRender);
		DummyViewport->GetDebugCanvas()->Flush_GameThread(true);

		HighResScreenshotEndFrame(DummyViewport.Get());
		
		FlushRenderingCommands();

		--FrameDelay;
	}

	EngineShowFlags->SetHighResScreenshotMask(MaskShowFlagBackup);
	EngineShowFlags->MotionBlur = MotionBlurShowFlagBackup;
	bool bIsScreenshotSaved = ViewportClientPtr->ProcessScreenShots(DummyViewport.Get());

	SceneColorFormatVar->Set(OldSceneColorFormat, ECVF_SetByCode);
	PostColorFormatVar->Set(OldPostColorFormat, ECVF_SetByCode);
	if (ForceLODVar)
	{
		ForceLODVar->Set(OldForceLOD, ECVF_SetByCode);
	}

	BeginReleaseResource(DummyViewport.Get());
	FlushRenderingCommands();
	DummyViewport.Reset();

	// once the screenshot is done we disable the feature to get only one frame
	GIsHighResScreenshot = false;
	bTakeHighResScreenShot = false;

	// Notification of a successful screenshot
	if ((GIsEditor || !IsFullscreen()) && !GIsAutomationTesting && bIsScreenshotSaved)
	{
		auto Message = NSLOCTEXT("UnrealClient", "HighResScreenshotSavedAs", "High resolution screenshot saved as");
		FNotificationInfo Info(Message);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 5.0f;
		Info.bUseSuccessFailIcons = false;
		Info.bUseLargeFont = false;
		
		const FString HyperLinkText = FPaths::ConvertRelativePathToFull(CachedScreenshotName);
		Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString SourceFilePath) 
		{
			FPlatformProcess::ExploreFolder(*(FPaths::GetPath(SourceFilePath)));
		}, HyperLinkText);
		Info.HyperlinkText = FText::FromString(HyperLinkText);
		
		FSlateNotificationManager::Get().AddNotification(Info);
		UE_LOGF(LogClient, Log, "%ls %ls", *Message.ToString(), *HyperLinkText);
	}

	// Start a new frame for the real viewport. Same as above, pass nullptr because BeginRenderFrame will be called
	// after this function returns.
	HighResScreenshotBeginFrame(nullptr);
}

FRHIGPUMask FViewport::GetGPUMask(FRHICommandListImmediate& RHICmdList) const
{
	// @todo - MGPU AFR is no longer supported. RHIGetViewportNextPresentGPUIndex should be removed.
	// We cannot call that function here as we no longer have the FRHIViewport instance.
	return FRHIGPUMask::GPU0();//RHIGetViewportNextPresentGPUIndex(GetViewportRHI()));
}

void AddStreamingViewInfo(UWorld& World, const FSceneView& View)
{
	const float StreamingScale = 1.f / FMath::Clamp(View.LODDistanceFactor, 0.2f, 1.f);
	IStreamingManager::Get().AddViewInformation(
		View.ViewMatrices.GetViewOrigin(),
		View.UnscaledViewRect.Width(),
		View.UnscaledViewRect.Width() * View.ViewMatrices.GetViewToClip().M[0][0],
		StreamingScale, false, 0, nullptr, &World);

	World.ViewLocationsRenderedLastFrame.Add(View.ViewMatrices.GetViewOrigin());

	FWorldCachedViewInfo& Info = World.CachedViewInfoRenderedLastFrame.AddDefaulted_GetRef();
	Info.ViewMatrix = View.ViewMatrices.GetWorldToView();
	Info.ProjectionMatrix = View.ViewMatrices.GetViewToClip();
	Info.ViewProjectionMatrix = View.ViewMatrices.GetWorldToClip();
	Info.ViewToWorld = View.ViewMatrices.GetViewToWorld();
}

void FViewport::Draw( bool bShouldPresent /*= true */)
{
	SCOPED_NAMED_EVENT(FViewport_Draw, FColor::Red);
	FViewportClient& ViewportClientRef = GetClientChecked();
	UWorld* World = ViewportClientRef.GetWorld();

	// See what screenshot related features are required
	static const auto CVarDumpFrames = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BufferVisualizationDumpFrames"));
	GIsHighResScreenshot = GIsHighResScreenshot || bTakeHighResScreenShot;
	bool bAnyScreenshotsRequired = FScreenshotRequest::IsScreenshotRequested() || GIsHighResScreenshot || GIsDumpingMovie;
	bool bBufferVisualizationDumpingRequired = bAnyScreenshotsRequired && CVarDumpFrames && CVarDumpFrames->GetValueOnGameThread();

	// if this is a game viewport, and game rendering is disabled, then we don't want to actually draw anything
	if (!(World && World->IsGameWorld() && !bIsGameRenderingEnabled))
	{
		if( GIsHighResScreenshot )
		{
			const bool bShowUI = false;
			const bool bAddFilenameSuffix = GetHighResScreenshotConfig().FilenameOverride.IsEmpty();
			const bool bHDRScreenshot = GetSceneHDREnabled();
			FScreenshotRequest::RequestScreenshot( FString(), bShowUI, bAddFilenameSuffix, bHDRScreenshot);
			HighResScreenshot();
		}
		else if(bAnyScreenshotsRequired && bBufferVisualizationDumpingRequired)
		{
			// request the screenshot early so we have the name setup that BufferVisualization can dump it's content
			const bool bShowUI = false;
			const bool bAddFilenameSuffix = true;
			FScreenshotRequest::RequestScreenshot( FString(), bShowUI, bAddFilenameSuffix );
		}
	
		if( SizeX > 0 && SizeY > 0 )
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VSync"));
			bool bLockToVsync = CVar->GetValueOnGameThread() != 0;
			ULocalPlayer* Player = (GEngine && World) ? GEngine->GetFirstGamePlayer(World) : NULL;
			if (Player && Player->PlayerController)
			{
				bLockToVsync |= (Player->PlayerController->bCinematicMode);
			}
	 		EnqueueBeginRenderFrame(bShouldPresent);

			// Calculate gamethread time (excluding idle time)
			{
				static uint64 LastFrameUpdated = MAX_uint64;
				if (GFrameCounter != LastFrameUpdated)
				{
					static uint32 Lastimestamp = 0;
					static bool bStarted = false;
					uint32 CurrentTime	= FPlatformTime::Cycles();
					UE::Stats::FThreadIdleStats& GameThread = UE::Stats::FThreadIdleStats::Get();
					if (bStarted)
					{
						uint32 ThreadTime	= CurrentTime - Lastimestamp;
						// add any stalls via sleep or fevent
						GGameThreadTime		= (ThreadTime > GameThread.Waits) ? (ThreadTime - GameThread.Waits) : ThreadTime;
						GGameThreadWaitTime = GameThread.Waits;

						// Compute GGameThreadTimeCriticalPath
						uint32 GameThreadNonCriticalPathIdle = GameThread.Waits - GameThread.WaitsCriticalPath;
						GGameThreadTimeCriticalPath = (ThreadTime > GameThreadNonCriticalPathIdle) ? (ThreadTime - GameThreadNonCriticalPathIdle) : ThreadTime;
					}
					else
					{
						bStarted = true;
					}

					LastFrameUpdated = GFrameCounter;
					Lastimestamp		= CurrentTime;
					GameThread.Reset();
					GameThread.IsCriticalPathCounter = 0;
				}
			}

			FCanvas Canvas(this, nullptr, World, World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel, FCanvas::CDM_DeferDrawing, ViewportClientRef.ShouldDPIScaleSceneCanvas() ? ViewportClientRef.GetDPIScale() : 1.0f);
			Canvas.SetRenderTargetRect(FIntRect(0, 0, SizeX, SizeY));
			{
				ViewportClientRef.Draw(this, &Canvas);
			}
			Canvas.Flush_GameThread();
				
			UGameViewportClient::OnViewportRendered().Broadcast(this);

			ViewportClientRef.ProcessScreenShots(this);
	
			// Slate doesn't present immediately. Tag the viewport as requiring vsync so that it happens.
			SetRequiresVsync(bLockToVsync);
			EnqueueEndRenderFrame(bLockToVsync, bShouldPresent);
		}
	}

	// countdown the present delay, and then stop the movie at the end
	// this doesn't need to be on rendering thread as long as we have a long enough delay (2 or 3 frames), because
	// the rendering thread will never be more than one frame behind
	if (PresentAndStopMovieDelay > 0)
	{
		PresentAndStopMovieDelay--;
		// stop any playing movie
		if (PresentAndStopMovieDelay == 0)
		{
			// Enable game rendering again if it isn't already.
			bIsGameRenderingEnabled = true;
		}
	}
}


void FViewport::InvalidateHitProxy()
{
	bHitProxiesCached = false;
	HitProxyMap.Invalidate();
	
	FCanvas* DebugCanvas = GetDebugCanvas();
	if (DebugCanvas)
	{
		DebugCanvas->SetHitProxy(nullptr);
	}
}



void FViewport::Invalidate()
{
	DeferInvalidateHitProxy();
	InvalidateDisplay();
}


void FViewport::DeferInvalidateHitProxy()
{
	// Default implementation does not defer.  Overridden implementations may.
	InvalidateHitProxy();
}

const TArray<FColor>& FViewport::GetRawHitProxyData(FIntRect InRect)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* ViewportClientPtr = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FScopedConditionalWorldSwitcher WorldSwitcher(ViewportClientPtr);

	const bool bIsRenderingStereo = GEngine->IsStereoscopic3D( this );

	bool bFetchHitProxyBytes = !bIsRenderingStereo && ( !bHitProxiesCached || (SizeY*SizeX) != CachedHitProxyData.Num() );

	if (bIsRenderingStereo)
	{
		// Stereo viewports don't support hit proxies, and we don't want to update them because it will adversely
		// affect performance.
		CachedHitProxyData.SetNumZeroed( SizeY * SizeX );
	}
	// If the hit proxy map isn't up to date, render the viewport client's hit proxies to it.
	else if (!bHitProxiesCached)
	{
		SCOPED_NAMED_EVENT(HitProxyMapGen, FColor::Red);
		
		RenderCaptureInterface::FScopedCapture RenderCapture(GHitProxyCaptureNextUpdate != 0, TEXT("Update Hit Proxies"));
		GHitProxyCaptureNextUpdate = 0;

		EnqueueBeginRenderFrame(false);

		FViewport* Viewport = this;
		ENQUEUE_RENDER_COMMAND(BeginDrawingCommandHitProxy)(
			[Viewport](FRHICommandListImmediate& RHICmdList)
			{
				// Set the hit proxy map's render target.
				// Clear the hit proxy map to white, which is overloaded to mean no hit proxy.
				FRHITexture* RenderTarget = Viewport->HitProxyMap.GetRenderTargetTexture();
				RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::Unknown, ERHIAccess::RTV));
				FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::Clear_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearHitProxyMap"));
				RHICmdList.EndRenderPass();
			});

		// Let the viewport client draw its hit proxies.
		FCanvas Canvas(&HitProxyMap, &HitProxyMap, ViewportClientPtr->GetWorld(), GetFeatureLevel(), FCanvas::CDM_DeferDrawing,
			ViewportClientPtr->ShouldDPIScaleSceneCanvas() ? ViewportClientPtr->GetDPIScale() : 1.0f);
		{
			ViewportClientPtr->Draw(this, &Canvas);
		}
		Canvas.Flush_GameThread();

		//Resolve surface to texture.
		FHitProxyMap* HitProxyMapPtr = &HitProxyMap;
		ENQUEUE_RENDER_COMMAND(UpdateHitProxyRTCommand)(
			[HitProxyMapPtr](FRHICommandListImmediate& RHICmdList)
			{
				TransitionAndCopyTexture(RHICmdList, HitProxyMapPtr->GetRenderTargetTexture(), HitProxyMapPtr->GetHitProxyCPUTexture(), {});
			});

		EnqueueEndRenderFrame(false, false);

		// Cache the hit proxies for the next GetHitProxyMap call.
		bHitProxiesCached = true;
	}

	if (bFetchHitProxyBytes)
	{
		// Read the hit proxy map surface data back.
		SCOPED_NAMED_EVENT(HitProxyReadback, FColor::Red);
		FIntRect ViewportRect(0, 0, SizeX, SizeY);
		struct FReadSurfaceContext
		{
			FViewport* Viewport;
			TArray<FColor>* OutData;
			FIntRect Rect;
		};
		FReadSurfaceContext Context =
		{
			this,
			&CachedHitProxyData,
			ViewportRect
		};

		ENQUEUE_RENDER_COMMAND(ReadSurfaceCommand)(
			[Context](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.ReadSurfaceData(
				Context.Viewport->HitProxyMap.GetHitProxyCPUTexture(),
				Context.Rect,
				*Context.OutData,
				FReadSurfaceDataFlags()
				);
			});
		FlushRenderingCommands();
	}

	return CachedHitProxyData;
}

void FViewport::GetHitProxyMap(FIntRect InRect,TArray<HHitProxy*>& OutMap)
{
	const TArray<FColor>& CachedData = GetRawHitProxyData(InRect);
	if (CachedData.Num()==0)
	{
		return;
	}
	
	// Map the hit proxy map surface data to hit proxies.
	OutMap.Empty(InRect.Width() * InRect.Height());
	for(int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y++)
	{
		if (!CachedData.IsValidIndex(int32(Y * SizeX)))
		{
			break;
		}

		const FColor* SourceData = &CachedData[Y * SizeX];
		for(int32 X = InRect.Min.X ;X < InRect.Max.X; X++)
		{
			FHitProxyId HitProxyId(SourceData[X]);
			OutMap.Add(GetHitProxyById(HitProxyId));
		}
	}
}

HHitProxy* FViewport::GetHitProxy(int32 X, int32 Y)
{
	const FIntPoint VPSize = GetSizeXY();
	// if we haven't been initialized, we can't really hit anything.
	if (VPSize == FIntPoint::ZeroValue)
	{
		return nullptr;
	}

	// Compute a HitProxySize x HitProxySize test region with the center at (X,Y).
	int32	MinX = X - HitProxySize,
			MinY = Y - HitProxySize,
			MaxX = X + HitProxySize,
			MaxY = Y + HitProxySize;
	
	// Clip the region to the viewport bounds.
	MinX = FMath::Clamp(MinX, 0, VPSize.X - 1);
	MinY = FMath::Clamp(MinY, 0, VPSize.Y - 1);
	MaxX = FMath::Clamp(MaxX, 0, VPSize.X - 1);
	MaxY = FMath::Clamp(MaxY, 0, VPSize.Y - 1);

	int32 TestSizeX	= MaxX - MinX + 1;
	int32 TestSizeY	= MaxY - MinY + 1;
	HHitProxy* HitProxy = NULL;

	if(TestSizeX > 0 && TestSizeY > 0)
	{
		// Read the hit proxy map from the device.
		TArray<HHitProxy*>	ProxyMap;
		GetHitProxyMap(FIntRect(MinX, MinY, MaxX + 1, MaxY + 1),ProxyMap);
		check(ProxyMap.Num() == TestSizeX * TestSizeY);

		// Find the hit proxy in the test region with the highest order.
		int32 ProxyIndex = TestSizeY/2 * TestSizeX + TestSizeX/2;
		check(ProxyIndex<ProxyMap.Num());
		HitProxy = ProxyMap[ProxyIndex];
	
		bool bIsOrtho = GetClientChecked().IsOrtho();

		for(int32 TestY = 0;TestY < TestSizeY;TestY++)
		{
			for(int32 TestX = 0;TestX < TestSizeX;TestX++)
			{
				HHitProxy* TestProxy = ProxyMap[TestY * TestSizeX + TestX];
				if(TestProxy && (!HitProxy || (bIsOrtho ? TestProxy->OrthoPriority : TestProxy->Priority) > (bIsOrtho ? HitProxy->OrthoPriority : HitProxy->Priority)))
				{
					HitProxy = TestProxy;
				}
			}
		}
	}

	return HitProxy;
}

void FViewport::GetActorsAndModelsInHitProxy(FIntRect InRect, TSet<AActor*>& OutActors, TSet<UModel*>& OutModels)
{
	OutActors.Empty();
	OutModels.Empty();

	EnumerateHitProxiesInRect(InRect, [&OutActors, &OutModels](HHitProxy* HitProxy)
			{
				if( HitProxy->IsA(HActor::StaticGetType()) )
				{
					AActor* Actor = ((HActor*)HitProxy)->Actor;
					if (Actor)
					{
						OutActors.Add(Actor);
					}
				}
				else if( HitProxy->IsA(HModel::StaticGetType()) )
				{
					OutModels.Add( ((HModel*)HitProxy)->GetModel() );
				}
				else if( HitProxy->IsA(HBSPBrushVert::StaticGetType()) )
				{
					HBSPBrushVert* HitBSPBrushVert = ((HBSPBrushVert*)HitProxy);
					if( HitBSPBrushVert->Brush.IsValid() )
					{
						OutActors.Add( HitBSPBrushVert->Brush.Get() );
					}
				}
		return true;
	});
}

FTypedElementHandle FViewport::GetElementHandleAtPoint(int32 X, int32 Y)
{
	if (HHitProxy* HitProxy = GetHitProxy(X, Y))
	{
		return HitProxy->GetElementHandle();
	}
	return FTypedElementHandle();
}

void FViewport::GetElementHandlesInRect(FIntRect InRect, FTypedElementListRef OutElementHandles)
{
	OutElementHandles->Reset();

	EnumerateHitProxiesInRect(InRect, [&OutElementHandles](HHitProxy* HitProxy)
	{
		if (FTypedElementHandle ElementHandle = HitProxy->GetElementHandle())
		{
			OutElementHandles->Add(MoveTemp(ElementHandle));
		}
		return true;
	});
}

void FViewport::EnumerateHitProxiesInRect(FIntRect InRect, TFunctionRef<bool(HHitProxy*)> InCallback)
{
	const TArray<FColor>& RawHitProxyData = GetRawHitProxyData(InRect);

	// Lower the resolution with massive box selects
	const int32 Step = (InRect.Width() > 500 && InRect.Height() > 500) ? 4 : 1;

	for (int32 Y = InRect.Min.Y; Y < InRect.Max.Y; Y = Y < InRect.Max.Y-1 ? FMath::Min(InRect.Max.Y-1, Y+Step) : ++Y)
	{
		const FColor* SourceData = &RawHitProxyData[Y * SizeX];
		for (int32 X = InRect.Min.X; X < InRect.Max.X; X = X < InRect.Max.X-1 ? FMath::Min(InRect.Max.X-1, X+Step) : ++X)
		{
			FHitProxyId HitProxyId(SourceData[X]);
			HHitProxy* HitProxy = GetHitProxyById(HitProxyId);

			if (HitProxy && !InCallback(HitProxy))
			{
				return;
			}
		}
	}
}

FIntRect FViewport::CalculateViewExtents(float AspectRatio, float DesiredAspectRatio, const FIntRect& ViewRect, const FIntPoint& DestSize)
{
	FIntRect Result = ViewRect;

	const float CurrentSizeX = ViewRect.Width();
	const float CurrentSizeY = ViewRect.Height();

	// the viewport's SizeX/SizeY may not always match the DesiredAspectRatio, so adjust the requested AspectRatio to compensate
	const float AdjustedAspectRatio = AspectRatio / (DesiredAspectRatio / ((float)DestSize.X / (float)DestSize.Y));

	// If desired, enforce a particular aspect ratio for the render of the scene. 
	// Results in black bars at top/bottom etc.
	const float AspectRatioDifference = AdjustedAspectRatio - (CurrentSizeX / CurrentSizeY);

	if (FMath::Abs(AspectRatioDifference) > 0.01f)
	{
		// If desired aspect ratio is bigger than current - we need black bars on top and bottom.
		if (AspectRatioDifference > 0.0f)
		{
			// Calculate desired Y size.
			const int32 NewSizeY = FMath::Max(1, FMath::RoundToInt(CurrentSizeX / AdjustedAspectRatio));
			Result.Min.Y = FMath::RoundToInt(0.5f * (CurrentSizeY - NewSizeY));
			Result.Max.Y = Result.Min.Y + NewSizeY;
			Result.Min.Y += ViewRect.Min.Y;
			Result.Max.Y += ViewRect.Min.Y;
		}
		// Otherwise - will place bars on the sides.
		else
		{
			const int32 NewSizeX = FMath::Max(1, FMath::RoundToInt(CurrentSizeY * AdjustedAspectRatio));
			Result.Min.X = FMath::RoundToInt(0.5f * (CurrentSizeX - NewSizeX));
			Result.Max.X = Result.Min.X + NewSizeX;
			Result.Min.X += ViewRect.Min.X;
			Result.Max.X += ViewRect.Min.X;
		}
	}

	return Result;
}

FIntRect FViewport::CalculateViewExtents(float AspectRatio, const FIntRect& ViewRect)
{
	return CalculateViewExtents(AspectRatio, GetDesiredAspectRatio(), ViewRect, GetSizeXY());
}

void FViewport::SetViewportClient(const TStrongPtrVariant<FViewportClient>& InViewportClient)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* const OldClient = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FViewportClient* const NewClient = InViewportClient.Get();
	if (OldClient == NewClient)
	{
		return;
	}

	if (OldClient)
	{
		OldClient->RemoveAssociation(*this);
	}

	Client = InViewportClient;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient = NewClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (NewClient)
	{
		bRequiresHitProxyStorage = NewClient->RequiresHitProxyStorage();
#if !WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (bRequiresHitProxyStorage)
		{
			UE_LOGF(LogClient, Warning, "Consoles don't need hitproxy storage - wasting memory!?");
		}
#endif

		NewClient->AddAssociation(*this);
	}
	else
	{
		bRequiresHitProxyStorage = false;
	}
}

void FViewport::ReleaseRHI()
{
	HitProxyMap.Release();
	RenderTargetTextureRHI.SafeRelease();
}

void FViewport::InitRHI(FRHICommandListBase& RHICmdList)
{
	if(bRequiresHitProxyStorage)
	{
		// Initialize the hit proxy map.
		HitProxyMap.Init(RHICmdList, SizeX, SizeY);
	}
}

ENGINE_API bool IsCtrlDown(FViewport* Viewport) { return (Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl)); }
ENGINE_API bool IsShiftDown(FViewport* Viewport) { return (Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift)); }
ENGINE_API bool IsAltDown(FViewport* Viewport) { return (Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt)); }


/** Constructor */
FViewport::FHitProxyMap::FHitProxyMap()
{
}


/** Destructor */
FViewport::FHitProxyMap::~FHitProxyMap()
{
}


void FViewport::FHitProxyMap::Init(FRHICommandListBase& RHICmdList, uint32 NewSizeX,uint32 NewSizeY)
{
	SizeX = NewSizeX;
	SizeY = NewSizeY;

	// Create a render target to store the hit proxy map.
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("HitProxyTexture"))
			.SetExtent(SizeX, SizeY)
			.SetFormat(PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetClearValue(FClearValueBinding::White)
			.SetInitialState(ERHIAccess::SRVMask);

		RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);
	}
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("HitProxyCPUTexture"))
			.SetExtent(SizeX, SizeY)
			.SetFormat(PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::CPUReadback);

		HitProxyCPUTexture = RHICmdList.CreateTexture(Desc);
	}
}

void FViewport::FHitProxyMap::Init(uint32 NewSizeX, uint32 NewSizeY)
{
	Init(FRHICommandListImmediate::Get(), NewSizeX, NewSizeY);
}

void FViewport::FHitProxyMap::Release()
{
	HitProxyCPUTexture.SafeRelease();
	RenderTargetTextureRHI.SafeRelease();
}

void FViewport::FHitProxyMap::Invalidate()
{
	HitProxies.Empty();
}

void FViewport::FHitProxyMap::AddHitProxy(HHitProxy* HitProxy)
{
	HitProxies.Add(HitProxy);
}


/** FGCObject: add UObject references to GC */
void FViewport::FHitProxyMap::AddReferencedObjects( FReferenceCollector& Collector )
{
	// Allow all of our hit proxy objects to serialize their references
	for( int32 CurProxyIndex = 0; CurProxyIndex < HitProxies.Num(); ++CurProxyIndex )
	{
		HHitProxy* CurProxy = HitProxies[ CurProxyIndex ];
		if( CurProxy != NULL )
		{
			CurProxy->AddReferencedObjects( Collector );
		}
	}
}

FString FViewport::FHitProxyMap::GetReferencerName() const
{
	return TEXT("FViewport::FHitProxyMap");
}

FViewportClient& FViewport::GetClientChecked() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FViewportClient* Result = ViewportClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	check(Result);
	return *Result;
}

/**
 * Globally enables/disables rendering
 *
 * @param bIsEnabled true if drawing should occur
 * @param PresentAndStopMovieDelay Number of frames to delay before enabling bPresent in RHIEndDrawingViewport, and before stopping the movie
 */
void FViewport::SetGameRenderingEnabled(bool bIsEnabled, int32 InPresentAndStopMovieDelay)
{
	bIsGameRenderingEnabled = bIsEnabled;
	PresentAndStopMovieDelay = InPresentAndStopMovieDelay;
}

/**
 * Handles freezing/unfreezing of rendering
 */
void FViewport::ProcessToggleFreezeCommand()
{
	bHasRequestedToggleFreeze = 1;
}

/**
 * Returns if there is a command to toggle freezing
 */
bool FViewport::HasToggleFreezeCommand()
{
	// save the current command
	bool ReturnVal = bHasRequestedToggleFreeze;
	
	// make sure that we no longer have the command, as we are now passing off "ownership"
	// of the command
	bHasRequestedToggleFreeze = false;

	// return what it was
	return ReturnVal;
}

ENGINE_API bool GetViewportScreenShot(FViewport* Viewport, TArray<FColor>& Bitmap, const FIntRect& ViewRect /*= FIntRect()*/, const FReadSurfaceDataFlags& ReadSurfaceFlags /*= FReadSurfaceDataFlags()*/)
{
	// Read the contents of the viewport into an array.
	if (Viewport->ReadPixels(Bitmap, ReadSurfaceFlags, ViewRect))
	{
		check(Bitmap.Num() == ViewRect.Area() || (Bitmap.Num() == Viewport->GetRenderTargetTextureSizeXY().X * Viewport->GetRenderTargetTextureSizeXY().Y));
		return true;
	}

	return false;
}

ENGINE_API bool GetViewportScreenShotHDR(FViewport* Viewport, TArray<FLinearColor>& Bitmap, const FIntRect& ViewRect /*= FIntRect()*/, const FReadSurfaceDataFlags& ReadSurfaceFlags /*= FReadSurfaceDataFlags(RCM_MinMax)*/)
{
	// Read the contents of the viewport into an array.
	if (Viewport->ReadLinearColorPixels(Bitmap, ReadSurfaceFlags, ViewRect))
	{
		check(Bitmap.Num() == ViewRect.Area() || (Bitmap.Num() == Viewport->GetRenderTargetTextureSizeXY().X * Viewport->GetRenderTargetTextureSizeXY().Y));
		return true;
	}

	return false;
}

extern bool ParseResolution( const TCHAR* InResolution, uint32& OutX, uint32& OutY, int32& WindowMode );

ENGINE_API bool GetHighResScreenShotInput(const TCHAR* Cmd, FOutputDevice& Ar, uint32& OutXRes, uint32& OutYRes, float& OutResMult, FIntRect& OutCaptureRegion, bool& OutShouldEnableMask, bool& OutDumpBufferVisualizationTargets, bool& OutCaptureHDR, FString& OutFilenameOverride, bool& OutUseDateTimeAsFileName)
{
	TArray<FString> Arguments;
	const FString FilenameSearchString = "filename=";

	// FParse::Value has better handling of escape characters than FParse::Token
	if (!FParse::Value(Cmd, *FilenameSearchString, OutFilenameOverride))
	{
		OutFilenameOverride.Reset();
	}

	FString Arg;
	while (FParse::Token(Cmd, Arg, true))
	{
		// Now skip filename since we already processed it
		if (!Arg.StartsWith(FilenameSearchString))
		{
			Arguments.Add(Arg);
		}
	}

	int32 NumArguments = Arguments.Num();

	if (NumArguments >= 1)
	{
		int32 WindowModeDummy;
		if( !ParseResolution( *Arguments[0], OutXRes, OutYRes, WindowModeDummy ) )
		{
			//If Cmd is valid and it's not a resolution then the input must be a multiplier.
			float Mult = FCString::Atof(*Arguments[0]);

			if( Mult > 0.0f && Arguments[0].IsNumeric() )
			{
				OutResMult = Mult;
			}
			else
			{
				Ar.Logf(TEXT("Error: Bad input. Input should be in either the form \"HighResShot 1920x1080\" or \"HighResShot 2\""));
				return false;
			}
		}
		else if( OutXRes <= 0 || OutYRes <= 0  )
		{
			Ar.Logf(TEXT("Error: Values must be greater than 0 in both dimensions"));
			return false;
		}
		else if( OutXRes > GetMax2DTextureDimension() || OutYRes > GetMax2DTextureDimension()  )
		{
			Ar.Logf(TEXT("Error: Screenshot size exceeds the maximum allowed texture size (%d x %d)"), GetMax2DTextureDimension(), GetMax2DTextureDimension());
			return false;
		}

		// Try and extract capture region from string
		int32 CaptureRegionX = NumArguments > 1 ? FCString::Atoi(*Arguments[1]) : 0;
		int32 CaptureRegionY = NumArguments > 2 ? FCString::Atoi(*Arguments[2]) : 0;
		int32 CaptureRegionWidth = NumArguments > 3 ? FCString::Atoi(*Arguments[3]) : 0;
		int32 CaptureRegionHeight = NumArguments > 4 ? FCString::Atoi(*Arguments[4]) : 0;
		OutCaptureRegion = FIntRect(CaptureRegionX, CaptureRegionY, CaptureRegionX + CaptureRegionWidth, CaptureRegionY + CaptureRegionHeight);

		OutShouldEnableMask = NumArguments > 5 ? FCString::Atoi(*Arguments[5]) != 0 : false;
		OutDumpBufferVisualizationTargets = NumArguments > 6 ? FCString::Atoi(*Arguments[6]) != 0 : false;
		OutCaptureHDR = NumArguments > 7 ? FCString::Atoi(*Arguments[7]) != 0 : false;
		OutUseDateTimeAsFileName = NumArguments > 8 ? FCString::Atoi(*Arguments[8]) != 0 : false;


		return true;
	}
	else
	{
		Ar.Logf(TEXT("Error: Bad input. Input should be in either the form \"HighResShot 1920x1080\" or \"HighResShot 2\""));
	}

	return false;
}

/** Tracks the viewport client that should process the stat command, can be NULL */
FCommonViewportClient* GStatProcessingViewportClient = NULL;


float FCommonViewportClient::GetDPIScale() const
{
	if (bShouldUpdateDPIScale)
	{
		CachedDPIScale = UpdateViewportClientWindowDPIScale();

		bShouldUpdateDPIScale = false;
	}

	return CachedDPIScale;
}

void FCommonViewportClient::DrawHighResScreenshotCaptureRegion(FCanvas& Canvas)
{
	const FLinearColor BoxColor = FLinearColor::Red;
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();

	FCanvasLineItem LineItem;

	// Draw the line a line in X and Y extending out from the center.
	LineItem.SetColor( BoxColor );
	LineItem.Draw( &Canvas, FVector2D(Config.UnscaledCaptureRegion.Min.X, Config.UnscaledCaptureRegion.Min.Y), FVector2D(Config.UnscaledCaptureRegion.Max.X, Config.UnscaledCaptureRegion.Min.Y) );
	LineItem.Draw( &Canvas, FVector2D(Config.UnscaledCaptureRegion.Max.X, Config.UnscaledCaptureRegion.Min.Y), FVector2D(Config.UnscaledCaptureRegion.Max.X, Config.UnscaledCaptureRegion.Max.Y));
	LineItem.Draw( &Canvas, FVector2D(Config.UnscaledCaptureRegion.Max.X, Config.UnscaledCaptureRegion.Max.Y), FVector2D(Config.UnscaledCaptureRegion.Min.X, Config.UnscaledCaptureRegion.Max.Y));
	LineItem.Draw( &Canvas, FVector2D(Config.UnscaledCaptureRegion.Min.X, Config.UnscaledCaptureRegion.Max.Y), FVector2D(Config.UnscaledCaptureRegion.Min.X, Config.UnscaledCaptureRegion.Min.Y));
}

void FViewportClient::RedrawRequested(FViewport* Viewport)
{
	Viewport->Draw();
}

void FViewportClient::RequestInvalidateHitProxy(FViewport* Viewport)
{
	Viewport->InvalidateHitProxy();
}

TOptional<TSharedRef<SWidget>> FViewportClient::MapCursor(FViewport* Viewport, const FCursorReply& CursorReply)
{
	return TOptional<TSharedRef<SWidget>>();
}

FPopupMethodReply FViewportClient::OnQueryPopupMethod() const
{
	return FPopupMethodReply::Unhandled();
}

FCommonViewportClient::~FCommonViewportClient()
{
	//make to clean up the global "stat" client when we delete the active one.
	if (GStatProcessingViewportClient == this)
	{
		GStatProcessingViewportClient = NULL;
	}
}

void FCommonViewportClient::RequestUpdateDPIScale()
{
	bShouldUpdateDPIScale = true;
}

float FCommonViewportClient::GetDPIDerivedResolutionFraction() const
{
	#if WITH_EDITOR
	if (GIsEditor)
	{
		// When in high res screenshot do not modify screen percentage based on dpi scale
		if (GIsHighResScreenshot)
		{
			return 1.0f;
		}

		static auto CVarEditorViewportHighDPIPtr = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.HighDPI"));

		if (CVarEditorViewportHighDPIPtr && CVarEditorViewportHighDPIPtr->GetInt() == 0)
		{
			return FMath::Min(1.0f / GetDPIScale(), 1.0f);
		}
	}
	#endif

	return 1.0f;
}

/**
 * FDummyViewport
 */

FDummyViewport::FDummyViewport()
	: DebugCanvas(nullptr)
{
}

TUniquePtr<FDummyViewport> FDummyViewport::Create(const TStrongPtrVariant<FViewportClient>& InClient)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TUniquePtr<FDummyViewport> Self = Create(InClient.Get());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Self->Client = InClient;
	return Self;
}

TUniquePtr<FDummyViewport> FDummyViewport::Create(FViewportClient* InClient)
{
	TUniquePtr<FDummyViewport> Self(new FDummyViewport());

	UWorld* World = InClient ? InClient->GetWorld() : nullptr;
	Self->DebugCanvas = new FCanvas(Self.Get(), nullptr, World, World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel);
	Self->DebugCanvas->SetAllowedModes(0);

	// Skip invocation of Add/Remove Association for screenshots
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Self->ViewportClient = InClient;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return Self;
}

FDummyViewport::~FDummyViewport()
{
	if (DebugCanvas != NULL)
	{
		delete DebugCanvas;
		DebugCanvas = NULL;
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ViewportClient = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FDummyViewport::InitRHI(FRHICommandListBase& RHICmdList)
{
	EPixelFormat DummyViewportFormat = bSceneHDREnabled ? GRHIHDRDisplayOutputFormat : PF_A2B10G10R10;
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("FDummyViewport"))
		.SetExtent(SizeX, SizeY)
		.SetFormat(DummyViewportFormat)
		.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask);

	RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);
}

EDisplayColorGamut FDummyViewport::GetDisplayColorGamut() const
{
	return DisplayColorGamut;
}

EDisplayOutputFormat FDummyViewport::GetDisplayOutputFormat() const
{
	return DisplayOutputFormat;
}

bool FDummyViewport::GetSceneHDREnabled() const
{
	return bSceneHDREnabled;
}
