// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
UnrealEngine.cpp: Implements the UEngine class and helpers.
=============================================================================*/

#include "UnrealEngine.h"
#include "Algo/Find.h"
#include "Engine/Blueprint.h"
#include "Async/AsyncFileHandle.h"
#include "AutoRTFM/Defines.h"
#include "Engine/GameInstance.h"
#include "Engine/LevelStreaming.h"
#include "EngineStats.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "AI/NavigationSystemBase.h"
#include "AI/NavigationSystemConfig.h"
#include "Engine/NetConnection.h"
#include "Misc/MessageDialog.h"
#include "Engine/PendingNetGame.h"
#include "Misc/FileHelper.h"
#include "EngineGlobals.h"
#include "Containers/Ticker.h"
#include "Misc/FrameValue.h"
#include "FramePro/FramePro.h"
#include "Misc/BufferedOutputDevice.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "HAL/PlatformFile.h"
#include "Misc/OutputDeviceFile.h"
#include "Math/ColorList.h"
#include "StateStreamManager.h"
#include "Stats/StatsData.h"
#include "Stats/StatsFile.h"
#include "Stats/StatsMisc.h"
#include "Stats/ThreadIdleStats.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreDelegates.h"
#include "Misc/StringFormatArg.h"
#include "Misc/TimeGuard.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "UObject/UObjectIterator.h"
#include "ParticleEmitterInstances.h"
#include "UObject/CoreRedirects.h"
#include "UObject/MetaData.h"
#include "Particles/ParticleEmitter.h"
#include "UObject/ObjectMemoryAnalyzer.h"
#include "Particles/TypeData/ParticleModuleTypeDataBase.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "PhysicsEngine/ConvexElem.h"
#include "Serialization/ArchiveCountMem.h"
#include "RenderAssetUpdate.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Misc/PackageName.h"
#include "TextureResource.h"
#include "UObject/LinkerLoad.h"
#include "GameMapsSettings.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/WildcardString.h"
#include "Misc/OutputDeviceConsole.h"
#include "Serialization/ArchiveReplaceOrClearExternalReferences.h"
#include "Materials/Material.h"
#include "GameFramework/GameModeBase.h"
#include "Particles/ParticleSystem.h"
#include "Performance/LatencyMarkerModule.h"
#include "Performance/MaxTickRateHandlerModule.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Components/SkeletalMeshComponent.h"
#include "EngineFontServices.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/VolumeTexture.h"
#include "Particles/ParticleModule.h"
#include "Particles/ParticleSystemComponent.h"
#include "Exporters/Exporter.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/LocalPlayer.h"
#include "Engine/StaticMesh.h"
#include "DistanceFieldAtlas.h"
#include "SystemSettings.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "UObject/GarbageCollectionHistory.h"
#include "UObject/ReachabilityAnalysis.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWindow.h"
#include "Engine/EngineCustomTimeStep.h"
#include "Engine/LevelStreamingPersistent.h"
#include "Engine/ObjectReferencer.h"
#include "Engine/TextureLODSettings.h"
#include "Engine/TimecodeProvider.h"
#include "Engine/SystemTimeTimecodeProvider.h"
#include "Net/OnlineEngineInterface.h"
#include "Engine/Console.h"
#include "UObject/Stack.h"
#include "VisualLogger/VisualLogger.h"
#include "SkeletalMeshMerge.h"
#include "ShaderCompiler.h"
#include "Slate/SlateSoundDevice.h"
#include "DerivedDataCacheInterface.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "EngineAnalytics.h"
#include "Elements/Framework/EngineElements.h"
#include "TickTaskManagerInterface.h"
#include "Net/NetworkProfiler.h"
#include "IHeadMountedDisplayModule.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "IXRLoadingScreen.h"
#include "Audio/AudioDebug.h"
#include "AudioDevice.h"
#include "Animation/SkeletalMeshActor.h"
#include "Animation/AnimSequence.h"
#include "Engine/Canvas.h"
#include "GameFramework/HUD.h"
#include "GameFramework/Character.h"
#include "GameDelegates.h"
#include "MoviePlayerProxy.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/LevelStreamingVolume.h"
#include "Engine/LevelScriptActor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "DynamicResolutionState.h"
#include "Engine/ViewportStatsSubsystem.h"
#include "GPUSkinCache.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "HAL/PlatformMemoryHelpers.h"
#include "RenderCore.h"
#include "Misc/DelayedAutoRegister.h"
#include <exception>
#include "AssetCompilingManager.h"
#include "UObject/UObjectSerializeContext.h"
#include "VerseVM/VVMDuplication.h"
#include "PSOPrecacheSettings.h"
#include "Components/ActorComponent.h"

#include "Iris/IrisConfig.h"
#include "Iris/ReplicationSystem/NetObjectFactoryRegistry.h"

#include "IUniversalObjectLocatorModule.h"

#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"

#include "Components/TextRenderComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "IO/IoDispatcher.h"

#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "UniversalObjectLocators/AssetLocatorFragment.h"
#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"

#if WITH_EDITOR
#include "PackageTools.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "LandscapeSubsystem.h"
#include "TextureCompiler.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Framework/Notifications/NotificationManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "Reinstancing/ReinstancingInternal.h"
#include "UObject/InstanceDataObjectUtils.h"
#endif
// @todo this is here only due to circular dependency to AIModule. To be removed

#if WITH_EDITORONLY_DATA
#include "ObjectEditorUtils.h"
#endif

#if FRAMEPRO_ENABLED
#include "FramePro/FrameProProfiler.h"
#endif

#include "HardwareInfo.h"
#include "EngineModule.h"
#include "UnrealExporter.h"
#include "BufferVisualizationData.h"
#include "NaniteVisualizationData.h"
#include "VirtualShadowMapVisualizationData.h"
#include "RayTracingVisualizationData.h"
#include "VT/VirtualTextureVisualizationData.h"
#include "Misc/HotReloadInterface.h"
#include "Widgets/Testing/STestSuite.h"
#include "Engine/DemoNetDriver.h"
#include "Widgets/Images/SThrobber.h"
#include "Engine/TextureCube.h"
#include "AI/Navigation/AvoidanceManager.h"
#include "Engine/GameEngine.h"
#include "PhysicsEngine/PhysicsCollisionHandler.h"
#include "Components/BrushComponent.h"
#include "GameFramework/GameUserSettings.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/UserInterfaceSettings.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Materials/MaterialRenderProxy.h"
#include "Misc/TransactionCommon.h"

#include "IMessageRpcClient.h"
#include "IMessagingRpcModule.h"
#include "IPortalRpcModule.h"
#include "IPortalRpcLocator.h"
#include "IPortalServicesModule.h"
#include "IPortalServiceLocator.h"
#include "Misc/TypeContainer.h"

#include "IMovieSceneCapture.h"
#include "MovieSceneCaptureModule.h"
#include "GameFramework/OnlineSession.h"
#include "ProfilingDebugging/ABTesting.h"
#include "Performance/EnginePerformanceTargets.h"

#include "InstancedReferenceSubobjectHelper.h"
#include "Misc/EngineBuildSettings.h"
#include "Engine/LODActor.h"
#include "Engine/AssetManager.h"
#include "GameplayTagsManager.h"
#include "PrimitiveDrawingUtils.h"

#if !UE_BUILD_SHIPPING
#include "IAutomationWorkerModule.h"
#endif	// UE_BUILD_SHIPPING

#if !(UE_BUILD_SHIPPING) || ENABLE_PGO_PROFILE
#include "ParseExecCommands.h"
#endif

#if ENABLE_LOC_TESTING
#include "LocalizationModule.h"
#endif

#include "GeneralProjectSettings.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/CoreSettings.h"
#include "IEyeTrackerModule.h"
#include "Interfaces/IPluginManager.h"
#include "Animation/SkeletonRemappingRegistry.h"

#if !UE_BUILD_SHIPPING
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Streaming/StreamingManagerTexture.h"
#endif

#include "Net/Iris/ReplicationSystem/NetEngineFactories.h"

#include "HAL/FileManagerGeneric.h"
#include "UObject/ReferenceChainSearch.h"

#include "Particles/ParticleSystemManager.h"
#include "ObjectTrace.h"
#include "ObjectPropertyTrace.h"
#include "Animation/SkinWeightProfileManager.h"

#include "Particles/ParticlePerfStatsManager.h"

#include "IDeviceProfileSelectorModule.h"
#include "HDRHelper.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/OverridableManager.h"

// MMV support in FakeStereoRenderingDevice
#include "StereoRenderTargetManager.h"
#include "ScreenRendering.h"

#if WITH_DUMPGPU
#include "RenderGraphBuilder.h"
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"
#endif

#if WITH_DUMPGPU

static TAutoConsoleVariable<int32> GDumpGPUScreenshot(
	TEXT("r.DumpGPU.Screenshot"), 1,
	TEXT("Whether to take a final screenshot."),
	ECVF_Default);

#endif

DEFINE_LOG_CATEGORY(LogEngine);
IMPLEMENT_MODULE( FEngineModule, Engine );

#define LOCTEXT_NAMESPACE "UnrealEngine"

void InitScreenPercentage();
void InitCustomDepth();

namespace UE::Transaction
{
// This method adds UActorComponent objects to the outer object scan in
// FSerializedObjectDataWriter and FDiffableObjectDataWriter.  This was the 
// only reference to Engine types used by the mentioned above classes.
bool IsActorComponentCreatedByConstructionScript(const UObject* Obj)
{
	const UActorComponent* AsComponent = Cast<UActorComponent>(Obj);
	return AsComponent && AsComponent->IsCreatedByConstructionScript();
}
} // namespace UE::Transaction

void OnChangeEngineCVarRequiringRecreateRenderState(IConsoleVariable* Var)
{
	// Propgate cvar change to static draw lists
	FGlobalComponentRecreateRenderStateContext Context;
}

void FEngineModule::StartupModule()
{
	using namespace UE::UniversalObjectLocator;
	using namespace UE::Transaction;
	
	FPersistentObjectRef::GetOnUseOuterDelegates().Add(&IsActorComponentCreatedByConstructionScript);

	// Setup delegate callback for ProfilingHelpers to access current map name
	extern const FString GetMapNameStatic();
	GGetMapNameDelegate.BindStatic(&GetMapNameStatic);

	static IConsoleVariable* CacheWPOPrimitivesVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.CacheWPOPrimitives"));
	CacheWPOPrimitivesVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeEngineCVarRequiringRecreateRenderState));

	static auto CVARShowMaterialDrawEvents = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ShowMaterialDrawEvents"));
	if (CVARShowMaterialDrawEvents)
	{
		CVARShowMaterialDrawEvents->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeEngineCVarRequiringRecreateRenderState));
	}

	SuspendTextureStreamingRenderTasks = &SuspendRenderAssetStreaming;
	ResumeTextureStreamingRenderTasks = &ResumeRenderAssetStreaming;

	InitScreenPercentage();
	InitCustomDepth();

	FParticleSystemWorldManager::OnStartup();

#if WITH_EDITOR
	USkinnedMeshComponent::BindWorldDelegates();
#endif

	FInstancedStruct::NetSerializeScriptStructDelegate.BindLambda([](FInstancedStruct& InstancedStruct, FArchive& Ar, UPackageMap* Map)
	{
		UPackageMapClient* MapClient = Cast<UPackageMapClient>(Map);
		check(::IsValid(MapClient));

		UNetConnection* NetConnection = MapClient->GetConnection();
		check(::IsValid(NetConnection));
		check(::IsValid(NetConnection->GetDriver()));

		UStruct* NonConstStruct = const_cast<UScriptStruct*>(InstancedStruct.GetScriptStruct());
		const TSharedPtr<FRepLayout> RepLayout = NetConnection->GetDriver()->GetStructRepLayout(NonConstStruct);
		check(RepLayout.IsValid());

		bool bHasUnmapped = false;
		RepLayout->SerializePropertiesForStruct(NonConstStruct, static_cast<FBitArchive&>(Ar), Map, InstancedStruct.GetMutableMemory(), bHasUnmapped);
		return true;
	});

	IPrimitiveComponent::AddImplementer(TComponentInterfaceImplementation<IPrimitiveComponent>{UPrimitiveComponent::StaticClass(),  [](UObject* Obj){return Cast<UPrimitiveComponent>(Obj)->GetPrimitiveComponentInterface();}});
	IStaticMeshComponent::AddImplementer(TComponentInterfaceImplementation<IStaticMeshComponent>{UStaticMeshComponent::StaticClass(), [](UObject* Obj){return Cast<UStaticMeshComponent>(Obj)->GetStaticMeshComponentInterface();}});
	ISkinnedMeshComponent::AddImplementer(TComponentInterfaceImplementation<ISkinnedMeshComponent>{USkinnedMeshComponent::StaticClass(), [](UObject* Obj) {return Cast<USkinnedMeshComponent>(Obj)->GetSkinnedMeshComponentInterface();}});

	IPrimitiveComponent::AddProvider({AActor::StaticClass(),  [](UObject* Obj) -> IPrimitiveComponent*
	{
		if (AActor* Actor = Cast<AActor>(Obj))
		{
			if (UPrimitiveComponent* Primitive = Actor->GetComponentByClass<UPrimitiveComponent>())
			{
				return Primitive->GetPrimitiveComponentInterface();
			}
		}
		return nullptr;
	}});

#if OBJECT_TRACE_ENABLED
	FObjectTrace::Init();
#endif

#if OBJECT_PROPERTY_TRACE_ENABLED
	FObjectPropertyTrace::Init();
#endif

#if TRACE_FILTERING_ENABLED
	FTraceFilter::Init();
#endif

	FSkinWeightProfileManager::OnStartup();

	UE::Anim::FSkeletonRemappingRegistry::Init();

	FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady,
	[]
	{
		UE::Net::InitEngineNetObjectFactories();
	});

	IUniversalObjectLocatorModule& UolModule = FModuleManager::Get().LoadModuleChecked<IUniversalObjectLocatorModule>("UniversalObjectLocator");

	FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady,
		[&UolModule]
		{
			// Initialize built-in UOL fragment types as soon as the object system is ready
			{
				FFragmentTypeParameters FragmentTypeParams("actor", NSLOCTEXT("Engine", "ActorLocatorFragment", "Actor"));
				FragmentTypeParams.PrimaryEditorType = "Actor";
				FActorLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FActorLocatorFragment>(FragmentTypeParams);
			}
			{
				FFragmentTypeParameters FragmentTypeParams("asset", NSLOCTEXT("Engine", "AssetLocatorFragment", "Asset"));
				FragmentTypeParams.PrimaryEditorType = "Asset";
				FragmentTypeParams.Flags = EFragmentTypeFlags::CanBeLoaded;
				FAssetLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAssetLocatorFragment>(FragmentTypeParams);
			}
			{
				FFragmentTypeParameters FragmentTypeParams("animinst", NSLOCTEXT("Engine", "AnimInstanceLocatorFragment", "AnimInstance"));
				FragmentTypeParams.PrimaryEditorType = "AnimInstance";
				FAnimInstanceLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimInstanceLocatorFragment>(FragmentTypeParams);
			}
			{
				FActorLocatorFragmentResolveParameter::ParameterType = UolModule.RegisterParameterType<FActorLocatorFragmentResolveParameter>();
			}
		}
	);
}

void FEngineModule::ShutdownModule()
{
	using namespace UE::Transaction;

	IPrimitiveComponent::RemoveImplementer(UPrimitiveComponent::StaticClass());
	IStaticMeshComponent::RemoveImplementer(UStaticMeshComponent::StaticClass());
	ISkinnedMeshComponent::RemoveImplementer(USkinnedMeshComponent::StaticClass());
	IPrimitiveComponent::RemoveProvider(AActor::StaticClass());

	UE::Net::ShutdownEngineNetObjectFactories();

#if TRACE_FILTERING_ENABLED
	FTraceFilter::Destroy();
#endif

#if OBJECT_TRACE_ENABLED
	FObjectTrace::Destroy();
#endif
#if OBJECT_PROPERTY_TRACE_ENABLED
	FObjectPropertyTrace::Destroy();
#endif

	FParticleSystemWorldManager::OnShutdown();

	FSkinWeightProfileManager::OnShutdown();

	UE::Anim::FSkeletonRemappingRegistry::Destroy();

	FPersistentObjectRef::GetOnUseOuterDelegates().Remove(&IsActorComponentCreatedByConstructionScript);
}

/* Global variables
*****************************************************************************/

/**
* Global engine pointer. Can be 0 so don't use without checking.
*/
ENGINE_API UEngine*	GEngine = NULL;

int32 GShowMaterialDrawEvents = 0;

#if WANTS_DRAW_MESH_EVENTS
/**
* true if we debug material names with SCOPED_DRAW_EVENT.	 
*/
static FAutoConsoleVariableRef CVARShowMaterialDrawEvents(
	TEXT("r.ShowMaterialDrawEvents"),
	GShowMaterialDrawEvents,
	TEXT("Whether to emit a draw event around every mesh draw call with information about the assets used.  Introduces severe CPU and GPU overhead when enabled, but useful for debugging."),
	ECVF_Default
);
#endif

#if !UE_BUILD_SHIPPING && WITH_DYNAMIC_RESOLUTION
static TAutoConsoleVariable<float> CVarForceDynamicResScreenPercentage(
	TEXT("r.DynamicRes.TestScreenPercentage"),
	0,
	TEXT("Forces the screen percentage to a particular value with dynamic res.\n")
	TEXT(" 0: Disabled (default); > 0: Screen percentage is enabled.\n"),
	ECVF_Default | ECVF_RenderThreadSafe);
#endif

/** System resolution instance */
FSystemResolution GSystemResolution;

ENGINE_API int32 GUnbuiltHLODCount = 0;

static FAutoConsoleVariable CVarSystemResolution(
	TEXT("r.SetRes"),
	TEXT("1280x720w"),
	TEXT("Set the display resolution for the current game view. Has no effect in the editor.\n")
	TEXT("e.g. 1280x720w for windowed\n")
	TEXT("     1920x1080f for fullscreen\n")
	TEXT("     1920x1080wf for windowed fullscreen\n")
);

static FAutoConsoleVariable CVarMonitorIndex(
	TEXT("r.SetRes.MonitorIndex"),
	-1,
	TEXT("Set the monitor to be used by r.SetRes. Has no effect in the editor.\n")
	TEXT("-1 for legacy behavior")
);

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<float> CVarSetOverrideFPS(
	TEXT("t.OverrideFPS"),
	0.0f,
	TEXT("This allows to override the frame time measurement with a fixed fps number (game can run faster or slower).\n")
	TEXT("<=0:off, in frames per second, e.g. 60"),
	ECVF_Cheat);
#endif // !UE_BUILD_SHIPPING

#if WITH_EDITOR
static TAutoConsoleVariable<int> CVarDynamicResAllowInPIE(
	TEXT("r.DynamicRes.AllowInPIE"),
	1,
	TEXT("When Dynamic Resolution is enabled for the project on PC: select if Dynamic Resolution is allowed during PIE in Editor\n")
	TEXT(" 0: Not Allowed;\n")
	TEXT(" 1: Allowed (default);\n"),
	ECVF_RenderThreadSafe | ECVF_Default);
#endif

static TAutoConsoleVariable<int> CVarDynamicResOperationMode(
	TEXT("r.DynamicRes.OperationMode"),
	0,
	TEXT("Select the operation mode for dynamic resolution.\n")
	TEXT(" 0: Disabled (default);\n")
	TEXT(" 1: Enable according to the game user settings;\n")
	TEXT(" 2: Enable regardless of the game user settings."),
	ECVF_RenderThreadSafe | ECVF_Default);

// Should we show errors and warnings (when DurationOfErrorsAndWarningsOnHUD is greater than zero), or only errors?
int32 GSuppressWarningsInOnScreenDisplay = 0;
static FAutoConsoleVariableRef GSuppressWarningsInOnScreenDisplayCVar(
	TEXT("Engine.SupressWarningsInOnScreenDisplay"),
	GSuppressWarningsInOnScreenDisplay,
	TEXT("0: Show both errors and warnings on screen, 1: Show only errors on screen (in either case only when DurationOfErrorsAndWarningsOnHUD is greater than zero)"),
	ECVF_Default
);

// Should we TrimMemory in the middle of LoadMap - this can reduce memory spike during startup at the expense of load time 
// if some objects need to be reloaded due to a GC happening partway through
int32 GDelayTrimMemoryDuringMapLoadMode = 0;
static FAutoConsoleVariableRef GDelayTrimMemoryDuringMapLoadModeCVar(
	TEXT("Engine.DelayTrimMemoryDuringMapLoadMode"),
	GDelayTrimMemoryDuringMapLoadMode,
	TEXT("0: TrimMemory during LoadMap as normal\n")
	TEXT("1: Delay TrimMemory until the end of LoadMap (initial boot up)\n")
	TEXT("2: Delay TrimMemory in _every_ LoadMap call"),
	ECVF_Default
);

// Flush Rendering and RHI before GC in trim memory
int32 GSuspendAssetStreamerBeforePerformingFullGC = 0;
static FAutoConsoleVariableRef GSuspendAssetStreamerBeforePerformingFullGCCVar(
	TEXT("Engine.SuspendAssetStreamerBeforePerformingFullGC"),
	GSuspendAssetStreamerBeforePerformingFullGC,
	TEXT("Suspend asset streamer before performing a full GC (default: 0)"),
	ECVF_Default
);

static int32 GVerifyLoadMapWorldCleanup_Severity = 2;
static FAutoConsoleVariableRef CVarVerifyLoadMapWorldCleanup_Severity(
#if UE_BUILD_SHIPPING
	TEXT("Engine.VerifyLoadMapWorldCleanup.Severity.Shipping"),
#else
	TEXT("Engine.VerifyLoadMapWorldCleanup.Severity"),
#endif
	GVerifyLoadMapWorldCleanup_Severity,
	TEXT("Controls severity of logging when the engine detects that a UWorld was leaked during LoadMap.\n")
	TEXT("0 - all reference tracing and logging is disabled\n")
	TEXT("1 - logs an error\n")
	TEXT("2 - ensure\n")
	TEXT("3 - fatal error\n"),
	ECVF_Default
);

static int32 GVerifyLoadMapWorldCleanup_TraceMode = (UE_BUILD_SHIPPING ? 0 : 1);
static FAutoConsoleVariableRef CVarVerifyLoadMapWorldCleanup_TraceMode(
#if UE_BUILD_SHIPPING
	TEXT("Engine.VerifyLoadMapWorldCleanup.TraceMode.Shipping"),
#else
	TEXT("Engine.VerifyLoadMapWorldCleanup.TraceMode"),
#endif
	GVerifyLoadMapWorldCleanup_TraceMode,
	TEXT("Controls detail level of reference tracing when the engine detects that a world was leaked during LoadMap.\n")
	TEXT("0 - direct references only\n")
	TEXT("1 - full reference trace"),
	ECVF_Default
);

#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<bool> CVarCsvAlwaysShowFrameCount(
	TEXT("csv.AlwaysShowFrameCount"),
	false,
	TEXT("If enabled, we show the frame count in non-shipping builds, even if screen messages are disabled")
);

TAutoConsoleVariable<bool> CVarAllowExtraDevMemoryMessageDisplayed(
	TEXT("Debug.AllowExtraDevMemoryMessageDisplayed"),
	true,
	TEXT("If true, we allow the extra memory detection message to be shown in non-shipping builds")
);
#endif


#if !UE_BUILD_SHIPPING
class FDisplayCVarListExecHelper : public FSelfRegisteringExec
{
	virtual bool Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		if (FParse::Command(&Cmd, TEXT("DisplayCVarList")))
		{
			ConsoleVariablesToVisualize.Empty();

			FString CVarList(Cmd);
			if (!CVarList.IsEmpty())
			{
				TArray<FString> CVars;
				CVarList.ParseIntoArray(CVars, TEXT(","));

				for (FString& CVarString : CVars)
				{
					CVarString.TrimStartAndEndInline();

					IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(
						[this](const TCHAR* Key, IConsoleObject* ConsoleObject)
						{
							if (IConsoleVariable* AsVariable = ConsoleObject->AsVariable())
							{
								ConsoleVariablesToVisualize.Add(FString(Key), AsVariable);
							}

						}),
						*CVarString);
				}
			}

			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("LogCVarList")))
		{
			FString CVarList(Cmd);
			if (!CVarList.IsEmpty())
			{
				TArray<FString> CVars;
				CVarList.ParseIntoArray(CVars, TEXT(","));

				Ar.Logf(TEXT("Name, Value"));

				for (FString& CVarString : CVars)
				{
					CVarString.TrimStartAndEndInline();

					IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(
						[this, &Ar](const TCHAR* Key, IConsoleObject* ConsoleObject)
						{
							if (IConsoleVariable* AsVariable = ConsoleObject->AsVariable())
							{
								Ar.Logf(TEXT("%s, %s"), Key, *AsVariable->GetString());
							}
						}),
						*CVarString);
				}
			}

			return true;
		}
		return false;
	}

public:
	TMap<FString, IConsoleVariable*> ConsoleVariablesToVisualize;
};
static FDisplayCVarListExecHelper GDisplayCVarListExecHelper;
#endif // !UE_BUILD_SHIPPING

/** Whether texture memory has been corrupted because we ran out of memory in the pool. */
bool GIsTextureMemoryCorrupted = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
/** Whether PrepareMapChange is attempting to load a map that doesn't exist */
bool GIsPrepareMapChangeBroken = false;
#endif

// Thread config 
void InitThreadConfig();

#if WITH_EDITOR
// A debugging aid set when we switch out different play worlds during Play In Editor / PIE
ENGINE_API FString GPlayInEditorContextString("invalid");
#endif

ENGINE_API FString GetDebugStringForWorld(const UWorld* World)
{
	FString WorldName;

	if (World == nullptr)
	{
		WorldName = NSLOCTEXT("Engine", "PlayWorldBeingCreated", "(World Being Created)").ToString();
	}
	else if (World->WorldType == EWorldType::Editor)
	{
		WorldName = NSLOCTEXT("Engine", "PlayWorldIsEditor", "Editor").ToString();
	}
	else if (World->IsGameWorld())
	{
		FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);

		switch (World->GetNetMode())
		{
		case NM_Standalone:
			WorldName = NSLOCTEXT("Engine", "PlayWorldIsStandalone", "Standalone").ToString();
			break;

		case NM_ListenServer:
			WorldName = NSLOCTEXT("Engine", "PlayWorldIsListenServer", "Listen Server").ToString();
			break;

		case NM_DedicatedServer:
			WorldName = NSLOCTEXT("Engine", "PlayWorldIsDedicatedServer", "Dedicated Server").ToString();
			break;

		case NM_Client:
			// 0 is always the server, use PIEInstance so it matches the in-editor UI
			WorldName = FText::Format(NSLOCTEXT("Engine", "PlayWorldIsClient", "Client {0}"), FText::AsNumber(WorldContext ? WorldContext->PIEInstance : 0)).ToString();
			break;

		default:
			unimplemented();
		};

		if ((WorldContext != nullptr) && !WorldContext->CustomDescription.IsEmpty())
		{
			WorldName += TEXT(" ") + WorldContext->CustomDescription;
		}
	}

	return WorldName;
}

ENGINE_API void UpdatePlayInEditorWorldDebugString(const FWorldContext* WorldContext)
{
#if WITH_EDITOR
	if (!GIsEditor)
	{
		return;
	}

	if (WorldContext == nullptr)
	{
		ensure(UE::GetPlayInEditorID() == INDEX_NONE);
		GPlayInEditorContextString = NSLOCTEXT("Engine", "PlayWorldIsNotActive", "Not in a play world").ToString();
	}
	else
	{
		GPlayInEditorContextString = GetDebugStringForWorld(WorldContext->World());
	}
#endif
}

FTemporaryPlayInEditorIDOverride::FTemporaryPlayInEditorIDOverride(int32 NewOverrideID)
	: PreviousID(UE::GetPlayInEditorID())
{
	SetID(NewOverrideID);
}

FTemporaryPlayInEditorIDOverride::~FTemporaryPlayInEditorIDOverride()
{
	SetID(PreviousID);
}

void FTemporaryPlayInEditorIDOverride::SetID(int32 NewID)
{
	if (UE::GetPlayInEditorID() != NewID)
	{
		UE::SetPlayInEditorID(NewID);
		UpdatePlayInEditorWorldDebugString(GEngine->GetWorldContextFromPIEInstance(NewID));
	}
}

// We expose these variables to everyone as we need to access them in other files via an extern
ENGINE_API float GAverageFPS = 0.0f;
ENGINE_API float GAverageMS = 0.0f;
ENGINE_API double GLastMemoryWarningTime = 0.f;
ENGINE_API bool GIsLowMemory = false;


void CalculateFPSTimings()
{
	// Calculate the average frame time via continued averaging.
	static double LastTime = 0.0;
	double CurrentTime = FPlatformTime::Seconds();
	float FrameTimeMS = (float)((CurrentTime - LastTime) * 1000.0);

	static auto CVarVsync      = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
	static auto CVarGTSyncType = IConsoleManager::Get().FindConsoleVariable(TEXT("r.GTSyncType"));
	
	if (CVarGTSyncType->GetInt() == 2 && CVarVsync->GetInt() != 0)
	{
		float RHIFrameTime = RHIGetFrameTime();
		if (RHIFrameTime != 0)
		{
			FrameTimeMS = RHIFrameTime;
		}
	}
	
	// Exponential moving average with a 0.1 smoothing factor.
	GAverageMS = GAverageMS * 0.9f + FrameTimeMS * 0.1f;

	LastTime = CurrentTime;

	// Calculate average framerate.
	GAverageFPS = 1000.f / GAverageMS;
}

static FCachedSystemScalabilityCVars GCachedScalabilityCVars;

const FCachedSystemScalabilityCVars& GetCachedScalabilityCVars()
{
	check(GCachedScalabilityCVars.bInitialized);
	return GCachedScalabilityCVars;
}

FCachedSystemScalabilityCVars::FCachedSystemScalabilityCVars()
	: bInitialized(false)
	, DetailMode(-1)
	, CookOutUnusedDetailModeComponents(0)
	, MaterialQualityLevel(EMaterialQualityLevel::Num)
	, MaxShadowResolution(-1)
	, MaxCSMShadowResolution(-1)
	, ViewDistanceScale(-1)
	, ViewDistanceScaleSquared(-1)
	, StaticMeshLODDistanceScale(1.0f)
	, SkeletalMeshOverlayDistanceScale(1.0f)
	, MaxAnisotropy(-1)
	
{

}

bool FCachedSystemScalabilityCVars::operator==(const FCachedSystemScalabilityCVars& Other) const
{
	return DetailMode == Other.DetailMode &&
		MaterialQualityLevel == Other.MaterialQualityLevel &&
		MaxShadowResolution == Other.MaxShadowResolution &&
		MaxCSMShadowResolution == Other.MaxCSMShadowResolution &&
		ViewDistanceScale == Other.ViewDistanceScale &&
		ViewDistanceScaleSquared == Other.ViewDistanceScaleSquared &&
		StaticMeshLODDistanceScale == Other.StaticMeshLODDistanceScale &&
		SkeletalMeshOverlayDistanceScale == Other.SkeletalMeshOverlayDistanceScale;
}

void ScalabilityCVarsSinkCallback()
{
	IConsoleManager& ConsoleMan = IConsoleManager::Get();

	FCachedSystemScalabilityCVars LocalScalabilityCVars = GCachedScalabilityCVars;

	{
		static const auto CookOutUnusedDetailModeComponents = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.CookOutUnusedDetailModeComponents"));
		LocalScalabilityCVars.CookOutUnusedDetailModeComponents = CookOutUnusedDetailModeComponents->GetValueOnGameThread();
	}

	{
		static const auto DetailMode = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.DetailMode"));
		LocalScalabilityCVars.DetailMode = DetailMode->GetValueOnGameThread();
	}

	{
		static const auto* MaxAnisotropy = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.MaxAnisotropy"));
		LocalScalabilityCVars.MaxAnisotropy = MaxAnisotropy->GetValueOnGameThread();
	}

	{
		static const auto* MaxShadowResolution = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.Shadow.MaxResolution"));
		LocalScalabilityCVars.MaxShadowResolution = MaxShadowResolution->GetValueOnGameThread();
	}

	{
		static const auto* MaxCSMShadowResolution = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.Shadow.MaxCSMResolution"));
		LocalScalabilityCVars.MaxCSMShadowResolution = MaxCSMShadowResolution->GetValueOnGameThread();
	}

	{
		static const auto ViewDistanceScale = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale"));
		static const auto ApplySecondaryViewDistanceScale = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.ViewDistanceScale.ApplySecondaryScale"));
		static const auto SecondaryViewDistanceScale = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.SecondaryScale"));
		bool bApplySecondary = (ApplySecondaryViewDistanceScale->GetValueOnGameThread() > 0);
		LocalScalabilityCVars.ViewDistanceScale = FMath::Max(ViewDistanceScale->GetValueOnGameThread(), 0.0f);
		LocalScalabilityCVars.ViewDistanceScale *= bApplySecondary ? FMath::Max(SecondaryViewDistanceScale->GetValueOnGameThread(), 0.0f) : 1.0f;
		LocalScalabilityCVars.ViewDistanceScaleSquared = FMath::Square(LocalScalabilityCVars.ViewDistanceScale);
	}

	{
		static const auto FieldOfViewMinAngle = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.FieldOfViewMinAngle"));
		static const auto FieldOfViewMinAngleScale = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.FieldOfViewMinAngleScale"));
		static const auto FieldOfViewMaxAngle = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.FieldOfViewMaxAngle"));
		static const auto FieldOfViewMaxAngleScale = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.FieldOfViewMaxAngleScale"));
		static const auto FieldOfViewAffectsHLOD = ConsoleMan.FindTConsoleVariableDataInt(TEXT("r.ViewDistanceScale.FieldOfViewAffectsHLOD"));
		LocalScalabilityCVars.FieldOfViewMinAngle = FMath::Clamp(FieldOfViewMinAngle->GetValueOnGameThread(), 0.0f, 360.f);
		LocalScalabilityCVars.FieldOfViewMinAngleScale = FMath::Max(FieldOfViewMinAngleScale->GetValueOnGameThread(), 0.0f);
		LocalScalabilityCVars.FieldOfViewMaxAngle = FMath::Clamp(FieldOfViewMaxAngle->GetValueOnGameThread(), 0.0f, 360.f);
		LocalScalabilityCVars.FieldOfViewMaxAngleScale = FMath::Max(FieldOfViewMaxAngleScale->GetValueOnGameThread(), 0.0f);
		LocalScalabilityCVars.FieldOfViewAffectsHLOD = FieldOfViewAffectsHLOD->GetValueOnGameThread() != 0 ? 1 : 0;
	}

	{
		static const auto MaterialQualityLevelVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaterialQualityLevel"));
		LocalScalabilityCVars.MaterialQualityLevel = (EMaterialQualityLevel::Type)FMath::Clamp(MaterialQualityLevelVar->GetValueOnGameThread(), 0, (int32)EMaterialQualityLevel::Num - 1);
	}

	{
		static const auto StaticMeshLODDistanceScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.StaticMeshLODDistanceScale"));
		LocalScalabilityCVars.StaticMeshLODDistanceScale = StaticMeshLODDistanceScale->GetFloat();
	}

	{
		static const auto SkeletalMeshOverlayDistanceScale = ConsoleMan.FindTConsoleVariableDataFloat(TEXT("r.ViewDistanceScale.SkeletalMeshOverlay"));
		LocalScalabilityCVars.SkeletalMeshOverlayDistanceScale = FMath::Max(SkeletalMeshOverlayDistanceScale->GetValueOnGameThread(), 0.0f);
		LocalScalabilityCVars.SkeletalMeshOverlayDistanceScale *= LocalScalabilityCVars.ViewDistanceScale;
	}

	LocalScalabilityCVars.bInitialized = true;

	if ((LocalScalabilityCVars == GCachedScalabilityCVars) && GCachedScalabilityCVars.bInitialized)
	{
		return;
	}

	// Need to flush any in-flight uniform expression cache operations, when material quality changes.  The material quality level affects
	// the result returned by calls to FMaterialRenderProxy::GetMaterialNoFallback, and in-flight operations will have been using a uniform
	// buffer layout from the old version of the material, leading to an assert.
	ENQUEUE_RENDER_COMMAND(UpdateDeferredCachedUniformExpressions)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();
		});

	FlushRenderingCommands();

	if (!GCachedScalabilityCVars.bInitialized)
	{
		// optimization: the first time we assume the render thread wasn't started and we don't need to destroy proxies
		GCachedScalabilityCVars = LocalScalabilityCVars;
	}
	else
	{
		bool bRecreateRenderstate = false;
		bool bCacheResourceShaders = false;
		// Don't cache _all_ shaders explicitly and let On Demand Shader Compilation (ODSC) handle compiling the shaders visible.
		const bool bCacheAllRemainingShaders = false;

		if (LocalScalabilityCVars.DetailMode != GCachedScalabilityCVars.DetailMode)
		{
			bRecreateRenderstate = true;
		}

		if (LocalScalabilityCVars.MaterialQualityLevel != GCachedScalabilityCVars.MaterialQualityLevel)
		{
			// If the material quality level changes we need to potentially create new FMaterialRenderResources
			// for the new quality level.
			bCacheResourceShaders = true;
		}

		if (bRecreateRenderstate || bCacheResourceShaders)
		{
			// Make the render state rebuild object before updating the cached values, because its constructor calls UpdateAllPrimitiveSceneInfos(), which
			// may end up using the material shader maps for the quality level stored in GCachedScalabilityCVars.MaterialQualityLevel. If that value already
			// points to the new quality level, the shader maps won't exist, and we'll crash.
			FGlobalComponentRecreateRenderStateContext Recreate;

			// after FlushRenderingCommands() to not have render thread pick up the data partially
			GCachedScalabilityCVars = LocalScalabilityCVars;

			if (bCacheResourceShaders)
			{
				// For all materials, UMaterial::CacheResourceShadersForRendering
				UMaterial::AllMaterialsCacheResourceShadersForRendering(false, bCacheAllRemainingShaders);
				UMaterialInstance::AllMaterialsCacheResourceShadersForRendering(true, bCacheAllRemainingShaders);
			}
		}
		else
		{
			GCachedScalabilityCVars = LocalScalabilityCVars;
		}
	}
}

bool ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY, int32& OutWindowMode)
{
	if(*InResolution)
	{
		FString CmdString(InResolution);
		CmdString = CmdString.TrimStartAndEnd().ToLower();

		//Retrieve the X dimensional value
		const uint32 X = FMath::Max(FCString::Atof(*CmdString), 0.0f);

		// Determine whether the user has entered a resolution and extract the Y dimension.
		FString YString;

		// Find separator between values (Example of expected format: 1280x768)
		const TCHAR* YValue = NULL;
		if(FCString::Strchr(*CmdString,'x'))
		{
			YValue = const_cast<TCHAR*> (FCString::Strchr(*CmdString,'x')+1);
			YString = YValue;
			// Remove any whitespace from the end of the string
			YString = YString.TrimStartAndEnd();
		}

		// If the Y dimensional value exists then setup to use the specified resolution.
		uint32 Y = 0;
		if ( YValue && YString.Len() > 0 )
		{
			// See if there is a fullscreen flag on the end
			FString FullScreenChar = YString.Mid(YString.Len() - 1);
			FString WindowFullScreenChars = YString.Mid(YString.Len() - 2);
			int32 WindowMode = OutWindowMode;
			if (!FullScreenChar.IsNumeric())
			{
				int StringTripLen = 0;

				if (WindowFullScreenChars == TEXT("wf"))
				{
					WindowMode = EWindowMode::WindowedFullscreen;
					StringTripLen = 2;
				}
				else if (FullScreenChar == TEXT("f"))
				{
					WindowMode = EWindowMode::Fullscreen;
					StringTripLen = 1;
				}
				else if (FullScreenChar == TEXT("w"))
				{
					WindowMode = EWindowMode::Windowed;
					StringTripLen = 1;
				}

				YString.LeftInline(YString.Len() - StringTripLen, EAllowShrinking::No);
				YString.TrimStartAndEndInline();
			}

			if (YString.IsNumeric())
			{
				Y = FMath::Max(FCString::Atof(YValue), 0.0f);
				OutX = X;
				OutY = Y;
				OutWindowMode = WindowMode;
				return true;
			}
		}
	}
	return false;
}

void ENGINE_API HDRSettingChangedSinkCallback()
{
	static const auto CVarHDROutputEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.EnableHDROutput"));
	check(CVarHDROutputEnabled);
	static const auto CVarHDRMinLuminanceLog10 = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.HDR.Display.MinLuminanceLog10"));
	static const auto CVarHDRMidLuminance = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.HDR.Display.MidLuminance"));
	static const auto CVarHDRMaxLuminance = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Display.MaxLuminance"));
	static const auto CVarHDRSceneColorMultiplier = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.HDR.Aces.SceneColorMultiplier"));
	static const auto CVarHDRGamutCompression = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.HDR.Aces.GamutCompression"));

	if (GRHIVendorId == 0)
	{
		return;
	}

	static float GHDRMinLuminnanceLog10 = CVarHDRMinLuminanceLog10->GetValueOnAnyThread();
	static float GHDRMidLuminnance = CVarHDRMidLuminance->GetValueOnAnyThread();
	static int32 GHDRMaxLuminnance = CVarHDRMaxLuminance->GetValueOnAnyThread();
	static float GHDRSceneColorMultiplier = CVarHDRSceneColorMultiplier->GetValueOnAnyThread();
	static float GHDRGamutCompression = CVarHDRGamutCompression->GetValueOnAnyThread();
	static int32 GDisplayNitLevel = 1000;

	bool bIsHDREnabled = IsHDREnabled();
	const UGameUserSettings* const GameUserSettings = GEngine ? GEngine->GetGameUserSettings() : nullptr;
	int32 DisplayNitLevel = GameUserSettings ? GameUserSettings->GetCurrentHDRDisplayNits() : 0;

	// We'll naively fall back to 1000 if DisplayNits is 0. This can happen also if HDR is disabled from GameUserSettings
	if (DisplayNitLevel == 0)
	{
		DisplayNitLevel = 1000;
	}

	if (GameUserSettings)
	{
		static IConsoleVariable* const CVarHDRDisplayOverrideOSMaxLuminance = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.Display.OverrideOSMaxLuminance"));
		CVarHDRDisplayOverrideOSMaxLuminance->Set(GameUserSettings->IsHDRCalibrationUsed());

		static IConsoleVariable* const CVarHDRPaperWhite = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.PaperWhite"));
		CVarHDRPaperWhite->Set(GameUserSettings->GetHDRPaperWhiteNits());

		static IConsoleVariable* const CVarHDRUILuminance = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.Luminance"));
		static IConsoleVariable* const CVarHDRUILuminanceMode = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.UI.Luminance.Mode"));
		if (CVarHDRUILuminance && CVarHDRUILuminanceMode)
		{
			CVarHDRUILuminance->Set(GameUserSettings->GetHDRUILuminanceNits());
			CVarHDRUILuminanceMode->Set(static_cast<int32>(GameUserSettings->IsHDRUILuminanceSeparate()));
		}
	}

	if(bIsHDREnabled != GRHIIsHDREnabled || !FMath::IsNearlyEqual(GHDRMinLuminnanceLog10, CVarHDRMinLuminanceLog10->GetValueOnAnyThread())
		|| !FMath::IsNearlyEqual(GHDRMidLuminnance, CVarHDRMidLuminance->GetValueOnAnyThread())
		|| (GHDRMaxLuminnance != CVarHDRMaxLuminance->GetValueOnAnyThread()) || (DisplayNitLevel != GDisplayNitLevel)
		|| !FMath::IsNearlyEqual(GHDRSceneColorMultiplier, CVarHDRSceneColorMultiplier->GetValueOnAnyThread())
		|| !FMath::IsNearlyEqual(GHDRGamutCompression, CVarHDRGamutCompression->GetValueOnAnyThread())
		)
	{
		HDRConfigureCVars(bIsHDREnabled, DisplayNitLevel, false);

		// Now set the HDR setting.
		GRHIIsHDREnabled = bIsHDREnabled;
		GHDRMinLuminnanceLog10 = CVarHDRMinLuminanceLog10->GetValueOnAnyThread();
		GHDRMidLuminnance = CVarHDRMidLuminance->GetValueOnAnyThread();
		GHDRMaxLuminnance = CVarHDRMaxLuminance->GetValueOnAnyThread();
		GHDRSceneColorMultiplier = CVarHDRSceneColorMultiplier->GetValueOnAnyThread();
		GHDRGamutCompression = CVarHDRGamutCompression->GetValueOnAnyThread();
		GDisplayNitLevel = DisplayNitLevel;
	}
}

void SystemResolutionSinkCallback()
{
	// In embedded mode, we want to use the window as it is currently sized,
	// rather than pulling pre-existing sizes from the config.
	if (GUELibraryOverrideSettings.bIsEmbedded)
	{
		GSystemResolution.ResX          = GUELibraryOverrideSettings.WindowWidth;
		GSystemResolution.ResY          = GUELibraryOverrideSettings.WindowHeight;
		GSystemResolution.WindowMode    = EWindowMode::Windowed;
		GSystemResolution.bForceRefresh = false;
		return;
	}

	const FString ResString = CVarSystemResolution->GetString();

	uint32 ResX, ResY;
	int32 WindowModeInt = GSystemResolution.WindowMode;

	if (ParseResolution(*ResString, ResX, ResY, WindowModeInt))
	{
		EWindowMode::Type WindowMode = EWindowMode::ConvertIntToWindowMode(WindowModeInt);
		const int32 MonitorIndex = CVarMonitorIndex->GetInt();

		if( GSystemResolution.ResX != ResX ||
			GSystemResolution.ResY != ResY ||
			GSystemResolution.WindowMode != WindowMode ||
			GSystemResolution.MonitorIndex != MonitorIndex ||
			GSystemResolution.bForceRefresh)
		{
			GSystemResolution.ResX = ResX;
			GSystemResolution.ResY = ResY;
			GSystemResolution.WindowMode = WindowMode;
			GSystemResolution.MonitorIndex = MonitorIndex;
			GSystemResolution.bForceRefresh = false;

			// tell anyone listening about the change
			FCoreDelegates::OnSystemResolutionChanged.Broadcast(ResX, ResY);

			if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Resizing viewport due to setres change, %d x %d, MonitorIndex=%d\n"), ResX, ResY, MonitorIndex);
				GEngine->GameViewport->Viewport->SetMonitorIndexForResize(MonitorIndex);
				GEngine->GameViewport->Viewport->ResizeFrame(ResX, ResY, WindowMode);
			}
		}
	}
}

/**
* if we need to update the sample states
*/
void RefreshSamplerStatesCallback()
{
	if (FApp::CanEverRender() == false)
	{
		// Avoid unnecessary work when running in dedicated server mode.
		return;
	}

	bool bRefreshSamplerStates = false;

	{
		float MipMapBiasOffset = UTexture2D::GetGlobalMipMapLODBias();
		static float LastMipMapLODBias = 0;

		if(LastMipMapLODBias != MipMapBiasOffset)
		{
			LastMipMapLODBias = MipMapBiasOffset;
			bRefreshSamplerStates = true;
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaxAnisotropy"));
		int32 MaxAnisotropy = CVar->GetValueOnGameThread();
		// compare against the default so with that number we avoid RefreshSamplerStates() calls on startup
		// todo: This can be improved since we now have many defaults (see BaseScalability.ini)
		static int32 LastMaxAnisotropy = 4;

		if(LastMaxAnisotropy != MaxAnisotropy)
		{
			LastMaxAnisotropy = MaxAnisotropy;
			bRefreshSamplerStates = true;
		}
	}

	if(bRefreshSamplerStates)
	{
		for (TObjectIterator<UTexture2D>It; It; ++It)
		{
			UTexture2D* Texture = *It;
			Texture->RefreshSamplerStates();
		}

		// The shared sampler states don't have an associated texture so must be manually refreshed
		if (GDynamicRHI && (Wrap_WorldGroupSettings || Clamp_WorldGroupSettings))
		{
			FSharedSamplerState* WrapState = Wrap_WorldGroupSettings;
			FSharedSamplerState* ClampState = Clamp_WorldGroupSettings;
			ENQUEUE_RENDER_COMMAND(RefreshSharedSamplerStatesCommand)(
				[WrapState, ClampState](FRHICommandListImmediate& RHICmdList)
			{
				if (WrapState)
				{
					WrapState->ReleaseRHI();
					WrapState->InitRHI(RHICmdList);
				}

				if (ClampState)
				{
					ClampState->ReleaseRHI();
					ClampState->InitRHI(RHICmdList);
				}
			}
			);
		}

		UMaterialInterface::RecacheAllMaterialUniformExpressions(false);
	}
}

void RefreshEngineSettings()
{
	HDRSettingChangedSinkCallback();
	RefreshSamplerStatesCallback();
	ScalabilityCVarsSinkCallback();
	SystemResolutionSinkCallback();
}

FConsoleVariableSinkHandle GRefreshEngineSettingsSinkHandle;

ENGINE_API void InitializeRenderingCVarsCaching()
{
	GRefreshEngineSettingsSinkHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateStatic(&RefreshEngineSettings));

	// Initialise this to invalid
	GCachedScalabilityCVars.MaterialQualityLevel = EMaterialQualityLevel::Num;

	// Initial cache
	SystemResolutionSinkCallback();
	ScalabilityCVarsSinkCallback();
}

static void ShutdownRenderingCVarsCaching()
{
	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(GRefreshEngineSettingsSinkHandle);
}

static bool HandleDumpShaderPipelineStatsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FString FlagStr(FParse::Token(Cmd, 0));
	EShaderPlatform Platform = GMaxRHIShaderPlatform;
	if (FlagStr.Len() > 0)
	{
		Platform = ShaderFormatToLegacyShaderPlatform(FName(*FlagStr));
	}
	Ar.Logf(TEXT("Dumping shader pipeline stats for platform %s"), *LegacyShaderPlatformToShaderFormat(Platform).ToString());

	DumpShaderPipelineStats(Platform);
	return true;
}

namespace
{
	/**
	* Attempts to set process limits as configured in Engine.ini or elsewhere.
	* Assumed to be called during initialization.
	*/
	void SetConfiguredProcessLimits()
	{
		int32 VirtualMemoryLimitInKB = 0;
		if (GConfig)
		{
			GConfig->GetInt(TEXT("ProcessLimits"), TEXT("VirtualMemoryLimitInKB"), VirtualMemoryLimitInKB, GEngineIni);
		}

		// command line parameters take precendence
		FParse::Value(FCommandLine::Get(), TEXT("virtmemkb="), VirtualMemoryLimitInKB);

		if (VirtualMemoryLimitInKB > 0)
		{
			UE_LOGF(LogInit, Display, "Limiting process virtual memory size to %d KB", VirtualMemoryLimitInKB);
			if (!FPlatformProcess::SetProcessLimits(EProcessResource::VirtualMemory, static_cast< uint64 >(VirtualMemoryLimitInKB) * 1024))
			{
				UE_LOGF(LogInit, Fatal, "Could not limit process virtual memory usage to %d KB", VirtualMemoryLimitInKB);
			}
		}
	}

	UWorld* CreatePIEWorldByLoadingFromPackage(const FWorldContext& WorldContext, const FString& SourceWorldPackage, UPackage*& OutPackage)
	{
		// Load map from the disk in case editor does not have it
		const FString PIEPackageName = UWorld::ConvertToPIEPackageName(SourceWorldPackage, WorldContext.PIEInstance);

		// Set the world type in the static map, so that UWorld::PostLoad can set the world type
		const FName PIEPackageFName = FName(*PIEPackageName);
		UWorld::WorldTypePreLoadMap.FindOrAdd( PIEPackageFName ) = WorldContext.WorldType;
		FSoftObjectPath::AddPIEPackageName(PIEPackageFName);

		uint32 LoadFlags = LOAD_None;
		UPackage* NewPackage = CreatePackage( *PIEPackageName);
		if (NewPackage != nullptr && WorldContext.WorldType == EWorldType::PIE)
		{
			NewPackage->SetPIEInstanceID(WorldContext.PIEInstance);
			NewPackage->SetPackageFlags(PKG_PlayInEditor);
			LoadFlags |= LOAD_PackageForPIE;
		}
		OutPackage = LoadPackage(NewPackage, *SourceWorldPackage, LoadFlags);

		// Clean up the world type list now that PostLoad has occurred
		UWorld::WorldTypePreLoadMap.Remove( PIEPackageFName );

		if (OutPackage == nullptr)
		{
			return nullptr;
		}

		UWorld* NewWorld = UWorld::FindWorldInPackage(OutPackage);

		// If the world was not found, follow a redirector if there is one.
		if ( !NewWorld )
		{
			NewWorld = UWorld::FollowWorldRedirectorInPackage(OutPackage);
			if ( NewWorld )
			{
				OutPackage = NewWorld->GetOutermost();
			}
		}

		check(NewWorld);

		OutPackage->SetPIEInstanceID(WorldContext.PIEInstance);
		OutPackage->SetPackageFlags(PKG_PlayInEditor);

		// Rename streaming levels to PIE
		for (ULevelStreaming* StreamingLevel : NewWorld->GetStreamingLevels())
		{
			StreamingLevel->RenameForPIE(WorldContext.PIEInstance);
		}

		return NewWorld;
	}
}

/*-----------------------------------------------------------------------------
Object class implementation.
-----------------------------------------------------------------------------*/

/**
* Compresses and decompresses thumbnails using the PNG format.  This is used by the package loading and
* saving process.
*/
class FPNGThumbnailCompressor
	: public FThumbnailCompressionInterface
{

public:

	/**
	* Compresses an image
	*
	* @param	InUncompressedData	The uncompressed image data
	* @param	InWidth				Width of the image
	* @param	InHeight			Height of the image
	* @param	OutCompressedData	[Out] Compressed image data
	*
	* @return	true if the image was compressed successfully, otherwise false if an error occurred
	*/
	virtual bool CompressImage( const TArray< uint8 >& InUncompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutCompressedData )
	{
		// This incorrectly treats the colors as being RGBA order, not BGRA , eg. NOT FColor
		//	that is benign as long as CompressImage/DecompressImage do the same thing
		//	but if you used different functions (eg. ones that decode directly do FImage or assume FColor) then be aware.

		bool bSucceeded = false;
		OutCompressedData.Reset();
		if( InUncompressedData.Num() > 0 )
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper( EImageFormat::PNG );
			if ( ImageWrapper.IsValid() && ImageWrapper->SetRaw( &InUncompressedData[ 0 ], InUncompressedData.Num(), InWidth, InHeight, ERGBFormat::RGBA, 8 ) )
			{
				OutCompressedData = ImageWrapper->GetCompressed();
				bSucceeded = true;
			}
		}

		return bSucceeded;
	}


	/**
	* Decompresses an image
	*
	* @param	InCompressedData	The compressed image data
	* @param	InWidth				Width of the image
	* @param	InHeight			Height of the image
	* @param	OutUncompressedData	[Out] Uncompressed image data
	*
	* @return	true if the image was decompressed successfully, otherwise false if an error occurred
	*/
	virtual bool DecompressImage( const TArray< uint8 >& InCompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutUncompressedData )
	{
		// This incorrectly treats the colors as being RGBA order, not BGRA , eg. NOT FColor
		//	that is benign as long as CompressImage/DecompressImage do the same thing
		//	but if you used different functions (eg. ones that decode directly do FImage or assume FColor) then be aware.

		bool bSucceeded = false;
		OutUncompressedData.Reset();
		if( InCompressedData.Num() > 0 )
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper( EImageFormat::PNG );
			if ( ImageWrapper.IsValid() && ImageWrapper->SetCompressed( &InCompressedData[ 0 ], InCompressedData.Num() ) )
			{
				check( ImageWrapper->GetWidth() == InWidth );
				check( ImageWrapper->GetHeight() == InHeight );
				if ( ImageWrapper->GetRaw( ERGBFormat::RGBA, 8, OutUncompressedData) )
				{
					bSucceeded = true;
				}
			}
		}

		return bSucceeded;
	}
	
	/** Get name of compressor
	 * 
	 * @return Name of thumbnail compressor
	 */
	FName GetThumbnailCompressorName() const override
	{
		static FName CompressorName = "PNG";
		return CompressorName;
	}

	/** Is lossless compression
	 * 
	 * @return true if compression is lossless
	 */
	bool IsLosslessCompression() const override
	{
		return true;
	}

	FStringView GetMimeType() const override
	{
		return FStringView(TEXT("image/png"));
	}
};


class FJPEGThumbnailCompressor
	: public FThumbnailCompressionInterface
{

public:

	/**
	* Compresses an image
	*
	* @param	InUncompressedData	The uncompressed image data
	* @param	InWidth				Width of the image
	* @param	InHeight			Height of the image
	* @param	OutCompressedData	[Out] Compressed image data
	*
	* @return	true if the image was compressed successfully, otherwise false if an error occurred
	*/
	virtual bool CompressImage( const TArray< uint8 >& InUncompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutCompressedData )
	{
		// This incorrectly treats the colors as being RGBA order, not BGRA , eg. NOT FColor
		//	that is benign as long as CompressImage/DecompressImage do the same thing
		//	but if you used different functions (eg. ones that decode directly do FImage or assume FColor) then be aware.

		// BTW this does make the JPEG slightly less ideal because RB are swapped from how they should be converted to CbCr

		bool bSucceeded = false;
		OutCompressedData.Reset();
		if( InUncompressedData.Num() > 0 )
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper( EImageFormat::JPEG );
			if ( ImageWrapper.IsValid() && ImageWrapper->SetRaw( &InUncompressedData[ 0 ], InUncompressedData.Num(), InWidth, InHeight, ERGBFormat::RGBA, 8 ) )
			{
				int ThumbnailJPEGQuality = 65;
				OutCompressedData = ImageWrapper->GetCompressed(ThumbnailJPEGQuality);
				bSucceeded = true;
			}
		}

		return bSucceeded;
	}


	/**
	* Decompresses an image
	*
	* @param	InCompressedData	The compressed image data
	* @param	InWidth				Width of the image
	* @param	InHeight			Height of the image
	* @param	OutUncompressedData	[Out] Uncompressed image data
	*
	* @return	true if the image was decompressed successfully, otherwise false if an error occurred
	*/
	virtual bool DecompressImage( const TArray< uint8 >& InCompressedData, const int32 InWidth, const int32 InHeight, TArray< uint8 >& OutUncompressedData )
	{
		// This incorrectly treats the colors as being RGBA order, not BGRA , eg. NOT FColor
		//	that is benign as long as CompressImage/DecompressImage do the same thing
		//	but if you used different functions (eg. ones that decode directly do FImage or assume FColor) then be aware.

		bool bSucceeded = false;
		OutUncompressedData.Reset();
		if( InCompressedData.Num() > 0 )
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper( EImageFormat::JPEG );
			if ( ImageWrapper.IsValid() && ImageWrapper->SetCompressed( &InCompressedData[ 0 ], InCompressedData.Num() ) )
			{
				check( ImageWrapper->GetWidth() == InWidth );
				check( ImageWrapper->GetHeight() == InHeight );
				if ( ImageWrapper->GetRaw( ERGBFormat::RGBA, 8, OutUncompressedData) )
				{
					bSucceeded = true;
				}
			}
		}

		return bSucceeded;
	}

	/** Get name of compressor
	 * 
	 * @return Name of thumbnail compressor
	 */
	FName GetThumbnailCompressorName() const override
	{
		static FName CompressorName = "JPEG";
		return CompressorName;
	}

	/** Is lossless compression
	 * 
	 * @return true if compression is lossless
	 */
	bool IsLosslessCompression() const override
	{
		return false;
	}

	FStringView GetMimeType() const override
	{
		return FStringView(TEXT("image/jpeg"));
	}
};


/**
* Helper class inhibiting screen saver by e.g. moving the mouse by 0 pixels every 50 seconds.
*/
class FScreenSaverInhibitor : public FRunnable
{
public:
	/** Default constructor. */
	FScreenSaverInhibitor()
	{}

protected:

	bool Init() override
	{ 
		return true; 
	}

	void Stop() override
	{
		bEnabled.store(false, std::memory_order_relaxed);
	}

	/**
	* Prevents screensaver from kicking in by calling FPlatformMisc::PreventScreenSaver every 50 seconds.
	*/
	uint32 Run() override
	{
		while( bEnabled.load(std::memory_order_relaxed) )
		{
			const int32 NUM_SECONDS_TO_SLEEP = 50;
			for( int32 Sec = 0; Sec < NUM_SECONDS_TO_SLEEP && bEnabled.load(std::memory_order_relaxed); ++Sec )
			{
				FPlatformProcess::Sleep( 1 );
			}
			FPlatformApplicationMisc::PreventScreenSaver();
		}
		return 0;
	}

	std::atomic<bool> bEnabled{ true };
};

/*-----------------------------------------------------------------------------
FWorldContext
-----------------------------------------------------------------------------*/

void FWorldContext::SetCurrentWorld(UWorld* World)
{
	if (World != nullptr && AudioDeviceID != INDEX_NONE)
	{
		if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
		{
			// Set the world's audio device handle so that audio components playing in the 
			// world will use the correct audio device instance.
			DeviceManager->SetAudioDevice(*World, AudioDeviceID);
		}
	}

	for (int32 idx = 0; idx < ExternalReferences.Num(); ++idx)
	{
		if (ExternalReferences[idx] && *ExternalReferences[idx] == ThisCurrentWorld)
		{
			*ExternalReferences[idx] = World;
		}
	}

	UWorld* OldWorld = ThisCurrentWorld;
	ThisCurrentWorld = World;

	if (OwningGameInstance)
	{
		OwningGameInstance->OnWorldChanged(OldWorld, ThisCurrentWorld);
	}
}

void FWorldContext::AddReferencedObjects(FReferenceCollector& Collector, const UObject* ReferencingObject)
{
	// TODO: This is awfully unsafe as anything in a WorldContext that changes may not be referenced
	//	 hopefully a utility to push the WorldContext back in to the collector with property collection
	//       will happen in the future
	Collector.AddReferencedObject(PendingNetGame, ReferencingObject);
	for (FFullyLoadedPackagesInfo& PackageInfo : PackagesToFullyLoad)
	{
		Collector.AddReferencedObjects(PackageInfo.LoadedObjects, ReferencingObject);
	}
	Collector.AddReferencedObjects(LoadedLevelsForPendingMapChange, ReferencingObject);
	Collector.AddReferencedObjects(ObjectReferencers, ReferencingObject);
	Collector.AddReferencedObject(GameViewport, ReferencingObject);
	Collector.AddReferencedObject(OwningGameInstance, ReferencingObject);
	for (FNamedNetDriver& ActiveNetDriver : ActiveNetDrivers)
	{
		Collector.AddReferencedObject(ActiveNetDriver.NetDriver, ReferencingObject);
	}
	Collector.AddReferencedObject(ThisCurrentWorld, ReferencingObject);
}

/*-----------------------------------------------------------------------------
World/ Level/ Actor GC verification.
-----------------------------------------------------------------------------*/

#if STATS

/** Used by a delegate for access to player's viewpoint from StatsNotifyProviders */
void GetFirstPlayerViewPoint(FVector& out_Location, FRotator& out_Rotation)
{
	ULocalPlayer* Player = GEngine->GetDebugLocalPlayer();
	if( Player != NULL && Player->PlayerController != NULL )
	{
		// Calculate the player's view information.
		Player->PlayerController->GetPlayerViewPoint( out_Location, out_Rotation );		
	}
}

#endif


/*-----------------------------------------------------------------------------
Engine init and exit.
-----------------------------------------------------------------------------*/

/** Callback from OS when we get a low memory warning.
* Note: might not be called from the game thread
*/
static void EngineMemoryWarningHandler(const FGenericMemoryWarningContext& GenericContext)
{
	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("EngineMemoryWarningHandler: Mem Used %.2f MB, Streaming Texture Memory %.2f MB, Non Streaming Texture Memory %.2f MB, OS Free %.2f MB\n"), 
		Stats.UsedPhysical / 1048576.0f, 
		static_cast<double>(GRHIGlobals.StreamingTextureMemorySizeInKB) / 1024.0,
		static_cast<double>(GRHIGlobals.NonStreamingTextureMemorySizeInKB) / 1024.0, //-V567
		Stats.AvailablePhysical / 1048576.0f);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	static const auto OOMMemReportVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("Debug.OOMMemReport")); 
	const int32 OOMMemReport = OOMMemReportVar ? OOMMemReportVar->GetValueOnAnyThread() : false;
	if( OOMMemReport )
	{
		GEngine->Exec(NULL, TEXT("OBJ LIST"));
		GEngine->Exec(NULL, TEXT("MEM FROMREPORT"));
	}
#endif

	GLastMemoryWarningTime = FPlatformTime::Seconds();
}

UEngine::FOnNewStatRegistered UEngine::NewStatDelegate;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarStressTestGCWhileStreaming(
	TEXT("gc.StressTestGC"),
	0,
	TEXT("If set to 1, the engine will attempt to trigger GC each frame while async loading."));
static TAutoConsoleVariable<int32> CVarForceCollectGarbageEveryFrame(
	TEXT("gc.ForceCollectGarbageEveryFrame"),
	0,
	TEXT("If set to 1, the engine will force GC each frame."));
#endif

int32 GPerformGCWhileAsyncLoading = 0;
static FAutoConsoleVariableRef CVarPerformGCWhileAsyncLoading(
	TEXT("gc.PerformGCWhileAsyncLoading"),
	GPerformGCWhileAsyncLoading,
	TEXT("Allow performing GC even if there's async loading in progress. 0 - Disabled, 1 - Enabled, 2 - Enabled when low memory is detected (see gc.LowMemory.MemoryThresholdMB)"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarCollectGarbageEveryFrame(
	TEXT("gc.CollectGarbageEveryFrame"),
	0,
	TEXT("Used to debug garbage collection...Collects garbage every N frames if the value is > 0."));


static TAutoConsoleVariable<int32> CVarContinuousIncrementalGC(
	TEXT("gc.ContinuousIncrementalGC"),
	0,
	TEXT("Used to debug garbage collection...Kicks off Incremental Garbage Collection as soon as the previous one finishes."));

static float GTimeBetweenPurgingPendingKillObjects = 60.0f;
static FAutoConsoleVariableRef CVarTimeBetweenPurgingPendingKillObjects(
	TEXT("gc.TimeBetweenPurgingPendingKillObjects"),
	GTimeBetweenPurgingPendingKillObjects,
	TEXT("Time in seconds (game time) we should wait between purging object references to objects that are pending kill."),
	ECVF_Default
);

static float GTimeBetweenPurgingPendingKillObjectsOnIdleServerMultiplier = 10.0f;
static FAutoConsoleVariableRef CVarTimeBetweenPurgingPendingKillObjectsOnIdleServerMultiplier(
	TEXT("gc.TimeBetweenPurgingPendingKillObjectsOnIdleServerMultiplier"),
	GTimeBetweenPurgingPendingKillObjectsOnIdleServerMultiplier,
	TEXT("Multiplier to apply to time between purging pending kill objects when on an idle server."),
	ECVF_Default
);

static float GLowMemoryTimeBetweenPurgingPendingKillObjects = 30.0f;
static FAutoConsoleVariableRef CVarLowMemoryTimeBetweenPurgingPendingKillObjects(
	TEXT("gc.LowMemory.TimeBetweenPurgingPendingKillObjects"),
	GLowMemoryTimeBetweenPurgingPendingKillObjects,
	TEXT("Time in seconds (game time) we should wait between purging object references to objects that are pending kill when we're low on memory"),
	ECVF_Default
);

// see also: s.ForceGCAfterLevelStreamedOut, s.ContinuouslyIncrementalGCWhileLevelsPendingPurge
static float GLowMemoryTimeBetweenPurgingPendingLevels = 15.0f;
static FAutoConsoleVariableRef CVarLowMemoryTimeBetweenPurgingPendingLevels(
	TEXT("gc.LowMemory.TimeBetweenPurgingPendingLevels"),
	GLowMemoryTimeBetweenPurgingPendingLevels,
	TEXT("Time in seconds (game time) we should wait between GC when we're low on memory and there are levels pending unload"),
	ECVF_Default
);

static float GLowMemoryMemoryThresholdMB = 0.0f;
static FAutoConsoleVariableRef CVarLowMemoryThresholdMB(
	TEXT("gc.LowMemory.MemoryThresholdMB"),
	GLowMemoryMemoryThresholdMB,
	TEXT("Memory threshold for low memory GC mode, in MB"),
	ECVF_Default
);

static float GLowMemoryIncrementalGCTimePerFrame = 0.002f; // 2ms
static FAutoConsoleVariableRef CVarLowMemoryIncrementalGCTimePerFrame(
	TEXT("gc.LowMemory.IncrementalGCTimePerFrame"),
	GLowMemoryIncrementalGCTimePerFrame,
	TEXT("How much time is allowed for incremental GC each frame in seconds if memory is low"),
	ECVF_Default
);

static float GIncrementalGCTimePerFrame = 0.002f; // 2ms
static FAutoConsoleVariableRef CVarIncrementalGCTimePerFrame(
	TEXT("gc.IncrementalGCTimePerFrame"),
	GIncrementalGCTimePerFrame,
	TEXT("How much time is allowed for incremental GC each frame in seconds"),
	ECVF_Default
);

#if UE_ENABLE_LOG_STACK_ON_FORCE_GC
bool GLogStackOnForceGC = false;
#endif

static float GPauseGCTimeLimitSeconds = 30.0;
static FAutoConsoleVariableRef CPauseGCTimeLimitSeconds(
	TEXT("gc.PauseGCTimeLimitSeconds"),
	GPauseGCTimeLimitSeconds,
	TEXT("If GC is paused, it will resume automatically after this duration"),
	ECVF_Default
);

static int32 GPauseGCFreeMemThresholdMB = 300;
static FAutoConsoleVariableRef CPauseGCFreeMemThresholdMB(
	TEXT("gc.PauseGCFreeMemThresholdMB"),
	GPauseGCFreeMemThresholdMB,
	TEXT("If GC is paused, it will resume automatically if free physical memory is less than this threshold"),
	ECVF_Default
);

static bool GPauseGCDeferredMapLoadTrim = false;
static FAutoConsoleVariableRef CPauseGCDeferredMapLoadTrim(
	TEXT("gc.PauseGCDeferredMapLoadTrim"),
	GPauseGCDeferredMapLoadTrim,
	TEXT("If true, LoadMap TrimMemory will be replicated after GC resumes.\n")
	TEXT("If false, LoadMap TrimMemory will be skipped but we'll do a full purge GC on resume"),
	ECVF_Default
);

static bool GPauseGCForceNoFullPurgeOnResume = false;
static FAutoConsoleVariableRef CPauseGCForceNoFullPurgeOnResume(
	TEXT("gc.PauseGCForceNoFullPurgeOnResume"),
	GPauseGCForceNoFullPurgeOnResume,
	TEXT("If true, clear the FullPurge flag on resume; any full purge GCs requested while paused will be cleared\n"),
	ECVF_Default
);


enum class EGcRequestType : uint8
{
	Incremental,
	FullPurge,
	LoadMapTrim,
};

inline const TCHAR* LexToString(EGcRequestType Type)
{
	switch (Type)
	{
	case EGcRequestType::Incremental:		return TEXT("Incremental");
	case EGcRequestType::FullPurge:			return TEXT("FullPurge");
	case EGcRequestType::LoadMapTrim:		return TEXT("LoadMapTrim");
	default:								return TEXT("Unknown");
	}
}

// Handles pausing and resuming garbage collection
class FGarbageCollectionPause
{
public:
	static FGarbageCollectionPause& Get()
	{
		static FGarbageCollectionPause Inst;
		return Inst;
	}
	bool IsPaused() const
	{
		return StartTime > 0.0;
	}

	bool Resume()
	{
		check(IsInGameThread());
		if (!IsPaused())
		{
			// Nothing to do
			return false;
		}
		// If a trim was requested then do it now
		UE_LOGF(LogGarbage, Log, "Garbage collection resumed");
		ResumeInternal();
		if (GPauseGCDeferredMapLoadTrim && bTrimRequested && GEngine)
		{
			UE_LOGF(LogGarbage, Log, "TrimMemory after GC resumed");
			GEngine->TrimMemory();
		}
		return true;
	}

	void ResumeInternal()
	{
		StartTime = 0.0;
	}

	bool ShouldResume() const
	{
		if (GPauseGCTimeLimitSeconds > 0.0 && FPlatformTime::Seconds() - StartTime > GPauseGCTimeLimitSeconds)
		{
			UE_LOGF(LogGarbage, Log, "Resuming GC due to time limit: %.2fs", GPauseGCTimeLimitSeconds);
			return true;
		}
		if (GPauseGCFreeMemThresholdMB > 0 && PlatformMemoryHelpers::GetFrameMemoryStats().AvailablePhysical < ((uint64)GPauseGCFreeMemThresholdMB << 20ull))
		{
			UE_LOGF(LogGarbage, Log, "Resuming GC due to mem threshold: %dMB", GPauseGCFreeMemThresholdMB);
			return true;
		}
		return false;
	}

	void Pause()
	{
		check(IsInGameThread());
		if (IsPaused())
		{
			// Nothing to do
			return;
		}
		MissedGcCount = 0;
		bTrimRequested = false;
		StartTime = FPlatformTime::Seconds();

		UE_LOGF(LogGarbage, Log, "Garbage collection paused");
		if (GPauseGCFreeMemThresholdMB > 0)
		{
			UE_LOGF(LogGarbage, Log, "  Free mem threshold: %dMB", GPauseGCFreeMemThresholdMB);
		}
		if (GPauseGCTimeLimitSeconds > 0.0)
		{
			UE_LOGF(LogGarbage, Log, "  Time limit: %.2fs", GPauseGCTimeLimitSeconds);
		}
	}

	// Called from ConditionalCollectGarbage. Returns true if the GC should proceed (usually the case if not paused)
	// Will unpause if conditions are met
	bool OnGc(EGcRequestType GcType)
	{
		if (!IsPaused())
		{
			return true;
		}

		// Check if we should auto-resume at this point (ie under free mem threshold/over time limit)
		if ( ShouldResume() )
		{
			ResumeInternal();
			UE_LOGF(LogGarbage, Log, "Garbage collection resumed (automatic)");

			// If a trim was requested and we're not about to trim then do one one now (this prevents an in-flight FullPurge request)
			if (GPauseGCDeferredMapLoadTrim && bTrimRequested && GcType != EGcRequestType::LoadMapTrim && GEngine)
			{
				UE_LOGF(LogGarbage, Log, "TrimMemory after GC resumed");
				GEngine->TrimMemory();
				return false;
			}

			return true;
		}

		if (GcType == EGcRequestType::LoadMapTrim)
		{
			bTrimRequested = true;
		}

		UE_CLOG(MissedGcCount < 8 || GcType == EGcRequestType::LoadMapTrim, LogGarbage, Log, TEXT("GC is paused. Skipped GC (Type: %s)"), LexToString(GcType));
		MissedGcCount++;
		return false;
	}

	int32 GetMissedGcCount() const
	{
		return MissedGcCount;
	}

private:
	FGarbageCollectionPause() {}

	FTSTicker::FDelegateHandle TickHandle;
	double StartTime = 0.0;
	int32 MissedGcCount = 0;
	bool bTrimRequested = false;
};

void UEngine::PauseGarbageCollection()
{
	FGarbageCollectionPause::Get().Pause();
}

bool UEngine::ResumeGarbageCollection()
{
	if (FGarbageCollectionPause::Get().Resume())
	{
		// Clear any pending full purges on resume, if configured
		if (GPauseGCForceNoFullPurgeOnResume)
		{
			bFullPurgeTriggered = false;
		}
		return true;
	}
	return false;
}

bool UEngine::IsGarbageCollectionPaused()
{
	return FGarbageCollectionPause::Get().IsPaused();
}


void UEngine::SendWorldEndOfFrameUpdates()
{
	// Gather worlds that need EOF updates
	// This must be done in two steps as the object hash table is locked during ForEachObjectOfClass so any NewObject calls would fail
	TArray<UWorld*, TInlineAllocator<16>> Worlds;
	ForEachObjectOfClass(UWorld::StaticClass(), [&](UObject* WorldObj)
	{
		if (UWorld* World = CastChecked<UWorld>(WorldObj))
		{
			if (World->HasEndOfFrameUpdates())
			{
				Worlds.Add(World);
			}
		}
	});

	for (UWorld* World : Worlds)
	{
		World->FinishAsyncSendAllEndOfFrameUpdates();

		// Make sure deferred component updates have been sent to the rendering thread before deleting any UObjects which the rendering thread may be referencing
		// This fixes rendering thread crashes in the following order of operations 1) UMeshComponent::SetMaterial 2) GC 3) Rendering command that dereferences the UMaterial
		World->SendAllEndOfFrameUpdates();
	}
}

void UEngine::PreGarbageCollect()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEngine::PreGarbageCollect);

	SendWorldEndOfFrameUpdates();
}

float GetLowMemoryGCTimer(const float DefaultTimeBetweenGC)
{
	float MBFree = float(PlatformMemoryHelpers::GetFrameMemoryStats().AvailablePhysical / 1024 / 1024);
#if !UE_BUILD_SHIPPING
	MBFree -= float(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024 / 1024);
#endif

	GIsLowMemory = (MBFree <= GLowMemoryMemoryThresholdMB);
	if (GIsLowMemory)
	{
		if (GLowMemoryTimeBetweenPurgingPendingKillObjects < DefaultTimeBetweenGC)
		{
			return GLowMemoryTimeBetweenPurgingPendingKillObjects;
		}
	}

	return DefaultTimeBetweenGC;
}

float UEngine::GetTimeBetweenGarbageCollectionPasses() const
{
	bool bAtLeastOnePlayerConnected = false;

	if (IsRunningDedicatedServer())
	{
		for (const FWorldContext& Context : WorldList)
		{
			if (UWorld* World = Context.World())
			{
				if (World->NetDriver && World->NetDriver->ClientConnections.Num() > 0)
				{
					bAtLeastOnePlayerConnected = true;
					break;
				}
			}
		}
	}

	return GetTimeBetweenGarbageCollectionPasses(bAtLeastOnePlayerConnected);
}

float UEngine::GetTimeBetweenGarbageCollectionPasses(bool bHasPlayersConnected) const
{
	float TimeBetweenGC = GTimeBetweenPurgingPendingKillObjects;

	if (IsRunningDedicatedServer())
	{
		if (!bHasPlayersConnected)
		{
			TimeBetweenGC *= GTimeBetweenPurgingPendingKillObjectsOnIdleServerMultiplier;
		}
	}

	// Do more frequent GCs if we're under the low memory threshold (if enabled)
	if (GLowMemoryMemoryThresholdMB > 0.0)
	{
		TimeBetweenGC = GetLowMemoryGCTimer(TimeBetweenGC);
	}

	return TimeBetweenGC;
}

void UEngine::ForceGarbageCollection(bool bForcePurge/*=false*/)
{
	TimeSinceLastPendingKillPurge = 1.0f + GetTimeBetweenGarbageCollectionPasses();
	bFullPurgeTriggered = bFullPurgeTriggered || bForcePurge;

#if UE_ENABLE_LOG_STACK_ON_FORCE_GC
	if (GLogStackOnForceGC)
	{
		UE_LOGF(LogOutputDevice, Log, "ForceGarbageCollection called, logging stack");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Log);
	}
#endif
}

void UEngine::DelayGarbageCollection()
{
	bShouldDelayGarbageCollect = true;
}

void UEngine::SetTimeUntilNextGarbageCollection(const float MinTimeUntilNextPass)
{
	const float TimeBetweenPurgingPendingKillObjects = GetTimeBetweenGarbageCollectionPasses();

	// This can make it go negative if the desired interval is longer than the typical interval, but it's only ever compared against TimeBetweenPurgingPendingKillObjects
	TimeSinceLastPendingKillPurge = TimeBetweenPurgingPendingKillObjects - MinTimeUntilNextPass;
}

EGarbageCollectionType UEngine::ShouldForceGarbageCollection()
{
	return EGarbageCollectionType::None;
}

float UEngine::GetIncrementalGCTimePerFrame()
{
	float IncGCTime = GIncrementalGCTimePerFrame;
	if (GLowMemoryMemoryThresholdMB > 0.0)
	{
		float MBFree = float(PlatformMemoryHelpers::GetFrameMemoryStats().AvailablePhysical / 1024 / 1024);
#if !UE_BUILD_SHIPPING
		MBFree -= float(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024 / 1024);
#endif
		if (MBFree <= GLowMemoryMemoryThresholdMB && GLowMemoryIncrementalGCTimePerFrame > GIncrementalGCTimePerFrame)
		{
			IncGCTime = GLowMemoryIncrementalGCTimePerFrame;
		}
	}
	return IncGCTime;
}

void UEngine::ConditionalCollectGarbage()
{
	AutoRTFM::UnreachableIfTransactional();

	if (GFrameCounter != LastGCFrame)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ConditionalCollectGarbage);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (CVarStressTestGCWhileStreaming.GetValueOnGameThread() && IsAsyncLoading())
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
		}
		else if (CVarForceCollectGarbageEveryFrame.GetValueOnGameThread())
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
		}
		else
#endif
		{
			EGarbageCollectionType ForceTriggerPurge = ShouldForceGarbageCollection();
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			if (ForceTriggerPurge == EGarbageCollectionType::None && UE::GC::ShouldFrankenGCRun())
			{
				ForceTriggerPurge = EGarbageCollectionType::Incremental;
			}
#endif
			if (ForceTriggerPurge != EGarbageCollectionType::None)
			{
				ForceGarbageCollection(ForceTriggerPurge == EGarbageCollectionType::Full);
			}

			if (bFullPurgeTriggered)
			{
				if (FGarbageCollectionPause::Get().OnGc(EGcRequestType::FullPurge))
				{
					TGuardValue<bool> _FullGC(bGCPerformingFullPurge, true);
					if (TryCollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true))
					{
						ForEachObjectOfClass(UWorld::StaticClass(), [](UObject* World)
							{
								CastChecked<UWorld>(World)->CleanupActors();
							});
						bFullPurgeTriggered = false;
						bShouldDelayGarbageCollect = false;
						TimeSinceLastPendingKillPurge = 0.0f;
					}
				}
			}
			else
			{
				const bool bTestForPlayers = IsRunningDedicatedServer();
				bool bHasAWorldBegunPlay = false;
				bool bHasPlayersConnected = false;

				// Look for conditions in the worlds that would change the GC frequency
				for (const FWorldContext& Context : WorldList)
				{
					if (UWorld* World = Context.World())
					{
						if (World->HasBegunPlay())
						{
							bHasAWorldBegunPlay = true;
						}

						if (bTestForPlayers && World->NetDriver && World->NetDriver->ClientConnections.Num() > 0)
						{
							bHasPlayersConnected = true;
						}

						// If we found the conditions we wanted, no need to continue iterating
						if (bHasAWorldBegunPlay && (!bTestForPlayers || bHasPlayersConnected))
						{
							break;
						}
					}
				}

				if (bHasAWorldBegunPlay)
				{
					TimeSinceLastPendingKillPurge += FApp::GetDeltaTime();

					const float TimeBetweenPurgingPendingKillObjects = GetTimeBetweenGarbageCollectionPasses(bHasPlayersConnected);

					if (FGarbageCollectionPause::Get().OnGc(EGcRequestType::Incremental))
					{
						// See if we should delay garbage collect for this frame
						if (bShouldDelayGarbageCollect)
						{
							bShouldDelayGarbageCollect = false;
						}
						else if (IsIncrementalReachabilityAnalysisPending())
						{
							SCOPE_CYCLE_COUNTER(STAT_GCMarkTime);
							PerformIncrementalReachabilityAnalysis(GetReachabilityAnalysisTimeLimit());
						}
						// Perform incremental purge update if it's pending or in progress.
						else if (!IsIncrementalPurgePending()
							// Purge reference to pending kill objects every now and so often.
							&& (TimeSinceLastPendingKillPurge > TimeBetweenPurgingPendingKillObjects) && TimeBetweenPurgingPendingKillObjects > 0.f)
						{
							SCOPE_CYCLE_COUNTER(STAT_GCMarkTime);
							PerformGarbageCollectionAndCleanupActors();
						}
						else
						{
							SCOPE_CYCLE_COUNTER(STAT_GCSweepTime);
							float IncGCTime = GetIncrementalGCTimePerFrame();
							IncrementalPurgeGarbage(true, IncGCTime);
						}
					}
				}
			}
		}

		if (const int32 Interval = CVarCollectGarbageEveryFrame.GetValueOnGameThread())
		{
			if (0 == (GFrameCounter % Interval))
			{
				ForceGarbageCollection(true);
			}
		}
		else if (CVarContinuousIncrementalGC.GetValueOnGameThread() > 0 && !IsIncrementalReachabilityAnalysisPending() && !IsIncrementalUnhashPending() && !IsIncrementalPurgePending())
		{
			ForceGarbageCollection(false);
		}

		LastGCFrame = GFrameCounter;
	}
}

void UEngine::PerformGarbageCollectionAndCleanupActors()
{
	// We don't collect garbage while there are outstanding async load requests as we would need
	// to block on loading the remaining data.
	const bool bPerformGCWhileAsyncLoading = (GPerformGCWhileAsyncLoading == 1) || (GPerformGCWhileAsyncLoading == 2 && GIsLowMemory);
	if (bPerformGCWhileAsyncLoading || !IsAsyncLoading())
	{
		bool bForcePurge = true;
		for (FWorldContext& Context : WorldList)
		{
			UWorld* World = Context.World();
			if (World != nullptr && World->IsGameWorld())
			{
				bForcePurge = false;
				break;
			}
		}

		// Perform housekeeping.
		TGuardValue<bool> _FullGC(bGCPerformingFullPurge, bForcePurge);
		if (TryCollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bForcePurge))
		{
			ForEachObjectOfClass(UWorld::StaticClass(), [](UObject* World)
			{
				CastChecked<UWorld>(World)->CleanupActors();
			});

			// Reset counter.
			TimeSinceLastPendingKillPurge = 0.0f;
			bFullPurgeTriggered = false;
			LastGCFrame = GFrameCounter;
		}
	}
}

//
// Initialize the engine.
//
void UEngine::Init(IEngineLoop* InEngineLoop)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEngine::Init);
	UE_LOGF(LogEngine, Log, "Initializing Engine...");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Engine Initialized"), STAT_EngineStartup, STATGROUP_LoadTime);

	// Start capturing errors and warnings
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	ErrorsAndWarningsCollector = MakePimpl<FErrorsAndWarningsCollector>();
#endif

#if WITH_EDITOR
	if(!FEngineBuildSettings::IsInternalBuild())
	{
		TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();

		for (auto Plugin : EnabledPlugins)
		{
			const FPluginDescriptor& Desc = Plugin->GetDescriptor();

			// encode a minimal plugin description for the crash reporter
			FString DescStr;
			TSharedRef< TJsonWriter<> > WriterRef = TJsonWriterFactory<>::Create(&DescStr);
			TJsonWriter<>& Writer = WriterRef.Get();			
			Writer.WriteObjectStart();
			Writer.WriteValue(TEXT("Version"), Desc.Version);
			Writer.WriteValue(TEXT("VersionName"), Desc.VersionName);
			Writer.WriteValue(TEXT("FriendlyName"), Desc.FriendlyName);
			Writer.WriteObjectEnd();
			Writer.Close();

			FGenericCrashContext::AddPlugin(DescStr);
		}
	}
#endif

	// Set the memory warning handler
	if (!FPlatformMisc::HasMemoryWarningHandler())
	{
		FPlatformMisc::SetMemoryWarningHandler(EngineMemoryWarningHandler);
	}

	EngineLoop = InEngineLoop;

	// Elements.
	RegisterEngineElements();

	// Subsystems.
	FURL::StaticInit();
	EngineSubsystemCollection.Initialize(this);

#if !UE_BUILD_SHIPPING
	// Check for overrides to the default map on the command line
	TCHAR MapName[512];
	if ( FParse::Value(FCommandLine::Get(), TEXT("DEFAULTMAP="), MapName, UE_ARRAY_COUNT(MapName)) )
	{
		UE_LOGF(LogEngine, Log, "Overriding default map to %ls", MapName);

		FString MapString = FString(MapName);
		UGameMapsSettings::SetGameDefaultMap(MapString);
	}
#endif // !UE_BUILD_SHIPPING

	InitializeRunningAverageDeltaTime();

	// Add to root.
	AddToRoot();

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddStatic(UEngine::PreGarbageCollect);

	FCoreUObjectDelegates::GetGarbageCollectStartedDelegate().AddLambda([]()
		{
			if (GSuspendAssetStreamerBeforePerformingFullGC != 0 && GEngine != nullptr && GEngine->bGCPerformingFullPurge)
			{
				// Flush render and RHI commands in case there are some UStreamableRenderAssets with pending RenderAssetUpdate RHI tasks
				ENQUEUE_RENDER_COMMAND(FlushCommand)(
					[](FRHICommandListImmediate& RHICmdList)
					{
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
					});
				FlushRenderingCommands();

				SuspendRenderAssetStreaming();
			}
		});

	FCoreUObjectDelegates::GetPostGarbageCollect().AddLambda([]()
		{
			if (GSuspendAssetStreamerBeforePerformingFullGC != 0 && GEngine != nullptr && GEngine->bGCPerformingFullPurge && IsAssetStreamingSuspended())
			{
				ResumeRenderAssetStreaming();
			}
		});


	if (!FApp::IsProjectNameEmpty())
	{
		// Initialize the HMDs and motion controllers, if any
		InitializeHMDDevice();

		// Initialize attached eye tracking devices, if any
		InitializeEyeTrackingDevice();
	}

	// Disable the screensaver when running the game.
	if( GIsClient && !GIsEditor )
	{
		EnableScreenSaver( false );
	}
	
	// enforce disabling screen saver when it's requested from commandline
	if (FParse::Param(FCommandLine::Get(), TEXT("keepscreenawake")))
	{
		FPlatformApplicationMisc::ControlScreensaver(FPlatformApplicationMisc::EScreenSaverAction::Disable);
	}

	if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
	{
		// If Slate is being used, initialize the renderer after RHIInit 
		FSlateApplication& CurrentSlateApp = FSlateApplication::Get();
		CurrentSlateApp.InitializeSound( TSharedRef<FSlateSoundDevice>( new FSlateSoundDevice() ) );

#if !UE_BUILD_SHIPPING
		// Create test windows (if we were asked to do that)
		if( FParse::Param( FCommandLine::Get(), TEXT("SlateDebug") ) )
		{
			RestoreSlateTestSuite();
		}
#endif // #if !UE_BUILD_SHIPPING
	}

	// Assign thumbnail compressor/decompressor
	FObjectThumbnail::SetThumbnailCompressors( new FPNGThumbnailCompressor() , new FJPEGThumbnailCompressor() );

	//UEngine::StaticClass()->GetDefaultObject(true);
	LoadObject<UClass>(UEngine::StaticClass()->GetOuter(), *UEngine::StaticClass()->GetName(), NULL, LOAD_Quiet|LOAD_NoWarn, NULL );
	// This reads the Engine.ini file to get the proper DefaultMaterial, etc.
	LoadConfig();

	SetConfiguredProcessLimits();

	bIsOverridingSelectedColor = false;

	// Set colors for selection materials
	SelectedMaterialColor = DefaultSelectedMaterialColor;
	SelectionOutlineColor = DefaultSelectedMaterialColor;

	if (IsDefaultBackBufferLinearSDR())
	{
		DisplayGamma = 1.0f;
	}

#if WITH_EDITOR
	FModuleManager::Get().LoadModule("WorldPartitionEditor");
#endif

	UPSOPrecacheSettingsManager::SetupFallbackMaterial();

	InitializeObjectReferences();

	if (GConfig)
	{
		bool bTemp = true;
		GConfig->GetBool(TEXT("/Script/Engine.Engine"), TEXT("bEnableOnScreenDebugMessages"), bTemp, GEngineIni);
		bEnableOnScreenDebugMessages = bTemp ? true : false;
		bEnableOnScreenDebugMessagesDisplay = bEnableOnScreenDebugMessages;
	}

	// Update Script Maximum loop iteration count
	FBlueprintCoreDelegates::SetScriptMaximumLoopIterations( GEngine->MaximumLoopIterationCount );

	SetNearClipPlaneGlobals(NearClipPlane);

	UTextRenderComponent::InitializeMIDCache();

	if (GIsEditor)
	{
		// Create a WorldContext for the editor to use and create an initially empty world.
		FWorldContext &InitialWorldContext = CreateNewWorldContext(EWorldType::Editor);
		InitialWorldContext.SetCurrentWorld( UWorld::CreateWorld( EWorldType::Editor, true ) );
		GWorld = InitialWorldContext.World();
	}

	

	// Make sure networking checksum has access to project version
	const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
	FNetworkVersion::SetProjectVersion(*ProjectSettings.ProjectVersion);

#if !(UE_BUILD_SHIPPING) || ENABLE_PGO_PROFILE
	// Optionally Exec an exec file
	FString Temp;
	if (FParse::Value(FCommandLine::Get(), TEXT("EXEC="), Temp))
	{
		GEngine->DeferredCommands.Add(FString("exec ") + Temp);
	}

	// Optionally exec commands passed in the command line.
	ParseExecCommands::QueueDeferredCommands(ParseExecCommands::ParseExecCmdsFromCommandLine(TEXT("ExecCmds")));

	// optionally set the vsync console variable
	if (FParse::Param(FCommandLine::Get(), TEXT("vsync")))
	{
		GEngine->DeferredCommands.Add(TEXT("r.vsync 1"));
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("novsync")))
	{
		GEngine->DeferredCommands.Add(TEXT("r.vsync 0"));
	}
#endif // !(UE_BUILD_SHIPPING) || ENABLE_PGO_PROFILE

	// register the engine with the travel and network failure broadcasts
	// games can override these to provide proper behavior in each error case
	OnTravelFailure().AddUObject(this, &UEngine::HandleTravelFailure);
	OnNetworkFailure().AddUObject(this, &UEngine::HandleNetworkFailure);
	OnNetworkLagStateChanged().AddUObject(this, &UEngine::HandleNetworkLagStateChanged);

	UE_LOGF(LogInit, Log, "Texture streaming: %ls", IStreamingManager::Get().IsTextureStreamingEnabled() ? TEXT("Enabled") : TEXT("Disabled") );

	// Initialize the online subsystem as early as possible
	FOnlineExternalUIChanged OnExternalUIChangeDelegate;
	OnExternalUIChangeDelegate.BindUObject(this, &UEngine::OnExternalUIChange);
	UOnlineEngineInterface::Get()->BindToExternalUIOpening(OnExternalUIChangeDelegate);

	// Initialise buffer visualization system data
	GetBufferVisualizationData().Initialize();
	GetNaniteVisualizationData().Initialize();
	GetVirtualShadowMapVisualizationData().Initialize();
	GetRayTracingVisualizationData(); // force initialization
	GetVirtualTextureVisualizationData().Initialize();

	// Initialize Portal services
	if (!IsRunningCommandlet() && !IsRunningDedicatedServer())
	{
		InitializePortalServices();
	}

	// Connect the engine analytics provider
	FEngineAnalytics::Initialize();

	// Initialize the audio device after a world context is setup
	InitializeAudioDeviceManager();

	// Dynamically load engine runtime modules
	{
		FModuleManager::Get().LoadModule("ImageWriteQueue");
		FModuleManager::Get().LoadModuleChecked("StreamingPauseRendering");
		FModuleManager::Get().LoadModuleChecked("MovieScene");
		FModuleManager::Get().LoadModuleChecked("MovieSceneTracks");
		FModuleManager::Get().LoadModule("LevelSequence");
		FModuleManager::Get().LoadModule("CinematicCamera");
#if WITH_EDITOR
		// The SparseVolumeTexture module containing the importer is only loaded and used in the editor.
		FModuleManager::Get().LoadModuleChecked("SparseVolumeTexture");
#endif
	}

	// Record large world coordinate state
	CSV_METADATA(TEXT("LargeWorldCoordinates"), TEXT("1"));	// LWC_TODO: Remove

	// Finish asset manager loading
	check(AssetManager);
	AssetManager->FinishInitialLoading();

	bool bIsRHS = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("DevOptions.Debug"), TEXT("bEngineStatsOnRHS"), bIsRHS, GEngineIni);
	}

	// Add the stats to the list, note this is also the order that they get rendered in if active.
#if !UE_BUILD_SHIPPING
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Version"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display build version string which includes the Changelist number.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatVersion), FEngineStatToggle(), bIsRHS));
#endif // !UE_BUILD_SHIPPING
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_NamedEvents"), TEXT("STATCAT_Engine"), FText::GetEmpty(), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatNamedEvents), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_VerboseNamedEvents"), TEXT("STATCAT_Engine"), FText::GetEmpty(), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatVerboseNamedEvents), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_FPS"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display average FPS.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatFPS), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatFPS), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Summary"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display UsedPhysical memory stat.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatSummary), FEngineStatToggle(), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Unit"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display frame times for Game Thread, Render Thread and GPU.  Also displays draw calls and primitive counts.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatUnit), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatUnit), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Thermals"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display thermal state of the device if applicable.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatThermals), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatThermals), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_DrawCount"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display draw call counts per render pass.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatDrawCount), FEngineStatToggle(), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Hitches"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display a scrolling hitch time when frame hitches occur.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatHitches), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatHitches), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_AI"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display number of APlayerControllers and the number that were rendered to screen.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatAI), FEngineStatToggle(), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Timecode"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display a time from a timecode provider.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatTimecode), FEngineStatToggle(), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_FrameCounter"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display the global, steadily increasing frame counter.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatFrameCounter), FEngineStatToggle(), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_ColorList"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display a list of common FColors and their names.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatColorList), FEngineStatToggle()));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Levels"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display a list of names of all the loaded levels.")), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatLevels), FEngineStatToggle()));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Detailed"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Display detailed frame and fps timings.")), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatDetailed)));
#if !UE_BUILD_SHIPPING
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_UnitCriticalPath"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Same as stat unit, but with critical path values also displayed.")), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatUnitCriticalPath)));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_UnitMax"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Same as stat unit, but with max values also displayed.")), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatUnitMax)));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_UnitGraph"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Displays a frame time stats graphed over time.")), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatUnitGraph)));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_UnitTime"), TEXT("STATCAT_Engine"), FText::GetEmpty(), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatUnitTime)));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_Raw"), TEXT("STATCAT_Engine"), FText::GetEmpty(), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatRaw)));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_ParticlePerf"), TEXT("STATCAT_Engine"), FText::GetEmpty(), FEngineStatRender::CreateUObject(this, &UEngine::RenderStatParticlePerf), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatParticlePerf), bIsRHS));
	EngineStats.Add(FEngineStatFuncs(TEXT("STAT_TSR"), TEXT("STATCAT_Engine"), FText::FromString(TEXT("Same as stat unit, but with additional TSR settings.")), FEngineStatRender(), FEngineStatToggle::CreateUObject(this, &UEngine::ToggleStatTSR)));
#endif // !UE_BUILD_SHIPPING

	// Let any listeners know about the new stats
	for (int32 StatIdx = 0; StatIdx < EngineStats.Num(); StatIdx++)
	{
		const FEngineStatFuncs& EngineStat = EngineStats[StatIdx];
		NewStatDelegate.Broadcast(EngineStat.CommandName, EngineStat.CategoryName, EngineStat.DescriptionString);
	}

#if UE_NET_TRACE_ENABLED
	uint32 NetTraceVerbosity;
	if(FParse::Value(FCommandLine::Get(), TEXT("NetTrace="), NetTraceVerbosity))
	{
		FNetTrace::SetTraceVerbosity(NetTraceVerbosity);
	}
#endif

	// Record the analytics for any attached HMD devices
	RecordHMDAnalytics();

#if !UE_BUILD_SHIPPING
	UE_CLOGF(FPlatformMemory::IsExtraDevelopmentMemoryAvailable(), LogInit, Warning, "Running with %lldMB of extra development memory!",FPlatformMemory::GetExtraDevelopmentMemorySize()/1024ull/1024ull);
#endif

	InitThreadConfig();
}

void UEngine::Start()
{
	// Start the game!
}

void UEngine::RegisterBeginStreamingPauseRenderingDelegate( FBeginStreamingPauseDelegate* InDelegate )
{
	BeginStreamingPauseDelegate = InDelegate;
}

void UEngine::RegisterEndStreamingPauseRenderingDelegate( FEndStreamingPauseDelegate* InDelegate )
{
	EndStreamingPauseDelegate = InDelegate;
}

void UEngine::OnExternalUIChange(bool bInIsOpening)
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().ExternalUIChange(bInIsOpening);
	}
}

void UEngine::ReleaseAudioDeviceManager()
{
	// Release the main audio device handle.
	MainAudioDeviceHandle.Reset();
	AudioDeviceManager = nullptr;
}

void UEngine::PreExit()
{
	FCoreDelegates::OnEnginePreExit.Broadcast();

#if UE_NET_TRACE_ENABLED
	FNetTrace::SetTraceVerbosity(0);
#endif

	if (IMovieSceneCaptureModule* Module = FModuleManager::GetModulePtr<IMovieSceneCaptureModule>( "MovieSceneCapture" ))
	{
		Module->DestroyAllActiveCaptures();
	}

	UTextRenderComponent::ShutdownMIDCache();

	ShutdownRenderingCVarsCaching();
	const bool bIsEngineShutdown = true;
	FEngineAnalytics::Shutdown(bIsEngineShutdown);

	if (ScreenSaverInhibitor)
	{
		// Resume the thread to avoid a deadlock while waiting for finish.
		ScreenSaverInhibitor->Suspend( false );
		delete ScreenSaverInhibitor;
		ScreenSaverInhibitor = nullptr;
	}

	delete ScreenSaverInhibitorRunnable;

	// Shutdown Timecode Provider
	if (bIsCurrentTimecodeProviderInitialized && TimecodeProvider)
	{
		TimecodeProvider->Shutdown(this);
	}
	TimecodeProvider = nullptr;

	// Shutdown CustomTimeStep
	if (bIsCurrentCustomTimeStepInitialized && CustomTimeStep)
	{
		CustomTimeStep->Shutdown(this);
	}
	CustomTimeStep = nullptr;

	ShutdownHMD();

#if WITH_DYNAMIC_RESOLUTION
	DynamicResolutionState.Reset();
	NextDynamicResolutionState.Reset();
#endif

	EngineSubsystemCollection.Deinitialize();
}

void UEngine::ShutdownHMD()
{
	// we can't just nulify these pointers here since RenderThread still might use them.
	auto SavedStereo = StereoRenderingDevice;
	auto SavedHMD = XRSystem;
	auto SavedViewExtentions = ViewExtensions;

	FlushRenderingCommands();
	StereoRenderingDevice.Reset();
	XRSystem.Reset();
}

void UEngine::TickDeferredCommands()
{
	SCOPE_TIME_GUARD(TEXT("UEngine::TickDeferredCommands"));

	const double StartTime = FPlatformTime::Seconds();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_UEngine_TickDeferredCommands);
	// Execute all currently queued deferred commands (allows commands to be queued up for next frame).
	const int32 DeferredCommandsCount = DeferredCommands.Num();
	for( int32 DeferredCommandsIndex=0; DeferredCommandsIndex<DeferredCommandsCount; DeferredCommandsIndex++ )
	{
		CSV_EVENT_GLOBAL(TEXT("Cmd: %s"), *DeferredCommands[DeferredCommandsIndex]);
		// Use LocalPlayer if available...
		ULocalPlayer* LocalPlayer = GetDebugLocalPlayer();
		if( LocalPlayer )
		{
			LocalPlayer->Exec( LocalPlayer->GetWorld(), *DeferredCommands[DeferredCommandsIndex], *GLog );
		}
		// and fall back to UEngine otherwise.
		else
		{
			Exec( GWorld, *DeferredCommands[DeferredCommandsIndex], *GLog );
		}
	}

	const double ElapsedTimeMS = (FPlatformTime::Seconds() - StartTime) * 1000.0;

	// If we're not in the editor, and commands took more than our target frame time to execute, print them out so there's a paper trail
	if (GIsEditor == false && ElapsedTimeMS >= FEnginePerformanceTargets::GetTargetFrameTimeThresholdMS())
	{
		UE_LOGF(LogEngine, Warning, "UEngine::TickDeferredCommands took %.02fms to execute %d commands!", ElapsedTimeMS, DeferredCommandsCount);

		for (int32 i = 0; i < DeferredCommandsCount; i++)
		{
			UE_LOGF(LogEngine, Warning, "\t%ls", *DeferredCommands[i]);
		}	
	}

	DeferredCommands.RemoveAt(0, DeferredCommandsCount);
}

void PumpABTest()
{
#if ENABLE_ABTEST
	const TCHAR* Command = FABTest::Get().TickAndGetCommand();
	if (Command)
	{
		GEngine->Exec(nullptr, Command, *GLog);
	}
#endif
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

struct FTimedMemReport
{
	FTimedMemReport()
		: TotalTime(0.f)
		, DumpDelayTime(-1.f)
	{
		if(const TCHAR* CommandLine = FCommandLine::Get())
		{
			FParse::Value(CommandLine, TEXT("TimedMemoryReport="), DumpDelayTime);
		}
	}

	static FTimedMemReport& Get()
	{
		static FTimedMemReport Singleton;
		return Singleton;
	}

	static void SetDumpDelayParse(const TArray<FString>& Args)
	{
		if(Args.Num())
		{
			float DumpDelay = FCString::Atof(*Args[0]);
			Get().SetDumpDelay(DumpDelay);
		}
	}

	void SetDumpDelay(float InDumpDelay)
	{
		DumpDelayTime = InDumpDelay;
		TotalTime = 0.f;	//reset time
	}

	void PumpTimedMemoryReports()
	{
		if (DumpDelayTime > 0)
		{
			TotalTime += FApp::GetDeltaTime();
			if (TotalTime > DumpDelayTime)
			{
				GEngine->Exec(nullptr, TEXT("memreport"), *GLog);
				TotalTime = 0.f;
			}
		}
	}

private:
	float TotalTime;
	float DumpDelayTime;
};

static FAutoConsoleCommand SetTimedMemReport(TEXT("TimedMemReport.Delay"), TEXT("Determines how long to wait before getting a memreport. < 0 is off"), FConsoleCommandWithArgsDelegate::CreateStatic(&FTimedMemReport::SetDumpDelayParse), ECVF_Cheat);



#endif

double UEngine::CorrectNegativeTimeDelta(double DeltaRealTime)
{
	UE_LOGF(LogEngine, Warning, "Detected negative delta time - ignoring");
	return 0.01;
}

void UEngine::SetInputSampleLatencyMarker(uint64 FrameNumber)
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());

	RHIFrameInfoSetInputSampleMarker(FrameNumber);

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetInputSampleLatencyMarker(FrameNumber);
	}
}

void UEngine::SetSimulationLatencyMarkerStart(uint64 FrameNumber)
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetSimulationLatencyMarkerStart(FrameNumber);
	}
}

void UEngine::SetSimulationLatencyMarkerEnd(uint64 FrameNumber)
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetSimulationLatencyMarkerEnd(FrameNumber);
	}
}

void UEngine::SetPresentLatencyMarkerStart(uint64 FrameNumber)
{
	IModularFeatures::Get().LockModularFeatureList();
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetPresentLatencyMarkerStart(FrameNumber);
	}
}

void UEngine::SetPresentLatencyMarkerEnd(uint64 FrameNumber)
{
	IModularFeatures::Get().LockModularFeatureList();
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetPresentLatencyMarkerEnd(FrameNumber);
	}
}


void UEngine::SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber)
{
	IModularFeatures::Get().LockModularFeatureList();
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetRenderSubmitLatencyMarkerStart(FrameNumber);
	}
}

void UEngine::SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber)
{
	IModularFeatures::Get().LockModularFeatureList();
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetRenderSubmitLatencyMarkerEnd(FrameNumber);
	}
}

void UEngine::SetFlashIndicatorLatencyMarker(uint64 FrameNumber)
{
	TArray<ILatencyMarkerModule*> LatencyMarkerModules = IModularFeatures::Get().GetModularFeatureImplementations<ILatencyMarkerModule>(ILatencyMarkerModule::GetModularFeatureName());

	for (ILatencyMarkerModule* LatencyMarkerModule : LatencyMarkerModules)
	{
		LatencyMarkerModule->SetFlashIndicatorLatencyMarker(FrameNumber);
	}
}

void UEngine::UpdateTimeAndHandleMaxTickRate()
{
	PumpABTest();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FTimedMemReport::Get().PumpTimedMemoryReports();
#endif

	if (CustomTimeStep)
	{
		bool bRunEngineCode = CustomTimeStep->UpdateTimeStep(this);
		if (!bRunEngineCode)
		{
			UpdateTimecode();
			FApp::SetGameTime(FApp::GetGameTime() + FApp::GetDeltaTime());
			return;
		}
	}

	// This is always in realtime and is not adjusted by fixed framerate. Start slightly below current real time
	static double LastRealTime = FPlatformTime::Seconds() - 0.0001;
	static bool bTimeWasManipulated = false;
	bool bTimeWasManipulatedDebug = bTimeWasManipulated;	//Just used for logging of previous frame

															// Figure out whether we want to use real or fixed time step.
#if WITH_FIXED_TIME_STEP_SUPPORT
	const bool bUseFixedTimeStep = FApp::IsBenchmarking() || FApp::UseFixedTimeStep();
#else
	const bool bUseFixedTimeStep = false;
#endif
	static bool bPreviousUseFixedTimeStep = bUseFixedTimeStep;

	// Updates logical last time to match logical current time from last tick
	FApp::UpdateLastTime();

	// Calculate delta time and update time.
	if( bUseFixedTimeStep )
	{
		// In fixed time step we set the real times to the logical time, this causes it to lie about how fast it is going
		bTimeWasManipulated = true;
		const float FrameRate = FApp::GetFixedDeltaTime();
		FApp::SetDeltaTime(FrameRate);
		LastRealTime = FApp::GetCurrentTime();
		FApp::SetCurrentTime(FApp::GetCurrentTime() + FApp::GetDeltaTime());
	}
	else
	{
		// Updates logical time to real time, this may be changed by fixed frame rate below
		double CurrentRealTime = FPlatformTime::Seconds();
		FApp::SetCurrentTime(CurrentRealTime);

		// Did we just switch from a fixed time step to real-time?  If so, then we'll update our
		// cached 'last time' so our current interval isn't huge (or negative!)
		if (bUseFixedTimeStep != bPreviousUseFixedTimeStep)
		{
			if (bUseFixedFrameRate)
			{
				LastRealTime = CurrentRealTime - (1.0 / FixedFrameRate);
			}
			else
			{
				LastRealTime = CurrentRealTime - FApp::GetDeltaTime();
			}
			bTimeWasManipulated = false;
		}

		// Calculate delta time, this is in real time seconds
		float DeltaRealTime = CurrentRealTime - LastRealTime;

		// Negative delta time means something is wrong with the system. Error out so user can address issue.
		if(DeltaRealTime < 0 )
		{
			DeltaRealTime = CorrectNegativeTimeDelta(DeltaRealTime);
		}

		// Give engine chance to update frame rate smoothing
		UpdateRunningAverageDeltaTime(DeltaRealTime);

		// Get max tick rate based on network settings and current delta time.
		const float GivenMaxTickRate = GetMaxTickRate(DeltaRealTime);
		const float MaxTickRate = FABTest::StaticIsActive() ? 0.0f : (bUseFixedFrameRate ? FixedFrameRate : GivenMaxTickRate);
		float WaitTime		= 0;
		// Convert from max FPS to wait time.
		if( MaxTickRate > 0 )
		{
			WaitTime = FMath::Max( 1.f / MaxTickRate - DeltaRealTime, 0.f );
		}

		bool bMaxTickRateHandled = false;
		TArray<IMaxTickRateHandlerModule*> MaxTickRateHandlerModules = IModularFeatures::Get().GetModularFeatureImplementations<IMaxTickRateHandlerModule>(IMaxTickRateHandlerModule::GetModularFeatureName());

		for (IMaxTickRateHandlerModule* MaxTickRateHandler : MaxTickRateHandlerModules)
		{
			if (MaxTickRateHandler->HandleMaxTickRate(MaxTickRate))
			{
				bMaxTickRateHandled = true;
				break;
			}
		}

		// Enforce maximum framerate and smooth framerate by waiting.
		double ActualWaitTime = 0.f;
		if( WaitTime > 0 )
		{
			// track all this waiting so that Game Thread is correct
			UE::Stats::FThreadIdleStats::FScopeIdle Scope;

			FSimpleScopeSecondsCounter ActualWaitTimeCounter(ActualWaitTime);
			double WaitEndTime = CurrentRealTime + WaitTime;

			SCOPE_CYCLE_COUNTER(STAT_GameTickWaitTime);
			SCOPE_CYCLE_COUNTER(STAT_GameIdleTime);

			if (!bMaxTickRateHandled)
			{
				if (IsRunningDedicatedServer()) // We aren't so concerned about wall time with a server, lots of CPU is wasted spinning. I suspect there is more to do with sleeping and time on dedicated servers.
				{
					FPlatformProcess::SleepNoStats(WaitTime);
				}
				else
				{
					// Sleep if we're waiting more than 5 ms. We set the scheduler granularity to 1 ms
					// at startup on PC. We reserve 2 ms of slack time which we will wait for by giving
					// up our timeslice.
					if (WaitTime > 5 / 1000.f)
					{
						// For improved handling of drag and drop, continue to pump messages while throttled down
						if (GIsEditor && ShouldThrottleCPUUsage())
						{
							do
							{
								FPlatformProcess::SleepNoStats(0.005f);
								FPlatformApplicationMisc::PumpMessages(true);
							} while (ShouldThrottleCPUUsage() && FPlatformTime::Seconds() < (WaitEndTime - 0.005f));
						}
						else
						{
							FPlatformProcess::SleepNoStats(WaitTime - 0.002f);
						}
					}

					// Give up timeslice for remainder of wait time.
					while (FPlatformTime::Seconds() < WaitEndTime)
					{
						FPlatformProcess::SleepNoStats(0);
					}
				}
			}
			CurrentRealTime = FPlatformTime::Seconds();

			if(bUseFixedFrameRate)
			{
				// We are on fixed framerate but had to delay, we set the current time with a fixed time step, which will set Delta below
				const double FrameTime = 1.0 / FixedFrameRate;
				FApp::SetCurrentTime(LastRealTime + FrameTime);
				bTimeWasManipulated = true;
			}
			else
			{
				FApp::SetCurrentTime(CurrentRealTime);
			}
		}
		else if(bUseFixedFrameRate && MaxTickRate == FixedFrameRate)
		{
			// We are doing fixed framerate and the real delta time is bigger than our desired delta time. In this case we start falling behind real time (and that's ok)
			const double FrameTime = 1.0 / FixedFrameRate;
			FApp::SetCurrentTime(LastRealTime + FrameTime);
			bTimeWasManipulated = true;
		}

		SET_FLOAT_STAT(STAT_GameTickWantedWaitTime,WaitTime * 1000.f);
		double AdditionalWaitTime = ActualWaitTime - static_cast<double>(WaitTime);
		double AdditionalWaitTimeInMs = AdditionalWaitTime * 1000.0;
		SET_FLOAT_STAT(STAT_GameTickAdditionalWaitTime,FMath::Max<float>(static_cast<float>(AdditionalWaitTimeInMs),0.f));

		// Update logical delta time based on logical current time
		FApp::SetDeltaTime(FApp::GetCurrentTime() - LastRealTime);
		FApp::SetIdleTime(ActualWaitTime);
		FApp::SetIdleTimeOvershoot(FMath::Max(0.0, AdditionalWaitTime)); // don't report a negative value.

																		 // Negative delta time means something is wrong with the system. Error out so user can address issue.
		if( FApp::GetDeltaTime() < 0 )
		{
			UE_LOGF(LogEngine, Warning, "Detected negative delta time - ignoring");
			FApp::SetDeltaTime(0.01);
		}

		LastRealTime = CurrentRealTime;

		// Enforce a maximum delta time if wanted.
		UGameEngine* GameEngine = Cast<UGameEngine>(this);
		const float MaxDeltaTime = GameEngine ? GameEngine->MaxDeltaTime : 0.f;
		if( MaxDeltaTime > 0.f )
		{
			UWorld* World = NULL;

			int32 NumGamePlayers = 0;
			for (int32 WorldIndex = 0; WorldIndex < WorldList.Num(); ++WorldIndex)
			{
				if (WorldList[WorldIndex].WorldType == EWorldType::Game && WorldList[WorldIndex].OwningGameInstance != NULL)
				{
					World = WorldList[WorldIndex].World();
					NumGamePlayers = WorldList[WorldIndex].OwningGameInstance->GetNumLocalPlayers();
					break;
				}
			}

			// We don't want to modify delta time if we are dealing with network clients as either host or client.
			if( World != NULL
				// Not having a game info implies being a client.
				&& ( ( World->GetAuthGameMode() != NULL
					// NumPlayers and GamePlayer only match in standalone game types and handles the case of splitscreen.
					&&	World->GetAuthGameMode()->GetNumPlayers() == NumGamePlayers ) ) )
			{
				// Happy clamping!
				FApp::SetDeltaTime(FMath::Min<double>(FApp::GetDeltaTime(), MaxDeltaTime));
			}
		}
	}

	bPreviousUseFixedTimeStep = bUseFixedTimeStep;

#if !UE_BUILD_SHIPPING
	{
		float OverrideFPS = CVarSetOverrideFPS.GetValueOnGameThread();
		if(OverrideFPS >= 0.001f)
		{
			// in seconds
			FApp::SetDeltaTime(1.0f / OverrideFPS);
			LastRealTime = FApp::GetCurrentTime();
			FApp::SetCurrentTime(FApp::GetCurrentTime() + FApp::GetDeltaTime());
			bTimeWasManipulated = true;
		}
	}
#endif // !UE_BUILD_SHIPPING

	UpdateTimecode();

	FApp::SetGameTime(FApp::GetGameTime() + FApp::GetDeltaTime());
}

void UEngine::ReinitializeCustomTimeStep()
{
	if (CustomTimeStep)
	{
		if (bIsCurrentCustomTimeStepInitialized)
		{
		CustomTimeStep->Shutdown(this);
		}
		bIsCurrentCustomTimeStepInitialized = CustomTimeStep->Initialize(this);
	}
}

bool UEngine::SetCustomTimeStep(UEngineCustomTimeStep* InCustomTimeStep)
{
	if (InCustomTimeStep != CustomTimeStep)
	{
		if (CustomTimeStep && bIsCurrentCustomTimeStepInitialized)
		{
			CustomTimeStep->Shutdown(this);
		}

		bIsCurrentCustomTimeStepInitialized = false;
		CustomTimeStep = IsValid(InCustomTimeStep) ? InCustomTimeStep : nullptr;

		if (CustomTimeStep)
		{
			bIsCurrentCustomTimeStepInitialized = CustomTimeStep->Initialize(this);
		}
		OnCustomTimeStepChanged().Broadcast();
	}
	return bIsCurrentCustomTimeStepInitialized;
}

void UEngine::ReinitializeTimecodeProvider()
{
	UTimecodeProvider* Provider = GetTimecodeProvider();
	if (Provider)
	{
		if (bIsCurrentTimecodeProviderInitialized)
		{
			Provider->Shutdown(this);
		}
		bIsCurrentTimecodeProviderInitialized = Provider->Initialize(this);
	}
}

bool UEngine::SetTimecodeProvider(UTimecodeProvider* InTimecodeProvider)
{
	if (InTimecodeProvider != TimecodeProvider)
	{
		if (TimecodeProvider && bIsCurrentTimecodeProviderInitialized)
		{
			TimecodeProvider->Shutdown(this);
		}

		bIsCurrentTimecodeProviderInitialized = false;
		TimecodeProvider = IsValid(InTimecodeProvider) ? InTimecodeProvider : nullptr;

		if (TimecodeProvider)
		{
			bIsCurrentTimecodeProviderInitialized = TimecodeProvider->Initialize(this);
		}
		OnTimecodeProviderChanged().Broadcast();
	}
	return bIsCurrentTimecodeProviderInitialized;
}

void UEngine::UpdateTimecode()
{
	FApp::InvalidateCurrentFrameTime();

	if (UTimecodeProvider* Provider = GetTimecodeProvider())
	{
		Provider->FetchAndUpdate();

		if (Provider->GetSynchronizationState() == ETimecodeProviderSynchronizationState::Synchronized)
	{
		FApp::SetCurrentFrameTime(Provider->GetDelayedQualifiedFrameTime());
	}
	}
}

void UEngine::ParseCommandline()
{
	if( FParse::Param( FCommandLine::Get(), TEXT("noailogging")) )
	{
		bDisableAILogging = true;
	}

	if( FParse::Param( FCommandLine::Get(), TEXT("enableailogging")) )
	{
		bDisableAILogging = false;
	}

#if !UE_BUILD_SHIPPING
	uint64 MaxAllocVal = 0;

	if (FParse::Value(FCommandLine::Get(), TEXT("MaxAlloc="), MaxAllocVal))
	{
		FMalloc::MaxSingleAlloc = MaxAllocVal;
	}
#endif
}


/**
* Loads a special material and verifies that it is marked as a special material (some shaders
* will only be compiled for materials marked as "special engine material")
*
* @param MaterialName Variable name of the material in the engine to identify which material can't be loaded
* @param MaterialNamePath Fully qualified name of a material to load/find
* @param Material Reference to a material object pointer that will be filled out
* @param bCheckUsage Check if the material has been marked to be used as a special engine material
*/
template <typename T>
void LoadSpecialMaterial(const FString& MaterialName, const FString& MaterialNamePath, T& Material, bool bCheckUsage)
{
	// only bother with materials that aren't already loaded
	if (Material == NULL)
	{
		// find or load the object
		Material = LoadObject<UMaterial>(NULL, *MaterialNamePath, NULL, LOAD_None, NULL);

		if (!Material)
		{
#if !WITH_EDITORONLY_DATA
			UE_LOGF(LogEngine, Log, "ERROR: Failed to load special material '%ls' from path '%ls'. This will probably have bad consequences (depending on its use).", *MaterialName, *MaterialNamePath);
#else
			UE_LOGF(LogEngine, Fatal, "Failed to load special material '%ls' from path '%ls'.", *MaterialName, *MaterialNamePath);
#endif
		}
		// if the material wasn't marked as being a special engine material, then not all of the shaders 
		// will have been compiled on it by this point, so we need to compile them and alert the use
		// to set the bit
		else if (!Material->bUsedAsSpecialEngineMaterial && bCheckUsage) 
		{
#if !WITH_EDITOR
			// consoles must have the flag set properly in the editor
			UE_LOGF(LogEngine, Fatal,"The special material (%ls) was not marked with bUsedAsSpecialEngineMaterial. Make sure this flag is set in the editor, save the package, and compile shaders for this platform", *MaterialNamePath);
#else
			Material->bUsedAsSpecialEngineMaterial = true;
			Material->MarkPackageDirty();

			// make sure all necessary shaders for the default are compiled, now that the flag is set
			Material->PostEditChange();

			FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("Engine", "SpecialMaterialConfiguredIncorrectly", "The special material ({0}) has not been marked with bUsedAsSpecialEngineMaterial.\nThis will prevent shader precompiling properly, so the flag has been set automatically.\nMake sure to save the package and distribute to everyone using this material."), FText::FromString( MaterialNamePath ) ) );
#endif
		}
	}
}

/**
* Loads a special material interface and, in editor builds - verifies that it can be loaded,
* while reporting only a log message only in non-editor builds.
*
* @param MaterialName Variable name of the material in the engine to identify which material can't be loaded
* @param MaterialNamePath Fully qualified name of a material to load/find
*/
UMaterialInterface* LoadSpecialMaterialInterface(const FString& MaterialName, const FString& MaterialNamePath)
{
	UMaterialInterface* Material = nullptr;

	// find or load the object
	Material = LoadObject<UMaterialInterface>(NULL, *MaterialNamePath, NULL, LOAD_None, NULL);

	if (!Material)
	{
#if !WITH_EDITORONLY_DATA
		UE_LOGF(LogEngine, Log, "ERROR: Failed to load special material '%ls' from path '%ls'. This will probably have bad consequences (depending on its use).", *MaterialName, *MaterialNamePath);
#else
		UE_LOGF(LogEngine, Fatal, "Failed to load special material '%ls' from path '%ls'.", *MaterialName, *MaterialNamePath);
#endif
	}

	return Material;
}

template<typename ClassType>
void LoadEngineClass(FSoftClassPath& ClassName, TSubclassOf<ClassType>& EngineClassRef)
{
	if ( EngineClassRef == nullptr )
	{
		EngineClassRef = LoadClass<ClassType>(nullptr, *ClassName.ToString());

		// try redirects
		if (EngineClassRef == nullptr)
		{
			ClassName.FixupCoreRedirects();
			EngineClassRef = LoadClass<ClassType>(nullptr, *ClassName.ToString());
		}

		if (EngineClassRef == nullptr)
		{
			EngineClassRef = ClassType::StaticClass();
			UE_LOGF(LogEngine, Error, "Failed to load '%ls', falling back to '%ls'", *ClassName.ToString(), *EngineClassRef->GetName());
		}
	}
}

/** Special function that loads an engine texture and adds it to root set in cooked builds. 
  * This is to prevent the textures from being added to GC clusters (root set objects are not clustered) 
	* because otherwise since they're always going to be referenced by UUnrealEngine object which will 
	* prevent the clusters from being GC'd. */
template <typename TextureType>
static void LoadEngineTexture(TextureType*& InOutTexture, const TCHAR* InName)
{
	if (!InOutTexture)
	{
		InOutTexture = LoadObject<TextureType>(nullptr, InName, nullptr, LOAD_None, nullptr);
	}
	if (FPlatformProperties::RequiresCookedData() && InOutTexture)
	{
		InOutTexture->AddToRoot();
	}
}

template <typename TextureType>
static void LoadEngineTexture(TObjectPtr<TextureType>& InOutTexture, const TCHAR* InName)
{
	// Fully copied from specialization above to allow for mutable pointer access without relying on deprecated explicit cast to mutable
	if (!InOutTexture)
	{
		InOutTexture = LoadObject<TextureType>(nullptr, InName, nullptr, LOAD_None, nullptr);
	}
	if (FPlatformProperties::RequiresCookedData() && InOutTexture)
	{
		InOutTexture->AddToRoot();
	}
}

static void LoadCustomTimeStep(UEngine* Engine)
{
	if (Engine->CustomTimeStepClassName.IsValid())
	{
		if (UClass* CustomTimeStepClass = Engine->CustomTimeStepClassName.TryLoadClass<UEngineCustomTimeStep>())
		{
			UEngineCustomTimeStep* NewCustomTimeStep = NewObject<UEngineCustomTimeStep>(Engine, CustomTimeStepClass);
			if (!Engine->SetCustomTimeStep(NewCustomTimeStep))
			{
				UE_LOGF(LogEngine, Warning, "InitializeCustomTimeStep - Failed to intialize CustomTimeStep '%ls'.", *NewCustomTimeStep->GetPathName());
			}
		}
		else
		{
			Engine->SetCustomTimeStep(nullptr);
			UE_LOGF(LogEngine, Error, "Engine config value CustomTimeStepClassName '%ls' is not a valid class name.", *Engine->CustomTimeStepClassName.ToString());
		}
	}
	else
	{
		Engine->SetCustomTimeStep(nullptr);
	}
}

static void ResetCustomTimeStep()
{
	LoadCustomTimeStep(GEngine);
}

static FAutoConsoleCommand GResetCustomTimeStep(
	TEXT("CustomTimeStep.reset"),
	TEXT("Resets the current custom step."),
	FConsoleCommandDelegate::CreateStatic(&ResetCustomTimeStep));


static void LoadTimecodeProvider(UEngine* Engine)
{
	if (Engine->TimecodeProviderClassName.IsValid())
{
		if (UClass* TimecodeProviderClass = Engine->TimecodeProviderClassName.TryLoadClass<UTimecodeProvider>())
	{
			UTimecodeProvider* NewTimecodeProvider = NewObject<UTimecodeProvider>(Engine, TimecodeProviderClass);
			if (!Engine->SetTimecodeProvider(NewTimecodeProvider))
		{
				UE_LOGF(LogEngine, Warning, "InitializeTimecodeProvider - Failed to intialize TimecodeProvider '%ls'.", *NewTimecodeProvider->GetPathName());
			}
		}
		else
		{
			Engine->SetTimecodeProvider(nullptr);
			UE_LOGF(LogEngine, Error, "Engine config value TimecodeProviderClassName '%ls' is not a valid class name.", *Engine->TimecodeProviderClassName.ToString());
		}
	}
	else if (Engine->bGenerateDefaultTimecode)
	{
		FName ObjectName = MakeUniqueObjectName(Engine, USystemTimeTimecodeProvider::StaticClass(), "DefaultTimecodeProvider");
		USystemTimeTimecodeProvider* NewTimecodeProvider = NewObject<USystemTimeTimecodeProvider>(Engine, ObjectName);
		NewTimecodeProvider->FrameRate = Engine->GenerateDefaultTimecodeFrameRate;
		NewTimecodeProvider->FrameDelay = Engine->GenerateDefaultTimecodeFrameDelay;
		NewTimecodeProvider->bGenerateFullFrame = false;
#if PLATFORM_DESKTOP
		NewTimecodeProvider->bUseHighPerformanceClock = false;
#else
		//If a user wishes to have an accurate TC value on console, they should set an accurate TC provider in their project settings.
		NewTimecodeProvider->bUseHighPerformanceClock = true;
#endif
		Engine->SetTimecodeProvider(NewTimecodeProvider);
	}
	else
	{
		Engine->SetTimecodeProvider(nullptr);
	}
}

static void ResetTimecode()
{
	LoadTimecodeProvider(GEngine);
}

static FAutoConsoleCommand GResetTimecode(
	TEXT("TimecodeProvider.reset"),
	TEXT("Resets the current timecode provider."),
	FConsoleCommandDelegate::CreateStatic(&ResetTimecode));

/**
* Loads all Engine object references from their corresponding config entries.
*/
void UEngine::InitializeObjectReferences()
{
	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Object References"));
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("UEngine::InitializeObjectReferences"));
	SCOPED_BOOT_TIMING("UEngine::InitializeObjectReferences");
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UEngine::InitializeObjectReferences"), STAT_InitializeObjectReferences, STATGROUP_LoadTime);

	// This initializes the tag data if it hasn't been already, we need to do this before loading any game data
	UGameplayTagsManager::Get();

	// initialize the special engine/editor materials
	if (AllowDebugViewmodes())
	{
		// Materials that are needed in-game if debug viewmodes are allowed
		LoadSpecialMaterial(TEXT("WireframeMaterialName"), WireframeMaterialName, WireframeMaterial, true);
		LoadSpecialMaterial(TEXT("LevelColorationLitMaterialName"), LevelColorationLitMaterialName, LevelColorationLitMaterial, true);
		LoadSpecialMaterial(TEXT("LevelColorationUnlitMaterialName"), LevelColorationUnlitMaterialName, LevelColorationUnlitMaterial, true);
		LoadSpecialMaterial(TEXT("LightingTexelDensityName"), LightingTexelDensityName, LightingTexelDensityMaterial, false);
		LoadSpecialMaterial(TEXT("ShadedLevelColorationLitMaterialName"), ShadedLevelColorationLitMaterialName, ShadedLevelColorationLitMaterial, true);
		LoadSpecialMaterial(TEXT("ShadedLevelColorationUnlitMaterialName"), ShadedLevelColorationUnlitMaterialName, ShadedLevelColorationUnlitMaterial, true);
		LoadSpecialMaterial(TEXT("VertexColorMaterialName"), VertexColorMaterialName, VertexColorMaterial, false);
		LoadSpecialMaterial(TEXT("VertexColorViewModeMaterialName_ColorOnly"), VertexColorViewModeMaterialName_ColorOnly, VertexColorViewModeMaterial_ColorOnly, false);
		LoadSpecialMaterial(TEXT("VertexColorViewModeMaterialName_AlphaAsColor"), VertexColorViewModeMaterialName_AlphaAsColor, VertexColorViewModeMaterial_AlphaAsColor, false);
		LoadSpecialMaterial(TEXT("VertexColorViewModeMaterialName_RedOnly"), VertexColorViewModeMaterialName_RedOnly, VertexColorViewModeMaterial_RedOnly, false);
		LoadSpecialMaterial(TEXT("VertexColorViewModeMaterialName_GreenOnly"), VertexColorViewModeMaterialName_GreenOnly, VertexColorViewModeMaterial_GreenOnly, false);
		LoadSpecialMaterial(TEXT("VertexColorViewModeMaterialName_BlueOnly"), VertexColorViewModeMaterialName_BlueOnly, VertexColorViewModeMaterial_BlueOnly, false);
		LoadSpecialMaterial(TEXT("TextureColorViewModeMaterialName"), TextureColorViewModeMaterialName, TextureColorViewModeMaterial, false);
	}

	// Nanite materials
	LoadSpecialMaterial(TEXT("NaniteHiddenSectionMaterialName"), NaniteHiddenSectionMaterialName, NaniteHiddenSectionMaterial, false);

	// Materials that may or may not be needed when debug viewmodes are disabled but haven't been fixed up yet
	LoadSpecialMaterial(TEXT("RemoveSurfaceMaterialName"), RemoveSurfaceMaterialName.ToString(), RemoveSurfaceMaterial, false);

	// these one's are needed both editor and standalone 
	LoadSpecialMaterial(TEXT("DebugMeshMaterialName"), DebugMeshMaterialName.ToString(), DebugMeshMaterial, false);
	LoadSpecialMaterial(TEXT("EmissiveMeshMaterialName"), EmissiveMeshMaterialName.ToString(), EmissiveMeshMaterial, false);
	LoadSpecialMaterial(TEXT("InvalidLightmapSettingsMaterialName"), InvalidLightmapSettingsMaterialName.ToString(), InvalidLightmapSettingsMaterial, false);
	LoadSpecialMaterial(TEXT("ArrowMaterialName"), ArrowMaterialName.ToString(), ArrowMaterial, false);

#if !UE_BUILD_SHIPPING || WITH_EDITORONLY_DATA
	ArrowMaterialYellow = UMaterialInstanceDynamic::Create(ArrowMaterial, nullptr);
	ArrowMaterialYellow->SetVectorParameterValue("GizmoColor", FLinearColor::Yellow);

	LoadSpecialMaterial(TEXT("ConstraintLimitMaterialName"), TEXT("/Engine/EngineMaterials/PhAT_JointLimitMaterial.PhAT_JointLimitMaterial"), ConstraintLimitMaterial, false);

	ConstraintLimitMaterialX = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialX->SetVectorParameterValue(FName("Color"), FLinearColor::Red);
	ConstraintLimitMaterialX->SetScalarParameterValue(FName("Desaturation"), 0.6f);
	ConstraintLimitMaterialXAxis = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialXAxis->SetVectorParameterValue(FName("Color"), FLinearColor::Red);

	ConstraintLimitMaterialY = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialY->SetVectorParameterValue(FName("Color"), FLinearColor::Green);
	ConstraintLimitMaterialY->SetScalarParameterValue(FName("Desaturation"), 0.6f);
	ConstraintLimitMaterialYAxis = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialYAxis->SetVectorParameterValue(FName("Color"), FLinearColor::Green);

	ConstraintLimitMaterialZ = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialZ->SetVectorParameterValue(FName("Color"), FLinearColor::Blue);
	ConstraintLimitMaterialZ->SetScalarParameterValue(FName("Desaturation"), 0.6f);
	ConstraintLimitMaterialZAxis = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialZAxis->SetVectorParameterValue(FName("Color"), FLinearColor::Blue);

	ConstraintLimitMaterialPrismatic = UMaterialInstanceDynamic::Create(ConstraintLimitMaterial, NULL);
	ConstraintLimitMaterialPrismatic->SetVectorParameterValue(FName("Color"), FLinearColor(FColor::Orange));
#endif


	if (GIsEditor && !IsRunningCommandlet())
	{
		// Materials that are only needed in the interactive editor
#if WITH_EDITORONLY_DATA
		LoadSpecialMaterial(TEXT("GeomMaterialName"), GeomMaterialName.ToString(), GeomMaterial, false);
		LoadSpecialMaterial(TEXT("EditorBrushMaterialName"), EditorBrushMaterialName.ToString(), EditorBrushMaterial, false);
		LoadSpecialMaterial(TEXT("BoneWeightMaterialName"), BoneWeightMaterialName.ToString(), BoneWeightMaterial, false);
		LoadSpecialMaterial(TEXT("ClothPaintMaterialName"), ClothPaintMaterialName.ToString(), ClothPaintMaterial, false);
		LoadSpecialMaterial(TEXT("ClothPaintMaterialWireframeName"), ClothPaintMaterialWireframeName.ToString(), ClothPaintMaterialWireframe, false);
		LoadSpecialMaterial(TEXT("ClothPaintOpaqueMaterialName"), ClothPaintOpaqueMaterialName.ToString(), ClothPaintOpaqueMaterial, false);
		LoadSpecialMaterial(TEXT("ClothPaintOpaqueMaterialWireframeName"), ClothPaintOpaqueMaterialWireframeName.ToString(), ClothPaintOpaqueMaterialWireframe, false);
		LoadSpecialMaterial(TEXT("DebugEditorMaterialName"), DebugEditorMaterialName.ToString(), DebugEditorMaterial, false);
		LoadSpecialMaterial(TEXT("PhysicalMaterialMaskMaterialName"), PhysicalMaterialMaskMaterialName.ToString(), PhysicalMaterialMaskMaterial, false);
		LoadSpecialMaterial(TEXT("TexturePaintingMaskMaterialName"), TexturePaintingMaskMaterialName.ToString(), TexturePaintingMaskMaterial, false);

		ClayMaterial = LoadSpecialMaterialInterface(TEXT("ClayMaterialName"), ClayMaterialName.ToString());
		LoadSpecialMaterial(TEXT("ZebraMaterialName"), ZebraMaterialName.ToString(), ZebraMaterial, false);
		ZebraMaterialInstance = UMaterialInstanceDynamic::Create(ZebraMaterial, nullptr);
		FrontBackFaceMaterial = LoadSpecialMaterialInterface(TEXT("FrontBackFaceMaterialName"), FrontBackFaceMaterialName.ToString());
		RandomColorMaterial = LoadSpecialMaterialInterface(TEXT("RandomColorMaterialName"), RandomColorMaterialName.ToString());

		ClothPaintMaterialInstance = UMaterialInstanceDynamic::Create(ClothPaintMaterial, nullptr);
		ClothPaintMaterialWireframeInstance = UMaterialInstanceDynamic::Create(ClothPaintMaterialWireframe, nullptr);
		ClothPaintOpaqueMaterialInstance = UMaterialInstanceDynamic::Create(ClothPaintOpaqueMaterial, nullptr);
		ClothPaintOpaqueMaterialWireframeInstance = UMaterialInstanceDynamic::Create(ClothPaintOpaqueMaterialWireframe, nullptr);
#endif
		FString ValidPreviewShadowsIndicatorMaterialName = PreviewShadowsIndicatorMaterialName.ToString();
		if (ValidPreviewShadowsIndicatorMaterialName.IsEmpty())
		{
			UE_LOGF(LogEngine, Warning, "The default Preview Shadow Indicator material was not found. Using default Preview Shadow Indicator material instead. Please make sure to have the default Preview Shadow Indicator material set up correctly.");
			ValidPreviewShadowsIndicatorMaterialName = TEXT("/Engine/EditorMaterials/PreviewShadowIndicatorMaterial.PreviewShadowIndicatorMaterial");
		}
		LoadSpecialMaterial(TEXT("PreviewShadowsIndicatorMaterialName"), ValidPreviewShadowsIndicatorMaterialName, PreviewShadowsIndicatorMaterial, false);

		//@TODO: This should move into the editor (used in editor modes exclusively)
		if (DefaultBSPVertexTexture == NULL)
		{
			DefaultBSPVertexTexture = LoadObject<UTexture2D>(NULL, *DefaultBSPVertexTextureName.ToString(), NULL, LOAD_None, NULL);
		}
	}

#if WITH_EDITORONLY_DATA
	if (GIsEditor)
	{
		LoadSpecialMaterial(TEXT("DefaultFlattenMaterialName"), DefaultFlattenMaterialName.ToString(), DefaultFlattenMaterial, false);
		LoadSpecialMaterial(TEXT("DefaultHLODFlattenMaterialName"), DefaultHLODFlattenMaterialName.ToString(), DefaultHLODFlattenMaterial, false);
		LoadSpecialMaterial(TEXT("DefaultLandscapeFlattenMaterialName"), DefaultLandscapeFlattenMaterialName.ToString(), DefaultLandscapeFlattenMaterial, false);
	}
#endif

	LoadEngineTexture(DefaultTexture, *DefaultTextureName.ToString());
	LoadEngineTexture(DefaultDiffuseTexture, *DefaultDiffuseTextureName.ToString());
	LoadEngineTexture(HighFrequencyNoiseTexture, *HighFrequencyNoiseTextureName.ToString());
	LoadEngineTexture(DefaultBokehTexture, *DefaultBokehTextureName.ToString());
	LoadEngineTexture(MiniFontTexture, *MiniFontTextureName.ToString());
	LoadEngineTexture(WeightMapPlaceholderTexture, *WeightMapPlaceholderTextureName.ToString());
	LoadEngineTexture(WeightMapArrayPlaceholderTexture,  *WeightMapArrayPlaceholderTextureName.ToString());
	LoadEngineTexture(LightMapDensityTexture, *LightMapDensityTextureName.ToString());
	ConditionallyLoadPreIntegratedSkinBRDFTexture();
	LoadLTCTextures();
	LoadEnergyTextures();

#if WITH_EDITOR
	// Avoid breaking some engine textures that might be cached very early (i.e. BlueNoise)
	FTextureCompilingManager::Get().FinishAllCompilation();
#endif

	if ( DefaultPhysMaterial == NULL )
	{
		DefaultPhysMaterial = LoadObject<UPhysicalMaterial>(NULL, *DefaultPhysMaterialName.ToString(), NULL, LOAD_None, NULL);
		if(!DefaultPhysMaterial)
		{
			UE_LOGF(LogEngine, Error, "The default physical material (%ls) was not found. Please make sure you have your default physical material set up correctly.", *DefaultPhysMaterialName.ToString());
			DefaultPhysMaterial = NewObject<UPhysicalMaterial>();
		}
	}
	
	UPhysicalMaterial::SetEngineDefaultPhysMaterial(DefaultPhysMaterial);

	if (DefaultDestructiblePhysMaterial == NULL)
	{
		FString ValidDefaultDestructiblePhysMaterialName = DefaultDestructiblePhysMaterialName.ToString();
		if (ValidDefaultDestructiblePhysMaterialName.IsEmpty())
		{
			ValidDefaultDestructiblePhysMaterialName = TEXT("/Engine/EngineMaterials/DefaultDestructiblePhysicalMaterial.DefaultDestructiblePhysicalMaterial");
		}

		DefaultDestructiblePhysMaterial = LoadObject<UPhysicalMaterial>(NULL, *ValidDefaultDestructiblePhysMaterialName, NULL, LOAD_None, NULL);
		if (!DefaultDestructiblePhysMaterial)
		{
			UE_LOGF(LogEngine, Error, "The default physical material for destruction (%ls) was not found. Please make sure you have a default physical material fo destruction set up correctly.", *ValidDefaultDestructiblePhysMaterialName);
			DefaultDestructiblePhysMaterial = NewObject<UPhysicalMaterial>();
		}
	}

	UPhysicalMaterial::SetEngineDefaultDestructiblePhysMaterial(DefaultPhysMaterial);

	LoadEngineClass<UConsole>(ConsoleClassName, ConsoleClass);
	LoadEngineClass<UGameViewportClient>(GameViewportClientClassName, GameViewportClientClass);
	LoadEngineClass<ULocalPlayer>(LocalPlayerClassName, LocalPlayerClass);
	LoadEngineClass<AWorldSettings>(WorldSettingsClassName, WorldSettingsClass);
	LoadEngineClass<UNavigationSystemBase>(NavigationSystemClassName, NavigationSystemClass);
	LoadEngineClass<UNavigationSystemConfig>(NavigationSystemConfigClassName, NavigationSystemConfigClass);
	if (AvoidanceManagerClassName.IsValid())
	{
		LoadEngineClass<UAvoidanceManager>(AvoidanceManagerClassName, AvoidanceManagerClass);
	}
	LoadEngineClass<UGameUserSettings>(GameUserSettingsClassName, GameUserSettingsClass);
	LoadEngineClass<ALevelScriptActor>(LevelScriptActorClassName, LevelScriptActorClass);

	// It is valid for physics collision handler to be None
	if (PhysicsCollisionHandlerClassName.IsValid())
	{
		LoadEngineClass<UPhysicsCollisionHandler>(PhysicsCollisionHandlerClassName, PhysicsCollisionHandlerClass);
	}

	// set the font object pointers, unless on server
	if (!IsRunningDedicatedServer())
	{
		auto CreateFontObjectFromDefaultFont = [](const FName InObjectName, const FName InLegacyFontName, const int32 InLegacyFontSize) -> UFont*
		{
			UFont* FontPtr = NewObject<UFont>(GetTransientPackage(), InObjectName, RF_Transient);
			FontPtr->FontCacheType = EFontCacheType::Runtime;
			FontPtr->RuntimeFontSource = ERuntimeFontSource::CoreStyleDefault;
			FontPtr->LegacyFontName = InLegacyFontName;
			FontPtr->LegacyFontSize = InLegacyFontSize;
			return FontPtr;
		};

		TStrongObjectPtr<UFont> DefaultTinyFont;
		auto GetDefaultTinyFont = [&DefaultTinyFont, &CreateFontObjectFromDefaultFont]() -> UFont*
		{
			if (!DefaultTinyFont)
			{
				DefaultTinyFont.Reset(CreateFontObjectFromDefaultFont(FName("DefaultTinyFont"), FName("Light"), 8));
			}
			return DefaultTinyFont.Get();
		};

		TStrongObjectPtr<UFont> DefaultMonoFont;
		auto GetDefaultMonoFont = [&DefaultMonoFont, &CreateFontObjectFromDefaultFont]() -> UFont*
		{
			if (!DefaultMonoFont)
			{
				DefaultMonoFont.Reset(CreateFontObjectFromDefaultFont(FName("DefaultMonoFont"), FName("Mono"), 10));
			}
			return DefaultMonoFont.Get();
		};

		TStrongObjectPtr<UFont> DefaultRegularFont;
		auto GetDefaultRegularFont = [&DefaultRegularFont, &CreateFontObjectFromDefaultFont]() -> UFont*
		{
			if (!DefaultRegularFont)
			{
				DefaultRegularFont.Reset(CreateFontObjectFromDefaultFont(FName("DefaultRegularFont"), FName("Regular"), 10));
			}
			return DefaultRegularFont.Get();
		};

		auto ConditionalLoadEngineFont = [](UFont*& FontPtr, const FString& FontName, TFunctionRef<UFont*()> FallbackFontFactory)
		{
			if (!FontPtr)
			{
				if (FontName.Len() > 0)
				{
					FontPtr = LoadObject<UFont>(nullptr, *FontName, nullptr, LOAD_None, nullptr);
				}
				if (!FontPtr)
				{
					FontPtr = FallbackFontFactory();
				}
			}
		};

		// Standard fonts.
		ConditionalLoadEngineFont(static_cast<UFont*&>(TinyFont), TinyFontName.ToString(), GetDefaultTinyFont);
		ConditionalLoadEngineFont(static_cast<UFont*&>(SmallFont), SmallFontName.ToString(), GetDefaultRegularFont);
		ConditionalLoadEngineFont(static_cast<UFont*&>(MediumFont), MediumFontName.ToString(), GetDefaultRegularFont);
		ConditionalLoadEngineFont(static_cast<UFont*&>(LargeFont), LargeFontName.ToString(), GetDefaultRegularFont);
		ConditionalLoadEngineFont(static_cast<UFont*&>(SubtitleFont), SubtitleFontName.ToString(), GetDefaultRegularFont);
		ConditionalLoadEngineFont(static_cast<UFont*&>(MonospaceFont), MonospaceFontName.ToString(), GetDefaultMonoFont);

		// Additional fonts.
		AdditionalFonts.Empty(AdditionalFontNames.Num());
		for (const FString& FontName : AdditionalFontNames)
		{
			UFont* NewFont = nullptr;
			ConditionalLoadEngineFont(NewFont, FontName, GetDefaultRegularFont);
			AdditionalFonts.Add(NewFont);
		}
	}

	LoadCustomTimeStep(this);
	LoadTimecodeProvider(this);

	if (GameSingleton == nullptr && GameSingletonClassName.ToString().Len() > 0)
	{
		UClass *SingletonClass = LoadClass<UObject>(nullptr, *GameSingletonClassName.ToString());

		if (SingletonClass)
		{
			GameSingleton = NewObject<UObject>(this, SingletonClass);
		}
		else
		{
			UE_LOGF(LogEngine, Error, "Engine config value GameSingletonClassName '%ls' is not a valid class name.", *GameSingletonClassName.ToString());
		}
	}

	if (AssetManager == nullptr)
	{
		UClass* SingletonClass = nullptr;
		if (AssetManagerClassName.ToString().Len() > 0)
		{
			SingletonClass = LoadClass<UObject>(nullptr, *AssetManagerClassName.ToString());
		}
		if (!SingletonClass)
		{
			UE_LOGF(LogEngine, Fatal, "Engine config value AssetManagerClassName '%ls' is not a valid class name.", *AssetManagerClassName.ToString());
		}

		AssetManager = NewObject<UAssetManager>(this, SingletonClass);
		check(AssetManager);
		if (!FParse::Param(FCommandLine::Get(), TEXT("SkipAssetScan")))
		{
			AssetManager->StartInitialLoading();
		}
	}

	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>(UUserInterfaceSettings::StaticClass());
	UISettings->ForceLoadResources();
}

void UEngine::LoadDefaultBloomTexture()
{
	if (DefaultBloomKernelTexture == nullptr)
	{
		LoadEngineTexture(DefaultBloomKernelTexture, *DefaultBloomKernelTextureName.ToString());
	}
}

void UEngine::LoadBlueNoiseTexture(bool LoadVector2BlueNoiseTexture)
{
	// Always load the scalar blue noise texture.
	if (BlueNoiseScalarTexture == nullptr)
	{
		if (GetFeatureLevelShadingPath(GMaxRHIFeatureLevel) == EShadingPath::Mobile)
		{
			LoadEngineTexture(BlueNoiseScalarTexture, *BlueNoiseScalarMobileTextureName.ToString());
		}
		else
		{
			LoadEngineTexture(BlueNoiseScalarTexture, *BlueNoiseScalarTextureName.ToString());
		}
	}

	// Only load the vector2 blue noise texture if required.
	if (LoadVector2BlueNoiseTexture && BlueNoiseVec2Texture == nullptr)
	{
		LoadEngineTexture(BlueNoiseVec2Texture, *BlueNoiseVec2TextureName.ToString());
	}
}

void UEngine::LoadDefaultFilmGrainTexture()
{
	if (DefaultFilmGrainTexture == nullptr && DefaultFilmGrainTextureName.IsValid())
	{
		LoadEngineTexture(DefaultFilmGrainTexture, *DefaultFilmGrainTextureName.ToString());
	}
}

void UEngine::ConditionallyLoadPreIntegratedSkinBRDFTexture()
{
	if (PreIntegratedSkinBRDFTexture == nullptr)
	{
		uint32 ShadingModelsMask = GetPlatformShadingModelsMask(GMaxRHIShaderPlatform);
		uint32 SkinShadingMask = (1u << (uint32)MSM_PreintegratedSkin);
		if (GIsEditor || (ShadingModelsMask & SkinShadingMask) != 0)
		{
			LoadEngineTexture(PreIntegratedSkinBRDFTexture, *PreIntegratedSkinBRDFTextureName.ToString());
		}
	}
}

void UEngine::LoadLTCTextures()
{
	const bool bGGX = true;
	const bool bSheen = true;
	if (bGGX && GGXLTCAmpTexture == nullptr && GGXLTCAmpTextureName.IsValid())
	{
		LoadEngineTexture(GGXLTCAmpTexture, *GGXLTCAmpTextureName.ToString());
	}
	if (bGGX && GGXLTCMatTexture == nullptr && GGXLTCMatTextureName.IsValid())
	{
		LoadEngineTexture(GGXLTCMatTexture, *GGXLTCMatTextureName.ToString());
	}
	if (bSheen && SheenLTCTexture == nullptr && SheenLTCTextureName.IsValid())
	{
		LoadEngineTexture(SheenLTCTexture, *SheenLTCTextureName.ToString());
	}
}

void UEngine::LoadEnergyTextures()
{
	#if RHI_RAYTRACING
	static const auto CVarPT = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PathTracing"));
	const bool bPathTracingEnabled = CVarPT && CVarPT->GetValueOnAnyThread() > 0;
	#else
	const bool bPathTracingEnabled = false;
	#endif

	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();

	static const auto CVarEnergyConservation = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Material.EnergyConservation"));
	const bool bEnergyConservationEnabled = CVarEnergyConservation && CVarEnergyConservation->GetValueOnAnyThread() > 0;

	// NOTE: this logic must match GetSettings from ShadingEnergyConservation.cpp
	const bool bLoadEnergyConservationTextures = bPathTracingEnabled || bSubstrateEnabled || bEnergyConservationEnabled;

	if (bLoadEnergyConservationTextures && GGXReflectionEnergyTexture == nullptr && GGXReflectionEnergyTextureName.IsValid())
	{
		LoadEngineTexture(GGXReflectionEnergyTexture, *GGXReflectionEnergyTextureName.ToString());
	}
	if (bLoadEnergyConservationTextures && GGXMaxSpecEnergyTexture == nullptr && GGXMaxSpecEnergyTextureName.IsValid())
	{
		LoadEngineTexture(GGXMaxSpecEnergyTexture, *GGXMaxSpecEnergyTextureName.ToString());
	}

	if (bPathTracingEnabled)
	{
		if (GGXTransmissionEnergyTexture == nullptr && GGXTransmissionEnergyTextureName.IsValid())
		{
			LoadEngineTexture(GGXTransmissionEnergyTexture, *GGXTransmissionEnergyTextureName.ToString());
		}

		if (!bSubstrateEnabled && SheenEnergyTexture == nullptr && SheenLegacyEnergyTextureName.IsValid())
		{
			LoadEngineTexture(SheenEnergyTexture, *SheenLegacyEnergyTextureName.ToString());
		}

		if (bSubstrateEnabled && SheenEnergyTexture == nullptr && SheenEnergyTextureName.IsValid())
		{
			LoadEngineTexture(SheenEnergyTexture, *SheenEnergyTextureName.ToString());
		}

		if (bSubstrateEnabled && DiffuseEnergyTexture == nullptr && DiffuseEnergyTextureName.IsValid())
		{
			LoadEngineTexture(DiffuseEnergyTexture, *DiffuseEnergyTextureName.ToString());
		}
	}
}

void UEngine::LoadGlintTextures()
{
	if (GlintTexture == nullptr && GlintTextureName.IsValid())
	{
		LoadEngineTexture(GlintTexture, *GlintTextureName.ToString());
	}
	if (GlintTexture2 == nullptr && GlintTexture2Name.IsValid())
	{
		LoadEngineTexture(GlintTexture2, *GlintTexture2Name.ToString());
	}
}

void UEngine::LoadHairLUTTextures()
{
	if (HairDualScatteringTexture == nullptr && HairDualScatteringTextureName.IsValid())
	{
		LoadEngineTexture(HairDualScatteringTexture, *HairDualScatteringTextureName.ToString());
	}
	if (HairDirectionalAlbedoTexture == nullptr && HairDirectionalAlbedoTextureName.IsValid())
	{
		LoadEngineTexture(HairDirectionalAlbedoTexture, *HairDirectionalAlbedoTextureName.ToString());
	}
	if (HairCoverageTexture == nullptr && HairCoverageTextureName.IsValid())
	{
		LoadEngineTexture(HairCoverageTexture, *HairCoverageTextureName.ToString());
	}
}

void UEngine::LoadSimpleVolumeTextures()
{
	if (SimpleVolumeTexture == nullptr && SimpleVolumeTextureName.IsValid())
	{
		LoadEngineTexture(SimpleVolumeTexture, *SimpleVolumeTextureName.ToString());
	}
	if (SimpleVolumeEnvTexture == nullptr && SimpleVolumeEnvTextureName.IsValid())
	{
		LoadEngineTexture(SimpleVolumeEnvTexture, *SimpleVolumeEnvTextureName.ToString());
	}
}

void UEngine::LoadSMAATextures()
{
	if (SMAAAreaTexture == nullptr && SMAAAreaTextureName.IsValid())
	{
		LoadEngineTexture(SMAAAreaTexture, *SMAAAreaTextureName.ToString());
	}
	if (SMAASearchTexture == nullptr && SMAASearchTextureName.IsValid())
	{
		LoadEngineTexture(SMAASearchTexture, *SMAASearchTextureName.ToString());
	}
}

void UEngine::InitializePortalServices()
{
	IMessagingRpcModule* MessagingRpcModule = nullptr;
	IPortalRpcModule* PortalRpcModule = nullptr;
	IPortalServicesModule* PortalServicesModule = nullptr;

#if WITH_PORTAL_SERVICES && UE_EDITOR
	MessagingRpcModule = static_cast<IMessagingRpcModule*>(FModuleManager::Get().LoadModule("MessagingRpc"));
	PortalRpcModule = static_cast<IPortalRpcModule*>(FModuleManager::Get().LoadModule("PortalRpc"));
	PortalServicesModule = static_cast<IPortalServicesModule*>(FModuleManager::Get().LoadModule("PortalServices"));
#endif

	if (MessagingRpcModule &&
		PortalRpcModule &&
		PortalServicesModule)
	{
		// Initialize Portal services
		PortalRpcClient = MessagingRpcModule->CreateRpcClient();
		{
			// @todo gmp: catch timeouts?
		}

		PortalRpcLocator = PortalRpcModule->CreateLocator();
		{
			PortalRpcLocator->OnServerLocated().BindLambda([this]() { PortalRpcClient->Connect(PortalRpcLocator->GetServerAddress()); });
			PortalRpcLocator->OnServerLost().BindLambda([this]() { PortalRpcClient->Disconnect(); });
		}

		ServiceDependencies = MakeShareable(new FTypeContainer);
		{
			ServiceDependencies->RegisterInstance<IMessageRpcClient>(PortalRpcClient.ToSharedRef());
		}

		ServiceLocator = PortalServicesModule->CreateLocator(ServiceDependencies.ToSharedRef());
		{
			// @todo add any Engine specific Portal services here
			ServiceLocator->Configure(TEXT("IPortalApplicationWindow"), TEXT("*"), "PortalProxies");
			ServiceLocator->Configure(TEXT("IPortalUser"), TEXT("*"), "PortalProxies");
			ServiceLocator->Configure(TEXT("IPortalUserLogin"), TEXT("*"), "PortalProxies");
		}
	}
	else
	{
		class FNullPortalServiceLocator
			: public IPortalServiceLocator
		{
			virtual void Configure(const FString& ServiceName, const FWildcardString& ProductId, const FName& ServiceModule) override
			{

			}

			virtual TSharedPtr<IPortalService> GetService(const FString& ServiceName, const FString& ProductId) override
			{
				return nullptr;
			}
		};

		ServiceLocator = MakeShareable(new FNullPortalServiceLocator());
	}
}

//
// Exit the engine.
//
void UEngine::FinishDestroy()
{
	// Remove from root.
	RemoveFromRoot();

	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// shut down all subsystems.
		GEngine = NULL;
		ReleaseAudioDeviceManager();

		FURL::StaticExit();
	}

	Super::FinishDestroy();
}

void UEngine::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// count memory
	if (Ar.IsCountingMemory())
	{
		// Only use the main audio device when counting memory
		if (FAudioDeviceHandle AudioDevice = GetMainAudioDevice())
		{
			AudioDevice->CountBytes(Ar);
		}
	}
}

void UEngine::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UEngine* This = CastChecked<UEngine>(InThis);

	// track objects in all the active audio devices
	if (This->AudioDeviceManager)
	{
		This->AudioDeviceManager->AddReferencedObjects(Collector);
	}

	// TODO: This is quite dangerous as FWorldContext::AddReferencedObjects could fail to be updated when something it
	//       references changes.  Hopefully something will come along that will allow the ustruct to be provided to the
	//       collector in a property handling method
	for (FWorldContext& Context : This->WorldList)
	{
		Context.AddReferencedObjects(Collector, This);
	}

	// Keep Engine subsystems alive
	This->EngineSubsystemCollection.AddReferencedObjects(This, Collector);

	Super::AddReferencedObjects(This, Collector);
}


void UEngine::CleanupGameViewport()
{
	for (auto WorldIt = WorldList.CreateIterator(); WorldIt; ++WorldIt)
	{
		FWorldContext &Context = *WorldIt;

		if ( Context.OwningGameInstance != NULL )
		{
			Context.OwningGameInstance->CleanupGameViewport();
		}

		if ( Context.GameViewport != NULL && Context.GameViewport->Viewport == NULL )
		{
			if (Context.GameViewport == GameViewport)
			{
				GameViewport = NULL;
			}

			Context.GameViewport->DetachViewportClient();
			Context.GameViewport = NULL;
		}
	}
}

bool UEngine::IsEditor()
{
	return GIsEditor;
}


UFont* UEngine::GetTinyFont()
{
	return GEngine->TinyFont;
}


UFont* UEngine::GetSmallFont()
{
	return GEngine->SmallFont;
}


UFont* UEngine::GetMediumFont()
{
	return GEngine->MediumFont;
}

/**
* Returns the engine's default large font
*/
UFont* UEngine::GetLargeFont()
{
	return GEngine->LargeFont;
}

/**
* Returns the engine's default subtitle font
*/
UFont* UEngine::GetSubtitleFont()
{
	return GEngine->SubtitleFont;
}

/**
* Returns the engine's default monospace font
*/
UFont* UEngine::GetMonospaceFont()
{
	return GEngine->MonospaceFont;
}

/**
* Returns the specified additional font.
*
* @param	AdditionalFontIndex		Index into the AddtionalFonts array.
*/
UFont* UEngine::GetAdditionalFont(int32 AdditionalFontIndex)
{
	return GEngine->AdditionalFonts.IsValidIndex(AdditionalFontIndex) ? GEngine->AdditionalFonts[AdditionalFontIndex] : NULL;
}

class FAudioDeviceManager* UEngine::GetAudioDeviceManager()
{
	return AudioDeviceManager;
}

uint32 UEngine::GetMainAudioDeviceID() const
{
	return MainAudioDeviceHandle.GetDeviceID();
}

FAudioDeviceHandle UEngine::GetMainAudioDevice()
{
	if (AudioDeviceManager != nullptr)
	{
		return AudioDeviceManager->GetMainAudioDeviceHandle();
	}
	return FAudioDeviceHandle();
}

FAudioDevice* UEngine::GetMainAudioDeviceRaw()
{
	if (AudioDeviceManager != nullptr && MainAudioDeviceHandle.IsValid())
	{
		return AudioDeviceManager->GetMainAudioDeviceHandle().GetAudioDevice();
	}
	return nullptr;
}

FAudioDeviceHandle UEngine::GetActiveAudioDevice()
{
	if (AudioDeviceManager != nullptr)
	{
		return AudioDeviceManager->GetActiveAudioDevice();
	}
	return FAudioDeviceHandle();
}

void UEngine::InitializeAudioDeviceManager()
{
	FAudioDeviceManager::Initialize();
	AudioDeviceManager = FAudioDeviceManager::Get();
	if (AudioDeviceManager)
	{
		AudioDeviceManager->CreateMainAudioDevice();
		MainAudioDeviceHandle = AudioDeviceManager->GetMainAudioDeviceHandle();
	}
}

bool UEngine::UseSound() const
{
	return AudioDeviceManager != nullptr;
}

/**
* A fake stereo rendering device used to test stereo rendering without an attached device.
*/
class FFakeStereoRenderingDevice : public IStereoRendering, public IStereoRenderTargetManager
{
public:
	FFakeStereoRenderingDevice(int ViewportWidth = 640, int ViewportHeight = 480, int RequestedNumViews = 2) 
		: FOVInDegrees(GetDefault<UGeneralProjectSettings>()->FOVForFakeStereoRenderingDevice)
		, TopFOVRatio(GetDefault<UGeneralProjectSettings>()->TopFOVRatioForFakeStereoRenderingDevice)
		, DifferenceBetweenEyes(GetDefault<UGeneralProjectSettings>()->DifferenceBetweenEyesForFakeStereoRenderingDevice)
		, Width(ViewportWidth)
		, Height(ViewportHeight)
		, NumViews(RequestedNumViews)
		, bLayeredRTs(true)
	{
		static TAutoConsoleVariable<float> CVarEmulateStereoFOV(TEXT("r.StereoEmulationFOV"), 0, TEXT("FOV in degrees, of the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		static TAutoConsoleVariable<int32> CVarEmulateStereoWidth(TEXT("r.StereoEmulationWidth"), 0, TEXT("Width of the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		static TAutoConsoleVariable<int32> CVarEmulateStereoHeight(TEXT("r.StereoEmulationHeight"), 0, TEXT("Height of the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		static TAutoConsoleVariable<int32> CVarEmulateStereoViews(TEXT("r.StereoEmulationViews"), 0, TEXT("Number of views in the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		float FOV = CVarEmulateStereoFOV.GetValueOnAnyThread();
		if (FOV != 0)
		{
			FOVInDegrees = FMath::Clamp(FOV, 20.f, 300.f);
		}
		int32 W = CVarEmulateStereoWidth.GetValueOnAnyThread();
		int32 H = CVarEmulateStereoHeight.GetValueOnAnyThread();
		int32 V = CVarEmulateStereoViews.GetValueOnAnyThread();
		if (W != 0)
		{
			Width = FMath::Clamp(W, 100, 10000);
		}
		if (H != 0)
		{
			Height = FMath::Clamp(H, 100, 10000);
		}
		if (V != 0)
		{
			NumViews = FMath::Clamp(V, 1, 32);
		}

		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(GMaxRHIShaderPlatform);
		bLayeredRTs = Aspects.IsMobileMultiViewEnabled();
	}

	virtual ~FFakeStereoRenderingDevice() {}

	virtual bool IsStereoEnabled() const override { return true; }

	virtual bool EnableStereo(bool stereo = true) override { return true; }

	virtual int32 GetDesiredNumberOfViews(bool bStereoRequested) const { return (bStereoRequested) ? NumViews : 1; }

	virtual EStereoscopicPass GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const override
	{
		if (!bStereoRequested)
		{
			return EStereoscopicPass::eSSP_FULL;
		}
#if WITH_MGPU
		// We have to do all the work for secondary views that are on a different GPU than
		// the primary view.
		else if (GNumExplicitGPUsForRendering > 1)
		{
			return EStereoscopicPass::eSSP_PRIMARY;
		}
#endif

		return ViewIndex % 2 == 0 ? EStereoscopicPass::eSSP_PRIMARY : EStereoscopicPass::eSSP_SECONDARY;
	}

	virtual void AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override
	{
		SizeX = SizeX / 2;
		SizeY = SizeY / (NumViews / 2);
		X += SizeX * (ViewIndex % 2);
		Y += SizeY * (ViewIndex / 2);
	}

	virtual void SetFinalViewRect(class FRHICommandListImmediate& RHICmdList, const int32 ViewIndex, const FIntRect& FinalViewRect) override
	{
		static TAutoConsoleVariable<int32> CVarEmulateStereoWidth(TEXT("r.StereoEmulationWidth"), 0, TEXT("Width of the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		static TAutoConsoleVariable<int32> CVarEmulateStereoHeight(TEXT("r.StereoEmulationHeight"), 0, TEXT("Height of the imaginable HMD for stereo emulation"), ECVF_ReadOnly);
		int32 W = CVarEmulateStereoWidth.GetValueOnAnyThread();
		int32 H = CVarEmulateStereoHeight.GetValueOnAnyThread();
		
		bool bForcedStereoRes = false;
		if (W != 0)
		{
			Width = FMath::Clamp(W, 100, 10000);
			bForcedStereoRes = true;
		}
		if (H != 0)
		{
			Height = FMath::Clamp(H, 100, 10000);
			bForcedStereoRes = true;
		}

		if (ViewIndex == 0 && !bForcedStereoRes)
		{
			Width = FinalViewRect.Width();
			Height = FinalViewRect.Height();
		}
	}

	virtual void CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float WorldToMeters, FVector& ViewLocation) override
	{
		// 32mm, 1/2 average interpupillary distance
		float EyeOffset = GetDefault<UGeneralProjectSettings>()->EyeOffsetForFakeStereoRenderingDevice * WorldToMeters;

		if (ViewIndex == eSSE_MONOSCOPIC)
		{
			// Shift offset backwards so that the combined monoscopic frustum fully contains the frustums of each eye
			const float UnifiedFov = FOVInDegrees * (1.f + DifferenceBetweenEyes);
			const float HalfFov = FMath::DegreesToRadians(UnifiedFov) / 2.f;

			const float CenterOffset = EyeOffset / FMath::Tan(HalfFov);
			ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(-CenterOffset, 0, 0));
		}
		else
		{
			const float PassOffset = (ViewIndex % 2 == 0) ? -EyeOffset : EyeOffset;
			ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0,PassOffset,0));
		}
	}

	virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const override
	{
		float AngleUp = TopFOVRatio * FOVInDegrees;
		float AngleDown = -(1.0f - TopFOVRatio) * FOVInDegrees;

		// Use sane default values
		float AngleRight = 0.5f * FOVInDegrees;
		float AngleLeft = -0.5f * FOVInDegrees;

		// When difference is 0, the left and right angles are equal for each eye. When 1, each eye is only looking one way (i.e. left eye has AngleRight == 0 and vice versa)
		float PrimaryEyeCoeff = (1.0f + DifferenceBetweenEyes) / 2.0f;
		float SecondaryEyeCoeff = (1.0f - PrimaryEyeCoeff);
		if (ViewIndex == eSSE_LEFT_EYE)
		{
			AngleLeft = -(PrimaryEyeCoeff) * FOVInDegrees;
			AngleRight = SecondaryEyeCoeff * FOVInDegrees;
		}
		else if (ViewIndex == eSSE_RIGHT_EYE)
		{
			AngleLeft = -(SecondaryEyeCoeff) * FOVInDegrees;
			AngleRight = PrimaryEyeCoeff * FOVInDegrees;
		}

		// Needed for frustum culling in instanced stereo or mobile-multi view
		else if (ViewIndex == eSSE_MONOSCOPIC)
		{
			// Use the leftmost AngleLeft and the rightmost AngleRight
			float MaxEyeCoeff = FMath::Max(PrimaryEyeCoeff, SecondaryEyeCoeff);
			AngleLeft = -(MaxEyeCoeff) * FOVInDegrees;
			AngleRight = MaxEyeCoeff * FOVInDegrees;
		}

		AngleUp = tan(FMath::DegreesToRadians(AngleUp));
		AngleDown = tan(FMath::DegreesToRadians(AngleDown));
		AngleLeft = tan(FMath::DegreesToRadians(AngleLeft));
		AngleRight = tan(FMath::DegreesToRadians(AngleRight));

		float ZNear = GNearClippingPlane_RenderThread;

		float SumRL = (AngleRight + AngleLeft);
		float SumTB = (AngleUp + AngleDown);
		float InvRL = (1.0f / (AngleRight - AngleLeft));
		float InvTB = (1.0f / (AngleUp - AngleDown));

		FMatrix Mat = FMatrix(
			FPlane((2.0f * InvRL), 0.0f, 0.0f, 0.0f),
			FPlane(0.0f, (2.0f * InvTB), 0.0f, 0.0f),
			FPlane((SumRL * -InvRL), (SumTB * -InvTB), 0.0f, 1.0f),
			FPlane(0.0f, 0.0f, ZNear, 0.0f)
		);

		return Mat;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FScreenUnwrapSlicesPass, )
		RDG_TEXTURE_ACCESS(SrcTexture, ERHIAccess::SRVGraphics)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	virtual void RenderTexture_RenderThread(class FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const
	{
		check(IsInRenderingThread());
		checkf(bLayeredRTs, TEXT("Non-layered stereo can use the backbuffer directly"));

		FScreenUnwrapSlicesPass* Params = GraphBuilder.AllocParameters<FScreenUnwrapSlicesPass>();
		Params->SrcTexture = SrcTexture;
		Params->RenderTargets[0] = FRenderTargetBinding(BackBuffer, ERenderTargetLoadAction::EClear);
		GraphBuilder.AddPass(RDG_EVENT_NAME("RenderTexture_RenderThread"), Params, ERDGPassFlags::Raster,
			[Params, this](FRHICommandListImmediate& RHICmdList)
			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIShaderPlatform);

				TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
				TShaderMapRef<FScreenUnwrapSlicesPS> PixelShader(ShaderMap);

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), Params->SrcTexture->GetRHI());

				const int SourceWidth = Width;
				const int SourceHeight = Height;
				const int TargetWidth = Width;
				const int TargetHeight = Height;

				const int TargetX = 0;
				const int TargetY = 0;

				IRendererModule& RendererModule = FModuleManager::GetModuleChecked<IRendererModule>(FName("Renderer"));
				RendererModule.DrawRectangle(
					RHICmdList,
					(float)TargetX, (float)TargetY,
					(float)SourceWidth, (float)SourceHeight,
					0.f, 0.f,
					(float)SourceWidth, (float)SourceHeight,
					FIntPoint(TargetWidth, TargetHeight),
					FIntPoint(TargetWidth, TargetHeight),
					VertexShader,
					EDRF_UseTriangleOptimization);
			});
	};

	virtual IStereoRenderTargetManager* GetRenderTargetManager() override { return this; }

	virtual bool ShouldUseSeparateRenderTarget() const { return bLayeredRTs && IsStereoEnabled(); };

	virtual bool AllocateRenderTargetTextures(FRHICommandListBase& RHICmdList, uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumLayers, ETextureCreateFlags Flags, ETextureCreateFlags TargetableTextureFlags, TArray<FTextureRHIRef>& OutTargetableTextures, TArray<FTextureRHIRef>& OutShaderResourceTextures, uint32 NumSamples = 1)
	{ 
		check(IsInRenderingThread());
		checkf(bLayeredRTs, TEXT("Non-layered stereo can use the backbuffer directly"));

		// We're only creating a 1x target here, but we don't know whether it'll be the targeted texture
		// or the resolve texture. Because of this, we unify the input flags.
		ETextureCreateFlags UnifiedCreateFlags = Flags | TargetableTextureFlags;

		// We need to ensure we can sample from the texture in CopyTexture
		UnifiedCreateFlags |= TexCreate_ShaderResource;

		// We assume this could be used as a resolve target
		//UnifiedCreateFlags |= TexCreate_ResolveTargetable;

		// Some render APIs require us to present in RT layouts/configs,
		// so even if app won't use this texture as RT, we need the flag.
		UnifiedCreateFlags |= TexCreate_RenderTargetable;

		UnifiedCreateFlags |= TexCreate_SRGB;
		ETextureCreateFlags AuxiliaryCreateFlags = ETextureCreateFlags::None;

		// support foveation?
		//if (FBFoveationImageGenerator && FBFoveationImageGenerator->IsFoveationExtensionEnabled())
		//{
		//	AuxiliaryCreateFlags |= TexCreate_Foveation;
		//}

		FClearValueBinding ClearColor = FClearValueBinding::Transparent;

		uint8 ActualFormat = Format;
		FIntPoint Extent = FIntPoint(Width, Height);

		const TCHAR* EmulateStereoSwapchainDebugName = TEXT("EmulateStereoSwapchainImage");
		FRHITextureCreateDesc SwapchainImageDesc = //true ?
			FRHITextureCreateDesc::Create2DArray(EmulateStereoSwapchainDebugName).SetArraySize(2); //:
			//FRHITextureCreateDesc::Create2D(EmulateStereoSwapchainDebugName);

		SwapchainImageDesc.SetExtent(Width, Height);
		SwapchainImageDesc.SetFormat(static_cast<EPixelFormat>(ActualFormat));
		SwapchainImageDesc.SetFlags(UnifiedCreateFlags);

		// this will be owned by the caller
		FTextureRHIRef NewImage = RHICmdList.CreateTexture(SwapchainImageDesc);

		OutTargetableTextures.Add(NewImage);
		OutShaderResourceTextures = OutTargetableTextures;

		return true;

	}

	virtual void CalculateRenderTargetSize(const class FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY)
	{ 
		InOutSizeX = Width;
		InOutSizeY = Height;
	}

	virtual bool NeedReAllocateViewportRenderTarget(const class FViewport& Viewport) { return false; };

	/** max(HFOV, VFOV) in degrees of imaginable HMD */
	float FOVInDegrees;
	/** Proportion between Top FOV and Bottom FOV */
	float TopFOVRatio;
	/** How dissimilar are the eye frustums. If set to 0, they will be totally symmetric, if set to 1, totally different. */
	float DifferenceBetweenEyes;

	/** resolution of imaginable HMD (per eye) */
	int32 Width, Height;
	/** views of imaginable HMD */
	int32 NumViews;
	/** Whether the rendertargets are layered (like in MMV case) or not (side-by-side). */
	bool bLayeredRTs;
};

bool UEngine::InitializeHMDDevice()
{
	if (!IsRunningCommandlet())
	{
		static TAutoConsoleVariable<int32> CVarEmulateStereo(TEXT("r.EnableStereoEmulation"), 0, TEXT("Emulate stereo rendering"), ECVF_ReadOnly);
		if (FParse::Param(FCommandLine::Get(), TEXT("emulatestereo")) || CVarEmulateStereo.GetValueOnAnyThread() != 0)
		{
			TSharedPtr<FFakeStereoRenderingDevice, ESPMode::ThreadSafe> FakeStereoDevice(new FFakeStereoRenderingDevice(GSystemResolution.ResX / 2, GSystemResolution.ResY));
			StereoRenderingDevice = FakeStereoDevice;
		}
		// No reason to connect an HMD on a dedicated server.  Also fixes dedicated servers stealing the oculus connection.
		else if (!XRSystem.IsValid() && !FParse::Param(FCommandLine::Get(), TEXT("nohmd")) && !IsRunningDedicatedServer())
		{
			// Get a list of modules that implement this feature
			FName Type = IHeadMountedDisplayModule::GetModularFeatureName();
			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			TArray<IHeadMountedDisplayModule*> HMDModules = ModularFeatures.GetModularFeatureImplementations<IHeadMountedDisplayModule>(Type);

			// Check whether the user passed in an explicit HMD module on the command line
			FString ExplicitHMDName;
			bool bUseExplicitHMDDevice = FParse::Value(FCommandLine::Get(), TEXT("hmd="), ExplicitHMDName);

			// Sort modules by priority
			HMDModules.Sort(IHeadMountedDisplayModule::FCompareModulePriority());

			// Select first module with a connected HMD able to create a device
			IHeadMountedDisplayModule* HMDModuleSelected = nullptr;
			TArray<IHeadMountedDisplayModule*> HMDModulesDisconnected;

			for (auto HMDModuleIt = HMDModules.CreateIterator(); HMDModuleIt; ++HMDModuleIt)
			{
				IHeadMountedDisplayModule* HMDModule = *HMDModuleIt;

				// Skip all non-matching modules when an explicit module name has been specified on the command line
				if (bUseExplicitHMDDevice)
				{
					TArray<FString> HMDAliases;
					HMDModule->GetModuleAliases(HMDAliases);
					HMDAliases.Add(HMDModule->GetModuleKeyName());

					bool bMatchesExplicitDevice = false;
					for (const FString& HMDModuleName : HMDAliases)
					{
						if (ExplicitHMDName.Equals(HMDModuleName, ESearchCase::IgnoreCase))
						{
							bMatchesExplicitDevice = true;
							break;
						}
					}

					if (!bMatchesExplicitDevice)
					{
						continue;
					}
				}

				if(HMDModule->IsHMDConnected())
				{
					XRSystem = HMDModule->CreateTrackingSystem();

					if (XRSystem.IsValid())
					{
						HMDModuleSelected = HMDModule;
						break;
					}
				}
				else
				{
					HMDModulesDisconnected.Add(HMDModule);
				}
			}

			// If no module selected yet, just select first module able to create a device, even if HMD is not connected.
			if (!HMDModuleSelected)
			{
				for (auto HMDModuleIt = HMDModulesDisconnected.CreateIterator(); HMDModuleIt; ++HMDModuleIt)
				{
					IHeadMountedDisplayModule* HMDModule = *HMDModuleIt;

					XRSystem = HMDModule->CreateTrackingSystem();

					if (XRSystem.IsValid())
					{
						HMDModuleSelected = HMDModule;
						break;
					}
				}
			}

			// Unregister modules which were not selected, since they will not be used.
			for (auto HMDModuleIt = HMDModules.CreateIterator(); HMDModuleIt; ++HMDModuleIt)
			{
				IHeadMountedDisplayModule* HMDModule = *HMDModuleIt;

				if(HMDModule != HMDModuleSelected)
				{
					ModularFeatures.UnregisterModularFeature(Type, HMDModule);
				}
			}

			// If we found a valid XRSystem, use it to get a stereo rendering device, if available
			if (XRSystem.IsValid() && !FParse::Param(FCommandLine::Get(), TEXT("noxrstereo")))
			{
				StereoRenderingDevice = XRSystem->GetStereoRenderingDevice();
				const bool bShouldStartInVR = StereoRenderingDevice.IsValid() && 
											  !GIsEditor &&
											  IStereoRendering::IsStartInVR();
				if (bShouldStartInVR)
				{
					StereoRenderingDevice->EnableStereo(true);
				}
			}
			// Else log an error if we got an explicit module name on the command line
			else if (bUseExplicitHMDDevice)
			{
				UE_LOGF(LogInit, Error, "Failed to find or initialize HMD module named '%ls'. HMD mode will be disabled.", *ExplicitHMDName);
			}

		}
	}

	return StereoRenderingDevice.IsValid();
}

bool UEngine::InitializeEyeTrackingDevice()
{
	if (!IsRunningCommandlet() && !EyeTrackingDevice.IsValid())
	{
		// No reason to connect an eye tracking on a dedicated server.
		if (!FParse::Param(FCommandLine::Get(), TEXT("noeyetracking")) && !IsRunningDedicatedServer())
		{
			IModularFeatures& ModularFeatures = IModularFeatures::Get();

			// Get a list of modules that implement this feature
			//FName const ModularFeatureName = IEyeTrackerModule::Get().GetModularFeatureName();
			//LUMIN_MERGE - Circular dependency doesn't allow this
			FName const ModularFeatureName = "EyeTracker";// IEyeTrackerModule::Get().GetModularFeatureName();
			TArray<IEyeTrackerModule*> ETModules = ModularFeatures.GetModularFeatureImplementations<IEyeTrackerModule>(ModularFeatureName);

			// Check whether the user passed in an explicit HMD module on the command line
			FString ExplicitETName;
			bool const bUseExplicitETDevice = FParse::Value(FCommandLine::Get(), TEXT("eyetracking="), ExplicitETName);

			// #todo: ask the HMD if it had a preferred eyetracker and prioritize that, since some have built-in eyetracking devices
			// Sort modules by priority
			//LUMIN_MERGE - Circular dependency doesn't allow this
			//ETModules.Sort(IEyeTrackerModule::FCompareModulePriority());

			// Select first module with a connected eyetracker able to create a device
			IEyeTrackerModule* ETModuleSelected = nullptr;
			TArray<IEyeTrackerModule*> ETModulesDisconnected;

			for (auto ETModuleIt = ETModules.CreateIterator(); ETModuleIt; ++ETModuleIt)
			{
				IEyeTrackerModule* const ETModule = *ETModuleIt;

				// Skip all non-matching modules when an explicit module name has been specified on the command line
				if (bUseExplicitETDevice && !ExplicitETName.Equals(ETModule->GetModuleKeyName(), ESearchCase::IgnoreCase))
				{
					continue;
				}
 
 				if (ETModule->IsEyeTrackerConnected())
 				{
 					EyeTrackingDevice = ETModule->CreateEyeTracker();
 
 					if (EyeTrackingDevice.IsValid())
 					{
						ETModuleSelected = ETModule;
 						break;
 					}
 				}
 				else
 				{
					ETModulesDisconnected.Add(ETModule);
 				}
			}

			// If no module selected yet, just select first module able to create a device, even if HMD is not connected.
			if (!ETModuleSelected)
			{
				for (auto ETModuleIt = ETModulesDisconnected.CreateIterator(); ETModuleIt; ++ETModuleIt)
				{
					IEyeTrackerModule* const ETModule = *ETModuleIt;
					EyeTrackingDevice = ETModule->CreateEyeTracker();
					if (EyeTrackingDevice.IsValid())
					{
						ETModuleSelected = ETModule;
						break;
					}
				}
			}

			// Unregister modules which were not selected, since they will not be used.
			for (auto ETModuleIt = ETModules.CreateIterator(); ETModuleIt; ++ETModuleIt)
			{
				IEyeTrackerModule* const ETModule = *ETModuleIt;
				if (ETModule != ETModuleSelected)
				{
					ModularFeatures.UnregisterModularFeature(ModularFeatureName, ETModule);
				}
			}

			// If we found a valid HMDDevice, use this as our StereoRenderingDevice
			if (EyeTrackingDevice.IsValid() == false)
			{
				// log an error if we got an explicit module name on the command line
				if (bUseExplicitETDevice)
				{
					UE_LOGF(LogInit, Error, "Failed to find or initialize EyeTracker module named '%ls'. Eye tracking will be disabled.", *ExplicitETName);
				}
			}
		}
	}

	return EyeTrackingDevice.IsValid();
}


void UEngine::RecordHMDAnalytics()
{
	if (XRSystem.IsValid() && !FParse::Param(FCommandLine::Get(), TEXT("nohmd")) && XRSystem->GetHMDDevice() && XRSystem->GetHMDDevice()->IsHMDConnected())
	{
		XRSystem->GetHMDDevice()->RecordAnalytics();
	}
}

bool UEngine::HasMultipleLocalPlayers(UWorld* InWorld)
{
	if (InWorld == NULL)
	{
		// If no specified world, return true if any world context has multiple local players
		for (auto It = WorldList.CreateIterator(); It; ++It)
		{
			if (It->WorldType == EWorldType::GameRPC)
			{
				continue;
			}

			if (It->OwningGameInstance != NULL && It->OwningGameInstance->GetNumLocalPlayers() > 1)
			{
				return true;
			}
		}

		return false;
	}

	return (GetNumGamePlayers(InWorld) > 1);
}

/** @return whether we're currently running with stereoscopic 3D enabled */
bool UEngine::IsStereoscopic3D(const FViewport* InViewport) const
{
	return (!InViewport || InViewport->IsStereoRenderingAllowed()) &&
		(StereoRenderingDevice.IsValid() && StereoRenderingDevice->IsStereoEnabled());
}

ULocalPlayer* GetLocalPlayerFromControllerId_local(const TArray<class ULocalPlayer*>& GamePlayers, const int32 ControllerId)
{
	for ( ULocalPlayer* const Player : GamePlayers )
	{
		if ( Player && Player->GetControllerId() == ControllerId )
		{
			return Player;
		}
	}

	return nullptr;
}

ULocalPlayer* UEngine::GetLocalPlayerFromControllerId( const UGameViewportClient* InViewport, const int32 ControllerId ) const
{
	if (GetWorldContextFromGameViewport(InViewport) != nullptr)
	{
		const TArray<class ULocalPlayer*>& GamePlayers = GetGamePlayers(InViewport);
		return GetLocalPlayerFromControllerId_local(GamePlayers, ControllerId);
	}
	return nullptr;
}

ULocalPlayer* UEngine::GetLocalPlayerFromControllerId( UWorld * InWorld, const int32 ControllerId ) const
{
	const TArray<class ULocalPlayer*>& GamePlayers = GetGamePlayers(InWorld);
	return GetLocalPlayerFromControllerId_local(GamePlayers, ControllerId);
}

namespace UE::Private
{
	static ULocalPlayer* GetLocalPlayerFromInputDevice(const TArray<ULocalPlayer*>& GamePlayers, const FInputDeviceId InputDevice)
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		const FPlatformUserId PlatUser = DeviceMapper.GetUserForInputDevice(InputDevice);

		for (ULocalPlayer* const Player : GamePlayers)
		{
			if (Player && Player->GetPlatformUserId() == PlatUser)
			{
				return Player;
			}
		}
	
		return nullptr;
	}
}	// namespace UE::Private

ULocalPlayer* UEngine::GetLocalPlayerFromInputDevice(const UGameViewportClient* InViewport, const FInputDeviceId InputDevice) const
{
	if (GetWorldContextFromGameViewport(InViewport) != nullptr)
	{
		const TArray<class ULocalPlayer*>& GamePlayers = GetGamePlayers(InViewport);
		return UE::Private::GetLocalPlayerFromInputDevice(GamePlayers, InputDevice);
	}
	return nullptr;
}

ULocalPlayer* UEngine::GetLocalPlayerFromInputDevice(UWorld * InWorld, const FInputDeviceId InputDevice) const
{
	const TArray<class ULocalPlayer*>& GamePlayers = GetGamePlayers(InWorld);
	return UE::Private::GetLocalPlayerFromInputDevice(GamePlayers, InputDevice);
}

void UEngine::SwapControllerId(ULocalPlayer *NewPlayer, const int32 CurrentControllerId, const int32 NewControllerID) const
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		if (It->OwningGameInstance == nullptr)
		{
			continue;
		}

		const TArray<class ULocalPlayer*> & LocalPlayers = It->OwningGameInstance->GetLocalPlayers();

		if (LocalPlayers.Contains(NewPlayer))
		{
			// This is the world context that NewPlayer belongs to, see if anyone is using its CurrentControllerId
			for (ULocalPlayer* LocalPlayer : LocalPlayers)
			{
				if (LocalPlayer && LocalPlayer != NewPlayer && LocalPlayer->GetControllerId() == NewControllerID)
				{
					LocalPlayer->SetControllerId( CurrentControllerId);
					return;
				}
			}
		}
	}
}

void UEngine::SwapPlatformUserId(ULocalPlayer *NewPlayer, const FPlatformUserId CurrentUserId, const FPlatformUserId NewUserID) const
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		if (It->OwningGameInstance == nullptr)
		{
			continue;
		}

		const TArray<class ULocalPlayer*> & LocalPlayers = It->OwningGameInstance->GetLocalPlayers();

		if (LocalPlayers.Contains(NewPlayer))
		{
			// This is the world context that NewPlayer belongs to, see if anyone is using their CurrentUserId
			for (ULocalPlayer* LocalPlayer : LocalPlayers)
			{
				if (LocalPlayer && LocalPlayer != NewPlayer && LocalPlayer->GetPlatformUserId() == NewUserID)
				{
					LocalPlayer->SetPlatformUserId(CurrentUserId);
					return;
				}
			}
		}
	}
}

APlayerController* UEngine::GetFirstLocalPlayerController(const UWorld* InWorld)
{
	const FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);

	return (Context.OwningGameInstance ? Context.OwningGameInstance->GetFirstLocalPlayerController(InWorld) : nullptr);
}

void UEngine::GetAllLocalPlayerControllers(TArray<APlayerController*> & PlayerList)
{
	for (auto It = WorldList.CreateIterator(); It; ++It)
	{
		if ( It->OwningGameInstance == NULL )
		{
			continue;
		}
		for (auto PlayerIt = It->OwningGameInstance->GetLocalPlayerIterator(); PlayerIt; ++PlayerIt)
		{
			ULocalPlayer *Player = *PlayerIt;
			if (Player->PlayerController)
			{
				PlayerList.Add( Player->PlayerController );
			}
		}
	}
}


/*-----------------------------------------------------------------------------
Input.
-----------------------------------------------------------------------------*/

#if !UE_BUILD_SHIPPING

/**
* Helper structure for sorting textures by relative cost.
*/
struct FSortedTexture 
{
	int32		MaxAllowedSizeX;	// This is the disk size when cooked.
	int32		MaxAllowedSizeY;
	EPixelFormat Format;
	int32		CurSizeX;
	int32		CurSizeY;
	int32		LODBias;
	int32		MaxAllowedSize;
	int32		CurrentSize;
	FString		Name;
	int32		LODGroup;
	bool		bIsStreaming;
	bool		bIsUnknownRef;
	bool		bIsVirtual;
	bool		bIsUncompressed;
	int32		UsageCount;
	int32		NumMips;

	/** Constructor, initializing every member variable with passed in values. */
	FSortedTexture(	int32 InMaxAllowedSizeX, int32 InMaxAllowedSizeY, EPixelFormat InFormat, int32 InCurSizeX, int32 InCurSizeY, int32 InLODBias, int32 InMaxAllowedSize, int32 InCurrentSize, const FString& InName, int32 InLODGroup, bool bInIsStreaming, bool bInIsUnknownRef, bool bInIsVirtual, int32 InUsageCount, int32 InNumMips, bool bInIsUncompressed)
		:	MaxAllowedSizeX( InMaxAllowedSizeX )
		,	MaxAllowedSizeY( InMaxAllowedSizeY )
		,	Format( InFormat )
		,	CurSizeX( InCurSizeX )
		,	CurSizeY( InCurSizeY )
		,	LODBias( InLODBias )
		,	MaxAllowedSize( InMaxAllowedSize )
		,	CurrentSize( InCurrentSize )
		,	Name( InName )
		,	LODGroup( InLODGroup )
		,	bIsStreaming( bInIsStreaming )
		,	bIsUnknownRef(bInIsUnknownRef)
		,	bIsVirtual( bInIsVirtual )
		,	bIsUncompressed( bInIsUncompressed )
		,	UsageCount( InUsageCount )
		,	NumMips( InNumMips )
	{}
};
struct FCompareFSortedTexture
{
	bool bAlphaSort;
	FCompareFSortedTexture( bool InAlphaSort )
		: bAlphaSort( InAlphaSort )
	{}
	FORCEINLINE bool operator()(const FSortedTexture& A, const FSortedTexture& B) const
	{
		if (bAlphaSort || A.CurrentSize == B.CurrentSize)
		{
			return A.Name < B.Name;
		}

		return B.CurrentSize < A.CurrentSize;
	}
};

/**
* Helper structure for sorting static meshes by relative cost.
*/
struct FSortedStaticMesh
{
	int32			NumKB;
	int32			MaxKB;
	int32			ResKBExc;
	int32			ResKBInc;
	int32			ResKBIncMobile;
	int32			ResKBResident;
	int32			DistanceFieldKB;
	int32			LodCount;
	int32			ResidentLodCount;
	int32			MobileMinLOD;
	int32			VertexCountLod0;
	int32			VertexCountLod1;
	int32			VertexCountLod2;
	int32			VertexCountTotal;
	int32			VertexCountTotalMobile;
	int32			VertexCountCollision;
	int32			ShapeCountCollision;
	int32			ComponentUsageCount;
	int32			InstanceUsageCount;
	int32			CPUAccessOverheadKB;
	bool			bUnknownRef;
	bool			bIsStreaming;
	UStaticMesh*	Mesh;
	FString			Name;

	/** Constructor, initializing every member variable with passed in values. */
	FSortedStaticMesh(UStaticMesh* InMesh, int32 InNumKB, int32 InMaxKB, int32 InResKBExc, int32 InResKBInc, int32 InResKBIncMobile, int32 InResKBResident, int32 InDistanceFieldKB, int32 InLodCount, int32 InResidentLodCount, int32 InMobileMinLOD, int32 InVertexCountLod0, int32 InVertexCountLod1, int32 InVertexCountLod2, int32 InVertexCountTotal, int32 InVertexCountTotalMobile, int32 InVertexCountCollision, int32 InShapeCountCollision, int32 InComponentUsageCount, int32 InInstanceUsageCount, int32 InCPUAccessOverheadKB, bool bInUnknownRef, bool bInIsStreaming, FString InName)
		: NumKB(InNumKB)
		, MaxKB(InMaxKB)
		, ResKBExc(InResKBExc)
		, ResKBInc(InResKBInc)
		, ResKBIncMobile(InResKBIncMobile)
		, ResKBResident(InResKBResident)
		, DistanceFieldKB(InDistanceFieldKB)
		, LodCount(InLodCount)
		, ResidentLodCount(InResidentLodCount)
		, MobileMinLOD(InMobileMinLOD)
		, VertexCountLod0(InVertexCountLod0)
		, VertexCountLod1(InVertexCountLod1)
		, VertexCountLod2(InVertexCountLod2)
		, VertexCountTotal(InVertexCountTotal)
		, VertexCountTotalMobile(InVertexCountTotalMobile)
		, VertexCountCollision(InVertexCountCollision)
		, ShapeCountCollision(InShapeCountCollision)
		, ComponentUsageCount(InComponentUsageCount)
		, InstanceUsageCount(InInstanceUsageCount)
		, CPUAccessOverheadKB(InCPUAccessOverheadKB)
		, bUnknownRef(bInUnknownRef)
		, bIsStreaming(bInIsStreaming)
		, Mesh(InMesh)		
		, Name(InName)
	{}
};
struct FCompareFSortedStaticMesh
{
	enum ESortType
	{
		Default,
		InstanceCount,
		Mobile,
		Alpha
	} SortBy;

	FCompareFSortedStaticMesh(ESortType InSortBy)
		: SortBy(InSortBy)
	{}
	FORCEINLINE bool operator()(const FSortedStaticMesh& A, const FSortedStaticMesh& B) const
	{
		switch (SortBy)
		{
		case InstanceCount:
			return B.InstanceUsageCount < A.InstanceUsageCount;
			
		case Mobile:
			return B.VertexCountTotalMobile < A.VertexCountTotalMobile;

		case Alpha:
			return A.Name < B.Name;
		
		case Default:
		default:
			{
				if ((B.NumKB + B.ResKBExc) == (A.NumKB + A.ResKBExc))
		{
					return A.Name < B.Name; // default to alpha
		}

		return (B.NumKB + B.ResKBExc) < (A.NumKB + A.ResKBExc);
	}
		}
	}
};

/**
* Helper structure for sorting skeletal meshes by relative cost.
*/
struct FSortedSkeletalMesh
{
	int32		NumKB;
	int32		MaxKB;
	int32		ResKBExc;
	int32		ResKBInc;
	int32		ResKBIncMobile;
	int32		LodCount;
	int32		MobileMinLOD;
	int32		MeshDisablesMinLODStripping;
	int32		bSupportLODStreaming;
	int32		MaxNumStreamedLODs;
	int32		MaxNumOptionalLODs;
	int32		VertexCountLod0;
	int32		VertexCountLod1;
	int32		VertexCountLod2;
	int32		VertexCountTotal;
	int32		VertexCountTotalMobile;
	int32		VertexCountCollision;
	int32		ResidentLodCount;
	int32		ResidentResKBExc;
	int32		CPUAccessOverheadKB;
	bool		bUnknownRef;
	bool		bIsStreaming;
	FString		Name;

	/** Constructor, initializing every member variable with passed in values. */
	FSortedSkeletalMesh(int32 InNumKB, int32 InMaxKB, int32 InResKBExc, int32 InResKBInc, int32 InResKBIncMobile, int32 InLodCount, int32 InMobileMinLOD,
		int32 InMeshDisablesMinLODStripping, int32 bInSupportLODStreaming, int32 InMaxNumStreamedLODs, int32 InMaxNumOptionalLODs, int32 InVertexCountLod0,
		int32 InVertexCountLod1, int32 InVertexCountLod2, int32 InVertexCountTotal, int32 InVertexCountTotalMobile, int32 InVertexCountCollision,
		int32 InResidentLodCount, int32 InResidentResKBExc, int32 InCPUAccessOverheadKB, bool bInUnknownRef, bool bInIsStreaming, FString InName)
		: NumKB(InNumKB)
		, MaxKB(InMaxKB)
		, ResKBExc(InResKBExc)
		, ResKBInc(InResKBInc)
		, ResKBIncMobile(InResKBIncMobile)
		, LodCount(InLodCount)
		, MobileMinLOD(InMobileMinLOD)
		, MeshDisablesMinLODStripping(InMeshDisablesMinLODStripping)
		, bSupportLODStreaming(bInSupportLODStreaming)
		, MaxNumStreamedLODs(InMaxNumStreamedLODs)
		, MaxNumOptionalLODs(InMaxNumOptionalLODs)
		, VertexCountLod0(InVertexCountLod0)
		, VertexCountLod1(InVertexCountLod1)
		, VertexCountLod2(InVertexCountLod2)
		, VertexCountTotal(InVertexCountTotal)
		, VertexCountTotalMobile(InVertexCountTotalMobile)
		, VertexCountCollision(InVertexCountCollision)
		, ResidentLodCount(InResidentLodCount)
		, ResidentResKBExc(InResidentResKBExc)
		, CPUAccessOverheadKB(InCPUAccessOverheadKB)
		, bUnknownRef(bInUnknownRef)
		, bIsStreaming(bInIsStreaming)
		, Name(InName)
	{}
};
struct FCompareFSortedSkeletalMesh
{
	bool bAlphaSort;
	bool bMobileSort;
	FCompareFSortedSkeletalMesh(bool InAlphaSort, bool InMobileSort)
		: bAlphaSort(InAlphaSort)
		, bMobileSort(InMobileSort)
	{}
	FORCEINLINE bool operator()(const FSortedSkeletalMesh& A, const FSortedSkeletalMesh& B) const
	{
		if (bMobileSort)
		{
			return ((B.VertexCountTotalMobile) < (A.VertexCountTotalMobile));
		}

		if (bAlphaSort || (B.NumKB + B.ResKBExc) == (A.NumKB + A.ResKBExc))
		{
			return A.Name < B.Name;
		}

		return (B.NumKB + B.ResKBExc) < (A.NumKB + A.ResKBExc);
	}
};

/**
* Helper structure for sorting anims by relative cost.
*/
struct FSortedAnimAsset
{
	int32		NumKB;
	int32		MaxKB;
	int32		ResKBExc;
	int32		ResKBInc;
	FName		AnimAssetType;
	FString		Name;
	int32		NumKeys;
	float		SequenceLength;
	float		RateScale;
	int32		NumCurves;
	FString		DebugString;

	FSortedAnimAsset(int32 InNumKB, int32 InMaxKB, int32 InResKBExc, int32 InResKBInc, FName InAnimAssetType, FString InName, int32 InNumKeys, float InSequenceLength, float InRateScale, int32 InNumCurves, const FString& InDebugString)
		: NumKB(InNumKB)
		, MaxKB(InMaxKB)
		, ResKBExc(InResKBExc)
		, ResKBInc(InResKBInc)
		, AnimAssetType(InAnimAssetType)
		, Name(InName)
		, NumKeys(InNumKeys)
		, SequenceLength(InSequenceLength)
		, RateScale(InRateScale)
		, NumCurves(InNumCurves)
		, DebugString(InDebugString)
	{}
};
struct FCompareFSortedAnimAsset
{
	bool bAlphaSort;
	FCompareFSortedAnimAsset(bool InAlphaSort)
		: bAlphaSort(InAlphaSort)
	{}
	FORCEINLINE bool operator()(const FSortedAnimAsset& A, const FSortedAnimAsset& B) const
	{
		if (bAlphaSort || (B.NumKB + B.ResKBExc) == (A.NumKB + A.ResKBExc))
		{
			return A.Name < B.Name;
		}

		return (B.NumKB + B.ResKBExc) < (A.NumKB + A.ResKBExc);
	}
};


/** Helper struct for sorting anim sets by size */
struct FSortedSet
{
	FString Name;
	int32		Size;

	FSortedSet( const FString& InName, int32 InSize )
		:	Name(InName)
		,	Size(InSize)
	{}
};
struct FCompareFSortedSet
{
	bool bAlphaSort;
	FCompareFSortedSet( bool InAlphaSort )
		: bAlphaSort( InAlphaSort )
	{}
	FORCEINLINE bool operator()( const FSortedSet& A, const FSortedSet& B ) const
	{
		if (bAlphaSort || A.Size == B.Size)
		{
			return A.Name < B.Name;
		}

		return B.Size < A.Size;
	}
};

struct FSortedParticleSet
{
	FString Name;
	int32		Size;
	int32		PSysSize;
	int32		ModuleSize;
	int32		ComponentSize;
	int32		ComponentCount;
	FResourceSizeEx ComponentResourceSize;
	FResourceSizeEx ComponentTrueResourceSize;

	FSortedParticleSet( const FString& InName, int32 InSize, int32 InPSysSize, int32 InModuleSize, 
		int32 InComponentSize, int32 InComponentCount, FResourceSizeEx InComponentResourceSize, FResourceSizeEx InComponentTrueResourceSize) :
		Name(InName)
		, Size(InSize)
		, PSysSize(InPSysSize)
		, ModuleSize(InModuleSize)
		, ComponentSize(InComponentSize)
		, ComponentCount(InComponentCount)
		, ComponentResourceSize(InComponentResourceSize)
		, ComponentTrueResourceSize(InComponentTrueResourceSize)
	{}

	FSortedParticleSet(const FString& InName)
		: Name(InName)
		, Size(0)
		, PSysSize(0)
		, ModuleSize(0)
		, ComponentSize(0)
		, ComponentCount(0)
		, ComponentResourceSize(EResourceSizeMode::EstimatedTotal)
		, ComponentTrueResourceSize(EResourceSizeMode::Exclusive)
	{
	}

	const FSortedParticleSet& operator += (const FSortedParticleSet& InOther)
	{
		Size += InOther.Size;
		PSysSize += InOther.PSysSize;
		ModuleSize += InOther.ModuleSize;
		ComponentSize += InOther.ComponentSize;
		ComponentCount += InOther.ComponentCount;
		ComponentResourceSize += InOther.ComponentResourceSize;
		ComponentTrueResourceSize += InOther.ComponentTrueResourceSize;
		return *this;
	}

	void Dump(FOutputDevice& InArchive)
	{
		InArchive.Logf(TEXT("%10d,%s,%d,%d,%d,%d,%d,%d"),
			Size, *Name, PSysSize, ModuleSize, ComponentSize,
			ComponentCount, ComponentResourceSize.GetTotalMemoryBytes(), ComponentTrueResourceSize.GetTotalMemoryBytes());
	}
};

struct FCompareFSortedParticleSet
{
	bool bAlphaSort;
	FCompareFSortedParticleSet( bool InAlphaSort )
		: bAlphaSort( InAlphaSort )
	{}
	FORCEINLINE bool operator()( const FSortedParticleSet& A, const FSortedParticleSet& B ) const
	{
		if (bAlphaSort || A.Size == B.Size)
		{
			return A.Name < B.Name;
		}

		return B.Size < A.Size;
	}
};


static void ShowSubobjectGraph( FOutputDevice& Ar, UObject* CurrentObject, const FString& IndentString )
{
	if ( CurrentObject == NULL )
	{
		Ar.Logf(TEXT("%sX NULL"), *IndentString);
	}
	else
	{
		TArray<UObject*> ReferencedObjs;
		FReferenceFinder RefCollector( ReferencedObjs, CurrentObject, true, false, false, false);
		RefCollector.FindReferences( CurrentObject );

		if ( ReferencedObjs.Num() == 0 )
		{
			Ar.Logf(TEXT("%s. %s"), *IndentString, IndentString.Len() == 0 ? *CurrentObject->GetPathName() : *CurrentObject->GetName());
		}
		else
		{
			Ar.Logf(TEXT("%s+ %s"), *IndentString, IndentString.Len() == 0 ? *CurrentObject->GetPathName() : *CurrentObject->GetName());
			for ( int32 ObjIndex = 0; ObjIndex < ReferencedObjs.Num(); ObjIndex++ )
			{
				ShowSubobjectGraph( Ar, ReferencedObjs[ObjIndex], IndentString + TEXT( "|\t" ) );
			}
		}
	}
}

namespace UE
{

struct FItem
{
	UClass* Class;
	int32 Count;
	SIZE_T Num;
	SIZE_T Max;
	/** Only exclusive resource size, the truer resource size. */
	FResourceSizeEx TrueResourceSize;

	FItem(UClass* InClass = nullptr)
		: Class(InClass), Count(0), Num(0), Max(0), TrueResourceSize()
	{}

	FItem(UClass* InClass, int32 InCount, SIZE_T InNum, SIZE_T InMax, FResourceSizeEx InTrueResourceSize) :
		Class(InClass),
		Count(InCount),
		Num(InNum),
		Max(InMax),
		TrueResourceSize(InTrueResourceSize)
	{}

	void Add(FArchiveCountMem& Ar, FResourceSizeEx InTrueResourceSize)
	{
		Count++;
		Num += Ar.GetNum();
		Max += Ar.GetMax();
		TrueResourceSize += InTrueResourceSize;
	}

	void Add(SIZE_T InNumBytes, SIZE_T InMaxBytes, FResourceSizeEx InTrueResourceSize)
	{
		Count++;
		Num += InNumBytes;
		Max += InMaxBytes;
		TrueResourceSize += InTrueResourceSize;
	}
};

struct FSubItem
{
	UObject* Object;
	/** Size of the object, counting containers as current usage */
	SIZE_T Num;
	/** Size of the object, counting containers as total allocated (max usage) */
	SIZE_T Max;
	/** Resource size of the object and all of its references, the 'old-style'. */
	SIZE_T ResourceSize;
	/** Only exclusive resource size, the truer resource size. */
	FResourceSizeEx TrueResourceSize;

	FSubItem(UObject* InObject, SIZE_T InNum, SIZE_T InMax, FResourceSizeEx InTrueResourceSize)
		: Object(InObject), Num(InNum), Max(InMax), TrueResourceSize(InTrueResourceSize)
	{}
};

} // namespace UE

#endif // !UE_BUILD_SHIPPING

MSVC_PRAGMA(warning(push))
MSVC_PRAGMA(warning(disable : 4717))
MSVC_PRAGMA(warning(disable : 6387)) // CauseCrtError intentionally passes nullptr to printf

#if defined (__clang__) 
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Winfinite-recursion"
#endif


// clang can optimize this out (http://stackoverflow.com/questions/18478078/clang-infinite-tail-recursion-optimization), make the function do some useful work

volatile int GInfiniteRecursionCount = 0;
int InfiniteRecursionFunction(int B)
{
	GInfiniteRecursionCount += InfiniteRecursionFunction(B + 1);
	return GInfiniteRecursionCount;
}

// used to test CRT invalid parameter handling
void CauseCrtError()
{
#if PLATFORM_WINDOWS
	constexpr const char* volatile CauseCrtErrorNullptr = nullptr;
	printf(CauseCrtErrorNullptr); //-V575
#endif
}

#if defined (__clang__) 
#pragma clang diagnostic pop
#endif

MSVC_PRAGMA(warning(pop))

/** DEBUG used for exe "DEBUG BUFFEROVERFLOW" */
static void BufferOverflowFunction(SIZE_T BufferSize, const ANSICHAR* Buffer) 
{
	ANSICHAR LocalBuffer[32];
	LocalBuffer[0] = LocalBuffer[31] = 0; //if BufferSize is 0 then there's nothing to print out!

	BufferSize = FMath::Min<SIZE_T>(BufferSize, UE_ARRAY_COUNT(LocalBuffer)-1);

	for( uint32 i = 0; i < BufferSize; i++ ) 
	{
		LocalBuffer[i] = Buffer[i];
	}
	UE_LOGF(LogEngine, Log, "BufferOverflowFunction BufferSize=%d LocalBuffer=%ls",(int32)BufferSize, ANSI_TO_TCHAR(LocalBuffer));
}

#if UE_ALLOW_EXEC_COMMANDS
bool UEngine::Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	// If we don't have a viewport specified to catch the stat commands, use to the game viewport
	if (GStatProcessingViewportClient == NULL)
	{
		GStatProcessingViewportClient = GameViewport;
	}

	if (FExec::Exec( InWorld, Cmd, Ar ))
	{
		return true;
	}

	// See if any other subsystems claim the command.
	if (StaticExec(InWorld, Cmd,Ar) == true)
	{
		return true;
	}


	if (GDebugToolExec && (GDebugToolExec->Exec( InWorld, Cmd,Ar) == true))
	{
		return true;
	}

	if (UE::Private::GMalloc && (UE::Private::GMalloc->Exec( InWorld, Cmd,Ar) == true))
	{
		return true;
	}

	if (GSystemSettings.Exec( InWorld, Cmd,Ar) == true)
	{
		return true;
	}

	FAudioDevice* AudioDevice = nullptr;
	if (InWorld)
	{
		AudioDevice = InWorld->GetAudioDeviceRaw();
	}
	else
	{
		AudioDevice = GetMainAudioDeviceRaw();
	}

	if (AudioDevice && AudioDevice->Exec(InWorld, Cmd, Ar) == true)
	{
		return true;
	}

	if (FPlatformMisc::Exec( InWorld, Cmd,Ar) == true)
	{
		return true;
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (FLowLevelMemTracker::Get().Exec(Cmd, Ar))
	{
		return true;
	}
#endif

	{
		FString CultureName;
		if (FParse::Value(Cmd, TEXT("CULTURE="), CultureName))
		{
			FInternationalization::Get().SetCurrentCulture(CultureName);
			return true;
		}
	}

	{
		FString LanguageName;
		if (FParse::Value(Cmd, TEXT("LANGUAGE="), LanguageName))
		{
			FInternationalization::Get().SetCurrentLanguage(LanguageName);
			return true;
		}
	}

	{
		FString LocaleName;
		if (FParse::Value(Cmd, TEXT("LOCALE="), LocaleName))
		{
			FInternationalization::Get().SetCurrentLocale(LocaleName);
			return true;
		}
	}

#if ENABLE_LOC_TESTING
	{
		FString ConfigFilePath;
		if (FParse::Value(Cmd, TEXT("REGENLOC="), ConfigFilePath))
		{
			ILocalizationModule::Get().HandleRegenLocCommand(ConfigFilePath);
		}
	}
#endif

	// Handle engine command line.
	if ( FParse::Command(&Cmd,TEXT("FLUSHLOG")) )
	{
		return HandleFlushLogCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd, TEXT("GAMEVER")) ||  FParse::Command(&Cmd, TEXT("GAMEVERSION")))
	{
		return HandleGameVerCommand( Cmd, Ar );
	}
#if 0
	else if (FParse::Command(&Cmd, TEXT("HOTFIXTEST")))
	{
		FTestHotFixPayload Test;
		Test.ValueToReturn = true;
		Test.Result = false;
		Test.Message = FString("Hi there");
		FCoreDelegates::GetHotfixDelegate(EHotfixDelegates::Test).ExecuteIfBound(&Test, sizeof(FTestHotFixPayload));
		check(Test.ValueToReturn == Test.Result);
	}
#endif
	else if( FParse::Command(&Cmd,TEXT("STAT")) )
	{
		return HandleStatCommand(InWorld, GStatProcessingViewportClient, Cmd, Ar);
	}
	else if( FParse::Command(&Cmd,TEXT("STOPMOVIECAPTURE")) && GIsEditor )
	{
		return HandleStopMovieCaptureCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("CRACKURL")) )
	{
		return HandleCrackURLCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("DEFER")) )
	{
		return HandleDeferCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd, TEXT("OPEN") ) )
	{
		return HandleOpenCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("STREAMMAP")) )
	{
		return HandleStreamMapCommand( Cmd, Ar, InWorld );
	}
#if WITH_SERVER_CODE
	else if (FParse::Command(&Cmd, TEXT("SERVERTRAVEL")) )
	{
		return HandleServerTravelCommand( Cmd, Ar, InWorld );
	}
#endif // WITH_SERVER_CODE
	else if( FParse::Command( &Cmd, TEXT("DISCONNECT")) )
	{
		return HandleDisconnectCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("RECONNECT")) )
	{
		return HandleReconnectCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("TRAVEL") ) )
	{
		return HandleTravelCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("CE")))
	{
		return HandleCeCommand( InWorld, Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("GAMMA") ) )
	{
		return HandleGammaCommand( Cmd, Ar );
	}
#if STATS
	else if( FParse::Command( &Cmd, TEXT("DUMPPARTICLEMEM") ) )
	{
		return HandleDumpParticleMemCommand( Cmd, Ar );
	}
#endif

#if WITH_DUMPGPU
	else if (FParse::Command(&Cmd, TEXT("DUMPGPU")))
	{
		return HandleDumpGPUCommand(Cmd, Ar);
	}
#endif

#if WITH_GPUDEBUGCRASH
	else if (FParse::Command(&Cmd, TEXT("GPUDEBUGCRASH")))
	{
		return HandleGPUDebugCrashCommand(Cmd, Ar);
	}
#endif

#if UE_ENABLE_ARRAY_SLACK_TRACKING
	else if (FParse::Command(&Cmd, TEXT("SLACKREPORT")))
	{
		ArraySlackTrackGenerateReport(Cmd, Ar);
		return true;
	}
#endif

	else if ( FParse::Command(&Cmd,TEXT("SCALABILITY")) )
	{
		Scalability::ProcessCommand(Cmd, Ar);
		return true;
	}
	else if(IConsoleManager::Get().ProcessUserConsoleInput(Cmd, Ar, InWorld))
	{
		// console variable interaction (get value, set value or get help)
		return true;
	}
	else if (!IStreamingManager::HasShutdown() && IStreamingManager::Get().Exec( InWorld, Cmd,Ar ))
	{
		// The streaming manager has handled the exec command.
	}
	else if( FParse::Command(&Cmd, TEXT("DUMPTICKS")) )
	{
		return HandleDumpTicksCommand( InWorld, Cmd, Ar );
	}
#if USE_NETWORK_PROFILER
	else if( FParse::Command(&Cmd,TEXT("NETPROFILE")) )
	{
		GNetworkProfiler.Exec( InWorld, Cmd, Ar );
	}
#endif
	else 
	{
		return false;
	}

	return true;
}
#endif // UE_ALLOW_EXEC_COMMANDS

bool UEngine::Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("DumpConsoleCommands")))
	{
		return HandleDumpConsoleCommandsCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("REDIRECTTOFILE")))
	{
		return HandleRedirectOutputCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPAVAILABLERESOLUTIONS")))
	{
		return HandleDumpAvailableResolutionsCommand( Cmd, Ar );
	}
	else if(FParse::Command(&Cmd,TEXT("ANIMSEQSTATS")))
	{
		return HandleAnimSeqStatsCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("CountDisabledParticleItems")))
	{
		return HandleCountDisabledParticleItemsCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd, TEXT("VIEWNAMES") ) )
	{
		return HandleViewnamesCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("FREEZESTREAMING")) )
	{
		return HandleFreezeStreamingCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command(&Cmd,TEXT("FREEZEALL")) )
	{
		return HandleFreezeAllCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command(&Cmd,TEXT("ToggleRenderingThread")) )
	{
		return HandleToggleRenderingThreadCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("RecompileShaders")) )				    
	{
		return HandleRecompileShadersCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("RecompileGlobalShaders")) )
	{
		return HandleRecompileGlobalShadersCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("DUMPSHADERSTATS")) )
	{
		return HandleDumpShaderStatsCommand( Cmd, Ar );		
	}
	else if( FParse::Command(&Cmd,TEXT("DUMPMATERIALSTATS")) )
	{
		return HandleDumpMaterialStatsCommand( Cmd, Ar );	
	}
	else if (FParse::Command(&Cmd, TEXT("DumpShaderPipelineStats")))
	{
		return HandleDumpShaderPipelineStatsCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("visrt")))
	{
		extern bool HandleVisualizeRT();
		return HandleVisualizeRT();
	}
	else if( FParse::Command(&Cmd,TEXT("SHADERCOMPLEXITY")) )
	{
		return HandleShaderComplexityCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("FREEZERENDERING")) )
	{
		return HandleFreezeRenderingCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command(&Cmd,TEXT("SHOWLOG")) )
	{
		return HandleShowLogCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("STARTFPSCHART")) )
	{
		return HandleStartFPSChartCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("STOPFPSCHART")) )
	{
		return HandleStopFPSChartCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("DumpLevelScriptActors")))
	{
		return HandleDumpLevelScriptActorsCommand( InWorld, Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("KE")) || FParse::Command(&Cmd, TEXT("KISMETEVENT")))
	{
		return HandleKismetEventCommand( InWorld, Cmd, Ar );
	}
	else if(FParse::Command(&Cmd,TEXT("LISTTEXTURES")))
	{
		return HandleListTexturesCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("LISTSTATICMESHES")))
	{
		return HandleListStaticMeshesCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("LISTSKELETALMESHES")))
	{
		return HandleListSkeletalMeshesCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("LISTANIMS")))
	{
		return HandleListAnimsCommand( Cmd, Ar );
	}
	else if(FParse::Command(&Cmd,TEXT("REMOTETEXTURESTATS")))
	{
		return HandleRemoteTextureStatsCommand( Cmd, Ar );
	}
	else if(FParse::Command(&Cmd,TEXT("LISTPARTICLESYSTEMS")))
	{
		return HandleListParticleSystemsCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("LISTSPAWNEDACTORS")) )
	{
		return HandleListSpawnedActorsCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("MemReport") ) )
	{
		return HandleMemReportCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("MemReportDeferred") ) )
	{
		return HandleMemReportDeferredCommand( Cmd, Ar, InWorld );
	}
	else if ( FParse::Command( &Cmd, TEXT("SkeletalMeshReport") ))
	{
		return HandleSkeletalMeshReportCommand(Cmd, Ar, InWorld);
	}
	else if( FParse::Command( &Cmd, TEXT("PARTICLEMESHUSAGE") ) )
	{
		return HandleParticleMeshUsageCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd, TEXT("DUMPPARTICLECOUNTS") ) )
	{
		return HandleDumpParticleCountsCommand( Cmd, Ar );
	}
	// we can't always do an obj linkers, as cooked games have their linkers tossed out.  So we need to look at the actual packages which are loaded
	else if( FParse::Command( &Cmd, TEXT("ListLoadedPackages") ) )
	{
		return HandleListLoadedPackagesCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd,TEXT("MEM")) )
	{
		return HandleMemCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("LOGOUTSTATLEVELS")) )
	{
		return HandleLogoutStatLevelsCommand( Cmd, Ar, InWorld );
	}
	else if( FParse::Command( &Cmd, TEXT("DEBUG") ) )
	{
		return HandleDebugCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("MERGEMESH")))
	{
		return HandleMergeMeshCommand( Cmd, Ar, InWorld );
	}
	else if (FParse::Command(&Cmd, TEXT("CONTENTCOMPARISON")))
	{
		return HandleContentComparisonCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("TOGGLEGTPSYSLOD")))
	{
		return HandleTogglegtPsysLODCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("OBJ")) )
	{
		return HandleObjCommand( Cmd, Ar );
	}
	else if( FParse::Command( &Cmd, TEXT("TESTSLATEGAMEUI")) && InWorld && InWorld->IsGameWorld() )
	{
		return HandleTestslateGameUICommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("DIR")) )		// DIR [path\pattern]
	{
		return HandleDirCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("TRACKPARTICLERENDERINGSTATS")) )
	{
		return HandleTrackParticleRenderingStatsCommand( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd,TEXT("DUMPALLOCS")) )
	{
		return HandleDumpAllocatorStats( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd,TEXT("HEAPCHECK")) )
	{
		return HandleHeapCheckCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("TOGGLEONSCREENDEBUGMESSAGEDISPLAY")))
	{
		return HandleToggleOnscreenDebugMessageDisplayCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("TOGGLEONSCREENDEBUGMESSAGESYSTEM")))
	{
		return HandleToggleOnscreenDebugMessageSystemCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("DISABLEALLSCREENMESSAGES")) || FParse::Command(&Cmd,TEXT("DISABLESCREENMESSAGES")))
	{
		return HandleDisableAllScreenMessagesCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("ENABLEALLSCREENMESSAGES")) || FParse::Command(&Cmd,TEXT("ENABLESCREENMESSAGES")))
	{
		return HandleEnableAllScreenMessagesCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd,TEXT("TOGGLEALLSCREENMESSAGES")) || FParse::Command(&Cmd,TEXT("TOGGLESCREENMESSAGES")) || FParse::Command(&Cmd,TEXT("CAPTUREMODE")))
	{
		return HandleToggleAllScreenMessagesCommand( Cmd, Ar );
	}
	else if( FParse::Command(&Cmd,TEXT("CONFIGHASH")) )
	{
		return HandleConfigHashCommand( Cmd, Ar );
	}
	else if ( FParse::Command(&Cmd,TEXT("CONFIGMEM")) )
	{
		return HandleConfigMemCommand( Cmd, Ar );
	}
	else if (FParse::Command(&Cmd, TEXT("GETINI")))
	{
		return HandleGetIniCommand(Cmd, Ar);
	}
#endif // !UE_BUILD_SHIPPING

#if	!(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_HOT_RELOAD
	else if (FParse::Command(&Cmd, TEXT("HotReload")))
	{
		return HandleHotReloadCommand(Cmd, Ar);
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_HOT_RELOAD

	return false;
}

bool UEngine::Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_EDITOR && !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("DumpShaderCompileStats")))
	{
		return HandleDumpShaderCompileStatsCommand(Cmd, Ar);
	}
#endif

	return false;
}

bool UEngine::HandleFlushLogCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GLog->FlushThreadedLogs();
	GLog->Flush();
	return true;
}

bool UEngine::HandleGameVerCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	const TArray<FString, TInlineAllocator<4> > VersionDetails = {
		FString::Printf(TEXT("GameVersion Branch: %s"), *FApp::GetBranchName()),
		FString::Printf(TEXT("Configuration: %s"), LexToString( FApp::GetBuildConfiguration() )),
		FString::Printf(TEXT("Build: %s"), FApp::GetBuildVersion()),
		FString::Printf(TEXT("CommandLine: %s"), FCommandLine::Get()),
	};

	const bool bMultiline = FCString::Stristr(Cmd, TEXT("-m")) != nullptr;
	const FString VersionString = FString::Join(VersionDetails, bMultiline ? TEXT("\n") : TEXT(", "));

	Ar.Logf( TEXT("%s"), *VersionString );
	FPlatformApplicationMisc::ClipboardCopy( *VersionString );

	if (FCString::Stristr(Cmd, TEXT("-display")))
	{
		GEngine->AddOnScreenDebugMessage(-1, 5.0f, FColor::Red, VersionString);
	}

	return 1;
}

bool UEngine::HandleStatCommand( UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if STATS
	if (FParse::Command(&Cmd, TEXT("help")))
	{
		const TSet<FName>& StatGroupNames = FStatGroupGameThreadNotifier::Get().StatGroupNames;
		TArray<FString> StatNames;
		for (const FName& StatName : StatGroupNames)
		{
			FString StatNameStr = StatName.ToString();
			StatNameStr.RemoveFromStart(TEXT("STATGROUP_"));
			StatNames.Add(StatNameStr);
		}

		for (int32 StatIdx = 0; StatIdx < EngineStats.Num(); StatIdx++)
		{
			const FEngineStatFuncs& EngineStat = EngineStats[StatIdx];
			const FString EngineStatString = FString::Printf(TEXT("%s: %s"), *EngineStat.CommandNameString, *EngineStat.DescriptionString.ToString());
			StatNames.Add(EngineStatString);
		}

		StatNames.Sort();
		for (const FString& StatName : StatNames)
		{
			Ar.Logf(TEXT("%s"), *StatName);
		}

		return true;
	}
#endif //STATS

	const TCHAR* Temp = Cmd;
	for (int32 StatIdx = 0; StatIdx < EngineStats.Num(); StatIdx++)
	{
		const FEngineStatFuncs& EngineStat = EngineStats[StatIdx];
		if (FParse::Command( &Temp, *EngineStat.CommandNameString ))
		{
			if (EngineStat.ToggleFunc.IsBound())
			{
				return ViewportClient ? EngineStat.ToggleFunc.Execute(World, ViewportClient, Temp) : false;
			}
			return true;
		}
	}
	return false;
}

bool UEngine::HandleStopMovieCaptureCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (IMovieSceneCaptureInterface* CaptureInterface = IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture())
	{
		CaptureInterface->Close();
		return true;
	}
	return false;
}

bool UEngine::HandleCrackURLCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FURL URL(NULL,Cmd,TRAVEL_Absolute);
	if( URL.Valid )
	{
		Ar.Logf( TEXT("     Protocol: %s"), *URL.Protocol );
		Ar.Logf( TEXT("         Host: %s"), *URL.Host );
		Ar.Logf( TEXT("         Port: %i"), URL.Port );
		Ar.Logf( TEXT("          Map: %s"), *URL.Map );
		Ar.Logf( TEXT("   NumOptions: %i"), URL.Op.Num() );
		for( int32 i=0; i<URL.Op.Num(); i++ )
			Ar.Logf( TEXT("     Option %i: %s"), i, *URL.Op[i] );
		Ar.Logf( TEXT("       Portal: %s"), *URL.Portal );
		Ar.Logf( TEXT("       String: '%s'"), *URL.ToString() );
	}
	else Ar.Logf( TEXT("BAD URL") );
	return 1;
}

bool UEngine::HandleDeferCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	DeferredCommands.Add(Cmd);
	return 1;
}

bool UEngine::HandleCeCommand( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString ErrorMessage = "No level found for CE processing";
	bool bResult = false;

	// Try to execute the command on all level script actors
	for (TArray<ULevel*>::TConstIterator it = InWorld->GetLevels().CreateConstIterator(); it; ++it)
	{
		ULevel* CurrentLevel = *it;
		if (CurrentLevel)
		{

			if (CurrentLevel->GetLevelScriptActor())
			{
				ErrorMessage.Empty();

				// return true if at least one level handles the command
				bResult |= CurrentLevel->GetLevelScriptActor()->CallFunctionByNameWithArguments( Cmd, Ar, NULL, true );
			}
		}
	}

	if (!bResult)
	{
		ErrorMessage = FString::Printf(TEXT("CE command '%s' wasn't processed for levels from world '%s'."), Cmd, *InWorld->GetPathName());
	}

	if (!ErrorMessage.IsEmpty())
	{
		UE_LOGF( LogEngine, Error, "%ls", *ErrorMessage );
	}

	// the command was processed (resulted in executing the command or an error message) - no other spot handles "CE"
	return true;
}


bool UEngine::HandleDumpTicksCommand( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	bool bShowEnabled = true;
	bool bShowDisabled = true;
	bool bGrouped = false;
	if (FParse::Command( &Cmd, TEXT( "GROUPED" ) ))
	{
		bGrouped = true;
	}
	// Handle optional parameters, will dump all tick functions by default.
	else if (FParse::Command( &Cmd, TEXT( "ENABLED" ) ))
	{
		bShowDisabled = false;
	}
	else if (FParse::Command(&Cmd, TEXT("DISABLED")))
	{
		bShowEnabled = false;
	}

	FTickTaskManagerInterface::Get().DumpAllTickFunctions( Ar, InWorld, bShowEnabled, bShowDisabled, bGrouped);
	return true;
}

bool UEngine::HandleGammaCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	DisplayGamma = (*Cmd != 0) ? FMath::Clamp<float>( FCString::Atof( *FParse::Token( Cmd, false ) ), 0.5f, 5.0f ) : 2.2f;
	return true;
}

bool UEngine::HandleShowLogCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Toggle display of console log window.
	if (GLogConsole)
	{
		GLogConsole->Show( !GLogConsole->IsShown() );
	}
	return 1;
}

#if STATS
bool UEngine::HandleDumpParticleMemCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FParticleMemoryStatManager::DumpParticleMemoryStats( Ar );
	return true;
}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_HOT_RELOAD
bool UEngine::HandleHotReloadCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString Module( FParse::Token( Cmd, 0 ) );
	FString PackagePath( FString( "/Script/" ) + Module );
	UPackage *Package = FindPackage( NULL, *PackagePath );
	if (!Package)
	{
		Ar.Logf( TEXT( "Could not HotReload '%s', package not found in memory" ), *Module );
	}
	else
	{
		Ar.Logf( TEXT( "HotReloading %s..." ), *Module );
		TArray< UPackage*> PackagesToRebind;
		PackagesToRebind.Add( Package );
		IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>( "HotReload" );
		const ECompilationResult::Type CompilationResult = HotReloadSupport.RebindPackages( PackagesToRebind, EHotReloadFlags::WaitForCompletion, Ar );
	}
	return true;
}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_HOT_RELOAD

#if !UE_BUILD_SHIPPING

static void DumpHelp(UWorld* InWorld)
{
	FString FilePath = FPaths::ProjectSavedDir() + TEXT("ConsoleHelp.html");

	UE_LOGF(LogEngine, Display,
		"Console Help:\n" "=============\n" "\n" "A console variable is an engine wide key-value pair. The key is a string usually starting with the subsystem\n" "prefix followed by '.' e.g. r.BloomQuality. Values can have different types such as float, int, or string.\n" "\n" "Console variables can be put into ini files (e.g. ConsoleVariables.ini or BaseEngine.ini) with this syntax:\n" "<Console variable> = <value>\n" "\n" "A console command has no state associated with it and gets executed immediately.\n" "\n" "DumpCVars                   Lists all console variables\n" "DumpCCmds                   Lists all registered console commands. Not all commands are registered.\n" "DumpConsoleCommands         Lists all console commands, registered and unregistered\n" "\n" "<Console variable>          Get the console variable value and how it was last set\n" "<Console variable> ?        Get the console variable help text\n" "<Console variable> <value>  Set the console variable value\n" "\n" "<Console command> [params]  Execute the console command with optional parameters\n" "\n" "To browse console variables open this file: '%ls'", *FilePath);

	const bool bSuccess = ConsoleCommandLibrary_DumpLibraryHTML(InWorld, *GEngine, FilePath);

	// Notification in editor
#if WITH_EDITOR
	if (bSuccess && FPaths::FileExists(FilePath))
	{
		const FText Message = NSLOCTEXT("UnrealEd", "ConsoleHelpExported", "Console help file saved at:");
		FNotificationInfo Info(Message);
		Info.bFireAndForget = true;
		Info.ExpireDuration = 5.0f;
		Info.bUseSuccessFailIcons = false;
		Info.bUseLargeFont = false;

		const FString HyperLinkPath = FPaths::ConvertRelativePathToFull(FilePath);
		Info.SubText = FText::FromString(HyperLinkPath);
		Info.Hyperlink = FSimpleDelegate::CreateStatic([](FString SourceFilePath)
		{
			// open folder, you can choose the browser yourself
			FPlatformProcess::ExploreFolder(*(FPaths::GetPath(SourceFilePath)));
		}, HyperLinkPath);
		Info.HyperlinkText = NSLOCTEXT("UnrealEd", "ConsoleHelpFolderLink", "Open file location...");
		Info.WidthOverride = FOptionalSize();

		FSlateNotificationManager::Get().AddNotification(Info);

		// Always try to open the help file on Windows (including in -game, etc...)
#if PLATFORM_WINDOWS
		const FString LaunchableURL = FString("file://") + HyperLinkPath;
		FPlatformProcess::LaunchURL(*LaunchableURL, nullptr, nullptr);
#endif
	}
	else
	{
		const FText Message = NSLOCTEXT("UnrealEd", "ConsoleHelpExportFailed", "Console help file generation failed.");
		FNotificationInfo Info(Message);

		const FString DocTemplateFile = FPaths::Combine(FPaths::EngineDir(), "Documentation/Extras/ConsoleHelpTemplate.html");
		if (!FPaths::FileExists(DocTemplateFile))
		{
			Info.SubText = NSLOCTEXT("UnrealEd", "ConsoleHelpExportFailedTip", "The template HTML file could not be found in the Engine/Documentation/Extras folder.\n\nIf you use a version control system or Unreal Game Sync (UGS), check that this folder is not excluded by your sync settings.");
		}
		Info.bFireAndForget = true;
		Info.ExpireDuration = 10.0f;
		Info.bUseSuccessFailIcons = false;
		Info.bUseLargeFont = false;

		FSlateNotificationManager::Get().AddNotification(Info);
	}
#endif// WITH_EDITOR
}
static FAutoConsoleCommandWithWorld GConsoleCommandHelp(
	TEXT("help"),
	TEXT("Outputs some helptext to the console and the log"),
	FConsoleCommandWithWorldDelegate::CreateStatic(DumpHelp)
);

bool UEngine::HandleDumpConsoleCommandsCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	Ar.Logf(TEXT("DumpConsoleCommands: %s*"), Cmd);
	Ar.Logf(TEXT(""));
	ConsoleCommandLibrary_DumpLibrary(InWorld, *GEngine, Cmd, Ar);
	return true;
}

bool UEngine::HandleDumpAvailableResolutionsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE_LOGF(LogEngine, Log, "DumpAvailableResolutions");

	FScreenResolutionArray ResArray;
	if (RHIGetAvailableResolutions(ResArray, false))
	{
		for (int32 ModeIndex = 0; ModeIndex < ResArray.Num(); ModeIndex++)
		{
			FScreenResolutionRHI& ScreenRes = ResArray[ModeIndex];
			UE_LOGF(LogEngine, Log, "DefaultAdapter - %4d x %4d @ %d", 
				ScreenRes.Width, ScreenRes.Height, ScreenRes.RefreshRate);
		}
	}
	else
	{
		UE_LOGF(LogEngine, Log, "Failed to get available resolutions!");
	}
	return true;
}
bool UEngine::HandleAnimSeqStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	extern void GatherAnimSequenceStats(FOutputDevice& Ar);
	GatherAnimSequenceStats( Ar );
	return true;
}

bool UEngine::HandleCountDisabledParticleItemsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	int32 ParticleSystemCount = 0;
	int32 EmitterCount = 0;
	int32 DisabledEmitterCount = 0;
	int32 CookedOutEmitterCount = 0;
	int32 LODLevelCount = 0;
	int32 DisabledLODLevelCount = 0;
	int32 ModuleCount = 0;
	int32 DisabledModuleCount = 0;
	TMap<FString, int32> ModuleMap;
	for (TObjectIterator<UParticleSystem> It; It; ++It)
	{
		ParticleSystemCount++;

		TArray<UParticleModule*> ProcessedModules;
		TArray<UParticleModule*> DisabledModules;

		UParticleSystem* PSys = *It;
		for (int32 EmitterIdx = 0; EmitterIdx < PSys->Emitters.Num(); EmitterIdx++)
		{
			UParticleEmitter* Emitter = PSys->Emitters[EmitterIdx];
			if (Emitter != NULL)
			{
				bool bDisabledEmitter = true;
				EmitterCount++;
				if (Emitter->bCookedOut == true)
				{
					CookedOutEmitterCount++;
				}
				for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
				{
					UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
					if (LODLevel != NULL)
					{
						LODLevelCount++;
						if (LODLevel->bEnabled == false)
						{
							DisabledLODLevelCount++;
						}
						else
						{
							bDisabledEmitter = false;
						}
						for (int32 ModuleIdx = -3; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
						{
							UParticleModule* Module = NULL;
							switch (ModuleIdx)
							{
							case -3:	Module = LODLevel->RequiredModule;		break;
							case -2:	Module = LODLevel->SpawnModule;			break;
							case -1:	Module = LODLevel->TypeDataModule;		break;
							default:	Module = LODLevel->Modules[ModuleIdx];	break;
							}

							int32 DummyIdx;
							if ((Module != NULL) && (ProcessedModules.Find(Module, DummyIdx) == false))
							{
								ModuleCount++;
								ProcessedModules.AddUnique(Module);
								if (Module->bEnabled == false)
								{
									check(DisabledModules.Find(Module, DummyIdx) == false);
									DisabledModules.AddUnique(Module);
									DisabledModuleCount++;
								}

								FString ModuleName = Module->GetClass()->GetName();
								int32* ModuleCounter = ModuleMap.Find(ModuleName);
								if (ModuleCounter == NULL)
								{
									int32 TempInt = 0;
									ModuleMap.Add(ModuleName, TempInt);
									ModuleCounter = ModuleMap.Find(ModuleName);
								}
								check(ModuleCounter != NULL);
								*ModuleCounter = (*ModuleCounter + 1);
							}
						}
					}
				}

				if (bDisabledEmitter)
				{
					DisabledEmitterCount++;
				}
			}
		}
	}

	UE_LOGF(LogEngine, Log, "%5d particle systems w/ %7d emitters (%5d disabled or %5.3f%% - %4d cookedout)",
		ParticleSystemCount, EmitterCount, DisabledEmitterCount, (float)DisabledEmitterCount / (float)EmitterCount, CookedOutEmitterCount);
	UE_LOGF(LogEngine, Log, "\t%8d lodlevels (%5d disabled or %5.3f%%)",
		LODLevelCount, DisabledLODLevelCount, (float)DisabledLODLevelCount / (float)LODLevelCount);
	UE_LOGF(LogEngine, Log, "\t\t%10d modules (%5d disabled or %5.3f%%)",
		ModuleCount, DisabledModuleCount, (float)DisabledModuleCount / (float)ModuleCount);
	for (TMap<FString,int32>::TIterator It(ModuleMap); It; ++It)
	{
		FString ModuleName = It.Key();
		int32 ModuleCounter = It.Value();

		UE_LOGF(LogEngine, Log, "\t\t\t%4d....%ls", ModuleCounter, *ModuleName);
	}

	return true;
}

// View all names or the last N names added to the name table. Useful for tracking down name table bloat
bool UEngine::HandleViewnamesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	const TArray<const FNameEntry*> Entries = FName::DebugDump();
	TConstArrayView<const FNameEntry*> IterEntries = Entries;

	if(int32 NumLast = 0; FParse::Value(Cmd, TEXT("NUM="), NumLast))
	{ 
		IterEntries.RightInline(NumLast);
	}

#if UE_FNAME_OUTLINE_NUMBER
	bool bNumbered = FParse::Param(Cmd, TEXT("NUMBERED"));
	bool bPlain = FParse::Param(Cmd, TEXT("PLAIN"));
	if (!bNumbered && !bPlain)
	{
		bNumbered = bPlain = true;
	}
#endif

	for (const FNameEntry* Entry : IterEntries)
	{	
#if UE_FNAME_OUTLINE_NUMBER
		if( Entry->IsNumbered() && bNumbered)
		{
			Entry->DebugDump(Ar);
		}
		else if (!Entry->IsNumbered() && bPlain)
#endif
		{
			Entry->DebugDump(Ar);
		}
	}

	return true;
}

bool UEngine::HandleFreezeStreamingCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	ProcessToggleFreezeStreamingCommand(InWorld);
	return true;
}

bool UEngine::HandleFreezeAllCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld  )
{
	ProcessToggleFreezeCommand( InWorld );
	ProcessToggleFreezeStreamingCommand(InWorld);
	return true;
}

extern void ToggleFreezeFoliageCulling();

bool UEngine::HandleFreezeRenderingCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld  )
{
	ProcessToggleFreezeCommand( InWorld );

	ToggleFreezeFoliageCulling();

	return true;
}

bool UEngine::HandleShaderComplexityCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString FlagStr(FParse::Token(Cmd, 0));
	if( FlagStr.Len() > 0 )
	{
		if( FCString::Stricmp(*FlagStr,TEXT("MAX"))==0)
		{
			float NewMax = FCString::Atof(Cmd);
			if (NewMax > 0.0f)
			{
				GEngine->MaxPixelShaderAdditiveComplexityCount = NewMax;
			}
		}
		else
		{
			Ar.Logf( TEXT("Format is 'shadercomplexity [toggleadditive] [togglepixel] [max $int]"));
			return true;
		}

		float CurrentMax = GEngine->MaxPixelShaderAdditiveComplexityCount;

		Ar.Logf( TEXT("New ShaderComplexity Settings: Max = %f"), CurrentMax);
	} 
	else
	{
		Ar.Logf( TEXT("Format is 'shadercomplexity [max $int]"));
	}
	return true; 
}

bool UEngine::HandleToggleRenderingThreadCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	check(IsInGameThread());
	GPendingUseThreadedRendering = !GIsThreadedRendering;

	Ar.Logf(TEXT("RenderThread will be %s."), !GIsThreadedRendering ? TEXT("enabled") : TEXT("disabled"));
	return true;
}

bool UEngine::HandleRecompileShadersCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	return RecompileShaders(Cmd, Ar);
}

bool UEngine::HandleRecompileGlobalShadersCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	extern void RecompileGlobalShaders();
	RecompileGlobalShaders();
	return 1;
}

bool UEngine::HandleDumpShaderStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString FlagStr(FParse::Token(Cmd, 0));
	EShaderPlatform Platform = GMaxRHIShaderPlatform;
	if (FlagStr.Len() > 0)
	{
		Platform = ShaderFormatToLegacyShaderPlatform(FName(*FlagStr));
	}
	Ar.Logf(TEXT("Dumping shader stats for platform %s"), *LegacyShaderPlatformToShaderFormat(Platform).ToString());
	// Dump info on all loaded shaders regardless of platform and frequency.
	DumpShaderStats( Platform, SF_NumFrequencies );
	return true;
}

bool UEngine::HandleDumpMaterialStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString FlagStr(FParse::Token(Cmd, 0));
	EShaderPlatform Platform = GMaxRHIShaderPlatform;
	if (FlagStr.Len() > 0)
	{
		Platform = ShaderFormatToLegacyShaderPlatform(FName(*FlagStr));
	}
	Ar.Logf(TEXT("Dumping material stats for platform %s"), *LegacyShaderPlatformToShaderFormat(Platform).ToString());
	// Dump info on all loaded shaders regardless of platform and frequency.
	extern ENGINE_API void DumpMaterialStats(EShaderPlatform);
	DumpMaterialStats( Platform );
	return true;
}

#if WITH_EDITOR
bool UEngine::HandleDumpShaderCompileStatsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	Ar.Logf(TEXT("Dumping shader compile stats"));
	GShaderCompilerStats->WriteStats(&Ar);
	return true;
}
#endif

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if WITH_DUMPGPU

bool UEngine::HandleDumpGPUCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!FApp::CanEverRender())
	{
		return true;
	}

	check(IsInGameThread());

	TArray<FString> Args;
	FString ResourceDumpDirectory = FRDGBuilder::BeginResourceDump(Cmd, TEXT("DumpGPUCommand"));

	if (!ResourceDumpDirectory.IsEmpty())
	{
		// Dump a screenshot
		if (GDumpGPUScreenshot.GetValueOnGameThread())
		{
			FString FileName = ResourceDumpDirectory / TEXT("Base/Screenshot.png");
			FScreenshotRequest::RequestScreenshot(FileName, /* bShowUI = */ true, /* bAddFilenameSuffix = */ false);
		}
	}

	return true;
}

#endif // WITH_DUMPGPU

#if WITH_GPUDEBUGCRASH
bool UEngine::HandleGPUDebugCrashCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!FApp::CanEverRender())
	{
		return true;
	}

	TArray<FString> Args;
	Ar.Logf(TEXT("Scheduling GPU crash."));

	bool bSupportsPageFaultsAndPlatformBreak = (GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D11 || GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::OpenGL) ? false : true;

	// schedule the GPU crash
	if (FParse::Command(&Cmd, TEXT("pagefault")) && bSupportsPageFaultsAndPlatformBreak)
	{
		GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::Type_PageFault;
	}
	if (FParse::Command(&Cmd, TEXT("platformbreak")) && bSupportsPageFaultsAndPlatformBreak)
	{
		GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::Type_PlatformBreak;
	}
	if (FParse::Command(&Cmd, TEXT("assert")) && bSupportsPageFaultsAndPlatformBreak)
	{
		GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::Type_Assert;
	}
	if (FParse::Command(&Cmd, TEXT("cmdlistcorruption")))
	{
		GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::Type_CmdListCorruption;
	}
	if (FParse::Command(&Cmd, TEXT("hang")) || !bSupportsPageFaultsAndPlatformBreak)
	{
		if (!bSupportsPageFaultsAndPlatformBreak)
		{
			Ar.Logf(TEXT("Current RHI does not support GPU page faults on demand or PLATFORM_BREAK(). Falling back to GPU hang for requested crash."));
		}
		GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::Type_Hang;
	}
	
	if (GRHIGlobals.TriggerGPUCrash == ERequestedGPUCrash::None)
	{
		Ar.Logf(TEXT("Please specify a crash type for your requested GPU crash. Crash will be scheduled on direct queue unless compute queue is specified. Usage: gpudebugcrash [hang|pagefault|platformbreak [compute]]"));
	}
	else
	{
		if (FParse::Command(&Cmd, TEXT("compute")))
		{
			GRHIGlobals.TriggerGPUCrash |= ERequestedGPUCrash::Queue_Compute;
		}
		else
		{
			GRHIGlobals.TriggerGPUCrash |= ERequestedGPUCrash::Queue_Direct;
		}
	}

	return true;
}
#endif

#if !UE_BUILD_SHIPPING
bool UEngine::HandleStartFPSChartCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// start the chart data capture
	const FString Label = FParse::Token(Cmd, 0);
	StartFPSChart( Label, /*bRecordPerFrameTimes=*/ true );
	return true;
}

bool UEngine::HandleStopFPSChartCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	// stop the chart data capture and log it
	const FString MapName = InWorld ? InWorld->GetMapName() : TEXT("None");
	StopFPSChart( MapName );
	return true;
}

bool UEngine::HandleDumpLevelScriptActorsCommand( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Dumps the object properties for all level script actors
	for( TArray<ULevel*>::TConstIterator  it = InWorld->GetLevels().CreateConstIterator(); it; ++it )
	{
		ULevel* CurrentLevel = *it;
		if( CurrentLevel && CurrentLevel->GetLevelScriptActor() )
		{
			AActor* LSActor = CurrentLevel->GetLevelScriptActor();
			if( LSActor )
			{
				UE_LOGF(LogEngine, Log, "--- %ls (%ls) ---", *LSActor->GetName(), *LSActor->GetOutermost()->GetName())
					for( TFieldIterator<FProperty> PropertyIt(LSActor->GetClass(), EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt )
					{
						FObjectPropertyBase* MyProperty = CastField<FObjectPropertyBase>(*PropertyIt);
						if( MyProperty )
						{
							UObject* PointedObject = NULL;
							PointedObject = MyProperty->GetObjectPropertyValue_InContainer(LSActor);

							if( PointedObject != NULL )
							{
								UObject* PointedOutermost = PointedObject->GetOutermost();
								UE_LOGF(LogEngine, Log, "%ls: %ls (%ls)", *MyProperty->GetName(), *PointedObject->GetName(), *PointedOutermost->GetName());
							}

						}
					}
			}
		}
	}
	return true;
}

bool UEngine::HandleKismetEventCommand(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	FString const ObjectName = FParse::Token(Cmd, 0);
	if (ObjectName == TEXT("*"))
	{
		// TObjectIterator excludes RF_ClassDefaultObjects but not default SubObjects which belong to them, so we'll see if this object belongs to a CDO.
		const auto IsCDOOwnedArchtype = [](UObject* InObject)
		{
			if (!InObject->HasAnyFlags(RF_ArchetypeObject))
			{
				return false;
			}

			InObject = InObject->GetOuter();
			while (InObject)
			{
				if (InObject->HasAnyFlags(RF_ClassDefaultObject))
				{
					return true;
				}

				InObject = InObject->GetOuter();
			}

			return false;
		};

		// Gather all non-CDO, non-default subobject objects in the target world.
		// Call the function in a separate pass to avoid deadlocks from the target
		// function triggering and waiting for off-thread work that needs the 
		// GUObjectArray lock, like FlushAsyncLoading.
		int32 NumInstanceCallsSucceeded = 0;
		TArray<UObject*> CandidateObjects;
		for (TObjectIterator<UObject> It(EObjectFlags::RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UObject* const Obj = *It;

			const bool bIsCDOOwnedArchtype = IsCDOOwnedArchtype(Obj);
			if (!bIsCDOOwnedArchtype)
			{
				UWorld const* const ObjWorld = Obj->GetWorld();
				if (ObjWorld == InWorld)
				{
					CandidateObjects.Add(Obj);
				}
			}
		}

		// Try to call the target function on all candidate objects. Only succeeds
		// if the object has a matching function and signature.
		for (UObject* Obj : CandidateObjects)
		{
			if (IsValid(Obj))
			{
				const bool bSucceeded = Obj->CallFunctionByNameWithArguments(Cmd, Ar, nullptr, true);
				NumInstanceCallsSucceeded += bSucceeded ? 1 : 0;
			}
		}

		Ar.Logf(TEXT("Called '%s' on everything in the world and %d instances succeeded"), Cmd, NumInstanceCallsSucceeded);
	}
	else
	{
		UObject* ObjectToMatch = FindFirstObject<UObject>(*ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("HandleKismetEventCommand"));

		if (ObjectToMatch == nullptr)
		{
			Ar.Logf(TEXT("Failed to find an object named '%s'.  Specify a valid object name, class name, or * (if using a class name for a BP, don't forget the _C)"), *ObjectName);
		}
		else
		{
			if (UClass* ClassToMatch = Cast<UClass>(ObjectToMatch))
			{
				// Gather all eligible instances of the class. 
				// Call the function in a separate pass to avoid deadlocks from the target
				// function triggering and waiting for off-thread work that needs the 
				// GUObjectArray lock, like FlushAsyncLoading.
				int32 NumInstancesFound = 0;
				int32 NumInstanceCallsSucceeded = 0;
				TArray<UObject*> MatchingObjects;
				for (FThreadSafeObjectIterator It(ClassToMatch); It; ++It)
				{
					UObject* const Obj = *It;
					UWorld const* const ObjWorld = Obj->GetWorld();
					if (ObjWorld == InWorld)
					{
						++NumInstancesFound;
						MatchingObjects.Add(Obj);
					}
				}

				// Call the target function all eligible objects.
				for (UObject* Obj : MatchingObjects)
				{
					if (IsValid(Obj))
					{
						const bool bSucceeded = Obj->CallFunctionByNameWithArguments(Cmd, Ar, nullptr, true);
						NumInstanceCallsSucceeded += bSucceeded ? 1 : 0;
					}
				}

				Ar.Logf(TEXT("Called '%s' on %d instance(s) of class '%s' (%d succeeded)"), Cmd, NumInstancesFound, *ClassToMatch->GetPathName(), NumInstanceCallsSucceeded);
			}
			else
			{
				// Call it on the specific instance specified
				const bool bSucceeded = ObjectToMatch->CallFunctionByNameWithArguments(Cmd, Ar, nullptr, true);
				Ar.Logf(TEXT("Called '%s' on instance '%s' (call %s)"), Cmd, *ObjectToMatch->GetPathName(), bSucceeded ? TEXT("succeeded") : TEXT("failed"));
			}
		}
	}

	return true;
}

bool UEngine::HandleListTexturesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	const bool bShouldOnlyListStreaming = FParse::Command(&Cmd, TEXT("STREAMING"));
	const bool bShouldOnlyListNonStreaming = FParse::Command(&Cmd, TEXT("NONSTREAMING")) && !bShouldOnlyListStreaming;
	const bool bShouldOnlyListForced = FParse::Command(&Cmd, TEXT("FORCED")) && !bShouldOnlyListStreaming && !bShouldOnlyListNonStreaming;
	const bool bShouldOnlyListUncompressed = FParse::Command(&Cmd, TEXT("UNCOMPRESSED")) && !bShouldOnlyListStreaming && !bShouldOnlyListNonStreaming && !bShouldOnlyListForced;
	const bool bShouldOnlyListNonVT = FParse::Command(&Cmd, TEXT("NONVT")) && !bShouldOnlyListStreaming && !bShouldOnlyListNonStreaming && !bShouldOnlyListForced && !bShouldOnlyListUncompressed;
	const bool bListUnused = FParse::Command(&Cmd, TEXT("UNUSED"));
	const bool bAlphaSort = FParse::Param( Cmd, TEXT("ALPHASORT") );
	const bool bCSV = FParse::Param( Cmd, TEXT("CSV") );
	const bool bUnknownRefOnly = FParse::Param(Cmd, TEXT("unknownrefonly"));

	Ar.Logf(TEXT("Listing %s%s%s textures."),
		bShouldOnlyListForced ? TEXT("forced") : bShouldOnlyListNonStreaming ? TEXT("non streaming") : bShouldOnlyListStreaming ? TEXT("streaming") : bShouldOnlyListUncompressed ? TEXT("uncompressed") : bShouldOnlyListNonVT ? TEXT("NONVT") : TEXT("all"),
		bUnknownRefOnly ? TEXT(" unknown-ref") : TEXT(""),
		bListUnused ? TEXT(" unused") : TEXT(""));

	// Find out how many times a texture is referenced by primitive components.
	TMap<UTexture*,int32> TextureToUsageMap;
	for( TObjectIterator<UPrimitiveComponent> It; It; ++It )
	{
		UPrimitiveComponent* PrimitiveComponent = *It;

		// Use the existing texture streaming functionality to gather referenced textures. Worth noting
		// that GetStreamingTextureInfo doesn't check whether a texture is actually streamable or not
		// and is also implemented for skeletal meshes and such.
		FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), PrimitiveComponent);
		TArray<FStreamingRenderAssetPrimitiveInfo> StreamingTextures;
		PrimitiveComponent->GetStreamingRenderAssetInfo( LevelContext, StreamingTextures );

		// Increase usage count for all referenced textures
		for( int32 TextureIndex=0; TextureIndex<StreamingTextures.Num(); TextureIndex++ )
		{
			UTexture* Texture = Cast<UTexture>(StreamingTextures[TextureIndex].RenderAsset);
			if( Texture )
			{
				// Initializes UsageCount to 0 if texture is not found.
				int32 UsageCount = TextureToUsageMap.FindRef( Texture );
				TextureToUsageMap.Add(Texture, UsageCount+1);
			}
		}
	}

	FRenderAssetStreamingManager* Streamer = nullptr;
	if (!IStreamingManager::HasShutdown() && !!CVarSetTextureStreaming.GetValueOnAnyThread())
	{
		Streamer = (FRenderAssetStreamingManager*)&IStreamingManager::Get().GetRenderAssetStreamingManager();
		Streamer->UpdateResourceStreaming(0.f, true);
	}

	int32 NumApplicableToMinSize = 0;
	// Collect textures.
	TArray<FSortedTexture> SortedTextures;
	for( TObjectIterator<UTexture> It; It; ++It )
	{
		UTexture*			Texture				= *It;
		UTexture2D*			Texture2D			= Cast<UTexture2D>(Texture);
		UTexture2DArray*	Texture2DArray		= Cast<UTexture2DArray>(Texture);
		UTextureCube*		TextureCube			= Cast<UTextureCube>(Texture);
		UVolumeTexture*		Texture3D			= Cast<UVolumeTexture>(Texture);
		UTextureRenderTarget2D* RenderTexture	= Cast<UTextureRenderTarget2D>(Texture);

		int32				LODGroup			= Texture->LODGroup;
		int32				NumMips				= 1;
		int32				MaxResLODBias		= Texture->GetCachedLODBias();;
		int32				MaxAllowedSizeX		= FMath::Max<int32>(static_cast<int32>(Texture->GetSurfaceWidth()) >> MaxResLODBias, 1);
		int32				MaxAllowedSizeY		= FMath::Max<int32>(static_cast<int32>(Texture->GetSurfaceHeight()) >> MaxResLODBias, 1);
		EPixelFormat		Format				= PF_Unknown;
		int32				DroppedMips			= MaxResLODBias;
		int32				CurSizeX			= FMath::Max<int32>(static_cast<int32>(Texture->GetSurfaceWidth()) >> DroppedMips, 1);
		int32				CurSizeY			= FMath::Max<int32>(static_cast<int32>(Texture->GetSurfaceHeight()) >> DroppedMips, 1);
		bool				bIsStreamingTexture = Texture->GetStreamingIndex() != INDEX_NONE;
		int32				MaxAllowedSize		= Texture->CalcTextureMemorySizeEnum(TMC_AllMipsBiased);
		int32				CurrentSize			= Texture->CalcTextureMemorySizeEnum(TMC_ResidentMips);
		int32				UsageCount			= TextureToUsageMap.FindRef(Texture);
		bool				bIsForced			= Texture->ShouldMipLevelsBeForcedResident() && bIsStreamingTexture;
		bool				bIsVirtual			= Texture->IsCurrentlyVirtualTextured();
		bool				bIsUncompressed		= Texture->IsUncompressed();

		bool bUnknownRef = false;
		if (Streamer)
		{
			FStreamingRenderAsset* Asset = Streamer->GetStreamingRenderAsset(Texture);
			if (Asset)
			{
				bUnknownRef = Asset->GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic);
			}
		}

		if (bUnknownRefOnly && !bUnknownRef)
		{
			continue;
		}

		if (Texture2D != nullptr)
		{
			NumMips				= Texture2D->GetNumMips();
			MaxResLODBias		= NumMips - Texture2D->GetNumMipsAllowed(false);
			MaxAllowedSizeX		= FMath::Max<int32>(Texture2D->GetSizeX() >> MaxResLODBias, 1);
			MaxAllowedSizeY		= FMath::Max<int32>(Texture2D->GetSizeY() >> MaxResLODBias, 1);
			Format				= Texture2D->GetPixelFormat();
			DroppedMips			= Texture2D->GetNumMips() - Texture2D->GetNumResidentMips();
			CurSizeX			= FMath::Max<int32>(Texture2D->GetSizeX() >> DroppedMips, 1);
			CurSizeY			= FMath::Max<int32>(Texture2D->GetSizeY() >> DroppedMips, 1);

			if ((NumMips >= Texture2D->GetMinTextureResidentMipCount()) && bIsStreamingTexture)
			{
				NumApplicableToMinSize++;
			}
		}
		else if (Texture2DArray != nullptr)
		{
			NumMips				= Texture2DArray->GetNumMips();
			Format				= Texture2DArray->GetPixelFormat();

			DroppedMips			= Texture2DArray->GetNumMips() - Texture2DArray->GetNumResidentMips();
			CurSizeX			= FMath::Max<int32>(Texture2DArray->GetSizeX() >> DroppedMips, 1);
			CurSizeY			= FMath::Max<int32>(Texture2DArray->GetSizeY() >> DroppedMips, 1);
		}
		else if (TextureCube != nullptr)
		{
			NumMips				= TextureCube->GetNumMips();
			Format				= TextureCube->GetPixelFormat();
		}
		else if (Texture3D != nullptr)
		{
			NumMips				= Texture3D->GetNumMips();
			Format				= Texture3D->GetPixelFormat();
		}
		else if (RenderTexture != nullptr)
		{
			NumMips = RenderTexture->GetNumMips();
			Format = RenderTexture->GetFormat();
		}

		if (bListUnused && UsageCount != 0)
		{
			continue;
		}

		if( (bShouldOnlyListStreaming && bIsStreamingTexture) ||	
			(bShouldOnlyListNonStreaming && !bIsStreamingTexture) ||
			(bShouldOnlyListForced && bIsForced) ||
			(bShouldOnlyListUncompressed && bIsUncompressed) ||
			(bShouldOnlyListNonVT && !bIsVirtual) ||
			(!bShouldOnlyListStreaming && !bShouldOnlyListNonStreaming && !bShouldOnlyListForced && !bShouldOnlyListUncompressed && !bShouldOnlyListNonVT) )
		{
			new(SortedTextures) FSortedTexture( 
				MaxAllowedSizeX,
				MaxAllowedSizeY,
				Format,
				CurSizeX,
				CurSizeY,
				MaxResLODBias, 
				MaxAllowedSize,
				CurrentSize,
				Texture->GetPathName(), 
				LODGroup, 
				bIsStreamingTexture,
				bUnknownRef,
				bIsVirtual,
				UsageCount,
				NumMips,
				bIsUncompressed);
		}
	}

	// Sort textures by cost.
	SortedTextures.Sort( FCompareFSortedTexture( bAlphaSort ) );

	// Retrieve mapping from LOD group enum value to text representation.
	TArray<FString> TextureGroupNames = UTextureLODSettings::GetTextureGroupNames();

	TArray<uint64> TextureGroupCurrentSizes;
	TArray<uint64> TextureGroupMaxAllowedSizes;

	TArray<uint64> FormatCurrentSizes;
	TArray<uint64> FormatMaxAllowedSizes;

	TextureGroupCurrentSizes.AddZeroed(TextureGroupNames.Num());
	TextureGroupMaxAllowedSizes.AddZeroed(TextureGroupNames.Num());

	FormatCurrentSizes.AddZeroed(PF_MAX);
	FormatMaxAllowedSizes.AddZeroed(PF_MAX);

	// Display.
	uint64 TotalMaxAllowedSize = 0;
	uint64 TotalCurrentSize	= 0;

	if (bCSV)
	{
		Ar.Logf(TEXT("Max Width,Max Height,Max Size (KB),Bias Authored,Current Width,Current Height,Current Size (KB),Format,LODGroup,Name,Streaming,UnknownRef,VT,Usage Count,NumMips,Uncompressed"));
	}
	else if (!FPlatformProperties::RequiresCookedData())
	{
		Ar.Logf(TEXT("MaxAllowedSize: Width x Height (Size in KB, Authored Bias), Current/InMem: Width x Height (Size in KB), Format, LODGroup, Name, Streaming, UnknownRef, VT, Usage Count, NumMips, Uncompressed"));
	}
	else
	{
		Ar.Logf(TEXT("Cooked/OnDisk: Width x Height (Size in KB, Authored Bias), Current/InMem: Width x Height (Size in KB), Format, LODGroup, Name, Streaming, UnknownRef, VT, Usage Count, NumMips, Uncompressed"));
	}

	for( int32 TextureIndex=0; TextureIndex<SortedTextures.Num(); TextureIndex++ )
	{
		const FSortedTexture& SortedTexture = SortedTextures[TextureIndex];
		const bool bValidTextureGroup = TextureGroupNames.IsValidIndex(SortedTexture.LODGroup);

		FString AuthoredBiasString("?");
		if (!FPlatformProperties::RequiresCookedData())
		{
			AuthoredBiasString.Empty();
			AuthoredBiasString.AppendInt(SortedTexture.LODBias);
		}

		if (bCSV)
		{
			Ar.Logf(TEXT("%i,%i,%i,%s,%i,%i,%i,%s,%s,%s,%s,%s,%s,%i,%i,%s"),
				SortedTexture.MaxAllowedSizeX, SortedTexture.MaxAllowedSizeY, (SortedTexture.MaxAllowedSize + 512) / 1024,
				*AuthoredBiasString,
				SortedTexture.CurSizeX, SortedTexture.CurSizeY, (SortedTexture.CurrentSize + 512) / 1024,
				GetPixelFormatString(SortedTexture.Format),
				bValidTextureGroup ? *TextureGroupNames[SortedTexture.LODGroup] : TEXT("INVALID"),
				*SortedTexture.Name,
				SortedTexture.bIsStreaming ? TEXT("YES") : TEXT("NO"),
				SortedTexture.bIsUnknownRef ? TEXT("YES") : TEXT("NO"),
				SortedTexture.bIsVirtual ? TEXT("YES") : TEXT("NO"),
				SortedTexture.UsageCount,
				SortedTexture.NumMips,
				SortedTexture.bIsUncompressed ? TEXT("YES") : TEXT("NO"));
		}
		else
		{
			Ar.Logf(TEXT("%ix%i (%i KB, %s), %ix%i (%i KB), %s, %s, %s, %s, %s, %s, %i, %i, %s"),
				SortedTexture.MaxAllowedSizeX, SortedTexture.MaxAllowedSizeY, (SortedTexture.MaxAllowedSize + 512) / 1024,
				*AuthoredBiasString,
				SortedTexture.CurSizeX, SortedTexture.CurSizeY, (SortedTexture.CurrentSize + 512) / 1024,
				GetPixelFormatString(SortedTexture.Format),
				bValidTextureGroup ? *TextureGroupNames[SortedTexture.LODGroup] : TEXT("INVALID"),
				*SortedTexture.Name,
				SortedTexture.bIsStreaming ? TEXT("YES") : TEXT("NO"),
				SortedTexture.bIsUnknownRef ? TEXT("YES") : TEXT("NO"),
				SortedTexture.bIsVirtual ? TEXT("YES") : TEXT("NO"),
				SortedTexture.UsageCount,
				SortedTexture.NumMips,
				SortedTexture.bIsUncompressed ? TEXT("YES") : TEXT("NO"));
		}

		if (bValidTextureGroup)
		{
			TextureGroupCurrentSizes[SortedTexture.LODGroup] += SortedTexture.CurrentSize;
			TextureGroupMaxAllowedSizes[SortedTexture.LODGroup] += SortedTexture.MaxAllowedSize;
		}

		if (SortedTexture.Format >= 0 && SortedTexture.Format < PF_MAX)
		{
			FormatCurrentSizes[SortedTexture.Format] += SortedTexture.CurrentSize;
			FormatMaxAllowedSizes[SortedTexture.Format] += SortedTexture.MaxAllowedSize;
		}

		TotalMaxAllowedSize	+= SortedTexture.MaxAllowedSize;
		TotalCurrentSize	+= SortedTexture.CurrentSize;
	}

	Ar.Logf(TEXT("Total size: InMem= %.2f MB  OnDisk= %.2f MB  Count=%d, CountApplicableToMin=%d"), (double)TotalCurrentSize / 1024. / 1024., (double)TotalMaxAllowedSize / 1024. / 1024., SortedTextures.Num(), NumApplicableToMinSize);
	for (int32 i = 0; i < PF_MAX; ++i)
	{
		if (FormatCurrentSizes[i] > 0 || FormatMaxAllowedSizes[i] > 0)
		{
			Ar.Logf(TEXT("Total %s size: InMem= %.2f MB  OnDisk= %.2f MB "), GetPixelFormatString((EPixelFormat)i), (double)FormatCurrentSizes[i] / 1024. / 1024., (double)FormatMaxAllowedSizes[i] / 1024. / 1024.);
		}
	}

	for (int32 i = 0; i < TextureGroupCurrentSizes.Num(); ++i)
	{
		if (TextureGroupCurrentSizes[i] > 0 || TextureGroupMaxAllowedSizes[i] > 0)
		{
			Ar.Logf(TEXT("Total %s size: InMem= %.2f MB  OnDisk= %.2f MB "), *TextureGroupNames[i], (double)TextureGroupCurrentSizes[i] / 1024. / 1024., (double)TextureGroupMaxAllowedSizes[i] / 1024. / 1024.);
		}
	}
	return true;
}

bool UEngine::HandleListStaticMeshesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	FCompareFSortedStaticMesh::ESortType SortType = FCompareFSortedStaticMesh::ESortType::Default;
	SortType = FParse::Param(Cmd, TEXT("ALPHASORT"))? FCompareFSortedStaticMesh::ESortType::Alpha : SortType;
	SortType = FParse::Param(Cmd, TEXT("MOBILESORT")) ? FCompareFSortedStaticMesh::ESortType::Mobile : SortType;
	SortType = FParse::Param(Cmd, TEXT("INSTANCESORT")) ? FCompareFSortedStaticMesh::ESortType::InstanceCount : SortType;

	const bool bCSV = FParse::Param(Cmd, TEXT("CSV"));
	const bool bUsedComponents = FParse::Param(Cmd, TEXT("usedcomponents"));
	const bool bUnknownRefOnly = FParse::Param(Cmd, TEXT("unknownrefonly"));
	const bool bGameWorld = FParse::Param(Cmd, TEXT("game"));
	const bool bUsage = FParse::Param(Cmd, TEXT("-?"));
	
	// Non-editor builds literally don't have the data to determine mobile lods or vert data. The data prints out incorrectly and is confusing. Just remove it.
	const bool bHasMobileColumns = (bool)WITH_EDITORONLY_DATA;
	Ar.Logf(TEXT("Listing all static meshes."));
	if (bUsage)
	{
		Ar.Logf(TEXT(
			"\n Optional params: \n"
			"-alphasort: sort alphabetically \n"
			"-mobilesort: sort by mobile verts \n"
			"-instancesort: sort by instance count \n"
			"-game: only consider game world \n"
			"-usedcomponents: print the all components used by each mesh \n"
			"-unknownrefonly: list only static meshes using unknown ref heuristic for LOD streaming \n"
			"-csv: output in csv format"));
	}

	FRenderAssetStreamingManager* Streamer = nullptr;
	if (!IStreamingManager::HasShutdown() && !!CVarSetTextureStreaming.GetValueOnAnyThread())
	{
		Streamer = (FRenderAssetStreamingManager*)&IStreamingManager::Get().GetRenderAssetStreamingManager();
		Streamer->UpdateResourceStreaming(0.f, true);
	}

	//Collect usage counts
	TMap<UStaticMesh*, TArray<UStaticMeshComponent*>> UsageList;
	TMap<UStaticMesh*, int32> InstanceCounts;
	for (TObjectIterator<UStaticMeshComponent> It; It; ++It)
	{
		UStaticMeshComponent* MeshComponent = *It;
		if (bGameWorld && (!MeshComponent->GetWorld() || !MeshComponent->GetWorld()->IsGameWorld()))
		{
			continue;
		}
		
		UStaticMesh* Mesh = MeshComponent->GetStaticMesh();

		TArray<UStaticMeshComponent*>& MeshUsageArray = UsageList.FindOrAdd(Mesh);
		MeshUsageArray.Add(MeshComponent);

		int32& InstanceCount = InstanceCounts.FindOrAdd(Mesh, 0);
		if (UInstancedStaticMeshComponent* ISM = Cast<UInstancedStaticMeshComponent>(MeshComponent))
		{
			InstanceCount += ISM->GetInstanceCount();
		}
		else
		{
			InstanceCount++;
		}
	}

	// Collect meshes.
	TArray<FSortedStaticMesh> SortedMeshes;
	for(TObjectIterator<UStaticMesh> It; It; ++It)
	{
		UStaticMesh* Mesh = *It;

		bool bUnknownRef = false;
		bool bIsStreaming = (Mesh != nullptr ? Mesh->GetStreamingIndex() != INDEX_NONE : false);
		
		if (Streamer)
		{
			FStreamingRenderAsset* Asset = Streamer->GetStreamingRenderAsset(Mesh);
			if (Asset)
			{
				bUnknownRef = Asset->GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic);
			}
		}

		if (bUnknownRefOnly && !bUnknownRef)
		{
			continue;
		}

		const TArray<UStaticMeshComponent*>* MeshUsageList = UsageList.Find(Mesh);
		int32 ComponentUsageCount = MeshUsageList ? MeshUsageList->Num() : 0;

		int32* InstanceCountPtr = InstanceCounts.Find(Mesh);
		int32 InstanceUsageCount = InstanceCountPtr ? *InstanceCountPtr : 0;

		if (bGameWorld && !InstanceUsageCount)
		{
			continue;
		}

		FArchiveCountMem Count(Mesh);
		FResourceSizeEx ResourceSizeExc = FResourceSizeEx(EResourceSizeMode::Exclusive);
		Mesh->GetResourceSizeEx(ResourceSizeExc);
		FResourceSizeEx ResourceSizeInc = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
		Mesh->GetResourceSizeEx(ResourceSizeInc);

		FResourceSizeEx DistanceFieldSizeExc = FResourceSizeEx(EResourceSizeMode::Exclusive);
		if (Mesh->GetRenderData() && Mesh->GetRenderData()->LODResources[0].DistanceFieldData)
		{
			Mesh->GetRenderData()->LODResources[0].DistanceFieldData->GetResourceSizeEx(DistanceFieldSizeExc);
		}

		int32 NumKB = (Count.GetNum() + 512) / 1024;
		int32 MaxKB = (Count.GetMax() + 512) / 1024;
		int32 ResKBExc = (ResourceSizeExc.GetTotalMemoryBytes() + 512) / 1024;
		int32 ResKBInc = (ResourceSizeInc.GetTotalMemoryBytes() + 512) / 1024;
		int32 ResKBIncMobile = 0; //Update mobilesort once implemented
		int32 DistanceFieldKB = (DistanceFieldSizeExc.GetTotalMemoryBytes() + 512) / 1024;
		int32 LodCount = Mesh->GetNumLODs();
		int32 VertexCountLod0 = LodCount > 0 ? Mesh->GetNumVertices(0) : 0;
		int32 VertexCountLod1 = LodCount > 1 ? Mesh->GetNumVertices(1) : 0;
		int32 VertexCountLod2 = LodCount > 2 ? Mesh->GetNumVertices(2) : 0;
		int32 MobileMinLOD = -1;

#if WITH_EDITORONLY_DATA 
		if (Mesh->IsMinLodQualityLevelEnable() && Mesh->GetQualityLevelMinLOD().PerQuality.Find(0/*Low*/) != nullptr)
		{
			MobileMinLOD = *Mesh->GetQualityLevelMinLOD().PerQuality.Find(0);
		}
		if (Mesh->GetMinLOD().PerPlatform.Find(("Mobile")) != nullptr)
		{
			MobileMinLOD = *Mesh->GetMinLOD().PerPlatform.Find(("Mobile"));
		}
#endif

		int32 CollisionShapeCount = 0;
		if (Mesh->GetBodySetup())
		{
			CollisionShapeCount = Mesh->GetBodySetup()->AggGeom.GetElementCount();
		}

		int32 VertexCountTotal = 0;
		int32 VertexCountTotalMobile = 0;
		int32 ResidentLodCount = 0;
		int32 ResidentResKBExc = 0;
		int32 NumMissingLODs = 0;
		int32 CPUAccessOverheadKB = 0;
		FResourceSizeEx EvictedResourceSize(EResourceSizeMode::Exclusive);
		if (Mesh->GetRenderData())
		{
			NumMissingLODs = Mesh->GetRenderData()->CurrentFirstLODIdx;
			ResidentLodCount = LodCount - NumMissingLODs;
			CPUAccessOverheadKB = (Mesh->GetRenderData()->GetCPUAccessMemoryOverhead() + 512) / 1024;
		}
		for(int32 i = 0; i < LodCount; i++)
		{
			VertexCountTotal += Mesh->GetNumVertices(i);
			VertexCountTotalMobile += i >= MobileMinLOD ? Mesh->GetNumVertices(i) : 0;
			if (i < NumMissingLODs)
			{
				Mesh->GetRenderData()->LODResources[i].GetResourceSizeEx(EvictedResourceSize);
			}
		}
		ResidentResKBExc = (ResourceSizeExc.GetTotalMemoryBytes() - EvictedResourceSize.GetTotalMemoryBytes() + 512) / 1024;

		int32 VertexCountCollision = 0;
		if(Mesh->GetBodySetup())
		{
			for (auto& TriMesh : Mesh->GetBodySetup()->TriMeshGeometries)
			{
				VertexCountCollision += TriMesh->Particles().Size();
			}
		}

		FString Name = Mesh->GetFullName();

		new(SortedMeshes) FSortedStaticMesh(
			Mesh,
			NumKB,
			MaxKB,
			ResKBExc,
			ResKBInc,
			ResKBIncMobile,
			ResidentResKBExc,
			DistanceFieldKB,
			LodCount,
			ResidentLodCount,
			MobileMinLOD,
			VertexCountLod0,
			VertexCountLod1,
			VertexCountLod2,
			VertexCountTotal,
			VertexCountTotalMobile,
			VertexCountCollision,
			CollisionShapeCount,
			ComponentUsageCount,
			InstanceUsageCount,
			CPUAccessOverheadKB,
			bUnknownRef,
			bIsStreaming,
			Name);
	}

	// Sort meshes by cost or alphabetically.
	SortedMeshes.Sort(FCompareFSortedStaticMesh(SortType));

	// Display.
	int64 TotalNumKB = 0;
	int64 TotalMaxKB = 0;
	int64 TotalResKBExc = 0;
	int64 TotalResKBInc = 0;
	int64 TotalResKBResident = 0;
	int64 TotalResKBIncMobile = 0;
	int64 TotalDistanceFieldKB = 0;
	int32 TotalVertexCount = 0;
	int32 TotalVertexCountMobile = 0;
	int64 TotalCPUAccessOverheadKB = 0;
	int64 TotalComponents = 0;
	int64 TotalInstances = 0;

	FString HeaderString = bCSV ? FString(",NumKB,MaxKB,ResKBExc,ResKBInc,ResKBResi,DistFieldKB,LODCount,ResiLODCount,VertsLOD0,VertsLOD1,VertsLOD2,Verts Total,Verts Coll,Coll Shapes,NumComps,NumInst,Nanite,CPUAccessOverheadKB,UnknownRef,Streaming")
		: FString(",       NumKB,       MaxKB,    ResKBExc,    ResKBInc,   ResKBResi, DistFieldKB,    LODCount,ResiLODCount,   VertsLOD0,   VertsLOD1,   VertsLOD2, Verts Total,  Verts Coll, Coll Shapes,    NumComps,     NumInst,      Nanite, CPUAccessOverheadKB,  UnknownRef,  Streaming");
	FString MobileHeaderString = bCSV ? FString(",ResKBIncMob,Verts Mobile,MobileMinLOD")
		: FString(", ResKBIncMob,Verts Mobile,MobileMinLOD");
	if (bHasMobileColumns)
	{
		HeaderString = HeaderString.Append(MobileHeaderString);
	}

	if (bCSV)
	{
		Ar.Logf(TEXT("%s,Name"), *HeaderString);
	}
	else
	{
		Ar.Logf(TEXT("%s, Name"), *HeaderString);
	}

	for(int32 MeshIndex = 0; MeshIndex<SortedMeshes.Num(); MeshIndex++)
	{
		const FSortedStaticMesh& SortedMesh = SortedMeshes[MeshIndex];

		if (bHasMobileColumns)
		{
			Ar.Logf(TEXT(", %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %s, %19i, %s, %s, %11i, %11i, %11i, %s"),
				SortedMesh.NumKB,
				SortedMesh.MaxKB,
				SortedMesh.ResKBExc,
				SortedMesh.ResKBInc,
				SortedMesh.ResKBResident,
				SortedMesh.DistanceFieldKB,
				SortedMesh.LodCount,
				SortedMesh.ResidentLodCount,
				SortedMesh.VertexCountLod0,
				SortedMesh.VertexCountLod1,
				SortedMesh.VertexCountLod2,
				SortedMesh.VertexCountTotal,
				SortedMesh.VertexCountCollision,
				SortedMesh.ShapeCountCollision,
				SortedMesh.ComponentUsageCount,
				SortedMesh.InstanceUsageCount,
				SortedMesh.Mesh->HasValidNaniteData() ? TEXT("        Yes") : TEXT("         No"),
				SortedMesh.CPUAccessOverheadKB,
				SortedMesh.bUnknownRef ? TEXT("        Yes") : TEXT("         No"),
				SortedMesh.bIsStreaming ? TEXT("        Yes") : TEXT("         No"),
				SortedMesh.ResKBIncMobile,
				SortedMesh.VertexCountTotalMobile,
				SortedMesh.MobileMinLOD,
				*SortedMesh.Name);
		}
		else
		{
			Ar.Logf(TEXT(", %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %11i, %s, %19i, %s, %s, %s"),
				SortedMesh.NumKB,
				SortedMesh.MaxKB,
				SortedMesh.ResKBExc,
				SortedMesh.ResKBInc,
				SortedMesh.ResKBResident,
				SortedMesh.DistanceFieldKB,
				SortedMesh.LodCount,
				SortedMesh.ResidentLodCount,
				SortedMesh.VertexCountLod0,
				SortedMesh.VertexCountLod1,
				SortedMesh.VertexCountLod2,
				SortedMesh.VertexCountTotal,
				SortedMesh.VertexCountCollision,
				SortedMesh.ShapeCountCollision,
				SortedMesh.ComponentUsageCount,
				SortedMesh.InstanceUsageCount,
				SortedMesh.Mesh->HasValidNaniteData() ? TEXT("        Yes") : TEXT("         No"),
				SortedMesh.CPUAccessOverheadKB,
				SortedMesh.bUnknownRef ? TEXT("        Yes") : TEXT("         No"),
				SortedMesh.bIsStreaming ? TEXT("        Yes") : TEXT("         No"),
				*SortedMesh.Name);
		}
		
		TotalNumKB += SortedMesh.NumKB;
		TotalMaxKB += SortedMesh.MaxKB;
		TotalResKBExc += SortedMesh.ResKBExc;
		TotalResKBInc += SortedMesh.ResKBInc;
		TotalResKBResident += SortedMesh.ResKBResident;
		TotalResKBIncMobile += SortedMesh.ResKBIncMobile;
		TotalDistanceFieldKB += SortedMesh.DistanceFieldKB;
		TotalVertexCount += SortedMesh.VertexCountTotal;
		TotalVertexCountMobile += SortedMesh.VertexCountTotalMobile;
		TotalCPUAccessOverheadKB += SortedMesh.CPUAccessOverheadKB;
		TotalComponents += SortedMesh.ComponentUsageCount;
		TotalInstances += SortedMesh.InstanceUsageCount;
	}

	if (bCSV)
	{
		Ar.Logf(TEXT("Total NumKB: %lld,Total MaxKB: %lld,Total ResKB Exc: %lld,Total ResKB Inc: %lld,Total ResKB Inc Mobile: %lld,Total ResKB Resident: %lld,Total Distance Field: %lld,Total CPUAccess Overhead: %lld,Total Vertex Count: %i,Total Vertex Count Mobile: %i,Component Count: %i,Instance Count: %i,Static Mesh Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, TotalResKBIncMobile, TotalResKBResident, TotalDistanceFieldKB, TotalCPUAccessOverheadKB, TotalVertexCount, TotalVertexCountMobile, TotalComponents, TotalInstances, SortedMeshes.Num());
	}
	else
	{
		Ar.Logf(TEXT("Total NumKB: %lld KB, Total MaxKB: %lld KB, Total ResKB Exc: %lld KB, Total ResKB Inc: %lld KB, Total ResKB Inc Mobile: %lld KB, Total ResKB Resident: %lld KB, Total Distance Field: %lld KB, Total CPUAccess Overhead: %lld KB, Total Vertex Count: %i, Total Vertex Count Mobile: %i, Component Count: %i, Instance Count: %i, Static Mesh Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, TotalResKBIncMobile, TotalResKBResident, TotalDistanceFieldKB, TotalCPUAccessOverheadKB, TotalVertexCount, TotalVertexCountMobile, TotalComponents, TotalInstances, SortedMeshes.Num());
	}

	if (bUsedComponents)
	{
		for (int32 MeshIndex = 0; MeshIndex < SortedMeshes.Num(); MeshIndex++)
		{
			const FSortedStaticMesh& SortedMesh = SortedMeshes[MeshIndex];
			const TArray<UStaticMeshComponent*>* MeshUsageList = UsageList.Find(SortedMesh.Mesh);
			if (MeshUsageList)
			{
				Ar.Logf(TEXT("%s mesh has %i UStaticMeshComponents referencing"), *SortedMesh.Name, MeshUsageList->Num());
				for (int32 MeshComponentIndex = 0; MeshComponentIndex < MeshUsageList->Num(); ++MeshComponentIndex)
				{
					const UStaticMeshComponent* MeshComponent = (*MeshUsageList)[MeshComponentIndex];
					Ar.Logf(TEXT("\t\t%s"), *MeshComponent->GetPathName());
				}
			}
			else
			{
				Ar.Logf(TEXT("%s mesh has no UStaticMeshComponents referencing"), *SortedMesh.Name);
			}
		}
	}

	return true;
}

bool UEngine::HandleListSkeletalMeshesCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	const bool bAlphaSort = FParse::Param(Cmd, TEXT("ALPHASORT"));
	const bool bMobileSort = FParse::Param(Cmd, TEXT("MOBILESORT"));
	const bool bUnknownRefOnly = FParse::Param(Cmd, TEXT("unknownrefonly"));
	const bool bCSV = FParse::Param(Cmd, TEXT("CSV"));

	Ar.Logf(TEXT("Listing all skeletal meshes."));

	FRenderAssetStreamingManager* Streamer = nullptr;
	if (!IStreamingManager::HasShutdown() && !!CVarSetTextureStreaming.GetValueOnAnyThread())
	{
		Streamer = (FRenderAssetStreamingManager*)&IStreamingManager::Get().GetRenderAssetStreamingManager();
		Streamer->UpdateResourceStreaming(0.f, true);
	}

	// Collect meshes.
	TArray<FSortedSkeletalMesh> SortedMeshes;
	for (TObjectIterator<USkeletalMesh> It; It; ++It)
	{
		USkeletalMesh* Mesh = *It;

		bool bUnknownRef = false;
		bool bIsStreaming = Mesh->GetStreamingIndex() != INDEX_NONE;
		
		if (Streamer)
		{
			FStreamingRenderAsset* Asset = Streamer->GetStreamingRenderAsset(Mesh);
			if (Asset)
			{
				bUnknownRef = Asset->GetFlag(FStreamingRenderAssetFlags::UseUnkownRefHeuristic);
			}
		}

		if (bUnknownRefOnly && !bUnknownRef)
		{
			continue;
		}

		FArchiveCountMem Count(Mesh);
		FResourceSizeEx ResourceSizeExc = FResourceSizeEx(EResourceSizeMode::Exclusive);
		Mesh->GetResourceSizeEx(ResourceSizeExc);
		FResourceSizeEx ResourceSizeInc = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
		Mesh->GetResourceSizeEx(ResourceSizeInc);

		int32 NumKB = (Count.GetNum() + 512) / 1024;
		int32 MaxKB = (Count.GetMax() + 512) / 1024;
		int32 ResKBExc = (ResourceSizeExc.GetTotalMemoryBytes() + 512) / 1024;
		int32 ResKBInc = (ResourceSizeInc.GetTotalMemoryBytes() + 512) / 1024;
		int32 ResKBIncMobile = 0; // Update mobilesort once implemented

		int32 LodCount = 0;
		int32 MobileMinLOD = -1;
		int32 MeshDisablesMinLODStripping = 0;
		int32 bSupportLODStreaming = -1;
		int32 MaxNumStreamedLODs = -1;
		int32 MaxNumOptionalLODs = -1;
#if WITH_EDITORONLY_DATA
		if (Mesh->IsMinLodQualityLevelEnable() && Mesh->GetQualityLevelMinLod().PerQuality.Find(0) != nullptr)
		{
			MobileMinLOD = *Mesh->GetQualityLevelMinLod().PerQuality.Find(0/*Low*/);
		}
		else if(Mesh->GetMinLod().PerPlatform.Find(("Mobile")) != nullptr)
		{
			MobileMinLOD = *Mesh->GetMinLod().PerPlatform.Find(("Mobile"));
		}

		MeshDisablesMinLODStripping = Mesh->GetDisableBelowMinLodStripping().Default ? 1 : 0;
		if (Mesh->GetDisableBelowMinLodStripping().PerPlatform.Find(("Mobile")) != nullptr)
		{
			MeshDisablesMinLODStripping = *Mesh->GetDisableBelowMinLodStripping().PerPlatform.Find(("Mobile")) ? 1 : 0;
		}

		bSupportLODStreaming = Mesh->GetSupportLODStreaming().Default;
		if (const bool* Found = Mesh->GetSupportLODStreaming().PerPlatform.Find("Mobile"))
		{
			bSupportLODStreaming = *Found;
		}

		MaxNumStreamedLODs = Mesh->GetMaxNumStreamedLODs().Default;
		if (const int32* Found = Mesh->GetMaxNumStreamedLODs().PerPlatform.Find("Mobile"))
		{
			MaxNumStreamedLODs = *Found;
		}

		MaxNumOptionalLODs = Mesh->GetMaxNumOptionalLODs().Default;
		if (const int32* Found = Mesh->GetMaxNumOptionalLODs().PerPlatform.Find("Mobile"))
		{
			MaxNumOptionalLODs = *Found;
		}
#endif

		int32 VertexCountLod0 = 0;
		int32 VertexCountLod1 = 0;
		int32 VertexCountLod2 = 0;
		int32 VertexCountTotal = 0;
		int32 VertexCountTotalMobile = 0;

		int32 ResidentLodCount = 0;
		int32 ResidentResKBExc = 0;
		int32 CPUAccessOverheadKB = 0;

		if(Mesh->GetResourceForRendering() && Mesh->GetResourceForRendering()->LODRenderData.Num() > 0)
		{
			TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderData = Mesh->GetResourceForRendering()->LODRenderData;
			const int32 NumMissingLODs = Mesh->GetResourceForRendering()->CurrentFirstLODIdx;
			FResourceSizeEx EvictedResSize(EResourceSizeMode::Exclusive);

			LodCount = LodRenderData.Num();
			ResidentLodCount = LodCount - NumMissingLODs;
			VertexCountLod0 = LodCount > 0 ? LodRenderData[0].GetNumVertices() : 0;
			VertexCountLod1 = LodCount > 1 ? LodRenderData[1].GetNumVertices() : 0;
			VertexCountLod2 = LodCount > 2 ? LodRenderData[2].GetNumVertices() : 0;

			for(int32 i = 0; i < LodCount; i++)
			{
				VertexCountTotal += LodRenderData[i].GetNumVertices();
				VertexCountTotalMobile += i >= MobileMinLOD || MeshDisablesMinLODStripping == 1 ? LodRenderData[i].GetNumVertices() : 0;

				if (i < NumMissingLODs)
				{
					LodRenderData[i].GetResourceSizeEx(EvictedResSize);
				}
			}

			ResidentResKBExc = (ResourceSizeExc.GetTotalMemoryBytes() - EvictedResSize.GetTotalMemoryBytes() + 512) / 1024;
			CPUAccessOverheadKB = (Mesh->GetResourceForRendering()->GetCPUAccessMemoryOverhead() + 512) / 1024;
		}

		int32 VertexCountCollision = 0;
		const USkeletalMesh* MeshConst = Mesh;
		if (MeshConst->GetBodySetup())
		{
			for (auto& TriMesh : MeshConst->GetBodySetup()->TriMeshGeometries)
			{
				VertexCountCollision += TriMesh->Particles().Size();
			}
		}

		FString Name = Mesh->GetFullName();

		new(SortedMeshes) FSortedSkeletalMesh(
			NumKB,
			MaxKB,
			ResKBExc,
			ResKBInc,
			ResKBIncMobile,
			LodCount,
			MobileMinLOD,
			MeshDisablesMinLODStripping,
			bSupportLODStreaming,
			MaxNumStreamedLODs,
			MaxNumOptionalLODs,
			VertexCountLod0,
			VertexCountLod1,
			VertexCountLod2,
			VertexCountTotal,
			VertexCountTotalMobile,
			VertexCountCollision,
			ResidentLodCount,
			ResidentResKBExc,
			CPUAccessOverheadKB,
			bUnknownRef,
			bIsStreaming,
			Name);
	}

	// Sort meshes by cost or alphabetically.
	SortedMeshes.Sort(FCompareFSortedSkeletalMesh(bAlphaSort, bMobileSort));

	// Display.
	int64 TotalNumKB = 0;
	int64 TotalMaxKB = 0;
	int64 TotalResKBExc = 0;
	int64 TotalResKBInc = 0;
	int64 TotalResKBResident = 0;
	int64 TotalResKBIncMobile = 0;
	int32 TotalVertexCount = 0;
	int32 TotalVertexCountMobile = 0;
	int64 TotalCPUAccessOverheadKB = 0;

	if (bCSV)
	{
		Ar.Logf(TEXT(",NumKB,MaxKB,ResKBExc,ResKBInc,ResKBIncMobile,ResKBResident,LOD Count,Resident LOD Count,MobileMinLOD,DisableMinLODStripping,bSupportLODStreaming,MaxNumStreamedLODs,MaxNumOptionalLODs,Verts LOD0,Verts LOD1,Verts LOD2,Verts Total,Verts Total Mobile,Verts Collision,CPUAccessOverheadKB,UnknownRef,Streaming,Name"));
	}
	else
	{
		Ar.Logf(TEXT("       NumKB       MaxKB    ResKBExc    ResKBInc  ResKBIncMobile  ResKBResident  NumLODs  NumResidentLODs  MobileMinLOD  DisableMinLODStripping  bSupportLODStreaming  MaxNumStreamedLODs  MaxNumOptionalLODs  VertsLOD0   VertsLOD1   VertsLOD2   VertsTotal  VertsTotalMobile  VertsColl  CPUAccessOverheadKB  UnknownRef  Streaming Name"));
	}

	for (int32 MeshIndex = 0; MeshIndex<SortedMeshes.Num(); MeshIndex++)
	{
		const FSortedSkeletalMesh& SortedMesh = SortedMeshes[MeshIndex];

		if (bCSV)
		{
			Ar.Logf(TEXT(",%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%i,%s,%s,%s"),
				SortedMesh.NumKB, SortedMesh.MaxKB, SortedMesh.ResKBExc, SortedMesh.ResKBInc, SortedMesh.ResKBIncMobile, SortedMesh.ResidentResKBExc, SortedMesh.LodCount, SortedMesh.ResidentLodCount, SortedMesh.MobileMinLOD,
				SortedMesh.MeshDisablesMinLODStripping, SortedMesh.bSupportLODStreaming, SortedMesh.MaxNumStreamedLODs, SortedMesh.MaxNumOptionalLODs,
				SortedMesh.VertexCountLod0, SortedMesh.VertexCountLod1, SortedMesh.VertexCountLod2, SortedMesh.VertexCountTotal, SortedMesh.VertexCountTotalMobile, SortedMesh.VertexCountCollision,
				SortedMesh.CPUAccessOverheadKB, SortedMesh.bUnknownRef ? TEXT("Yes") : TEXT("No"), SortedMesh.bIsStreaming ? TEXT("Yes") : TEXT("No"),
				*SortedMesh.Name);
		}
		else
		{
			Ar.Logf(TEXT("%9i KB %8i KB %8i KB %8i KB  %11i KB %10i KB %10i %10i %14i %23i %21i %19i %19i %10i %11i %11i %12i %17i %10i %8i KB  %s %s %s"),
				SortedMesh.NumKB, SortedMesh.MaxKB, SortedMesh.ResKBExc, SortedMesh.ResKBInc, SortedMesh.ResKBIncMobile, SortedMesh.ResidentResKBExc, SortedMesh.LodCount, SortedMesh.ResidentLodCount, SortedMesh.MobileMinLOD,
				SortedMesh.MeshDisablesMinLODStripping, SortedMesh.bSupportLODStreaming, SortedMesh.MaxNumStreamedLODs, SortedMesh.MaxNumOptionalLODs,
				SortedMesh.VertexCountLod0, SortedMesh.VertexCountLod1, SortedMesh.VertexCountLod2, SortedMesh.VertexCountTotal, SortedMesh.VertexCountTotalMobile, SortedMesh.VertexCountCollision,
				SortedMesh.CPUAccessOverheadKB, SortedMesh.bUnknownRef ? TEXT("Yes") : TEXT("No"), SortedMesh.bIsStreaming ? TEXT("Yes") : TEXT("No"),
				*SortedMesh.Name);
		}

		TotalNumKB += SortedMesh.NumKB;
		TotalMaxKB += SortedMesh.MaxKB;
		TotalResKBExc += SortedMesh.ResKBExc;
		TotalResKBInc += SortedMesh.ResKBInc;
		TotalResKBResident += SortedMesh.ResidentResKBExc;
		TotalResKBIncMobile += SortedMesh.ResKBIncMobile;
		TotalVertexCount += SortedMesh.VertexCountTotal;
		TotalVertexCountMobile += SortedMesh.VertexCountTotalMobile;
		TotalCPUAccessOverheadKB += SortedMesh.CPUAccessOverheadKB;
	}

	if (bCSV)
	{
		Ar.Logf(TEXT("Total NumKB: %lld,Total MaxKB: %lld,Total ResKB Exc: %lld,Total ResKB Inc %lld,Total ResKB Inc Mobile %lld,Total ResKB Resident %lld,Total CPUAccess Overhead %lld,Total Vertex Count: %i,Total Vertex Count Mobile: %i,Static Mesh Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, TotalResKBIncMobile, TotalResKBResident, TotalCPUAccessOverheadKB, TotalVertexCount, TotalVertexCountMobile, SortedMeshes.Num());
	}
	else
	{
		Ar.Logf(TEXT("Total NumKB: %lld KB, Total MaxKB: %lld KB, Total ResKB Exc: %lld KB, Total ResKB Inc %lld KB,  Total ResKB Inc Mobile %lld KB, Total ResKB Resident %lld KB, Total CPUAccess Overhead %lld KB, Total Vertex Count: %i, Total Vertex Count Mobile: %i, Static Mesh Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, TotalResKBIncMobile, TotalResKBResident, TotalCPUAccessOverheadKB, TotalVertexCount, TotalVertexCountMobile, SortedMeshes.Num());
	}

	return true;
}

bool UEngine::HandleListAnimsCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	const bool bAlphaSort = FParse::Param(Cmd, TEXT("ALPHASORT"));
	const bool bCSV = FParse::Param(Cmd, TEXT("CSV"));

	Ar.Logf(TEXT("Listing all animations."));

	TArray<FSortedAnimAsset> SortedAnimAssets;
	for (TObjectIterator<UAnimationAsset> It; It; ++It)
	{
		UAnimationAsset* AnimAsset = *It;

		FArchiveCountMem Count(AnimAsset);
		FResourceSizeEx ResourceSizeExc = FResourceSizeEx(EResourceSizeMode::Exclusive);
		AnimAsset->GetResourceSizeEx(ResourceSizeExc);
		FResourceSizeEx ResourceSizeInc = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
		AnimAsset->GetResourceSizeEx(ResourceSizeInc);

		int32 NumKB = (Count.GetNum() + 512) / 1024;
		int32 MaxKB = (Count.GetMax() + 512) / 1024;
		int32 ResKBExc = (ResourceSizeExc.GetTotalMemoryBytes() + 512) / 1024;
		int32 ResKBInc = (ResourceSizeInc.GetTotalMemoryBytes() + 512) / 1024;

		FString Name = AnimAsset->GetPathName();

		FName AnimAssetType = AnimAsset->GetClass()->GetFName();

		int32 NumKeys = 0;
		float SequenceLength = 0.f;
		float RateScale = 1.f;
		int32 NumCurves = 0;
		FString DebugString;

		if (const UAnimSequence* AnimSeq = Cast<UAnimSequence>(AnimAsset))
		{
			NumKeys = AnimSeq->GetNumberOfSampledKeys();
			SequenceLength = AnimSeq->GetPlayLength();
			RateScale = AnimSeq->RateScale;
			NumCurves = AnimSeq->GetCurveData().FloatCurves.Num();

			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			if (CompressedAnimSequence.Get().CompressedDataStructure != nullptr)
			{
				DebugString = CompressedAnimSequence.Get().CompressedDataStructure->GetDebugString();
			}
			else
			{
				DebugString = TEXT("Missing CompressedDataStructure");
			}
		}

		new(SortedAnimAssets) FSortedAnimAsset(
			NumKB,
			MaxKB,
			ResKBExc,
			ResKBInc,
			AnimAssetType,
			Name,
			NumKeys,
			SequenceLength,
			RateScale,
			NumCurves,
			DebugString
		);
	}

	// Sort meshes by cost or alphabetically.
	SortedAnimAssets.Sort(FCompareFSortedAnimAsset(bAlphaSort));

	// Display.
	int64 TotalNumKB = 0;
	int64 TotalMaxKB = 0;
	int64 TotalResKBExc = 0;
	int64 TotalResKBInc = 0;

	if (bCSV)
	{
		Ar.Logf(TEXT(",NumKB,MaxKB,ResKBExc,ResKBInc,Type,Name,Length,RateScale,NumCurves,TranslationFormat,RotationFormat,ScaleFormat"));
	}
	else
	{
		Ar.Logf(TEXT(", NumKB, MaxKB, ResKBExc, ResKBInc, Type, Name, Length, RateScale, NumCurves, TranslationFormat, RotationFormat, ScaleFormat"));
	}

	for (const FSortedAnimAsset& SortedAsset : SortedAnimAssets)
	{

		if (bCSV)
		{
			Ar.Logf(TEXT(",%i,%i,%i,%i,%s,%s,%f,%f,%i,%s"),
				SortedAsset.NumKB, SortedAsset.MaxKB, SortedAsset.ResKBExc, SortedAsset.ResKBInc, *SortedAsset.AnimAssetType.ToString(), *SortedAsset.Name, SortedAsset.SequenceLength, SortedAsset.RateScale, SortedAsset.NumCurves, *SortedAsset.DebugString);
		} 
		else
		{
			Ar.Logf(TEXT(", %i, %i, %i, %i, %s, %s, %f, %f, %i, %s"),
				SortedAsset.NumKB, SortedAsset.MaxKB, SortedAsset.ResKBExc, SortedAsset.ResKBInc, *SortedAsset.AnimAssetType.ToString(), *SortedAsset.Name, SortedAsset.SequenceLength, SortedAsset.RateScale, SortedAsset.NumCurves, *SortedAsset.DebugString);
		}

		TotalNumKB += SortedAsset.NumKB;
		TotalMaxKB += SortedAsset.MaxKB;
		TotalResKBExc += SortedAsset.ResKBExc;
		TotalResKBInc += SortedAsset.ResKBInc;
	}

	if (bCSV)
	{
		Ar.Logf(TEXT("Total NumKB: %lld,Total MaxKB: %lld,Total ResKB Exc: %lld,Total ResKB Inc: %lld,Anim Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, SortedAnimAssets.Num());
	} 
	else
	{
		Ar.Logf(TEXT("Total NumKB: %lld KB, Total MaxKB: %lld KB, Total ResKB Exc: %lld KB, Total ResKB Inc: %lld KB, Anim Count: %i"),
			TotalNumKB, TotalMaxKB, TotalResKBExc, TotalResKBInc, SortedAnimAssets.Num());
	}

	return true;
}

bool UEngine::HandleRemoteTextureStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Address which sent the command.  We will send stats back to this address
	FString Addr(FParse::Token(Cmd, 0));
	// Port to send to
	FString Port(FParse::Token(Cmd, 0));

	// Make an IP address. 	// @TODO ONLINE - Revisit "send over network"
	//FIpAddr SrcAddr;
	//SrcAddr.Port = FCString::Atoi( *Port );
	//SrcAddr.Addr = FCString::Atoi( *Addr );

	// Gather stats.
	double LastTime = FApp::GetLastTime();

	UE_LOGF(LogEngine, Log,  "Remote AssetsStats request received." );

	TMap<UTexture2D*,int32> TextureToUsageMap;

	TArray<UMaterialInterface*> UsedMaterials;
	TArray<UTexture*> UsedTextures;

	// Find out how many times a texture is referenced by primitive components.
	for( TObjectIterator<UPrimitiveComponent> It; It; ++It )
	{
		UPrimitiveComponent* PrimitiveComponent = *It;

		UsedMaterials.Reset();
		// Get the used materials off the primitive component so we can find the textures
		PrimitiveComponent->GetUsedMaterials( UsedMaterials );
		for( int32 MatIndex = 0; MatIndex < UsedMaterials.Num(); ++MatIndex )
		{
			// Ensure we dont have any NULL elements.
			if( UsedMaterials[ MatIndex ] )
			{
				UsedTextures.Reset();
				UsedMaterials[MatIndex]->GetUsedTextures(UsedTextures, GetCurrentMaterialQualityLevelChecked());

				// Increase usage count for all referenced textures
				for( int32 TextureIndex=0; TextureIndex<UsedTextures.Num(); TextureIndex++ )
				{
					UTexture2D* Texture = Cast<UTexture2D>( UsedTextures[TextureIndex] );
					if( Texture )
					{
						// Initializes UsageCount to 0 if texture is not found.
						int32 UsageCount = TextureToUsageMap.FindRef( Texture );
						TextureToUsageMap.Add(Texture, UsageCount+1);
					}
				}
			}
		}
	}

	for(TObjectIterator<UTexture> It; It; ++It)
	{
		UTexture* Texture = *It;
		FString FullyQualifiedPath = Texture->GetPathName();
		FString MaxDim = FString::Printf( TEXT( "%dx%d" ), (int32)Texture->GetSurfaceWidth(), (int32)Texture->GetSurfaceHeight() );

		uint32 GroupId = Texture->LODGroup;
		uint32 FullyLoadedInBytes = Texture->CalcTextureMemorySizeEnum( TMC_AllMips );
		uint32 CurrentInBytes = Texture->CalcTextureMemorySizeEnum( TMC_ResidentMips );
		FString TexType;	// e.g. "2D", "Cube", ""
		uint32 FormatId = 0;
		float LastTimeRendered = FLT_MAX;
		uint32 NumUses = 0;
		int32 LODBias = Texture->GetCachedLODBias();
		FTexture* Resource = Texture->GetResource();

		if(Resource)
		{
			LastTimeRendered = LastTime - Resource->LastRenderTime;
		}

		FString CurrentDim = "?";
		UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
		if(Texture2D)
		{
			FormatId = Texture2D->GetPixelFormat();
			TexType = TEXT("2D");
			NumUses = TextureToUsageMap.FindRef( Texture2D );

			// Calculate in game current dimensions 
			const int32 DroppedMips = Texture2D->GetNumMips() - Texture2D->GetNumResidentMips();
			CurrentDim = FString::Printf(TEXT("%dx%d"), Texture2D->GetSizeX() >> DroppedMips, Texture2D->GetSizeY() >> DroppedMips);
		}
		else
		{
			UTextureCube* TextureCube = Cast<UTextureCube>(Texture);
			if(TextureCube)
			{
				FormatId = TextureCube->GetPixelFormat();
				TexType = TEXT("Cube");
				// Calculate in game current dimensions
				CurrentDim = FString::Printf(TEXT("%dx%d"), TextureCube->GetSizeX(), TextureCube->GetSizeY());
			}
		}

		float CurrentKB = CurrentInBytes / 1024.0f;
		float FullyLoadedKB = FullyLoadedInBytes / 1024.0f;

		// @TODO ONLINE - Revisit "send over network"
		// 10KB should be enough
		//FNboSerializeToBuffer PayloadWriter(10 * 1024);
		//PayloadWriter << FullyQualifiedPath << MaxDim << CurrentDim << TexType << FormatId << GroupId << FullyLoadedKB << CurrentKB << LastTimeRendered << NumUses << LODBias;	
	}
	return true;
}

bool UEngine::HandleListParticleSystemsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TArray<FString> Switches;
	TArray<FString> Tokens;
	FCommandLine::Parse(Cmd, Tokens, Switches);

	const bool bAlphaSort = (Tokens.Find(TEXT("ALPHASORT")) != INDEX_NONE) || (Switches.Find(TEXT("ALPHASORT")) != INDEX_NONE);
	bool bDumpMesh = (Tokens.Find(TEXT("DUMPMESH")) != INDEX_NONE) || (Switches.Find(TEXT("DUMPMESH")) != INDEX_NONE);

	TArray<FSortedParticleSet> SortedSets;
	TMap<UObject *,int32> SortMap;

	FString Description;
	for( TObjectIterator<UFXSystemAsset> SystemIt; SystemIt; ++SystemIt )
	{			
		UFXSystemAsset* Tree = *SystemIt;
		Description = FString::Printf(TEXT("%s"), *Tree->GetPathName());

		int32 RootSize = 0;
		if (Tree->IsA<UParticleSystem>())
		{
			RootSize = FArchiveCountMem(Tree).GetMax();
		}
		else
		{
			FResourceSizeEx ResSize = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
			Tree->GetResourceSizeEx(ResSize);
			RootSize = ResSize.GetTotalMemoryBytes();
		}

		SortedSets.Add(FSortedParticleSet(Description, RootSize, RootSize, 0, 0, 0, FResourceSizeEx(EResourceSizeMode::EstimatedTotal), FResourceSizeEx(EResourceSizeMode::Exclusive)));
		SortMap.Add(Tree,SortedSets.Num() - 1);
	}

	for( TObjectIterator<UParticleModule> It; It; ++It )
	{
		UParticleModule* Module = *It;
		int32 *pIndex = SortMap.Find(Module->GetOuter());

		if (pIndex && SortedSets.IsValidIndex(*pIndex))
		{
			FSortedParticleSet &Set = SortedSets[*pIndex];
			FArchiveCountMem ModuleCount( Module );
			Set.ModuleSize += ModuleCount.GetMax();
			Set.Size += ModuleCount.GetMax();
		}
	}

	for( TObjectIterator<UFXSystemComponent> It; It; ++It )
	{
		UFXSystemComponent* Comp = *It;
		int32 *pIndex = SortMap.Find(Comp->GetFXSystemAsset());

		if (pIndex && SortedSets.IsValidIndex(*pIndex))
		{				
			FSortedParticleSet &Set = SortedSets[*pIndex];				
			FArchiveCountMem ComponentCount( Comp );
			Set.ComponentSize += ComponentCount.GetMax();

			// Save this for adding to the total
			FResourceSizeEx CompResSize = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
			Comp->GetResourceSizeEx(CompResSize);

			Set.ComponentResourceSize += CompResSize;
			Comp->GetResourceSizeEx(Set.ComponentTrueResourceSize);

			Set.Size += ComponentCount.GetMax();
			Set.Size += CompResSize.GetTotalMemoryBytes();
			Set.ComponentCount++;

			if (UParticleSystemComponent* PSC = Cast<UParticleSystemComponent>(Comp))
			{
				UParticleSystem* Tree = PSC->Template;
				if (bDumpMesh && Tree != NULL)
				{
					for (int32 EmitterIdx = 0; (EmitterIdx < Tree->Emitters.Num()); EmitterIdx++)
					{
						if (UParticleEmitter* Emitter = Tree->Emitters[EmitterIdx])
						{
							// Have to check each LOD level...
							if (Emitter->LODLevels.Num() > 0)
							{
								if (UParticleLODLevel* LODLevel = Emitter->LODLevels[0])
								{
									if (LODLevel->RequiredModule->bUseLocalSpace == true)
									{
										UParticleModuleTypeDataMesh* MeshTD = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule);
										if (MeshTD != NULL)
										{
											int32 InstCount = 0;
											// MESH EMITTER
											if (EmitterIdx < PSC->EmitterInstances.Num())
											{
												FParticleEmitterInstance* Inst = PSC->EmitterInstances[EmitterIdx];
												if (Inst != NULL)
												{
													InstCount = Inst->ActiveParticles;
												}

												UE_LOGF(LogEngine, Warning, "---> PSys w/ mesh emitters: %2d %4d %ls %ls ", EmitterIdx, InstCount, 
													PSC->SceneProxy ? TEXT("Y") : TEXT("N"),
													*(Tree->GetPathName()));
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Sort anim sets by cost
	SortedSets.Sort( FCompareFSortedParticleSet( bAlphaSort ) );

	// Now print them out.
	Ar.Logf(TEXT("ParticleSystems:"));
	Ar.Logf(TEXT("Size,Name,PSysSize,ModuleSize,ComponentSize,ComponentCount,CompResSize,CompTrueResSize"));
	FSortedParticleSet TotalSet(TEXT("Total"));
	int32 TotalSize = 0;
	for(int32 i=0; i<SortedSets.Num(); i++)
	{
		FSortedParticleSet& SetInfo = SortedSets[i];
		TotalSize += SetInfo.Size;
		TotalSet += SetInfo;
		SetInfo.Dump(Ar);
	}
	TotalSet.Dump(Ar);
	return true;
}

bool UEngine::HandleListSpawnedActorsCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	if( InWorld )
	{
		const float	TimeSeconds		    = InWorld->GetTimeSeconds();

		// Create alphanumerically sorted list of actors in persistent level.
		TArray<AActor*> SortedActorList = ObjectPtrDecay(InWorld->PersistentLevel->Actors);
		SortedActorList.Remove(nullptr);
		SortedActorList.Sort();

		Ar.Logf(TEXT("Listing spawned actors in persistent level:"));
		Ar.Logf(TEXT("Total: %d" ), SortedActorList.Num());

		if ( GetNumGamePlayers(InWorld) )
		{
			// If have local player, give info on distance to player
			const FVector PlayerLocation = GetGamePlayers(InWorld)[0]->LastViewLocation;

			// Iterate over all non-static actors and log detailed information.
			Ar.Logf(TEXT("TimeUnseen,TimeAlive,Distance,Class,Name,Owner"));
			for( int32 ActorIndex=0; ActorIndex<SortedActorList.Num(); ActorIndex++ )
			{
				AActor* Actor = SortedActorList[ActorIndex];
				if( !Actor->IsNetStartupActor() )
				{
					// Calculate time actor has been alive for. Certain actors can be spawned before TimeSeconds is valid
					// so we manually reset them to the same time as TimeSeconds.
					float TimeAlive	= TimeSeconds - Actor->CreationTime;
					if( TimeAlive < 0 )
					{
						TimeAlive = TimeSeconds;
					}
					const float TimeUnseen = TimeSeconds - Actor->GetLastRenderTime();
					const float DistanceToPlayer = FVector::Dist( Actor->GetActorLocation(), PlayerLocation );
					Ar.Logf(TEXT("%6.2f,%6.2f,%8.0f,%s,%s,%s"),TimeUnseen,TimeAlive,DistanceToPlayer,*Actor->GetClass()->GetName(),*Actor->GetName(),*GetNameSafe(Actor->GetOwner()));
				}
			}
		}
		else
		{
			// Iterate over all non-static actors and log detailed information.
			Ar.Logf(TEXT("TimeAlive,Class,Name,Owner"));
			for( int32 ActorIndex=0; ActorIndex<SortedActorList.Num(); ActorIndex++ )
			{
				AActor* Actor = SortedActorList[ActorIndex];
				if( !Actor->IsNetStartupActor() )
				{
					// Calculate time actor has been alive for. Certain actors can be spawned before TimeSeconds is valid
					// so we manually reset them to the same time as TimeSeconds.
					float TimeAlive	= TimeSeconds - Actor->CreationTime;
					if( TimeAlive < 0 )
					{
						TimeAlive = TimeSeconds;
					}
					Ar.Logf(TEXT("%6.2f,%s,%s,%s"),TimeAlive,*Actor->GetClass()->GetName(),*Actor->GetName(),*GetNameSafe(Actor->GetOwner()));
				}
			}
		}
	}
	else
	{
		Ar.Logf(TEXT("LISTSPAWNEDACTORS failed."));
	}
	return true;
}

bool UEngine::HandleMemReportCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	// This will defer the report to the end of the frame so we can force a GC and get a real report with no gcable objects
	GEngine->DeferredCommands.Add(FString::Printf(TEXT("MemReportDeferred %s"), Cmd ));

	return true;
}

void UEngine::WriteMemReportMetadata( FOutputDevice& Ar, UWorld* InWorld )
{
	FString ConfigString = LexToString(FApp::GetBuildConfiguration());
	uint32 ChangelistNumber = FEngineVersion::Current().GetChangelist();

	// Set the device name to the platform name as a fallback
	FString DeviceName = FPlatformProperties::PlatformName();

	// Attempt to get the specific device name from the runtime device profile selector
	FString DeviceProfile = "None";
	FString DeviceProfileSelectionModule;
	if (GConfig->GetString(TEXT("DeviceProfileManager"), TEXT("DeviceProfileSelectionModule"), DeviceProfileSelectionModule, GEngineIni))
	{
		if (IDeviceProfileSelectorModule* DPSelectorModule = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>(*DeviceProfileSelectionModule))
		{
			DeviceName = DPSelectorModule->GetRuntimeDeviceProfileName();
		}
	}

	Ar.Logf(TEXT("Changelist: %d"), ChangelistNumber);
	Ar.Logf(TEXT("Config: %s"), *ConfigString);
	Ar.Logf(TEXT("Device Name: %s"), *DeviceName);
	Ar.Logf(TEXT("Device Profile: %s"), *DeviceProfile);
	Ar.Logf(TEXT("CommandLine Options: %s"), FCommandLine::Get());
	Ar.Logf(TEXT("Time Since Boot: %.02f Seconds"), FPlatformTime::Seconds() - GStartTime);

	// List Name, Location, and Rotation of each local PlayerController (useful context for client mem)
	TArray<APlayerController*> LocalPlayerControllers;
	GetAllLocalPlayerControllers(LocalPlayerControllers);
	for (APlayerController* LocalPlayerController : LocalPlayerControllers)
	{
		FVector OutLocation;
		FRotator OutRotation;
		LocalPlayerController->GetPlayerViewPoint(OutLocation, OutRotation);
		Ar.Logf(TEXT("Local PlayerController: %s View Location: %s View Rotation: %s"),
			*LocalPlayerController->GetName(),
			*OutLocation.ToString(),
			*OutRotation.ToString());
	}
}

bool UEngine::HandleMemReportDeferredCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
#if ALLOW_DEBUG_FILES
	QUICK_SCOPE_CYCLE_COUNTER(HandleMemReportDeferredCommand);

	FSlowHeartBeatScope HeartBeatSuspend;

	const bool bLogOutputToFile = !FParse::Param( Cmd, TEXT("LOG") );
	const bool bCSV = FParse::Param(Cmd, TEXT("CSV"));
	FString InTypeName;
	FParse::Value( Cmd, TEXT("TYPE="), InTypeName);
	FString InFileName;
	FParse::Value(Cmd, TEXT("NAME="), InFileName);

	if (FParse::Param(Cmd, TEXT("FULL")))
	{
		InTypeName = TEXT("FULL");
	}
	if (InTypeName.Equals(TEXT("DEFAULT"), ESearchCase::IgnoreCase))
	{
		InTypeName.Empty();
	}
	const FString SectionToRead = FString::Format(TEXT("MemReport{0}Commands"), { InTypeName });

	UE_LOGF(LogEngine, Log, "MemReportDeferred: Executing \"%ls\" profile.", *InTypeName);

	// Turn off as it makes diffing hard
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	// Flush rendering and do a GC
	FlushAsyncLoading();
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS,true);
	FlushRenderingCommands();

	FOutputDevice* ReportAr = &Ar;
	TUniquePtr<FArchive> FileAr;
	TOptional<FOutputDeviceArchiveWrapper> FileArWrapper;
	FString FilenameFull;

	if (bLogOutputToFile)
	{	
		const FString PathName = *(FPaths::ProfilingDir() + TEXT("MemReports/"));
		IFileManager::Get().MakeDirectory( *PathName );

		const FString Filename = CreateProfileFilename(InFileName, TEXT(".memreport"), true );
		FilenameFull = PathName + Filename;

		FileAr = TUniquePtr<FArchive>(IFileManager::Get().CreateDebugFileWriter(*FilenameFull));
		FileArWrapper.Emplace(FileAr.Get());
		ReportAr = FileArWrapper.GetPtrOrNull();

		UE_LOGF(LogEngine, Log, "MemReportDeferred: saving to %ls", *FilenameFull);
	}


	// log out some useful information in the header
	{
		WriteMemReportMetadata(*ReportAr, InWorld);
		ReportAr->Logf(TEXT(""));
	}


	// Run commands from the ini
	TArray<FString> CommandsToRun;
	if (GConfig->GetArray(*SectionToRead, TEXT("Cmd"), CommandsToRun, GEngineIni))
	{
		for (FString& Command : CommandsToRun)
		{
			if (bCSV)
			{
				Command += TEXT(" -csv");
			}

			ReportAr->Logf(TEXT("MemReport: Begin command \"%s\""), *Command);
			Exec(InWorld, *Command, *ReportAr);
			ReportAr->Logf(TEXT("MemReport: End command \"%s\"") LINE_TERMINATOR, *Command);
		}
	}

	if (bLogOutputToFile)
	{
		CommandsToRun.Reset();
		if (GConfig->GetArray(*SectionToRead, TEXT("File"), CommandsToRun, GEngineIni))
		{
			const FString DirectoryName = FPaths::GetBaseFilename(FilenameFull, false) + TEXT("/");
			IFileManager::Get().MakeDirectory( *DirectoryName );

			for (FString& Command : CommandsToRun)
			{
				const FString ValidFilePath = DirectoryName + FPaths::MakeValidFileName(Command) + (bCSV ? TEXT(".csv") : TEXT(".txt"));

				TUniquePtr<FArchive> CommandFile(IFileManager::Get().CreateDebugFileWriter(*ValidFilePath));
				FOutputDeviceArchiveWrapper CommandFileAr(CommandFile.Get());
				if (bCSV)
				{
					Command += TEXT(" -csv");
				}

				Exec(InWorld, *Command, CommandFileAr);
				UE_LOGF(LogEngine, Log, "MemReportDeferred: Written \"%ls\" output to \"%ls\"", *Command, *ValidFilePath);
			}
		}
	}
#endif

	return true;
}

bool UEngine::HandleSkeletalMeshReportCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	Ar.Logf(TEXT("+++ Skeletal Mesh Report +++"));

	auto BytesToKB = [&](uint32 Bytes) -> uint32
	{
		return (Bytes + 512) / 1024;
	};

	auto GetResourceSizeKB = [&](const FRHIResource* Resource) -> uint32
	{
		int32 SizeInKB = 0;
#if RHI_ENABLE_RESOURCE_INFO
		FRHIResourceInfo ResourceInfo;
		if (Resource && Resource->GetResourceInfo(ResourceInfo))
		{
			SizeInKB = BytesToKB(ResourceInfo.VRamAllocation.AllocationSize);
		}
#endif
		return SizeInKB;
	};

	bool bSkinCacheGlobalDefault = GetSkinCacheDefaultBehavior() == ESkinCacheDefaultBehavior::Inclusive;

	int32 NumMeshes = 0;
	FRHIResourceInfo ResourceInfo;
	for (TObjectIterator<USkeletalMesh> It; It; ++It)
	{
		USkeletalMesh* Mesh = *It;
		FString Name = Mesh->GetFullName();

		if (Mesh->GetResourceForRendering() && Mesh->GetResourceForRendering()->LODRenderData.Num() > 0)
		{
			Ar.Logf(TEXT("%s"), *Name);

			TIndirectArray<FSkeletalMeshLODRenderData>& LodRenderDatas = Mesh->GetResourceForRendering()->LODRenderData;
			for (int32 LODIdx = 0; LODIdx < LodRenderDatas.Num(); LODIdx++)
			{
				const FSkeletalMeshLODRenderData& RenderData = LodRenderDatas[LODIdx];
	
				FSkeletalMeshLODInfo* LodInfo = Mesh->GetLODInfo(LODIdx);
				check(LodInfo);
				bool bSkinCachedLODEnabled = FGPUSkinCache::IsGPUSkinCacheEnabled(GMaxRHIFeatureLevel) &&
					(LodInfo->SkinCacheUsage == ESkinCacheUsage::Auto ? bSkinCacheGlobalDefault : LodInfo->SkinCacheUsage == ESkinCacheUsage::Enabled);

				int32 MorphDataKB = GetResourceSizeKB(RenderData.MorphTargetVertexInfoBuffers.MorphDataBuffer);
				int32 ClothVertexBufferKB = GetResourceSizeKB(RenderData.ClothVertexBuffer.VertexBufferRHI);
				int32 SkinWeightDataVertexBufferKB = GetResourceSizeKB(RenderData.SkinWeightVertexBuffer.GetDataVertexBuffer()->VertexBufferRHI);
				int32 SkinWeightLookupVertexBufferKB = GetResourceSizeKB(RenderData.SkinWeightVertexBuffer.GetLookupVertexBuffer()->VertexBufferRHI);
				int32 PositionVertexBufferKB = GetResourceSizeKB(RenderData.StaticVertexBuffers.PositionVertexBuffer.VertexBufferRHI);
				int32 TangentVertexBufferKB = GetResourceSizeKB(RenderData.StaticVertexBuffers.StaticMeshVertexBuffer.TangentsVertexBuffer.VertexBufferRHI);
				int32 TexCoordVertexBufferKB = GetResourceSizeKB(RenderData.StaticVertexBuffers.StaticMeshVertexBuffer.TexCoordVertexBuffer.VertexBufferRHI);
				int32 IndexBufferKB = GetResourceSizeKB(RenderData.MultiSizeIndexContainer.GetIndexBuffer()->IndexBufferRHI);
				int32 ColorBufferKB = GetResourceSizeKB(RenderData.StaticVertexBuffers.ColorVertexBuffer.VertexBufferRHI);
				int32 TotalResourceKB = MorphDataKB + ClothVertexBufferKB + SkinWeightDataVertexBufferKB + SkinWeightLookupVertexBufferKB + PositionVertexBufferKB + TangentVertexBufferKB 
										+ TexCoordVertexBufferKB + IndexBufferKB + ColorBufferKB;

				FString RecomputeTangentSections;
				for (int32 SectionIdx = 0; SectionIdx < RenderData.RenderSections.Num(); SectionIdx++)
				{
					const FSkelMeshRenderSection& Section = RenderData.RenderSections[SectionIdx];
					if (Section.bRecomputeTangent)
					{
						if (RecomputeTangentSections.IsEmpty())
						{
							RecomputeTangentSections = TEXT("[Section]") + FString::FromInt(SectionIdx);
						}
						else
						{
							RecomputeTangentSections = RecomputeTangentSections + TEXT("/") + FString::FromInt(SectionIdx);
						}
					}
					
				}
				if (RecomputeTangentSections.IsEmpty())
				{
					RecomputeTangentSections = TEXT("Off");
				}

				const int32 MaxBoneInfluences = RenderData.SkinWeightVertexBuffer.GetMaxBoneInfluences();
				const bool bUseUnlimitedBoneInfluences = FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(MaxBoneInfluences);

				Ar.Logf(TEXT("  [LOD%d]: RefSkeletonBones=%3d, RequiredBones=%3d, ActiveBones=%3d, NumMorphTargets=%4d, MaxBoneInfluences=%2d, 16BitBoneIndex=%s, UnlimitedBoneInfluence=%s, TotalResource=%5dKB, IndexBuffer=%4dKB, PositionVB=%4dKB, TangentVB=%4dKB, TexCoordVB=%4dKB, ColorVB=%4dKB, SkinWeightDataVB=%4dKB, SkinWeightLookupVB=%4dKB, MorphData=%4dKB, ClothVB=%4dKB, SkinCache=%s, RecomputeTangent=%s, SupportRayTracing=%s, RayTracingMinLOD=%d"), LODIdx,
					Mesh->GetRefSkeleton().GetNum(),
					RenderData.RequiredBones.Num(),
					RenderData.ActiveBoneIndices.Num(),
					Mesh->GetMorphTargets().Num(),
					RenderData.SkinWeightVertexBuffer.GetMaxBoneInfluences(),
					RenderData.SkinWeightVertexBuffer.Use16BitBoneIndex() ? TEXT("Y") : TEXT("N"),
					bUseUnlimitedBoneInfluences ? TEXT("Y") : TEXT("N"),
					TotalResourceKB,
					IndexBufferKB,
					PositionVertexBufferKB,
					TangentVertexBufferKB,
					TexCoordVertexBufferKB,
					ColorBufferKB,
					SkinWeightDataVertexBufferKB,
					SkinWeightLookupVertexBufferKB,
					MorphDataKB,
					ClothVertexBufferKB,
					bSkinCachedLODEnabled ? TEXT("Y") : TEXT("N"),
					*RecomputeTangentSections,
					Mesh->GetSupportRayTracing() ? TEXT("Y") : TEXT("N"),
					Mesh->GetRayTracingMinLOD());
			}
		}

		NumMeshes++;
	}

	Ar.Logf(TEXT("Total %d skeletal meshes."), NumMeshes);
	Ar.Logf(TEXT("--- Skeletal Mesh Report ---"));

	return true;
}

bool UEngine::HandleParticleMeshUsageCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	// Mapping from static mesh to particle systems using it.
	TMultiMap<UStaticMesh*,UParticleSystem*> StaticMeshToParticleSystemMap;
	// Unique array of referenced static meshes, used for sorting and index into map.
	TArray<UStaticMesh*> UniqueReferencedMeshes;

	// Iterate over all mesh modules to find and keep track of mesh to system mappings.
	for( TObjectIterator<UParticleModuleTypeDataMesh> It; It; ++It )
	{
		UStaticMesh* StaticMesh = It->Mesh;
		if( StaticMesh )
		{
			// Find particle system in outer chain.
			UParticleSystem* ParticleSystem = NULL;
			UObject* Outer = It->GetOuter();
			while( Outer && !ParticleSystem )
			{
				ParticleSystem = Cast<UParticleSystem>(Outer);
				Outer = Outer->GetOuter();
			}

			// Add unique mapping from static mesh to particle system.
			if( ParticleSystem )
			{
				StaticMeshToParticleSystemMap.AddUnique( StaticMesh, ParticleSystem );
				UniqueReferencedMeshes.AddUnique( StaticMesh );
			}
		}
	}

	struct FCompareUStaticMeshByResourceSize
	{
		FORCEINLINE bool operator()( UStaticMesh& A, UStaticMesh& B ) const
		{
			const SIZE_T ResourceSizeA = A.GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
			const SIZE_T ResourceSizeB = B.GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
			return ResourceSizeB < ResourceSizeA;
		}
	};

	// Sort by resource size.
	UniqueReferencedMeshes.Sort( FCompareUStaticMeshByResourceSize() );

	// Calculate total size for summary.
	int32 TotalSize = 0;
	for( int32 StaticMeshIndex=0; StaticMeshIndex<UniqueReferencedMeshes.Num(); StaticMeshIndex++ )
	{
		UStaticMesh* StaticMesh	= UniqueReferencedMeshes[StaticMeshIndex];
		const SIZE_T StaticMeshResourceSize = StaticMesh->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
		TotalSize += StaticMeshResourceSize;
	}

	// Log sorted summary.
	Ar.Logf(TEXT("%5i KByte of static meshes referenced by particle systems:"),TotalSize / 1024);
	for( int32 StaticMeshIndex=0; StaticMeshIndex<UniqueReferencedMeshes.Num(); StaticMeshIndex++ )
	{
		UStaticMesh* StaticMesh	= UniqueReferencedMeshes[StaticMeshIndex];

		// Find all particle systems using this static mesh.
		TArray<UParticleSystem*> ParticleSystems;
		StaticMeshToParticleSystemMap.MultiFind( StaticMesh, ParticleSystems );

		const SIZE_T StaticMeshResourceSize = StaticMesh->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);

		// Log meshes including resource size and referencing particle systems.
		Ar.Logf(TEXT("%5i KByte  %s"), StaticMeshResourceSize / 1024, *StaticMesh->GetFullName());
		for( int32 ParticleSystemIndex=0; ParticleSystemIndex<ParticleSystems.Num(); ParticleSystemIndex++ )
		{
			UParticleSystem* ParticleSystem = ParticleSystems[ParticleSystemIndex];
			Ar.Logf(TEXT("             %s"),*ParticleSystem->GetFullName());
		}
	}

	return true;
}

class ParticleSystemUsage
{
public:
	UParticleSystem* Template;
	int32	Count;
	int32	ActiveTotal;
	int32	MaxActiveTotal;
	// Reported whether the emitters are instanced or not...
	int32	StoredMaxActiveTotal;

	TArray<int32>		EmitterActiveTotal;
	TArray<int32>		EmitterMaxActiveTotal;
	// Reported whether the emitters are instanced or not...
	TArray<int32>		EmitterStoredMaxActiveTotal;

	ParticleSystemUsage() :
		Template(NULL),
		Count(0),
		ActiveTotal(0),
		MaxActiveTotal(0),
		StoredMaxActiveTotal(0)
	{
	}

	~ParticleSystemUsage()
	{
	}
};

bool UEngine::HandleDumpParticleCountsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TMap<UParticleSystem*, ParticleSystemUsage> UsageMap;

	bool bTrackUsage = FParse::Command(&Cmd, TEXT("USAGE"));
	bool bTrackUsageOnly = FParse::Command(&Cmd, TEXT("USAGEONLY"));
	for( TObjectIterator<UObject> It; It; ++It )
	{
		UParticleSystemComponent* PSysComp = Cast<UParticleSystemComponent>(*It);
		if (PSysComp)
		{
			ParticleSystemUsage* Usage = NULL;

			if (bTrackUsageOnly == false)
			{
				Ar.Logf( TEXT("ParticleSystemComponent %s"), *(PSysComp->GetName()));
			}

			UParticleSystem* PSysTemplate = PSysComp->Template;
			if (PSysTemplate != NULL)
			{
				if (bTrackUsage || bTrackUsageOnly)
				{
					ParticleSystemUsage* pUsage = UsageMap.Find(PSysTemplate);
					if (pUsage == NULL)
					{
						ParticleSystemUsage TempUsage;
						TempUsage.Template = PSysTemplate;
						TempUsage.Count = 1;

						UsageMap.Add(PSysTemplate, TempUsage);
						Usage = UsageMap.Find(PSysTemplate);
						check(Usage);
					}					
					else
					{
						Usage = pUsage;
						Usage->Count++;
					}
				}
				if (bTrackUsageOnly == false)
				{
					Ar.Logf( TEXT("\tTemplate         : %s"), *(PSysTemplate->GetPathName()));
				}
			}
			else
			{
				if (bTrackUsageOnly == false)
				{
					Ar.Logf( TEXT("\tTemplate         : %s"), TEXT("NULL"));
				}
			}

			// Dump each emitter
			int32 TotalActiveCount = 0;
			if (bTrackUsageOnly == false)
			{
				Ar.Logf( TEXT("\tEmitterCount     : %d"), PSysComp->EmitterInstances.Num());
			}

			if (PSysComp->EmitterInstances.Num() > 0)
			{
				for (int32 EmitterIndex = 0; EmitterIndex < PSysComp->EmitterInstances.Num(); EmitterIndex++)
				{
					FParticleEmitterInstance* EmitInst = PSysComp->EmitterInstances[EmitterIndex];
					if (EmitInst)
					{
						UParticleLODLevel* LODLevel = EmitInst->SpriteTemplate ? ToRawPtr(EmitInst->SpriteTemplate->LODLevels[0]) : NULL;
						if (bTrackUsageOnly == false)
						{
							Ar.Logf( TEXT("\t\tEmitter %2d:\tActive = %4d\tMaxActive = %4d"), 
								EmitterIndex, EmitInst->ActiveParticles, EmitInst->MaxActiveParticles);
						}
						TotalActiveCount += EmitInst->MaxActiveParticles;
						if (bTrackUsage || bTrackUsageOnly)
						{
							check(Usage);
							Usage->ActiveTotal += EmitInst->ActiveParticles;
							Usage->MaxActiveTotal += EmitInst->MaxActiveParticles;
							Usage->StoredMaxActiveTotal += EmitInst->MaxActiveParticles;
							if (Usage->EmitterActiveTotal.Num() <= EmitterIndex)
							{
								int32 CheckIndex;
								CheckIndex = Usage->EmitterActiveTotal.AddZeroed(1);
								check(CheckIndex == EmitterIndex);
								CheckIndex = Usage->EmitterMaxActiveTotal.AddZeroed(1);
								check(CheckIndex == EmitterIndex);
								CheckIndex = Usage->EmitterStoredMaxActiveTotal.AddZeroed(1);
								check(CheckIndex == EmitterIndex);
							}
							Usage->EmitterActiveTotal[EmitterIndex] = Usage->EmitterActiveTotal[EmitterIndex] + EmitInst->ActiveParticles;
							Usage->EmitterMaxActiveTotal[EmitterIndex] = Usage->EmitterMaxActiveTotal[EmitterIndex] + EmitInst->MaxActiveParticles;
							Usage->EmitterStoredMaxActiveTotal[EmitterIndex] = Usage->EmitterStoredMaxActiveTotal[EmitterIndex] + EmitInst->MaxActiveParticles;
						}
					}
					else
					{
						if (bTrackUsageOnly == false)
						{
							Ar.Logf( TEXT("\t\tEmitter %2d:\tActive = %4d\tMaxActive = %4d"), EmitterIndex, 0, 0);
						}
					}
				}
			}
			else
				if (PSysTemplate != NULL)
				{
					for (int32 EmitterIndex = 0; EmitterIndex < PSysTemplate->Emitters.Num(); EmitterIndex++)
					{
						UParticleEmitter* Emitter = PSysTemplate->Emitters[EmitterIndex];
						if (Emitter)
						{
							int32 MaxActive = 0;

							for (int32 LODIndex = 0; LODIndex < Emitter->LODLevels.Num(); LODIndex++)
							{
								UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIndex];
								if (LODLevel)
								{
									if (LODLevel->PeakActiveParticles > MaxActive)
									{
										MaxActive = LODLevel->PeakActiveParticles;
									}
								}
							}

							if (bTrackUsage || bTrackUsageOnly)
							{
								check(Usage);
								Usage->StoredMaxActiveTotal += MaxActive;
								if (Usage->EmitterStoredMaxActiveTotal.Num() <= EmitterIndex)
								{
									int32 CheckIndex;
									CheckIndex = Usage->EmitterActiveTotal.AddZeroed(1);
									check(CheckIndex == EmitterIndex);
									CheckIndex = Usage->EmitterMaxActiveTotal.AddZeroed(1);
									check(CheckIndex == EmitterIndex);
									CheckIndex = Usage->EmitterStoredMaxActiveTotal.AddZeroed(1);
									check(CheckIndex == EmitterIndex);
								}
								// Don't update the non-stored entries...
								Usage->EmitterStoredMaxActiveTotal[EmitterIndex] = Usage->EmitterStoredMaxActiveTotal[EmitterIndex] + MaxActive;
							}
						}
					}
				}
			if (bTrackUsageOnly == false)
			{
				Ar.Logf( TEXT("\tTotalActiveCount : %d"), TotalActiveCount);
			}
		}
	}

	if (bTrackUsage || bTrackUsageOnly)
	{
		Ar.Logf( TEXT("PARTICLE USAGE DUMP:"));
		for (TMap<UParticleSystem*, ParticleSystemUsage>::TIterator It(UsageMap); It; ++It)
		{
			ParticleSystemUsage& Usage = It.Value();
			UParticleSystem* Template = Usage.Template;
			check(Template);

			Ar.Logf( TEXT("\tParticleSystem..%s"), *(Usage.Template->GetPathName()));
			Ar.Logf( TEXT("\t\tCount.....................%d"), Usage.Count);
			Ar.Logf( TEXT("\t\tActiveTotal...............%5d"), Usage.ActiveTotal);
			Ar.Logf( TEXT("\t\tMaxActiveTotal............%5d (%4d per instance)"), Usage.MaxActiveTotal, (Usage.MaxActiveTotal / Usage.Count));
			Ar.Logf( TEXT("\t\tPotentialMaxActiveTotal...%5d (%4d per instance)"), Usage.StoredMaxActiveTotal, (Usage.StoredMaxActiveTotal / Usage.Count));
			Ar.Logf( TEXT("\t\tEmitters..................%d"), Usage.EmitterActiveTotal.Num());
			check(Usage.EmitterActiveTotal.Num() == Usage.EmitterMaxActiveTotal.Num());
			for (int32 EmitterIndex = 0; EmitterIndex < Usage.EmitterActiveTotal.Num(); EmitterIndex++)
			{
				int32 EActiveTotal = Usage.EmitterActiveTotal[EmitterIndex];
				int32 EMaxActiveTotal = Usage.EmitterMaxActiveTotal[EmitterIndex];
				int32 EStoredMaxActiveTotal = Usage.EmitterStoredMaxActiveTotal[EmitterIndex];
				Ar.Logf( TEXT("\t\t\tEmitter %2d - AT = %5d, MT = %5d (%4d per emitter), Potential MT = %5d (%4d per emitter)"),
					EmitterIndex, EActiveTotal,
					EMaxActiveTotal, (EMaxActiveTotal / Usage.Count),
					EStoredMaxActiveTotal, (EStoredMaxActiveTotal / Usage.Count)
				);
			}
		}
	}
	return true;
}


bool UEngine::HandleListLoadedPackagesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	struct FPackageInfo
	{
		FString Name;
		float LoadTime;
		UClass* AssetType;

		FPackageInfo(UPackage* InPackage)
		{
			Name = InPackage->GetPathName();
			LoadTime = InPackage->GetLoadTime();
			AssetType = nullptr;
		}
	};

	TArray<FPackageInfo> Packages;

	TArray<UObject*> ObjectsInPackageTemp;

	for (TObjectIterator<UPackage> It; It; ++It)
	{
		UPackage* Package = *It;

		const bool bIsARootPackage = Package->GetOuter() == nullptr;

		if (bIsARootPackage == true)
		{
			const int32 NewIndex = Packages.Emplace(Package);

			// Determine the contained asset type
			ObjectsInPackageTemp.Reset();
			GetObjectsWithOuter(Package, /*out*/ ObjectsInPackageTemp, EGetObjectsFlags::None);

			UClass* AssetType = nullptr;
			for (UObject* Object : ObjectsInPackageTemp)
			{
				if (!Object->IsA(UClass::StaticClass()) && !Object->HasAnyFlags(RF_ClassDefaultObject))
				{
					AssetType = Object->GetClass();
					break;
				}
			}

			Packages[NewIndex].AssetType = AssetType;
		}
	}

	// Sort by name
	Packages.Sort([](const FPackageInfo& A, const FPackageInfo& B) { return A.Name < B.Name; });

	Ar.Logf(TEXT("List of all loaded packages"));
	Ar.Logf(TEXT("Name,Type,LoadTime"), Packages.Num());
	for (const FPackageInfo& Info : Packages)
	{
		Ar.Logf(TEXT("%s,%s,%f"), *Info.Name, (Info.AssetType != nullptr) ? *Info.AssetType->GetName() : TEXT("unknown"), Info.LoadTime);
	}

	Ar.Logf( TEXT( "Total Number Of Packages Loaded: %i " ), Packages.Num() );

	return true;
}

bool UEngine::HandleMemCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	const FString Token = FParse::Token( Cmd, 0 );
	const bool bDetailed = ( Token == TEXT("DETAILED") || Token == TEXT("STAT"));
	const bool bReport = ( Token == TEXT("FROMREPORT") );

	if (!bReport)
	{
		// Mem report is called 
		FlushAsyncLoading();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS,true);
		FlushRenderingCommands();
	}

#if !NO_LOGGING
	const FName CategoryName(LogMemory.GetCategoryName());
#else
	const FName CategoryName("LogMemory");
#endif
	FPlatformMemory::DumpStats( Ar );
	Ar.CategorizedLogf( CategoryName, ELogVerbosity::Log, TEXT("") );
	UE::Private::GMalloc->DumpAllocatorStats( Ar );

	if( bDetailed || bReport)
	{
		Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Memory Stats:") );
		Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("FMemStack (gamethread) current size = %.2f MB"), FMemStack::Get().GetByteCount() / (1024.0f * 1024.0f));
		Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("FPageAllocator (all threads) allocation size [used/ unused] = [%.2f / %.2f] MB"), (FPageAllocator::Get().BytesUsed()) / (1024.0f * 1024.0f), (FPageAllocator::Get().BytesFree()) / (1024.0f * 1024.0f));
		Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("Nametable memory usage = %.2f MB"), FName::GetNameTableMemorySize() / (1024.0f * 1024.0f));

		FAssetRegistryModule* AssetRegistryModule = FModuleManager::LoadModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
		if (AssetRegistryModule)
		{
			Ar.CategorizedLogf(CategoryName, ELogVerbosity::Log, TEXT("AssetRegistry memory usage = %.2f MB"), AssetRegistryModule->Get().GetAllocatedSize() / (1024.0f * 1024.0f));
		}

#if STATS
		TArray<FStatMessage> Stats;
		GetPermanentStats(Stats);

		FName NAME_STATGROUP_SceneMemory( FStatGroup_STATGROUP_SceneMemory::GetGroupName() );
		FName NAME_STATGROUP_Memory(FStatGroup_STATGROUP_Memory::GetGroupName());
		FName NAME_STATGROUP_TextureGroup("STATGROUP_TextureGroup");
		FName NAME_STATGROUP_RHI(FStatGroup_STATGROUP_RHI::GetGroupName());

		for (int32 Index = 0; Index < Stats.Num(); Index++)
		{
			FStatMessage const& Meta = Stats[Index];
			if (Meta.NameAndInfo.GetFlag(EStatMetaFlags::IsMemory))
			{
				Ar.CategorizedLogf( CategoryName, ELogVerbosity::Log, TEXT("%s"), *FStatsUtils::DebugPrint(Meta));
			}
		}
#endif
	}

	return true;
}

bool UEngine::HandleDebugCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if (FParse::Command(&Cmd, TEXT("RESETLOADERS")))
	{
		ResetLoaders( NULL );
		return true;
	}

	// Handle "DEBUG CRASH" etc. 
	return PerformError(Cmd, Ar);
}


bool UEngine::HandleMergeMeshCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	FString CmdCopy = Cmd;
	TArray<FString> Tokens;
	while (CmdCopy.Len() > 0)
	{
		const TCHAR* LocalCmd = *CmdCopy;
		FString Token = FParse::Token( LocalCmd, true );
		Tokens.Add( Token );
		CmdCopy.RightInline( CmdCopy.Len() - Token.Len() - 1, EAllowShrinking::No);
	}

	// array of source meshes that will be merged
	TArray<USkeletalMesh*> SourceMeshList;

	if (Tokens.Num() >= 2)
	{
		for (int32 I = 0; I<Tokens.Num(); ++I)
		{
			USkeletalMesh * SrcMesh = LoadObject<USkeletalMesh>( NULL, *Tokens[I], NULL, LOAD_None, NULL );
			if (SrcMesh)
			{
				SourceMeshList.Add( SrcMesh );
			}
		}
	}

	// find player controller skeletalmesh
	APawn * PlayerPawn = NULL;
	USkeletalMesh * PlayerMesh = NULL;
	for (FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		APlayerController* PlayerController = Iterator->Get();
		if (PlayerController && PlayerController->GetCharacter() != NULL && PlayerController->GetCharacter()->GetMesh())
		{
			PlayerPawn = PlayerController->GetCharacter();
			PlayerMesh = PlayerController->GetCharacter()->GetMesh()->GetSkeletalMeshAsset();
			break;
		}
	}

	if (PlayerMesh)
	{
		if (SourceMeshList.Num() == 0)
		{
			SourceMeshList.Add( PlayerMesh );
			SourceMeshList.Add( PlayerMesh );
		}
	}
	else
	{
		// we don't have a pawn (because we couldn't find a mesh), use any pawn as a spawn point
		for (FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PlayerController = Iterator->Get();
			if (PlayerController && PlayerController->GetPawn() != NULL)
			{
				PlayerPawn = PlayerController->GetPawn();
				break;
			}
		}
	}

	if (PlayerPawn && SourceMeshList.Num() >= 2)
	{
		// create the composite mesh
		USkeletalMesh* CompositeMesh = NewObject<USkeletalMesh>( GetTransientPackage(), NAME_None, RF_Transient );

		TArray<FSkelMeshMergeSectionMapping> InForceSectionMapping;
		// create an instance of the FSkeletalMeshMerge utility
		FSkeletalMeshMerge MeshMergeUtil( CompositeMesh, SourceMeshList, InForceSectionMapping, 0 );

		// merge the source meshes into the composite mesh
		if (!MeshMergeUtil.DoMerge())
		{
			// handle errors
			// ...
			UE_LOGF( LogEngine, Log, "DoMerge Error: Merge Mesh Test Failed" );
			return true;
		}

		FVector SpawnLocation = PlayerPawn->GetActorLocation() + PlayerPawn->GetActorForwardVector()*50.f;

		// set the new composite mesh in the existing skeletal mesh component
		ASkeletalMeshActor* const SMA = PlayerPawn->GetWorld()->SpawnActor<ASkeletalMeshActor>( SpawnLocation, PlayerPawn->GetActorRotation()*-1 );
		if (SMA)
		{
			SMA->GetSkeletalMeshComponent()->SetSkeletalMesh( CompositeMesh );
		}
	}

	return true;
}

bool UEngine::HandleContentComparisonCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TArray<FString> Tokens, Switches;
	FCommandLine::Parse(Cmd, Tokens, Switches);
	if (Tokens.Num() > 0)
	{
		// The first token MUST be the base class name of interest
		FString BaseClassName = Tokens[0];
		TArray<FString> BaseClassesToIgnore;
		int32 Depth = 1;
		for (int32 TokenIdx = 1; TokenIdx < Tokens.Num(); TokenIdx++)
		{
			FString Token = Tokens[TokenIdx];
			FString TempString;
			if (FParse::Value(*Token, TEXT("DEPTH="), TempString))
			{
				Depth = FCString::Atoi(*TempString);
			}
			else
			{
				BaseClassesToIgnore.Add(Token);
				UE_LOGF(LogEngine, Log, "Added ignored base class: %ls", *Token);
			}
		}

		UE_LOGF(LogEngine, Log, "Calling CompareClasses w/ Depth of %d on %ls", Depth, *BaseClassName);
		UE_LOGF(LogEngine, Log, "Ignoring base classes:");
		for (int32 DumpIdx = 0; DumpIdx < BaseClassesToIgnore.Num(); DumpIdx++)
		{
			UE_LOGF(LogEngine, Log, "\t%ls", *(BaseClassesToIgnore[DumpIdx]));
		}
		FContentComparisonHelper ContentComparisonHelper;
		ContentComparisonHelper.CompareClasses(BaseClassName, BaseClassesToIgnore, Depth);
	}
	return true;
}

bool UEngine::HandleTogglegtPsysLODCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	extern bool GbEnableGameThreadLODCalculation;
	GbEnableGameThreadLODCalculation = !GbEnableGameThreadLODCalculation;
	UE_LOG(LogEngine, Warning, TEXT("Particle LOD determination is now on the %s thread!"),
		GbEnableGameThreadLODCalculation ? TEXT("GAME") : TEXT("RENDER"));
	return true;
}

#if !UE_BUILD_SHIPPING

static int32 FindAmountSavedWithCutoff(UClass* AnalyzedClass, int32 PercentageAllowed, FOutputDevice* Ar)
{

	// this is our default values to compare too
	UObject* Defaults = AnalyzedClass->GetDefaultObject();

	// cache instances of the class
	TArray<UObject*> Instances;
	for (FThreadSafeObjectIterator It(AnalyzedClass); It; ++It)
	{
		if (It->IsTemplate(RF_ClassDefaultObject))
		{
			continue;
		}
		Instances.Add(*It);
	}

	struct FPerPropInfo
	{
		FPerPropInfo(int32 InSize) : DiffCount(0), Size(InSize), bIsOnChoppingBlock(false) {}
		int32 DiffCount;
		int32 Size;
		bool bIsOnChoppingBlock;
	};

	TMap<FProperty*, FPerPropInfo> PropertyDiffCount;

	// work on each property of the class
	for (TFieldIterator<FProperty> It(AnalyzedClass); It; ++It)
	{
		PropertyDiffCount.Add(*It, FPerPropInfo(It->GetSize()));
	}

	// now go over each instance comparing for Identicalness
	for (auto& It : PropertyDiffCount)
	{
		for (UObject* Instance : Instances)
		{
			// is this property for this instance different than the default?
			if (!It.Key->Identical(((int8*)Defaults) + It.Key->GetOffset_ForInternal(), ((int8*)Instance) + It.Key->GetOffset_ForInternal()))
			{
				// count the differences
				It.Value.DiffCount++;
			}
		}
	}

	// now report back about differences!
	if (Ar)
	{
		Ar->Logf(TEXT(""));
		Ar->Logf(TEXT(""));
		Ar->Logf(TEXT("Analysis of class %s. Listed are properties that mostly identical to defaults"), *AnalyzedClass->GetName(), Instances.Num());
	}
	int32 TotalSizeOfChoppingBlock = 0;
	for (auto& It : PropertyDiffCount)
	{
		float PercentDifferent = 100.0f * (float)It.Value.DiffCount / (float)Instances.Num();
		if (PercentDifferent <= PercentageAllowed)
		{
			if (Ar)
			{
				Ar->Logf(TEXT("[REMV] %2.2f%% differ from default value of %s"), PercentDifferent, *It.Key->GetName());
			}
			It.Value.bIsOnChoppingBlock = true;
			TotalSizeOfChoppingBlock += It.Value.Size;
		}
		// 		else if (Ar)
		// 		{
		// 			Ar->Logf(TEXT("[KEEP] %2.2f%% differ from default value of %s"), PercentDifferent, *It.Key->GetName());
		// 		}
	}


	int32 NumInstancesThatNeedAtLeastOne = 0;
	for (UObject* Instance : Instances)
	{
		for (auto& It : PropertyDiffCount)
		{
			// is this property for this instance different than the default?
			if (It.Value.bIsOnChoppingBlock &&
				!It.Key->Identical(((int8*)Defaults) + It.Key->GetOffset_ForInternal(), ((int8*)Instance) + It.Key->GetOffset_ForInternal()))
			{
				NumInstancesThatNeedAtLeastOne++;
				break;
			}
		}
	}

	// calculate how much we could save here
	int32 AmountSaved = (Instances.Num() - NumInstancesThatNeedAtLeastOne) * TotalSizeOfChoppingBlock;

	if (Ar)
	{
		if (PercentageAllowed == 0)
		{
			Ar->Logf(TEXT("--- With no property differences allowed ---"));
			Ar->Logf(TEXT("If the listed properties are removed none of the %d would be affected. All properties are identical to defaults."), Instances.Num());
		}
		else
		{
			Ar->Logf(TEXT("--- %d percent of instances with different properties ---"), PercentageAllowed);
			Ar->Logf(TEXT("If the listed properties are removed %d (of %d) instances would need the extended properties."), NumInstancesThatNeedAtLeastOne, Instances.Num());
		}
		Ar->Logf(TEXT("This would save %d bytes per instance, for a total of %.3fmb."), TotalSizeOfChoppingBlock, (float)AmountSaved / (1024.0f * 1024.0f));
	}

	return AmountSaved;
}

static void HandlePropAnalysisCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	UClass* AnalyzedClass = nullptr;
	int32 MinPercentToCheck = 0;
	int32 MaxPercentToCheck = 25;

	// get command parameters
	ParseObject<UClass>(Cmd, TEXT("CLASS="), AnalyzedClass, nullptr);
	if (AnalyzedClass == nullptr)
	{
		Ar.Log(TEXT("Usage: PROPANALYSIS CLASS=<class> [MIN=<minpercentdiff> MAX=<maxpercentdiff>]"));
		return;
	}

	FParse::Value(Cmd, TEXT("MIN="), MinPercentToCheck);
	FParse::Value(Cmd, TEXT("MAX="), MaxPercentToCheck);

	int32 MaxAmountSavedPercent = 0;
	int32 MaxAmountSaved = 0;
	for (int32 Percent = MinPercentToCheck; Percent <= MaxPercentToCheck; Percent++)
	{
		int32 AmountSaved = FindAmountSavedWithCutoff(AnalyzedClass, Percent, nullptr);
		if (AmountSaved > MaxAmountSaved)
		{
			MaxAmountSaved = AmountSaved;
			MaxAmountSavedPercent = Percent;
		}
	}

	FindAmountSavedWithCutoff(AnalyzedClass, 0, &Ar);
	FindAmountSavedWithCutoff(AnalyzedClass, MaxAmountSavedPercent, &Ar);
}

#endif

struct FHierarchyNode
{
	UObject* This;
	UObject* Parent;
	TSet<UObject*> Children;
	TSet<UObject*> Items;
	int64 Inc;
	int64 Exc;
	int32 IncCount;
	int32 ExcCount;
	FHierarchyNode()
		: This(NULL)
		, Parent(NULL)
		, Inc(-1)
		, Exc(-1)
		, IncCount(-1)
		, ExcCount(-1)
	{
	}
	bool operator< (FHierarchyNode const& Other) const
	{
		return Inc > Other.Inc;
	}

	bool IsLeaf()
	{
		return Children.Num() + Items.Num() == 0;
	}
};

struct FHierarchy
{
	int64 Limit;
	TMap<UObject*, FHierarchyNode> Nodes;

	FHierarchy(int32 InLimit)
		: Limit(InLimit)
	{
	}
	FHierarchyNode& AddFlat(UObject* This)
	{
		FHierarchyNode& Node = Nodes.FindOrAdd(This);
		if (!Node.This && This)
		{
			Node.This = This;
			Node.Parent = NULL;
			AddFlat(NULL).Children.Add(This);
		}
		return Node;
	}
	FHierarchyNode& AddOuter(UObject* This)
	{
		FHierarchyNode& Node = Nodes.FindOrAdd(This);
		if (!Node.This && This)
		{
			Node.This = This;
			Node.Parent = This->GetOuter();
			AddOuter(Node.Parent).Children.Add(This);
		}
		return Node;
	}
	FHierarchyNode& AddClass(UClass* This)
	{
		FHierarchyNode& Node = Nodes.FindOrAdd(This);
		if (!Node.This && This)
		{
			Node.This = This;
			Node.Parent = This->GetSuperClass();
			AddClass(This->GetSuperClass()).Children.Add(This);
		}
		return Node;
	}
	void AddClassInstance(UObject* This)
	{
		if (!This->IsA(UClass::StaticClass()))
		{
			AddClass(This->GetClass()).Items.Add(This);
			FHierarchyNode& Node = Nodes.FindOrAdd(This);
			if (!Node.This)
			{
				Node.This = This;
				Node.Parent = This->GetClass();
			}
		}
		else
		{
			AddClass((UClass*)This);
		}
	}
	FHierarchyNode& Compute(UObject* This, TMap<UObject*, UE::FSubItem> const& Objects, bool bCountItems)
	{
		FHierarchyNode& Node = Nodes.FindChecked(This);
		if (Node.Inc < 0)
		{
			Node.Exc = 0;
			Node.ExcCount = 1;
			if (This)
			{
				UE::FSubItem const& Item = Objects.FindChecked(This);
				Node.Exc += Item.Max;
				Node.Exc += Item.TrueResourceSize.GetTotalMemoryBytes();
				if (bCountItems)
				{
					Node.ExcCount += Node.Items.Num();
				}
				else
				{
					Node.ExcCount += Node.Children.Num();
				}
			}
			Node.Inc = Node.Exc;
			Node.IncCount = Node.ExcCount;
			for (TSet<UObject*>::TConstIterator It(Node.Children); It; ++It)
			{
				FHierarchyNode& Child = Compute(*It, Objects, bCountItems);
				Node.Inc += Child.Inc;
				if (!bCountItems)
				{
					Node.IncCount += Child.IncCount;
				}
			}
			for (TSet<UObject*>::TConstIterator It(Node.Items); It; ++It)
			{
				FHierarchyNode& Child = Compute(*It, Objects, bCountItems);
				Node.Inc += Child.Inc;
				if (bCountItems)
				{
					Node.IncCount += Child.IncCount;
				}
			}
		}
		return Node;
	}
	void SortSet(TSet<UObject*> const& In, TArray<FHierarchyNode>& Out)
	{
		Out.Empty(In.Num());
		for (TSet<UObject*>::TConstIterator It(In); It; ++It)
		{
			Out.Add(Nodes.FindChecked(*It));
		}
		Out.Sort();
	}
	static FString Size(uint64 Mem)
	{
		if (Mem / 1024 < 10000)
		{
			return FString::Printf(TEXT("%4lldK"), Mem / 1024);
		}
		if (Mem / (1024*1024) < 10000)
		{
			return FString::Printf(TEXT("%4lldM"), Mem / (1024*1024));
		}
		return FString::Printf(TEXT("%4lldG"), Mem / (1024*1024*1024));
	}
	void LogSet(TSet<UObject*> const& In, UClass* ClassToCheck, bool bCntItems, int Indent)
	{
		TArray<FHierarchyNode> Children;
		SortSet(In, Children);
		int32 Index = 0;
		for (; Index < Children.Num(); Index++)
		{
			UObject* Child = Children[Index].This;
			// Only makes sense for flat hierarchy.
			const bool bIsClassToCheck = Child->IsA( ClassToCheck );
			if( !bIsClassToCheck )
			{
				continue;
			}

			if( !Log( Child, ClassToCheck, bCntItems, Indent + 1, Index + 1 < Children.Num() ) )
			{
				break;
			}
		}
		if (Index < Children.Num())
		{
			int32 NumExtra = 0;
			FHierarchyNode Extra;
			Extra.Exc = 0;
			Extra.Inc = 0;
			Extra.ExcCount = 0;
			Extra.IncCount = 0;
			for (; Index < Children.Num(); Index++)
			{
				Extra.Exc += Children[Index].Exc;
				Extra.Inc += Children[Index].Inc;
				Extra.ExcCount += Children[Index].ExcCount;
				Extra.IncCount += Children[Index].IncCount;
				NumExtra++;
			}
			FString Line = FString::Printf(TEXT("%s        %5d %s (%d)"), *Size(Extra.Inc), Extra.IncCount, TEXT("More"), NumExtra);
			UE_LOGF(LogEngine, Log, "%ls%ls" , FCString::Spc(2 * (Indent + 1)), *Line);
		}
	}
	bool Log(UObject* This, UClass* ClassToCheck, bool bCountItems, int Indent = 0, bool bAllowCull = true)
	{
		FHierarchyNode& Node = Nodes.FindChecked(This);
		if (bAllowCull && Node.Inc < Limit && Node.Exc < Limit)
		{
			return false;
		}
		FString Line;
		if (Node.IsLeaf())
		{
			Line = FString::Printf(TEXT("%s        %5d %s"), *Size(Node.Inc), Node.IncCount, Node.This ? *Node.This->GetFullName() : TEXT("Root"));
			UE_LOGF(LogEngine, Log, "%ls%ls" , FCString::Spc(2 * Indent), *Line);
		}
		else
		{
			Line = FString::Printf(TEXT("%s %sx %5d %s"), *Size(Node.Inc), *Size(Node.Exc), Node.IncCount, Node.This ? *Node.This->GetFullName() : TEXT("Root"));
			UE_LOGF(LogEngine, Log, "%ls%ls" , FCString::Spc(2 * Indent), *Line);
			if (bCountItems && Node.Children.Num())
			{
				UE_LOGF(LogEngine, Log, "%ls%ls" , FCString::Spc(2 * (Indent + 1)), TEXT("Child Classes"));
			}
			LogSet(Node.Children, ClassToCheck, bCountItems, Indent + 2);

			if (bCountItems && Node.Items.Num())
			{
				UE_LOGF(LogEngine, Log, "%ls%ls" , FCString::Spc(2 * (Indent + 1)), TEXT("Instances"));
			}
			LogSet(Node.Items, ClassToCheck, bCountItems, Indent);
		}

		return true;
	}
};

// #TODO Move to ObjectCommads.cpp or ObjectExec.cpp
bool UEngine::HandleObjCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	if( FParse::Command(&Cmd,TEXT("GARBAGE")) || FParse::Command(&Cmd,TEXT("GC")) )
	{
		bool bIncremental = FParse::Command(&Cmd, TEXT("INCREMENTAL"));

		// Purge unclaimed objects.
		Ar.Logf(TEXT("Collecting garbage and resetting GC timer."));
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, !bIncremental);
		TimeSinceLastPendingKillPurge = 0.f;
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("TRYGC")))
	{
		bool bIncremental = FParse::Command(&Cmd, TEXT("INCREMENTAL"));

		// Purge unclaimed objects.		
		if (TryCollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, !bIncremental))
		{
			Ar.Logf(TEXT("Collecting garbage and resetting GC timer."));
			TimeSinceLastPendingKillPurge = 0.f;
		}
		else
		{
			Ar.Logf(TEXT("Garbage collection blocked by other threads."));
		}
		return true;
	}
#if WITH_EDITOR
	else if (FParse::Command(&Cmd, TEXT("RELOADPKG")) || FParse::Command(&Cmd, TEXT("RELOADPACKAGE")))
	{
		if (UPackage* Pkg = nullptr;
			ParseObject<UPackage>(Cmd, TEXT("PACKAGE="), Pkg, nullptr))
		{
			if (FText ReloadPkgError;
				UPackageTools::ReloadPackages({ Pkg }, ReloadPkgError, EReloadPackagesInteractionMode::AssumePositive))
			{
				UE_SUPPRESS(LogExec, Log, Ar.Log(TEXT("Package reloaded successfully.")));
			}
			else
			{
				UE_SUPPRESS(LogExec, Error, Ar.Logf(TEXT("Package failed to reload: %s."), *ReloadPkgError.ToString()));
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Error, Ar.Log(TEXT("Package wasn't found.")));
		}
		return true;
	}
#endif	// #if WITH_EDITOR
#if !UE_BUILD_SHIPPING
	else if (FParse::Command(&Cmd,TEXT("LIST2")))
	{			
		UClass* ClassToCheck = NULL;
		ParseObject<UClass>(Cmd, TEXT("CLASS="  ), ClassToCheck, nullptr);

		if (ClassToCheck == NULL)
		{
			ClassToCheck = UObject::StaticClass();
		}

		FObjectMemoryAnalyzer MemAnalyze(ClassToCheck);
		MemAnalyze.PrintResults(Ar, FObjectMemoryAnalyzer::EPrintFlags::PrintReferences);
		return true;
	}
	else if (FParse::Command(&Cmd,TEXT("MemSub")))
	{
		struct FCompareByInclusiveSize
		{
			FORCEINLINE bool operator()( const UE::FItem& A, const UE::FItem& B ) const 
			{ 
				return A.Max > B.Max; 
			}
		};

		struct FLocal
		{
			static void GetReferencedObjs( UObject* CurrentObject, TArray<UObject*>& out_ReferencedObjs )
			{
				TArray<UObject*> ReferencedObjs;
				FReferenceFinder RefCollector( ReferencedObjs, CurrentObject, true, false, false, false );
				RefCollector.FindReferences( CurrentObject );

				out_ReferencedObjs.Append( ReferencedObjs );
				for( UObject*& RefObj : ReferencedObjs )
				{
					GetReferencedObjs( RefObj, out_ReferencedObjs );
				}
			}
		};

		int32 Limit = 16;
		FParse::Value(Cmd, TEXT("CULL="), Limit);
		Limit *= 1024;

		UClass* ClassToCheck = NULL;
		ParseObject<UClass>(Cmd, TEXT("CLASS="  ), ClassToCheck, nullptr);
		if (ClassToCheck == NULL)
		{
			ClassToCheck = UObject::StaticClass();
		}

		TMap<UClass*,UE::FItem> ObjectsByClass;

		Ar.Logf( TEXT("**********************************************") );
		Ar.Logf( TEXT("Obj MemSub for class '%s'"), *ClassToCheck->GetName() );
		Ar.Logf( TEXT("") );

		for( FThreadSafeObjectIterator It(ClassToCheck); It; ++It )
		{
			UObject* Obj = *It;
			if( Obj->IsTemplate( RF_ClassDefaultObject ) )
			{
				continue;
			}

			// Get references.
			TArray<UObject*> ReferencedObjects;
			FLocal::GetReferencedObjs( Obj, ReferencedObjects );

			// Calculate memory usage.
			UE::FItem ThisObject( Obj->GetClass() );
			for( UObject*& RefObj : ReferencedObjects )
			{
				FArchiveCountMem Count( RefObj );
				FResourceSizeEx TrueResourceSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
				It->GetResourceSizeEx( TrueResourceSize );
				ThisObject.Add( Count, TrueResourceSize );
			}

			UE::FItem& ClassObjects = ObjectsByClass.FindOrAdd( ThisObject.Class );
			ClassObjects.Count++;
			ClassObjects.Num += ThisObject.Num;
			ClassObjects.Max += ThisObject.Max;
			ClassObjects.TrueResourceSize += ThisObject.TrueResourceSize;
		}

		ObjectsByClass.ValueSort( FCompareByInclusiveSize() );

		Ar.Logf( 
			TEXT("%32s %12s %12s %12s %12s %12s %12s %12s %12s %12s"), 
			TEXT("Class"),
			TEXT("IncMax"),
			TEXT("IncNum"),
			TEXT("ResExc"),
			TEXT("ResExcDedSys"),
			TEXT("ResExcShrSys"),
			TEXT("ResExcDedVid"),
			TEXT("ResExcShrVid"),
			TEXT("ResExcUnk"),
			TEXT("Count") 
		);

		UE::FItem Total;
		UE::FItem Culled;
		for( const auto& It : ObjectsByClass )
		{
			UClass* Class = It.Key;
			const UE::FItem& ClassObjects = It.Value;

			if( ClassObjects.Max < (SIZE_T)Limit )
			{
				Culled.Count += ClassObjects.Count;
				Culled.Num += ClassObjects.Num;
				Culled.Max += ClassObjects.Max;
				Culled.TrueResourceSize += ClassObjects.TrueResourceSize;
			}
			else
			{
				Ar.Logf( TEXT("%32s %12s %12s %12s %12s %12s %12s %12i"), 
					*Class->GetName(), 
					*FHierarchy::Size(ClassObjects.Max), 
					*FHierarchy::Size(ClassObjects.Num), 
					*FHierarchy::Size(ClassObjects.TrueResourceSize.GetTotalMemoryBytes()),
					*FHierarchy::Size(ClassObjects.TrueResourceSize.GetDedicatedSystemMemoryBytes()),
					*FHierarchy::Size(ClassObjects.TrueResourceSize.GetDedicatedVideoMemoryBytes()),
					*FHierarchy::Size(ClassObjects.TrueResourceSize.GetUnknownMemoryBytes()),
					ClassObjects.Count 
				);

			}

			Total.Count += ClassObjects.Count;
			Total.Num += ClassObjects.Num;
			Total.Max += ClassObjects.Max;
			Total.TrueResourceSize += ClassObjects.TrueResourceSize;
		}

		if( Culled.Count > 0 )
		{
			Ar.Logf( TEXT("") );
			Ar.Logf( TEXT("%32s %12s %12s %12s %12s %12s %12s %12i"), 
				TEXT("(Culled)"), 
				*FHierarchy::Size(Culled.Max), 
				*FHierarchy::Size(Culled.Num), 
				*FHierarchy::Size(Culled.TrueResourceSize.GetTotalMemoryBytes()),
				*FHierarchy::Size(Culled.TrueResourceSize.GetDedicatedSystemMemoryBytes()),
				*FHierarchy::Size(Culled.TrueResourceSize.GetDedicatedVideoMemoryBytes()),
				*FHierarchy::Size(Culled.TrueResourceSize.GetUnknownMemoryBytes()),
				Culled.Count 
			);
		}

		Ar.Logf( TEXT("") );
		Ar.Logf( TEXT("%32s %12s %12s %12s %12s %12s %12s %12i"), 
			TEXT("Total"), 
			*FHierarchy::Size(Total.Max), 
			*FHierarchy::Size(Total.Num), 
			*FHierarchy::Size(Total.TrueResourceSize.GetTotalMemoryBytes()),
			*FHierarchy::Size(Total.TrueResourceSize.GetDedicatedSystemMemoryBytes()),
			*FHierarchy::Size(Total.TrueResourceSize.GetDedicatedVideoMemoryBytes()),
			*FHierarchy::Size(Total.TrueResourceSize.GetUnknownMemoryBytes()),
			Total.Count 
		);
		Ar.Logf( TEXT("**********************************************") );
		return true;
	}
	else if (FParse::Command(&Cmd,TEXT("Mem")))
	{
		int32 Limit = 50;
		FParse::Value(Cmd, TEXT("CULL="), Limit);
		Limit *= 1024;

		UClass* ClassToCheck = NULL;
		ParseObject<UClass>(Cmd, TEXT("CLASS="  ), ClassToCheck, nullptr);

		if (ClassToCheck == NULL)
		{
			ClassToCheck = UObject::StaticClass();
		}
		else
		{
			// Class is set, so lower a bit the limit.
			Limit /= 10;
		}

		FHierarchy Classes(Limit);
		FHierarchy Outers(Limit);
		FHierarchy Flat(Limit);

		TMap<UObject*, UE::FSubItem> Objects;
		for( FThreadSafeObjectIterator It; It; ++It )
		{
			FArchiveCountMem Count( *It );
			FResourceSizeEx TrueResourceSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
			It->GetResourceSizeEx(TrueResourceSize);
			Objects.Add(*It, UE::FSubItem(*It, Count.GetNum(), Count.GetMax(), TrueResourceSize));
			Classes.AddClassInstance(*It);
			Outers.AddOuter(*It);
			Flat.AddFlat(*It);
		}

		UE_LOGF(LogEngine, Log, "********************************************** By Outer Hierarchy" );
		Outers.Compute(NULL, Objects, false);
		Outers.Log( ClassToCheck, UObject::StaticClass(), false );


		UE_LOGF(LogEngine, Log, "********************************************** By Class Hierarchy" );
		Classes.Compute(NULL, Objects, true);
		Classes.Log( ClassToCheck, UObject::StaticClass(), true );

		UE_LOGF(LogEngine, Log, "********************************************** Flat" );
		Flat.Compute(NULL, Objects, false);
		Flat.Log( NULL, ClassToCheck, false );
		UE_LOGF(LogEngine, Log, "**********************************************" );

		return true;
	}
	else if( FParse::Command(&Cmd,TEXT("LIST")) )
	{
		static TSet<FObjectKey> ForgottenObjects;

		// "obj list forget" will prevent all current objects from being reported in future "obj list" commands.
		// "obj list remember" clears that list
		if (FParse::Command(&Cmd, TEXT("FORGET")))
		{
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				ForgottenObjects.Add(FObjectKey(*It));
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("REMEMBER")))
		{
			ForgottenObjects.Empty();
			return true;
		}

		FSlowHeartBeatScope SuspendHeartBeat;
		FDisableHitchDetectorScope SuspendGameThreadHitch;

		FString CmdLineOut = FString::Printf(TEXT("Obj List: %s"), Cmd);
		Ar.Log( *CmdLineOut );
		Ar.Log( TEXT("Objects:") );
		Ar.Log( TEXT("") );

		UClass* CheckType = nullptr;
		UClass* MetaClass = nullptr;

		// allow checking for any Outer, not just a UPackage
		UObject* CheckOuter = nullptr;
		UPackage* InsidePackage = nullptr;
		UObject* InsideObject = nullptr;
		UClass* InsideClass = nullptr;
		ParseObject<UClass>(Cmd, TEXT("CLASS="  ), CheckType, nullptr);
		ParseObject<UObject>(Cmd, TEXT("OUTER="), CheckOuter, nullptr);

#if UE_WITH_REMOTE_OBJECT_HANDLE
		FRemoteServerId ServerId;
		{
			FString ServerIdText;
			if (FParse::Value(Cmd, TEXT("SERVERID="), ServerIdText))
			{
				ServerId = FRemoteServerId::FromString(ServerIdText);
			}
		}
#endif

		ParseObject<UPackage>(Cmd, TEXT("PACKAGE="), InsidePackage, nullptr);
		if (InsidePackage == nullptr)
		{
			ParseObject<UObject>(Cmd, TEXT("INSIDE="), InsideObject, nullptr);
		}
		if (InsidePackage == nullptr && InsideObject == nullptr)
		{
			ParseObject<UClass>(Cmd, TEXT("INSIDECLASS="), InsideClass, nullptr);
		}
		int32 Depth = -1;
		FParse::Value(Cmd, TEXT("DEPTH="), Depth);

		FString ObjectName;
		FParse::Value(Cmd, TEXT("NAME="), ObjectName);

		FString ObjectNameSubString;
		FParse::Value(Cmd, TEXT("NAMESUB="), ObjectNameSubString);

		TMap<UClass*, UE::FItem> ObjectsByClass;
		TArray<UE::FSubItem> Objects;
		UE::FItem Total;

		// support specifying metaclasses when listing class objects
		if ( CheckType && CheckType->IsChildOf(UClass::StaticClass()) )
		{
			FString ClassName;
			FParse::Value(Cmd, TEXT("TYPE="), ClassName);
			MetaClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("LIST command"));
		}

		const bool bAll				= FParse::Param( Cmd, TEXT("ALL") );
		// When listing objects with remote object handles enabled skip memory size calculation as the memory counting archive may accidentally attempt to resolve remote objects
		const bool bShowMemory = !FParse::Param(Cmd, TEXT("SKIPMEMORYSIZE")) && !UE_WITH_REMOTE_OBJECT_HANDLE;

		// if we specified a parameter in the command, but no objects of that parameter were found,
		// and they didn't specify "all", then don't list all objects
		if ( bAll ||
			((CheckType		||	!FCString::Strifind(Cmd,TEXT("CLASS=")))
				&&	(MetaClass		||	!FCString::Strifind(Cmd,TEXT("TYPE=")))
				&&	(CheckOuter		||	!FCString::Strifind(Cmd,TEXT("OUTER=")))
				&&	(InsidePackage	||	!FCString::Strifind(Cmd,TEXT("PACKAGE="))) 
				&&	(InsideObject	||	!FCString::Strifind(Cmd,TEXT("INSIDE=")))
				&&	(InsideClass || !FCString::Strifind(Cmd, TEXT("INSIDECLASS=")))))
		{
			const bool bTrackDetailedObjectInfo = bAll || (CheckType != NULL && CheckType != UObject::StaticClass()) || CheckOuter != nullptr || InsideObject != nullptr || InsidePackage != nullptr || InsideClass != nullptr || !ObjectName.IsEmpty() 
#if UE_WITH_REMOTE_OBJECT_HANDLE
				|| ServerId.IsValid()
#endif
				;
			const bool bOnlyListGCObjects = FParse::Param(Cmd, TEXT("GCONLY"));
			const bool bOnlyListGCObjectsNoClusters = FParse::Param(Cmd, TEXT("GCNOCLUSTERS"));
			const bool bOnlyListRootObjects = FParse::Param(Cmd, TEXT("ROOTONLY"));
			const bool bOnlyListRefCountedObjects = FParse::Param(Cmd, TEXT("REFCOUNTEDONLY"));
			const bool bShouldIncludeDefaultObjects = FParse::Param(Cmd, TEXT("INCLUDEDEFAULTS"));
			const bool bOnlyListDefaultObjects = FParse::Param(Cmd, TEXT("DEFAULTSONLY"));
			const bool bShowDetailedObjectInfo = FParse::Param(Cmd, TEXT("NODETAILEDINFO")) == false && bTrackDetailedObjectInfo;
			const bool bOnlyShowDisregardGC = FParse::Param(Cmd, TEXT("GCDISREGARDONLY"));
#if UE_WITH_REMOTE_OBJECT_HANDLE
			const bool bOnlyListBorrowedObjects = FParse::Param(Cmd, TEXT("BORROWED"));
#endif

			for( FThreadSafeObjectIterator It; It; ++It )
			{
				if (ForgottenObjects.Contains(FObjectKey(*It)))
				{
					continue;
				}
				if (It->IsTemplate(RF_ClassDefaultObject))
				{
					if( !bShouldIncludeDefaultObjects )
					{
						continue;
					}
				}
				else if( bOnlyListDefaultObjects )
				{
					continue;
				}

				if ( bOnlyShowDisregardGC && !GUObjectArray.IsDisregardForGC(*It))
				{
					continue;
				}

				if ( bOnlyListGCObjects && GUObjectArray.IsDisregardForGC(*It) )
				{
					continue;
				}

				if (bOnlyListGCObjectsNoClusters)
				{
					if (GUObjectArray.IsDisregardForGC(*It))
					{
						continue;
					}
					FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(*It);
					if (ObjectItem->GetOwnerIndex() > 0)
					{
						continue;
					}
				}

				if ( bOnlyListRootObjects && !It->IsRooted() )
				{
					continue;
				}

				if (bOnlyListRefCountedObjects && !It->GetRefCount())
				{
					continue;
				}

#if UE_WITH_REMOTE_OBJECT_HANDLE
				if (bOnlyListBorrowedObjects && !UE::RemoteObject::Handle::IsOwned(*It))
				{
					ensureAlwaysMsgf(It->HasAnyInternalFlags(EInternalObjectFlags::Borrowed), TEXT("Object %s is not owned by this server but it's not borrowed either"), *It->GetPathName());
					continue;
				}
#endif

				if ( CheckType && !It->IsA(CheckType) )
				{
					continue;
				}

				if ( CheckOuter && It->GetOuter() != CheckOuter )
				{
					continue;
				}

				if ( InsidePackage && !It->IsIn(InsidePackage) )
				{
					continue;
				}

				if ( InsideObject && !It->IsIn(InsideObject) )
				{
					continue;
				}

				if (InsideClass && !It->IsInA(InsideClass))
				{
					continue;
				}

				if (!ObjectName.IsEmpty() && It->GetName() != ObjectName)
				{
					continue;
				}

				if (!ObjectNameSubString.IsEmpty())
				{
					FString FullName = *It->GetFullName();
					if(!FullName.Contains(ObjectNameSubString, ESearchCase::IgnoreCase))
					{
						continue;
					}
				}

				if ( MetaClass )
				{
					UClass* ClassObj = Cast<UClass>(*It);
					if ( ClassObj && !ClassObj->IsChildOf(MetaClass) )
					{
						continue;
					}
				}

#if UE_WITH_REMOTE_OBJECT_HANDLE
				if (ServerId.IsValid())
				{
					FRemoteObjectId ObjectId(*It);
					if (ObjectId.GetServerId() != ServerId)
					{
						continue;
					}
				}
#endif

				SIZE_T NumBytes = 0;
				SIZE_T MaxBytes = 0;
				FResourceSizeEx TrueResourceSize = FResourceSizeEx(EResourceSizeMode::Exclusive);
				if (bShowMemory)
				{
					FArchiveCountMem Count(*It);
					NumBytes = Count.GetNum();
					MaxBytes = Count.GetMax();
					It->GetResourceSizeEx(TrueResourceSize);
				}

				// which class are we going to file this object under? by default, it's class
				UClass* ClassToUse = It->GetClass();
				// if we specified a depth to use, then put this object into the class Depth away from Object
				if (Depth != -1)
				{
					UClass* Travel = ClassToUse;
					// go up the class hierarchy chain, using a trail pointer Depth away
					for (int32 Up = 0; Up < Depth && Travel != UObject::StaticClass(); Up++)
					{
						Travel = Travel->GetSuperClass();
					}
					// when travel is a UObject, ClassToUse will be pointing to a class Depth away
					while (Travel != UObject::StaticClass())
					{
						Travel = Travel->GetSuperClass();
						ClassToUse = ClassToUse->GetSuperClass();
					}
				}

				UE::FItem& ClassData = ObjectsByClass.FindOrAdd(ClassToUse);
				ClassData.Class = ClassToUse;

				if( bShowDetailedObjectInfo )
				{
					Objects.Add(UE::FSubItem(*It, NumBytes, MaxBytes, TrueResourceSize));
				}
				ClassData.Add(NumBytes, MaxBytes, TrueResourceSize);
				Total.Add(NumBytes, MaxBytes, TrueResourceSize);
			}
		}

		const bool bAlphaSort = FParse::Param( Cmd, TEXT("ALPHASORT") );
		const bool bCountSort = FParse::Param( Cmd, TEXT("COUNTSORT") );
		const bool bResourceSizeSort = FParse::Param(Cmd, TEXT("RESOURCESIZESORT"));
		const bool bCSV = FParse::Param(Cmd, TEXT("CSV"));
		const bool bShowFullClassName = FParse::Param(Cmd, TEXT("FULLCLASSNAME"));
		const bool bVerboseObjectOutput = FParse::Param(Cmd, TEXT("VERBOSE"));

		if( Objects.Num() )
		{
			struct FCompareFSubItem
			{
				bool bAlphaSort, bResourceSizeSort;
				FCompareFSubItem( bool InAlphaSort, bool InResourceSizeSort)
					: bAlphaSort( InAlphaSort ),
					  bResourceSizeSort( InResourceSizeSort )
				{}

				FORCEINLINE bool operator()( const UE::FSubItem& A, const UE::FSubItem& B ) const
				{
					if (bAlphaSort)
					{
						return A.Object->GetPathName() < B.Object->GetPathName();
					}
					else if (bResourceSizeSort)
					{
						return (B.TrueResourceSize.GetTotalMemoryBytes() < A.TrueResourceSize.GetTotalMemoryBytes());
					}

					return B.Max < A.Max;
				}
			};
			Objects.Sort( FCompareFSubItem( bAlphaSort, bResourceSizeSort) );

			if (bCSV)
			{
				Ar.Logf(TEXT(",Object,NumKB,MaxKB,ResExcKB,ResExcDedSysKB,ResExcDedVidKB,ResExcUnkKB"));
			}
			else
			{
				Ar.Logf(
					TEXT("%140s")
#if UE_WITH_REMOTE_OBJECT_HANDLE
					TEXT("%23s")
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
					TEXT("%10s %10s %10s %15s %15s %15s"),
					TEXT("Object"),
#if UE_WITH_REMOTE_OBJECT_HANDLE
					TEXT("RemoteId"),
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
					TEXT("NumKB"),
					TEXT("MaxKB"),
					TEXT("ResExcKB"),
					TEXT("ResExcDedSysKB"),
					TEXT("ResExcDedVidKB"),
					TEXT("ResExcUnkKB")
				);
			}

			for (const UE::FSubItem& ObjItem : Objects)
			{
				if (bCSV)
				{
					Ar.Logf(
						TEXT(",%s,%.2f,%.2f,%.2f,%.2f,%.2f,%.2f"),
						*ObjItem.Object->GetFullName(),
						ObjItem.Num / 1024.0f,
						ObjItem.Max / 1024.0f,
						ObjItem.TrueResourceSize.GetTotalMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetDedicatedSystemMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetDedicatedVideoMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetUnknownMemoryBytes() / 1024.0f
					);

				}
				else
				{
					Ar.Logf(
						TEXT("%140s")
#if UE_WITH_REMOTE_OBJECT_HANDLE
						TEXT("%23s")
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
						TEXT("%10.2f %10.2f %10.2f %15.2f %15.2f %15.2f"),
						*ObjItem.Object->GetFullName(),
#if UE_WITH_REMOTE_OBJECT_HANDLE
						*FRemoteObjectId(ObjItem.Object).ToString(),
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
						ObjItem.Num / 1024.0f,
						ObjItem.Max / 1024.0f,
						ObjItem.TrueResourceSize.GetTotalMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetDedicatedSystemMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetDedicatedVideoMemoryBytes() / 1024.0f,
						ObjItem.TrueResourceSize.GetUnknownMemoryBytes() / 1024.0f
					);

					if (bVerboseObjectOutput)
					{
						ObjItem.TrueResourceSize.LogSummary(Ar);
					}
				}
			}
			Ar.Log(TEXT(""));
		}

		if (!ObjectsByClass.IsEmpty())
		{
			struct FCompareFItem
			{
				bool bAlphaSort, bCountSort, bResourceSizeSort;
				FCompareFItem( bool InAlphaSort, bool InCountSort, bool InResourceSizeSort )
					: bAlphaSort( InAlphaSort )
					, bCountSort( InCountSort )
					, bResourceSizeSort( InResourceSizeSort )
				{}
				FORCEINLINE bool operator()( const UE::FItem& A, const UE::FItem& B ) const
				{
					return bAlphaSort ? (A.Class->GetName() < B.Class->GetName()) : bCountSort ? (B.Count < A.Count) : bResourceSizeSort ? (B.TrueResourceSize.GetTotalMemoryBytes() < A.TrueResourceSize.GetTotalMemoryBytes()) : (B.Max < A.Max);
				}
			};
			TArray<UE::FItem> SortedClasses;
			ObjectsByClass.GenerateValueArray(SortedClasses);
			SortedClasses.Sort(FCompareFItem(bAlphaSort, bCountSort, bResourceSizeSort));
			
			if (bCSV)
			{
				if (bShowMemory)
				{
					Ar.Logf(TEXT(",Class,Count,NumKB,MaxKB,ResExcKB,ResExcDedSysKB,ResExcDedVidKB,ResExcUnkKB"));
				}
				else
				{
					Ar.Logf(TEXT(", Class, Count"));
				}
			}
			else
			{
				if (bShowMemory)
				{
					Ar.Logf(
						TEXT(" %100s %8s %10s %10s %10s %15s %15s %15s"),
						TEXT("Class"),
						TEXT("Count"),
						TEXT("NumKB"),
						TEXT("MaxKB"),
						TEXT("ResExcKB"),
						TEXT("ResExcDedSysKB"),
						TEXT("ResExcDedVidKB"),
						TEXT("ResExcUnkKB")
					);
				}
				else
				{
					Ar.Logf(TEXT(" %100s %8s"), TEXT("Class"), TEXT("Count"));
				}
			}

			for (const UE::FItem& ClassData : SortedClasses)
			{
				if (bCSV)
				{
					if (bShowMemory)
					{
						Ar.Logf(TEXT(",%s,%i,%f,%f,%f,%f,%f,%f"),
							bShowFullClassName ? *ClassData.Class->GetFullName() : *ClassData.Class->GetName(),
							(int32)ClassData.Count,
							ClassData.Num / 1024.0f,
							ClassData.Max / 1024.0f,
							ClassData.TrueResourceSize.GetTotalMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetDedicatedSystemMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetDedicatedVideoMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetUnknownMemoryBytes() / 1024.0f
						);
					}
					else
					{
						Ar.Logf(TEXT(", %s, %i"),
							bShowFullClassName ? *ClassData.Class->GetFullName() : *ClassData.Class->GetName(),
							(int32)ClassData.Count
						);
					}
				}
				else
				{
					if (bShowMemory)
					{
						Ar.Logf(TEXT(" %100s %8i %10.2f %10.2f %10.2f %15.2f %15.2f %15.2f"),
							bShowFullClassName ? *ClassData.Class->GetFullName() : *ClassData.Class->GetName(),
							(int32)ClassData.Count,
							ClassData.Num / 1024.0f,
							ClassData.Max / 1024.0f,
							ClassData.TrueResourceSize.GetTotalMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetDedicatedSystemMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetDedicatedVideoMemoryBytes() / 1024.0f,
							ClassData.TrueResourceSize.GetUnknownMemoryBytes() / 1024.0f
						);
					}
					else
					{
						Ar.Logf(TEXT(" %100s %8i"),
							bShowFullClassName ? *ClassData.Class->GetFullName() : *ClassData.Class->GetName(),
							(int32)ClassData.Count
						);
					}
				}
			}
			Ar.Log( TEXT("") );
		}
		if (bShowMemory)
		{
			Ar.Logf(
				TEXT("%i Objects (Total: %.3fM / Max: %.3fM / Res: %.3fM | ResDedSys: %.3fM / ResDedVid: %.3fM / ResUnknown: %.3fM)"),
				Total.Count,
				(float)Total.Num / 1024.0 / 1024.0,
				(float)Total.Max / 1024.0 / 1024.0,
				(float)Total.TrueResourceSize.GetTotalMemoryBytes() / 1024.0 / 1024.0,
				(float)Total.TrueResourceSize.GetDedicatedSystemMemoryBytes() / 1024.0 / 1024.0,
				(float)Total.TrueResourceSize.GetDedicatedVideoMemoryBytes() / 1024.0 / 1024.0,
				(float)Total.TrueResourceSize.GetUnknownMemoryBytes() / 1024.0 / 1024.0
			);
		}
		else
		{
			Ar.Logf(TEXT("%i Objects"), Total.Count);
		}
		return true;

	}
	else if (FParse::Command(&Cmd, TEXT("COUNT")))
	{
		FSlowHeartBeatScope SuspendHeartBeat;
		FDisableHitchDetectorScope SuspendGameThreadHitch;

		TMap<FString, int32> ClassMap;

		for( FThreadSafeObjectIterator It; It; ++It )
		{
			UClass* Class = It->GetClass();

			// Iterate through containers to find struct properties
			for (TFieldIterator<FProperty> ItProp(Class, EFieldIterationFlags::IncludeAll); ItProp; ++ItProp)
			{
				FProperty* Property = *ItProp;
				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					if (FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(*It));
						if (int32 Num = ArrayHelper.Num())
						{
							FString StructName;
							StructProperty->GetCPPMacroType(StructName);
							ClassMap.FindOrAdd(StructName, 0) += Num;
						}
					}
				}
				else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
				{
					if (FStructProperty* StructProperty = CastField<FStructProperty>(SetProperty->ElementProp))
					{
						FScriptSetHelper SetHelper(SetProperty, SetProperty->template ContainerPtrToValuePtr<void>(*It));
						if (int32 Num = SetHelper.Num())
						{
							FString StructName;
							StructProperty->GetCPPMacroType(StructName);
							ClassMap.FindOrAdd(StructName, 0) += Num;
						}
					}
				}
				else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
				{
					FScriptMapHelper MapHelper(MapProperty, MapProperty->template ContainerPtrToValuePtr<void>(*It));
					if (int32 Num = MapHelper.Num())
					{
						if (FStructProperty* StructProperty = CastField<FStructProperty>(MapProperty->KeyProp))
						{
							FString StructName;
							StructProperty->GetCPPMacroType(StructName);
							ClassMap.FindOrAdd(StructName, 0) += Num;
						}
						if (FStructProperty* StructProperty = CastField<FStructProperty>(MapProperty->ValueProp))
						{
							FString StructName;
							StructProperty->GetCPPMacroType(StructName);
							ClassMap.FindOrAdd(StructName, 0) += Num;
						}
					}
				}
			}

			while (Class && !Class->IsNative())
			{
				Class = Class->GetSuperClass();
			}

			FString ClassName = FString::Format(TEXT("{0}{1}"), { Class->GetPrefixCPP(), Class->GetName() });
			ClassMap.FindOrAdd(ClassName,0)++;
		}

		if( ClassMap.Num() )
		{
			FOutputDeviceFile CSVFile{ *FPaths::Combine(*FPaths::ProfilingDir(), TEXT("ObjCount.csv")) };
			CSVFile.SetSuppressEventTag(true);

			for (const auto& pair : ClassMap)
			{
				CSVFile.Logf(ELogVerbosity::Log, TEXT("%s,%i"), *pair.Key, pair.Value);
			}
		}
		return true;
	}
	
	else if (FParse::Command(&Cmd, TEXT("PROPANALYSIS")))
	{
		HandlePropAnalysisCommand(Cmd, Ar);
		return true;
	}
	else if ( FParse::Command(&Cmd,TEXT("COMPONENTS")) )
	{
		UObject* Obj=NULL;
		FString ObjectName;

		if ( FParse::Token(Cmd,ObjectName,true) )
		{
			Obj = FindFirstObject<UObject>(*ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("OBJ COMPONENTS"));

			if ( Obj != NULL )
			{
				Ar.Log(TEXT(""));
				DumpComponents(Obj);
				Ar.Log(TEXT(""));
			}
			else
			{
				Ar.Logf(TEXT("No objects found named '%s'"), *ObjectName);
			}
		}
		else
		{
			Ar.Logf(TEXT("Syntax: OBJ COMPONENTS <Name Of Object>"));
		}
		return true;
	}
	else if ( FParse::Command(&Cmd,TEXT("DUMP")) )
	{
		// Dump all variable values for the specified object
		// supports specifying categories to hide or show
		// OBJ DUMP playercontroller0 hide="actor,object,lighting,movement"     OR
		// OBJ DUMP playercontroller0 show="playercontroller,controller"        OR
		// OBJ DUMP class=playercontroller name=playercontroller0 show=object OR
		// OBJ DUMP playercontroller0 recurse=true
		TCHAR ObjectName[1024];
		UObject* Obj = NULL;
		UClass* Cls = NULL;

		TArray<FString> HiddenCategories, ShowingCategories;

		if ( !ParseObject<UClass>( Cmd, TEXT("CLASS="), Cls, nullptr) || !ParseObject(Cmd,TEXT("NAME="), Cls, Obj, nullptr) )
		{
			if ( FParse::Token(Cmd,ObjectName,UE_ARRAY_COUNT(ObjectName), 1) )
			{
				Obj = FindFirstObject<UObject>(ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("DUMP"));
			}
		}

		if ( Obj )
		{
			if ( Cast<UClass>(Obj) != NULL )
			{
				Obj = Cast<UClass>(Obj)->GetDefaultObject();
			}

			FString Value;

			Ar.Logf(TEXT(""));

			const bool bRecurse = FParse::Value(Cmd, TEXT("RECURSE=true"), Value);
			Ar.Logf(TEXT("*** Property dump for object %s'%s' ***"), bRecurse ? TEXT("(Recursive) ") : TEXT(""), *Obj->GetFullName() );

			if ( bRecurse )
			{
				const FExportObjectInnerContext Context;
				ExportProperties( &Context, Ar, Obj->GetClass(), (uint8*)Obj, 0, Obj->GetArchetype()->GetClass(), (uint8*)Obj->GetArchetype(), Obj, PPF_IncludeTransient );
			}
			else
			{
#if WITH_EDITORONLY_DATA
				//@todo: add support to FParse::Value() for specifying characters that should be ignored
				if ( FParse::Value(Cmd, TEXT("HIDE="), Value/*, TEXT(",")*/) )
				{
					Value.ParseIntoArray(HiddenCategories,TEXT(","),1);
				}
				else if ( FParse::Value(Cmd, TEXT("SHOW="), Value/*, TEXT(",")*/) )
				{
					Value.ParseIntoArray(ShowingCategories,TEXT(","),1);
				}
#endif
				UClass* LastOwnerClass = NULL;
				for ( TFieldIterator<FProperty> It(Obj->GetClass()); It; ++It )
				{
					UClass* Owner = It->GetOwnerClass();

					Value.Empty();
#if WITH_EDITOR
					if ( HiddenCategories.Num() )
					{
						const FString Category = FObjectEditorUtils::GetCategory(*It);
						int32 i;
						for ( i = 0; i < HiddenCategories.Num(); i++ )
						{
							if ( !Category.IsEmpty() && HiddenCategories[i] == Category )
							{
								break;
							}

							if ( HiddenCategories[i] == *Owner->GetName() )
							{
								break;
							}
						}

						if ( i < HiddenCategories.Num() )
						{
							continue;
						}
					}
					else if ( ShowingCategories.Num() )
					{
						const FString Category = FObjectEditorUtils::GetCategory(*It);
						int32 i;
						for ( i = 0; i < ShowingCategories.Num(); i++ )
						{
							if ( !Category.IsEmpty() && ShowingCategories[i] == Category )
							{
								break;
							}

							if ( ShowingCategories[i] == *Owner->GetName() )
							{
								break;
							}
						}

						if ( i == ShowingCategories.Num() )
						{
							continue;
						}
					}
#endif // #if WITH_EDITOR
					if ( LastOwnerClass != Owner )
					{
						LastOwnerClass = Owner;
						Ar.Logf(TEXT("=== %s properties ==="), *LastOwnerClass->GetName());
					}

					if ( It->ArrayDim > 1 )
					{
						for ( int32 i = 0; i < It->ArrayDim; i++ )
						{
							Value.Empty();
							It->ExportText_InContainer(i, Value, Obj, Obj, Obj, PPF_IncludeTransient);
							Ar.Logf(TEXT("  %s[%i]=%s"), *It->GetName(), i, *Value);
						}
					}
					else
					{
						FArrayProperty* ArrayProp = CastField<FArrayProperty>(*It);
						if ( ArrayProp != NULL )
						{
							FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, Obj);
							for( int32 i=0; i<FMath::Min(ArrayHelper.Num(),100); i++ )
							{
								Value.Empty();
								ArrayProp->Inner->ExportTextItem_Direct( Value, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), Obj, PPF_IncludeTransient );
								Ar.Logf(TEXT("  %s(%i)=%s"), *ArrayProp->GetName(), i, *Value);
							}

							if ( ArrayHelper.Num() >= 100 )
							{
								Ar.Logf(TEXT("  ... %i more elements"), ArrayHelper.Num() - 99);
							}
						}
						else
						{
							It->ExportText_InContainer(0, Value, Obj, Obj, Obj, PPF_IncludeTransient);
							Ar.Logf(TEXT("  %s=%s"), *It->GetName(), *Value);
						}
					}
				}
			}

			TMap<FString,FString> NativePropertyValues;
			if ( Obj->GetNativePropertyValues(NativePropertyValues) )
			{
				int32 LargestKey = 0;
				for ( TMap<FString,FString>::TIterator It(NativePropertyValues); It; ++It )
				{
					LargestKey = FMath::Max(LargestKey, It.Key().Len());
				}

				Ar.Log(TEXT("=== Native properties ==="));
				for ( TMap<FString,FString>::TIterator It(NativePropertyValues); It; ++It )
				{
					Ar.Logf(TEXT("  %s%s"), *It.Key().RightPad(LargestKey), *It.Value());
				}
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("No objects found using command '%s'"), Cmd));
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("HASH")))
	{
		const bool bShowHashBucketCollisionInfo = FParse::Param(Cmd, TEXT("SHOWBUCKETCOLLISIONS"));
		LogHashStatistics(Ar, bShowHashBucketCollisionInfo);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("HASHOUTER")))
	{
		const bool bShowHashBucketCollisionInfo = FParse::Param(Cmd, TEXT("SHOWBUCKETCOLLISIONS"));
		LogHashOuterStatistics(Ar, bShowHashBucketCollisionInfo);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("HASHFNAME")))
	{
		FName::DisplayHash(Ar);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("OVERHEAD")))
	{
		EObjectMemoryOverheadOptions Options = EObjectMemoryOverheadOptions::None;
		if (FParse::Param(Cmd, TEXT("DETAILED")))
		{
			Options |= EObjectMemoryOverheadOptions::ShowIndividualStats;
		}
		if (FParse::Param(Cmd, TEXT("WITHREFLECTION")))
		{
			Options |= EObjectMemoryOverheadOptions::IncludeReflectionData;
		}
		LogHashMemoryOverheadStatistics(Ar, Options);
		return true;
	}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	else if (FParse::Command(&Cmd, TEXT("REMOTE")))
	{
		using namespace UE::RemoteObject::Handle;

		FRemoteObjectId RemoteId;
		FString IdText;

		if (FParse::Value(Cmd, TEXT("ID="), IdText))
		{
			RemoteId = FRemoteObjectId::FromString(IdText);
		}

		if (RemoteId.IsValid())
		{
			Ar.Log(*RemoteId.ToString(ERemoteIdToStringVerbosity::Max));
		}
		else
		{
			Ar.Logf(TEXT("Unable to parse remote object id '%s'"), *IdText);
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("SERVERIDLIST")))
	{
		// Lists all existing objects with the specified server id (e.g. -ID=Asset). Can write the output to a predefined file inside of the project's Saved folder (-SAVETOFILE).
		FString ServerIdText;
		FRemoteServerId ServerId(ERemoteServerIdConstants::Local);
		if (FParse::Value(Cmd, TEXT("ID="), ServerIdText))
		{
			ServerId = FRemoteServerId::FromString(ServerIdText);
		}

		TArray<FRemoteObjectId> ObjectsIdList;
		if (ServerId.IsValid())
		{			
			for (TObjectIterator<UObject> It; It; ++It)
			{
				FRemoteObjectId ObjectId(*It);
				if (ObjectId.GetServerId() == ServerId)
				{
					ObjectsIdList.Add(ObjectId);
				}
			}
			ObjectsIdList.Sort();
		}
		else
		{
			Ar.Logf(TEXT("Unable to parse server id '%s'"), *ServerIdText);
			return true;
		}

		if (FParse::Param(Cmd, TEXT("SAVETOFILE")))
		{
			uint32 LocalServerIdNumber = FRemoteServerId::GetLocalServerId().GetIdNumber();;
			const FString Filename = FPaths::Combine(FPaths::ProjectSavedDir(), FString::Printf(TEXT("ServerIdList-%s_S%u.txt"), *ServerId.ToString(), LocalServerIdNumber));
			TArray<FString> Lines;

			for (FRemoteObjectId ObjectId : ObjectsIdList)
			{
				Lines.Add(ObjectId.ToString(ERemoteIdToStringVerbosity::FullName));
			}
			FFileHelper::SaveStringArrayToFile(Lines, *Filename);
		}
		else
		{
			for (FRemoteObjectId ObjectId : ObjectsIdList)
			{
				Ar.Log(*ObjectId.ToString(ERemoteIdToStringVerbosity::FullName));
			}
		}
		return true;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
#endif // !UE_BUILD_SHIPPING
	else
	{
		// OBJ command but not supported here
		return false;
	}
}

bool UEngine::HandleDirCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TArray<FString> Files;
	TArray<FString> Directories;

	IFileManager::Get().FindFiles( Files, Cmd, 1, 0 );
	IFileManager::Get().FindFiles( Directories, Cmd, 0, 1 );

	// Directories
	Directories.Sort();
	for( int32 x = 0 ; x < Directories.Num() ; x++ )
	{
		Ar.Logf( TEXT("[%s]"), *Directories[x] );
	}

	// Files
	Files.Sort();
	for( int32 x = 0 ; x < Files.Num() ; x++ )
	{
		Ar.Logf( TEXT("[%s]"), *Files[x] );
	}

	return true;
}

bool UEngine::HandleTrackParticleRenderingStatsCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	extern float GTimeBetweenParticleRenderStatCaptures;
	extern float GMinParticleDrawTimeToTrack;
	FString FlagStr(FParse::Token(Cmd, 0));
	if (FlagStr.Len() > 0)
	{
		GTimeBetweenParticleRenderStatCaptures = FCString::Atof(*FlagStr);
	}

	FString FlagStr2(FParse::Token(Cmd, 0));
	if (FlagStr2.Len() > 0)
	{
		GMinParticleDrawTimeToTrack = FCString::Atof(*FlagStr2);
	}

	extern bool GTrackParticleRenderingStats;
	GTrackParticleRenderingStats = !GTrackParticleRenderingStats;
	if (GTrackParticleRenderingStats)
	{
		if (GetCachedScalabilityCVars().DetailMode >= DM_High)
		{
			Ar.Logf(TEXT("Currently in at least high detail mode, note that particle stats will only be captured in medium or low detail modes (eg splitscreen)."));
		}
		Ar.Logf(TEXT("Enabled particle render stat tracking with %.1fs between captures, min tracked time of %.4fs, use DUMPPARTICLERENDERINGSTATS to save results."),
			GTimeBetweenParticleRenderStatCaptures, GMinParticleDrawTimeToTrack);
	}
	else
	{
		Ar.Logf(TEXT("Disabled particle render stat tracking."));
	}
	return 1;
}

bool UEngine::HandleDumpAllocatorStats( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::Private::GMalloc->DumpAllocatorStats(Ar);
	return true;
}

bool UEngine::HandleHeapCheckCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	UE::Private::GMalloc->ValidateHeap();
	return true;
}

bool UEngine::HandleToggleOnscreenDebugMessageDisplayCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GEngine->bEnableOnScreenDebugMessagesDisplay = !GEngine->bEnableOnScreenDebugMessagesDisplay;
	UE_LOGF(LogEngine, Log, "OnScreenDebug Message Display is now %ls", 
		GEngine->bEnableOnScreenDebugMessagesDisplay ? TEXT("ENABLED") : TEXT("DISABLED"));
	if ((GEngine->bEnableOnScreenDebugMessagesDisplay == true) && (GEngine->bEnableOnScreenDebugMessages == false))
	{
		UE_LOGF(LogEngine, Log, "OnScreenDebug Message system is DISABLED!");
	}
	return true;
}

bool UEngine::HandleToggleOnscreenDebugMessageSystemCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GEngine->bEnableOnScreenDebugMessages = !GEngine->bEnableOnScreenDebugMessages;
	UE_LOGF(LogEngine, Log, "OnScreenDebug Message System is now %ls", 
		GEngine->bEnableOnScreenDebugMessages ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEngine::HandleDisableAllScreenMessagesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GAreScreenMessagesEnabled = false;
	UE_LOGF(LogEngine, Log, "Onscreen warnings/messages are now DISABLED");
	return true;
}


bool UEngine::HandleEnableAllScreenMessagesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GAreScreenMessagesEnabled = true;
	UE_LOGF(LogEngine, Log, "Onscreen warnings/messages are now ENABLED");
	return true;
}


bool UEngine::HandleToggleAllScreenMessagesCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GAreScreenMessagesEnabled = !GAreScreenMessagesEnabled;
	UE_LOGF(LogEngine, Log, "Onscreen warnings/messages are now %ls",
		GAreScreenMessagesEnabled ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEngine::HandleTestslateGameUICommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	TSharedRef<SWidget> GameUI = 
		SNew( SHorizontalBox )
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( 5.0f )
		.HAlign( HAlign_Left )
		.VAlign( VAlign_Top )
		[
			SNew( SButton )
			.Text( NSLOCTEXT("UnrealEd", "TestSlateGameUIButtonText", "Test Button!") )
		]
	+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		.Padding( 5.0f )
		.FillWidth(0.66f)
		[
			SNew(SThrobber)
		];

	GEngine->GameViewport->AddViewportWidgetContent( GameUI );
	return true;
}

bool UEngine::HandleConfigHashCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FString ConfigFilename;
	if ( FParse::Token(Cmd, ConfigFilename, true) )
	{
		if ( ConfigFilename == TEXT("NAMESONLY") )
		{
			Ar.Log( TEXT("Files map:") );
			for ( const FString& Filename : GConfig->GetFilenames() )
			{
				// base filename is what Dump() compares against
				Ar.Logf(TEXT("FileName: %s (%s)"), *FPaths::GetBaseFilename(Filename), *Filename);
			}
		}
		else
		{
			Ar.Logf(TEXT("Attempting to dump data for config file: %s"), *ConfigFilename);
			GConfig->Dump(Ar, *ConfigFilename);
		}
	}
	else
	{
		GConfig->Dump( Ar );
	}
	return true;
}

bool UEngine::HandleConfigMemCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	GConfig->ShowMemoryUsage(Ar);
	return true;
}

bool UEngine::HandleGetIniCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Format: GetIni [Platform@]IniFile:Section Key
	TCHAR IniPlusSectionName[256];
	TCHAR KeyName[256];

	if (FParse::Token(Cmd, IniPlusSectionName, UE_ARRAY_COUNT(IniPlusSectionName), true))
	{
		FString IniPlusSection = IniPlusSectionName;

		// look for the optional Platform prefix which means to look up another platform's version of this ini setting
		int32 PlatformDelim = IniPlusSection.Find(TEXT("@"));
		FConfigCacheIni* ConfigSystem = GConfig;
		if (PlatformDelim > 0)
		{
			FString PlatformName = IniPlusSection.Mid(0, PlatformDelim);
			IniPlusSection = IniPlusSection.Mid(PlatformDelim + 1);

			ConfigSystem = FConfigCacheIni::ForPlatform(*PlatformName);
			if (ConfigSystem == nullptr)
			{
				Ar.Logf(TEXT("Failed to find config system for platform %s"), *PlatformName);
				return false;
			}
		}

		int32 IniDelim = IniPlusSection.Find(TEXT(":"));
		FString IniName;
		FString SectionName = (IniDelim != INDEX_NONE ? IniPlusSection.Mid(IniDelim+1) : IniPlusSection);
		SectionName.TrimQuotesInline();

		if (IniDelim != INDEX_NONE)
		{
			// Check for hardcoded engine-ini:, game-ini: etc. parsing, and if not found fallback to raw string
			const FString* HardcodedIni = GetIniFilenameFromObjectsReference(IniPlusSection);

			if (HardcodedIni != NULL)
			{
				IniName = *HardcodedIni;
			}
			else
			{
				TArray<FString> ConfigList;
				ConfigSystem->GetConfigFilenames(ConfigList);

				// @todo this could probably simplified to just FindBranch passing in the part before the : to both params of FindBranch
				
				// first try exact match (this helps with known files that have no .ini extension, etc)
				FString SearchString = IniPlusSection.Left(IniDelim);
				const FString* Result = Algo::FindByPredicate(ConfigList, [&SearchString](const FString& Test) { return FPaths::GetCleanFilename(Test) == SearchString; });
				
				// now try with .ini added if not already a .ini
				if (!SearchString.EndsWith(TEXT(".ini")))
				{
					SearchString += ".ini";
					if (Result == nullptr)
					{
						Result = Algo::FindByPredicate(ConfigList, [&SearchString](const FString& Test) { return FPaths::GetCleanFilename(Test) == SearchString; });
					}
				}

				// finally fallback to old method of substring search with .ini extension
				if (Result == nullptr)
				{
					Result = Algo::FindByPredicate(ConfigList, [&SearchString](const FString& Test) { return Test.Contains(SearchString); });
				}

				if (Result == nullptr)
				{
					UE_SUPPRESS(LogExec, Warning,
						Ar.Logf(TEXT("Failed to find IniFile '%s' (note: can only search loaded ini files)."), *SearchString));
				}
				else
				{
					IniName = *Result;
				}
			}
		}
		else
		{
			IniName = GEngineIni;
		}

		const TCHAR* ValueTypes[] =
		{
			TEXT(" SET"),
			TEXT(" ADD"),
			TEXT("UNIQ"),
			TEXT("REMV"),
			TEXT("EMPT"),
			TEXT(" CLR"),
			TEXT(" AOS"),
			TEXT(" POC"),
			TEXT("COMB"),
			TEXT("COMB"),
		};
		
		if (!IniName.IsEmpty() && !SectionName.IsEmpty())
		{
			if (FParse::Token(Cmd, KeyName, UE_ARRAY_COUNT(KeyName), true))
			{
				FConfigBranch* Branch = ConfigSystem->FindBranch(*IniName, *IniName);
				if (Branch != nullptr)
				{
					// reusable logger for various layers in the Branch
					auto LogLayer = [&Ar, KeyName, &SectionName, ValueTypes](const FConfigFile& File, const FString& Filename, const TCHAR* Desc)
					{
						bool bLogged = false;
						const FConfigSection* Section = File.FindSection(*SectionName);
						if (Section != nullptr)
						{
							TArray<FConfigValue> Values;
							Section->MultiFind(KeyName, Values, true);
							for (const FConfigValue& Value : Values)
							{
								if (!bLogged)
								{
									Ar.Logf(TEXT("%s: %s"), Desc, *Filename);
									bLogged = true;
								}
								Ar.Logf(TEXT("  [%s] %s"), ValueTypes[(int)Value.ValueType], *Value.GetSavedValue());
							}
						}
						return bLogged;
					};
					auto LogLayerStream = [&Ar, KeyName, &SectionName, ValueTypes](const FConfigCommandStream& File, const FString& Filename, const TCHAR* Desc)
					{
						bool bLogged = false;
						const FConfigCommandStreamSection* Section = File.Find(*SectionName);
						if (Section != nullptr)
						{
							TArray<FConfigValue> Values;
							Section->MultiFind(KeyName, Values, true);
							for (const FConfigValue& Value : Values)
							{
								if (!bLogged)
								{
									Ar.Logf(TEXT("%s: %s"), Desc, *Filename);
									bLogged = true;
								}
								Ar.Logf(TEXT("  [%s] %s"), ValueTypes[(int)Value.ValueType], *Value.GetSavedValue());
							}
						}
						return bLogged;
					};

					for (auto Pair : Branch->StaticLayers)
					{
						LogLayerStream(Pair.Value, Pair.Key, TEXT("Static File"));
					}
					
					bool bHadDynamicValue = false;
					for (TDoubleLinkedList<FConfigCommandStream*>::TIterator Node(Branch->DynamicLayers.GetHead()); Node; ++Node)
					{
						bHadDynamicValue |= LogLayerStream(**Node, Node->Filename, TEXT("Dynamic File"));
					}

					// no need to display some layers if there are no dynamic layers at all
					if (bHadDynamicValue || Branch->StaticLayers.Num() == 0)
					{
						LogLayer(Branch->CombinedStaticLayers, "", TEXT("Static Combined Value(s)"));
					}
					if (bHadDynamicValue)
					{
						LogLayer(Branch->FinalCombinedLayers, "", TEXT("Static+Dynamic Combined Value(s)"));
					}

					LogLayerStream(Branch->CommandLineOverrides, "", TEXT("CommandLine Overrides:"));
					LogLayer(Branch->InMemoryFile, "", TEXT("InMemory Layer Value(s)"));

					// now log the actual final result
					{
						TArray<FString> FinalValues;
						ConfigSystem->GetArray(*SectionName, KeyName, FinalValues, IniName);
						
						Ar.Logf(TEXT(" "));
						Ar.Logf(TEXT("Final Value(s) [should match InMemory above!]"));
						Ar.Logf(TEXT("[%s]"), *SectionName);
						
						if (FinalValues.Num() == 1)
						{
							Ar.Logf(TEXT("%s = %s"), KeyName, *FinalValues[0]);
						}
						else
						{
							for (const FString& CurValue : FinalValues)
							{
								Ar.Logf(TEXT("  %s"), *CurValue);
							}
						}
					}
				}
				else
				{
					UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Failed to get config key '%s', from section '%s', in ini file '%s'."),
						KeyName, *SectionName, *IniName));
				}
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning,
					Ar.Logf(TEXT("No Key specified. Command format: GetIni [Platform@]IniFile:Section Key")));
			}
		}
		else if (IniName.IsEmpty())
		{
			UE_SUPPRESS(LogExec, Warning,
				Ar.Logf(TEXT("IniFile parsing failed (%s). Command format: GetIni [Platform@]IniFile:Section Key"),
					IniPlusSectionName));
		}
		else // if (SectionName.IsEmpty())
		{
			UE_SUPPRESS(LogExec, Warning,
				Ar.Logf(TEXT("Section parsing failed (%s). Command format: GetIni [Platform@]IniFile:Section Key"),
					IniPlusSectionName));
		}
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning,
			Ar.Logf(TEXT("No Section specified. Command format: GetIni [Platform@]IniFile:Section Key")))
	}

	return true;
}

bool UEngine::HandleRedirectOutputCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
#if ALLOW_DEBUG_FILES
	FString FilePath = FParse::Token(Cmd, true);
	FPaths::NormalizeFilename(FilePath);
	if (!FPaths::CollapseRelativeDirectories(FilePath))
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: can't redirect outside of Saved folder: %s"), *FilePath);
		return false;
	}
	FPaths::RemoveDuplicateSlashes(FilePath);

	FText ErrorReason;
	if (!FPaths::ValidatePath(FilePath, &ErrorReason))
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: %s"), *ErrorReason.ToString());
		return false;
	}

	// Skip preceeding spaces and tabs.
	while (FChar::IsWhitespace(*Cmd))
	{
		Cmd++;
	}

	if (*Cmd == 0)
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("Redirect skipped: the command is empty"));
		return false;
	}

	static const TCHAR DeferCommandName[] = TEXT("DEFER");
	static const size_t DeferCommandSize = UE_ARRAY_COUNT(DeferCommandName) - 1;

	if (FCString::Strnicmp(Cmd, DeferCommandName, DeferCommandSize) == 0)
	{
		if (FChar::IsWhitespace(*(Cmd + DeferCommandSize)))
		{
			Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: can't redirect deferred command"));
			return false;
		}
	}

	FilePath = FPaths::ProjectSavedDir() + FilePath;

	const FString OutputDir = FPaths::GetPath(FilePath);
	if (!IFileManager::Get().MakeDirectory(*OutputDir, true))
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: can't create path %s"), *OutputDir);
		return false;
	}

	TUniquePtr<FArchive> CommandFile(IFileManager::Get().CreateDebugFileWriter(*FilePath));
	if (!CommandFile)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: Can't create file writer. See logs."));
		return false;
	}

	FOutputDeviceArchiveWrapper CommandFileAr(CommandFile.Get());

	FOutputDeviceRedirector Redirector;
	Redirector.AddOutputDevice(&CommandFileAr);
	Redirector.AddOutputDevice(GLog);

	bool bHandled = false;
	if (ULocalPlayer* Player = GetDebugLocalPlayer())
	{
		bHandled = Player->Exec(InWorld, Cmd, Redirector);
	}
	else
	{
		bHandled = Exec(InWorld, Cmd, Redirector);
	}

	Redirector.Flush();

	// Propagate status of inner command to upstream code.
	return bHandled;
#else
	Ar.Logf(ELogVerbosity::Error, TEXT("Redirect failed: Writing to files is not available."));
	return false;
#endif
}

#endif // !UE_BUILD_SHIPPING

/** Helper function to cause a stack overflow crash */
FORCENOINLINE void StackOverflowFunction(int32* DummyArg)
{
	int32 StackArray[8196];
	FMemory::Memset(StackArray, 0, sizeof(StackArray));
	if (StackArray[0] == 0)
	{
		UE_LOGF(LogEngine, VeryVerbose, "StackOverflowFunction(%d)", DummyArg ? DummyArg[0] : 0);
		StackOverflowFunction(StackArray);
	}
}

// This function is used to add dummy work at the end of "debug thread.."
// console commands' implementations. The intention is to better condition
// return addresses. Return addresses can often be the address of the next
// statement after a function call, especially if the call isn't expected to
// return a value or the value's discarded. For calls at the end of a function
// the return addresses can often resolve to unexpected places (due to the
// compiler inlining for example). This can cause confusing callstacks that are
// difficult to test and observe. Adding a call to this after keeps the failing
// address more to more appropriate values.
FORCENOINLINE static void DebugDummyWorkAfterFailure()
{
	UE_LOGF(LogEngine, Log, "A dummy log message");
}

#if !UE_BUILD_SHIPPING

class FDebugMemoryEater
{
public:
	static FDebugMemoryEater& Get()
	{
		static FDebugMemoryEater Instance;

		return Instance;
	}

	bool Start(const TCHAR* Cmd, FOutputDevice& Ar)
	{
		Target       = 0;
		LimitPerTick = 10 * 1024 * 1024;

		if (!ParseSizeParam(Cmd, TEXT("Target="), Target))
		{
			Ar.Logf(TEXT("Failed to parse Target parameter!"));

			return false;
		}

		if (!ParseSizeParam(Cmd, TEXT("TickMax="), LimitPerTick))
		{
			Ar.Logf(TEXT("Failed to parse TickMax parameter!"));

			return false;
		}

		Ar.Logf(TEXT("Target Free Memory: %llu, Max Allocation Per Tick: %llu"), Target, LimitPerTick);

		StartTicking();

		return true;
	}

	void Stop()
	{
		StopTicking();
	}

	void Purge()
	{
		while (void* MemoryBlock = PopMemoryBlock())
		{
			FMemory::Free(MemoryBlock);
		}
	}

	bool Tick(float DeltaSeconds)
	{
		constexpr uint64 MemoryBlockSize = 64 * 1024;

		uint64 AllocatedThisTick = 0;

		FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

		while (Stats.AvailablePhysical > Target && AllocatedThisTick < LimitPerTick)
		{
			void* Ptr = FMemory::Malloc(MemoryBlockSize);

			if (Ptr)
			{
				AllocatedThisTick += MemoryBlockSize;

				PushMemoryBlock(Ptr);
			}
			else
			{
				break;
			}

			// We could just subtract the amount allocated from the amount of free memory queried before the loop but let's be precise and query again.
			Stats = FPlatformMemory::GetStats();
		}

		return (Stats.AvailablePhysical > Target);
	}

private:
	bool ParseSizeParam(const TCHAR* Cmd, const TCHAR* Match, uint64& Value)
	{
		FString SizeString;
		uint64  SizeMultiplier = 1;

		if (!FParse::Value(Cmd, Match, SizeString))
		{
			return true;
		}

		static constexpr TCHAR const* SuffixKB  = TEXT("kb");
		static constexpr TCHAR const* SuffixMB  = TEXT("mb");
		static constexpr TCHAR const* SuffixGB  = TEXT("gb");
		static constexpr uint32       SuffixLen = 2;

		if (SizeString.EndsWith(SuffixKB, SuffixLen))
		{
			SizeMultiplier = 1024;

			SizeString = SizeString.LeftChop(SuffixLen);
		}
		else if (SizeString.EndsWith(SuffixMB, SuffixLen))
		{
			SizeMultiplier = 1024 * 1024;

			SizeString = SizeString.LeftChop(SuffixLen);
		}
		else if (SizeString.EndsWith(SuffixGB, SuffixLen))
		{
			SizeMultiplier = 1024 * 1024 * 1024;

			SizeString = SizeString.LeftChop(SuffixLen);
		}

		if (SizeString.IsEmpty())
		{
			return false;
		}

		for (int32 i = 0; i < SizeString.Len(); ++i)
		{
			if (!FChar::IsDigit(SizeString[i]))
			{
				return false;
			}
		}

		Value = FPlatformString::Atoi64(*SizeString) * SizeMultiplier;

		return true;
	}

	void StartTicking()
	{
		if (TickHandle.IsValid())
		{
			StopTicking();
		}

		FTSTicker& Ticker = FTSTicker::GetCoreTicker();

		TickHandle = Ticker.AddTicker(FTickerDelegate::CreateRaw(this, &FDebugMemoryEater::Tick));
	}

	void StopTicking()
	{
		if (TickHandle.IsValid())
		{
			FTSTicker& Ticker = FTSTicker::GetCoreTicker();

			Ticker.RemoveTicker(TickHandle);

			TickHandle.Reset();
		}
	}

	void PushMemoryBlock(void* Ptr)
	{
		BlockHeader* MemoryBlock = (BlockHeader*)Ptr;

		MemoryBlock->Next = MemoryListHead;
		MemoryListHead    = MemoryBlock;
	}

	void* PopMemoryBlock()
	{
		if (!MemoryListHead)
		{
			return nullptr;
		}

		BlockHeader* PrevHead = MemoryListHead;

		MemoryListHead = MemoryListHead->Next;

		return PrevHead;
	}

	FTSTicker::FDelegateHandle TickHandle;

	uint64 Target;
	uint64 LimitPerTick;

	struct BlockHeader
	{
		BlockHeader* Next;
	};

	BlockHeader* MemoryListHead = nullptr;
};

#endif

extern CORE_API bool GIsGPUCrashed;

#if !UE_BUILD_SHIPPING
PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
bool PerformBlockingError(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("TERMINATE")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		UE_LOGF(LogEngine, Warning, "%ls", TEXT("Terminating from the gamethread at your request"));
		std::terminate();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("ABORT")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		UE_LOGF(LogEngine, Warning, "%ls", TEXT("Aborting from the gamethread at your request"));
		std::abort();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("SOFTLOCK")))
	{
		Ar.Log(TEXT("Hanging the current thread"));
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		while (1)
		{
			FPlatformProcess::Sleep(1.0f);
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("INFINITELOOP")))
	{
		Ar.Log(TEXT("Hanging the current thread (CPU-intensive)"));
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		for (;;)
		{
		}
		return true;
	}
	return false;
}
PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS
#endif // !UE_BUILD_SHIPPING

bool UEngine::PerformError(const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("RENDERCRASH")))
	{
		struct FRender
		{
			static void Crash(FRHICommandList&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				UE_LOGF(LogEngine, Fatal, "Crashing the renderthread at your request");
				DebugDummyWorkAfterFailure();
			}
		};

		ENQUEUE_RENDER_COMMAND(CauseRenderThreadCrash)(&FRender::Crash);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("RENDERCHECK")))
	{
		struct FRender
		{
			static void Check(FRHICommandList&)
			{				
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				check(!"Crashing the renderthread via check(0) at your request");
				DebugDummyWorkAfterFailure();
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadCrash)(&FRender::Check);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("RENDERGPF")))
	{
		struct FRender
		{
			static void GPF(FRHICommandList&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				UE_FORCE_CRASH();
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadCrash)(&FRender::GPF);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("RHICRASH")))
	{
		struct FCrash
		{
			static void RHIThreadCrash(FRHICommandList& CmdList)
			{
				// Test breadcrumb system
				RHI_BREADCRUMB_EVENT(CmdList, "Debug RHI Thread Crash");

				CmdList.EnqueueLambda([](FRHICommandListBase& ExecutingCmdList)
				{
					UE_LOGF(LogEngine, Warning, "Printed warning to log.");
					FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
					UE_LOGF(LogEngine, Fatal, "Crashing the RHI thread at your request.");
				});
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseRHIThreadCrash)(&FCrash::RHIThreadCrash);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("RENDERFATAL")))
	{
		struct FRender
		{
			static void Fatal(FRHICommandList&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				LowLevelFatalError(TEXT("FError::LowLevelFatal test"));
				DebugDummyWorkAfterFailure();
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadCrash)(&FRender::Fatal);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("RENDERENSURE")))
	{
		struct FRender
		{
			static void Ensure(FRHICommandList&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
			if (!ensure(0))
			{
				UE_LOGF(LogEngine, Warning, "Ensure condition failed (this is the expected behavior).");
			}
				DebugDummyWorkAfterFailure();
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadEnsure)(&FRender::Ensure);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("THREADCRASH")))
	{
		struct FThread
		{
			static void Crash(ENamedThreads::Type, const  FGraphEventRef&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				UE_LOGF(LogEngine, Fatal, "Crashing the worker thread at your request");
				DebugDummyWorkAfterFailure();
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FDelegateGraphTask.FThread::Crash"),
		STAT_FDelegateGraphTask_FThread__Crash,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::Crash),
				GET_STATID(STAT_FDelegateGraphTask_FThread__Crash)
			),
			ENamedThreads::GameThread
		);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("THREADCHECK")))
	{
		struct FThread
		{
			static void Check(ENamedThreads::Type, const FGraphEventRef&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				check(!"Crashing a worker thread via check(0) at your request");
				DebugDummyWorkAfterFailure();
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FDelegateGraphTask.FThread::Check"),
		STAT_FDelegateGraphTask_FThread__Check,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::Check),
				GET_STATID(STAT_FDelegateGraphTask_FThread__Check)
			),
			ENamedThreads::GameThread
		);
		return true;
	}
	if (FParse::Command(&Cmd, TEXT("THREADGPF")))
	{
		struct FThread
		{
			static void GPF(ENamedThreads::Type, const FGraphEventRef&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				UE_FORCE_CRASH();
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FDelegateGraphTask.FThread::GPF"),
		STAT_FDelegateGraphTask_FThread__GPF,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::GPF),
				GET_STATID(STAT_FDelegateGraphTask_FThread__GPF)
			),
			ENamedThreads::GameThread
		);
		return true;
	}

	if (FParse::Command(&Cmd, TEXT("THREADEDCHECKS")))
	{
		struct FThreaded
		{
			static void Checks()
				{
					FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);

					// We now test if assert behavior is deterministic when called from different threads or
					// if one thread could end up with a different behavior due to a race condition.
					// We expect that the result will always be the assert being reported and not end-up
					// with an Int 3 being handled by the exception handler.
					const bool CrashingTheWorkerThreadAtYourRequest = false;
					check(CrashingTheWorkerThreadAtYourRequest);
				DebugDummyWorkAfterFailure();
				}
		};

		UE_LOGF(LogEngine, Warning, "Queuing several multi-thread checks.");
		for (int32 Index = 0; Index < 16; ++Index)
		{
			Async(EAsyncExecution::ThreadPool, &FThreaded::Checks);
		};

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("TWOTHREADSCRASH")))
	{
		class FThreadPoolCrash : public IQueuedWork
		{
		private:
			double CrashDelay;
		public:
			FThreadPoolCrash(double InCrashDelay)
				: CrashDelay(InCrashDelay)
			{
			}
			void Abandon()
			{
			}
			void DoThreadedWork()
			{
				double CrashTime = FPlatformTime::Seconds() + CrashDelay;
				do 
				{
					if (FPlatformTime::Seconds() >= CrashTime)
					{
						UE_LOGF(LogEngine, Warning, "Printed warning to log.");
						FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
						UE_LOGF(LogEngine, Fatal, "Crashing the worker thread at your request");
						break;
					}
					else
					{
						FPlatformProcess::Sleep(0);
					}
				} while (true);
				DebugDummyWorkAfterFailure();
			}
		};

		UE_LOGF(LogEngine, Warning, "Queuing two tasks to crash.");
		GThreadPool->AddQueuedWork(new FThreadPoolCrash(0.100));
		GThreadPool->AddQueuedWork(new FThreadPoolCrash(0.110));

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("TWOTHREADSGPF")))
	{
		class FThreadPoolCrash : public IQueuedWork
		{
		private:
			double CrashDelay;
		public:
			FThreadPoolCrash(double InCrashDelay)
				: CrashDelay(InCrashDelay)
			{
			}
			void Abandon()
			{
			}
			void DoThreadedWork()
			{
				double CrashTime = FPlatformTime::Seconds() + CrashDelay;
				do
				{
					if (FPlatformTime::Seconds() >= CrashTime)
					{
						UE_LOGF(LogEngine, Warning, "Printed warning to log.");
						FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
						UE_FORCE_CRASH();
						break;
					}
					else
					{
						FPlatformProcess::Sleep(0);
					}
				} while (true);
				DebugDummyWorkAfterFailure();
			}
		};

		UE_LOGF(LogEngine, Warning, "Queuing two tasks to crash.");
		GThreadPool->AddQueuedWork(new FThreadPoolCrash(0.100));
		GThreadPool->AddQueuedWork(new FThreadPoolCrash(0.110));

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("THREADENSURE")))
	{
		struct FThread
		{
			static void Ensure(ENamedThreads::Type, const FGraphEventRef&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				ensure(0);
				DebugDummyWorkAfterFailure();
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FThread::Ensure"),
		STAT_FThread__Ensure,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::Ensure),
				GET_STATID(STAT_FThread__Ensure)),
			ENamedThreads::GameThread
		);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("THREADFATAL")))
	{
		struct FThread
		{
			static void Fatal(ENamedThreads::Type, const FGraphEventRef&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				LowLevelFatalError(TEXT("FError::LowLevelFatal test"));
				DebugDummyWorkAfterFailure();
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FThread::Fatal"),
		STAT_FThread__Fatal,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::Fatal),
				GET_STATID(STAT_FThread__Fatal)),
			ENamedThreads::GameThread
		);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CRASH")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		UE_LOGF(LogEngine, Fatal, "%ls", TEXT("Crashing the gamethread at your request"));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GPUCRASH")))
	{
		struct FRender
		{
			static void GpuCrash(FRHICommandList&)
			{
				UE_LOGF(LogEngine, Warning, "Printed warning to log.");
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				GIsGPUCrashed = true;
				UE_LOGF(LogEngine, Fatal, "Simulating a GPU crash at your request");
				DebugDummyWorkAfterFailure();
			}
		};
		ENQUEUE_RENDER_COMMAND(CauseGpuCrash)(&FRender::GpuCrash);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CHECK")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		check(!"Crashing the game thread via check(0) at your request");
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GPF")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		Ar.Log(TEXT("Crashing with voluntary GPF"));
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		UE_FORCE_CRASH();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("ENSURE")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		if (!ensure(0))
		{
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("ENSUREALWAYS")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		if (!ensureAlways(0))
		{
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("FATAL")))
	{
		UE_LOGF(LogEngine, Warning, "Printed warning to log.");
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		LowLevelFatalError(TEXT("FError::LowLevelFatal test"));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("BUFFEROVERRUN")))
	{
		// stack overflow test - this case should be caught by /GS (Buffer Overflow Check) compile option
		ANSICHAR SrcBuffer[] = "12345678901234567890123456789012345678901234567890";
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		BufferOverflowFunction(UE_ARRAY_COUNT(SrcBuffer), SrcBuffer);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CRTINVALID")))
	{
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		CauseCrtError();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("STALL")))
	{
		float Seconds = FCString::Atof(Cmd);
		if (Seconds == 0.0f)
		{
			Seconds = 10.0f;
		}
		SCOPE_CYCLE_COUNTER(STAT_IntentionalHitch);
		Ar.Logf(ELogVerbosity::Display, TEXT("Stalling for %.0f seconds."), Seconds);
		FPlatformProcess::Sleep(Seconds);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("HITCH")))
	{
		float Seconds = FCString::Atof(Cmd) / 1000.0f;
		if (Seconds == 0.0f)
		{
			Seconds = 1.0f;
		}
		SCOPE_CYCLE_COUNTER(STAT_IntentionalHitch);
		FPlatformProcess::Sleep(Seconds);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("RENDERHITCH")))
	{
		float Seconds = FCString::Atof(Cmd) / 1000.0f;
		if (Seconds == 0.0f)
		{
			Seconds = 1.0f;
		}
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadHitch)(
			[Seconds](FRHICommandListImmediate& RHICmdList)
			{
				SCOPE_CYCLE_COUNTER(STAT_IntentionalHitch);
				FPlatformProcess::Sleep(Seconds);
			});
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("SPIN")))
	{
		float Seconds = FCString::Atof(Cmd) / 1000.0f;
		if (Seconds == 0.0f)
		{
			Seconds = 1.0f;
		}
		SCOPE_CYCLE_COUNTER(STAT_IntentionalHitch);
		double StartTime = FPlatformTime::Seconds();
		while (FPlatformTime::Seconds() < StartTime + Seconds)
		{
			
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("RENDERSPIN")))
	{
		float Seconds = FCString::Atof(Cmd) / 1000.0f;
		if (Seconds == 0.0f)
		{
			Seconds = 1.0f;
		}
		ENQUEUE_RENDER_COMMAND(CauseRenderThreadHitch)(
			[Seconds](FRHICommandListImmediate& RHICmdList)
			{
				SCOPE_CYCLE_COUNTER(STAT_IntentionalHitch);
				double StartTime = FPlatformTime::Seconds();
				while (FPlatformTime::Seconds() < StartTime + Seconds) {}
			});
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("LONGLOG")))
	{
		UE_LOGF(LogEngine, Log, "This is going to be a really long log message to test the code to resize the buffer used to log with. %2048ls", TEXT("HAHA, this isn't really a long string, but it sure has a lot of spaces!"));
	}
	else if (FParse::Command(&Cmd, TEXT("RECURSE")))
	{
		Ar.Logf(TEXT("Recursing to create a very deep callstack."));
		GLog->Flush();
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		InfiniteRecursionFunction(1);
		Ar.Logf(TEXT("You will never see this log line."));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("THREADRECURSE")))
	{
		Ar.Log(TEXT("Recursing to create a very deep callstack (in a separate thread)."));
		struct FThread
		{
			static void InfiniteRecursion(ENamedThreads::Type, const FGraphEventRef&)
			{
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				InfiniteRecursionFunction(1);
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FThread::InfiniteRecursion"),
		STAT_FThread__InfiniteRecursion,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::InfiniteRecursion),
				GET_STATID(STAT_FThread__InfiniteRecursion)),
			ENamedThreads::GameThread
		);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("EATMEM")))
	{
		if (FParse::Command(&Cmd, TEXT("TICK")))
		{
			if (FParse::Command(&Cmd, TEXT("STOP")))
			{
				FDebugMemoryEater::Get().Stop();
			}
			else if (FParse::Command(&Cmd, TEXT("PURGE")))
			{
				FDebugMemoryEater::Get().Stop();
				FDebugMemoryEater::Get().Purge();
			}
			else
			{
				Ar.Log(TEXT("Gradually eat memory every Tick until there's less than the requested target left"));

				if (!FDebugMemoryEater::Get().Start(Cmd, Ar))
				{
					Ar.Log(TEXT("Failed to start the memory eater!"));
				}
			}
		}
		else
		{
			Ar.Log(TEXT("Eating up all available memory"));
			FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
			while (1)
			{
#				if PLATFORM_WINDOWS
					/* Windows does not allow overcommitting, so allocating the memory is sufficient to cause an OOM.
					   Running memset on the memory block will cause Windows to start swapping memory to/from the page file
					   and will ultimately result in an application hang instead of OOM. */
					(void)FMemory::Malloc(65536 * 1024);
#				else
					void* Eat = FMemory::Malloc(65536);
					FMemory::Memset(Eat, 0, 65536);
#				endif
			} //-V773
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("OOM")))
	{
		auto ParseFromCmd = [](const TCHAR*& Cmd, uint32 DefaultValue)
		{
			uint32 ParsedValue = DefaultValue;

			const FString Token = FParse::Token(Cmd, false);
			if (!Token.IsEmpty())
			{
				LexFromString(ParsedValue, *Token);
				if (ParsedValue == 0)
				{
					ParsedValue = DefaultValue;
				}
			}
			
			return ParsedValue;
		};
	
		// By default we leak 1mb per frame in chunks of 64kb
		const uint32 LeakPerFrameMiB = ParseFromCmd(Cmd, 1);
		const uint32 AllocSizeKiB = ParseFromCmd(Cmd, 64);
		
		const uint32 NumAllocs = FMath::DivideAndRoundUp(LeakPerFrameMiB * 1024, AllocSizeKiB);
		const uint32 AllocSizeBytes = AllocSizeKiB * 1024;

		Ar.Logf(TEXT("Will continuously allocate %u MiB per frame in %u KiB chunks until the process runs OOM"), LeakPerFrameMiB, AllocSizeKiB);

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([NumAllocs, AllocSizeBytes](float)
		{
			for (uint32 Index = 0; Index < NumAllocs; ++Index)
			{
				void* Eat = FMemory::Malloc(AllocSizeBytes);
				FMemory::Memset(Eat, 0, AllocSizeBytes);
			} //-V773

			return true;
		}));

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("ALLOC")))
	{
		int64 MBToAllocate = FCString::Atoi64(Cmd);
		if (MBToAllocate < 0)
		{
			Ar.Log(TEXT("Cannot allocate a negative amount of memory. Nice try though"));
			return false;
		}
		else
		{
			Ar.Logf(TEXT("Allocating %d MB..."), MBToAllocate);
		}


		constexpr static size_t BytesPerMegabyte = 1024 * 1024;
		while (MBToAllocate > 0)
		{
			// allocate 1 MB at a time (trying to malloc a huge number of bytes in one call seems to silently fail)
			void* Eat = FMemory::Malloc(BytesPerMegabyte);
			// newly allocated memory must be touched
			FMemory::Memset(Eat, 0, BytesPerMegabyte);
			MBToAllocate--;
		} //-V773

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("GPUALLOC")))
	{
		int64 MBToAllocate = FCString::Atoi64(Cmd);
		if (MBToAllocate < 0)
		{
			Ar.Log(TEXT("Cannot allocate a negative amount of memory. Nice try though"));
			return false;
		}
		else
		{
			Ar.Logf(TEXT("Allocating %d MB of VRAM..."), MBToAllocate);
		}

		static TArray<FBufferRHIRef> BufferRefs;
		ENQUEUE_RENDER_COMMAND(GPUOOMAllocate)([MBToAllocate](FRHICommandListImmediate& RHICmdList)
		{
			// variables captured by value are const
			size_t RemainingMB = MBToAllocate;
			while (RemainingMB > 0)
			{
				// allocate up to 64MB at a time
				const size_t CurrentAllocMB = FMath::Min<size_t>(64, RemainingMB);
				const size_t CurrentAllocBytes = CurrentAllocMB * 1024 * 1024;

				// create and fill buffer
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex(TEXT("Debug GPUAlloc"), CurrentAllocBytes)
					.SetInitActionInitializer()
					.DetermineInitialState();

				FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
				
				// only need to touch some of the memory
				Initializer.FillWithValue('d');
				
				FBufferRHIRef Buf = Initializer.Finalize();

				// store temporarily to avoid RAII
				BufferRefs.Add(Buf);
				
				RemainingMB -= CurrentAllocMB;
			}
		});
	}
	else if (FParse::Command(&Cmd, TEXT("STACKOVERFLOW")))
	{
		Ar.Log(TEXT("Infinite recursion to cause stack overflow"));
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		StackOverflowFunction(nullptr);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("THREADSTACKOVERFLOW")))
	{
		Ar.Log(TEXT("Infinite recursion to cause stack overflow will happen in a separate thread."));
		struct FThread
		{
			static void StackOverflow(ENamedThreads::Type, const FGraphEventRef&)
			{
				FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
				StackOverflowFunction(nullptr);
			}
		};

		DECLARE_CYCLE_STAT(TEXT("FThread::StackOverflow"),
		STAT_FThread__StackOverflow,
			STATGROUP_TaskGraphTasks);

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(
			FDelegateGraphTask::CreateAndDispatchWhenReady(
				FDelegateGraphTask::FDelegate::CreateStatic(FThread::StackOverflow),
				GET_STATID(STAT_FThread__StackOverflow)),
			ENamedThreads::GameThread
		);
		return true;
	}

	else if (FParse::Command(&Cmd, TEXT("SLEEP")))
	{
		Ar.Log(TEXT("Sleep for 1 hour. This should crash after a few seconds in cooked builds."));
		FPlatformProcess::Sleep(3600);
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("AUDIOGPF")))
	{
		struct FAudio
		{
			static void GPF()
			{
				UE_FORCE_CRASH();
			}
		};
		FAudioThread::RunCommandOnAudioThread(&FAudio::GPF, TStatId());
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("AUDIOCHECK")))
	{
		struct FAudio
		{
			static void Check()
		{
			check(!"Crashing the audio thread via check(0) at your request");
				DebugDummyWorkAfterFailure();
			}
		};
		FAudioThread::RunCommandOnAudioThread(&FAudio::Check, TStatId());
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CRASHCONTEXTWRITER")))
	{
		TArray<uint8> TestData = { 0xd, 0xe, 0xa, 0xd };
		UE_ADD_CRASH_CONTEXT_SCOPE([&](FCrashContextExtendedWriter& Writer) { 
				TCHAR TextBuffer[1024] = { 0 };
				FCString::Sprintf(TextBuffer, TEXT("Last error was: %d"), FPlatformMisc::GetLastError());
				Writer.AddString(TEXT("TestText"), TextBuffer); 
				Writer.AddBuffer(TEXT("TestData"), TestData.GetData(), TestData.Num());
				
		});
		FGenericCrashContext::SetCrashTrigger(ECrashTrigger::Debug);
		UE_LOGF(LogEngine, Fatal, "Crashing the worker thread at your request");
	}
	else if (PerformBlockingError(Cmd, Ar))
	{
		return true;
	}
#if USING_ADDRESS_SANITISER
	else if (FParse::Command(&Cmd, TEXT("USEAFTERFREE")))
	{
		static constexpr size_t UseAfterFreeArraySize = 32;
		uint8* UseAfterFreeArray = reinterpret_cast<uint8*>(FMemory::Malloc(UseAfterFreeArraySize));
		FMemory::Memzero(UseAfterFreeArray, UseAfterFreeArraySize);
		FMemory::Free(UseAfterFreeArray);
		
		// Reuse deleted array
		FMemory::Memzero(UseAfterFreeArray, UseAfterFreeArraySize);
	}
	else if (FParse::Command(&Cmd, TEXT("USEAFTERFREEMEMSTACK")))
	{
		static constexpr size_t MemStackUseAfterFreeArraySize = 4096; 
		uint8* DanglingMemStackPtr = nullptr;
		{
			FMemMark Mark(FMemStack::Get());
			DanglingMemStackPtr = static_cast<uint8*>(FMemStack::Get().Alloc(MemStackUseAfterFreeArraySize, 16));
		}

		// Do another Memstack allocation to validate the freed chunk from above has been put in quarantine
		{
			FMemMark Mark(FMemStack::Get());
			FMemStack::Get().Alloc(MemStackUseAfterFreeArraySize, 16);
		}

		// Reuse deleted array
		FMemory::Memzero(DanglingMemStackPtr, MemStackUseAfterFreeArraySize);
	}
#endif
#endif // !UE_BUILD_SHIPPING
	return false;
}

/** Uses StatColorMappings to find a color for this stat's value. */
bool UEngine::GetStatValueColoration(const FString& StatName, float Value, FColor& OutColor)
{
	for(const FStatColorMapping& Mapping : StatColorMappings)
	{
		if(StatName == Mapping.StatName)
		{
			const int32 NumPoints = Mapping.ColorMap.Num();

			// If no point in curve, return the Default value we passed in.
			if( NumPoints == 0 )
			{
				return false;
			}

			// If only one point, or before the first point in the curve, return the first points value.
			if( NumPoints < 2 || (Value <= Mapping.ColorMap[0].In) )
			{
				OutColor = Mapping.ColorMap[0].Out;
				return true;
			}

			// If beyond the last point in the curve, return its value.
			if( Value >= Mapping.ColorMap[NumPoints-1].In )
			{
				OutColor = Mapping.ColorMap[NumPoints-1].Out;
				return true;
			}

			// Somewhere with curve range - linear search to find value.
			for( int32 PointIndex=1; PointIndex<NumPoints; PointIndex++ )
			{	
				if( Value < Mapping.ColorMap[PointIndex].In )
				{
					if (Mapping.DisableBlend)
					{
						OutColor = Mapping.ColorMap[PointIndex].Out;
					}
					else
					{
						const float Diff = Mapping.ColorMap[PointIndex].In - Mapping.ColorMap[PointIndex-1].In;
						const float Alpha = (Value - Mapping.ColorMap[PointIndex-1].In) / Diff;

						FLinearColor A(Mapping.ColorMap[PointIndex-1].Out);
						FVector AV(A.R, A.G, A.B);

						FLinearColor B(Mapping.ColorMap[PointIndex].Out);
						FVector BV(B.R, B.G, B.B);

						FVector OutColorV = FMath::Lerp( AV, BV, Alpha );
						OutColor = FLinearColor(OutColorV.X, OutColorV.Y, OutColorV.Z).ToFColor(true);
					}

					return true;
				}
			}

			OutColor = Mapping.ColorMap[NumPoints-1].Out;
			return true;
		}
	}

	// No entry for this stat name
	return false;
}

void UEngine::OnLostFocusPause(bool EnablePause)
{
	if( bPauseOnLossOfFocus )
	{
		for (auto It = WorldList.CreateIterator(); It; ++It)
		{
			FWorldContext &Context = *It;

			if ( Context.OwningGameInstance == NULL )
			{
				continue;
			}

			const TArray<class ULocalPlayer*> & LocalPlayers = Context.OwningGameInstance->GetLocalPlayers();

			// Iterate over all players and pause / unpause them
			// Note: pausing / unpausing the player is done via their HUD pausing / unpausing
			for (int32 PlayerIndex = 0; PlayerIndex < LocalPlayers.Num(); ++PlayerIndex)
			{
				APlayerController* PlayerController = LocalPlayers[PlayerIndex]->PlayerController;
				if(PlayerController && PlayerController->MyHUD)
				{
					PlayerController->MyHUD->OnLostFocusPause(EnablePause);
				}
			}
		}
	}
}

static TAutoConsoleVariable<float> CVarMaxFPS(
	TEXT("t.MaxFPS"),0.f,
	TEXT("Caps FPS to the given value.  Set to <= 0 to be uncapped."));
// CauseHitches cvar
static TAutoConsoleVariable<int32> CVarCauseHitches(
	TEXT("CauseHitches"),0,
	TEXT("Causes a 200ms hitch every second. Size of the hitch is controlled by CauseHitchesHitchMS"));

static TAutoConsoleVariable<int32> CVarCauseHitchesMS(
	TEXT("CauseHitchesHitchMS"), 200,
	TEXT("Controls the size of the hitch caused by CauseHitches in ms."));

static TAutoConsoleVariable<int32> CVarUnsteadyFPS(
	TEXT("t.UnsteadyFPS"),0,
	TEXT("Causes FPS to bounce around randomly in the 8-32 range."));

void UEngine::InitializeRunningAverageDeltaTime()
{
	// Running average delta time, initial value at 100 FPS so fast machines don't have to creep up
	// to a good frame rate due to code limiting upward "mobility".
	RunningAverageDeltaTime = 1 / 100.f;
}

bool UEngine::IsAllowedFramerateSmoothing() const
{
	return FPlatformProperties::AllowsFramerateSmoothing() && bSmoothFrameRate && !bForceDisableFrameRateSmoothing && !IsRunningDedicatedServer();
}

/** Compute tick rate limitor. */
void UEngine::UpdateRunningAverageDeltaTime(float DeltaTime, bool bAllowFrameRateSmoothing)
{
	if (bAllowFrameRateSmoothing && IsAllowedFramerateSmoothing())
	{
		// Smooth the framerate if wanted. The code uses a simplistic running average. Other approaches, like reserving
		// a percentage of time, ended up creating negative feedback loops in conjunction with GPU load and were abandonend.
		if( DeltaTime < 0.0f )
		{
#if PLATFORM_ANDROID
			UE_LOGF(LogEngine, Warning, "Detected negative delta time - ignoring");
			DeltaTime = 0.01;
#elif (UE_BUILD_SHIPPING && WITH_EDITOR)
			// End users don't have access to the secure parts of UDN. The localized string points to the release notes,
			// which should include a link to the AMD CPU drivers download site.
			UE_LOGF(LogEngine, Fatal, "%ls", TEXT("CPU time drift detected! Please consult release notes on how to address this."));
#else
			// Send developers to the support list thread.
			UE_LOGF(LogEngine, Fatal, "Negative delta time! Please see https://udn.epicgames.com/lists/showpost.php?list=ue3bugs&id=4364");
#endif
		}

		// Keep track of running average over 300 frames, clamping at min of 5 FPS for individual delta times.
		RunningAverageDeltaTime = FMath::Lerp<float>( RunningAverageDeltaTime, FMath::Min<float>( DeltaTime, 0.2f ), 1 / 300.f );
	}
}


/** Get tick rate limitor. */
float UEngine::GetMaxTickRate(float DeltaTime, bool bAllowFrameRateSmoothing) const
{
	float MaxTickRate = 0;

	if (bAllowFrameRateSmoothing && IsAllowedFramerateSmoothing())
	{
		// Work in FPS domain as that is what the function will return.
		MaxTickRate = 1.f / RunningAverageDeltaTime;

		// Clamp FPS into ini defined min/ max range.
		if (SmoothedFrameRateRange.HasLowerBound())
		{
			MaxTickRate = FMath::Max( MaxTickRate, SmoothedFrameRateRange.GetLowerBoundValue() );
		}
		if (SmoothedFrameRateRange.HasUpperBound())
		{
			MaxTickRate = FMath::Min( MaxTickRate, SmoothedFrameRateRange.GetUpperBoundValue() );
		}
	}

	if (CVarCauseHitches.GetValueOnAnyThread())
	{
		static float RunningHitchTimer = 0.f;
		RunningHitchTimer += DeltaTime;
		float SleepTime = float(CVarCauseHitchesMS.GetValueOnAnyThread()) / 1000.0f;
		if (RunningHitchTimer > 1.f + SleepTime)
		{
			// hitch!
			UE_LOGF(LogEngine, Display, "Hitching by request!");
			FPlatformProcess::Sleep(SleepTime);
			RunningHitchTimer = 0.f;
		}
	}

	if (CVarUnsteadyFPS.GetValueOnAnyThread())
	{
		static float LastMaxTickRate = 20.f;
		float RandDelta = FMath::FRandRange(-5.f, 5.f);
		MaxTickRate = FMath::Clamp(LastMaxTickRate + RandDelta, 8.f, 32.f);
		LastMaxTickRate = MaxTickRate;
	}
	else if (CVarMaxFPS.GetValueOnAnyThread() > 0)
	{
		MaxTickRate = CVarMaxFPS.GetValueOnAnyThread();
	}

	return MaxTickRate;
}

float UEngine::GetMaxFPS() const
{
	return CVarMaxFPS.GetValueOnAnyThread();
}

void UEngine::SetMaxFPS(const float MaxFPS)
{
	IConsoleVariable* ConsoleVariable = CVarMaxFPS.AsVariable();

	const EConsoleVariableFlags LastSetReason = (EConsoleVariableFlags)(ConsoleVariable->GetFlags() & ECVF_SetByMask);
	const EConsoleVariableFlags ThisSetReason = (LastSetReason == ECVF_SetByConstructor) ? ECVF_SetByScalability : LastSetReason;

	ConsoleVariable->Set(MaxFPS, ThisSetReason);
}

double UEngine::GetUnifiedTimeBudgetForStreaming(float DeltaSeconds, bool bHighPriority)
{
	double UnifiedTime = (GAsyncLoadingTimeLimit + GLevelStreamingActorsUpdateTimeLimit) / 1000.0f;
	if (bHighPriority)
	{
		UnifiedTime += (GPriorityAsyncLoadingExtraTime + GPriorityLevelStreamingActorsUpdateExtraTime) / 1000.0f;
	}

	// TODO handle FPS scaling
	return UnifiedTime;
}

void UEngine::HandleUnifiedStreaming(float DeltaSeconds)
{
	SCOPED_NAMED_EVENT(Streaming, FColor::Silver);
	SCOPE_TIME_GUARD(TEXT("UEngine::HandleUnifiedStreaming"));

	// Give the async loading code more time if we're performing a high priority load or are in seamless travel
	bool bIsHighPriorityLoad = false;

	for (int32 WorldIdx = 0; WorldIdx < WorldList.Num(); ++WorldIdx)
	{
		FWorldContext& Context = WorldList[WorldIdx];
		if (Context.World() != nullptr && Context.World()->ShouldTick())
		{
			AWorldSettings* WorldSettings = Context.World()->GetWorldSettings();
			if (WorldSettings && (WorldSettings->bHighPriorityLoadingLocal || WorldSettings->bHighPriorityLoading || Context.World()->IsInSeamlessTravel()))
			{
				bIsHighPriorityLoad = true;
				break;
			}
		}
	}

	// Start the timeout before first async load
	double UnifiedBudget = GetUnifiedTimeBudgetForStreaming(DeltaSeconds, bIsHighPriorityLoad);
	UE::FTimeout StreamingTimeout(UnifiedBudget);

	double AsyncLoadBudget = (GAsyncLoadingTimeLimit / 1000.0);
	if (bIsHighPriorityLoad)
	{
		AsyncLoadBudget += (GPriorityAsyncLoadingExtraTime / 1000.0);
	}

	// Handle async loading on the game thread right before level streaming so they can share a budget
	// Use the full time during high priority loads to match the existing code in UWorld::Tick
	StaticTick(DeltaSeconds, (GAsyncLoadingUseFullTimeLimit || bIsHighPriorityLoad), AsyncLoadBudget);

	// If the process async loading did not use the full time budget it will be granted to UpdateLevelStreaming for all ticking worlds
	// If process async loading exceeded the budget - UpdateLevelStreaming will amortize the hitch
	for (int32 WorldIdx = 0; WorldIdx < WorldList.Num(); ++WorldIdx)
	{
		FWorldContext& Context = WorldList[WorldIdx];
		if (Context.World() != nullptr && Context.World()->ShouldTick())
		{
			Context.World()->UpdateLevelStreaming(StreamingTimeout);
		}
	}

	// If the Update LevelStreaming did not use full time budget, use the rest for async asset loading
	constexpr double ExtraProcessingThreshold = 0.05 * 0.001;
	double RemainingTime = StreamingTimeout.GetRemainingSeconds();

	if (RemainingTime > ExtraProcessingThreshold)
	{
		constexpr bool bUseTimeLimit = true;
		constexpr bool bUseFullTimeLimit = false;
		ProcessAsyncLoading(bUseTimeLimit, bUseFullTimeLimit, RemainingTime);
	}
}

void UEngine::RegisterViewport(TSharedRef<FViewport> Viewport)
{
	using namespace UE;

	TUniqueLock Lock(ActiveViewportsMutex);
	for (const TSharedRef<FViewport>& Existing : ActiveViewports)
	{
		if (&Existing.Get() == &Viewport.Get())
		{
			return;
		}
	}
	ActiveViewports.Add(Viewport);
}

void UEngine::UnregisterViewport(TSharedRef<FViewport> Viewport)
{
	using namespace UE;

	TUniqueLock Lock(ActiveViewportsMutex);
	int32 ToRemoveIdx = ActiveViewports.IndexOfByPredicate([&Viewport](const TSharedRef<FViewport>& Existing)
	{
		return &Existing.Get() == &Viewport.Get();
	});
	if (ToRemoveIdx != INDEX_NONE)
	{
		ActiveViewports.RemoveAtSwap(ToRemoveIdx);
	}
}

void UEngine::RedrawViewports(bool bShouldPresent)
{
	using namespace UE;

	TArray<TSharedRef<FViewport>> ViewportsCopy;
	{
		TUniqueLock Lock(ActiveViewportsMutex);
		ViewportsCopy = ActiveViewports;
	}

	if (ViewportsCopy.Num() == 0)
	{
		return;
	}

	// Make sure we don't attempt to render a frame in parallel with the loading screen, which might deadlock because of interleaving two different
	// frames.  It could happen because the Game thread invokes RedrawViewports right after blocking loads when the loading thread is still spinning.
	FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer();
	FScopeLock ScopeLock(SlateRenderer->GetResourceCriticalSection());

	for (const TSharedRef<FViewport>& Viewport : ViewportsCopy)
	{
		Viewport->Draw(bShouldPresent);
	}
}

void UEngine::EnableScreenSaver( bool bEnable )
{
#if PLATFORM_DESKTOP
	if (IsEngineExitRequested())
	{
		return;
	}

	FString EnvVariable = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-DisallowScreenSaverInhibitor"));
	const bool bDisallowScreenSaverInhibitor = FString(EnvVariable).ToBool();

	// By default we allow to use screen saver inhibitor, but in some cases user can override this setting.
	if( !bDisallowScreenSaverInhibitor )
	{
		// try a simpler API first
		if ( !FPlatformApplicationMisc::ControlScreensaver( bEnable ? FPlatformApplicationMisc::EScreenSaverAction::Enable : FPlatformApplicationMisc::EScreenSaverAction::Disable ) )
		{
			// Screen saver inhibitor disabled if no multithreading is available.
			if (FPlatformProcess::SupportsMultithreading() )
			{
				if( !ScreenSaverInhibitor )
				{
					// Create thread inhibiting screen saver while it is running.
					ScreenSaverInhibitorRunnable = new FScreenSaverInhibitor();
					ScreenSaverInhibitor = FRunnableThread::Create(ScreenSaverInhibitorRunnable, TEXT("ScreenSaverInhibitor"), 16 * 1024, TPri_Normal, FPlatformAffinity::GetPoolThreadMask());
					// Only actually run when needed to not bypass group policies for screensaver, etc.
					ScreenSaverInhibitor->Suspend( true );
					ScreenSaverInhibitorSemaphore = 0;
				}

				if( bEnable && ScreenSaverInhibitorSemaphore > 0)
				{
					if( --ScreenSaverInhibitorSemaphore == 0 )
					{	
						// If the semaphore is zero and we are enabling the screensaver
						// the thread preventing the screen saver should be suspended
						ScreenSaverInhibitor->Suspend( true );
					}
				}
				else if( !bEnable )
				{
					if( ++ScreenSaverInhibitorSemaphore == 1 )
					{
						// If the semaphore is just becoming one, the thread 
						// is was not running so enable it.
						ScreenSaverInhibitor->Suspend( false );
					}
				}
			}
		}
	}
#endif
}

/**
* Adds a world location as a secondary view location for purposes of texture streaming.
* Lasts one frame, or a specified number of seconds (for overriding locations only).
*
* @param InLoc					World-space view origin
* @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
* @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other locations
* @param OverrideDuration		How long the streaming system should keep checking this location if bOverrideLocation is true, in seconds. 0 means just for the next Tick.
*/
void UEngine::AddTextureStreamingLoc(FVector InLoc, float BoostFactor, bool bOverrideLocation, float OverrideDuration)
{
	IStreamingManager::Get().AddViewLocation(InLoc, BoostFactor, bOverrideLocation, OverrideDuration);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
// Critical section used in AddOnScreenDebugMessage to handle concurrent insert.
static FTransactionallySafeCriticalSection GOnScreenMessageCS;
#endif

/**
*	This function will add a debug message to the onscreen message list.
*	It will be displayed for FrameCount frames.
*
*	@param	Key				A unique key to prevent the same message from being added multiple times.
*	@param	TimeToDisplay	How long to display the message, in seconds.
*	@param	DisplayColor	The color to display the text in.
*	@param	DebugMessage	The message to display.
*/
void UEngine::AddOnScreenDebugMessage(uint64 Key, float TimeToDisplay, FColor DisplayColor, const FString& DebugMessage, bool bNewerOnTop, const FVector2D& TextScale)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (bEnableOnScreenDebugMessages == true)
	{
		// Because some components add their message in concurrent work, we need a CS here.
		UE::TScopeLock ScopeLock(GOnScreenMessageCS);
		if (Key == (uint64)-1)
		{
			if (bNewerOnTop)
			{
				FScreenMessageString* NewMessage = new(PriorityScreenMessages)FScreenMessageString();
				check(NewMessage);
				NewMessage->Key = Key;
				NewMessage->ScreenMessage = DebugMessage;
				NewMessage->DisplayColor = DisplayColor;
				NewMessage->TimeToDisplay = TimeToDisplay;
				NewMessage->TextScale = TextScale;
				NewMessage->CurrentTimeDisplayed = 0.0f;				
			}
			else
			{
				FScreenMessageString NewMessage;
				NewMessage.CurrentTimeDisplayed = 0.0f;
				NewMessage.Key = Key;
				NewMessage.DisplayColor = DisplayColor;
				NewMessage.TimeToDisplay = TimeToDisplay;
				NewMessage.ScreenMessage = DebugMessage;
				NewMessage.TextScale = TextScale;
				PriorityScreenMessages.Insert(NewMessage, 0);
			}
		}
		else
		{
			FScreenMessageString* Message = ScreenMessages.Find(Key);
			if (Message == NULL)
			{
				FScreenMessageString NewMessage;
				NewMessage.CurrentTimeDisplayed = 0.0f;
				NewMessage.Key = Key;
				NewMessage.TextScale = TextScale;
				NewMessage.DisplayColor = DisplayColor;
				NewMessage.TimeToDisplay = TimeToDisplay;
				NewMessage.ScreenMessage = DebugMessage;				
				NewMessage.TextScale = TextScale;
				ScreenMessages.Add((int32)Key, NewMessage);
			}
			else
			{
				// Set the message, and update the time to display and reset the current time.
				Message->ScreenMessage = DebugMessage;
				Message->DisplayColor = DisplayColor;
				Message->TextScale = TextScale;
				Message->TimeToDisplay = TimeToDisplay;
				Message->CurrentTimeDisplayed = 0.0f;				
				Message->TextScale = TextScale;
			}
		}
	}
#endif
}

void UEngine::RemoveOnScreenDebugMessage(uint64 Key)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		// Because some components add their message in concurrent work, we need a CS here.
		UE::TScopeLock ScopeLock(GOnScreenMessageCS);
		ScreenMessages.Remove((int32)Key);
	}
#endif
}

struct UEngine::FErrorsAndWarningsCollector : public FBufferedOutputDevice
{
	FErrorsAndWarningsCollector();
	~FErrorsAndWarningsCollector();

	bool Tick(float Seconds);

	TArray<FBufferedLine>		ActiveLines;
	TMap<uint32, uint32>		MessagesToCountMap;
	FTSTicker::FDelegateHandle	TickerHandle;
	float						DisplayTime;
};

UEngine::FErrorsAndWarningsCollector::FErrorsAndWarningsCollector()
{
	DisplayTime = 0.0f;
	GConfig->GetFloat(TEXT("/Script/Engine.Engine"), TEXT("DurationOfErrorsAndWarningsOnHUD"), DisplayTime, GEngineIni);

	if (DisplayTime > 0.0f)
	{
		SetVerbosity((GSuppressWarningsInOnScreenDisplay != 0) ? ELogVerbosity::Error : ELogVerbosity::Warning);
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &UEngine::FErrorsAndWarningsCollector::Tick), DisplayTime);
		FOutputDeviceRedirector::Get()->AddOutputDevice(this);
	}
}

UEngine::FErrorsAndWarningsCollector::~FErrorsAndWarningsCollector()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}
	// Note that TickerHandle might become invalid for other reasons, so it can not be used to guard the call to
	// RemoveOutputDevice. We don't need a separate guard variable, it's legal and cheap to call even if not added.
	FOutputDeviceRedirector::Get()->RemoveOutputDevice(this);
}

bool UEngine::FErrorsAndWarningsCollector::Tick(float Seconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FErrorsAndWarningsCollector_Tick);

	// Set this each tick, in case the cvar is changed at runtime
	SetVerbosity((GSuppressWarningsInOnScreenDisplay != 0) ? ELogVerbosity::Error : ELogVerbosity::Warning);

	{
		TArray<FBufferedLine> NewLines;
		GetContents(NewLines);
		ActiveLines.Append(MoveTemp(NewLines));
	}

	if (ActiveLines.Num())
	{
		int DupeCount = 0;
		int CurrentHash = 0;

		// Remove any dupes and count them
		do 
		{
			uint32 ThisHash = FCrc::StrCrc32(ActiveLines[DupeCount].Data.Get());

			if (CurrentHash && ThisHash != CurrentHash)
			{
				break;
			}

			CurrentHash = ThisHash;
			DupeCount++;

		} while (DupeCount < ActiveLines.Num());

		// Save off properties
		FString Msg = ActiveLines[0].Data.Get();
		ELogVerbosity::Type Verbosity = ActiveLines[0].Verbosity;

		// Remove any lines we condensed
		ActiveLines.RemoveAt(0, DupeCount);

		uint32* pCount = MessagesToCountMap.Find(CurrentHash);

		if (pCount)
		{
			DupeCount += (*pCount);
		}

		MessagesToCountMap.Add(CurrentHash, DupeCount);

		if (DupeCount > 1)
		{
			Msg = FString::Printf(TEXT("%s (x%d)"), *Msg, DupeCount);
		}

		const FColor LineColor = Verbosity <= ELogVerbosity::Error ? FColor::Red : FColor::Yellow;
		GEngine->AddOnScreenDebugMessage(-1, DisplayTime, LineColor, Msg);

		// Prune the list so we get new errors instead of ones from minutes ago
		const float MaxTimeToKeep = 30.f;
		const int32 MaxNumToKeep = FMath::TruncToInt(MaxTimeToKeep / DisplayTime);

		if (ActiveLines.Num() > MaxNumToKeep)
		{
			ActiveLines.RemoveAt(0, ActiveLines.Num() - MaxNumToKeep);
		}
	}

	return true;
}

/** Wrapper from int32 to uint64 */
void UEngine::AddOnScreenDebugMessage(int32 Key, float TimeToDisplay, FColor DisplayColor, const FString& DebugMessage, bool bNewerOnTop, const FVector2D& TextScale)
{
	AddOnScreenDebugMessage((uint64)Key, TimeToDisplay, DisplayColor, DebugMessage, bNewerOnTop, TextScale);
}

bool UEngine::OnScreenDebugMessageExists(uint64 Key)
{
#if !UE_BUILD_SHIPPING
	if (bEnableOnScreenDebugMessages == true)
	{
		if (Key == (uint64)-1)
		{
			// Priority messages assumed to always exist...
			// May want to check for there being none.
			return true;
		}

		if (ScreenMessages.Find(Key) != NULL)
		{
			return true;
		}
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

void UEngine::ClearOnScreenDebugMessages()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	{	
		// Because some components add their message in concurrent work, we need a CS here.
		UE::TScopeLock ScopeLock(GOnScreenMessageCS);
		ScreenMessages.Empty();
		PriorityScreenMessages.Empty();
	}
#endif // !UE_BUILD_SHIPPING
}

#if !UE_BUILD_SHIPPING
void UEngine::PerformanceCapture(UWorld* World, const FString& MapName, const FString& SequenceName, float EventTime)
{
	// todo
	uint32 t = IStreamingManager::Get().StreamAllResources(5.0f);
	ensure(!t);

	LogPerformanceCapture(World, MapName, SequenceName, EventTime);

	// can be define by command line -BuildName="ByCustomBuildName" or "CL<changelist>"
	FString BuildName = GetBuildNameForPerfTesting();

	// e.g. XboxOne, AllDesktop, Android_.., PS4
	FString PlatformName = FPlatformProperties::PlatformName();

	// e.g. D3D11,OpenGL,Vulcan,D3D12
	FString RHIName = "UnknownRHI";
	{
		// Create the folder name based on the hardware specs we have been provided
		FString HardwareDetails = FHardwareInfo::GetHardwareDetailsString();

		FString RHILookup = NAME_RHI.ToString() + TEXT("=");
		if (!FParse::Value(*HardwareDetails, *RHILookup, RHIName))
		{
			// todo error?
		}
	}

	FString CaptureName = FString::Printf(TEXT("Map(%s) Actor(%s) Time(%4.2f)"), *MapName, *GetName(), EventTime);

	FString ScreenshotName = FPaths::AutomationDir() / TEXT("RenderOutputValidation") / BuildName / PlatformName + TEXT("_") + RHIName / CaptureName + TEXT(".png");

	{
		UConsole* ViewportConsole = (GEngine->GameViewport != nullptr) ? GEngine->GameViewport->ViewportConsole : nullptr;
		FConsoleOutputDevice StrOut(ViewportConsole);

		StrOut.Logf(TEXT("  frame:%llu %s"), (uint64)GFrameCounter, *ScreenshotName);
	}

	const bool bShowUI = false;
	const bool bAddFilenameSuffix = false;
	FScreenshotRequest::RequestScreenshot( ScreenshotName, bShowUI, bAddFilenameSuffix );
}

void UEngine::LogPerformanceCapture(UWorld* World, const FString& MapName, const FString& SequenceName, float EventTime)
{
	const FString EventType = "PERF";
	const int32 ChangeList = FEngineVersion::Current().GetChangelist();

	extern ENGINE_API float GAverageFPS;

	if ((World != nullptr) && (World->GetGameViewport() != nullptr))
	{
		const FStatUnitData* StatUnitData = World->GetGameViewport()->GetStatUnitData();

		FAutomationPerformanceSnapshot PerfSnapshot;
		PerfSnapshot.Changelist = FString::FromInt(ChangeList);
		PerfSnapshot.BuildConfiguration = LexToString(FApp::GetBuildConfiguration());
		PerfSnapshot.MapName = MapName;
		PerfSnapshot.SequenceName = SequenceName;
		PerfSnapshot.AverageFPS = FString::Printf(TEXT("%0.2f"), GAverageFPS);
		PerfSnapshot.AverageFrameTime = FString::Printf(TEXT("%0.2f"), StatUnitData->FrameTime);
		PerfSnapshot.AverageGameThreadTime = FString::Printf(TEXT("%0.2f"), StatUnitData->GameThreadTime);
		PerfSnapshot.AverageRenderThreadTime = FString::Printf(TEXT("%0.2f"), StatUnitData->RenderThreadTime);
		PerfSnapshot.AverageGPUTime = FString::Printf(TEXT("%0.2f"), StatUnitData->GPUFrameTime[0]);
		// PerfSnapshot.PercentOfFramesAtLeast60FPS = ???;	// @todo
		// PerfSnapshot.PercentOfFramesAtLeast60FPS = ???;	// @todo

		const FString PerfSnapshotAsCommaDelimitedString = PerfSnapshot.ToCommaDelimetedString();

		FAutomationTestFramework::Get().AddAnalyticsItemToCurrentTest(
			FString::Printf(TEXT("%s,%s"), *PerfSnapshotAsCommaDelimitedString, *EventType));
	}
}
#endif	// UE_BUILD_SHIPPING

/** Transforms a location in 3D space into 'map space', in 2D */
static FVector2D TransformLocationToMap(FVector2D TopLeftPos, FVector2D BottomRightPos, FVector2D MapOrigin, const FVector2D& MapSize, FVector Loc)
{
	FVector2D MapPos;

	MapPos = MapOrigin;

	MapPos.X += MapSize.X * ((Loc.Y - TopLeftPos.Y)/(BottomRightPos.Y - TopLeftPos.Y));
	MapPos.Y += MapSize.Y * (1.0 - ((Loc.X - BottomRightPos.X)/(TopLeftPos.X - BottomRightPos.X)));	

	return MapPos;
}

/** Utility for drawing a volume geometry (as seen from above) onto the canvas */
static void DrawVolumeOnCanvas(const AVolume* Volume, FCanvas* Canvas, const FVector2D& TopLeftPos, const FVector2D& BottomRightPos, const FVector2D& MapOrigin, const FVector2D& MapSize, const FColor& VolColor)
{
	if(Volume && Volume->GetBrushComponent() && Volume->GetBrushComponent()->BrushBodySetup)
	{
		FTransform BrushTM = Volume->GetBrushComponent()->GetComponentTransform();

		// Iterate over each piece
		for(int32 ConIdx=0; ConIdx<Volume->GetBrushComponent()->BrushBodySetup->AggGeom.ConvexElems.Num(); ConIdx++)
		{
			FKConvexElem& ConvElem = Volume->GetBrushComponent()->BrushBodySetup->AggGeom.ConvexElems[ConIdx];

#if 0 // @todo physx fix this once we have convexelem drawing again
			// Draw each triangle that makes up the convex hull
			const int32 NumTris = ConvElem.FaceTriData.Num()/3;
			for(int32 i=0; i<NumTris; i++)
			{
				// Get the verts that make up this triangle.
				const int32 I0 = ConvElem.FaceTriData((i*3)+0);
				const int32 I1 = ConvElem.FaceTriData((i*3)+1);
				const int32 I2 = ConvElem.FaceTriData((i*3)+2);

				const FVector V0 = BrushTM.TransformPosition(ConvElem.VertexData(I0));
				const FVector V1 = BrushTM.TransformPosition(ConvElem.VertexData(I1));
				const FVector V2 = BrushTM.TransformPosition(ConvElem.VertexData(I2));

				// We only want to draw faces pointing up
				const FVector Edge0 = V1 - V0;
				const FVector Edge1 = V2 - V1;
				const FVector Normal = (Edge1 ^ Edge0).GetSafeNormal();
				if(Normal.Z > 0.01)
				{
					// Transform as 2d points in 'map space'
					const FVector2D M0 = TransformLocationToMap( TopLeftPos, BottomRightPos, MapOrigin, MapSize, V0 );
					const FVector2D M1 = TransformLocationToMap( TopLeftPos, BottomRightPos, MapOrigin, MapSize, V1 );
					const FVector2D M2 = TransformLocationToMap( TopLeftPos, BottomRightPos, MapOrigin, MapSize, V2 );

					// dummy UVs
					FVector2D UVCoords(0,0);
					Canvas->DrawTriangle2D( M0, UVCoords, M1, UVCoords, M2, UVCoords, FLinearColor(VolColor));					

					// Draw edges of face
					if( ConvElem.DirIsFaceEdge(ConvElem.VertexData(I0) - ConvElem.VertexData(I1)) )
					{
						DrawLine( Canvas, FVector(M0.X,M0.Y,0) , FVector(M1.X,M1.Y,0), VolColor );
					}

					if( ConvElem.DirIsFaceEdge(ConvElem.VertexData(I1) - ConvElem.VertexData(I2)) )
					{
						DrawLine( Canvas, FVector(M1.X,M1.Y,0) , FVector(M2.X,M2.Y,0), VolColor );
					}

					if( ConvElem.DirIsFaceEdge(ConvElem.VertexData(I2) - ConvElem.VertexData(I0)) )
					{
						DrawLine( Canvas, FVector(M2.X,M2.Y,0) , FVector(M0.X,M0.Y,0), VolColor );
					}
				}
			}
#endif
		}
	}
}

/** Util that takes a 2D vector and rotates it by RotAngle (given in radians) */
static FVector2D RotateVec2D(const FVector2D InVec, float RotAngle)
{
	FVector2D OutVec;
	OutVec.X = (InVec.X * FMath::Cos(RotAngle)) - (InVec.Y * FMath::Sin(RotAngle));
	OutVec.Y = (InVec.X * FMath::Sin(RotAngle)) + (InVec.Y * FMath::Cos(RotAngle));
	return OutVec;
}

#if !UE_BUILD_SHIPPING
bool UEngine::HandleLogoutStatLevelsCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	const TArray<FSubLevelStatus> SubLevelsStatusList = GetSubLevelsStatus(InWorld);

	Ar.Logf( TEXT( "Levels:" ) );

	// now draw the "map" name
	if (SubLevelsStatusList.Num())
	{
		// First entry - always persistent level
		FString MapName	= SubLevelsStatusList[0].PackageName.ToString();
		if (SubLevelsStatusList[0].bPlayerInside)
		{
			MapName = *FString::Printf( TEXT("->  %s"), *MapName );
		}
		else
		{
			MapName = *FString::Printf( TEXT("    %s"), *MapName );
		}

		Ar.Logf( TEXT( "%s" ), *MapName );
	}

	// now log the levels
	for (int32 LevelIdx = 1; LevelIdx < SubLevelsStatusList.Num(); ++LevelIdx)
	{
		const FSubLevelStatus& LevelStatus = SubLevelsStatusList[LevelIdx];
		FString DisplayName = LevelStatus.PackageName.ToString();
		const TCHAR* StatusName = ULevelStreaming::GetLevelStreamingStatusDisplayName(LevelStatus.StreamingStatus);

		if (LevelStatus.LODIndex != INDEX_NONE)
		{
			DisplayName += FString::Printf(TEXT(" [LOD%d]"), LevelStatus.LODIndex+1);
		}

		UPackage* LevelPackage = FindObjectFast<UPackage>( NULL, LevelStatus.PackageName );

		if( LevelPackage 
			&& (LevelPackage->GetLoadTime() > 0) 
			&& (LevelStatus.StreamingStatus != LEVEL_Unloaded) )
		{
			DisplayName += FString::Printf(TEXT(" - %4.1f sec"), LevelPackage->GetLoadTime());
		}
		else
		{
			const float LevelLoadPercentage = GetAsyncLoadPercentage(LevelStatus.PackageName);
			if (LevelLoadPercentage >= 0.0f)
			{
				const int32 Percentage = FMath::TruncToInt(LevelLoadPercentage);
				DisplayName += FString::Printf(TEXT(" - %3i %%"), Percentage);
			}
		}

		if ( LevelStatus.bPlayerInside )
		{
			DisplayName = *FString::Printf( TEXT("->  %s"), *DisplayName );
		}
		else
		{
			DisplayName = *FString::Printf( TEXT("    %s"), *DisplayName );
		}

		DisplayName = FString::Printf( TEXT("%s \t\t%s"), *DisplayName, StatusName );

		Ar.Logf( TEXT( "%s" ), *DisplayName );

	}

	return true;
}
#endif // !UE_BUILD_SHIPPING

/** Helper structure for sorting sounds by predefined criteria. */
struct FSoundInfo
{
	/** Path name to this sound. */
	FString PathName;

	/** Distance betweend a listener and this sound. */
	float Distance;

	/** Sound group this sound belongs to. */
	FName ClassName;

	/** Wave instances currently used by this sound. */
	TArray<FWaveInstance*> WaveInstances;

	FSoundInfo( FString InPathName, float InDistance, FName InClassName )
		: PathName( InPathName )
		, Distance( InDistance )
		, ClassName( InClassName )
	{}

	bool ComparePathNames( const FSoundInfo& Other ) const
	{
		return PathName < Other.PathName;
	}

	bool CompareDistance( const FSoundInfo& Other ) const
	{
		return Distance < Other.Distance;
	}

	bool CompareClass( const FSoundInfo& Other ) const
	{
		return ClassName.LexicalLess(Other.ClassName);
	}

	bool CompareWaveInstancesNum( const FSoundInfo& Other ) const
	{
		return Other.WaveInstances.Num() < WaveInstances.Num();
	}
};

struct FCompareFSoundInfoByName
{
	FORCEINLINE bool operator()( const FSoundInfo& A, const FSoundInfo& B ) const { return A.ComparePathNames( B ); }
};

struct FCompareFSoundInfoByDistance
{
	FORCEINLINE bool operator()( const FSoundInfo& A, const FSoundInfo& B ) const { return A.CompareDistance( B ); }
};

struct FCompareFSoundInfoByClass
{
	FORCEINLINE bool operator()( const FSoundInfo& A, const FSoundInfo& B ) const { return A.CompareClass( B ); }
};

struct FCompareFSoundInfoByWaveInstNum
{
	FORCEINLINE bool operator()( const FSoundInfo& A, const FSoundInfo& B ) const { return A.CompareWaveInstancesNum( B ); }
};

/** draws a property of the given object on the screen similarly to stats */
static void DrawProperty(UCanvas* CanvasObject, UObject* Obj, const FDebugDisplayProperty& PropData, FProperty* Prop, int32 X, int32& Y)
{
#if !UE_BUILD_SHIPPING
	checkSlow(PropData.bSpecialProperty || Prop != NULL);
	checkSlow(Prop == NULL || Obj->GetClass()->IsChildOf(Prop->GetOwnerClass()));

	FCanvas* Canvas = CanvasObject->Canvas;
	FString PropText, ValueText;
	if (!PropData.bSpecialProperty)
	{
		if (PropData.WithinClass != NULL)
		{
			PropText = FString::Printf(TEXT("%s.%s.%s.%s = "), *Obj->GetOutermost()->GetName(), *Obj->GetOuter()->GetName(), *Obj->GetName(), *Prop->GetName());
		}
		else
		{
			PropText = FString::Printf(TEXT("%s.%s.%s = "), *Obj->GetOutermost()->GetName(), *Obj->GetName(), *Prop->GetName());
		}
		if (Prop->ArrayDim == 1)
		{
			Prop->ExportText_InContainer(0, ValueText, Obj, Obj, Obj, PPF_IncludeTransient);
		}
		else
		{
			ValueText += TEXT("(");
			for (int32 i = 0; i < Prop->ArrayDim; i++)
			{
				Prop->ExportText_InContainer(i, ValueText, Obj, Obj, Obj, PPF_IncludeTransient);
				if (i + 1 < Prop->ArrayDim)
				{
					ValueText += TEXT(",");
				}
			}
			ValueText += TEXT(")");
		}
	}
	else
	{
		if (PropData.PropertyName == NAME_None)
		{
			if (PropData.WithinClass != NULL)
			{
				PropText = FString::Printf(TEXT("%s.%s.%s"), *Obj->GetOutermost()->GetName(), *Obj->GetOuter()->GetName(), *Obj->GetName());
			}
			else
			{
				PropText = FString::Printf(TEXT("%s.%s"), *Obj->GetOutermost()->GetName(), *Obj->GetName());
			}
			ValueText = TEXT("");
		}
		else
		{
			if (PropData.WithinClass != NULL)
			{
				PropText = FString::Printf(TEXT("%s.%s.%s.(%s) = "), *Obj->GetOutermost()->GetName(), *Obj->GetOuter()->GetName(), *Obj->GetName(), *PropData.PropertyName.ToString());
			}
			else
			{
				PropText = FString::Printf(TEXT("%s.%s.(%s) = "), *Obj->GetOutermost()->GetName(), *Obj->GetName(), *PropData.PropertyName.ToString());
			}

			if (PropData.PropertyName == NAME_Location)
			{
				AActor *Actor = Cast<AActor>(Obj);
				USceneComponent *Component = Cast<USceneComponent>(Obj);
				if (Actor != nullptr)
				{
					ValueText = Actor->GetActorLocation().ToString();
				}
				else if (Component != nullptr)
				{
					ValueText = Component->GetComponentLocation().ToString();
				}
				else
				{
					ValueText = TEXT("Unsupported for this type");
				}
			}
			else if (PropData.PropertyName == NAME_Rotation)
			{
				AActor *Actor = Cast<AActor>(Obj);
				USceneComponent *Component = Cast<USceneComponent>(Obj);
				if (Actor != nullptr)
				{
					ValueText = Actor->GetActorRotation().ToString();
				}
				else if (Component != nullptr)
				{
					ValueText = Component->GetComponentRotation().ToString();
				}
				else
				{
					ValueText = TEXT("Unsupported for this type");
				}
			}
		}
	}


	int32 CommaIdx = -1;
	bool bDrawPropName = true;
	do
	{
		FString Str = ValueText;
		CommaIdx = ValueText.Find( TEXT(","), ESearchCase::CaseSensitive);
		if( CommaIdx >= 0 )
		{
			Str = ValueText.Left(CommaIdx);
			ValueText.MidInline( CommaIdx+1, MAX_int32, EAllowShrinking::No );
		}

		int32 XL, YL;
		CanvasObject->ClippedStrLen(GEngine->GetSmallFont(), 1.0f, 1.0f, XL, YL, *PropText);
		FTextSizingParameters DrawParams(X, Y, CanvasObject->SizeX - X, 0, GEngine->GetSmallFont());
		TArray<FWrappedStringElement> TextLines;
		CanvasObject->WrapString(DrawParams, X + XL, *Str, TextLines);
		int32 XL2 = XL;
		if (TextLines.Num() > 0)
		{
			XL2 += FMath::TruncToInt(TextLines[0].LineExtent.X);
			for (int32 i = 1; i < TextLines.Num(); i++)
			{
				XL2 = FMath::Max<int32>(XL2, FMath::TruncToInt(TextLines[i].LineExtent.X));
			}
		}
		Canvas->DrawTile(  X, Y, XL2 + 1, YL * FMath::Max<int>(TextLines.Num(), 1), 0, 0, CanvasObject->DefaultTexture->GetSizeX(), CanvasObject->DefaultTexture->GetSizeY(),
			FLinearColor(0.5f, 0.5f, 0.5f, 0.5f), CanvasObject->DefaultTexture->GetResource());
		if( bDrawPropName )
		{
			bDrawPropName = false;
			Canvas->DrawShadowedString( X, Y, *PropText, GEngine->GetSmallFont(), FLinearColor(0.0f, 1.0f, 0.0f));
			if( TextLines.Num() > 1 )
			{
				Y += YL;
			}
		}
		if (TextLines.Num() > 0)
		{
			Canvas->DrawShadowedString( X + XL, Y, *TextLines[0].Value, GEngine->GetSmallFont(), FLinearColor(1.0f, 0.0f, 0.0f));
			for (int32 i = 1; i < TextLines.Num(); i++)
			{
				Canvas->DrawShadowedString( X, Y + YL * i, *TextLines[i].Value, GEngine->GetSmallFont(), FLinearColor(1.0f, 0.0f, 0.0f));
			}
			Y += YL * TextLines.Num();
		}
		else
		{
			Y += YL;
		}
	} while( CommaIdx >= 0 );
#endif // !UE_BUILD_SHIPPING
}

/** Basic timing collation - cannot use stats as these are not enabled in Win32 shipping */
static uint64 StatUnitLastFrameCounter = 0;
static uint32 StatUnitTotalFrameCount = 0;
static float StatUnitTotalFrameTime = 0.0f;
static float StatUnitTotalGameThreadTime = 0.0f;
static float StatUnitTotalRenderThreadTime = 0.0f;
static float StatUnitTotalGPUTime = 0.0f;
static float StatUnitTotalRHITTime = 0.0f;

void UEngine::GetAverageUnitTimes( TArray<float>& AverageTimes )
{
	uint32 FrameCount = 0;
	AverageTimes.AddZeroed( 5 );

	if( StatUnitTotalFrameCount > 0 )
	{
		AverageTimes[0] = StatUnitTotalFrameTime / StatUnitTotalFrameCount;
		AverageTimes[1] = StatUnitTotalGameThreadTime / StatUnitTotalFrameCount;
		AverageTimes[2] = StatUnitTotalGPUTime / StatUnitTotalFrameCount;
		AverageTimes[3] = StatUnitTotalRenderThreadTime / StatUnitTotalFrameCount;
		AverageTimes[4] = StatUnitTotalRHITTime / StatUnitTotalFrameCount;
	}

	/** Reset the counters for the next call */
	StatUnitTotalFrameCount = 0;
	StatUnitTotalFrameTime = 0.0f;
	StatUnitTotalGameThreadTime = 0.0f;
	StatUnitTotalRenderThreadTime = 0.0f;
	StatUnitTotalGPUTime = 0.0f;
	StatUnitTotalRHITTime = 0.0f;
}

void UEngine::SetAverageUnitTimes(float FrameTime, float RenderThreadTime, float GameThreadTime, float GPUFrameTime, float RHITFrameTime)
{
	/** Only record the information once for the current frame */
	if (StatUnitLastFrameCounter != GFrameCounter)
	{
		StatUnitLastFrameCounter = GFrameCounter;

		/** Total times over a play session for averaging purposes */
		StatUnitTotalFrameCount++;
		StatUnitTotalFrameTime += FrameTime;
		StatUnitTotalRenderThreadTime += RenderThreadTime;
		StatUnitTotalGameThreadTime += GameThreadTime;
		StatUnitTotalGPUTime += GPUFrameTime;
		StatUnitTotalRHITTime += RHITFrameTime;
	}
}

FColor UEngine::GetFrameTimeDisplayColor(float FrameTimeMS) const
{
	FColor StatRed(233, 109, 99);		// (Salmon) Red that survives video compression
	FColor StatGreen(127, 202, 159);	// (De York) Green that survives video compression
	FColor StatOrange(244, 186, 112);	// Orange that survives video compression

	const float UnacceptableTime = FEnginePerformanceTargets::GetUnacceptableFrameTimeThresholdMS();
	const float TargetTime = FEnginePerformanceTargets::GetTargetFrameTimeThresholdMS();

	return (FrameTimeMS > UnacceptableTime) ? StatRed : ((FrameTimeMS > TargetTime) ? StatOrange : StatGreen);
}

bool UEngine::ShouldThrottleCPUUsage() const
{
	return false;
}

bool UEngine::AreAllWindowsHidden() const
{
	if (!FSlateApplication::IsInitialized())
	{
		return true;
	}
	const TArray<TSharedRef<SWindow>> AllWindows = FSlateApplication::Get().GetInteractiveTopLevelWindows();

	bool bAllHidden = true;
	for (const TSharedRef<SWindow>& Window : AllWindows)
	{
		if (!Window->IsWindowMinimized() && Window->IsVisible())
		{
			bAllHidden = false;
			break;
		}
	}

	return bAllHidden;
}

#if !UE_BUILD_SHIPPING
/**
*	Renders warnings about the level that should be addressed prior to shipping
*
*	@param World			The World to render stats about
*	@param Viewport			The viewport to render to
*	@param Canvas			Canvas object to use for rendering
*	@param CanvasObject		Optional canvas object for visualizing properties
*	@param MessageX			X Pos to start drawing at on the Canvas
*	@param MessageY			Y Pos to draw at on the Canvas
*
*	@return The Y position in the canvas after the last drawn string
*/
float DrawMapWarnings(UWorld* World, FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, float MessageX, float MessageY)
{
	FCanvasTextItem SmallTextItem(FVector2D(0, 0), FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White);
	SmallTextItem.EnableShadow(FLinearColor::Black);

	const int32 FontSizeY = 20;

	if (GIsTextureMemoryCorrupted)
	{
		FCanvasTextItem TextItem(FVector2D(100, 200), LOCTEXT("OutOfTextureMemory", "RAN OUT OF TEXTURE MEMORY, EXPECT CORRUPTION AND GPU HANGS!"), GEngine->GetMediumFont(), FLinearColor::Red);
		TextItem.EnableShadow(FLinearColor::Black);
		Canvas->DrawItem(TextItem);
	}

	bool bIsPathTracing = false;
	if (Viewport && Viewport->GetClient() && Viewport->GetClient()->GetEngineShowFlags())
	{
		bIsPathTracing = Viewport->GetClient()->GetEngineShowFlags()->PathTracing != 0;
	}

	// Put the messages over fairly far to stay in the safe zone on consoles
	if (World->NumLightingUnbuiltObjects > 0 && !bIsPathTracing)
	{
		SmallTextItem.SetColor(FLinearColor::White);
		// Color unbuilt lighting red if encountered within the last second
		if (FApp::GetCurrentTime() - World->LastTimeUnbuiltLightingWasEncountered < 1)
		{
			SmallTextItem.SetColor(FLinearColor::Red);
		}

		int32 NumLightingScenariosEnabled = 0;

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevels()[LevelIndex];

			if (Level->bIsLightingScenario && Level->bIsVisible)
			{
				NumLightingScenariosEnabled++;
			}
		}

		bool bShouldDisplayCmdHint = false;
		
		if (NumLightingScenariosEnabled > 1)
		{
			SmallTextItem.Text = LOCTEXT("MULTIPLE_LIGHTING_SCENARIO_LEVELS_ENABLED", "MULTIPLE LIGHTING SCENARIO LEVELS ENABLED");		
		}
		else
		{
			// Use 'DumpUnbuiltLightInteractions' to investigate, if lighting is still unbuilt after a lighting build
			SmallTextItem.Text = FText::Format(LOCTEXT("LIGHTING_NEEDS_TO_BE_REBUILT_FMT", "LIGHTING NEEDS TO BE REBUILT ({0} unbuilt {0}|plural(one=object,other=objects))"), World->NumLightingUnbuiltObjects);
			bShouldDisplayCmdHint = true;
		}

		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;

		if (bShouldDisplayCmdHint)
		{
			SmallTextItem.SetColor(FLinearColor::White);
			SmallTextItem.Text = LOCTEXT("UNBUILT_CONSOLE_CMD_HINT", "Run console command 'DumpUnbuiltLightInteractions' to see what is unbuilt");
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}

	if (World->NumUnbuiltReflectionCaptures > 0 && !bIsPathTracing)
	{
		int32 NumLightingScenariosEnabled = 0;

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevels()[LevelIndex];

			if (Level->bIsLightingScenario && Level->bIsVisible)
			{
				NumLightingScenariosEnabled++;
			}
		}

		if (NumLightingScenariosEnabled <= 1)
		{
			SmallTextItem.SetColor(FLinearColor::White);
			SmallTextItem.Text = FText::Format(LOCTEXT("REFLECTION_CAPTURES_NEED_TO_BE_REBUILT_FMT", "REFLECTION CAPTURES NEED TO BE REBUILT ({0} unbuilt)"), World->NumUnbuiltReflectionCaptures);		

			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}

	if (World->WorldComposition)
	{
		SmallTextItem.SetColor(FLinearColor::Red);
		SmallTextItem.Text = LOCTEXT("WORLD_COMPOSITION_DEPRECATED_FMT", "WORLD COMPOSITION IS DEPRECATED, CONSIDER CONVERTING TO WORLD PARTITION");

		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Check HLOD clusters and show warning if unbuilt
	{
		static double LastCheckTime = 0;

		double TimeNow = FPlatformTime::Seconds();

		// Recheck every 20 secs to handle the case where levels may have been
		// Streamed in/out
		if ((TimeNow - LastCheckTime) > 20)
		{
			LastCheckTime = TimeNow;
			GUnbuiltHLODCount = 0;
			for (TActorIterator<ALODActor> HLODIt(World); HLODIt; ++HLODIt)
			{
				if (!HLODIt->IsBuilt() && HLODIt->HasValidLODChildren())
				{
					++GUnbuiltHLODCount;
				}
			}
		}

		if (GUnbuiltHLODCount)
		{
			SmallTextItem.SetColor(FLinearColor::Red);
			SmallTextItem.Text = FText::Format(LOCTEXT("HLOD_CLUSTERS_NEED_TO_BE_REBUILT_FMT", "HLOD CLUSTERS NEED TO BE REBUILT ({0} unbuilt {0}|plural(one=object,other=objects))"), GUnbuiltHLODCount);
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}

	if (UWorldPartitionHLODRuntimeSubsystem* HLODSubsystem = World->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>())
	{
		uint32 NumOutdatedHLODObjects = HLODSubsystem->GetNumOutdatedHLODObjects();
		if (NumOutdatedHLODObjects)
		{
			SmallTextItem.SetColor(FLinearColor::Yellow);
			SmallTextItem.Text = FText::Format(LOCTEXT("OUTDATED_HLOD_ACTORS_FMT", "Outdated HLODs detected, HLODs need to be rebuilt ({0} outdated)"), NumOutdatedHLODObjects);
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}
#endif

	if (World->NumTextureStreamingUnbuiltComponents > 0 || World->NumTextureStreamingDirtyResources > 0)
	{
		SmallTextItem.SetColor(FLinearColor::Red);
		SmallTextItem.Text = FText::Format(LOCTEXT("TEXTURE_STREAMING_NEEDS_TO_BE_REBUILT_FMT", "TEXTURE STREAMING NEEDS TO BE REBUILT ({0} {0}|plural(one=component,other=components), {1} {1}|plural(one=\"resource ref\",other=\"resource refs\"))"), World->NumTextureStreamingUnbuiltComponents, World->NumTextureStreamingDirtyResources);
		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}

	if (FPlatformProperties::SupportsTextureStreaming() && IStreamingManager::Get().IsTextureStreamingEnabled())
	{
		auto MemOver = IStreamingManager::Get().GetTextureStreamingManager().GetMemoryOverBudget();
		if (MemOver > 0)
		{
			SmallTextItem.SetColor(FLinearColor::Red);
			SmallTextItem.Text = FText::Format(LOCTEXT("TEXTURE_STREAMING_POOL_OVER_BUDGET_FMT", "TEXTURE STREAMING POOL OVER {0} BUDGET"), FText::AsMemory(MemOver));
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}

	if (World->bKismetScriptError)
	{
		SmallTextItem.Text = LOCTEXT("BlueprintInLevelHadCompileErrorMessage", "BLUEPRINT COMPILE ERROR");
		SmallTextItem.SetColor(FLinearColor::Red);
		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}

	SmallTextItem.SetColor(FLinearColor::White);

	for (IAssetCompilingManager* AssetCompilingManager : FAssetCompilingManager::Get().GetRegisteredManagers())
	{
		if (AssetCompilingManager->GetNumRemainingAssets() > 0)
		{
			// Always show asset plural form when displaying preparation status
			FText AssetTypePlural = FText::Format(AssetCompilingManager->GetAssetNameFormat(), FText::AsNumber(100));
		SmallTextItem.SetColor(FLinearColor::White);
			SmallTextItem.Text = FText::Format(LOCTEXT("AssetCompilingFmt", "Preparing {0} ({1})"), AssetTypePlural, AssetCompilingManager->GetNumRemainingAssets());
		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}
	}

	if (World->bIsLevelStreamingFrozen)
	{
		SmallTextItem.Text = LOCTEXT("Levelstreamingfrozen", "Level streaming frozen...");
		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	extern bool GIsPrepareMapChangeBroken;
	if (GIsPrepareMapChangeBroken)
	{
		SmallTextItem.Text = LOCTEXT("PrepareMapChangeError", "PrepareMapChange had a bad level name! Check the log (tagged with PREPAREMAPCHANGE) for info");
		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		MessageY += FontSizeY;
	}
#endif

	// ask any delegates for extra messages
	if (FCoreDelegates::OnGetOnScreenMessages.IsBound())
	{
		FCoreDelegates::FSeverityMessageMap ExtraMessages;
		FCoreDelegates::OnGetOnScreenMessages.Broadcast(ExtraMessages);

		// draw them all!
		for (auto It = ExtraMessages.CreateConstIterator(); It; ++It)
		{
			SmallTextItem.Text = It.Value();
			switch (It.Key())
			{
			case FCoreDelegates::EOnScreenMessageSeverity::Info:
				SmallTextItem.SetColor(FLinearColor::White);
				break;
			case FCoreDelegates::EOnScreenMessageSeverity::Warning:
				SmallTextItem.SetColor(FLinearColor::Yellow);
				break;
			case FCoreDelegates::EOnScreenMessageSeverity::Error:
				SmallTextItem.SetColor(FLinearColor::Red);
				break;
			}

			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
	}


	return MessageY;
}
#endif //!UE_BUILD_SHIPPING

/**
*	Renders warnings about the level that should be addressed prior to shipping
*
*	@param World			The World to render stats about
*	@param Viewport			The viewport to render to
*	@param Canvas			Canvas object to use for rendering
*	@param CanvasObject		Optional canvas object for visualizing properties
*	@param MessageX			X Pos to start drawing at on the Canvas
*	@param MessageY			Y Pos to draw at on the Canvas
*
*	@return The Y position in the canvas after the last drawn string
*/
float UEngine::DrawOnscreenDebugMessages(UWorld* World, FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, float MessageX, float MessageY)
{
	int32 YPos = MessageY;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static TFrameValue<bool> HasUpdatedScreenDebugMessages;

	const float InverseDPI = 1.f / Canvas->GetDPIScale();
	const int32 MaxYPos = CanvasObject ? CanvasObject->SizeY : 700;
	{
		// Because some components add their message in concurrent work, we need a CS here.
		UE::TScopeLock ScopeLock(GOnScreenMessageCS);

		// Calculate the final amount of vertices for the batch or else we will end up resizing many times as the batching system only reserves the first draw call.
		TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();
		if (FontCache.IsValid() && FontCache->GetNumAtlasPages() > 0)
		{
			int32 TotalCharacterCount = 0;
			for (const FScreenMessageString& Message : PriorityScreenMessages)
			{
				TotalCharacterCount += Message.ScreenMessage.Len();
			}

			for (const TTuple<int32, FScreenMessageString>& Tuple : ScreenMessages)
			{
				TotalCharacterCount += Tuple.Value.ScreenMessage.Len();
			}

			ISlateFontTexture* SlateFontTexture = FontCache->GetFontTexture(0);
			const FTexture* FontTexture = SlateFontTexture->GetEngineTexture();
			FBatchedElements* BatchedElements = Canvas->GetBatchedElements(FCanvas::ET_Triangle, nullptr, FontTexture, SE_BLEND_TranslucentAlphaOnly, FDepthFieldGlowInfo(), false);
			check(BatchedElements);

			const int32 TextEffects = 1; // Shadow
			const int32 NumTris = 2 * (TextEffects + 1);
			const int32 NumVerts = 4 * (TextEffects + 1);
			BatchedElements->ReserveTriangles(TotalCharacterCount * NumTris, FontTexture, SE_BLEND_TranslucentAlphaOnly);
			BatchedElements->ReserveVertices(TotalCharacterCount * NumVerts);
		}

		if (PriorityScreenMessages.Num() > 0)
		{
			FCanvasTextItem MessageTextItem(FVector2D(0, 0), FText::GetEmpty(), GetSmallFont(), FLinearColor::White);
			MessageTextItem.EnableShadow(FLinearColor::Black);
			for (int32 PrioIndex = PriorityScreenMessages.Num() - 1; PrioIndex >= 0; PrioIndex--)
			{
				FScreenMessageString& Message = PriorityScreenMessages[PrioIndex];
				if (YPos < MaxYPos)
				{
					MessageTextItem.Text = FText::FromString(Message.ScreenMessage);
					MessageTextItem.SetColor(Message.DisplayColor);
					MessageTextItem.Scale = Message.TextScale;
					Canvas->DrawItem(MessageTextItem, FVector2D(MessageX, YPos));
					YPos += MessageTextItem.DrawnSize.Y * InverseDPI * 1.15f;
				}
				if (!HasUpdatedScreenDebugMessages.IsSet())
				{
					Message.CurrentTimeDisplayed += World->GetDeltaSeconds();
					if (Message.CurrentTimeDisplayed >= Message.TimeToDisplay)
					{
						PriorityScreenMessages.RemoveAt(PrioIndex);
					}
				}
			}
		}

		if (ScreenMessages.Num() > 0)
		{
			FCanvasTextItem MessageTextItem(FVector2D(0, 0), FText::GetEmpty(), GetSmallFont(), FLinearColor::White);
			MessageTextItem.EnableShadow(FLinearColor::Black);
			for (TMap<int32, FScreenMessageString>::TIterator MsgIt(ScreenMessages); MsgIt; ++MsgIt)
			{
				FScreenMessageString& Message = MsgIt.Value();
				if (YPos < MaxYPos)
				{
					MessageTextItem.Text = FText::FromString(Message.ScreenMessage);
					MessageTextItem.SetColor(Message.DisplayColor);
					MessageTextItem.Scale = Message.TextScale;
					Canvas->DrawItem(MessageTextItem, FVector2D(MessageX, YPos));
					YPos += MessageTextItem.DrawnSize.Y * InverseDPI * 1.15f;
				}
				if (!HasUpdatedScreenDebugMessages.IsSet())
				{
					Message.CurrentTimeDisplayed += World->GetDeltaSeconds();
					if (Message.CurrentTimeDisplayed >= Message.TimeToDisplay)
					{
						MsgIt.RemoveCurrent();
					}
				}
			}
		}
	}
	// Flag variable that the update has already been done this frame
	HasUpdatedScreenDebugMessages = true;
#endif //#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	return YPos;
}


/**
*	Renders stats
*
*  @param World			The World to render stats about
*	@param Viewport			The viewport to render to
*	@param Canvas			Canvas object to use for rendering
*	@param CanvasObject		Optional canvas object for visualizing properties
*	@param DebugProperties	List of properties to visualize (in/out)
*	@param ViewLocation		Location of camera
*	@param ViewRotation		Rotation of camera
*/
void DrawStatsHUD( UWorld* World, FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, TArray<FDebugDisplayProperty>& DebugProperties, const FVector& ViewLocation, const FRotator& ViewRotation )
{
	LLM_SCOPE(ELLMTag::Stats);

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "DrawStatsHUD" ), STAT_DrawStatsHUD, STATGROUP_StatSystem );
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(DebugHUD);

	// We cannot draw without a world or a canvas.
	if (World == NULL || Canvas == NULL)
	{
		return;
	}

	float DPIScale = Canvas->GetDPIScale();
	
	FIntPoint TextureSize = Canvas->GetRenderTarget()->GetSizeXY();

	//@todo joeg: Move this stuff to a function, make safe to use on consoles by
	// respecting the various safe zones, and make it compile out.
	const int32 FPSXOffset	= (FPlatformProperties::SupportsWindowedMode() ? 110 : 250);
	const int32 StatsXOffset = 100;// FPlatformProperties::SupportsWindowedMode() ? 4 : 100;

	static const int32 MessageStartY = GIsEditor ? 45 : 100; // Account for safe frame
	int32 MessageY = MessageStartY;

	const FVector2D FontScale(1, 1);
	const int32 FontSizeY = 20;
#if !UE_BUILD_SHIPPING
	const int32 MessageX = 40;
	FCanvasTextItem SmallTextItem(FVector2D(0, 0), FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White);
	SmallTextItem.Scale = FontScale;
	SmallTextItem.EnableShadow(FLinearColor::Black);

	bool bShowScreenMessages = !GIsHighResScreenshot && !GIsDumpingMovie && GAreScreenMessagesEnabled;
	if (bShowScreenMessages)
	{
		// Draw map warnings?
		if (!GEngine->bSuppressMapWarnings)
		{
			MessageY = DrawMapWarnings(World, Viewport, Canvas, CanvasObject, MessageX, MessageY);
		}
#if ENABLE_AUDIO_DEBUG
		MessageY = Audio::FAudioDebugger::DrawDebugStats(*World, Viewport, Canvas, MessageY);
#endif // ENABLE_AUDIO_DEBUG

#if ENABLE_VISUAL_LOG
		if (FVisualLogger::Get().IsRecording() || FVisualLogger::Get().IsRecordingOnServer())
		{
			int32 XSize;
			int32 YSize;
			FText Text = LOCTEXT("VisLogRecordingActive", "VisLog recording active");
			StringSize(GEngine->GetSmallFont(), XSize, YSize, *Text.ToString());

			SmallTextItem.Position = FVector2D((int32)TextureSize.X - XSize - 16, 36);
			SmallTextItem.Text = Text;
			SmallTextItem.SetColor(FLinearColor::Red);
			SmallTextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(SmallTextItem);
			SmallTextItem.SetColor(FLinearColor::White);
		}
#endif

#if STATS
		if (GRHIGlobals.IsDebugLayerEnabled)
		{
			SmallTextItem.SetColor(FLinearColor::Red);
			SmallTextItem.Text = LOCTEXT("GRAPHICSDEBUGDEVICEWARNING", "PROFILING WITH DEBUG DEVICE ON!");
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
			SmallTextItem.SetColor(FLinearColor::White);
		}

		if (FThreadStats::IsCollectingData())
		{
			SmallTextItem.SetColor(FLinearColor::Red);
			if (!GEngine->bDisableAILogging)
			{
				SmallTextItem.Text = LOCTEXT("AIPROFILINGWARNING", "PROFILING WITH AI LOGGING ON!");
				Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
				MessageY += FontSizeY;
			}
			if (GShouldVerifyGCAssumptions)
			{
				SmallTextItem.Text = LOCTEXT("GCPROFILINGWARNING", "PROFILING WITH GC VERIFY ON!");
				Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
				MessageY += FontSizeY;
			}

#if UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8
			const bool bIsStatsFileActive = FCommandStatsFile::Get().IsStatFileActive();
			if (bIsStatsFileActive)
			{
				SmallTextItem.SetColor(FLinearColor::White);
				SmallTextItem.Text = FCommandStatsFile::Get().GetFileMetaDesc();
				Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
				MessageY += FontSizeY;
			}
#endif // UE_ENABLE_STATS_FILE_DEPRECATED_IN_5_8

#if ENABLE_LOW_LEVEL_MEM_TRACKER
			if (FLowLevelMemTracker::Get().IsEnabled() && !FPlatformMemory::IsExtraDevelopmentMemoryAvailable())
			{
				SmallTextItem.Text = LOCTEXT("MEMPROFILINGWARNINGLLM", "LLM enabled without Debug Memory enabled!");
				Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
				MessageY += FontSizeY;
			}
#endif
		}
#endif // STATS

		if (CVarAllowExtraDevMemoryMessageDisplayed.GetValueOnGameThread() && FPlatformMemory::IsExtraDevelopmentMemoryAvailable())
		{
			SmallTextItem.SetColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
			SmallTextItem.Text = LOCTEXT("LLMWARNING", "WARNING: Running with Debug Memory Enabled!");
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}

#if FRAMEPRO_ENABLED
		if (FFrameProProfiler::IsFrameProRecording())
		{
			SmallTextItem.SetColor(FLinearColor(0.0f, 1.0f, 0.0f, 1.0f));
			SmallTextItem.Text = LOCTEXT("FrameProRecording", "FramePro Recording");

			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
			MessageY += FontSizeY;
		}
#endif // FRAMEPRO_ENABLED

#if !(UE_BUILD_TEST)
		if (GEngine->bEnableOnScreenDebugMessagesDisplay && GEngine->bEnableOnScreenDebugMessages)
		{
			MessageY = GEngine->DrawOnscreenDebugMessages(World, Viewport, Canvas, CanvasObject, MessageX, MessageY);
		}
#endif // UE_BUILD_TEST

		TArray<FText> PlatformScreenWarnings;

		if (FPlatformMisc::GetPlatformScreenWarnings(PlatformScreenWarnings))
		{
			SmallTextItem.SetColor(FLinearColor::Red);

			for (FText& PlatformWarning : PlatformScreenWarnings)
			{
				SmallTextItem.Text = PlatformWarning;
				Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
				MessageY += FontSizeY;
			}
		}

		// Only output disable message if there actually were any
		if (MessageY != MessageStartY)
		{
			SmallTextItem.SetColor(FLinearColor(.05f, .05f, .05f, .2f));
			SmallTextItem.Text = LOCTEXT("DisableAllScreenMessagesToSuppress", "'DisableAllScreenMessages' to suppress");
			Canvas->DrawItem(SmallTextItem, FVector2D(MessageX + 50, MessageY));
			MessageY += 16;
		}

		// Draw the specified CVars values
		{
			if (GDisplayCVarListExecHelper.ConsoleVariablesToVisualize.Num())
			{
				for (const auto& CVarToVisualize : GDisplayCVarListExecHelper.ConsoleVariablesToVisualize)
				{
					const FString CurrentValue = CVarToVisualize.Value->GetString();
					const FString DefaultValue = CVarToVisualize.Value->GetDefaultValue();
					bool bSame = true;

					// Floats sometimes return true erroneously because they can be stringified as e.g '1' or '1.0' by different functions.
					if (CVarToVisualize.Value->IsVariableFloat())
					{
						const float A = CVarToVisualize.Value->GetFloat();
						const float B = FCString::Atof(*DefaultValue);
						bSame = FMath::IsNearlyEqual(A, B);
					}
					else
					{
						bSame = CurrentValue.Equals(DefaultValue);
					}

					Canvas->DrawShadowedString(MessageX, MessageY, *FString::Printf(TEXT("%s : %s"), *CVarToVisualize.Key, *CurrentValue), GEngine->GetSmallFont(), bSame ? FLinearColor::White : FLinearColor::Yellow);
 					MessageY += FontSizeY;
				}
			}
		}
	}

#if CSV_PROFILER
	if ( ( bShowScreenMessages || CVarCsvAlwaysShowFrameCount.GetValueOnGameThread() ) && ( FCsvProfiler::Get()->IsCapturing() || FCsvProfiler::Get()->IsWritingFile() ) )
	{
		if (FCsvProfiler::Get()->IsWritingFile())
		{
			SmallTextItem.SetColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f));
			SmallTextItem.Text = FText::Format(LOCTEXT("CsvProfilerWriteFmt", "CsvProfiler, please wait, writing file... Total frames: {0}"), FCsvProfiler::Get()->GetCaptureFrameNumber());
		}
		else
		{
			SmallTextItem.SetColor(FLinearColor(0.0f, 1.0f, 0.0f, 1.0f));
			SmallTextItem.Text = FText::Format(LOCTEXT("CsvProfilerFrameFmt", "CsvProfiler frame: {0}"), FCsvProfiler::Get()->GetCaptureFrameNumber());
		}

		MessageY += 250.0f;

		// Swap the shadow for a background to improve readability
		static FVector2D PrevDrawnSize(0.f); // one frame behind, but much faster than calling GetTextSize
		const FLinearColor BackgroundColor = FLinearColor(0.02f, 0.02f, 0.02f, 0.88f); // color used by stat group tables
		SmallTextItem.DisableShadow();
		Canvas->DrawTile(MessageX - 1.0f, MessageY - 1.0f, PrevDrawnSize.X + 2.0f, PrevDrawnSize.Y + 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, BackgroundColor);

		Canvas->DrawItem(SmallTextItem, FVector2D(MessageX, MessageY));
		SmallTextItem.EnableShadow(FLinearColor::Black); // reset
		PrevDrawnSize = SmallTextItem.DrawnSize;
		MessageY += FontSizeY;
	}
#endif // CSV_PROFILER

#endif // UE_BUILD_SHIPPING 

	{
		int32 RightSideX = ((CanvasObject) ? CanvasObject->SizeX : TextureSize.X) / Canvas->GetDPIScale() - FPSXOffset;
		int32 RightSideY = FMath::TruncToInt(TextureSize.Y * 0.20f) / Canvas->GetDPIScale();

		// give the viewport first shot at drawing stats
		RightSideY = Viewport->DrawStatsHUD(Canvas, StatsXOffset, RightSideY);

		// Named events are enabled through multiple paths, so draw them here
		RightSideY = GEngine->RenderNamedEventsEnabled(Canvas, RightSideX, RightSideY);

		// Render all the simple stats
		GEngine->RenderEngineStats(World, Viewport, Canvas, StatsXOffset /*LHSX*/, MessageY /*InOutLHSY*/, RightSideX /*RHSX*/, RightSideY /*InOutRHSY*/, &ViewLocation, &ViewRotation);

#if STATS
		extern void RenderStats(FViewport* Viewport, class FCanvas* Canvas, int32 X, int32 Y, int32 SizeX);

		int32 PixelSizeX = CanvasObject != nullptr ? CanvasObject->CachedDisplayWidth - CanvasObject->SafeZonePadX * 2 : TextureSize.X;

		RenderStats( Viewport, Canvas, StatsXOffset, MessageY, FMath::FloorToInt(PixelSizeX / Canvas->GetDPIScale()));
#endif
	}

	// Use the new Viewport stats subsystem to draw any additional items that the user may want to
	{
		if (UViewportStatsSubsystem* ViewportSubsystem = World->GetSubsystem<UViewportStatsSubsystem>())
		{
			ViewportSubsystem->Draw(Viewport, Canvas, CanvasObject, MessageY);
		}
	}

	// draw debug properties
#if !UE_BUILD_SHIPPING

	auto DrawDebugPropertiesForWorld = [Canvas, CanvasObject, &DebugProperties](UWorld* DebugWorld, int32& X, int32& Y)
	{
		// construct a list of objects relevant to "getall" type elements, so that we only have to do the object iterator once
		// we do the iterator each frame so that new objects will show up immediately
		struct FDebugClass
		{
			UClass* Class;
			UClass* WithinClass;
			FDebugClass(UClass* InClass, UClass* InWithinClass)
				: Class(InClass), WithinClass(InWithinClass)
			{}
		};
		TArray<FDebugClass> DebugClasses;
		DebugClasses.Reserve(DebugProperties.Num());
		for (int32 i = 0; i < DebugProperties.Num(); i++)
		{
			if (IsValid(DebugProperties[i].Obj))
			{
				UClass* Cls = Cast<UClass>(DebugProperties[i].Obj);
				if (Cls != NULL)
				{
					new(DebugClasses) FDebugClass(Cls, DebugProperties[i].WithinClass);
				}
			}
			else
			{
				// invalid, object was destroyed, etc. so remove the entry
				DebugProperties.RemoveAt(i--, 1);
			}
		}

		TSet<UObject*> RelevantObjects;
		for (const FDebugClass& DebugClass : DebugClasses)
		{
			if (DebugClass.Class)
			{
				TArray<UObject*> DebugObjectsOfClass;
				const bool bIncludeDerivedClasses = true;
				GetObjectsOfClass(DebugClass.Class, DebugObjectsOfClass, bIncludeDerivedClasses);
				for (UObject* Obj : DebugObjectsOfClass)
				{
					if (!Obj)
					{
						continue;
					}

					if (Obj->GetWorld() && Obj->GetWorld() != DebugWorld)
					{
						continue;
					}

					if ( !Obj->IsTemplate() &&
						(DebugClass.WithinClass == NULL || (Obj->GetOuter() != NULL && Obj->GetOuter()->GetClass()->IsChildOf(DebugClass.WithinClass))) )
					{
						RelevantObjects.Add(Obj);
					}
				}
			}
		}
		int32 MaxY = int32(Canvas->GetRenderTarget()->GetSizeXY().Y);
		for (int32 i = 0; i < DebugProperties.Num() && Y < MaxY; i++)
		{
			// we removed entries with invalid Obj above so no need to check for that here
			UClass* Cls = Cast<UClass>(DebugProperties[i].Obj);
			if (Cls != NULL)
			{
				FProperty* Prop = FindFProperty<FProperty>(Cls, DebugProperties[i].PropertyName);
				if (Prop != NULL || DebugProperties[i].bSpecialProperty)
				{
					// getall
					for (UObject* RelevantObject : RelevantObjects)
					{
						if ( RelevantObject->IsA(Cls) && IsValidChecked(RelevantObject) &&
							(DebugProperties[i].WithinClass == NULL || (RelevantObject->GetOuter() != NULL && RelevantObject->GetOuter()->GetClass()->IsChildOf(DebugProperties[i].WithinClass))) )
						{
							DrawProperty(CanvasObject, RelevantObject, DebugProperties[i], Prop, X, Y);
						}
					}
				}
				else
				{
					// invalid entry
					DebugProperties.RemoveAt(i--, 1);
				}
			}
			else
			{
				FProperty* Prop = FindFProperty<FProperty>(DebugProperties[i].Obj->GetClass(), DebugProperties[i].PropertyName);
				if (Prop != NULL || DebugProperties[i].bSpecialProperty)
				{
					DrawProperty(CanvasObject, DebugProperties[i].Obj, DebugProperties[i], Prop, X, Y);
				}
				else
				{
					DebugProperties.RemoveAt(i--, 1);
				}
			}
		}
	};

	// draw starting in the top left
	int32 X = StatsXOffset;
	int32 Y = FPlatformProperties::SupportsWindowedMode() ? 20 : 40;

	DrawDebugPropertiesForWorld(World, X, Y);
	
	// Draw properties for server world actors in PIE
	if (GIsEditor && World->WorldType == EWorldType::PIE && World->GetNetMode() == NM_Client && DebugProperties.Num() > 0)
	{
		for( TObjectIterator<UWorld> It; It; ++It)
		{
			UWorld* ServerWorld = *It;
			if (ServerWorld->WorldType == EWorldType::PIE && ServerWorld->GetNetMode() == NM_DedicatedServer)
			{
				FString WorldDesc = "[Server]";

				FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(ServerWorld);
				if (!WorldContext.CustomDescription.IsEmpty())
				{
					WorldDesc += TEXT("[") + WorldContext.CustomDescription + TEXT("]");
				}

				Y += 20;
				Y += Canvas->DrawShadowedString( X, Y, *WorldDesc, GEngine->GetSmallFont(), FLinearColor::Gray);
				DrawDebugPropertiesForWorld(ServerWorld, X, Y);
			}
		}
	}

#endif
}


/**
* Stats objects for Engine
*/
DEFINE_STAT(STAT_GameEngineTick);
DEFINE_STAT(STAT_GameViewportTick);
DEFINE_STAT(STAT_RedrawViewports);
DEFINE_STAT(STAT_UpdateLevelStreaming);
DEFINE_STAT(STAT_RHITickTime);
DEFINE_STAT(STAT_IntentionalHitch);
DEFINE_STAT(STAT_DeferredTickTime);

/** Input stat */
DEFINE_STAT(STAT_InputLatencyTime);

/** HUD stat */
DEFINE_STAT(STAT_HudTime);

/** Static mesh tris rendered */
DEFINE_STAT(STAT_StaticMeshTriangles);

/** Skeletal stats */
DEFINE_STAT(STAT_SkinningTime);
DEFINE_STAT(STAT_UpdateClothVertsTime);
DEFINE_STAT(STAT_SkelMeshTriangles);
DEFINE_STAT(STAT_SkelMeshDrawCalls);
DEFINE_STAT(STAT_CPUSkinVertices);
DEFINE_STAT(STAT_GPUSkinVertices);

/** Unit times */
DEFINE_STAT(STAT_UnitFrame);
DEFINE_STAT(STAT_UnitGame);
DEFINE_STAT(STAT_UnitRender);
DEFINE_STAT(STAT_UnitRHIT);
DEFINE_STAT(STAT_UnitGPU);


UFont* GetStatsFont()
{
	return GEngine->GetSmallFont();
}

namespace FFrameEndSync
{
	void Sync(bool bFullSync)
	{
		Sync(bFullSync
			? EFlushMode::Threads
			: EFlushMode::EndFrame);
	}
}

FString appGetStartupMap(const TCHAR* CommandLine)
{
	FURL DefaultURL;
	DefaultURL.LoadURLConfig( TEXT("DefaultPlayer"), GGameIni );

	// convert commandline to a URL
	FString Error;
	TCHAR Parm[4096]=TEXT("");

#if UE_BUILD_SHIPPING
	// In shipping don't allow an override
	CommandLine = NULL;
#endif // UE_BUILD_SHIPPING

	const TCHAR* Tmp = CommandLine ? CommandLine : TEXT("");
	if (!FParse::Token(Tmp, Parm, UE_ARRAY_COUNT(Parm), 0) || Parm[0] == '-')
	{
		const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
		FCString::Strcpy(Parm, *(GameMapsSettings->GetGameDefaultMap() + GameMapsSettings->LocalMapOptions));
	}
	FURL URL(&DefaultURL, Parm, TRAVEL_Partial);

	// strip off extension of the map if there is one
	return FPaths::GetBaseFilename(URL.Map);
}

#if WITH_EDITOR
FOnSwitchWorldForPIE FScopedConditionalWorldSwitcher::SwitchWorldForPIEDelegate;

FScopedConditionalWorldSwitcher::FScopedConditionalWorldSwitcher( FViewportClient* InViewportClient )
	: ViewportClient( InViewportClient )
	, OldWorld( nullptr )
{
	UWorld* InWorld = nullptr;
	if (InViewportClient)
	{
		InWorld = InViewportClient->GetWorld();
	}

	ConditionalSwitchWorld( ViewportClient, InWorld );
}

FScopedConditionalWorldSwitcher::FScopedConditionalWorldSwitcher(UWorld* InWorld)
	: ViewportClient( nullptr )
	, OldWorld( nullptr )
{
	if (InWorld)
	{
		ViewportClient = InWorld->GetGameViewport();
	}

	ConditionalSwitchWorld( ViewportClient, InWorld );
}

void FScopedConditionalWorldSwitcher::ConditionalSwitchWorld( FViewportClient* InViewportClient, UWorld* InWorld )
{
	// If we don't have a currently active world, we can't safely switch worlds because it won't get restored so do nothing
	if( GIsEditor && GWorld )
	{
		if ( InWorld && InWorld->WorldType == EWorldType::PIE && !GIsPlayInEditorWorld )
		{
			// We don't want to restore using Viewport client so clear it
			ViewportClient = nullptr;

			OldWorld = GWorld;
			const bool bSwitchToPIEWorld = true;

			// First check if we are being told to switch to a PIE world, if so always switch regardless of viewport
			SwitchWorldForPIEDelegate.ExecuteIfBound( bSwitchToPIEWorld, InWorld );
		}
		else if( ViewportClient && ViewportClient == GEngine->GameViewport && !GIsPlayInEditorWorld )
		{
			OldWorld = GWorld; 
			const bool bSwitchToPIEWorld = true;

			// If this is the main game viewport delegate will handle finding new world
			SwitchWorldForPIEDelegate.ExecuteIfBound( bSwitchToPIEWorld, nullptr );
		}
		else if( ViewportClient )
		{
			// Tell the viewport client to set the correct world and store what the world used to be
			OldWorld = ViewportClient->ConditionalSetWorld();
		}
	}
}

FScopedConditionalWorldSwitcher::~FScopedConditionalWorldSwitcher()
{
	// Only switch in the editor and if we made a switch (OldWorld not null)
	if( GIsEditor && OldWorld )
	{
		if( ViewportClient && ViewportClient == GEngine->GameViewport && GIsPlayInEditorWorld )
		{
			const bool bSwitchToPIEWorld = false;
			// Delegate must be valid
			SwitchWorldForPIEDelegate.ExecuteIfBound( bSwitchToPIEWorld, nullptr );
		}
		else if( ViewportClient )
		{
			// Tell the viewport client to restore the old world
			ViewportClient->ConditionalRestoreWorld( OldWorld );
		}
		else if( GIsPlayInEditorWorld )
		{
			// No viewport so always restore to old world
			const bool bSwitchToPIEWorld = false;
			SwitchWorldForPIEDelegate.ExecuteIfBound( bSwitchToPIEWorld, OldWorld );
		}
	}
}

#endif

void UEngine::OverrideSelectedMaterialColor( const FLinearColor& OverrideColor )
{
	bIsOverridingSelectedColor = true;
	SelectedMaterialColorOverride = OverrideColor;
}

void UEngine::RestoreSelectedMaterialColor()
{
	bIsOverridingSelectedColor = false;
}

void UEngine::GetDynamicResolutionCurrentStateInfos(FDynamicResolutionStateInfos& OutInfos) const
{
	OutInfos.Status = EDynamicResolutionStatus::Unsupported;
	OutInfos.ResolutionFractionApproximations.SetAll(-1.0f);
	OutInfos.ResolutionFractionUpperBounds.SetAll(-1.0f);

	#if WITH_DYNAMIC_RESOLUTION
	{
		float ForceResolutionFraction = 0.0f;

		#if !UE_BUILD_SHIPPING
			ForceResolutionFraction = CVarForceDynamicResScreenPercentage.GetValueOnGameThread() / 100.0f;
		#endif

		if (!DynamicResolutionState.IsValid())
		{
			return;
		}
		#if !UE_BUILD_SHIPPING
		else if (ForceResolutionFraction > 0)
		{
			OutInfos.Status = EDynamicResolutionStatus::DebugForceEnabled;
			OutInfos.ResolutionFractionApproximations = DynamicResolutionState->GetResolutionFractionsApproximation();
			OutInfos.ResolutionFractionApproximations[GDynamicPrimaryResolutionFraction] = ForceResolutionFraction;
			OutInfos.ResolutionFractionUpperBounds = DynamicResolutionState->GetResolutionFractionsUpperBoundBudgetValue();

			// If dynamic upper bound is enabled, set the upper bound to the forced resolution to properly simulate that resolution (otherwise clears/resolves may not be representative)
			static auto LocalCVarUpperBoundQuantization = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DynamicRes.UpperBoundQuantization"));
			if (LocalCVarUpperBoundQuantization && LocalCVarUpperBoundQuantization->GetFloat() > 0.0f)
			{
				OutInfos.ResolutionFractionUpperBounds[GDynamicPrimaryResolutionFraction] = ForceResolutionFraction;
			}
		}
		#endif
		else if (!DynamicResolutionState->IsSupported())
		{
			return;
		}
		else if (DynamicResolutionState->IsEnabled())
		{
			OutInfos.Status = EDynamicResolutionStatus::Enabled;
			OutInfos.ResolutionFractionApproximations = DynamicResolutionState->GetResolutionFractionsApproximation();
			OutInfos.ResolutionFractionUpperBounds = DynamicResolutionState->GetResolutionFractionsUpperBound();
		}
		else if (bIsDynamicResolutionPaused)
		{
			OutInfos.Status = EDynamicResolutionStatus::Paused;
		}
		else
		{
			OutInfos.Status = EDynamicResolutionStatus::Disabled;
		}

		// Min to ensure consistency when changing ResolutionFractionUpperBound setting.
		for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
		{
			const DynamicRenderScaling::FBudget& Budget = **BudgetIt;
			OutInfos.ResolutionFractionApproximations[Budget] = FMath::Min(
				OutInfos.ResolutionFractionApproximations[Budget],
				OutInfos.ResolutionFractionUpperBounds[Budget]);
		}
	}
	#endif // !WITH_DYNAMIC_RESOLUTION
}

void UEngine::PauseDynamicResolution()
{
	#if WITH_DYNAMIC_RESOLUTION
		ensureMsgf(!(DynamicResolutionState->IsEnabled() && bIsDynamicResolutionPaused),
			TEXT("Looks like the dynamic resolution state has enabled itself."));

		// Disable the state if it is enabled.
		if (DynamicResolutionState->IsEnabled())
		{
			DynamicResolutionState->SetEnabled(false);
		}
		bIsDynamicResolutionPaused = true;
	#endif // !UE_SERVER
}

#if WITH_DYNAMIC_RESOLUTION
bool UEngine::ShouldEnableDynamicResolutionState() const
{
	// If dynamic resolution is paused, the state will have to be disabled in any cases.
	if (bIsDynamicResolutionPaused)
	{
		return false;
	}

	int32 OperationMode = CVarDynamicResOperationMode.GetValueOnGameThread();
	
	// Whether dynamic resolution is allowed to be enabled.
	bool bEnable = ((OperationMode == 2) || (OperationMode == 1 && bDynamicResolutionEnableUserSetting)) && !bIsDynamicResolutionPaused;

#if WITH_EDITOR
	if (GIsEditor && bEnable)
	{
		int32 AllowedInPIE = CVarDynamicResAllowInPIE.GetValueOnGameThread();

		int32 bPIEContextCount = 0;
		for (auto It = WorldList.CreateConstIterator(); It; ++It)
		{
			const FWorldContext &Context = *It;
			if (Context.OwningGameInstance &&
				Context.World() &&
				Context.WorldType == EWorldType::PIE &&
				Context.GameViewport &&
				!Context.GameViewport->IsSimulateInEditorViewport() &&
				Context.World()->IsCameraMoveable())
			{
				bPIEContextCount++;
			}
		}

		bEnable = (bPIEContextCount == 1) && (AllowedInPIE == 1);
	}
#endif

	// Enable dynamic resolution if allowed, not paused.
	return bEnable;
}

void UEngine::UpdateDynamicResolutionStatus()
{
	if (!DynamicResolutionState.IsValid())
	{
		return;
	}
	bool bShouldEnabledDynamicResolutionState = ShouldEnableDynamicResolutionState();
	bool bIsEnabled = DynamicResolutionState->IsEnabled();
	if (bShouldEnabledDynamicResolutionState != bIsEnabled)
	{
		DynamicResolutionState->SetEnabled(bShouldEnabledDynamicResolutionState);
	}
}
#endif // WITH_DYNAMIC_RESOLUTION

void UEngine::EmitDynamicResolutionEvent(EDynamicResolutionStateEvent Event)
{
#if WITH_DYNAMIC_RESOLUTION
	// Early return if dedicated server of commandlet.
	if (IsRunningDedicatedServer() || IsRunningCommandlet())
	{
		check(!DynamicResolutionState.IsValid());
		return;
	}

	checkf(NextDynamicResolutionState.IsValid(), TEXT("Dynamic resolution state is required."));

	// Early return if have already fired this event.
	if (Event == LastDynamicResolutionEvent)
	{
		checkf(Event != EDynamicResolutionStateEvent::BeginFrame,
			TEXT("Begin dynamic resolution event should be fired exactly once"));
		return;
	}

	if (Event == EDynamicResolutionStateEvent::BeginFrame)
	{
		checkf(LastDynamicResolutionEvent == EDynamicResolutionStateEvent::EndFrame,
			TEXT("EDynamicResolutionStateEvent::BeginFrame should only happen after EDynamicResolutionStateEvent::EndFrame."));

		// Log dynamic resolution state change for support if something is going wrong with the heuristic.
		if (DynamicResolutionState != NextDynamicResolutionState)
		{
			// Disable the hold state.
			if (DynamicResolutionState->IsEnabled())
			{
				DynamicResolutionState->SetEnabled(false);
			}

			// Log dynamic resolution state change for support if something is going wrong with the heuristic.
			UE_LOGF(LogEngine, Log, "Changing dynamic resolution state.");

			DynamicResolutionState = NextDynamicResolutionState;
		}

		// Enable dynamic resolution state if has been changed or if the settings changed.
		UpdateDynamicResolutionStatus();

		DynamicResolutionState->ProcessEvent(EDynamicResolutionStateEvent::BeginFrame);
	}
	else if (LastDynamicResolutionEvent == EDynamicResolutionStateEvent::EndFrame)
	{
		// If we did not get begin frame, then it means it must be a redrawn for some reasons such as viewport resize.
		// In this case we just don't pass down a single event to the dynamic resolution event.
		return;
	}
	else
	{
		EDynamicResolutionStateEvent ExpectedPreviousEvent = EDynamicResolutionStateEvent(int32(Event) - 1);

		// If the previous event is going missing, automatically fire it.
		if (LastDynamicResolutionEvent != ExpectedPreviousEvent)
		{
			checkf(ExpectedPreviousEvent != EDynamicResolutionStateEvent::EndFrame,
				TEXT("EDynamicResolutionStateEvent::BeginFrame should not be automatically fired."));
			EmitDynamicResolutionEvent(ExpectedPreviousEvent);
			check(LastDynamicResolutionEvent == ExpectedPreviousEvent);
		}

		DynamicResolutionState->ProcessEvent(Event);
	}

	LastDynamicResolutionEvent = Event;
#endif // WITH_DYNAMIC_RESOLUTION
}

void UEngine::ChangeDynamicResolutionStateAtNextFrame(TSharedPtr< class IDynamicResolutionState > NewState)
{
#if WITH_DYNAMIC_RESOLUTION
	if (IsRunningDedicatedServer() || IsRunningCommandlet())
	{
		check(!DynamicResolutionState.IsValid());
		check(!NextDynamicResolutionState.IsValid());
		return;
	}

	// Since SetDynamicResolutionState() can happen between a dynamic resolution BeginFrame and EndFrame,
	// we only defer the dynamic resolution state to the next frame for simplicity in dynamic resolution
	// state implementations.
	NextDynamicResolutionState = NewState;
#endif
}

void UEngine::WorldAdded( UWorld* InWorld )
{
	WorldAddedEvent.Broadcast( InWorld );
}

void UEngine::WorldDestroyed( UWorld* InWorld )
{
	WorldDestroyedEvent.Broadcast( InWorld );
}

void UEngine::BroadcastNetworkFailure(UWorld * World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString /* = TEXT("") */)
{
	UE_LOGF(LogNet, Error, "UEngine::BroadcastNetworkFailure: FailureType = %ls, ErrorString = %ls, Driver = %ls",
		ENetworkFailure::ToString(FailureType), *ErrorString, (NetDriver ? *NetDriver->GetDescription() : TEXT("NONE")));

	NetworkFailureEvent.Broadcast(World, NetDriver, FailureType, ErrorString);
}

UWorld* UEngine::GetWorldFromContextObject(const UObject* Object, EGetWorldErrorMode ErrorMode) const
{
	if (Object == nullptr)
	{
		switch (ErrorMode)
		{
		case EGetWorldErrorMode::Assert:
			check(Object);
			break;
		case EGetWorldErrorMode::LogAndReturnNull:
			FFrame::KismetExecutionMessage(TEXT("A null object was passed as a world context object to UEngine::GetWorldFromContextObject()."), ELogVerbosity::Warning);
			//UE_LOGF(LogEngine, Warning, "UEngine::GetWorldFromContextObject() passed a nullptr");
			break;
		case EGetWorldErrorMode::ReturnNull:
			break;
		}
		return nullptr;
	}

	bool bSupported = true;
	UWorld* World = (ErrorMode == EGetWorldErrorMode::Assert) ? Object->GetWorldChecked(/*out*/ bSupported) : Object->GetWorld();
	if (bSupported && (World == nullptr) && (ErrorMode == EGetWorldErrorMode::LogAndReturnNull))
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("No world was found for object (%s) passed in to UEngine::GetWorldFromContextObject()."), *GetPathNameSafe(Object)), ELogVerbosity::Warning);
	}
	return (bSupported ? World : GWorld);
}

TArray<class ULocalPlayer*>::TConstIterator	UEngine::GetLocalPlayerIterator(UWorld* World)
{
	return GetGamePlayers(World).CreateConstIterator();
}

TArray<class ULocalPlayer*>::TConstIterator UEngine::GetLocalPlayerIterator(const UGameViewportClient *Viewport)
{
	return GetGamePlayers(Viewport).CreateConstIterator();
}

static TArray<ULocalPlayer*> FakeEmptyLocalPlayers;

const TArray<class ULocalPlayer*>& HandleFakeLocalPlayersList()
{
#if 0
	if (!IsRunningCommandlet())
	{
		UE_LOGF(LogLoad, Error, "WorldContext requested with NULL game instance object." );
		check(false);
	}
#endif
	check(FakeEmptyLocalPlayers.Num() == 0);
	return FakeEmptyLocalPlayers;
}

const TArray<class ULocalPlayer*>& UEngine::GetGamePlayers(UWorld* World) const
{
	const FWorldContext* Context = GetWorldContextFromWorld(World);
	if (Context && Context->OwningGameInstance)
	{
		return Context->OwningGameInstance->GetLocalPlayers();
	}
	else
	{
		return HandleFakeLocalPlayersList();
	}
}

const TArray<class ULocalPlayer*>& UEngine::GetGamePlayers(const UGameViewportClient* Viewport) const
{
	const FWorldContext* Context = GetWorldContextFromGameViewport(Viewport);
	if (Context && Context->OwningGameInstance)
	{
		return Context->OwningGameInstance->GetLocalPlayers();
	}
	else
	{
		return HandleFakeLocalPlayersList();
	}
}

ULocalPlayer* UEngine::FindFirstLocalPlayerFromControllerId(int32 ControllerId) const
{
	for (auto It=WorldList.CreateConstIterator(); It; ++It)
	{
		const FWorldContext &Context = *It;
		if (Context.World() && Context.OwningGameInstance && (Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::PIE))
		{
			const TArray<class ULocalPlayer*> & LocalPlayers = Context.OwningGameInstance->GetLocalPlayers();

			// Use this world context, look for the ULocalPlayer with this ControllerId
			for (ULocalPlayer* LocalPlayer : LocalPlayers)
			{
				if (LocalPlayer && LocalPlayer->GetControllerId() == ControllerId)
				{
					return LocalPlayer;
				}
			}
		}
	}

	return nullptr;
}

ULocalPlayer* UEngine::FindFirstLocalPlayerFromPlatformUserId(FPlatformUserId PlatformUserId) const
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		const FWorldContext& Context = *It;
		if (Context.World() && Context.OwningGameInstance && (Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::PIE))
		{
			const TArray<class ULocalPlayer*>& LocalPlayers = Context.OwningGameInstance->GetLocalPlayers();

			for (ULocalPlayer* LocalPlayer : LocalPlayers)
			{
				if (LocalPlayer && (!PlatformUserId.IsValid() || LocalPlayer->GetPlatformUserId() == PlatformUserId))
				{
					return LocalPlayer;
				}
			}
		}
	}

	return nullptr;
}

ULocalPlayer* UEngine::GetLocalPlayerFromPlatformUserId(UWorld* InWorld, const FPlatformUserId PlatformUserId) const
{
	for (ULocalPlayer* const Player : GetGamePlayers(InWorld))
	{
		if (Player && Player->GetPlatformUserId() == PlatformUserId)
		{
			return Player;
		}
	}

	return nullptr;
}

int32 UEngine::GetNumGamePlayers(UWorld *InWorld)
{
	return GetGamePlayers(InWorld).Num();
}

int32 UEngine::GetNumGamePlayers(const UGameViewportClient *InViewport)
{
	return GetGamePlayers(InViewport).Num();
}

ULocalPlayer* UEngine::GetGamePlayer( UWorld * InWorld, int32 InPlayer )
{
	const TArray<class ULocalPlayer*>& PlayerList = GetGamePlayers(InWorld);
	check( InPlayer < PlayerList.Num() );
	return PlayerList[ InPlayer ];
}

ULocalPlayer* UEngine::GetGamePlayer( const UGameViewportClient* InViewport, int32 InPlayer )
{
	const TArray<class ULocalPlayer*>& PlayerList = GetGamePlayers(InViewport);
	check( InPlayer < PlayerList.Num() );
	return PlayerList[ InPlayer ];
}

ULocalPlayer* UEngine::GetFirstGamePlayer(UWorld *InWorld)
{
	const TArray<class ULocalPlayer*>& PlayerList = GetGamePlayers(InWorld);
	return PlayerList.Num() != 0 ? PlayerList[0] : NULL;
}

ULocalPlayer* UEngine::GetFirstGamePlayer( UPendingNetGame *PendingNetGame )
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		if (It->PendingNetGame == PendingNetGame)
		{
			return ( It->OwningGameInstance != NULL ) ? It->OwningGameInstance->GetFirstGamePlayer() : NULL;
		}
	}
	return NULL;
}

ULocalPlayer* UEngine::GetFirstGamePlayer(const UGameViewportClient *InViewport )
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		if (It->GameViewport == InViewport)
		{
			return ( It->OwningGameInstance != NULL ) ? It->OwningGameInstance->GetFirstGamePlayer() : NULL;
		}
	}
	return NULL;
}

ULocalPlayer* UEngine::GetDebugLocalPlayer()
{
	for (auto It = WorldList.CreateConstIterator(); It; ++It)
	{
		if (It->WorldType == EWorldType::GameRPC)
		{
			continue;
		}

		if (It->OwningGameInstance != NULL && It->OwningGameInstance->GetFirstGamePlayer() != NULL )
		{
			return It->OwningGameInstance->GetFirstGamePlayer();
		}
	}
	return NULL;
}

#if !UE_BUILD_SHIPPING
// Moved this class from Class.cpp because FExportObjectInnerContext is defined in Engine
static class FCDODump : private FSelfRegisteringExec
{
	static FString ObjectString(UObject* Object)
	{
		UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

		FStringOutputDevice Archive;
		const FExportObjectInnerContext Context;
		UExporter::ExportToOutputDevice(&Context, Object, NULL, Archive, TEXT("copy"), 0, PPF_Copy | PPF_DebugDump, false);
		Archive.Log(TEXT("\r\n\r\n"));

		FString ExportedText;
		ExportedText = Archive;
		return ExportedText;
	}

	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		if(FParse::Command(&Cmd,TEXT("CDODump")))
		{
			FString All;
			TArray<UClass*> Classes;
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				UClass* Cls = *ClassIt;
				if (!Cls->IsChildOf(UClass::StaticClass()) && (Cls != UObject::StaticClass()) && (Cls->GetName() != TEXT("World")) && (Cls->GetName() != TEXT("Level")))
				{
					Classes.Add(Cls);
				}
			}
			Classes.Sort();

			for (int32 Index = 0; Index < Classes.Num(); Index++)
			{
				All += ObjectString(Classes[Index]->GetDefaultObject());
			}
			FString Filename = FPaths::ProjectSavedDir() / TEXT("CDO.txt");
			verify(FFileHelper::SaveStringToFile(All, *Filename));
			return true;
		}
		return false;
	}
} CDODump;
#endif // !UE_BUILD_SHIPPING

void UEngine::ShutdownWorldNetDriver( UWorld * World )
{
	if (World)
	{
		/**
		* Shut down the world's net driver, completely disconnecting any clients/servers connected 
		* at the time.   Destroys the net driver.
		*/
		UNetDriver* NetDriver = World->GetNetDriver();
		if (NetDriver)
		{
			UE_LOGF(LogNet, Log, "World NetDriver shutdown %ls [%ls]", *NetDriver->GetName(), *NetDriver->NetDriverName.ToString());
			World->SetNetDriver(NULL);
			DestroyNamedNetDriver(World, NetDriver->NetDriverName);
		}

		// Take care of the demo net driver specifically (so the world can clear the DemoNetDriver property)
		World->DestroyDemoNetDriver();

		// Also disconnect any net drivers that have this set as their world, to avoid GC issues
		FWorldContext &Context = GEngine->GetWorldContextFromWorldChecked(World);

		for (int32 Index = 0; Index < Context.ActiveNetDrivers.Num(); Index++)
		{
			NetDriver = Context.ActiveNetDrivers[Index].NetDriver;
			if (NetDriver && NetDriver->GetWorld() == World && !NetDriver->GetPendingDestruction())
			{
				UE_LOGF(LogNet, Log, "World NetDriver shutdown %ls [%ls]", *NetDriver->GetName(), *NetDriver->NetDriverName.ToString());
				DestroyNamedNetDriver(World, NetDriver->NetDriverName);
				Index--;
			}
		}
	}
}

void UEngine::ShutdownAllNetDrivers()
{
	for (auto It = WorldList.CreateIterator(); It; ++It)
	{
		TArray<FNamedNetDriver> & ActiveNetDrivers = It->ActiveNetDrivers;

		for (int32 Index = 0; Index < ActiveNetDrivers.Num(); Index++)
		{
			FNamedNetDriver& NamedNetDriver = ActiveNetDrivers[Index];
			UNetDriver* NetDriver = NamedNetDriver.NetDriver;
			if (NetDriver)
			{
				UE_LOGF(LogNet, Log, "World NetDriver shutdown %ls [%ls]", *NetDriver->GetName(), *NetDriver->NetDriverName.ToString());
				UWorld * World = NetDriver->GetWorld();
				if (World)
				{
					World->SetNetDriver(NULL);
				}
				NetDriver->SetWorld(NULL);
				DestroyNamedNetDriver(It->World(), NetDriver->NetDriverName);
				Index--;
			}
		}

		ActiveNetDrivers.Empty();
	}
}

UNetDriver* FindNamedNetDriver_Local(const TArray<FNamedNetDriver>& ActiveNetDrivers, FName NetDriverName)
{
	for (int32 Index = 0; Index < ActiveNetDrivers.Num(); Index++)
	{
		const FNamedNetDriver& NamedNetDriver = ActiveNetDrivers[Index];
		UNetDriver* NetDriver = NamedNetDriver.NetDriver;
		if (NetDriver && NetDriver->NetDriverName == NetDriverName)
		{
			return NetDriver;
		}
	}
	return NULL;
}

UNetDriver* UEngine::FindNamedNetDriver(const UWorld* InWorld, FName NetDriverName)
{
#if WITH_EDITOR
	const FWorldContext* WorldContext = GetWorldContextFromWorld(InWorld);
	if (WorldContext)
	{
		return FindNamedNetDriver_Local(WorldContext->ActiveNetDrivers, NetDriverName);
	}

	return nullptr;
#else
	return FindNamedNetDriver_Local(GetWorldContextFromWorldChecked(InWorld).ActiveNetDrivers, NetDriverName);
#endif // WITH_EDITOR
}

UNetDriver* UEngine::FindNamedNetDriver(const UPendingNetGame* InPendingNetGame, FName NetDriverName)
{
	return FindNamedNetDriver_Local(GetWorldContextFromPendingNetGameChecked(InPendingNetGame).ActiveNetDrivers, NetDriverName);
}

const FIrisNetDriverConfig* UEngine::GetIrisNetDriverConfig(FName InNetDriverDefinition, FName InNetDriverName) const
{
	const FIrisNetDriverConfig* IrisConfig = nullptr;

	/**
	 * Look for a config setting telling if this new NetDriver can use Iris or not
	 * To enable iris for a netdriver you need to add the proper configuration in Engine.ini like so:
	 *		[/Script/Engine.Engine]
	 *		+IrisNetDriverConfigs=(NetDriverName="MyNetDriver",bCanUseIris=true)
	 *		+IrisNetDriverConfigs=(NetDriverDefinition="MyNetDriverDef",bCanUseIris=false)
	 *		+IrisNetDriverConfigs=(NetDriverWildcardName="SecondaryNetDriver*",bCanUseIris=true);
	 *
	 */

	// Search for the exact name match
	if (IrisConfig == nullptr)
	{
		IrisConfig = IrisNetDriverConfigs.FindByPredicate([InNetDriverName](const FIrisNetDriverConfig& Config)
		{
			return Config.NetDriverName.IsNone() ? false : Config.NetDriverName == InNetDriverName;
		});
	}

	// Search for a wildcard match
	if (IrisConfig == nullptr)
	{
		const FString TempNetDriverName = InNetDriverName.ToString();
		IrisConfig = IrisNetDriverConfigs.FindByPredicate([TempNetDriverName](const FIrisNetDriverConfig& Config)
		{
			return Config.NetDriverWildcardName.IsEmpty() ? false : TempNetDriverName.MatchesWildcard(Config.NetDriverWildcardName);
		});
	}

	// Search for the definition match
	if (IrisConfig == nullptr)
	{
		IrisConfig = IrisNetDriverConfigs.FindByPredicate([InNetDriverDefinition](const FIrisNetDriverConfig& Config)
		{
			return Config.NetDriverDefinition.IsNone() ? false : Config.NetDriverDefinition == InNetDriverDefinition;
		});
	}

	return IrisConfig;
}

bool UEngine::WillNetDriverUseIris(const FWorldContext& Context, FName InNetDriverDefinition, FName InNetDriverName) const
{
	const FIrisNetDriverConfig* IrisConfig = GetIrisNetDriverConfig(InNetDriverDefinition, InNetDriverName);

	const bool bConfigCanUseIris = IrisConfig && IrisConfig->bCanUseIris;
	const bool bIsEngineDefaultIris = UE::Net::ShouldUseIrisReplication();

	// Default preference: use Iris if the config can & the engine cvar is enabled.
	bool bUseIrisRepSystem = bConfigCanUseIris && bIsEngineDefaultIris;

	if (IrisConfig)
	{
		if (IrisConfig->PreferredReplicationSystem == EReplicationSystem::Iris)
		{
			// If Iris is preferred and can be used, use it (unless overridden below)
			UE_CLOGF(!bConfigCanUseIris, LogNet, Log, "NetDriver %ls (NetDefinition %ls) is configured to prefer the Iris replication system, but bCanUseIris is false.", *InNetDriverName.ToString(), *InNetDriverDefinition.ToString());
			bUseIrisRepSystem = bConfigCanUseIris;
		}
		else if (IrisConfig->PreferredReplicationSystem == EReplicationSystem::Generic)
		{
			// If Generic is preferred, use it (unless overridden below)
			bUseIrisRepSystem = false;
		}
	}

	if (UGameInstance* ContextGameInstance = Context.OwningGameInstance)
	{
		EReplicationSystem GameInstanceDesiredRepSystem = ContextGameInstance->GetDesiredReplicationSystem(InNetDriverDefinition);

		// If the game instance requested to use the generic repsystem
		if (GameInstanceDesiredRepSystem == EReplicationSystem::Generic)
		{
			UE_CLOGF(bUseIrisRepSystem, LogNet, Log, "GameInstance %ls is forcing NetDriver %ls (NetDefinition %ls) to use the Generic replication system.", *GetNameSafe(ContextGameInstance), *InNetDriverName.ToString(), *InNetDriverDefinition.ToString());
			bUseIrisRepSystem = false;
		}
		// If the game instance requested to use the Iris repsystem
		else if (GameInstanceDesiredRepSystem == EReplicationSystem::Iris)
		{
			UE_CLOGF(!bUseIrisRepSystem && bConfigCanUseIris, LogNet, Log, "GameInstance %ls is forcing NetDriver %ls (NetDefinition %ls) to use the Iris replication system.", *GetNameSafe(ContextGameInstance), *InNetDriverName.ToString(), *InNetDriverDefinition.ToString());
				
			// Enable Iris ONLY if the config supports it.
			bUseIrisRepSystem = bConfigCanUseIris;
		}

#if WITH_EDITOR
		// In PIE let's make sure the clients follow what the server's game net driver is using
		if (Context.WorldType == EWorldType::PIE && !Context.RunAsDedicated && InNetDriverDefinition == NAME_GameNetDriver)
		{
			if (FWorldContext* ServerPIEContext = GEngine->GetWorldContextFromPIEInstance(0))
			{
				if (ServerPIEContext->RunAsDedicated)
				{
					for (const FNamedNetDriver& PieNetDriver : ServerPIEContext->ActiveNetDrivers)
					{
						if (PieNetDriver.NetDriverDef->DefName == NAME_GameNetDriver)
						{
							bUseIrisRepSystem = PieNetDriver.NetDriver->IsUsingIrisReplication();
							break;
						}
					}
				}
			}
		}
#endif
	}

	// Ignore all of the above if the cmdline is requesting a specific system
	const EReplicationSystem CmdlineRequest = UE::Net::GetUseIrisReplicationCmdlineValue();
	if (CmdlineRequest == EReplicationSystem::Iris)
	{
		UE_CLOGF(!bUseIrisRepSystem && bConfigCanUseIris, LogNet, Log, "Cmdline -UseIrisReplication=1 is forcing NetDriver %ls (NetDefinition %ls) to use the Iris replication system.", *InNetDriverName.ToString(), *InNetDriverDefinition.ToString());
		bUseIrisRepSystem = bConfigCanUseIris;
	}
	else if (CmdlineRequest == EReplicationSystem::Generic)
	{
		UE_CLOGF(bUseIrisRepSystem, LogNet, Log, "Cmdline -UseIrisReplication=0 is forcing NetDriver %ls (NetDefinition %ls) to use the Generic replication system.", *InNetDriverName.ToString(), *InNetDriverDefinition.ToString());
		bUseIrisRepSystem = false;
	}

	// Only use Iris if the module is loaded (this happens automatically if the Iris plugin is enabled)
	if (bUseIrisRepSystem)
	{
		if (!ensureMsgf(FModuleManager::Get().IsModuleLoaded("IrisCore"), TEXT("%s is not using Iris because the IrisCore module isn't loaded. Check whether the Iris plugin is enabled."), *InNetDriverName.ToString()))
		{
			return false;
		}
	}

	return bUseIrisRepSystem;
}

bool UEngine::WillNetDriverUseParallelNetConnectionTick(const FWorldContext& Context, const FName InNetDriverDefinition, const FName InNetDriverName) const
{
	const FIrisNetDriverConfig* IrisConfig = GetIrisNetDriverConfig(InNetDriverDefinition, InNetDriverName);
	const bool bWithParallelNetTick = IrisConfig ? IrisConfig->bCanUseParallelNetConnectionTick : false;
	return bWithParallelNetTick;
}

namespace UE::Private
{
	UNetDriver* CreateNetDriver_Local(UEngine* Engine, FWorldContext& Context, FName NetDriverDefinition, FName InNetDriverName)
	{
		UNetDriver* ReturnVal = nullptr;
		FNetDriverDefinition* Definition = nullptr;
		auto FindNetDriverDefPred =
			[NetDriverDefinition](const FNetDriverDefinition& CurDef)
		{
			return CurDef.DefName == NetDriverDefinition;
		};

#if !UE_BUILD_SHIPPING
		/**
		* Commandline override for the net driver.
		*
		* Format: (NOTE: Use quotes whenever the ',' character is used)
		*	Override the main/game net driver (most common usage):
		*		-NetDriverOverrides=DriverClassName
		*
		*	Override a specific/named net driver:
		*		-NetDriverOverrides="DefName,DriverClassName"
		*
		*	Override a specific driver, including fallback driver:
		*		-NetDriverOverrides="DefName,DriverClassName,DriverClassNameFallback"
		*
		*	Override multiple net drivers:
		*		-NetDriverOverrides="DriverClassName;DefName2,DriverClassName2"
		*
		*
		* Example:
		*	Use WebSocket for the main game net driver:
		*		-NetDriverOverrides=/Script/WebSocketNetworking.WebSocketNetDriver
		*
		*	Use WebSocket for the main game net driver, and the party beacon net driver
		*		-NetDriverOverrides="/Script/WebSocketNetworking.WebSocketNetDriver;BeaconNetDriver,/Script/WebSocketNetworking.WebSocketNetDriver"
		*/

		static TArray<FNetDriverDefinition> NetDriverOverrides = TArray<FNetDriverDefinition>();
		FString OverrideCmdLine;

		if (NetDriverOverrides.Num() == 0 && FParse::Value(FCommandLine::Get(), TEXT("NetDriverOverrides="), OverrideCmdLine))
		{
			TArray<FString> OverrideEntries;
			OverrideCmdLine.ParseIntoArray(OverrideEntries, TEXT(";"), false);

			UE_LOGF(LogNet, Log, "NetDriverOverrides:");

			for (FString& CurOverrideEntry : OverrideEntries)
			{
				TArray<FString> CurEntryParms;
				CurOverrideEntry.ParseIntoArray(CurEntryParms, TEXT(","), false);

				if (CurEntryParms.Num() == 0)
				{
					continue;
				}


				FString TargetDefName = (CurEntryParms.Num() > 1 ? CurEntryParms[0] : FName(NAME_GameNetDriver).ToString());
				FString OverrideClass = (CurEntryParms.Num() > 1 ? CurEntryParms[1] : CurEntryParms[0]);
				FString OverrideFallback;
				auto FindTargetDefPred =
					[TargetDefName](const FNetDriverDefinition& CurDef)
				{
					return CurDef.DefName == *TargetDefName;
				};

				if (CurEntryParms.Num() > 2)
				{
					OverrideFallback = CurEntryParms[2];
				}
				else if (FNetDriverDefinition* FallbackDef = Engine->NetDriverDefinitions.FindByPredicate(FindTargetDefPred))
				{
					OverrideFallback = FallbackDef->DriverClassNameFallback.ToString();
				}
				else
				{
					OverrideFallback = TEXT("/Script/OnlineSubsystemUtils.IpNetDriver");
				}

				if (TargetDefName.Len() == 0 || OverrideClass.Len() == 0 || OverrideFallback.Len() == 0)
				{
					continue;
				}


				if (!NetDriverOverrides.ContainsByPredicate(FindTargetDefPred))
				{
					FNetDriverDefinition NewDef;

					NewDef.DefName = *TargetDefName;
					NewDef.DriverClassName = FName(*OverrideClass);
					NewDef.DriverClassNameFallback = FName(*OverrideFallback);

					NetDriverOverrides.Add(NewDef);

					UE_LOGF(LogNet, Log, "- DefName: %ls, DriverClassName: %ls, DriverClassNameFallback: %ls", *TargetDefName,
						*OverrideClass, *OverrideFallback);
				}
			}
		}


		Definition = NetDriverOverrides.FindByPredicate(FindNetDriverDefPred);

		if (Definition != nullptr)
		{
			UE_LOGF(LogNet, Log, "Overriding NetDriver '%ls' with class: %ls", *NetDriverDefinition.ToString(),
				*Definition->DriverClassName.ToString());
		}
		else
#endif
		{
			Definition = Engine->NetDriverDefinitions.FindByPredicate(FindNetDriverDefPred);
		}

		if (Definition != nullptr)
		{
			const FString RedirectedDriverClassName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(Definition->DriverClassName.ToString())).ToString();

			UClass* NetDriverClass = StaticLoadClass(UNetDriver::StaticClass(), nullptr, *RedirectedDriverClassName, nullptr, LOAD_Quiet);

			// if it fails, then fall back to standard fallback
			if (NetDriverClass == nullptr || !NetDriverClass->GetDefaultObject<UNetDriver>()->IsAvailable())
			{
				NetDriverClass = StaticLoadClass(UNetDriver::StaticClass(), nullptr, *Definition->DriverClassNameFallback.ToString(),
					nullptr, LOAD_None);
			}

			if (NetDriverClass != nullptr)
			{
				ReturnVal = NewObject<UNetDriver>(GetTransientPackage(), NetDriverClass);

				check(ReturnVal != nullptr);

				const FName DriverName = InNetDriverName.IsNone() ? ReturnVal->GetFName() : InNetDriverName;
				const bool bInitializeWithIris = Engine->WillNetDriverUseIris(Context, NetDriverDefinition, DriverName);
				const bool bUseParallelNetConnectionTick = Engine->WillNetDriverUseParallelNetConnectionTick(Context, NetDriverDefinition, DriverName);

				if (bInitializeWithIris)
				{
					UE::Net::FNetObjectFactoryRegistry::SetFactoryRegistrationAllowed(false);
				}

				ReturnVal->SetNetDriverName(DriverName);
				ReturnVal->SetNetDriverDefinition(NetDriverDefinition);
				ReturnVal->PostCreation(bInitializeWithIris, bUseParallelNetConnectionTick);

				new(Context.ActiveNetDrivers) FNamedNetDriver(ReturnVal, Definition);

				FWorldDelegates::OnNetDriverCreated.Broadcast(Context.World(), ReturnVal);
			}
		}


		if (ReturnVal == nullptr)
		{
			UE_LOGF(LogNet, Log, "CreateNamedNetDriver failed to create driver from definition %ls", *NetDriverDefinition.ToString());
		}

		return ReturnVal;
	}

	bool CreateNamedNetDriver_Local(UEngine* Engine, FWorldContext& Context, FName NetDriverName, FName NetDriverDefinition)
	{
		UNetDriver* NetDriver = FindNamedNetDriver_Local(Context.ActiveNetDrivers, NetDriverName);
		if (NetDriver == nullptr)
		{
			NetDriver = CreateNetDriver_Local(Engine, Context, NetDriverDefinition, NetDriverName);
			if (NetDriver)
			{
				return true;
			}
		}

		if (NetDriver)
		{
			UE_LOGF(LogNet, Log, "CreateNamedNetDriver %ls already exists as %ls", *NetDriverName.ToString(), *NetDriver->GetName());
		}
		else
		{
			UE_LOGF(LogNet, Log, "CreateNamedNetDriver failed to create driver %ls from definition %ls", *NetDriverName.ToString(), *NetDriverDefinition.ToString());
		}

		return false;
	}

	void DestroyNamedNetDriver_Local(FWorldContext& Context, FName NetDriverName)
	{
		for (int32 Index = 0; Index < Context.ActiveNetDrivers.Num(); Index++)
		{
			FNamedNetDriver& NamedNetDriver = Context.ActiveNetDrivers[Index];
			UNetDriver* NetDriver = NamedNetDriver.NetDriver;

			if (NetDriver && NetDriver->NetDriverName == NetDriverName)
			{
				// Can't destroy immediately while inside the NetDriver's tick.
				if (NetDriver && NetDriver->IsInTick())
				{
					NetDriver->RequestNetDriverDestruction();
					continue;
				}

				UE_LOGF(LogNet, Log, "DestroyNamedNetDriver: %ls ", *NetDriver->GetDescription());
				
				Context.ActiveNetDrivers.RemoveAtSwap(Index);
				Context.GarbageObjectsToVerify.Add(NetDriver);
				NetDriver->SetWorld(NULL);
				NetDriver->Shutdown();
				NetDriver->LowLevelDestroy();

				if (UWorld* World = Context.World())
				{
					World->ClearNetDriver(NetDriver);
				}

				break;
			}
		}

		auto IsAnyNetDriverUsingIris = []()
		{
			const TIndirectArray<FWorldContext>& Worlds = GEngine->GetWorldContexts();
			for (const FWorldContext& Context : Worlds)
			{
				for (const FNamedNetDriver& NetDriver : Context.ActiveNetDrivers)
				{
					if (NetDriver.NetDriver && NetDriver.NetDriver->IsUsingIrisReplication())
					{
						return true;
					}
				}
			}
			return false;
		};
		
		if (!IsAnyNetDriverUsingIris())
		{
			UE::Net::FNetObjectFactoryRegistry::SetFactoryRegistrationAllowed(true);
		}
	}
} // end namespace UE::Private

UNetDriver* UEngine::CreateNetDriver(UWorld *InWorld, FName NetDriverDefinition)
{
	LLM_SCOPE(ELLMTag::Networking);
	return UE::Private::CreateNetDriver_Local(this, GetWorldContextFromWorldChecked(InWorld), NetDriverDefinition, NAME_None);
}

bool UEngine::CreateNamedNetDriver(UWorld *InWorld, FName NetDriverName, FName NetDriverDefinition)
{
	return UE::Private::CreateNamedNetDriver_Local(this, GetWorldContextFromWorldChecked(InWorld), NetDriverName, NetDriverDefinition);
}

bool UEngine::CreateNamedNetDriver(UPendingNetGame *PendingNetGame, FName NetDriverName, FName NetDriverDefinition)
{
	return UE::Private::CreateNamedNetDriver_Local(this, GetWorldContextFromPendingNetGameChecked(PendingNetGame), NetDriverName, NetDriverDefinition);
}

void UEngine::DestroyNamedNetDriver(UWorld *InWorld, FName NetDriverName)
{
	UE::Private::DestroyNamedNetDriver_Local(GetWorldContextFromWorldChecked(InWorld), NetDriverName);
}

void UEngine::DestroyNamedNetDriver(UPendingNetGame *PendingNetGame, FName NetDriverName)
{
	UE::Private::DestroyNamedNetDriver_Local(GetWorldContextFromPendingNetGameChecked(PendingNetGame), NetDriverName);
}

ENetMode UEngine::GetNetMode(const UWorld *World) const
{ 
	if (World)
	{
		return World->GetNetMode();
	}

	return NM_Standalone;
}

static inline void CallHandleDisconnectForFailure(UWorld* InWorld, UNetDriver* NetDriver)
{
	// No world will be created yet if you fail to initialize network driver while trying to connect via cmd line arg.

	// Calls any global delegates listening, such as on game mode
	FGameDelegates::Get().GetHandleDisconnectDelegate().Broadcast(InWorld, NetDriver);

	// A valid world or NetDriver is required to look up a GameInstance/ULocalPlayer.
	if (InWorld)
	{
		if (InWorld->GetGameInstance() != nullptr && InWorld->GetGameInstance()->GetOnlineSession() != nullptr)
		{
			InWorld->GetGameInstance()->GetOnlineSession()->HandleDisconnect(InWorld, NetDriver);
		}
	}
	else if(NetDriver && NetDriver->NetDriverName == NAME_PendingNetDriver)
	{
		// The only disconnect case without a valid InWorld, should be in a travel case where there is a pending game net driver.
		FWorldContext &Context = GEngine->GetWorldContextFromPendingNetGameNetDriverChecked(NetDriver);
		check(Context.OwningGameInstance != NULL && Context.OwningGameInstance->GetFirstGamePlayer() != NULL);

		if ( Context.OwningGameInstance->GetOnlineSession() != nullptr )
		{
			Context.OwningGameInstance->GetOnlineSession()->HandleDisconnect(Context.World(), NetDriver);
		}
	}
	else
	{
		// Handle disconnect should always have a valid world or net driver to give the call context
		UE_LOGF(LogNet, Error, "CallHandleDisconnectForFailure called without valid world or netdriver. (NetDriver: %ls", NetDriver ? *NetDriver->GetName() : TEXT("NULL") );
	}
}

void UEngine::HandleTravelFailure(UWorld* InWorld, ETravelFailure::Type FailureType, const FString& ErrorString)
{
	if (InWorld == NULL)
	{
		UE_LOGF(LogNet, Error, "TravelFailure: %ls, Reason for Failure: '%ls' with a NULL UWorld", ETravelFailure::ToString(FailureType), *ErrorString);
	}
	else
	{
		UE_LOGF(LogNet, Log, "TravelFailure: %ls, Reason for Failure: '%ls'", ETravelFailure::ToString(FailureType), *ErrorString);

		// Give the GameInstance a chance to handle the failure.
		HandleTravelFailure_NotifyGameInstance(InWorld, FailureType);

		// Cancel pending net game if there was one
		CancelPending(InWorld);

		// Any of these errors should attempt to load back to some stable map
		CallHandleDisconnectForFailure(InWorld, InWorld->GetNetDriver());
	}
}

void UEngine::HandleNetworkFailure(UWorld *World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	static double LastTimePrinted = 0.0f;
	if (FPlatformTime::Seconds() - LastTimePrinted > NetErrorLogInterval)
	{
		UE_LOGF(LogNet, Log, "NetworkFailure: %ls, Error: '%ls'", ENetworkFailure::ToString(FailureType), *ErrorString);
		LastTimePrinted = FPlatformTime::Seconds();
	}

	if (!NetDriver)
	{
		return;
	}

	// Only handle failure at this level for game or pending net drivers.
	FName NetDriverName = NetDriver->NetDriverName;
	if (NetDriverName == NAME_GameNetDriver || NetDriverName == NAME_PendingNetDriver)
	{
		// If this net driver has already been unregistered with this world, then don't handle it.
		if (World)
		{
			if (!FindNamedNetDriver(World, NetDriverName))
			{
				// This netdriver has already been destroyed (probably waiting for GC)
				return;
			}
		}

		// Give the GameInstance a chance to handle the failure.
		HandleNetworkFailure_NotifyGameInstance(World, NetDriver, FailureType);

		ENetMode FailureNetMode = NetDriver->GetNetMode();	// NetMode of the driver that failed
		bool bShouldTravel = true;

		switch (FailureType)
		{
		case ENetworkFailure::FailureReceived:
			break;
		case ENetworkFailure::PendingConnectionFailure:
			// TODO stop the connecting movie
			break;
		case ENetworkFailure::ConnectionLost:
			// Hosts don't travel when clients disconnect
			bShouldTravel = (FailureNetMode == NM_Client);
			break;
		case ENetworkFailure::ConnectionTimeout:
			// Hosts don't travel when clients disconnect
			bShouldTravel = (FailureNetMode == NM_Client);
			break;
		case ENetworkFailure::NetGuidMismatch:
		case ENetworkFailure::NetChecksumMismatch:
			// Hosts don't travel when clients have actor issues
			bShouldTravel = (FailureNetMode == NM_Client);
			break;
		case ENetworkFailure::NetDriverAlreadyExists:
		case ENetworkFailure::NetDriverCreateFailure:
		case ENetworkFailure::OutdatedClient:
		case ENetworkFailure::OutdatedServer:
		default:
			break;
		}

		if (bShouldTravel)
		{
			CallHandleDisconnectForFailure(World, NetDriver);
		}
	}
}

void UEngine::HandleNetworkLagStateChanged(UWorld* World, UNetDriver* NetDriver, ENetworkLagState::Type LagType)
{
	// Stub. Implement in subclasses
}

void UEngine::HandleNetworkFailure_NotifyGameInstance(UWorld *World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType)
{
	bool bIsServer = true;

	if (NetDriver != nullptr)
	{
		bIsServer = NetDriver->GetNetMode() != NM_Client;
	}

	if (World != nullptr && World->GetGameInstance() != nullptr)
	{
		World->GetGameInstance()->HandleNetworkError(FailureType, bIsServer);
	}
	else
	{
		// Since the UWorld passed in might be null, as well as the NetDriver's UWorld,
		// go through the world contexts until we find the one with this net driver.
		for (auto& Context : WorldList)
		{
			if (Context.PendingNetGame != nullptr &&
				Context.PendingNetGame->NetDriver == NetDriver &&
				Context.OwningGameInstance != nullptr)
			{
				// Use the GameInstance from the current context.
				Context.OwningGameInstance->HandleNetworkError(FailureType, bIsServer);
			}
		}
	}
}

void UEngine::HandleTravelFailure_NotifyGameInstance(UWorld *World, ETravelFailure::Type FailureType)
{
	if (World != nullptr && World->GetGameInstance() != nullptr)
	{
		World->GetGameInstance()->HandleTravelError(FailureType);
	}
}

void UEngine::SpawnServerActors(UWorld *World)
{
	TArray<FString> FullServerActors;

	FullServerActors.Append(ServerActors);
	FullServerActors.Append(RuntimeServerActors);

	for( int32 i=0; i < FullServerActors.Num(); i++ )
	{
		TCHAR Str[2048];
		const TCHAR* Ptr = * FullServerActors[i];
		if( FParse::Token( Ptr, Str, UE_ARRAY_COUNT(Str), 1 ) )
		{
			UE_LOGF(LogNet, Log, "Spawning: %ls", Str );
			UClass* HelperClass = StaticLoadClass( AActor::StaticClass(), NULL, Str, NULL, LOAD_None, NULL );
			AActor* Actor = World->SpawnActor( HelperClass );
			while( Actor && FParse::Token(Ptr,Str,UE_ARRAY_COUNT(Str),1) )
			{
				TCHAR* Value = FCString::Strchr(Str,'=');
				if( Value )
				{
					*Value++ = 0;
					for( TFieldIterator<FProperty> It(Actor->GetClass()); It; ++It )
					{
						if(	FCString::Stricmp(*It->GetName(),Str)==0
							&&	(It->PropertyFlags & CPF_Config) )
						{
							It->ImportText_InContainer( Value, Actor, Actor, 0 );
						}
					}
				}
			}
		}
	}
}

bool UEngine::HandleOpenCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld *InWorld )
{
	FWorldContext &WorldContext = GetWorldContextFromWorldChecked(InWorld);
	FURL TestURL(&WorldContext.LastURL, Cmd, TRAVEL_Absolute);
	if (TestURL.IsLocalInternal())
	{
		// make sure the file exists if we are opening a local file
		if (!MakeSureMapNameIsValid(TestURL.Map))
		{
			Ar.Logf(TEXT("ERROR: The map '%s' does not exist."), *TestURL.Map);
			return true;
		}
#if WITH_EDITOR
		else
		{
			// Next comes a complicated but necessary way of blocking a crash caused by opening a level when playing multiprocess as a client (that's not allowed because of streaming levels)
			ULevelEditorPlaySettings* PlayInSettings = GetMutableDefault<ULevelEditorPlaySettings>();
			check(PlayInSettings);
			bool bMultiProcess = !([&PlayInSettings] { bool RunUnderOneProcess(false); return (PlayInSettings->GetRunUnderOneProcess(RunUnderOneProcess) && RunUnderOneProcess); }());

			const EPlayNetMode PlayNetMode = [&PlayInSettings] { EPlayNetMode NetMode(PIE_Standalone); return (PlayInSettings->GetPlayNetMode(NetMode) ? NetMode : PIE_Standalone); }();
			bool bClientMode = PlayNetMode == EPlayNetMode::PIE_Client;

			if (bMultiProcess && bClientMode)
			{
				UE_LOGF(LogNet, Log, "%ls", TEXT("Opening a map is not allowed in this play mode (client mode + multiprocess)!"));
				return true;
			}
		}
#endif
	}

	SetClientTravel(InWorld, Cmd, TRAVEL_Absolute);
	return true;
}

bool UEngine::HandleTravelCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	FWorldContext &WorldContext = GetWorldContextFromWorldChecked(InWorld);
	FURL TestURL(&WorldContext.LastURL, Cmd, TRAVEL_Partial);
	if (TestURL.IsLocalInternal())
	{
		// make sure the file exists if we are opening a local file
		bool bMapFound = MakeSureMapNameIsValid(TestURL.Map);
		if (!bMapFound)
		{
			Ar.Logf(TEXT("ERROR: The map '%s' does not exist."), *TestURL.Map);
			return true;
		}
	}

	SetClientTravel( InWorld, Cmd, TRAVEL_Partial );
	return true;
}

bool UEngine::HandleStreamMapCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld *InWorld )
{
	FWorldContext &WorldContext = GetWorldContextFromWorldChecked(InWorld);
	FURL TestURL(&WorldContext.LastURL, Cmd, TRAVEL_Partial);
	if (TestURL.IsLocalInternal())
	{
		// make sure the file exists if we are opening a local file
		if (MakeSureMapNameIsValid(TestURL.Map) && TestURL.Valid)
		{
			for (const ULevel* const Level : InWorld->GetLevels())
			{
				if (Level->URL.Map == TestURL.Map)
				{
					Ar.Logf(TEXT("ERROR: The map '%s' is already loaded."), *TestURL.Map);
					return true;
				}
			}

			TArray<FName> LevelNames;
			LevelNames.Add(*TestURL.Map);

			FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);

			PrepareMapChange(Context, LevelNames);
			Context.bShouldCommitPendingMapChange = true;
			ConditionalCommitMapChange(Context);
		}
		else
		{
			Ar.Logf(TEXT("ERROR: The map '%s' does not exist."), *TestURL.Map);
		}
	}
	else
	{
		Ar.Logf(TEXT("ERROR: Can only perform streaming load for local URLs."));
	}
	return true;
}

#if WITH_SERVER_CODE
bool UEngine::HandleServerTravelCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld )
{
	if (!InWorld->IsNetMode(NM_Client))
	{
		FString URL(Cmd);
		FString MapName, Options;

		// Split the options off the map name before we validate it
		if (URL.Split(TEXT("?"), &MapName, &Options, ESearchCase::CaseSensitive) == false)
		{
			MapName = URL;
		}

		if (MakeSureMapNameIsValid(MapName))
		{
			// If there were options reconstitute the URL before sending in to the server travel call 
			URL = (Options.IsEmpty() ? MapName : MapName + TEXT("?") + Options);
			InWorld->ServerTravel(URL);
			return true;
		}
		else
		{
			Ar.Logf(TEXT("ERROR: The map '%s' is either short package name or does not exist."), *MapName);
		}
	}
	return false;
}
#endif

bool UEngine::HandleDisconnectCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld *InWorld )
{
	// This should only be called from typing 'disconnect' at the console. InWorld must have a valid WorldContext.
	check(InWorld);
	check(GetWorldContextFromWorld(InWorld));

	HandleDisconnect(InWorld, InWorld->GetNetDriver());
	return true;
}

void UEngine::HandleDisconnect( UWorld *InWorld, UNetDriver *NetDriver )
{
	// There must be some context for this disconnect
	check(InWorld || NetDriver);

	// InWorld might be null. It might also not map to any valid world context (for example, a pending net game disconnect)
	// If there is a context for this world, setup client travel.
	if (FWorldContext* WorldContext = GetWorldContextFromWorld(InWorld))
	{
		// Remove ?Listen parameter, if it exists
		WorldContext->LastURL.RemoveOption( TEXT("Listen") );
		WorldContext->LastURL.RemoveOption( TEXT("LAN") );

		// Net driver destruction will occur during LoadMap (prevents GetNetMode from changing output for the remainder of the frame)
		// The ?closed option will also make sure any PendingNetGame is cancelled in Browse
		SetClientTravel( InWorld, TEXT("?closed"), TRAVEL_Absolute );
	}
	else if (NetDriver)
	{
		// If the NetDriver that failed was a pending netgame driver, cancel the PendingNetGame
		CancelPending(NetDriver);

		// Shut down any existing game connections
		if (InWorld)
		{
			// Call this to remove the NetDriver from the world context's ActiveNetDriver list
			DestroyNamedNetDriver(InWorld, NetDriver->NetDriverName);
		}
		else
		{
			NetDriver->Shutdown();
			NetDriver->LowLevelDestroy();

			// In this case, the world is null and something went wrong, so we should travel back to the default world so that we
			// can get back to a good state.
			for (FWorldContext& PotentialWorldContext : WorldList)
			{
				if (PotentialWorldContext.WorldType == EWorldType::Game)
				{
					FURL DefaultURL;
					DefaultURL.LoadURLConfig(TEXT("DefaultPlayer"), GGameIni);
					const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
					if (GameMapsSettings)
					{
						PotentialWorldContext.TravelURL = FURL(&DefaultURL, *(GameMapsSettings->GetGameDefaultMap() + GameMapsSettings->LocalMapOptions), TRAVEL_Partial).ToString();
						PotentialWorldContext.TravelType = TRAVEL_Partial;
					}
				}
			}
		}
	}
}

bool UEngine::HandleReconnectCommand( const TCHAR* Cmd, FOutputDevice& Ar, UWorld *InWorld )
{
	FWorldContext &WorldContext = GetWorldContextFromWorldChecked(InWorld);
	if (WorldContext.LastRemoteURL.Valid && WorldContext.LastRemoteURL.Host != TEXT(""))
	{
		SetClientTravel(InWorld, *WorldContext.LastRemoteURL.ToString(), TRAVEL_Absolute);
	}
	return true;
}

bool UEngine::MakeSureMapNameIsValid(FString& InOutMapName)
{
	const FString TestMapName = UWorld::RemovePIEPrefix(InOutMapName);

	// Check if the map name is long package name and if it actually exists.
	// Short package names are only supported in non-shipping builds.
	bool bIsValid = !FPackageName::IsShortPackageName(TestMapName);
	if (bIsValid)
	{
		// If the user starts a multiplayer PIE session with an unsaved map,
		// DoesPackageExist won't find it, so we have to try to find the package in memory as well.
		bIsValid = (FindObjectFast<UPackage>(nullptr, FName(*TestMapName)) != nullptr) || FPackageName::DoesPackageExist(TestMapName);

		// If we're not in the editor, then we always want to strip off the PIE prefix.  We might be connected to
		// a PIE listen server.  In this case, we'll use our version of the map without the PIE prefix.
		if (bIsValid && !GIsEditor)
		{
			InOutMapName = TestMapName;
		}
	}
	else
	{
		// Look up in the AssetRegistry.
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		FName ExistingPackageName = AssetRegistry.GetFirstPackageByName(TestMapName);
		if (!ExistingPackageName.IsNone())
		{
			ExistingPackageName.ToString(InOutMapName);
			bIsValid = true;
		}
	}
	return bIsValid;
}

void UEngine::SetClientTravel( UWorld *InWorld, const TCHAR* NextURL, ETravelType InTravelType )
{
	FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);

	// set TravelURL.  Will be processed safely on the next tick in UGameEngine::Tick().
	Context.TravelURL    = NextURL;
	Context.TravelType   = InTravelType;

	// Prevent crashing the game by attempting to connect to own listen server
	if ( Context.LastURL.HasOption(TEXT("Listen")) )
	{
		Context.LastURL.RemoveOption(TEXT("Listen"));
	}
}

void UEngine::SetClientTravel( UPendingNetGame *PendingNetGame, const TCHAR* NextURL, ETravelType InTravelType )
{
	FWorldContext &Context = GetWorldContextFromPendingNetGameChecked(PendingNetGame);

	// set TravelURL.  Will be processed safely on the next tick in UGameEngine::Tick().
	Context.TravelURL    = NextURL;
	Context.TravelType   = InTravelType;

	// Prevent crashing the game by attempting to connect to own listen server
	if ( Context.LastURL.HasOption(TEXT("Listen")) )
	{
		Context.LastURL.RemoveOption(TEXT("Listen"));
	}
}

void UEngine::SetClientTravelFromPendingGameNetDriver( UNetDriver *PendingGameNetDriverGame, const TCHAR* NextURL, ETravelType InTravelType )
{
	// Find WorldContext whose pendingNetGame's NetDriver is the passed in net driver
	for( int32 idx=0; idx < WorldList.Num(); ++idx)
	{
		FWorldContext &Context = WorldList[idx];
		if (Context.PendingNetGame && Context.PendingNetGame->NetDriver == PendingGameNetDriverGame)
		{
			SetClientTravel( Context.PendingNetGame, NextURL, InTravelType );
			return;
		}
	}
	check(false);
}

EBrowseReturnVal::Type UEngine::Browse( FWorldContext& WorldContext, FURL URL, FString& Error )
{
	Error = TEXT("");
	WorldContext.TravelURL = TEXT("");

	UE_LOGSTATUS(Log, TEXT("Started Browse: \"%s\""), *URL.ToString());

	if (WorldContext.World() && WorldContext.World()->GetNetDriver())
	{
		const TCHAR* InTickErrorString = TEXT("Attempting to call UEngine::Browse and destroy the net driver while the net driver is ticking. Instead try using UWorld::ServerTravel or APlayerController::ClientTravel.");
		if (!ensureMsgf(!WorldContext.World()->GetNetDriver()->IsInTick(), TEXT("%s"), InTickErrorString))
		{
			Error = InTickErrorString;
			return EBrowseReturnVal::Failure;
		}
	}

	// Convert .unreal link files.
	const TCHAR* LinkStr = TEXT(".unreal");//!!
	if( FCString::Strstr(*URL.Map,LinkStr)-*URL.Map==FCString::Strlen(*URL.Map)-FCString::Strlen(LinkStr) )
	{
		UE_LOGF(LogNet, Log,  "Link: %ls", *URL.Map );
		FString NewUrlString;
		if( GConfig->GetString( TEXT("Link")/*!!*/, TEXT("Server"), NewUrlString, *URL.Map ) )
		{
			// Go to link.
			URL = FURL( NULL, *NewUrlString, TRAVEL_Absolute );//!!
		}
		else
		{
			// Invalid link.
			Error = FText::Format( NSLOCTEXT("Engine", "InvalidLink", "Invalid Link: {0}"), FText::FromString( URL.Map ) ).ToString();
			return EBrowseReturnVal::Failure;
		}
	}

	// Crack the URL.
	UE_LOGF(LogNet, Log, "Browse: %ls", *URL.ToString() );

	// Handle it.
	if( !URL.Valid )
	{
		// Unknown URL.
		Error = FText::Format( NSLOCTEXT("Engine", "InvalidUrl", "Invalid URL: {0}"), FText::FromString( URL.ToString() ) ).ToString();
		BroadcastTravelFailure(WorldContext.World(), ETravelFailure::InvalidURL, Error);
		return EBrowseReturnVal::Failure;
	}
	else if (URL.HasOption(TEXT("failed")) || URL.HasOption(TEXT("closed")))
	{
		if (WorldContext.PendingNetGame)
		{
			CancelPending(WorldContext);
		}
		if (WorldContext.World() != NULL)
		{
			ResetLoaders( WorldContext.World()->GetOuter() );
		}

		if (WorldContext.WorldType == EWorldType::GameRPC)
		{
			UE_LOGF(LogNet, Log, "RPC connection failed; retrying...");

			// Clean up all current net-drivers before we create a new one
			// Need to copy the array as DestroyNamedNetDriver_Local mutates it
			{
				TArray<FNamedNetDriver> ActiveNetDrivers = WorldContext.ActiveNetDrivers;
				for (const FNamedNetDriver& ActiveNetDriver : ActiveNetDrivers)
				{
					if (ActiveNetDriver.NetDriver)
					{
						UE::Private::DestroyNamedNetDriver_Local(WorldContext, ActiveNetDriver.NetDriver->NetDriverName);
					}
				}
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}

			{
				LLM_SCOPE(ELLMTag::Networking);

				// Just reload the RPC world (as we have no real map to load)
				WorldContext.PendingNetGame = NewObject<UPendingNetGame>();
				WorldContext.PendingNetGame->Initialize(WorldContext.LastURL);
				WorldContext.PendingNetGame->InitNetDriver();
			}

			if (!WorldContext.PendingNetGame->NetDriver)
			{
				// UPendingNetGame will set the appropriate error code and connection lost type, so
				// we just have to propagate that message to the game.
				BroadcastTravelFailure(WorldContext.World(), ETravelFailure::PendingNetGameCreateFailure, WorldContext.PendingNetGame->ConnectionError);
				WorldContext.PendingNetGame = NULL;
				return EBrowseReturnVal::Failure;
			}

			return EBrowseReturnVal::Pending;
		}

		// Browsing after a failure, load default map
		UE_LOGF(LogNet, Log, "Connection failed; returning to Entry");
		
		const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
		const FURL DefaultURL = FURL(&URL, *(GameMapsSettings->GetGameDefaultMap() + GameMapsSettings->LocalMapOptions), TRAVEL_Partial);

		if (!LoadMap(WorldContext, DefaultURL, NULL, Error))
		{
			HandleBrowseToDefaultMapFailure(WorldContext, DefaultURL.ToString(), Error);
			return EBrowseReturnVal::Failure;
		}
		
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

		// now remove "failed" and "closed" options from LastURL so it doesn't get copied on to future URLs
		WorldContext.LastURL.RemoveOption(TEXT("failed"));
		WorldContext.LastURL.RemoveOption(TEXT("closed"));
		return EBrowseReturnVal::Success;
	}
	else if( URL.HasOption(TEXT("restart")) )
	{
		// Handle restarting.
		URL = WorldContext.LastURL;
	}

	// Handle normal URL's.
	if (GDisallowNetworkTravel && URL.HasOption(TEXT("listen")))
	{
		Error = NSLOCTEXT("Engine", "UsedCheatCommands", "Console commands were used which are disallowed in netplay.  You must restart the game to create a match.").ToString();
		BroadcastTravelFailure(WorldContext.World(), ETravelFailure::CheatCommands, Error);
		return EBrowseReturnVal::Failure;
	}

	if (OnOverrideBrowseURL.IsBound())
	{
		return OnOverrideBrowseURL.Execute(WorldContext, URL, Error);
	}

	if( URL.IsLocalInternal() )
	{
		// Local map file.
		return LoadMap( WorldContext, URL, NULL, Error ) ? EBrowseReturnVal::Success : EBrowseReturnVal::Failure;
	}
	else if( URL.IsInternal() && GIsClient )
	{
		// Network URL.
		if( WorldContext.PendingNetGame )
		{
			CancelPending(WorldContext);
		}

		// Clean up the netdriver/socket so that the pending level succeeds
		if (WorldContext.World() && ShouldShutdownWorldNetDriver())
		{
			ShutdownWorldNetDriver(WorldContext.World());
		}

		WorldContext.PendingNetGame = NewObject<UPendingNetGame>();
		WorldContext.PendingNetGame->Initialize(URL);
		WorldContext.PendingNetGame->InitNetDriver();

		if( !WorldContext.PendingNetGame )
		{
			// If the initial packet sent in InitNetDriver results in a socket error, HandleDisconnect() and CancelPending() may be called, which will null the PendingNetGame.
			Error = NSLOCTEXT("Engine", "PendingNetGameInitFailure", "Error initializing the network driver.").ToString();
			BroadcastTravelFailure(WorldContext.World(), ETravelFailure::PendingNetGameCreateFailure, Error);
			return EBrowseReturnVal::Failure;
		}

		if( !WorldContext.PendingNetGame->NetDriver )
		{
			// UPendingNetGame will set the appropriate error code and connection lost type, so
			// we just have to propagate that message to the game.
			BroadcastTravelFailure(WorldContext.World(), ETravelFailure::PendingNetGameCreateFailure, WorldContext.PendingNetGame->ConnectionError);
			WorldContext.PendingNetGame = NULL;
			return EBrowseReturnVal::Failure;
		}
		return EBrowseReturnVal::Pending;
	}
	else if( URL.IsInternal() )
	{
		// Invalid.
		Error = NSLOCTEXT("Engine", "ServerOpen", "Servers can't open network URLs").ToString();
		return EBrowseReturnVal::Failure;
	}
	{
		// External URL - disabled by default.
		// Client->Viewports(0)->Exec(TEXT("ENDFULLSCREEN"));
		// FPlatformProcess::LaunchURL( *URL.ToString(), TEXT(""), &Error );
		return EBrowseReturnVal::Failure;
	}
}

void UEngine::CancelPending(UNetDriver* PendingNetGameDriver)
{
	if (PendingNetGameDriver==NULL)
	{
		return;
	}

	// Find WorldContext whose pendingNetGame's NetDriver is the passed in net driver
	for( int32 idx=0; idx < WorldList.Num(); ++idx)
	{
		FWorldContext &Context = WorldList[idx];
		if (Context.PendingNetGame && Context.PendingNetGame->NetDriver == PendingNetGameDriver)
		{
			// Kill this PendingNetGame
			CancelPending(Context);
			check(Context.PendingNetGame == NULL);	// Verify PendingNetGame was cleared in CancelPending
		}
	}
}

void UEngine::CancelPending(FWorldContext &Context)
{
	if (Context.PendingNetGame && Context.PendingNetGame->NetDriver && Context.PendingNetGame->NetDriver->ServerConnection)
	{
		Context.PendingNetGame->NetDriver->ServerConnection->Close();
		UE::Private::DestroyNamedNetDriver_Local(Context, Context.PendingNetGame->NetDriver->NetDriverName);
		Context.PendingNetGame->NetDriver = NULL;
	}

	Context.PendingNetGame = NULL;
}

bool UEngine::WorldIsPIEInNewViewport(UWorld *InWorld)
{
	// UEditorEngine will override to check slate state
	return false;
}

void UEngine::CancelPending(UWorld *InWorld, UPendingNetGame *NewPendingNetGame)
{
	FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);
	CancelPending(Context);
	Context.PendingNetGame = NewPendingNetGame;
}

void UEngine::CancelAllPending()
{
	for( int32 idx=0; idx < WorldList.Num(); ++idx)
	{
		FWorldContext &Context = WorldList[idx];
		CancelPending(Context);
	}
}

void UEngine::BrowseToDefaultMap( FWorldContext& Context )
{
	FString Error;
	FURL DefaultURL;
	DefaultURL.LoadURLConfig( TEXT("DefaultPlayer"), GGameIni );
	const UGameMapsSettings* GameMapsSettings = GetDefault<UGameMapsSettings>();
	const FString& TextURL = GameMapsSettings->GetGameDefaultMap() + GameMapsSettings->LocalMapOptions;

	if (Browse( Context, FURL(&DefaultURL, *TextURL, TRAVEL_Partial), Error ) != EBrowseReturnVal::Success)
	{
		HandleBrowseToDefaultMapFailure(Context, TextURL, Error);
	}
}

void UEngine::HandleBrowseToDefaultMapFailure(FWorldContext& Context, const FString& TextURL, const FString& Error)
{
	UE_LOGF(LogNet, Error, "Failed to load default map (%ls). Error: (%ls)", *TextURL, *Error);
	const FText Message = FText::Format(NSLOCTEXT("Engine", "FailedToLoadDefaultMap", "Error '{0}'. Exiting."), FText::FromString(Error));
	FMessageDialog::Open(EAppMsgType::Ok, Message);

	// Even though we're probably going to shut down anyway, create a dummy world since a lot of code expects it.
	if (Context.World() == nullptr)
	{
		Context.SetCurrentWorld( UWorld::CreateWorld( Context.WorldType, false ) );
	}
}

void UEngine::TickWorldTravel(FWorldContext& Context, float DeltaSeconds)
{
	// Handle seamless traveling
	if (Context.SeamlessTravelHandler.IsInTransition())
	{
		// Note: SeamlessTravelHandler.Tick may automatically update Context.World and GWorld internally
		Context.SeamlessTravelHandler.Tick();
	}

	// Handle server traveling.
	if (Context.World() == nullptr)
	{
		UE_LOGF(LogLoad, Error, "UEngine::TickWorldTravel has no world after ticking seamless travel handler.");
		BrowseToDefaultMap(Context);
		BroadcastTravelFailure(Context.World(), ETravelFailure::ServerTravelFailure, TEXT("UEngine::TickWorldTravel has no world after ticking seamless travel handler."));
		return;
	}

	if( !Context.World()->NextURL.IsEmpty() )
	{
		Context.World()->NextSwitchCountdown -= DeltaSeconds; //-V595
		if( Context.World()->NextSwitchCountdown <= 0.f )
		{
			UE_LOGF(LogEngine, Log,  "Server switch level: %ls", *Context.World()->NextURL );
			if (Context.World()->GetAuthGameMode() != NULL)
			{
				Context.World()->GetAuthGameMode()->StartToLeaveMap();
			}
			FString Error;
			FString NextURL = Context.World()->NextURL;
			EBrowseReturnVal::Type Ret = Browse( Context, FURL(&Context.LastURL,*NextURL,(ETravelType)Context.World()->NextTravelType), Error ); //-V595
			if (Ret != EBrowseReturnVal::Success )
			{
				UE_LOGF(LogLoad, Warning, "UEngine::TickWorldTravel failed to Handle server travel to URL: %ls. Error: %ls", *NextURL, *Error);
				ensureMsgf(Ret != EBrowseReturnVal::Pending, TEXT("Server travel should never create a pending net game"));

				// Failed to load a new map
				if (Context.World() != nullptr)
				{
					// If we didn't change worlds, clear out NextURL so we don't do this again next frame.
					Context.World()->NextURL = TEXT("");
				}
				else
				{
					// Our old world got stomped out. Load the default map
					BrowseToDefaultMap(Context);
				}

				// Let people know that we failed to server travel
				BroadcastTravelFailure(Context.World(), ETravelFailure::ServerTravelFailure, Error);
			}
			return;
		}
	}

	// Handle client traveling.
	if( !Context.TravelURL.IsEmpty() )
	{	
		AGameModeBase* const GameMode = Context.World()->GetAuthGameMode(); //-V595
		if (GameMode)
		{
			GameMode->StartToLeaveMap();
		}

		FString Error, TravelURLCopy = Context.TravelURL;
		if (Browse( Context, FURL(&Context.LastURL,*TravelURLCopy,(ETravelType)Context.TravelType), Error ) == EBrowseReturnVal::Failure)
		{
			// If the failure resulted in no world being loaded (we unloaded our last world, then failed to load the new one)
			// then load the default map to avoid getting in a situation where we have no valid UWorld.
			if (Context.World() == NULL)
			{
				BrowseToDefaultMap(Context);
			}

			// Let people know that we failed to client travel
			BroadcastTravelFailure(Context.World(), ETravelFailure::ClientTravelFailure, Error);
		}
		check(Context.World() != NULL);
		return;
	}

	// Update the pending level.
	if ( OnOverridePendingNetGameUpdate.ExecuteIfBound(Context, DeltaSeconds) )
	{
		return;
	}

	if( Context.PendingNetGame )
	{
		Context.PendingNetGame->Tick( DeltaSeconds );
		if ( Context.PendingNetGame && Context.PendingNetGame->ConnectionError.Len() > 0 )
		{
			BroadcastNetworkFailure(NULL, Context.PendingNetGame->NetDriver, ENetworkFailure::PendingConnectionFailure, Context.PendingNetGame->ConnectionError);
			CancelPending(Context);
		}
		else if (Context.PendingNetGame && Context.PendingNetGame->bSuccessfullyConnected && !Context.PendingNetGame->bSentJoinRequest && !Context.PendingNetGame->bLoadedMapSuccessfully && (Context.OwningGameInstance == NULL || !Context.OwningGameInstance->DelayPendingNetGameTravel()))
		{
			if (Context.PendingNetGame->HasFailedTravel())
			{
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::TravelFailure, TEXT("Travel failed for unknown reason"));
			}
			else if (!MakeSureMapNameIsValid(Context.PendingNetGame->URL.Map))
			{
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::PackageMissing, Context.PendingNetGame->URL.Map);
			}
			else if (!Context.PendingNetGame->bLoadedMapSuccessfully)
			{
				// Attempt to load the map.
				FString Error;

				const bool bLoadedMapSuccessfully = LoadMap(Context, Context.PendingNetGame->URL, Context.PendingNetGame, Error);

				if (Context.PendingNetGame != nullptr)
				{
					if (!Context.PendingNetGame->LoadMapCompleted(this, Context, bLoadedMapSuccessfully, Error))
					{
						BrowseToDefaultMap(Context);
						BroadcastTravelFailure(Context.World(), ETravelFailure::LoadMapFailure, Error);
					}
				}
				else
				{
					BrowseToDefaultMap(Context);
					BroadcastTravelFailure(Context.World(), ETravelFailure::TravelFailure, Error);
				}
			}
		}
		
		if (Context.PendingNetGame && Context.PendingNetGame->bLoadedMapSuccessfully && (Context.OwningGameInstance == NULL || !Context.OwningGameInstance->DelayCompletionOfPendingNetGameTravel()))
		{
			if (!Context.PendingNetGame->HasFailedTravel() )
			{
				Context.PendingNetGame->TravelCompleted(this, Context);
				Context.PendingNetGame = nullptr;
			}
			else
			{
				CancelPending(Context);
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::LoadMapFailure, TEXT("Travel failed for unknown reason"));
			}
		}
	}
	else if (TransitionType == ETransitionType::WaitingToConnect)
	{
		TransitionType = ETransitionType::None;
	}

	return;
}

bool UEngine::LoadMap( FWorldContext& WorldContext, FURL URL, class UPendingNetGame* Pending, FString& Error )
{
	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Loading Map %s"), *URL.Map);
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("LoadMap - %s"), *URL.Map);
	STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "LoadMap - " ) + URL.Map )) );
	TRACE_BOOKMARK(TEXT("LoadMap - %s"), *URL.Map);

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UEngine::LoadMap"), STAT_LoadMap, STATGROUP_LoadTime);

	LLM_SCOPE(ELLMTag::LoadMapMisc);

	const FString OriginalURLMap = URL.Map;
	URL.Map = UWorld::RemovePIEPrefix(URL.Map);

	FDisableHitchDetectorScope SuspendHitchDetector;

	NETWORK_PROFILER(GNetworkProfiler.TrackSessionChange(true,URL));

	FMoviePlayerProxyBlock MoviePlayerBlock;
	Error = TEXT("");

	FLoadTimeTracker::Get().ResetRawLoadTimes();

	// make sure level streaming isn't frozen
	if (WorldContext.World())
	{
		WorldContext.World()->bIsLevelStreamingFrozen = false;
	}

	// send a callback message
	FCoreUObjectDelegates::PreLoadMapWithContext.Broadcast(WorldContext, URL.Map);
	FCoreUObjectDelegates::PreLoadMap.Broadcast(URL.Map);
	FMoviePlayerProxy::BlockingTick();

	// make sure there is a matching PostLoadMap() no matter how we exit
	struct FPostLoadMapCaller
	{
		FPostLoadMapCaller()
			: bCalled(false)
		{}

		~FPostLoadMapCaller()
		{
			if (!bCalled)
			{
				FCoreUObjectDelegates::PostLoadMapWithWorld.Broadcast(nullptr);
			}
		}

		void Broadcast(UWorld* World)
		{
			if (ensure(!bCalled))
			{
				bCalled = true;
				FCoreUObjectDelegates::PostLoadMapWithWorld.Broadcast(World);
			}
		}

	private:
		bool bCalled;

	} PostLoadMapCaller;

	// Cancel any pending texture streaming requests.  This avoids a significant delay on consoles 
	// when loading a map and there are a lot of outstanding texture streaming requests from the previous map.
	UTexture2D::CancelPendingTextureStreaming();

	// play a load map movie if specified in ini
	bStartedLoadMapMovie = false;

	// clean up any per-map loaded packages for the map we are leaving
	if (WorldContext.World() && WorldContext.World()->PersistentLevel)
	{
		CleanupPackagesToFullyLoad(WorldContext, FULLYLOAD_Map, WorldContext.World()->PersistentLevel->GetOutermost()->GetName());
	}

	// cleanup the existing per-game pacakges
	// @todo: It should be possible to not unload/load packages if we are going from/to the same GameMode.
	//        would have to save the game pathname here and pass it in to SetGameMode below
	CleanupPackagesToFullyLoad(WorldContext, FULLYLOAD_Game_PreLoadClass, TEXT(""));
	CleanupPackagesToFullyLoad(WorldContext, FULLYLOAD_Game_PostLoadClass, TEXT(""));
	CleanupPackagesToFullyLoad(WorldContext, FULLYLOAD_Mutator, TEXT(""));
	

	// Cancel any pending async map changes after flushing async loading. We flush async loading before canceling the map change
	// to avoid completion after cancellation to not leave references to the "to be changed to" level around. Async loading is
	// implicitly flushed again later on during garbage collection.
	FlushAsyncLoading();
	CancelPendingMapChange(WorldContext);
	WorldContext.SeamlessTravelHandler.CancelTravel();

	double	StartTime = FPlatformTime::Seconds();

	UE_LOGF(LogLoad, Log,  "LoadMap: %ls", *URL.ToString() );
	GInitRunaway();

#if !UE_BUILD_SHIPPING
	const bool bOldWorldWasShowingCollisionForHiddenComponents = WorldContext.World() && WorldContext.World()->bCreateRenderStateForHiddenComponentsWithCollsion;
#endif

	// Unload the current world
	if( WorldContext.World() )
	{
		// Notify listeners that all levels will be removed before we start tearing down the world.
		FWorldDelegates::PreLevelRemovedFromWorld.Broadcast(nullptr, WorldContext.World());
		WorldContext.World()->BeginTearingDown();

		if(!URL.HasOption(TEXT("quiet")) )
		{
			TransitionType = ETransitionType::Loading;
			TransitionDescription = URL.Map;
			if (URL.HasOption(TEXT("Game=")))
			{
				TransitionGameMode = URL.GetOption(TEXT("Game="), TEXT(""));
			}
			else
			{
				TransitionGameMode = TEXT("");
			}

			// Display loading screen.		
			// Check if a loading movie is playing.  If so it is not safe to redraw the viewport due to potential race conditions with font rendering
			bool bIsDesktopLoadingMovieCurrentlyPlaying = FCoreDelegates::IsLoadingMovieCurrentlyPlaying.IsBound() ? FCoreDelegates::IsLoadingMovieCurrentlyPlaying.Execute() : false;

			// Check if a loading movie is playing on an XR device as well. Redrawing viewports with a dynamic OpenXR stereo layer active
			// causes a race condition with CopyTexture_RenderThread that results in acquiring swapchain images twice without releasing
			bool bIsXRLoadingMovieCurrentlyPlaying = false;
			if (GEngine && GEngine->XRSystem.IsValid())
			{
				IXRLoadingScreen* XRLoadingScreen = GEngine->XRSystem->GetLoadingScreen();
				bIsXRLoadingMovieCurrentlyPlaying = XRLoadingScreen ? XRLoadingScreen->IsPlayingLoadingMovie() : false;
			}

			if(!bIsDesktopLoadingMovieCurrentlyPlaying && !bIsXRLoadingMovieCurrentlyPlaying)
			{
				LoadMapRedrawViewports();
			}

			TransitionType = ETransitionType::None;
		}

		// Clean up networking
		ShutdownWorldNetDriver(WorldContext.World());

		// Force mark all streaming levels for stream out
		WorldContext.World()->bIsLevelStreamingFrozen = false;
		WorldContext.World()->SetShouldForceUnloadStreamingLevels(true);

		// Make sure there are no pending visibility requests.
		WorldContext.World()->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

		// send a message that all levels are going away (NULL means every sublevel is being removed
		// without a call to RemoveFromWorld for each)
		//if (WorldContext.World()->GetNumLevels() > 1)
		{
			// TODO: Consider actually broadcasting for each level?
			FWorldDelegates::LevelRemovedFromWorld.Broadcast(nullptr, WorldContext.World());
		}

		// Disassociate the players from their PlayerControllers in this world.
		if (WorldContext.OwningGameInstance != nullptr)
		{
			for(auto It = WorldContext.OwningGameInstance->GetLocalPlayerIterator(); It; ++It)
			{
				ULocalPlayer *Player = *It;
				if(Player->PlayerController && Player->PlayerController->GetWorld() == WorldContext.World())
				{
					if(Player->PlayerController->GetPawn())
					{
						WorldContext.World()->DestroyActor(Player->PlayerController->GetPawn(), true);
					}
					WorldContext.World()->DestroyActor(Player->PlayerController, true);
					Player->PlayerController = nullptr;
				}
				// reset split join info so we'll send one after loading the new map if necessary
				Player->bSentSplitJoin = false;
				// When loading maps, clear out mids that are referenced as they may prevent the world from shutting down cleanly and the local player will not be cleaned up until later
				Player->CleanupViewState(); 
			}
		}

		WorldContext.World()->EndPlay(EEndPlayReason::LevelTransition);

		// Do this after destroying pawns/playercontrollers, in case that spawns new things (e.g. dropped weapons)
		WorldContext.World()->CleanupWorld();

		if( GEngine )
		{
			// clear any "DISPLAY" properties referencing level objects
			if (GEngine->GameViewport != nullptr)
			{
				ClearDebugDisplayProperties();
			}

			GEngine->WorldDestroyed(WorldContext.World());
		}
		WorldContext.World()->RemoveFromRoot();

		// mark everything else contained in the world to be deleted
		for (auto LevelIt(WorldContext.World()->GetLevelIterator()); LevelIt; ++LevelIt)
		{
			const ULevel* Level = *LevelIt;
			if (Level)
			{
				CastChecked<UWorld>(Level->GetOuter())->MarkObjectsPendingKill();
			}
		}

		for (ULevelStreaming* LevelStreaming : WorldContext.World()->GetStreamingLevels())
		{
			// If an unloaded levelstreaming still has a loaded level we need to mark its objects to be deleted as well
			if (LevelStreaming->GetLoadedLevel() && (!LevelStreaming->ShouldBeLoaded() || !LevelStreaming->ShouldBeVisible()))
			{
				CastChecked<UWorld>(LevelStreaming->GetLoadedLevel()->GetOuter())->MarkObjectsPendingKill();
			}
		}

		// Stop all audio to remove references to current level.
		if (FAudioDevice* AudioDevice = WorldContext.World()->GetAudioDeviceRaw())
		{
			AudioDevice->Flush(WorldContext.World());
			AudioDevice->SetTransientPrimaryVolume(1.0f);
		}

		WorldContext.SetCurrentWorld(nullptr);
	}

	FMoviePlayerProxy::BlockingTick();
	const bool bGcAllowed = FGarbageCollectionPause::Get().OnGc(EGcRequestType::LoadMapTrim);
	if (!bGcAllowed)
	{
		// GC is paused so don't TrimMemory, but signal a full purge at the next GC
		ForceGarbageCollection(true);
	}
	else if (GDelayTrimMemoryDuringMapLoadMode == 0)
	{
		// trim memory to clear up allocations from the previous level (also flushes rendering)
		TrimMemory();
	}

	// Cancels the Forced StreamType for textures using a timer.
	if (!IStreamingManager::HasShutdown())
	{
		IStreamingManager::Get().CancelForcedResources();
	}

	if (FPlatformProperties::RequiresCookedData())
	{
		appDefragmentTexturePool();
		appDumpTextureMemoryStats(TEXT(""));
	}

	// if we aren't trimming memory above or GC is paused, then the world won't be fully cleaned up at this point, so don't bother checking
	if (GDelayTrimMemoryDuringMapLoadMode == 0 && bGcAllowed)
	{
		// Dump info
		CheckAndHandleStaleWorldObjectReferences(&WorldContext);
	}

	FMoviePlayerProxy::BlockingTick();

	WorldContext.OwningGameInstance->PreloadContentForURL(URL);

	UPackage* WorldPackage = NULL;
	UWorld*	NewWorld = NULL;

	// If this world is a PIE instance, we need to check if we are traveling to another PIE instance's world.
	// If we are, we need to set the PIERemapPrefix so that we load a copy of that world, instead of loading the
	// PIE world directly.
	if (!WorldContext.PIEPrefix.IsEmpty())
	{
		for (const FWorldContext& WorldContextFromList : WorldList)
		{
			// We want to ignore our own PIE instance so that we don't unnecessarily set the PIERemapPrefix if we are not traveling to
			// a server.
			if (WorldContextFromList.World() != WorldContext.World())
			{
				if (!WorldContextFromList.PIEPrefix.IsEmpty() && OriginalURLMap.Contains(WorldContextFromList.PIEPrefix))
				{
					FString SourceWorldPackage = URL.Map;

					// We are loading a new world for this context, so clear out PIE fixups that might be lingering.
					// (note we dont want to do this in DuplicateWorldForPIE, since that is also called on streaming worlds.
					UE::SetPlayInEditorID(WorldContext.PIEInstance);
					UpdatePlayInEditorWorldDebugString(&WorldContext);
					FLazyObjectPtr::ResetPIEFixups();

					NewWorld = UWorld::DuplicateWorldForPIE(SourceWorldPackage, nullptr);
					if (NewWorld == nullptr)
					{
						NewWorld = CreatePIEWorldByLoadingFromPackage(WorldContext, SourceWorldPackage, WorldPackage);
						if (NewWorld == nullptr)
						{
							Error = FString::Printf(TEXT("Failed to load package '%s' while in PIE"), *SourceWorldPackage);
							return false;
						}
					}
					else
					{
						WorldPackage = CastChecked<UPackage>(NewWorld->GetOuter());
					}

					NewWorld->StreamingLevelsPrefix = UWorld::BuildPIEPackagePrefix(WorldContext.PIEInstance);
					GIsPlayInEditorWorld = true;
				}
			}
		}
	}

	// Is this a minimal net RPC world?
	if (WorldContext.WorldType == EWorldType::GameRPC)
	{
		UGameInstance::CreateMinimalNetRPCWorld(*URL.Map, WorldPackage, NewWorld);
	}

	const FString URLTrueMapName = URL.Map;

	// Normal map loading
	if (NewWorld == NULL)
	{
		// Set the world type in the static map, so that UWorld::PostLoad can set the world type
		const FName URLMapFName = FName(*URL.Map);
		UWorld::WorldTypePreLoadMap.FindOrAdd( URLMapFName ) = WorldContext.WorldType;

		// See if the level is already in memory
		WorldPackage = FindPackage(nullptr, *URL.Map);

		bool bPackageAlreadyLoaded = (WorldPackage != nullptr);

		// If the level isn't already in memory, load level from disk
		if (WorldPackage == nullptr)
		{
			WorldPackage = LoadPackage(nullptr, *URL.Map, (WorldContext.WorldType == EWorldType::PIE ? LOAD_PackageForPIE : LOAD_None));
		}

		// Clean up the world type list now that PostLoad has occurred
		UWorld::WorldTypePreLoadMap.Remove( URLMapFName );

		if (WorldPackage == nullptr)
		{
			// it is now the responsibility of the caller to deal with a NULL return value and alert the user if necessary
			Error = FString::Printf(TEXT("Failed to load package '%s'"), *URL.Map);
			return false;
		}

		// Find the newly loaded world.
		NewWorld = UWorld::FindWorldInPackage(WorldPackage);

		// If the world was not found, it could be a redirector to a world. If so, follow it to the destination world.
		if ( !NewWorld )
		{
			NewWorld = UWorld::FollowWorldRedirectorInPackage(WorldPackage);
			if ( NewWorld )
			{
				// Treat this as an already loaded package because we were loaded by the redirector
				bPackageAlreadyLoaded = true;
				WorldPackage = NewWorld->GetOutermost();
			}
		}

		// This can still be null if the package name is ambiguous, for example if there exists a umap and uasset with the same
		// name.
		if (NewWorld == nullptr)
		{
			// it is now the responsibility of the caller to deal with a NULL return value and alert the user if necessary
			Error = FString::Printf(TEXT("Failed to load package '%s'"), *URL.Map);
			return false;
		}


		NewWorld->PersistentLevel->HandleLegacyMapBuildData();

		FScopeCycleCounterUObject MapScope(WorldPackage);

		if (WorldContext.WorldType == EWorldType::PIE)
		{
			// If we are a PIE world and the world we just found is already initialized, then we're probably reloading the editor world and we
			// need to create a PIE world by duplication instead
			if (bPackageAlreadyLoaded || NewWorld->WorldType == EWorldType::Editor)
			{
				if (WorldContext.PIEInstance == -1)
				{
					// Assume if we get here, that it's safe to just give a PIE instance so that we can duplicate the world 
					//	If we won't duplicate the world, we'll refer to the existing world (most likely the editor version, and it can be modified under our feet, which is bad)
					// So far, the only known way to get here is when we use the console "open" command while in a client PIE instance connected to non PIE server 
					// (i.e. multi process PIE where client is in current editor process, and dedicated server was launched as separate process)
					WorldContext.PIEInstance = 0;
				}

				NewWorld = CreatePIEWorldByDuplication(WorldContext, NewWorld, URL.Map);
				// CreatePIEWorldByDuplication clears GIsPlayInEditorWorld so set it again
				GIsPlayInEditorWorld = true;
			}
			// Otherwise we are probably loading new map while in PIE, so we need to rename world package and all streaming levels
			else if (WorldContext.PIEInstance != -1)
			{
				NewWorld->RenameToPIEWorld(WorldContext.PIEInstance);
			}
			ResetPIEAudioSetting(NewWorld);

#if WITH_EDITOR
			// PIE worlds should use the same feature level as the editor
			if (WorldContext.PIEWorldFeatureLevel != ERHIFeatureLevel::Num && NewWorld->GetFeatureLevel() != WorldContext.PIEWorldFeatureLevel)
			{
				NewWorld->ShaderPlatformChanged(GShaderPlatformForFeatureLevel[WorldContext.PIEWorldFeatureLevel]);
			}
#endif
		}
		else if (WorldContext.WorldType == EWorldType::Game)
		{
			// If we are a game world and the world we just found is already initialized, then we're probably trying to load
			// an independent fresh copy of the world into a different context. Create a package with a prefixed name
			// and load the world from disk to keep the instances independent. If this is the case, assume the creator
			// of the FWorldContext was aware and set WorldContext.PIEInstance to a valid value.
			if (NewWorld->bIsWorldInitialized && WorldContext.PIEInstance != -1)
			{
				NewWorld = CreatePIEWorldByLoadingFromPackage(WorldContext, URL.Map, WorldPackage);

				if (NewWorld == nullptr)
				{
					Error = FString::Printf(TEXT("Failed to load package '%s' into a new game world."), *URL.Map);
					return false;
				}
			}
		}
	}
	NewWorld->SetGameInstance(WorldContext.OwningGameInstance);

	GWorld = NewWorld;

	WorldContext.SetCurrentWorld(NewWorld);
	WorldContext.World()->WorldType = WorldContext.WorldType;

#if !UE_BUILD_SHIPPING
	GWorld->bCreateRenderStateForHiddenComponentsWithCollsion = bOldWorldWasShowingCollisionForHiddenComponents;
#endif

	// PIE worlds are not added to root and are initialized differently
	if (WorldContext.WorldType == EWorldType::PIE)
	{
		check(WorldContext.World()->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor));
		WorldContext.World()->ClearFlags(RF_Standalone);

		PostCreatePIEWorld(WorldContext.World());
	}
	else
	{
		WorldContext.World()->AddToRoot();

		// The world should not have been initialized before this
		if (ensure(!WorldContext.World()->bIsWorldInitialized))
		{
			WorldContext.World()->InitWorld();
		}
	}

#if WITH_STATE_STREAM
	FStateStreamLaneScope LaneScope(NewWorld->GetStateStreamManager(), NewWorld->GetLaneId());
#endif

	// Handle pending level.
	if( Pending )
	{
		check(Pending == WorldContext.PendingNetGame);
		MovePendingLevel(WorldContext);
	}
	else
	{
		check(!WorldContext.World()->GetNetDriver());
	}

	WorldContext.World()->SetGameMode(URL);

	if (FAudioDevice* AudioDevice = WorldContext.World()->GetAudioDeviceRaw())
	{
		if (AWorldSettings* WorldSettings = WorldContext.World()->GetWorldSettings())
		{
			AudioDevice->SetDefaultBaseSoundMix(WorldSettings->DefaultBaseSoundMix);
		}
		else
		{
			UE_LOGF(LogInit, Warning, "Unable to get world settings. Can't initialize default base soundmix.");
		}
	}

	// Listen for clients.
	if (Pending == NULL && (!GIsClient || URL.HasOption(TEXT("Listen"))))
	{
		if (!WorldContext.World()->Listen(URL))
		{
			UE_LOGF(LogNet, Error, "LoadMap: failed to Listen(%ls)", *URL.ToString());
		}
	}

	const TCHAR* MutatorString = URL.GetOption(TEXT("Mutator="), TEXT(""));
	if (MutatorString)
	{
		TArray<FString> Mutators;
		FString(MutatorString).ParseIntoArray(Mutators, TEXT(","), true);

		for (int32 MutatorIndex = 0; MutatorIndex < Mutators.Num(); MutatorIndex++)
		{
			LoadPackagesFully(WorldContext.World(), FULLYLOAD_Mutator, Mutators[MutatorIndex]);
		}
	}

	// Process global shader results before we try to render anything
	// Do this before we register components, as USkinnedMeshComponents require the GPU skin cache global shaders when creating render state.
	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(false, true);
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UEngine::LoadMap.LoadPackagesFully"), STAT_LoadMap_LoadPackagesFully, STATGROUP_LoadTime);

		// load any per-map packages
		check(WorldContext.World()->PersistentLevel);
		LoadPackagesFully(WorldContext.World(), FULLYLOAD_Map, WorldContext.World()->PersistentLevel->GetOutermost()->GetName());

		// Make sure "always loaded" sub-levels are fully loaded
		WorldContext.World()->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

		if (!GIsEditor && !IsRunningDedicatedServer())
		{
			// If requested, duplicate dynamic levels here after the source levels are created.
			WorldContext.World()->DuplicateRequestedLevels(FName(*URL.Map));
		}
	}
	FMoviePlayerProxy::BlockingTick();

#if WITH_EDITOR
	// Gives a chance to any assets being used for PIE/game to complete
	FAssetCompilingManager::Get().ProcessAsyncTasks();
#endif

	// Note that AI system will be created only if ai-system-creation conditions are met
	WorldContext.World()->CreateAISystem();

	// Initialize gameplay for the level.
	{
		FRegisterComponentContext Context(WorldContext.World());
		WorldContext.World()->InitializeActorsForPlay(URL, true, &Context);
	}

	// calling it after InitializeActorsForPlay has been called to have all potential bounding boxed initialized
	FNavigationSystem::AddNavigationSystemToWorld(*WorldContext.World(), FNavigationSystemRunMode::GameMode);

	// Flush streaming again in case registered world subsystems streaming interfaces have some more work to do
	WorldContext.World()->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

	// Remember the URL. Put this before spawning player controllers so that
	// a player controller can get the map name during initialization and
	// have it be correct
	WorldContext.LastURL = URL;
	WorldContext.LastURL.Map = URLTrueMapName;

	if (WorldContext.World()->GetNetMode() == NM_Client)
	{
		WorldContext.LastRemoteURL = URL;
	}
	FMoviePlayerProxy::BlockingTick();

	// Spawn play actors for all active local players
	if (WorldContext.OwningGameInstance != NULL)
	{
		for(auto It = WorldContext.OwningGameInstance->GetLocalPlayerIterator(); It; ++It)
		{
			FString Error2;
			if(!(*It)->SpawnPlayActor(URL.ToString(1),Error2,WorldContext.World()))
			{
				UE_LOGF(LogEngine, Fatal, "Couldn't spawn player: %ls", *Error2);
			}
		}
	}

	// Prime texture streaming.
	IStreamingManager::Get().NotifyLevelChange();

	if (GEngine && GEngine->XRSystem.IsValid())
	{
		GEngine->XRSystem->OnBeginPlay(WorldContext);
	}
	WorldContext.World()->BeginPlay();

	FMoviePlayerProxy::BlockingTick();
	// send a callback message
	PostLoadMapCaller.Broadcast(WorldContext.World());
	MoviePlayerBlock.Finish();
	FMoviePlayerProxy::BlockingForceFinished();

	WorldContext.World()->bWorldWasLoadedThisTick = true;

	// We want to update streaming immediately so that there's no tick prior to processing any levels that should be initially visible
	// that requires calculating the scene, so redraw everything now to take care of it all though don't present the frame.
	RedrawViewports(false);

	// RedrawViewports() may have added a dummy playerstart location. Remove all views to start from fresh the next Tick().
	IStreamingManager::Get().RemoveStreamingViews( RemoveStreamingViews_All );

	// See if we need to record network demos
	if ( WorldContext.World()->GetAuthGameMode() == NULL || !WorldContext.World()->GetAuthGameMode()->IsHandlingReplays() )
	{
		if ( URL.HasOption( TEXT( "DemoRec" ) ) && WorldContext.OwningGameInstance != nullptr )
		{
			const TCHAR* DemoRecName = URL.GetOption( TEXT( "DemoRec=" ), NULL );

			// Record the demo, optionally with the specified custom name.
			WorldContext.OwningGameInstance->StartRecordingReplay( FString(DemoRecName), WorldContext.World()->GetMapName(), URL.Op );
		}
	}

	STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "LoadMapComplete - " ) + URL.Map )) );
	TRACE_BOOKMARK(TEXT("LoadMapComplete - %s"), *URL.Map);

	double StopTime = FPlatformTime::Seconds();

	UE_LOGF(LogLoad, Log, "Took %f seconds to LoadMap(%ls)", StopTime - StartTime, *URL.Map);
	FLoadTimeTracker::Get().DumpRawLoadTimes();
	WorldContext.OwningGameInstance->LoadComplete(StopTime - StartTime, *URL.Map);

	// perform the delayed TrimMemory if desired
	if (GDelayTrimMemoryDuringMapLoadMode != 0 && bGcAllowed)
	{
		TrimMemory();

		if (GDelayTrimMemoryDuringMapLoadMode == 1)
		{
			// all future map loads should be normal
			GDelayTrimMemoryDuringMapLoadMode = 0;
		}
	}

	// Successfully started local level.
	UE_LOGSTATUS(Log, TEXT("Load map complete %s"), *URL.Map);
	return true;
}

void UEngine::TrimMemory()
{
	// Clean up the previous level out of memory.
	{
		if (GEngine != nullptr)
		{
			GEngine->bGCPerformingFullPurge = true;
			
			// Reset this to avoid a redundant full purge if there's one pending
			GEngine->bFullPurgeTriggered = false;
			GEngine->TimeSinceLastPendingKillPurge = 0.0f;
		}

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);

		if (GEngine != nullptr)
		{
			GEngine->bGCPerformingFullPurge = false;
		}
	}

	// For platforms which manage GPU memory directly we must Enqueue a flush, and wait for it to be processed
	// so that any pending frees that depend on the GPU will be processed.  Otherwise a whole map's worth of GPU memory
	// may be unavailable to load the next one.
	ENQUEUE_RENDER_COMMAND(FlushCommand)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
#if !WITH_EDITOR
			GetRendererModule().ReleaseVirtualTexturePendingResources();
#endif	
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		});
	FlushRenderingCommands();

	// Ask systems to trim memory where possible
	FCoreDelegates::GetMemoryTrimDelegate().Broadcast();
}

void UEngine::BlockTillLevelStreamingCompleted(UWorld* InWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UEngine_BlockTillLevelStreamingCompleted);

	check(InWorld);
	InWorld->BlockTillLevelStreamingCompleted();
}

void UEngine::CleanupPackagesToFullyLoad(FWorldContext &Context, EFullyLoadPackageType FullyLoadType, const FString& Tag)
{
	//UE_LOGF(LogEngine, Log, "------------------ CleanupPackagesToFullyLoad: %i, %ls",(int32)FullyLoadType, *Tag);
	for (int32 MapIndex = 0; MapIndex < Context.PackagesToFullyLoad.Num(); MapIndex++)
	{
		FFullyLoadedPackagesInfo& PackagesInfo = Context.PackagesToFullyLoad[MapIndex];
		// is this entry for the map/game?
		if (PackagesInfo.FullyLoadType == FullyLoadType && (PackagesInfo.Tag == Tag || Tag == TEXT("")))
		{
			// mark all objects from this map as unneeded
			for (int32 ObjectIndex = 0; ObjectIndex < PackagesInfo.LoadedObjects.Num(); ObjectIndex++)
			{
				//				UE_LOGF(LogEngine, Log, "Removing %ls from root", *PackagesInfo.LoadedObjects(ObjectIndex)->GetFullName());
				PackagesInfo.LoadedObjects[ObjectIndex]->RemoveFromRoot();	
			}
			// empty the array of pointers to the objects
			PackagesInfo.LoadedObjects.Empty();
		}
	}
}

void UEngine::CancelPendingMapChange(FWorldContext &Context)
{
	// Empty intermediate arrays.
	Context.LevelsToLoadForPendingMapChange.Empty();
	Context.LoadedLevelsForPendingMapChange.Empty();

	// Reset state and make sure conditional map change doesn't fire.
	Context.PendingMapChangeFailureDescription	= TEXT("");
	Context.bShouldCommitPendingMapChange		= false;

	// Reset array of levels to prepare for client.
	if( Context.World() )
	{
		Context.World()->PreparingLevelNames.Empty();
	}
}

/** Clear out the debug properties array that is storing values to show on the screen */
void UEngine::ClearDebugDisplayProperties()
{
	for (int32 i = 0; i < GameViewport->DebugProperties.Num(); i++)
	{
		if (GameViewport->DebugProperties[i].Obj == NULL)
		{
			GameViewport->DebugProperties.RemoveAt(i, 1);
			i--;
		}
		else
		{
			for (UObject* TestObj = GameViewport->DebugProperties[i].Obj; TestObj != NULL; TestObj = TestObj->GetOuter())
			{
				if (TestObj->IsA(ULevel::StaticClass()) || TestObj->IsA(UWorld::StaticClass()) || TestObj->IsA(AActor::StaticClass()))
				{
					GameViewport->DebugProperties.RemoveAt(i, 1);
					i--;
					break;
				}
			}
		}
	}
}

void UEngine::MovePendingLevel(FWorldContext &Context)
{
	check(Context.World());
	check(Context.PendingNetGame);

	Context.World()->SetNetDriver(Context.PendingNetGame->NetDriver);

	UNetDriver* NetDriver = Context.PendingNetGame->NetDriver;
	if (NetDriver)
	{
		// The pending net driver is renamed to the current "game net driver"
		NetDriver->SetNetDriverName(NAME_GameNetDriver);
		NetDriver->SetWorld(Context.World());

		FLevelCollection& SourceLevels = Context.World()->FindOrAddCollectionByType(ELevelCollectionType::DynamicSourceLevels);
		SourceLevels.SetNetDriver(NetDriver);

		if (FLevelCollection* StaticLevels = Context.World()->FindCollectionByType(ELevelCollectionType::StaticLevels))
		{
			StaticLevels->SetNetDriver(NetDriver);
		}
	}

	// Attach the DemoNetDriver to the world if there is one
	if (UDemoNetDriver* DemoNetDriver = Context.PendingNetGame->GetDemoNetDriver())
	{
		DemoNetDriver->SetWorld(Context.World());
		Context.World()->SetDemoNetDriver(DemoNetDriver);

		FLevelCollection& MainLevels = Context.World()->FindOrAddCollectionByType(ELevelCollectionType::DynamicSourceLevels);
		MainLevels.SetDemoNetDriver(DemoNetDriver);
	}

	// Reset the Navigation System
	Context.World()->SetNavigationSystem(nullptr);
}

void UEngine::LoadPackagesFully(UWorld * InWorld, EFullyLoadPackageType FullyLoadType, const FString& Tag)
{
	FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);
	//UE_LOGF(LogEngine, Log, "------------------ LoadPackagesFully: %i, %ls",(int32)FullyLoadType, *Tag);

	// look for all entries for the given map
	for (int32 MapIndex = ((Tag == TEXT("___TAILONLY___")) ? Context.PackagesToFullyLoad.Num() - 1 : 0); MapIndex < Context.PackagesToFullyLoad.Num(); MapIndex++)
	{
		FFullyLoadedPackagesInfo& PackagesInfo = Context.PackagesToFullyLoad[MapIndex];
		/*
		for (int32 PackageIndex = 0; PackageIndex < PackagesInfo.PackagesToLoad.Num(); PackageIndex++)
		{
		UE_LOGF(LogEngine, Log, "--------------------- Considering: %i, %ls, %ls",(int32)PackagesInfo.FullyLoadType, *PackagesInfo.Tag, *PackagesInfo.PackagesToLoad(PackageIndex).ToString());
		}
		*/

		// is this entry for the map/game?
		if (PackagesInfo.FullyLoadType == FullyLoadType && (PackagesInfo.Tag == Tag || Tag == TEXT("") || Tag == TEXT("___TAILONLY___")))
		{
			// go over all packages that need loading
			for (int32 PackageIndex = 0; PackageIndex < PackagesInfo.PackagesToLoad.Num(); PackageIndex++)
			{
				//UE_LOGF(LogEngine, Log, "------------------------ looking for %ls",*PackagesInfo.PackagesToLoad(PackageIndex).ToString());

				// look for the package in the package cache
				FString SFPackageName = PackagesInfo.PackagesToLoad[PackageIndex].ToString() + STANDALONE_SEEKFREE_SUFFIX;
				bool bFoundFile = false;
				FString PackagePath;
				if (FPackageName::DoesPackageExist(SFPackageName, &PackagePath))
				{
					bFoundFile = true;
				}
				else if ( (FPackageName::DoesPackageExist(PackagesInfo.PackagesToLoad[PackageIndex].ToString(), &PackagePath)) )
				{
					bFoundFile = true;
				}
				if (bFoundFile)
				{
					// load the package
					// @todo: This would be nice to be async probably, but how would we add it to the root? (LOAD_AddPackageToRoot?)
					//UE_LOGF(LogEngine, Log, "------------------ Fully loading %ls", *PackagePath);
					UPackage* Package = LoadPackage(NULL, *PackagePath, 0);

					// add package to root so we can find it
					Package->AddToRoot();

					// remember the object for unloading later
					PackagesInfo.LoadedObjects.Add(Package);

					// add the objects to the root set so that it will not be GC'd
					for (TObjectIterator<UObject> It; It; ++It)
					{
						if (It->IsIn(Package))
						{
							//							UE_LOGF(LogEngine, Log, "Adding %ls to root", *It->GetFullName());
							It->AddToRoot();

							// remember the object for unloading later
							PackagesInfo.LoadedObjects.Add(*It);
						}
					}
				}
				else
				{
					UE_LOGF(LogEngine, Log, "Failed to find Package %ls to FullyLoad [FullyLoadType = %d, Tag = %ls]", *PackagesInfo.PackagesToLoad[PackageIndex].ToString(), (int32)FullyLoadType, *Tag);
				}
			}
		}
		/*
		else
		{
		UE_LOGF(LogEngine, Log, "DIDN't MATCH!!!");
		for (int32 PackageIndex = 0; PackageIndex < PackagesInfo.PackagesToLoad.Num(); PackageIndex++)
		{
		UE_LOGF(LogEngine, Log, "DIDN't MATCH!!! %i, \"%ls\"(\"%ls\"), %ls",(int32)PackagesInfo.FullyLoadType, *PackagesInfo.Tag, *Tag, *PackagesInfo.PackagesToLoad(PackageIndex).ToString());
		}
		}
		*/
	}
}


void UEngine::UpdateTransitionType(UWorld *CurrentWorld)
{
	// Update the transition screen.
	if(TransitionType == ETransitionType::Connecting)
	{
		// Check to see if all players have finished connecting.
		TransitionType = ETransitionType::None;

		FWorldContext &Context = GetWorldContextFromWorldChecked(CurrentWorld);
		if (Context.OwningGameInstance != NULL)
		{
			for(auto It = Context.OwningGameInstance->GetLocalPlayerIterator(); It; ++It)
			{
				if(!(*It)->PlayerController)
				{
					// This player has not received a PlayerController from the server yet, so leave the connecting screen up.
					TransitionType = ETransitionType::Connecting;
					break;
				}
			}
		}
	}
	else if(TransitionType == ETransitionType::None || TransitionType == ETransitionType::Paused)
	{
		// Display a paused screen if the game is paused.
		TransitionType = (CurrentWorld->GetWorldSettings()->GetPauserPlayerState() != NULL) ? ETransitionType::Paused : ETransitionType::None;
	}
}

FWorldContext& UEngine::CreateNewWorldContext(EWorldType::Type WorldType)
{
	FWorldContext* NewWorldContext = new FWorldContext;
	WorldList.Add(NewWorldContext);
	NewWorldContext->WorldType = WorldType;
	NewWorldContext->ContextHandle = FName(*FString::Printf(TEXT("Context_%d"), NextWorldContextHandle++));

	return *NewWorldContext;
}

FWorldContext& HandleInvalidWorldContext()
{
	if (!IsRunningCommandlet())
	{
		UE_LOGF(LogLoad, Error, "WorldContext requested with invalid context object." );
		check(false);
	}

	return GEngine->CreateNewWorldContext(EWorldType::None);
}

FWorldContext* UEngine::GetWorldContextFromHandle(const FName WorldContextHandle)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.ContextHandle == WorldContextHandle)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromHandle(const FName WorldContextHandle) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.ContextHandle == WorldContextHandle)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromHandleChecked(const FName WorldContextHandle)
{
	if (FWorldContext* WorldContext = GetWorldContextFromHandle(WorldContextHandle))
	{
		return *WorldContext;
	}

	UE_LOGF(LogLoad, Warning, "WorldContext requested with invalid context handle %ls", *WorldContextHandle.ToString());
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromHandleChecked(const FName WorldContextHandle) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromHandle(WorldContextHandle))
	{
		return *WorldContext;
	}

	UE_LOGF(LogLoad, Warning, "WorldContext requested with invalid context handle %ls", *WorldContextHandle.ToString());
	return HandleInvalidWorldContext();
}

FWorldContext* UEngine::GetWorldContextFromWorld(const UWorld* InWorld)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.World() == InWorld)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromWorld(const UWorld* InWorld) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.World() == InWorld)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromWorldChecked(const UWorld *InWorld)
{
	if (FWorldContext* WorldContext = GetWorldContextFromWorld(InWorld))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromWorldChecked(const UWorld *InWorld) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromWorld(InWorld))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

UGameViewportClient* UEngine::GameViewportForWorld(const UWorld *InWorld) const
{
	const FWorldContext* Context = GetWorldContextFromWorld(InWorld);
	return (Context ? Context->GameViewport : NULL);
}

void UEngine::SetIsVanillaProduct(bool bInIsVanillaProduct)
{
	// set bIsVanillaProduct and if it changes broadcast the core delegate
	static bool bFirstCall = true;
	if (bFirstCall || bInIsVanillaProduct != bIsVanillaProduct)
	{
		bFirstCall = false;
		bIsVanillaProduct = bInIsVanillaProduct;
		FCoreDelegates::IsVanillaProductChanged.Broadcast(bIsVanillaProduct);
	}
}

FWorldContext* UEngine::GetWorldContextFromGameViewport(const UGameViewportClient *InViewport)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.GameViewport == InViewport)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromGameViewport(const UGameViewportClient *InViewport) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.GameViewport == InViewport)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromGameViewportChecked(const UGameViewportClient *InViewport)
{
	if (FWorldContext* WorldContext = GetWorldContextFromGameViewport(InViewport))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromGameViewportChecked(const UGameViewportClient *InViewport) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromGameViewport(InViewport))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

FWorldContext* UEngine::GetWorldContextFromPendingNetGame(const UPendingNetGame *InPendingNetGame)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.PendingNetGame == InPendingNetGame)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromPendingNetGame(const UPendingNetGame *InPendingNetGame) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.PendingNetGame == InPendingNetGame)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromPendingNetGameChecked(const UPendingNetGame *InPendingNetGame)
{
	if (FWorldContext* WorldContext = GetWorldContextFromPendingNetGame(InPendingNetGame))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromPendingNetGameChecked(const UPendingNetGame *InPendingNetGame) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromPendingNetGame(InPendingNetGame))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

FWorldContext* UEngine::GetWorldContextFromPendingNetGameNetDriver(const UNetDriver *InPendingNetDriver)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.PendingNetGame && WorldContext.PendingNetGame->NetDriver == InPendingNetDriver)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromPendingNetGameNetDriver(const UNetDriver *InPendingNetDriver) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.PendingNetGame && WorldContext.PendingNetGame->NetDriver == InPendingNetDriver)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromPendingNetGameNetDriverChecked(const UNetDriver *InPendingNetDriver)
{
	if (FWorldContext* WorldContext = GetWorldContextFromPendingNetGameNetDriver(InPendingNetDriver))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromPendingNetGameNetDriverChecked(const UNetDriver *InPendingNetDriver) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromPendingNetGameNetDriver(InPendingNetDriver))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

FWorldContext* UEngine::GetWorldContextFromPIEInstance(const int32 PIEInstance)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.WorldType == EWorldType::PIE && WorldContext.PIEInstance == PIEInstance)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

const FWorldContext* UEngine::GetWorldContextFromPIEInstance(const int32 PIEInstance) const
{
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.WorldType == EWorldType::PIE && WorldContext.PIEInstance == PIEInstance)
		{
			return &WorldContext;
		}
	}
	return nullptr;
}

FWorldContext& UEngine::GetWorldContextFromPIEInstanceChecked(const int32 PIEInstance)
{
	if (FWorldContext* WorldContext = GetWorldContextFromPIEInstance(PIEInstance))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

const FWorldContext& UEngine::GetWorldContextFromPIEInstanceChecked(const int32 PIEInstance) const
{
	if (const FWorldContext* WorldContext = GetWorldContextFromPIEInstance(PIEInstance))
	{
		return *WorldContext;
	}
	return HandleInvalidWorldContext();
}

UWorld* UEngine::GetCurrentPlayWorld(UWorld* PossiblePlayWorld) const
{
	UWorld* BestWorld = nullptr;
	for (const FWorldContext& WorldContext : WorldList)
	{
		if (PossiblePlayWorld && WorldContext.World() == PossiblePlayWorld)
		{
			// This is a game world and matches passed in world, return it
			if (WorldContext.WorldType == EWorldType::Game)
			{
				return WorldContext.World();
			}

#if WITH_EDITOR
			if (WorldContext.WorldType == EWorldType::PIE)
			{
				// This is a PIE world, and PIE instance is either not set or matches this world
				if (UE::GetPlayInEditorID() == -1 || UE::GetPlayInEditorID() == WorldContext.PIEInstance)
				{
					return WorldContext.World();
				}
				else
				{
					// If you see this warning, game code is traversing PIE world boundaries in an unsafe way. That should be fixed or GPlayInEditorID needs to be set properly
					UE_LOGF(LogEngine, Warning, "GetCurrentPlayWorld failed with ambiguous PIE world! GPlayInEditorID %d does not match %ls", UE::GetPlayInEditorID(), *PossiblePlayWorld->GetPathName());
				}
			}
#endif

			// We found the possible play world but it is is definitely not the current world, so return null due to ambiguity
			return nullptr;
		}

		if (BestWorld)
		{
			// We want to use the first found world unless it's the specified world to check
			continue;
		}

		// If it's a game world, try and set BestWorld. If World() is null, this won't do anything
		if (WorldContext.WorldType == EWorldType::Game)
		{
			BestWorld = WorldContext.World();
		}
#if WITH_EDITOR
		// This is a PIE world, PIE instance is set, and it matches this world
		else if (WorldContext.WorldType == EWorldType::PIE && UE::GetPlayInEditorID() != -1 && UE::GetPlayInEditorID() == WorldContext.PIEInstance)
		{
			BestWorld = WorldContext.World();
		}
#endif
	}

	return BestWorld;
}

UPendingNetGame* UEngine::PendingNetGameFromWorld( UWorld* InWorld )
{
	return GetWorldContextFromWorldChecked(InWorld).PendingNetGame;
}

void UEngine::DestroyWorldContext(UWorld * InWorld)
{
	for (int32 idx=0; idx < WorldList.Num(); ++idx)
	{
		if (WorldList[idx].World() == InWorld)
		{
			WorldContextDestroyedEvent.Broadcast(WorldList[idx]);

			// Set the current world to NULL so that any external referencers are cleaned up before we remove
			WorldList[idx].SetCurrentWorld(NULL);
			WorldList.RemoveAt(idx);
			break;
		}
	}
}

bool UEngine::WorldHasValidContext(UWorld *InWorld)
{
	return (GetWorldContextFromWorld(InWorld) != NULL);
}

bool UEngine::IsWorldDuplicate(const UWorld* const InWorld)
{
	// World is considered a duplicate if it's the outer of a level in a duplicate levels collection
	for (const FWorldContext& Context : WorldList)
	{
		if (Context.World())
		{	
			const FLevelCollection* const Collection = Context.World()->FindCollectionByType(ELevelCollectionType::DynamicDuplicatedLevels);
			if (Collection)
			{
				for (const ULevel* const Level : Collection->GetLevels())
				{
					if (Level->GetOuter() == InWorld)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void UEngine::CheckAndHandleStaleWorldObjectReferences(FWorldContext* WorldContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CheckAndHandleStaleWorldObjectReferences);
	// All worlds at this point should be the CurrentWorld of some context, preview worlds, or streaming level
	// worlds that are owned by the CurrentWorld of some context.
	TArray<UObject*> LeakedObjects;
	TSet<UPackage*> LeakedPackages;
	for( TObjectIterator<UWorld> It; It; ++It )
	{
		UWorld* World = *It;
		const bool bIsPersistentWorldType = (World->WorldType == EWorldType::Inactive) || (World->WorldType == EWorldType::EditorPreview);
		if (!bIsPersistentWorldType && !WorldHasValidContext(World))
		{
			if ((World->PersistentLevel == nullptr || !WorldHasValidContext(World->PersistentLevel->OwningWorld)) && !IsWorldDuplicate(World))
			{
				LeakedObjects.Add(World);
				LeakedPackages.Add(World->GetPackage());
			}
		}
	}

	if (WorldContext)
	{
		for (FObjectKey Key : WorldContext->GarbageObjectsToVerify)
		{
			if (UObject* Obj = Key.ResolveObjectPtrEvenIfGarbage())
			{
				LeakedObjects.Add(Obj);
			}
		}
		WorldContext->GarbageObjectsToVerify.Reset();
	}

	if (LeakedObjects.Num())
	{
		UE_LOGF(LogLoad, Error, "Some previously active worlds or related objects were not cleaned up by garbage collection!");
		UE_LOGF(LogLoad, Error, "Once a world has become active, it cannot be reused and must be destroyed and reloaded.");

		if (GVerifyLoadMapWorldCleanup_Severity != 0)
		{
			EPrintStaleReferencesOptions Options = EPrintStaleReferencesOptions::None;
			switch (GVerifyLoadMapWorldCleanup_Severity)
			{
			case 3:
				Options = EPrintStaleReferencesOptions::Fatal;
				break;
			case 2: 
				Options = EPrintStaleReferencesOptions::Ensure | EPrintStaleReferencesOptions::Error;
				break;
			case 1:
			default:
				Options = EPrintStaleReferencesOptions::Error;
				break;
			}	
			// Force fatal error if reference elimination is enabled as legacy behavior
			if (UObjectBaseUtility::IsGarbageEliminationEnabled())
			{
				Options = EPrintStaleReferencesOptions::Fatal;
			}
			
			if (GVerifyLoadMapWorldCleanup_TraceMode == 0)
			{
				Options |= EPrintStaleReferencesOptions::Minimal;
			}

			// Remove sublevels from the list if their owning world also leaked
			// Unless tracing is in minimal mode; in that case we may miss useful info
			if (!(Options & EPrintStaleReferencesOptions::Minimal))
			{
				for (int32 i = LeakedObjects.Num() - 1; i >= 0; --i)
				{
					UObject* LeakedObject = LeakedObjects[i];
					if (UWorld* World = Cast<UWorld>(LeakedObject))
					{
						if (!World->PersistentLevel || World->PersistentLevel->OwningWorld == World)
						{
							continue;
						}

						if (LeakedObjects.Contains(World->PersistentLevel->OwningWorld))
						{
							LeakedObjects.RemoveAt(i, EAllowShrinking::No);
						}
					}
				}
			}

			UE_LOGF(LogLoad, Error, "Dumping reference chains:");
			FReferenceChainSearch::FindAndPrintStaleReferencesToObjects(LeakedObjects, Options);
		}

		if (!UObjectBaseUtility::IsGarbageEliminationEnabled())
		{
			UE_LOGF(LogLoad, Warning, "Renaming leaked objects in case they need to be reloaded.");
			for (UPackage* Pkg : LeakedPackages)
			{
				FCoreUObjectInternalDelegates::GetOnLeakedPackageRenameDelegate().Broadcast(Pkg);
				FName BaseName = FName(Pkg->GetFName(), NAME_NO_NUMBER_INTERNAL);
				TStringBuilder<NAME_SIZE> Builder;
				Builder << TEXT("/Leaked");
				Builder << BaseName;
				FName NewName = MakeUniqueObjectName(nullptr, Pkg->GetClass(), FName(*Builder));
				Pkg->Rename(*NewName.ToString(), nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
			}
		}
	}
}

/*-----------------------------------------------------------------------------
Async persistent level map change.
-----------------------------------------------------------------------------*/

/**
* Callback function used in UGameEngine::PrepareMapChange to pass to LoadPackageAsync.
*
* @param	LevelPackage	level package that finished async loading
* @param	InGameEngine	pointer to game engine object to associated loaded level with so it won't be GC'ed
*/
static void AsyncMapChangeLevelLoadCompletionCallback(const FName& PackageName, UPackage* LevelPackage, EAsyncLoadingResult::Type Result, FName InWorldHandle)
{
	FWorldContext &Context = GEngine->GetWorldContextFromHandleChecked( InWorldHandle );

	if( LevelPackage )
	{	
		// Try to find a UWorld object in the level package.
		UWorld* World = UWorld::FindWorldInPackage(LevelPackage);

		// If the world was not found, try to follow a redirector if it exists
		if ( !World )
		{
			World = UWorld::FollowWorldRedirectorInPackage(LevelPackage);
			if ( World )
			{
				LevelPackage = World->GetOutermost();
			}
		}

		ULevel* Level = World ? ToRawPtr(World->PersistentLevel) : nullptr;	

		// Print out a warning and set the error if we couldn't find a level in this package.
		if( !Level )
		{
			// NULL levels can happen if existing package but not level is specified as a level name.
			Context.PendingMapChangeFailureDescription = FString::Printf(TEXT("Couldn't find level in package %s"), *LevelPackage->GetName());
			UE_LOGF(LogEngine, Error, "ERROR ERROR %ls was not found in the PackageCache It must exist or the Level Loading Action will FAIL!!!! ", *LevelPackage->GetName() );
			UE_LOGF(LogEngine, Warning, "%ls", *Context.PendingMapChangeFailureDescription );
			UE_LOGF(LogEngine, Error, "ERROR ERROR %ls was not found in the PackageCache It must exist or the Level Loading Action will FAIL!!!! ", *LevelPackage->GetName() );
		}

		// Add loaded level to array to prevent it from being garbage collected.
		Context.LoadedLevelsForPendingMapChange.Add( Level );
	}
	else
	{
		// Add NULL entry so we don't end up waiting forever on a level that is never going to be loaded.
		Context.LoadedLevelsForPendingMapChange.Add( nullptr );
		UE_LOGF(LogEngine, Warning, "NULL LevelPackage as argument to AsyncMapChangeLevelCompletionCallback" );
	}

	STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "PrepareMapChangeComplete - " ) + PackageName.ToString() )) );
	TRACE_BOOKMARK(TEXT("PrepareMapChangeComplete - %s"), *PackageName.ToString());
}


bool UEngine::PrepareMapChange(FWorldContext &Context, const TArray<FName>& LevelNames)
{
	// make sure level streaming isn't frozen
	Context.World()->bIsLevelStreamingFrozen = false;

	// Make sure we don't interrupt a pending map change in progress.
	if( !IsPreparingMapChange(Context) )
	{
		Context.LevelsToLoadForPendingMapChange.Reset();
		Context.LevelsToLoadForPendingMapChange.Append(LevelNames);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Verify that all levels specified are in the package file cache.
		for (const FName& LevelName : Context.LevelsToLoadForPendingMapChange)
		{
			if( !FPackageName::DoesPackageExist( LevelName.ToString() ) )
			{
				Context.LevelsToLoadForPendingMapChange.Empty();
				Context.PendingMapChangeFailureDescription = FString::Printf(TEXT("Couldn't find package for level '%s'"), *LevelName.ToString());
				// write it out immediately so make sure it's in the log even without a CommitMapChange happening
				UE_LOGF(LogEngine, Warning, "PREPAREMAPCHANGE: %ls", *Context.PendingMapChangeFailureDescription);

				// tell user on screen!
				extern bool GIsPrepareMapChangeBroken;
				GIsPrepareMapChangeBroken = true;

				return false;
			}
			//@todo streaming: make sure none of the maps are already loaded/ being loaded?
		}
#endif

		// copy LevelNames into the WorldInfo's array to keep track of the map change that we're preparing (primarily for servers so clients that join in progress can be notified)
		if (Context.World() != nullptr)
		{
			Context.World()->PreparingLevelNames = LevelNames;
		}

		// Kick off async loading of packages.
		for (const FName& LevelName : Context.LevelsToLoadForPendingMapChange)
		{
			STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, *(FString( TEXT( "PrepareMapChange - " ) + LevelName.ToString() )) );
			TRACE_BOOKMARK(TEXT("PrepareMapChange - %s"), *LevelName.ToString());
			LoadPackageAsync(LevelName.ToString(),
				FLoadPackageAsyncDelegate::CreateStatic(&AsyncMapChangeLevelLoadCompletionCallback, Context.ContextHandle)
			);
		}

		return true;
	}
	else
	{
		Context.PendingMapChangeFailureDescription = TEXT("Current map change still in progress");
		return false;
	}
}


FString UEngine::GetMapChangeFailureDescription(FWorldContext &Context)
{
	return Context.PendingMapChangeFailureDescription;
}


bool UEngine::IsPreparingMapChange(FWorldContext &Context)
{
	return Context.LevelsToLoadForPendingMapChange.Num() > 0;
}


bool UEngine::IsReadyForMapChange(FWorldContext &Context)
{
	return IsPreparingMapChange(Context) && (Context.LevelsToLoadForPendingMapChange.Num() == Context.LoadedLevelsForPendingMapChange.Num());
}


void UEngine::ConditionalCommitMapChange(FWorldContext &Context)
{
	// Check whether there actually is a pending map change and whether we want it to be committed yet.
	if( Context.bShouldCommitPendingMapChange && IsPreparingMapChange(Context) )
	{
		// Block on remaining async data.
		if( !IsReadyForMapChange(Context) )
		{
			FlushAsyncLoading();
			check( IsReadyForMapChange(Context) );
		}

		// Perform map change.
		if (!CommitMapChange(Context.World()))
		{
			UE_LOGF(LogEngine, Warning, "Committing map change via %ls was not successful: %ls", *GetFullName(), *GetMapChangeFailureDescription(Context));
		}
		// No pending map change - called commit without prepare.
		else
		{
			UE_LOGF(LogEngine, Log, "Committed map change via %ls", *GetFullName());
		}

		// We just commited, so reset the flag.
		Context.bShouldCommitPendingMapChange = false;
	}
}

/** struct to temporarily hold on to already loaded but unbound levels we're going to make visible at the end of CommitMapChange() while we first trigger GC */
struct FPendingStreamingLevelHolder : public FGCObject
{
public:
	TArray<TObjectPtr<ULevel>> Levels;

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObjects( Levels ); 
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FPendingStreamingLevelHolder");
	}
};


bool UEngine::CommitMapChange( FWorldContext &Context )
{
	if (!IsPreparingMapChange(Context))
	{
		Context.PendingMapChangeFailureDescription = TEXT("No map change is being prepared");
		return false;
	}
	else if (!IsReadyForMapChange(Context))
	{
		Context.PendingMapChangeFailureDescription = TEXT("Map change is not ready yet");
		return false;
	}
	else
	{
		check(Context.World());

		AGameModeBase* GameMode = Context.World()->GetAuthGameMode();

		// tell the game we are about to switch levels
		if (GameMode)
		{
			// get the actual persistent level's name
			FString PreviousMapName = Context.World()->PersistentLevel->GetOutermost()->GetName();
			FString NextMapName = Context.LevelsToLoadForPendingMapChange[0].ToString();

			// look for a persistent streamed in sublevel
			for (ULevelStreaming* LevelStreaming : Context.World()->GetStreamingLevels())
			{
				ULevelStreamingPersistent* PersistentLevel = Cast<ULevelStreamingPersistent>(LevelStreaming);
				if (PersistentLevel)
				{
					PreviousMapName = PersistentLevel->GetWorldAssetPackageName();
					// only one persistent level
					break;
				}
			}
			FGameDelegates::Get().GetPreCommitMapChangeDelegate().Broadcast(PreviousMapName, NextMapName);
		}

		// on the client, check if we already loaded pending levels to be made visible due to e.g. the PackageMap
		FPendingStreamingLevelHolder LevelHolder;
		if (Context.PendingLevelStreamingStatusUpdates.Num() > 0)
		{
			// Iterating over GCed ULevels. A TObjectIterator<ULevel> can not do this.
			for (TObjectIterator<UObject> It(true); It; ++It)
			{
				ULevel* Level = Cast<ULevel>(*It);
				if ( Level )
				{
					for (int32 i = 0; i < Context.PendingLevelStreamingStatusUpdates.Num(); i++)
					{
						if ( Level->GetOutermost()->GetFName() == Context.PendingLevelStreamingStatusUpdates[i].PackageName && 
							(Context.PendingLevelStreamingStatusUpdates[i].bShouldBeLoaded || Context.PendingLevelStreamingStatusUpdates[i].bShouldBeVisible) )
						{
							LevelHolder.Levels.Add(Level);
							break;
						}
					}
				}
			}
		}

		// we are no longer preparing this change
		Context.World()->PreparingLevelNames.Empty();

		// Iterate over level collection, marking them to be forcefully unloaded.
		for (ULevelStreaming* StreamingLevel : Context.World()->GetStreamingLevels())
		{
			if( StreamingLevel )
			{
				StreamingLevel->SetIsRequestingUnloadAndRemoval(true);
			}
		}

		// Collect garbage. @todo streaming: make sure that this doesn't stall due to async loading in the background
		{
			TGuardValue<bool> _FullGC(bGCPerformingFullPurge, true);
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
		}

		// The new fake persistent level is first in the LevelsToLoadForPendingMapChange array.
		FName	FakePersistentLevelName = Context.LevelsToLoadForPendingMapChange[0];
		ULevel*	FakePersistentLevel		= nullptr;
		// copy to WorldInfo to keep track of the last map change we performed (primarily for servers so clients that join in progress can be notified)
		// we don't need to remember secondary levels as the join code iterates over all streaming levels and updates them
		Context.World()->CommittedPersistentLevelName = FakePersistentLevelName;

		// Find level package in loaded levels array.
		for( int32 LevelIndex=0; LevelIndex < Context.LoadedLevelsForPendingMapChange.Num(); LevelIndex++ )
		{
			ULevel* Level = Context.LoadedLevelsForPendingMapChange[LevelIndex];

			// NULL levels can happen if existing package but not level is specified as a level name.
			if( Level && (FakePersistentLevelName == Level->GetOutermost()->GetFName()) )
			{
				FakePersistentLevel = Level;
				break;
			}
		}
		check( FakePersistentLevel );

		// Construct a new ULevelStreamingPersistent for the new persistent level.
		ULevelStreamingPersistent* LevelStreamingPersistent = NewObject<ULevelStreamingPersistent>(
			Context.World(),
			*FString::Printf(TEXT("LevelStreamingPersistent_%s"), *FakePersistentLevel->GetOutermost()->GetName()) );

		// Add it to the world info's list of levels.
		Context.World()->AddStreamingLevel(LevelStreamingPersistent);

		// Propagate level and name to streaming object.
		FStreamingLevelPrivateAccessor::SetLoadedLevel(LevelStreamingPersistent, FakePersistentLevel);
		LevelStreamingPersistent->SetWorldAssetByPackageName(FakePersistentLevelName);

		UWorld* FakeWorld = CastChecked<UWorld>(FakePersistentLevel->GetOuter());

		// Rename the newly loaded streaming levels so that their outer is correctly set to the main context's world,
		// rather than the fake world.
		TArray<ULevelStreaming*> StreamingLevelsToMove;
		StreamingLevelsToMove.SetNumUninitialized(FakeWorld->GetStreamingLevels().Num());
		for (int32 Index = FakeWorld->GetStreamingLevels().Num() - 1; Index >= 0; --Index)
		{
			ULevelStreaming* const FakeWorldStreamingLevel = FakeWorld->GetStreamingLevels()[Index];
			FakeWorldStreamingLevel->Rename(nullptr, Context.World(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			FakeWorld->RemoveStreamingLevelAt(Index);
			StreamingLevelsToMove[Index] = FakeWorldStreamingLevel;
		}

		// Move the secondary levels to the world info levels array.
		Context.World()->AddStreamingLevels(StreamingLevelsToMove);

		// fixup up any kismet streaming objects to force them to be loaded if they were preloaded, this
		// will keep streaming volumes from immediately unloading the levels that were just loaded
		for (ULevelStreaming* StreamingLevel : Context.World()->GetStreamingLevels())
		{
			// mark any kismet streamers to force be loaded
			if (StreamingLevel)
			{
				bool bWasFound = false;
				// was this one of the packages we wanted to load?
				for (int32 LoadLevelIndex = 0; LoadLevelIndex < Context.LevelsToLoadForPendingMapChange.Num(); LoadLevelIndex++)
				{
					if (Context.LevelsToLoadForPendingMapChange[LoadLevelIndex] == StreamingLevel->GetWorldAssetPackageFName())
					{
						bWasFound = true;
						break;
					}
				}

				// if this level was preloaded, mark it as to be loaded and visible
				if (bWasFound)
				{
					StreamingLevel->SetShouldBeLoaded(true);
					StreamingLevel->SetShouldBeVisible(true);

#if WITH_SERVER_CODE
					if (!Context.World()->IsNetMode(NM_Client))
					{
						const bool bShouldBeLoaded = StreamingLevel->ShouldBeLoaded();
						const bool bShouldBeVisible = StreamingLevel->ShouldBeVisible();

						// notify players of the change
						for( FConstPlayerControllerIterator Iterator = Context.World()->GetPlayerControllerIterator(); Iterator; ++Iterator )
						{
							if (APlayerController* PlayerController = Iterator->Get())
							{
								PlayerController->LevelStreamingStatusChanged(
									StreamingLevel,
									bShouldBeLoaded,
									bShouldBeVisible,
									StreamingLevel->bShouldBlockOnLoad,
									StreamingLevel->bShouldBlockOnUnload,
									StreamingLevel->GetLevelLODIndex());
							}
						}
					}
#endif // WITH_SERVER_CODE
				}
			}
		}

		// Update level streaming, forcing existing levels to be unloaded and their streaming objects 
		// removed from the world info.	We can't kick off async loading in this update as we want to 
		// collect garbage right below.
		Context.World()->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

		// make sure any looping sounds, etc are stopped

		if (FAudioDevice* AudioDevice = Context.World()->GetAudioDeviceRaw())
		{
			AudioDevice->StopAllSounds();
		}

		// Remove all unloaded levels from memory and perform full purge.
		{
			TGuardValue<bool> _FullGC(bGCPerformingFullPurge, true);
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
		}

		// if there are pending streaming changes replicated from the server, apply them immediately
		if (Context.PendingLevelStreamingStatusUpdates.Num() > 0)
		{
			for (const FLevelStreamingStatus& PendingUpdate : Context.PendingLevelStreamingStatusUpdates)
			{
				ULevelStreaming*const* Found = Context.World()->GetStreamingLevels().FindByPredicate([&](ULevelStreaming* Level){
					return Level && Level->GetWorldAssetPackageFName() == PendingUpdate.PackageName;
				});

				if (Found)
				{
					(*Found)->SetShouldBeLoaded(PendingUpdate.bShouldBeLoaded);
					(*Found)->SetShouldBeVisible(PendingUpdate.bShouldBeVisible);
					(*Found)->SetLevelLODIndex(PendingUpdate.LODIndex);
				}
				else
				{
					UE_LOGF(LogStreaming, Log, "Unable to find streaming object %ls", *PendingUpdate.PackageName.ToString());
				}
			}

			Context.PendingLevelStreamingStatusUpdates.Empty();

			Context.World()->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		}
		else
		{
			// Make sure there are no pending visibility requests.
			Context.World()->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);
		}

		// delay the use of streaming volumes for a few frames
		Context.World()->DelayStreamingVolumeUpdates(3);

		// Empty intermediate arrays.
		Context.LevelsToLoadForPendingMapChange.Empty();
		Context.LoadedLevelsForPendingMapChange.Empty();
		Context.PendingMapChangeFailureDescription = TEXT("");

		// Prime texture streaming.
		IStreamingManager::Get().NotifyLevelChange();

		// tell the game we are done switching levels
		if (GameMode)
		{
			FGameDelegates::Get().GetPostCommitMapChangeDelegate().Broadcast();
		}

		return true;
	}
}
void UEngine::AddNewPendingStreamingLevel(UWorld *InWorld, FName PackageName, bool bNewShouldBeLoaded, bool bNewShouldBeVisible, int32 LODIndex)
{
	FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);
	new(Context.PendingLevelStreamingStatusUpdates) FLevelStreamingStatus(PackageName, bNewShouldBeLoaded, bNewShouldBeVisible, LODIndex);
}

bool UEngine::ShouldCommitPendingMapChange(const UWorld *InWorld) const
{
	const FWorldContext* WorldContext = GetWorldContextFromWorld(InWorld);
	return (WorldContext ? WorldContext->bShouldCommitPendingMapChange : false);
}

void UEngine::SetShouldCommitPendingMapChange(UWorld *InWorld, bool NewShouldCommitPendingMapChange)
{
	FWorldContext &Context = GetWorldContextFromWorldChecked(InWorld);
	Context.bShouldCommitPendingMapChange = NewShouldCommitPendingMapChange;
}

FSeamlessTravelHandler&	UEngine::SeamlessTravelHandlerForWorld(UWorld *World)
{
	return GetWorldContextFromWorldChecked(World).SeamlessTravelHandler;
}

FURL& UEngine::LastURLFromWorld(UWorld *World)
{
	return GetWorldContextFromWorldChecked(World).LastURL;
}

void UEngine::CreateGameUserSettings()
{
	UGameUserSettings::LoadConfigIni();
	GameUserSettings = NewObject<UGameUserSettings>(GetTransientPackage(), GEngine->GameUserSettingsClass);
	GameUserSettings->SetToDefaults();
	GameUserSettings->LoadSettings();
}

const UGameUserSettings* UEngine::GetGameUserSettings() const
{
	if (GameUserSettings == NULL)
	{
		UEngine* ConstThis = const_cast< UEngine* >( this );	// Hack because Header Generator doesn't yet support mutable keyword
		ConstThis->CreateGameUserSettings();
	}
	return GameUserSettings;
}

UGameUserSettings* UEngine::GetGameUserSettings()
{
	if (GameUserSettings == NULL)
	{
		CreateGameUserSettings();
	}
	return GameUserSettings;
}

// Stores information (such as modified properties) for an instanced object (component or subobject)
// in the old CDO, to allow them to be reapplied to the new instance under the new CDO
struct FInstancedObjectRecord
{
	TArray<uint8> SavedProperties;
	UObject* OldInstance;
};

static TAutoConsoleVariable<int32> CVarDumpCopyPropertiesForUnrelatedObjects(
	TEXT("DumpCopyPropertiesForUnrelatedObjects"),
	0,
	TEXT("Dump the objects that are cross class copied")
);

/* 
* Houses base functionality shared between CPFUO archivers (FCPFUOWriter/FCPFUOReader) 
* Used to track whether tagged data is being processed (and whether we should be serializing it).
*/
struct FCPFUOArchive
{
public:
	FCPFUOArchive(bool bIncludeUntaggedDataIn)
		: bIncludeUntaggedData(bIncludeUntaggedDataIn)
		, TaggedDataScope(0)
	{}

	FCPFUOArchive(const FCPFUOArchive& DataSrc)
		: bIncludeUntaggedData(DataSrc.bIncludeUntaggedData)
		, TaggedDataScope(0)
	{}

protected:
	FORCEINLINE void OpenTaggedDataScope()  { ++TaggedDataScope; }
	FORCEINLINE void CloseTaggedDataScope() { --TaggedDataScope; }

	FORCEINLINE bool IsSerializationEnabled()
	{
		return bIncludeUntaggedData || (TaggedDataScope > 0);
	}

	bool bIncludeUntaggedData;
private:
	int32 TaggedDataScope;
};

struct FObjectCellWriter : public FObjectWriter
{
	Verse::FDuplicationContext& DuplicatedCells;

	FObjectCellWriter(TArray<uint8>& InBytes, Verse::FDuplicationContext& InDuplicatedCells)
		: FObjectWriter(InBytes)
		, DuplicatedCells(InDuplicatedCells)
	{
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override
	{
		DuplicatedCells.WriteCellReference(Cell);
		FObjectWriter::operator<<(Cell);
		return *this;
	}
#endif
};

#if WITH_EDITOR
static FName BlueprintCompilerGeneratedDefaultsName("BlueprintCompilerGeneratedDefaults");
#endif

/* Serializes and stores property data from a specified 'source' object. Only stores data compatible with a target destination object. */
struct FCPFUOWriter : public FObjectCellWriter, public FCPFUOArchive
{
public:
	/* Contains the source object's serialized data */
	TArray<uint8> SavedPropertyData;

public:
	FCPFUOWriter(UObject* SrcObject, UObject* DstObject, const UEngine::FCopyPropertiesForUnrelatedObjectsParams& Params, Verse::FDuplicationContext& InDuplicatedCells)
		: FObjectCellWriter(SavedPropertyData, InDuplicatedCells) // -V1050
		// if the two objects don't share a common native base class, then they may have different
		// serialization methods, which is dangerous (the data is not guaranteed to be homogeneous)
		// in that case, we have to stick with tagged properties only
		, FCPFUOArchive(FindNativeSuperClass(SrcObject) == FindNativeSuperClass(DstObject))
		, SourceObject(SrcObject)
		, SourceArchetype(Params.SourceObjectArchetype)
		, bSkipCompilerGeneratedDefaults(Params.bSkipCompilerGeneratedDefaults)
	{
		ArIgnoreArchetypeRef = true;
		ArNoDelta = !Params.bDoDelta;
		ArIgnoreClassRef = true;
		ArPortFlags = PPF_None;
		if (Params.bCopyDeprecatedProperties)
		{
			ArPortFlags |=  PPF_UseDeprecatedProperties;
		}
		if (Params.bPerformDuplication)
		{
			ArPortFlags |= PPF_Duplicate;
		}

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(DstObject));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		UE::FScopedObjectSerializeContext ObjectSerializeContext(SrcObject, *this);
		SrcObject->Serialize(*this);
	}

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Num) override
	{
		if (IsSerializationEnabled())
		{
			FObjectWriter::Serialize(Data, Num);
		}
	}

	virtual UObject* GetArchetypeFromLoader(const UObject* Obj) override
	{
		return (Obj == SourceObject && SourceArchetype ? SourceArchetype : nullptr);
	}

	virtual void MarkScriptSerializationStart(const UObject* Object) override { OpenTaggedDataScope(); }
	virtual void MarkScriptSerializationEnd(const UObject* Object) override   { CloseTaggedDataScope(); }

#if WITH_EDITOR
	virtual bool ShouldSkipProperty(const class FProperty* InProperty) const override
	{
		return (bSkipCompilerGeneratedDefaults && InProperty->HasMetaData(BlueprintCompilerGeneratedDefaultsName));
	}
#endif 
	//~ End FArchive Interface

private:
	static UClass* FindNativeSuperClass(UObject* Object)
	{
		UClass* Class = Object->GetClass();
		for (; Class; Class = Class->GetSuperClass())
		{
			if ((Class->ClassFlags & CLASS_Native) != 0)
			{
				break;
			}
		}
		return Class;
	}

	UObject* SourceObject;
	UObject* SourceArchetype;
	bool bSkipCompilerGeneratedDefaults;
};

/* Responsible for applying the saved property data from a FCPFUOWriter to a specified object */
struct FObjectReaderWithReplacement : public FObjectReader
{
public:
	FObjectReaderWithReplacement(UObject* Obj, const TArray<uint8>& InBytes, Verse::FDuplicationContext& InDuplicatedCells, bool bIgnoreClassRef = false, bool bIgnoreArchetypeRef = false, const TMap<UObject*, UObject*>* InReferenceReplacementMap = nullptr, const TMap<UClass*, UClass*>* InClassReferenceReplacementMap = nullptr)
		: FObjectReader(InBytes)
		, ReferenceReplacementMap(InReferenceReplacementMap)
		, ClassReferenceReplacementMap(InClassReferenceReplacementMap)
		, DuplicatedCells(InDuplicatedCells)
	{
		ArIgnoreClassRef = bIgnoreClassRef;
		ArIgnoreArchetypeRef = bIgnoreArchetypeRef;

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Obj));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		UE::FScopedObjectSerializeContext ObjectSerializeContext(Obj, *this);
		Obj->Serialize(*this);
	}

	FObjectReaderWithReplacement(const TArray<uint8>& InBytes, Verse::FDuplicationContext& InDuplicatedCells, const TMap<UObject*, UObject*>* InReferenceReplacementMap = nullptr, const TMap<UClass*, UClass*>* InClassReferenceReplacementMap = nullptr)
		: FObjectReader(InBytes)
		, ReferenceReplacementMap(InReferenceReplacementMap)
		, ClassReferenceReplacementMap(InClassReferenceReplacementMap)
		, DuplicatedCells(InDuplicatedCells)
	{
	}

	const TMap<UObject*, UObject*>* ReferenceReplacementMap = nullptr;
	const TMap<UClass*, UClass*>* ClassReferenceReplacementMap = nullptr;
	Verse::FDuplicationContext& DuplicatedCells;

	template <typename VALUE_TYPE>
	FORCEINLINE void ReplaceObjectReference(UObject* Object, VALUE_TYPE& Value) const
	{
		if(!Object)
		{
			return;
		}
		if (UObject*const* Reference = ReferenceReplacementMap ? ReferenceReplacementMap->Find(Object) : nullptr)
		{
			Value = const_cast<UObject*>(*Reference);
		}
		else if(ClassReferenceReplacementMap)
		{
			if (UClass* Class = Cast<UClass>(Object))
			{
				if (UClass*const* ClassReference = ClassReferenceReplacementMap->Find(Class))
				{
					Value = const_cast<UClass*>(*ClassReference);
				}
			}
		}
	}

	FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value.Get(), Value);
		return *this;
	}

	FArchive& operator<<(FSoftObjectPtr& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value.Get(), Value);
		return *this;
	}

	FArchive& operator<<(FSoftObjectPath& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value.ResolveObject(), Value);
		return *this;
	}

	FArchive& operator<<(FLazyObjectPtr& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value.Get(), Value);
		return *this;
	}

	FArchive& operator<<(FObjectPtr& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value.Get(), Value);
		return *this;
	}

	FArchive& operator<<(UObject*& Value) override
	{
		FObjectReader::operator<<(Value);
		ReplaceObjectReference(Value, Value);
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FArchive& operator<<(Verse::VCell*& Value) override
	{
		FObjectReader::operator<<(Value);
		Value = DuplicatedCells.ReadCellReference(Value);
		return *this;
	}
#endif
};

/* Responsible for applying the saved property data from a FCPFUOWriter to a specified object */
struct FCPFUOReader : public FObjectReaderWithReplacement, public FCPFUOArchive
{
public:
	FCPFUOReader(
		FCPFUOWriter& DataSrc, UObject* DstObject, const UEngine::FCopyPropertiesForUnrelatedObjectsParams& Params,
		Verse::FDuplicationContext& InDuplicatedCells, const TMap<UObject*, UObject*>* InReferenceReplacementMap = nullptr, const TMap<UClass*, UClass*>* InClassReferenceReplacementMap = nullptr
		)
		: FObjectReaderWithReplacement(DataSrc.SavedPropertyData, InDuplicatedCells, InReferenceReplacementMap, InClassReferenceReplacementMap)
		, FCPFUOArchive(DataSrc)
	{
		ArIgnoreArchetypeRef = true;
		ArIgnoreClassRef = true;

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(DstObject));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
		
		if (Params.bPerformDuplication)
		{
			ArPortFlags |= PPF_Duplicate;
		}

		UE::FScopedObjectSerializeContext ObjectSerializeContext(DstObject, *this);
		DstObject->Serialize(*this);
	}

	//~ Begin FArchive Interface
	virtual void Serialize(void* Data, int64 Num) override
	{
		if (IsSerializationEnabled())
		{
			FObjectReaderWithReplacement::Serialize(Data, Num);
		}
	}

	virtual void MarkScriptSerializationStart(const UObject* Object) override { OpenTaggedDataScope(); }
	virtual void MarkScriptSerializationEnd(const UObject* Object) override   { CloseTaggedDataScope(); }
	// ~End FArchive Interface
};

namespace UE::Engine::Private
{
	static bool bUseOldDuplicateEditInlineSubObjectsPath = true;
	static FAutoConsoleVariableRef CVarUseOldDuplicateEditInlineSubObjectsPath(
		TEXT("CPFUO.UseOldDuplicateEditInlineSubObjectsPath"),
		bUseOldDuplicateEditInlineSubObjectsPath,
		TEXT("If true, CPFUO will revert back to relying on property flags to duplicate editinline subobjects."));

	// Internal helper to duplicate or fix up any references to subobjects still owned by the old object that have not
	// yet been duplicated under the new object. These are typically editinline types that get instanced at edit time.
	static void DuplicateEditInlineSubObjects(
		UObject* OldObject,
		UObject* NewObject,
		TMap<UObject*, UObject*>& ReferenceReplacementMap,
		TArray<UObject*>& DuplicatedObjects,
		TMap<UObject*, UObject*>* OptionalMappings)
	{
		if (bUseOldDuplicateEditInlineSubObjectsPath)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FFindInstancedReferenceSubobjectHelper::Duplicate(OldObject, NewObject, ReferenceReplacementMap, DuplicatedObjects, OptionalMappings);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			return;
		}

		if (!OldObject->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference)
			|| !NewObject->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference))
		{
			return;
		}

		// Gather all subobjects referenced by the old object
		TArray<UObject*> OldObjectReferences;
		FReferenceFinder(OldObjectReferences, OldObject).FindReferences(OldObject);
		if (OldObjectReferences.IsEmpty())
		{
			return;
		}

		// Copy old subobjects referenced by non-transient object properties to the new object.
		// The iterator is inclusive of any container or struct property inner references as well.
		for (TPropertyValueIterator<FObjectProperty> It(NewObject->GetClass(), NewObject); It; ++It)
		{
			const FObjectProperty* ObjectProperty = It.Key();

			check(ObjectProperty);
			UObject* Obj = ObjectProperty->GetObjectPropertyValue(It.Value());
			if (!Obj)
			{
				continue;
			}

			// We only care about new references to subobjects still outered to the old object.
			const bool bNewObjectHasOldOuter = (Obj->GetOuter() == OldObject);
			if (!bNewObjectHasOldOuter)
			{
				continue;
			}

			// If the old object doesn't reference it anymore then we don't want to keep it either.
			// In this case the reference on the new object will be cleared during ref replacement.
			const bool bKeptByOld = OldObjectReferences.Contains(Obj);
			if (!bKeptByOld)
			{
				continue;
			}

			// We only care about non-transient references. Transient references still outered to
			// the old object will either be cleared during ref replacement or instanced elsewhere.
			const bool bIsTransient = ObjectProperty->HasAnyPropertyFlags(CPF_Transient);
			if (bIsTransient)
			{
				continue;
			}

			// Duplicate the instance if it hasn't already been handled. Otherwise, ensure that
			// we fix up the outer reference on the copy to correctly point back to the new object.
			const bool bNotHandledYet = !ReferenceReplacementMap.Contains(Obj) &&
				(!OptionalMappings || !OptionalMappings->Contains(Obj));
			if (bNotHandledYet)
			{
				// This name may have been taken by an instanced subobject, since we're copying from
				// an object we want to recreate it here with a duplicate of the old object. I'm 
				// using a rename call here to reserve the name for ourself. These objects come from
				// FObjectInitializer::InstanceSubobjects()
				UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), NewObject, Obj->GetFName());
				if (ExistingObject)
				{
					ExistingObject->Rename(
						nullptr,
						GetTransientPackage(),
						REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
				}

				UObject* NewEditInlineSubobject = StaticDuplicateObject(Obj, NewObject, Obj->GetFName());
				ReferenceReplacementMap.Add(Obj, NewEditInlineSubobject);
				if (OptionalMappings)
				{
					OptionalMappings->Add(Obj, NewEditInlineSubobject);
				}

				// NOTE: we cannot patch OldObject's linker table here, since we don't 
				//       know the relation between the two objects (one could be of a 
				//       super class, and the other a child)

				// We also need to make sure to fixup any properties here
				DuplicatedObjects.Add(NewEditInlineSubobject);
			}
			else
			{
				// make sure the object is outered correctly:
				check(bNewObjectHasOldOuter);
				UObject* RealNewSubobject = nullptr;
				if (OptionalMappings)
				{
					UObject* const* Entry = OptionalMappings->Find(Obj);
					if (Entry)
					{
						RealNewSubobject = *Entry;
					}
				}

				if (!RealNewSubobject)
				{
					RealNewSubobject = ReferenceReplacementMap.FindChecked(Obj);
				}
				if (RealNewSubobject->GetOuter() != NewObject)
				{
					UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), NewObject, RealNewSubobject->GetFName());
					if (ExistingObject)
					{
						ExistingObject->Rename(
							nullptr,
							GetTransientPackage(),
							REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
						ReferenceReplacementMap.Add(ExistingObject, RealNewSubobject);
					}
					RealNewSubobject->Rename(
						*(RealNewSubobject->GetName()),
						NewObject,
						REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
				}
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UEngine::FCopyPropertiesForUnrelatedObjectsParams::FCopyPropertiesForUnrelatedObjectsParams()
	: bAggressiveDefaultSubobjectReplacement(false)
	, bDoDelta(true)
	, bReplaceObjectClassReferences(true)
	, bCopyDeprecatedProperties(false)
	, bPreserveRootComponent(true)
	, bPerformDuplication(false)
	, bOnlyHandleDirectSubObjects(false)
	, bSkipCompilerGeneratedDefaults(false)
	, bNotifyObjectReplacement(false)
	, bClearReferences(true)
	, bReplaceInternalReferenceUponRead(false)
	, SourceObjectArchetype(nullptr)
	, OptionalReplacementMappings(nullptr)
	, OptionalOldToNewClassMappings(nullptr)
{}
UEngine::FCopyPropertiesForUnrelatedObjectsParams::FCopyPropertiesForUnrelatedObjectsParams(const FCopyPropertiesForUnrelatedObjectsParams&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS


TAutoConsoleVariable<bool> CVarAuditAggressiveReferenceReplacement(
	TEXT("cpfuo.AuditAggressiveReferenceReplacment"),
	1,
	TEXT("Whether to audit and report on reference replacements that come from the aggressive replacement path.\n"),
	ECVF_Cheat);

TAutoConsoleVariable<bool> CVarUseAggressiveReferenceReplacement(
	TEXT("cpfuo.UseAggressiveReferenceReplacment"),
	0,
	TEXT("Whether to aggressively replace references. This behavior is being deprecated but being left with the ability to toggle back on in case issues arise.\n"),
	ECVF_Default);

class FAggressiveReplacementAuditArchive : public FArchiveReplaceOrClearExternalReferences<UObject>
{
public:

	typedef FArchiveReplaceOrClearExternalReferences<UObject> TSuper;

	FAggressiveReplacementAuditArchive(UObject* InSearchObject, const TMap<UObject*, UObject*>& InReplacementMap, UPackage* InDestPackage, const TMap<UObject*, UObject*>& InAggressiveReplacements)
		: TSuper(InSearchObject, InReplacementMap, InDestPackage, EArchiveReplaceObjectFlags::DelayStart)
		, AggressiveReplacements(InAggressiveReplacements)
	{
		this->SerializeSearchObject();
	}

	FArchive& operator<<(UObject*& Obj)
	{
		UObject* PreResolveObj = Obj;
		TSuper::operator<<(Obj);
		if (UObject* const * AggressiveReplacementObj = AggressiveReplacements.Find(PreResolveObj))
		{
			ensureAlwaysMsgf(false, TEXT("UE-91735: Attempted aggressive reference replacement. '%s' would have been replaced by '%s' in '%s' for property '%s'"), *GetNameSafe(PreResolveObj), *GetNameSafe(*AggressiveReplacementObj), *GetNameSafe(GetSearchObject()), *GetNameSafe(GetSerializedProperty()));
		}

		return *this;
	}

private:

	const TMap<UObject*, UObject*>& AggressiveReplacements;
};

#if WITH_EDITOR
// When IDOs are enabled, if the old class has subobjects that the new class doesn't, we need new versions of those subobjects to exist so that we can round trip that data.
// this helper function is responsible for constructing those objects so the loose ido object references can point to them and they maintain their data
static void CreateLooseSubobjectsForCPFUO(const UObject* OldObject, UObject* NewObject, TArray<UObject*>& InOutSubobjectArray,
	const TMap<FString, int32>& OldInstanceMap, const TIndirectArray<FInstancedObjectRecord>& SavedInstances, const UEngine::FCopyPropertiesForUnrelatedObjectsParams& Params)
{
	TArray<UObject*> UnsortedNewInstances = MoveTemp(InOutSubobjectArray);

	TArray<int32> OldToNewIndex;
	OldToNewIndex.Init(INDEX_NONE, SavedInstances.Num());
	for (int32 NewInstanceIndex = 0; NewInstanceIndex < UnsortedNewInstances.Num(); ++NewInstanceIndex)
	{
		UObject* NewInstance = UnsortedNewInstances[NewInstanceIndex];
		if (const int32* pOldInstanceIndex = OldInstanceMap.Find(NewInstance->GetPathName(NewObject)))
		{
			OldToNewIndex[*pOldInstanceIndex] = NewInstanceIndex;
		}
		else
		{
			// this instance is in the new instance but wasn't in the old instance map. This isn't terribly unexpected so just keep it in the array
			InOutSubobjectArray.Add(NewInstance);
		}
	}

	for (int32 OldInstanceIndex = 0; OldInstanceIndex < SavedInstances.Num(); ++OldInstanceIndex)
	{
		if (OldToNewIndex[OldInstanceIndex] != INDEX_NONE)
		{
			InOutSubobjectArray.Add(UnsortedNewInstances[OldToNewIndex[OldInstanceIndex]]);
			continue;
		}
		// found a subobject that existed in the old graph and isn't in the new one. We'll create it so we don't loose that data
		const FInstancedObjectRecord& Record = SavedInstances[OldInstanceIndex];

		// determine the outer of this new object
		UObject* DestinationOuter = nullptr;
		// attempt to find outer via local data
		if (const int32* pOldOuterIndex = OldInstanceMap.Find(Record.OldInstance->GetOuter()->GetPathName(OldObject)))
		{
			const int32 NewOuterIndex = OldToNewIndex[*pOldOuterIndex];
			if (UnsortedNewInstances.IsValidIndex(NewOuterIndex))
			{
				DestinationOuter = UnsortedNewInstances[NewOuterIndex];
			}
		}
		// fallback to provided CPFUO replacement map
		if (Params.OptionalReplacementMappings && DestinationOuter == nullptr)
		{
			if (UObject* const* FoundOuter = Params.OptionalReplacementMappings->Find(Record.OldInstance->GetOuter()))
			{
				DestinationOuter = *FoundOuter;
			}
		}
		// fallback to topmost new object
		if (DestinationOuter == nullptr)
		{
			DestinationOuter = NewObject;
		}

		// determine the class of this new object
		UClass* OldClass = Record.OldInstance->GetClass();
		UClass* NewClass = OldClass;
		if (Params.OptionalOldToNewClassMappings)
		{
			if (UClass* const* FoundClass = Params.OptionalOldToNewClassMappings->Find(NewClass))
			{
				NewClass = *FoundClass;
			}
		}
		if (NewClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			// The old subobject class is a compiler artifact, with no reinstanced counterpart. Its source may have been renamed (or deleted) and thus it could not be recompiled.
			// In this case, if property bag features are enabled, we attempt to redirect to a placeholder type so we can serialize to a property bag in order to preserve its data.
			if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::ReplaceMissingReinstancedTypes))
			{
				FTopLevelAssetPath OldSubObjectClassPath = NewClass->GetReinstancedClassPathName();
				if (OldSubObjectClassPath.IsValid())
				{
					if (UPackage* OldSubObjectClassPackage = FindPackage(nullptr, *OldSubObjectClassPath.GetPackageName().ToString()))
					{
						NewClass = UE::FPropertyBagRepository::CreatePropertyBagPlaceholderClass(OldSubObjectClassPackage, NewClass->GetClass(), OldSubObjectClassPath.GetAssetName(), NewClass->GetFlags() & ~RF_Transient);
					}
				}
			}
		}

		FName NewName = Record.OldInstance->GetFName();
		EObjectFlags ObjectFlags = Record.OldInstance->GetFlags() & RF_Reinst;

		// Making sure the template we are about to use had all its references patched up
		UObject* Template = NewClass->GetDefaultObject();
		if (Params.OptionalReplacementMappings)
		{
			FArchiveReplaceObjectRef<UObject> ReplaceInCDOAr(Template, *Params.OptionalReplacementMappings);
		}

		UObject* NewInstance = ::NewObject<UObject>(DestinationOuter, NewClass, NewName, ObjectFlags, Template);
		InOutSubobjectArray.Add(NewInstance);
		OldToNewIndex[OldInstanceIndex] = UnsortedNewInstances.Num();
		UnsortedNewInstances.Add(NewInstance);
	}
}
#endif

void UEngine::CopyPropertiesForUnrelatedObjects(UObject* OldObject, UObject* NewObject, FCopyPropertiesForUnrelatedObjectsParams Params)
{
	// UObject::CollectDefaultSubobjects will not return all DSOs for the CDO (only returns inherited subobjects). It also
	// includes any subobjects that have a matching name on the archetype. This function returns all subobjects that have
	// a matching instancing on Object's archetype and any objects tagged as RF_DefaultSubObject. Non-DSO instanced subobjects
	// may be missing, but testing will determine that:
	const auto CollectAllSubobjects = [&Params](UObject* Object, TArray<UObject*>& OutSubobjectArray)
	{
		auto ShouldExcludeSubobjectFromCopy = FOverridableManager::Get().IsEnabled(Object) ?
			[](UObject* SubObject)
			{
				// Here we are trying to prevent an extra copy of COD direct subobjects. The problem is that IsDefaultSubobject() 
				// method is it not consistent when called on subobjects of an instance vs on subobjects of a CDO. The method 
				// PreCreateSubObjectsForReinstantiation creates and return non default sub objects to be copied. But if those 
				// subobject are then considered here by IsDefaultSubobject() as default sub object, it will trigger an extra copy. 
				// This double copy might be handle ok by Delta Serialization but the Overridable Serialization is unable to support it. 
				// The reason is that it implements container removal operations and might remove wrong items during the subsequent copies.
				// So for that reason the object that enabled Overridable serialization, we are just checking if the archetype 
				// of the subobject is a CDO which is partially what IsDefaultSubobject() does.
				return SubObject->GetArchetype()->HasAnyFlags(RF_ClassDefaultObject) && !SubObject->HasAnyFlags(RF_DefaultSubObject);
			}
			:
			[](UObject* SubObject)
			{
				return !SubObject->IsDefaultSubobject() && !SubObject->HasAnyFlags(RF_DefaultSubObject);
			};
			
		GetObjectsWithOuter(Object, OutSubobjectArray, Params.bOnlyHandleDirectSubObjects ? EGetObjectsFlags::None : EGetObjectsFlags::IncludeNestedObjects);

		// Remove contained objects that are not subobjects.
		for ( int32 ComponentIndex = 0; ComponentIndex < OutSubobjectArray.Num(); ComponentIndex++ )
		{
			UObject* PotentialComponent = OutSubobjectArray[ComponentIndex];
			if (ShouldExcludeSubobjectFromCopy(PotentialComponent))
			{
				OutSubobjectArray.RemoveAtSwap(ComponentIndex--);
			}
			else if(Params.bOnlyHandleDirectSubObjects)
			{
				// If it is a default sub object, get its default
				GetObjectsWithOuter(PotentialComponent, OutSubobjectArray, EGetObjectsFlags::None);
			}
		}
	};

	check(OldObject && NewObject);

#if WITH_EDITOR
	FGuid NewActorGuid;
	FGuid NewActorInstanceGuid;
	const UExternalDataLayerAsset* NewExternalDataLayerAsset = nullptr;
#endif

	// Bad idea to write data to an actor while its components are registered
	AActor* NewActor = Cast<AActor>(NewObject);
	if (NewActor != nullptr)
	{
		for (UActorComponent* Component : NewActor->GetComponents())
		{
			ensure(Component == nullptr || !Component->IsRegistered());
		}

#if WITH_EDITOR
		NewActorGuid = NewActor->GetActorGuid();
		NewActorInstanceGuid = NewActor->GetActorInstanceGuid();
		NewExternalDataLayerAsset = NewActor->GetExternalDataLayerAsset();
#endif
	}

	// If the new object is an Actor, save the root component reference, to be restored later
	USceneComponent* SavedRootComponent = nullptr;
	FObjectProperty* RootComponentProperty = nullptr;
	if (NewActor != nullptr && Params.bPreserveRootComponent)
	{
		RootComponentProperty = FindFProperty<FObjectProperty>(NewActor->GetClass(), "RootComponent");
		if (RootComponentProperty != nullptr)
		{
			SavedRootComponent = Cast<USceneComponent>(RootComponentProperty->GetObjectPropertyValue_InContainer(NewActor));
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
#endif

	// Serialize out the modified properties on the old default object
	TIndirectArray<FInstancedObjectRecord> SavedInstances;
	TMap<FString, int32> OldInstanceMap;
	Verse::FDuplicationContext DuplicatedCells;
	// Save the modified properties of the old CDO
	FCPFUOWriter Writer(OldObject, NewObject, Params, DuplicatedCells);

	{
		// Find all instanced objects of the old CDO, and save off their modified properties to be later applied to the newly instanced objects of the new CDO
		TArray<UObject*> Components;
		CollectAllSubobjects( OldObject, Components );

		for (UObject* OldInstance : Components)
		{
			FInstancedObjectRecord* pRecord = new FInstancedObjectRecord();
			SavedInstances.Add(pRecord);
			pRecord->OldInstance = OldInstance;
			OldInstanceMap.Add(OldInstance->GetPathName(OldObject), SavedInstances.Num() - 1);

			FObjectCellWriter SubObjWriter(pRecord->SavedProperties, DuplicatedCells);

			SubObjWriter.ArIgnoreArchetypeRef = true;
			SubObjWriter.ArIgnoreClassRef = true;
			SubObjWriter.ArNoDelta = !Params.bDoDelta;
			SubObjWriter.ArPortFlags = PPF_None;
			if (Params.bCopyDeprecatedProperties)
			{
				SubObjWriter.ArPortFlags |= PPF_UseDeprecatedProperties;
			}

		#if USE_STABLE_LOCALIZATION_KEYS
			if (GIsEditor && !(SubObjWriter.ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
			{
				SubObjWriter.SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(OldInstance));
			}
		#endif // USE_STABLE_LOCALIZATION_KEYS

			UE::FScopedObjectSerializeContext ObjectSerializeContext(OldInstance, SubObjWriter);
			OldInstance->Serialize(SubObjWriter);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	// Serialize and recreate VCells referenced by the old CDO and its components.
	//
	// TODO: Consider sharing DuplicatedCells across multiple calls to CPFUO (e.g. in the BP reinstancer)
	// as well as with the StaticDuplicateObject calls in DuplicateEditInlineSubObjects.
	TArray<uint8> SavedCellData;
	{
		FObjectCellWriter CellWriter(SavedCellData, DuplicatedCells);
		DuplicatedCells.WriteUnserializedCells(Context, CellWriter);

		FObjectReaderWithReplacement Reader(SavedCellData, DuplicatedCells);
		DuplicatedCells.CreateSerializedCells(Context, Reader);
	}
#endif

	// Gather references to old instances or objects that need to be replaced after we serialize in saved data
	TMap<UObject*, UObject*> ReferenceReplacementMap;
	ReferenceReplacementMap.Add(OldObject, NewObject);
	UObject* OldArchetype = OldObject->GetArchetype();
	UObject* NewArchetype = NewObject->GetArchetype();
	if (OldArchetype != NewArchetype)
	{
		// When an archetype is removed from a cdo, the GetArchetype will not return the right one here and could even point to the CDO so adding that replacement for a wrong archetype isn't good at all
		// Check the OptionalReplacementMappings if it already contains a mapping to that new archetype and do not add it to the ReferenceReplacementMap
		if(!Params.OptionalReplacementMappings || !Params.OptionalReplacementMappings->FindKey(NewArchetype))
		{
			ReferenceReplacementMap.Add(OldArchetype, NewArchetype);
		}
	}
	if (OldObject->GetClass() != NewObject->GetClass())
	{
		if (Params.bReplaceObjectClassReferences)
		{
			ReferenceReplacementMap.Add(OldObject->GetClass(), NewObject->GetClass());
		}
		ReferenceReplacementMap.Add(OldObject->GetClass()->GetDefaultObject(), NewObject->GetClass()->GetDefaultObject());
	}

	TArray<UObject*> ComponentsOnNewObject;
	TMap<UObject*, UObject*> AggressiveReplaceReferences;
#if !UE_BUILD_SHIPPING
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bAuditAggressiveReplacement = Params.bAggressiveDefaultSubobjectReplacement && CVarAuditAggressiveReferenceReplacement.GetValueOnAnyThread();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
	constexpr bool bAuditAggressiveReplacement = false;
#endif
	const bool bDoAggressiveReplacement = CVarUseAggressiveReferenceReplacement.GetValueOnAnyThread();

	{
		CollectAllSubobjects( NewObject, ComponentsOnNewObject );

#if WITH_EDITOR
		if (UE::IsInstanceDataObjectSupportEnabled(NewObject))
		{
			CreateLooseSubobjectsForCPFUO(OldObject, NewObject, ComponentsOnNewObject, OldInstanceMap, SavedInstances, Params);
		}
#endif

		// populate the ReferenceReplacementMap 
		for (int32 Index = 0; Index < ComponentsOnNewObject.Num(); Index++)
		{
			UObject* NewInstance = ComponentsOnNewObject[Index];
			if (int32* pOldInstanceIndex = OldInstanceMap.Find(NewInstance->GetPathName(NewObject)))
			{
				FInstancedObjectRecord& Record = SavedInstances[*pOldInstanceIndex];
				ReferenceReplacementMap.Add(Record.OldInstance, NewInstance);
				if(Params.OptionalReplacementMappings)
				{
					Params.OptionalReplacementMappings->Add(Record.OldInstance, NewInstance);
				}
				if (bDoAggressiveReplacement || bAuditAggressiveReplacement)
				{
					UClass* Class = OldObject->GetClass()->GetSuperClass();
					if (Class)
					{
						UObject* CDOInst = Class->GetDefaultSubobjectByName(NewInstance->GetFName());
						if (CDOInst)
						{
							if (bAuditAggressiveReplacement)
							{
								AggressiveReplaceReferences.Add(CDOInst, NewInstance);
							}
							if (bDoAggressiveReplacement)
							{
								ReferenceReplacementMap.Add(CDOInst, NewInstance);
							}
#if WITH_EDITOR
							UBlueprint* BPGeneratedBy = CastChecked<UBlueprint>(Class->ClassGeneratedBy.Get(), ECastCheckedType::NullAllowed);
							if (BPGeneratedBy && BPGeneratedBy->SkeletonGeneratedClass)
							{
								UObject* CDOInstS = BPGeneratedBy->SkeletonGeneratedClass->GetDefaultSubobjectByName(NewInstance->GetFName());
								if (CDOInstS)
								{
									if (bAuditAggressiveReplacement)
									{
										AggressiveReplaceReferences.Add(CDOInstS, NewInstance);
									}
									if (bDoAggressiveReplacement)
									{
										ReferenceReplacementMap.Add(CDOInstS, NewInstance);
									}
								}
							}
#endif // WITH_EDITOR
						}
					}
				}
			}
			else
			{
				bool bContainedInsideNewInstance = false;
				for (UObject* Parent = NewInstance->GetOuter(); Parent != NULL; Parent = Parent->GetOuter())
				{
					if (Parent == NewObject)
					{
						bContainedInsideNewInstance = true;
						break;
					}
				}

				if (!bContainedInsideNewInstance)
				{
					// A bad thing has happened and cannot be reasonably fixed at this point
					UE_LOGF(LogEngine, Log, "Warning: The CDO '%ls' references a component that does not have the CDO in its outer chain!", *NewObject->GetFullName());
				}
			}
		}

		TMap<UObject*, UObject*>* OptionalReferenceReplacementMap = nullptr;
		const TMap<UClass*, UClass*>* OptionalClassReferenceReplacementMap = nullptr;
		if (Params.bReplaceInternalReferenceUponRead)
		{
			OptionalReferenceReplacementMap = Params.OptionalReplacementMappings ? Params.OptionalReplacementMappings : &ReferenceReplacementMap;
			OptionalClassReferenceReplacementMap = Params.OptionalOldToNewClassMappings;
		}

		// Serialize in the modified properties from the old CDO to the new CDO
		if (Writer.SavedPropertyData.Num() > 0)
		{
			FCPFUOReader Reader(Writer, NewObject, Params, DuplicatedCells, OptionalReferenceReplacementMap, OptionalClassReferenceReplacementMap);
		}

		TArray<UObject*> EditInlineSubobjectsOfComponents;
		for (UObject* NewInstance : ComponentsOnNewObject)
		{
			if (int32* pOldInstanceIndex = OldInstanceMap.Find(NewInstance->GetPathName(NewObject)))
			{
				// Restore modified properties into the new instance
				FInstancedObjectRecord& Record = SavedInstances[*pOldInstanceIndex];
				FObjectReaderWithReplacement Reader(NewInstance, Record.SavedProperties, DuplicatedCells, true, true, OptionalReferenceReplacementMap, OptionalClassReferenceReplacementMap);
				UE::Engine::Private::DuplicateEditInlineSubObjects(Record.OldInstance, NewInstance, ReferenceReplacementMap, EditInlineSubobjectsOfComponents, Params.OptionalReplacementMappings);
			}
		}
		ComponentsOnNewObject.Append(MoveTemp(EditInlineSubobjectsOfComponents));
	}

	UE::Engine::Private::DuplicateEditInlineSubObjects(OldObject, NewObject, ReferenceReplacementMap, ComponentsOnNewObject, Params.OptionalReplacementMappings);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	// Serialize in VCells referenced by the new CDO and its components.
	{
		const TMap<UObject*, UObject*>* ObjectMap = Params.OptionalReplacementMappings ?
			Params.OptionalReplacementMappings : &ReferenceReplacementMap;
		const TMap<UClass*, UClass*>* ClassMap = Params.OptionalOldToNewClassMappings;

		FObjectReaderWithReplacement Reader(SavedCellData, DuplicatedCells, ObjectMap, ClassMap);
		DuplicatedCells.ReadSerializedCells(Context, Reader);
	}
#endif

	// Replace anything with an outer of the old object with NULL, unless it already has a replacement or is marked pending kill
	// Also, if requested, leave pointers to instances of renewed classes intact, assuming we are in the midst of reinstancing, and those will be reinstanced
	ForEachObjectWithOuter(OldObject, [&ReferenceReplacementMap, &Params](UObject* ObjectInOuter)
	{
		if (!ReferenceReplacementMap.Contains(ObjectInOuter) && (!Params.OptionalReplacementMappings || !Params.OptionalReplacementMappings->Contains(ObjectInOuter)))
		{
			ReferenceReplacementMap.Add(ObjectInOuter, nullptr);
		}
	}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);

	if(Params.bClearReferences)
	{
		UPackage* NewPackage = NewObject->GetOutermost();
		// Replace references to old classes and instances on this object with the corresponding new ones
		if (bAuditAggressiveReplacement)
		{
			FAggressiveReplacementAuditArchive ReplaceInCDOAr(NewObject, ReferenceReplacementMap, NewPackage, AggressiveReplaceReferences);
		}
		else
		{
			if (Params.OptionalReplacementMappings)
			{
				FArchiveReplaceObjectRef<UObject> OptionalReplaceInCDOAr(NewObject, *Params.OptionalReplacementMappings);
			}
			FArchiveReplaceOrClearExternalReferences<UObject> ReplaceInCDOAr(NewObject, ReferenceReplacementMap, NewPackage);
		}

		// Replace references inside each individual component. This is always required because if something is in ReferenceReplacementMap, the above replace code will skip fixing child properties
		for (int32 ComponentIndex = 0; ComponentIndex < ComponentsOnNewObject.Num(); ++ComponentIndex)
		{
			UObject* NewComponent = ComponentsOnNewObject[ComponentIndex];
			if (bAuditAggressiveReplacement)
			{
				FAggressiveReplacementAuditArchive ReplaceInComponentAr(NewComponent, ReferenceReplacementMap, NewPackage, AggressiveReplaceReferences);
			}
			else
			{
				if (Params.OptionalReplacementMappings)
				{
					FArchiveReplaceObjectRef<UObject> OptionalReplaceInCDOAr(NewComponent, *Params.OptionalReplacementMappings);
				}
				FArchiveReplaceOrClearExternalReferences<UObject> ReplaceInComponentAr(NewComponent, ReferenceReplacementMap, NewPackage);
			}
		}
	}

	// Restore the root component reference and other per-instance properties
	if (NewActor != nullptr && Params.bPreserveRootComponent)
	{
		if (RootComponentProperty != nullptr)
		{
			RootComponentProperty->SetObjectPropertyValue_InContainer(NewActor, SavedRootComponent);
		}

		NewActor->ResetOwnedComponents();

#if WITH_EDITOR
		if (NewActorGuid.IsValid())
		{
			FSetActorGuid SetActorGuid(NewActor, NewActorGuid);
		}

		if (NewActorInstanceGuid.IsValid())
		{
			FSetActorInstanceGuid SetActorInstanceGuid(NewActor, NewActorInstanceGuid);
		}

		// Restore External Data Layer Asset
		const UExternalDataLayerAsset* ExternalDataLayerAsset = NewActor->GetExternalDataLayerAsset();
		if (!GIsReinstancing  && (ExternalDataLayerAsset != NewExternalDataLayerAsset))
		{
			if (ExternalDataLayerAsset)
			{
				FAssignActorDataLayer::RemoveDataLayerAsset(NewActor, ExternalDataLayerAsset);
			}
			if (NewExternalDataLayerAsset)
			{
				FAssignActorDataLayer::AddDataLayerAsset(NewActor, NewExternalDataLayerAsset);
			}
		}
#endif
	}

	bool bDumpProperties = CVarDumpCopyPropertiesForUnrelatedObjects.GetValueOnAnyThread() != 0;
	// Uncomment the next line to debug CPFUO for a specific object:
	// bDumpProperties |= (NewObject->GetName().Find(TEXT("SpinTree")) != INDEX_NONE);
	if (bDumpProperties)
	{
		DumpObject(*FString::Printf(TEXT("CopyPropertiesForUnrelatedObjects: Old (%s)"), *OldObject->GetFullName()), OldObject);
		DumpObject(*FString::Printf(TEXT("CopyPropertiesForUnrelatedObjects: New (%s)"), *NewObject->GetFullName()), NewObject);
	}

	// Now notify any tools that aren't already updated via the FArchiveReplaceObjectRef path unless the OldObject is still being async loaded
	if (Params.bNotifyObjectReplacement && GEngine != nullptr &&
		!OldObject->HasAnyInternalFlags(EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading))
	{
		if (IsInGameThread())
		{
			GEngine->NotifyToolsOfObjectReplacement(ReferenceReplacementMap);
		}
		else
		{
			TMap<FWeakObjectPtr, FWeakObjectPtr> WeakReferenceReplacementMap;
			Algo::Transform(ReferenceReplacementMap, WeakReferenceReplacementMap, [](const auto& InPair)
				{ 
					return TTuple<FWeakObjectPtr, FWeakObjectPtr>(InPair.Key, InPair.Value);
				});

			ExecuteOnGameThread(
				TEXT("NotifyToolsOfObjectReplacement"),
				[WeakReferenceReplacementMap = MoveTemp(WeakReferenceReplacementMap)]()
				{
					TMap<UObject*, UObject*> ValidReferenceReplacementMap;
					Algo::TransformIf(WeakReferenceReplacementMap, ValidReferenceReplacementMap, 
						[](const auto& InPair)
						{
							return InPair.Key.IsValid();
						},
						[](const auto& InPair)
						{
							return TTuple<UObject*, UObject*>( InPair.Key.Get(), InPair.Value.Get() );
						});
					GEngine->NotifyToolsOfObjectReplacement(ValidReferenceReplacementMap);
				}
			);
		}
	}
}

int32 UEngine::GetGlobalFunctionCallspace(UFunction* Function, UObject* FunctionTarget, FFrame* Stack)
{
	check(Function);

	const bool bIsAuthoritativeFunc = Function->HasAnyFunctionFlags(FUNC_BlueprintAuthorityOnly);
	const bool bIsCosmeticFunc = Function->HasAnyFunctionFlags(FUNC_BlueprintCosmetic);

	// If this is an authority/cosmetic function, we need to try and find the global context to see if it's a dedicated server/client
	if (bIsAuthoritativeFunc || bIsCosmeticFunc)
	{
		UWorld* CurrentWorld = nullptr;
#if !WITH_EDITOR
		// In cooked builds there is only one active game world, so this will find it
		CurrentWorld = GetCurrentPlayWorld();
#else
		// In the editor there can be multiple possible PIE worlds at once due to client/server testing and globals not being set, so we need to look for a world to use as context
		UWorld* PossibleWorld = nullptr;

		// First look at function target, this will fail if it's a static function
		if (FunctionTarget)
		{
			PossibleWorld = FunctionTarget->GetWorld();
		}

		// Next check BP stack
		if (!PossibleWorld && Stack && Stack->Object)
			{
			PossibleWorld = Stack->Object->GetWorld();
			}

		CurrentWorld = GetCurrentPlayWorld(PossibleWorld);
#endif

		// If we found a game world context, check it for networking mode
		if (CurrentWorld)
		{
			ENetMode WorldNetMode = CurrentWorld->GetNetMode();
			if (WorldNetMode == NM_DedicatedServer && bIsCosmeticFunc)
			{
				return FunctionCallspace::Absorbed;
		}
			if (WorldNetMode == NM_Client && bIsAuthoritativeFunc)
		{
				return FunctionCallspace::Absorbed;
			}
			}
		}

	// If we can't find a net mode always call locally
	return FunctionCallspace::Local;
}

bool UEngine::ShouldAbsorbAuthorityOnlyEvent()
{
	UWorld* CurrentWorld = GetCurrentPlayWorld();
	if (CurrentWorld)
		{
		return (CurrentWorld->GetNetMode() == NM_Client);
	}

	return false;
}

bool UEngine::ShouldAbsorbCosmeticOnlyEvent()
{
	UWorld* CurrentWorld = GetCurrentPlayWorld();
	if (CurrentWorld)
	{
		return (CurrentWorld->GetNetMode() == NM_DedicatedServer);
		}

	return false;
}

static void SetNearClipPlane(const TArray<FString>& Args)
{
	const float MinClipPlane = 1.0f;
	float NewClipPlane = 20.0f;
	if (Args.Num())
	{
		NewClipPlane = FCString::Atof(*Args[0]);
	}

	SetNearClipPlaneGlobals(FMath::Max(NewClipPlane, MinClipPlane));

}
FAutoConsoleCommand GSetNearClipPlaneCmd(TEXT("r.SetNearClipPlane"),TEXT("Set the near clipping plane (in cm)"),FConsoleCommandWithArgsDelegate::CreateStatic(SetNearClipPlane));

static TAutoConsoleVariable<int32> CVarAllowHighQualityLightMaps(
	TEXT("r.HighQualityLightMaps"),
	1,
	TEXT("If set to 1, allow high quality lightmaps which don't bake in direct lighting of stationary lights"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);


bool AllowHighQualityLightmaps(const FStaticFeatureLevel FeatureLevel)
{
	return FPlatformProperties::SupportsHighQualityLightmaps()
		&& (FeatureLevel > ERHIFeatureLevel::ES3_1)
		&& (CVarAllowHighQualityLightMaps.GetValueOnAnyThread() != 0)
		&& !IsMobilePlatform(GShaderPlatformForFeatureLevel[FeatureLevel]);
}

void FSystemResolution::SetMonitorIndex(int32 MonitorIndex)
{
	CVarMonitorIndex->Set(MonitorIndex);
}

// Helper function for changing system resolution via the r.setres console command
void FSystemResolution::RequestResolutionChange(int32 InResX, int32 InResY, EWindowMode::Type InWindowMode)
{
	if (FPlatformMisc::FullscreenSameAsWindowedFullscreen() && (InWindowMode == EWindowMode::Fullscreen))
	{
		InWindowMode = EWindowMode::WindowedFullscreen;
	}

	FString WindowModeSuffix;
	switch (InWindowMode)
	{
	case EWindowMode::Windowed:
	{
		WindowModeSuffix = TEXT("w");
	} break;
	case EWindowMode::WindowedFullscreen:
	{
		WindowModeSuffix = TEXT("wf");
	} break;
	case EWindowMode::Fullscreen:
	{
		WindowModeSuffix = TEXT("f");
	} break;
	}

	FString NewValue = FString::Printf(TEXT("%dx%d%s"), InResX, InResY, *WindowModeSuffix);
	CVarSystemResolution->SetWithCurrentPriority(*NewValue);
}

void FSystemResolution::ForceRefresh()
{
	SetMonitorIndex(MonitorIndex);
	RequestResolutionChange(ResX, ResY, WindowMode);
	bForceRefresh = true;
}

//////////////////////////////////////////////////////////////////////////
// STATS
void UEngine::ExecEngineStat(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* InName)
{
	// Store a ptr to the viewport that needs to process this stat command
	GStatProcessingViewportClient = ViewportClient;

	FString StatCommand = "STAT ";
	StatCommand += InName;
	Exec(World, *StatCommand, *GLog);
}

bool UEngine::IsEngineStat(const FString& InName)
{
	for (int32 StatIdx = 0; StatIdx < EngineStats.Num(); StatIdx++)
	{
		const FEngineStatFuncs& EngineStat = EngineStats[StatIdx];
		if (EngineStat.CommandNameString == InName)
		{
			return true;
		}
	}
	return false;
}

void UEngine::SetEngineStat(UWorld* World, FCommonViewportClient* ViewportClient, const FString& InName, const bool bShow)
{
	if (ViewportClient && IsEngineStat(InName) && ViewportClient->IsStatEnabled(*InName) != bShow)
	{
		ExecEngineStat(World, ViewportClient, *InName);
	}
}

void UEngine::SetEngineStats(UWorld* World, FCommonViewportClient* ViewportClient, const TArray<FString>& InNames, const bool bShow)
{
	for (int32 StatIdx = 0; StatIdx < InNames.Num(); StatIdx++)
	{
		// If we need to disable, do it in the reverse order incase one stat affects another
		const int32 StatIndex = bShow ? StatIdx : (InNames.Num() - 1) - StatIdx;
		SetEngineStat(World, ViewportClient, *InNames[StatIndex], bShow);
	}
}

void UEngine::RenderEngineStats(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 LHSX, int32& InOutLHSY, int32 RHSX, int32& InOutRHSY, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	for (int32 StatIdx = 0; StatIdx < EngineStats.Num(); StatIdx++)
	{
		const FEngineStatFuncs& EngineStat = EngineStats[StatIdx];
		if (EngineStat.RenderFunc.IsBound() && (!Viewport->GetClient() || Viewport->GetClient()->IsStatEnabled(EngineStat.CommandNameString)))
		{
			// Render the stat either on the left or right hand side of the screen, keeping track of the new Y position
			const int32 StatX = EngineStat.bIsRHS ? RHSX : LHSX;
			int32* StatY = EngineStat.bIsRHS ? &InOutRHSY : &InOutLHSY;
			*StatY = EngineStat.RenderFunc.Execute(World, Viewport, Canvas, StatX, *StatY, ViewLocation, ViewRotation);
		}
	}
}

void UEngine::AddEngineStat(const FName& InCommandName, const FName& InCategoryName, const FText& InDescriptionString, UEngine::FEngineStatRender InRenderFunc /* = nullptr */, UEngine::FEngineStatToggle InToggleFunc /* = nullptr */, const bool bInIsRHS /* = false */)
{
	// Let any listeners know about the new stat
	const FEngineStatFuncs& EngineStat = EngineStats.Add_GetRef(FEngineStatFuncs(InCommandName, InCategoryName, InDescriptionString, InRenderFunc, InToggleFunc, bInIsRHS));
	NewStatDelegate.Broadcast(EngineStat.CommandName, EngineStat.CategoryName, EngineStat.DescriptionString);
}

void UEngine::RemoveEngineStat(const FName& InCommandName)
{
	if (int32 Index = EngineStats.IndexOfByPredicate([InCommandName](const FEngineStatFuncs& CheckStat) { return CheckStat.CommandName == InCommandName; }))
	{
		EngineStats.RemoveAt(Index);
	}
}

// VERSION
#if !UE_BUILD_SHIPPING
int32 UEngine::RenderStatVersion(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	if (!GIsHighResScreenshot && !GIsDumpingMovie && GAreScreenMessagesEnabled)
	{
		if (!bSuppressMapWarnings)
		{
			FText Text = FText::FromString(Viewport->AppVersionString);
			int32 XSize = 0;
			int32 YSize = 0;
			StringSize(GEngine->GetSmallFont(), XSize, YSize, *Text.ToString());

			FCanvasTextItem TextItem(FVector2D(X - XSize, Y), Text, GetSmallFont(), FLinearColor::Yellow);
			TextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(TextItem);
			Y += TextItem.DrawnSize.Y;
		}
	}
	return Y;
}
#endif // !UE_BUILD_SHIPPING

// DETAILED
bool UEngine::ToggleStatDetailed(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}

	// Each of these stats should call "Detailed -Skip" when they themselves are disabled
	static bool bSetup = false;
	static TArray<FString> DetailedStats;
	if (!bSetup)
	{
		bSetup = true;
		DetailedStats.Add(TEXT("FPS"));
		DetailedStats.Add(TEXT("Unit"));
		DetailedStats.Add(TEXT("UnitCriticalPath"));
		DetailedStats.Add(TEXT("UnitMax"));
		DetailedStats.Add(TEXT("UnitGraph"));
		DetailedStats.Add(TEXT("Raw"));
		DetailedStats.Add(TEXT("Version"));
		DetailedStats.Add(TEXT("TSR"));
	}

	// If any of the detailed stats are inactive, take this as enabling all, unless 'Skip' is specifically specified
	const bool bSkip = Stream ? FParse::Param(Stream, TEXT("Skip")) : false;
	if (!bSkip)
	{
		// Enable or disable all the other stats depending on the current state
		const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
		SetEngineStats(World, ViewportClient, DetailedStats, bShowDetailed);

		// Extra stat, needs to do the opposite of the others (order of exec unimportant)
		SetEngineStat(World, ViewportClient, TEXT("UnitTime"), !bShowDetailed);
	}

	return true;
}

// FPS
bool UEngine::ToggleStatFPS(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	const bool bShowFPS = ViewportClient->IsStatEnabled(TEXT("FPS"));
	const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
	if (!bShowFPS && bShowDetailed)
	{
		// Since we're turning this off, we also need to toggle off detailed too
		ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
	}

	return true;
}

int32 UEngine::RenderStatFPS(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Pick a larger font on console.
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GetSmallFont() : GetMediumFont();

	// Choose the counter color based on the average frame time.
	const FColor FPSColor = GetFrameTimeDisplayColor(GAverageMS);

	// Start drawing the various counters.
	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);

	if (CustomTimeStep)
	{
		ECustomTimeStepSynchronizationState State = CustomTimeStep->GetSynchronizationState();
		FString CustomTimeStepName = CustomTimeStep->GetDisplayName();
		int32 NewX = X - Font->GetStringSize(*CustomTimeStepName);
		switch (State)
		{
		case ECustomTimeStepSynchronizationState::Closed:
			Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s : Closed"), *CustomTimeStepName), Font, FColor::Red);
			break;
		case ECustomTimeStepSynchronizationState::Error:
			Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s : Error"), *CustomTimeStepName), Font, FColor::Red);
			break;
		case ECustomTimeStepSynchronizationState::Synchronized:
			Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s : Synchronized"), *CustomTimeStepName), Font, FColor::Green);
			break;
		case ECustomTimeStepSynchronizationState::Synchronizing:
			Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s : Synchronizing"), *CustomTimeStepName), Font, FColor::Yellow);
			break;
		default:
			check(false);
			break;
		}
		Y += RowHeight;
	}

	// Draw the FPS counter.
	Canvas->DrawShadowedString(
		X,
		Y,
		*FString::Printf(TEXT("%5.2f FPS"), GAverageFPS),
		Font,
		FPSColor
	);
	Y += RowHeight;

	// Draw the frame time.
	Canvas->DrawShadowedString(
		X,
		Y,
		*FString::Printf(TEXT("%5.2f ms"), GAverageMS),
		Font,
		FPSColor
	);
	Y += RowHeight;

	return Y;
}

// HITCHES
bool UEngine::ToggleStatHitches(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	FPlatformProcess::Sleep(0.11f); // cause a hitch so it is evidently working
	return false;
}

int32 UEngine::RenderStatHitches(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Forward this draw request to the viewport client
	if (Viewport->GetClient())
	{
		checkf(Viewport->GetClient()->GetStatHitchesData(), TEXT("StatHitchesData must be allocated for this viewport if you wish to display stat."));
		Y = Viewport->GetClient()->GetStatHitchesData()->DrawStat(Viewport, Canvas, X, Y);
	}
	return Y;
}

// SUMMARY
int32 UEngine::RenderStatSummary(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Pick a larger font on console.
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GetSmallFont() : GetMediumFont();

	// Retrieve allocation info.
	FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
	float MemoryInMByte = MemoryStats.UsedPhysical / 1024.f / 1024.f;

	// Draw the memory summary stats.
	Canvas->DrawShadowedString(
		X,
		Y,
		*FString::Printf(TEXT("%5.2f MByte"), MemoryInMByte),
		Font,
		FColor(30, 144, 255)
	);

	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);
	Y += RowHeight;
	return Y;
}

// NAMEDEVENTS
bool UEngine::ToggleStatNamedEvents(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}

	// Enable emission of named events and force enable cycle stats.
	if (ViewportClient->IsStatEnabled(TEXT("NamedEvents")))
	{
		if (GCycleStatsShouldEmitNamedEvents == 0)
		{
			StatsPrimaryEnableAdd();
		}
		GCycleStatsShouldEmitNamedEvents++;
	}
	// Disable emission of named events and force-enabling cycle stats.
	else
	{
		if (GCycleStatsShouldEmitNamedEvents == 1)
		{
			StatsPrimaryEnableSubtract();
		}
		GCycleStatsShouldEmitNamedEvents = FMath::Max(0, GCycleStatsShouldEmitNamedEvents - 1);
	}
	return false;
}

// VERBOSE NAMED EVENTS
bool UEngine::ToggleStatVerboseNamedEvents(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	GShouldEmitVerboseNamedEvents = !GShouldEmitVerboseNamedEvents;

	if (GShouldEmitVerboseNamedEvents && ViewportClient && !ViewportClient->IsStatEnabled(TEXT("NamedEvents")))
	{
		SetEngineStat(World, ViewportClient, TEXT("NamedEvents"), true);
	}

	return false;
}

int32 UEngine::RenderNamedEventsEnabled(FCanvas* Canvas, int32 X, int32 Y)
{
	if (GCycleStatsShouldEmitNamedEvents)
	{
		const FText Text = GShouldEmitVerboseNamedEvents ?
			LOCTEXT("VERBOSENAMEDEVENTSENABLED", "VERBOSE NAMED EVENTS ENABLED") :
			LOCTEXT("NAMEDEVENTSENABLED", "NAMED EVENTS ENABLED");
		X -= GShouldEmitVerboseNamedEvents ? 100 : 42;
		FCanvasTextItem TextItem(FVector2D(X, Y), Text, GetSmallFont(), FLinearColor::Blue);
		TextItem.EnableShadow(FLinearColor::Black);
		Canvas->DrawItem(TextItem);
		Y += TextItem.DrawnSize.Y;
	}
	return Y;
}

// COLORLIST
int32 UEngine::RenderStatColorList(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	UFont* Font = GetTinyFont();

	const int32 LineHeight = FMath::TruncToInt(Font->GetMaxCharHeight());
	const int32 ColorsNum = GColorList.GetColorsNum();
	const int32 MaxLinesInColumn = 35;
	const int32 ColumnsNum = FMath::CeilToInt((float)ColorsNum / (float)MaxLinesInColumn);

	Y += 16;
	const int32 SavedY = Y;
	int32 LowestY = Y + MaxLinesInColumn * LineHeight;

	// Draw columns with color list.
	for (int32 ColumnIndex = 0; ColumnIndex < ColumnsNum; ColumnIndex++)
	{
		int32 LineWidthMax = 0;

		for (int32 ColColorIndex = 0; ColColorIndex < MaxLinesInColumn; ColColorIndex++)
		{
			const int32 ColorIndex = ColumnIndex * MaxLinesInColumn + ColColorIndex;
			if (ColorIndex >= ColorsNum)
			{
				break;
			}

			const FColor& Color = GColorList.GetFColorByIndex(ColorIndex);
			const FString Line = *FString::Printf(TEXT("%3i %s %s"), ColorIndex, *GColorList.GetColorNameByIndex(ColorIndex), *Color.ToString());

			LineWidthMax = FMath::Max(LineWidthMax, Font->GetStringSize(*Line));

			Canvas->DrawShadowedString(X, Y, *Line, Font, FLinearColor(Color));
			Y += LineHeight;
		}

		X += LineWidthMax;
		LineWidthMax = 0;
		Y = SavedY;
	}
	return LowestY;
}

// LEVELS

int32 UEngine::RenderStatLevels(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	int32 MaxY = Y;
	const TArray<FSubLevelStatus> SubLevelsStatusList = GetSubLevelsStatus(World);

	// now do drawing to the screen

	// Render unloaded levels in red, loaded ones in yellow and visible ones in green. Blue signifies that a level is unloaded but
	// hasn't been garbage collected yet.
	Canvas->DrawShadowedString(X, Y, TEXT("Levels"), GetSmallFont(), FLinearColor::White);
	Y += 12;

	FString MapShortName;
	if (SubLevelsStatusList.Num())
	{
		// First entry - always persistent level
		FString MapName	= SubLevelsStatusList[0].PackageName.ToString();
		MapShortName = FPackageName::GetShortName(MapName);
		UPackage* LevelPackage = FindObjectFast<UPackage>(NULL, SubLevelsStatusList[0].PackageName);
		if (SubLevelsStatusList[0].bPlayerInside)
		{
			MapName = *FString::Printf(TEXT("->  %s - %4.1f sec"), *MapName, LevelPackage->GetLoadTime());
		}
		else
		{
			MapName = *FString::Printf(TEXT("    %s - %4.1f sec"), *MapName, LevelPackage->GetLoadTime());
		}

		Canvas->DrawShadowedString(X, Y, *MapName, GetSmallFont(), FColor(127, 127, 127));
		Y += 12;
	}

	int32 BaseY = Y;

	// now draw the levels
	float CurrentColumnWidth = 0.f;
	float TextWidth = 0.f;
	float TextHeight = 0.f;
	
	for (int32 LevelIdx = 1; LevelIdx < SubLevelsStatusList.Num(); ++LevelIdx)
	{
		const FSubLevelStatus& LevelStatus = SubLevelsStatusList[LevelIdx];

		// Wrap around at the bottom.
		if (Y > Canvas->GetRenderTarget()->GetSizeXY().Y - 30)
		{
			MaxY = FMath::Max(MaxY, Y);
			Y = BaseY;
			X += CurrentColumnWidth;
			CurrentColumnWidth = 0.f;
		}

		FColor	Color = ULevelStreaming::GetLevelStreamingStatusColor(LevelStatus.StreamingStatus);
		const TCHAR* StatusDisplayName = ULevelStreaming::GetLevelStreamingStatusDisplayName(LevelStatus.StreamingStatus);
		FString LevelPackageName = LevelStatus.LevelLabel.IsEmpty() ? LevelStatus.PackageName.ToString() : LevelStatus.LevelLabel;
		FString DisplayName = UWorld::RemovePIEPrefix(FPackageName::GetShortName(LevelPackageName));

		if (LevelStatus.LODIndex != INDEX_NONE)
		{
			DisplayName += FString::Printf(TEXT(" [LOD%d]"), LevelStatus.LODIndex+1);
		}

		UPackage* LevelPackage = FindObjectFast<UPackage>(NULL, LevelStatus.PackageName);

		if (LevelPackage
			&& (LevelPackage->GetLoadTime() > 0)
			&& (LevelStatus.StreamingStatus != LEVEL_Unloaded))
		{
			DisplayName += FString::Printf(TEXT(" - %4.1f sec"), LevelPackage->GetLoadTime());
		}
		else
		{
			const float AsyncLoadPercentage = GetAsyncLoadPercentage(*LevelStatus.PackageName.ToString());
			if (AsyncLoadPercentage >= 0.0f)
			{
				const int32 Percentage = FMath::TruncToInt(AsyncLoadPercentage);
				DisplayName += FString::Printf(TEXT(" - %3i %%"), Percentage);
			}
		}

		DisplayName = *FString::Printf(TEXT("%s %s %s (%s)"), (LevelStatus.bPlayerInside ? TEXT("->") : TEXT("  ")), (LevelStatus.bInConsiderList ? TEXT("*") : TEXT(" ")), *DisplayName, StatusDisplayName);

		Canvas->DrawShadowedString(X + 4, Y, *DisplayName, GetSmallFont(), Color);
		UCanvas::StrLen(GetSmallFont(), *DisplayName, TextWidth, TextHeight, /*bDPIAware*/false, Canvas);
		CurrentColumnWidth = FMath::Max(CurrentColumnWidth, TextWidth);
		Y += TextHeight;
	}
	return FMath::Max(MaxY, Y);
}

// LEVELMAP
int32 UEngine::RenderStatLevelMap(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	const FVector2D MapOrigin = FVector2D(512, 128);
	const FVector2D MapSize = FVector2D(512, 512);

	// Get status of each sublevel (by name)
	const TArray<FSubLevelStatus> SubLevelsStatusList = GetSubLevelsStatus(World);

	// First iterate to find bounds of all streaming volumes
	FBox AllVolBounds(ForceInit);
	for (const FSubLevelStatus& LevelStatus : SubLevelsStatusList)
	{
		ULevelStreaming* LevelStreaming = World->GetLevelStreamingForPackageName(LevelStatus.PackageName);
		if (LevelStreaming && LevelStreaming->bDrawOnLevelStatusMap)
		{
			AllVolBounds += LevelStreaming->GetStreamingVolumeBounds();
		}
	}

	// We need to ensure the XY aspect ratio of AllVolBounds is the same as the map

	// Work out scale factor between map and world
	FVector VolBoundsSize = (AllVolBounds.Max - AllVolBounds.Min);
	float ScaleX = MapSize.X / VolBoundsSize.X;
	float ScaleY = MapSize.Y / VolBoundsSize.Y;
	float UseScale = FMath::Min(ScaleX, ScaleY); // Pick the smallest scaling factor

												 // Resize AllVolBounds
	FVector NewVolBoundsSize = VolBoundsSize;
	NewVolBoundsSize.X = MapSize.X / UseScale;
	NewVolBoundsSize.Y = MapSize.Y / UseScale;
	FVector DeltaBounds = (NewVolBoundsSize - VolBoundsSize);
	AllVolBounds.Min -= 0.5f * DeltaBounds;
	AllVolBounds.Max += 0.5f * DeltaBounds;

	// Find world-space location for top-left and bottom-right corners of map
	FVector2D TopLeftPos(AllVolBounds.Max.X, AllVolBounds.Min.Y); // max X, min Y
	FVector2D BottomRightPos(AllVolBounds.Min.X, AllVolBounds.Max.Y); // min X, max Y


																	  // Now we iterate and actually draw volumes
	for (const FSubLevelStatus& LevelStatus : SubLevelsStatusList)
	{
		// Find the color to draw this level in
		FColor StatusColor = ULevelStreaming::GetLevelStreamingStatusColor(LevelStatus.StreamingStatus);
		StatusColor.A = 64; // make it translucent

		ULevelStreaming* LevelStreaming = World->GetLevelStreamingForPackageName(LevelStatus.PackageName);
		if (LevelStreaming && LevelStreaming->bDrawOnLevelStatusMap)
		{
			for (int32 VolIdx = 0; VolIdx < LevelStreaming->EditorStreamingVolumes.Num(); VolIdx++)
			{
				ALevelStreamingVolume* StreamingVol = LevelStreaming->EditorStreamingVolumes[VolIdx];
				if (StreamingVol)
				{
					DrawVolumeOnCanvas(StreamingVol, Canvas, TopLeftPos, BottomRightPos, MapOrigin, MapSize, StatusColor);
				}
			}
		}
	}



	// Now we want to draw the player(s) location on the map
	{
		// Find map location for arrow
		check(ViewLocation);
		const FVector2D PlayerMapPos = TransformLocationToMap(TopLeftPos, BottomRightPos, MapOrigin, MapSize, *ViewLocation);

		// Make verts for little rotated arrow
		check(ViewRotation);
		float PlayerYaw = (ViewRotation->Yaw * UE_PI / 180.f) - (0.5f * UE_PI); // We have to add 90 degrees because +X in world space means -Y in map space
		const FVector2D M0 = PlayerMapPos + RotateVec2D(FVector2D(7, 0), PlayerYaw);
		const FVector2D M1 = PlayerMapPos + RotateVec2D(FVector2D(-7, 5), PlayerYaw);
		const FVector2D M2 = PlayerMapPos + RotateVec2D(FVector2D(-7, -5), PlayerYaw);

		FCanvasTriangleItem TriItem(M0, M1, M2, GWhiteTexture);
		Canvas->DrawItem(TriItem);
	}
	return Y;
}

// UNIT
bool UEngine::ToggleStatUnit(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}

	const bool bShowUnitMaxTimes = ViewportClient->IsStatEnabled(TEXT("UnitMax"));
	if (bShowUnitMaxTimes != false)
	{
		// Toggle UnitMax back to Inactive
		ExecEngineStat(World, ViewportClient, TEXT("UnitMax"));

		// Force Unit back to Active if turning UnitMax off
		SetEngineStat(World, ViewportClient, TEXT("Unit"), true);
	}

	const bool bShowUnitTimes = ViewportClient->IsStatEnabled(TEXT("Unit"));
	const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
	if (!bShowUnitTimes && bShowDetailed)
	{
		// Since we're turning this off, we also need to toggle off detailed too
		ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
	}

	return true;
}

int32 UEngine::RenderStatUnit(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Forward this draw request to the viewport client
	if (Viewport->GetClient())
	{
		checkf(Viewport->GetClient()->GetStatUnitData(), TEXT("StatUnitData must be allocated for this viewport if you wish to display stat."));
		Y = Viewport->GetClient()->GetStatUnitData()->DrawStat(Viewport, Canvas, X, Y);
	}

	return Y;
}

// THERMALS
bool UEngine::ToggleStatThermals(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if (ViewportClient == nullptr)
	{
		// Ignore if all Viewports are closed.
		return false;
	}

	return true;
}

int32 UEngine::RenderStatThermals(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Forward this draw request to the viewport client
	if (Viewport->GetClient())
	{
		checkf(Viewport->GetClient()->GetStatThermalsData(), TEXT("StatThermalsData must be allocated for this viewport if you wish to display stat."));
		Y = Viewport->GetClient()->GetStatThermalsData()->DrawStat(Viewport, Canvas, X, Y);
	}

	return Y;
}

// DRAWCOUNT
int32 UEngine::RenderStatDrawCount(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
#if HAS_GPU_STATS

	int32 TotalCount[MAX_NUM_GPUS] = { 0 };
	// Display all the categories of draw counts.
	FRHIDrawStatsCategory::FManager const& Manager = FRHIDrawStatsCategory::GetManager();

	{
		FScopeLock Lock(&Manager.DisplayCountsCS);

		for (int32 Index = 0; Index < Manager.NumCategory; ++Index)
		{
			for (uint32 GPUIndex : FRHIGPUMask::All())
			{
				TotalCount[GPUIndex] += Manager.DisplayCounts[Index][GPUIndex];
				FRHIDrawStatsCategory* CategoryName = Manager.Array[Index];

				Canvas->DrawShadowedString(
					X - 100, Y,
					*FString::Printf(
						TEXT("%s%s: %i"),
						*CategoryName->Name.ToString(),
						GNumExplicitGPUsForRendering > 1 ? *FString::Printf(TEXT(" (GPU%u)"), GPUIndex) : TEXT(""),
						Manager.DisplayCounts[Index][GPUIndex]
					),
					GetSmallFont(),
					FColor::Green
				);

				Y += 12;
			}
		}
	}

	for (uint32 GPUIndex : FRHIGPUMask::All())
	{
		Canvas->DrawShadowedString(
			X - 100, Y,
			*FString::Printf(
				TEXT("Total%s: %i"),
				GNumExplicitGPUsForRendering > 1 ? *FString::Printf(TEXT(" (GPU%u)"), GPUIndex) : TEXT(""),
				TotalCount[GPUIndex]
			),
			GetSmallFont(),
			FColor::Green
		);

		Y += 12;
	}

#else
	Canvas->DrawShadowedString(X - 200,
		Y,
		*FString::Printf(TEXT("DrawCount only supported with HAS_GPU_STATS")),
		GetSmallFont(),
		FColor::Red);
#endif

	return Y;
}

#if !UE_BUILD_SHIPPING

// UNITCRITICALPATH
bool UEngine::ToggleStatUnitCriticalPath(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if (ViewportClient == nullptr)
	{
		// Ignore if all Viewports are closed.
		return false;
	}
	const bool bShowUnitCriticalPathTimes = ViewportClient->IsStatEnabled(TEXT("UnitCriticalPath"));
	if (bShowUnitCriticalPathTimes)
	{
		// Force Unit to Active
		SetEngineStat(World, ViewportClient, TEXT("Unit"), true);

		// Force UnitCriticalPath to true as Unit will have Toggled it back to false
		SetEngineStat(World, ViewportClient, TEXT("UnitCriticalPath"), true);
	}
	else
	{
		const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
		if (bShowDetailed)
		{
			// Since we're turning this off, we also need to toggle off detailed too
			ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
		}
	}
	return true;
}

// UNITMAX
bool UEngine::ToggleStatUnitMax(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}
	const bool bShowUnitMaxTimes = ViewportClient->IsStatEnabled(TEXT("UnitMax"));
	if (bShowUnitMaxTimes)
	{
		// Force Unit to Active
		SetEngineStat(World, ViewportClient, TEXT("Unit"), true);

		// Force UnitMax to true as Unit will have Toggled it back to false
		SetEngineStat(World, ViewportClient, TEXT("UnitMax"), true);
	}
	else
	{
		const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
		if (bShowDetailed)
		{
			// Since we're turning this off, we also need to toggle off detailed too
			ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
		}
	}
	return true;
}

// UNITGRAPH
bool UEngine::ToggleStatUnitGraph(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}
	const bool bShowUnitGraph = ViewportClient->IsStatEnabled(TEXT("UnitGraph"));
	if (bShowUnitGraph)
	{
		// Force Unit to Active
		SetEngineStat(World, ViewportClient, TEXT("Unit"), true);

		// Force UnitTime to Active
		SetEngineStat(World, ViewportClient, TEXT("UnitTime"), true);	
	}
	else
	{
		const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
		if (bShowDetailed)
		{
			// Since we're turning this off, we also need to toggle off detailed too
			ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
		}
	}
	return true;
}

// UNITTIME
bool UEngine::ToggleStatUnitTime(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if( ViewportClient == nullptr )
	{
		// Ignore if all Viewports are closed.
		return false;
	}
	const bool bShowUnitTime = ViewportClient->IsStatEnabled(TEXT("UnitTime"));
	if (bShowUnitTime)
	{
		// Force UnitGraph to Active
		SetEngineStat(World, ViewportClient, TEXT("UnitGraph"), true);
	}
	return true;
}

// RAW
bool UEngine::ToggleStatRaw(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	const bool bShowRaw = ViewportClient->IsStatEnabled(TEXT("Raw"));
	const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
	if (bShowRaw)
	{
		// Force UnitGraph to Active
		SetEngineStat(World, ViewportClient, TEXT("UnitGraph"), true);
	}
	else if (bShowDetailed)
	{
		// Since we're turning this off, we also need to toggle off detailed too
		ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
	}
	return true;
}

// TSR statistics
bool UEngine::ToggleStatTSR(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
	if (ViewportClient == nullptr)
	{
		// Ignore if all Viewports are closed.
		return false;
	}
	const bool bShowTSR = ViewportClient->IsStatEnabled(TEXT("TSR"));
	if (bShowTSR)
	{
		// Force Unit to Active
		SetEngineStat(World, ViewportClient, TEXT("Unit"), true);

		// Force TSR to true as Unit will have Toggled it back to false
		SetEngineStat(World, ViewportClient, TEXT("TSR"), true);
	}
	else
	{
		const bool bShowDetailed = ViewportClient->IsStatEnabled(TEXT("Detailed"));
		if (bShowDetailed)
		{
			// Since we're turning this off, we also need to toggle off detailed too
			ExecEngineStat(World, ViewportClient, TEXT("Detailed -Skip"));
		}
	}
	return true;
}

#endif

#if !UE_BUILD_SHIPPING

static TArray<FString> AssetLoadTest_Filenames;

static void TickAssetLoadTest(bool bInfinite = false)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_UEngine_TickAssetLoadTest);

	check(IsInGameThread());
	static FRandomStream RNG(FPlatformTime::Cycles());
	static int32 RequestsOutstanding = 0;
	static int32 NumProcessed = 0;
	if (!IsEngineExitRequested())
	{
		if (RequestsOutstanding < 100)
		{
			int32 IndexToDo;
			bool bFirstTime = false;
			if (NumProcessed < AssetLoadTest_Filenames.Num())
			{
				IndexToDo = (NumProcessed * 9973) % AssetLoadTest_Filenames.Num();
				bFirstTime = true;
			}
			else
			{
				IndexToDo = RNG.RandRange(0, AssetLoadTest_Filenames.Num() - 1);
			}
			FString TestFile = AssetLoadTest_Filenames[IndexToDo];
			int32 CapturedNumProcessed = NumProcessed;
			LoadPackageAsync(TestFile, FLoadPackageAsyncDelegate::CreateLambda(
				[CapturedNumProcessed, bFirstTime](const FName& PackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
			{
				check(IsInGameThread());
				check(Result == EAsyncLoadingResult::Succeeded);
				UE_LOGF(LogStreaming, Log, "UAssetLoadTest[%9d] - %ls %ls", CapturedNumProcessed, bFirstTime ? TEXT("Loaded") : TEXT("Reloaded"), *PackageName.ToString());
				RequestsOutstanding--;
			}
			));
			RequestsOutstanding++;
			NumProcessed++;
		}
		uint64 LocalFrameCounter = GFrameCounter;
		if (bInfinite)
		{
			LocalFrameCounter = NumProcessed / 10;
		}


		if (LocalFrameCounter % (30 * 13) == 0)
		{
			UE_LOGF(LogStreaming, Log, "Flushing Async Loading Test***************************************************");
			FlushAsyncLoading();
			check(RequestsOutstanding == 0);
		}
		if (bInfinite && LocalFrameCounter % (30 * 10) == 0)
		{
			UE_LOGF(LogStreaming, Log, "GC****************************************************************************");
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
			check(RequestsOutstanding == 0);
		}
	}
}

static void DoUAssetLoadTest(const TArray<FString>& Args)
{
	check(IsInGameThread());
	if (AssetLoadTest_Filenames.Num())
	{
		UE_LOGF(LogConsoleResponse, Display, "Already running");
		return;
	}
	FString MountPoint = FPaths::ProjectContentDir();
	if (Args.Num() > 0 && !Args[0].StartsWith(TEXT("-")))
	{
		MountPoint = Args[0];
	}
	bool bInfinite = (Args.Num() > 0 && Args[0].StartsWith(TEXT("-infiniteloop"))) || (Args.Num() > 1 && Args[1].StartsWith(TEXT("-infiniteloop")));

	UE_LOGF(LogConsoleResponse, Display, "Scanning disk, this might take 30s or more with a big game.");

	IFileManager::Get().FindFilesRecursive(AssetLoadTest_Filenames, *MountPoint, TEXT("*.uasset"), true, false);
	if (!AssetLoadTest_Filenames.Num())
	{
		UE_LOGF(LogConsoleResponse, Display, "Try again, no uasset files in %ls", *MountPoint);
		return;
	}

	if (bInfinite)
	{
		static FRandomStream RNG(FPlatformTime::Cycles());
		GEngine->Exec(nullptr, TEXT("log logstreaming log"));
		UE_LOGF(LogConsoleResponse, Display, "Not all aspects of the game are running in an infinite loop like this. Things like stats may not get flushed because the frame never ends. This might appear to be leaking memory.");
		while (true)
		{
			TickAssetLoadTest(true);
			GLog->FlushThreadedLogs();
			bool bUseTimeLimit = RNG.FRand() < .995f;
			bool bUseFullTimeLimit = bUseTimeLimit && RNG.FRand() < .5f;
			float TimeLimit = RNG.FRandRange(0.0011f, 0.025f);

			ProcessAsyncLoading(bUseTimeLimit, bUseFullTimeLimit, TimeLimit);
		}
	}
	else
	{
		GEngine->Exec(nullptr, TEXT("gc.TimeBetweenPurgingPendingKillObjects 10"));
		UE_LOGF(LogConsoleResponse, Display, "gc.TimeBetweenPurgingPendingKillObjects set to 10, you might want to adjust that or do gc.CollectGarbageEveryFrame 1");

		UE_LOGF(LogConsoleResponse, Display, "Testing %d uasset files in %ls", AssetLoadTest_Filenames.Num(), *MountPoint);
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([](float Delta) -> bool { TickAssetLoadTest(); return true; }));
	}

}

static FAutoConsoleCommand UAssetLoadTestCmd(
	TEXT("UAssetLoadTest"),
	TEXT("Continuously load assets and GC in the backgroud. Debugging option, this may or may not work with all assets. The test runs forever. If no arg is given all assets in /Game/Content are scanned."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DoUAssetLoadTest)
);

#endif

struct FThreadConfig
{
	FThreadConfig() { Priority = EThreadPriority::TPri_Normal;  Affinity = 0xFFFFFFFFFFFFFFFF;	}
	uint64 Affinity;
	EThreadPriority Priority;
};

static FThreadConfig GameThreadConfig;
static FThreadConfig RenderThreadConfig;
static FThreadConfig RHIThreadConfig;
static FThreadConfig TaskThreadConfig;
static FThreadConfig TaskBPThreadConfig;

void GetDefaultThreadConfigs()
{
	TaskThreadConfig.Affinity = FPlatformAffinity::GetTaskGraphThreadMask();
	TaskBPThreadConfig.Affinity = FPlatformAffinity::GetTaskGraphBackgroundTaskMask();
	GameThreadConfig.Affinity = FPlatformAffinity::GetMainGameMask();
	RenderThreadConfig.Affinity = FPlatformAffinity::GetRenderingThreadMask();
	RHIThreadConfig.Affinity = FPlatformAffinity::GetRHIThreadMask();

	TaskThreadConfig.Priority = FPlatformAffinity::GetTaskThreadPriority();
	TaskBPThreadConfig.Priority = FPlatformAffinity::GetTaskBPThreadPriority();
	GameThreadConfig.Priority = FPlatformAffinity::GetGameThreadPriority();
	RenderThreadConfig.Priority = FPlatformAffinity::GetRenderingThreadPriority();
	RHIThreadConfig.Priority = FPlatformAffinity::GetRHIThreadPriority();
}

EThreadPriority StringToThreadPriority(FString InString)
{
	if (InString == TEXT("TPri_Normal"))
	{
		return EThreadPriority::TPri_Normal;
	}
	if (InString == TEXT("TPri_AboveNormal"))
	{
		return EThreadPriority::TPri_AboveNormal;
	}
	if (InString == TEXT("TPri_BelowNormal"))
	{
		return EThreadPriority::TPri_BelowNormal;
	}
	if (InString == TEXT("TPri_Highest"))
	{
		return EThreadPriority::TPri_Highest;
	}
	if (InString == TEXT("TPri_Lowest"))
	{
		return EThreadPriority::TPri_Lowest;
	}
	if (InString == TEXT("TPri_SlightlyBelowNormal"))
		{
		return EThreadPriority::TPri_SlightlyBelowNormal;
		}
	if (InString == TEXT("TPri_TimeCritical"))
	{
		return EThreadPriority::TPri_TimeCritical;
	}
	return EThreadPriority::TPri_Num;
}


FString ThreadPriorityToString(EThreadPriority InPriority)
{
	switch (InPriority)
	{
	case EThreadPriority::TPri_Normal:
		return FString("TPri_Normal");
	case EThreadPriority::TPri_AboveNormal:
		return  FString("TPri_AboveNormal");
	case EThreadPriority::TPri_BelowNormal:
		return  FString("TPri_BelowNormal");
	case EThreadPriority::TPri_Highest:
		return  FString("TPri_Highest");
	case EThreadPriority::TPri_Lowest:
		return  FString("TPri_Lowest");
	case EThreadPriority::TPri_SlightlyBelowNormal:
		return  FString("TPri_SlightlyBelowNormal");
	case EThreadPriority::TPri_TimeCritical:
		return  FString("TPri_TimeCritical");
	case EThreadPriority::TPri_Num:
	default:
		break;
	};
	return  FString("TPri_Undefined");
}


void SetPriorityAndAffinityOnGameThread()
{
	if (GameThreadConfig.Priority != EThreadPriority::TPri_Num)
	{
		FPlatformProcess::SetThreadPriority(GameThreadConfig.Priority);
		UE_LOGF(LogConsoleResponse, Display, "GT Priority %ls", *ThreadPriorityToString(GameThreadConfig.Priority));
	}
	FPlatformProcess::SetThreadAffinityMask(GameThreadConfig.Affinity);
	UE_LOGF(LogConsoleResponse, Display, "GT Affinity %016llX", GameThreadConfig.Affinity);
}

void SetPriorityAndAffinityOnRenderThread()
{
	if (RenderThreadConfig.Priority != EThreadPriority::TPri_Num)
	{
		FPlatformProcess::SetThreadPriority(RenderThreadConfig.Priority);
		UE_LOGF(LogConsoleResponse, Display, "RT Priority %ls", *ThreadPriorityToString(RenderThreadConfig.Priority));
	}
	FPlatformProcess::SetThreadAffinityMask(RenderThreadConfig.Affinity);
	UE_LOGF(LogConsoleResponse, Display, "RT Affinity %016llX", RenderThreadConfig.Affinity);
}

void SetPriorityAndAffinityOnRHIThread()
{
	if (RHIThreadConfig.Priority != EThreadPriority::TPri_Num)
	{
		FPlatformProcess::SetThreadPriority(RHIThreadConfig.Priority);
		UE_LOGF(LogConsoleResponse, Display, "RHI Priority %ls", *ThreadPriorityToString(RHIThreadConfig.Priority));
	}
	FPlatformProcess::SetThreadAffinityMask(RHIThreadConfig.Affinity);
	UE_LOGF(LogConsoleResponse, Display, "RHI Affinity %016llX", RHIThreadConfig.Affinity);
}



static void SetupThreadConfig(const TArray<FString>& Args)
{
	UE_LOGF(LogConsoleResponse, Display, "Setting thread configurations");

	// Lazily load the default thread configs
	static bool bLoadedDefaults = false;
	bool bResetToDefaults = (Args.Num() && Args[0] == TEXT("default"));
	if (!bLoadedDefaults || bResetToDefaults)
	{
		GetDefaultThreadConfigs();
		bLoadedDefaults = true;
	}

	if (!bResetToDefaults)
	{
		for (const FString& ThreadConfigStr : Args)
		{
			TArray<FString> ThreadConfigEntries;
			ThreadConfigStr.ParseIntoArray(ThreadConfigEntries, TEXT(":"), true);
			if (ThreadConfigEntries.Num() >= 2)
			{
				FString ThreadName = ThreadConfigEntries[0];
				FThreadConfig* ThreadConfigToSet = nullptr;
				if (ThreadName == TEXT("GT"))
				{
					ThreadConfigToSet = &GameThreadConfig;
				}
				else if (ThreadName == TEXT("RT"))
				{
					ThreadConfigToSet = &RenderThreadConfig;
				}
				else if (ThreadName == TEXT("RHI"))
				{
					ThreadConfigToSet = &RHIThreadConfig;
				}
				else if (ThreadName == TEXT("Task"))
				{
					ThreadConfigToSet = &TaskThreadConfig;
				}
				else if (ThreadName == TEXT("TaskBP"))
				{
					ThreadConfigToSet = &TaskBPThreadConfig;
				}
				if (ThreadConfigToSet == nullptr)
				{
					UE_LOGF(LogConsoleResponse, Warning, "Thread name not found: %ls", *ThreadName);
					continue;
				}
				// Read the rest of the thread config args
				for (int i = 1; i < ThreadConfigEntries.Num(); i++)
				{
					// if the arg is a hex number we are setting affinity
					const FString& Value = ThreadConfigEntries[i];
					if (Value.StartsWith(TEXT("0x")))
					{
						ThreadConfigToSet->Affinity = FParse::HexNumber64(*Value);
					}
					// if the arg starts with TPri we are setting priority
					if (Value.StartsWith(TEXT("TPri")))
					{
						ThreadConfigToSet->Priority = StringToThreadPriority(Value);
					}
				}
			}
		}
	}

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateStatic(&SetPriorityAndAffinityOnRenderThread),
		TStatId(), NULL, ENamedThreads::GetRenderThread());

	if (IsRHIThreadRunning())
	{
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateStatic(&SetPriorityAndAffinityOnRHIThread),
			TStatId(), NULL, ENamedThreads::RHIThread);
	}
	check(IsInGameThread());
	SetPriorityAndAffinityOnGameThread();
	FlushRenderingCommands();

	// Block the render thread while we're restarting workers
	UE::FManualResetEvent ReadyEvent;
	UE::FManualResetEvent WaitEvent;
	UE::FManualResetEvent ExitEvent;
	ENQUEUE_RENDER_COMMAND(StallRT)(
		[&](FRHICommandListImmediate& RHICmdList)
		{
			ReadyEvent.Notify();
			WaitEvent.Wait();
			ExitEvent.Notify();
		}
	);

	// Wait until the RT is really idle
	ReadyEvent.Wait();

	// Try to retain the existing number of workers
	FTaskGraphInterface& TaskGraph = FTaskGraphInterface::Get();
	const int32 NumWorkerThreads = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
	const int32 NumBackgroundWorkers = FMath::Max(1, TaskGraph.GetNumBackgroundThreads());
	const int32 NumForegroundWorkers = FMath::Max(1, FMath::Max(TaskGraph.GetNumForegroundThreads(), NumWorkerThreads - NumBackgroundWorkers));

	// Setting the stack size won't have an effect for the threads created before InitializeConfigSystem() is called
	int32 TaskGraphStackSize = 0;

	const IConsoleVariable* TaskGraphStackSizeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TaskGraph.ThreadStackSize"));
	
	if (TaskGraphStackSizeCVar)
	{
		TaskGraphStackSize = TaskGraphStackSizeCVar->GetInt();
	}

	LowLevelTasks::FScheduler::Get().RestartWorkers(NumForegroundWorkers, NumBackgroundWorkers, FForkProcessHelper::IsForkedMultithreadInstance() ? FThread::Forkable : FThread::NonForkable, TaskThreadConfig.Priority, TaskBPThreadConfig.Priority, TaskThreadConfig.Affinity, TaskBPThreadConfig.Affinity, TaskGraphStackSize);

	WaitEvent.Notify();
	ExitEvent.Wait();

	UE_LOGF(LogConsoleResponse, Display, "Task Priority %ls", *ThreadPriorityToString(TaskThreadConfig.Priority));
	UE_LOGF(LogConsoleResponse, Display, "Task Affinity %016llX", TaskThreadConfig.Affinity);

	UE_LOGF(LogConsoleResponse, Display, "TaskBP Priority %ls", *ThreadPriorityToString(TaskBPThreadConfig.Priority));
	UE_LOGF(LogConsoleResponse, Display, "TaskBP Affinity %016llX", TaskBPThreadConfig.Affinity);

	GLog->FlushThreadedLogs();
}

// SetThreadConfig console command to change thread configurations/affinities at runtime. Corresponds to ThreadConfigs array in /Script/Engine.Engine
// Example: SetThreadConfig Task:TPri_Normal:0x3f TaskBP:TPri_SlightlyBelowNormal:0x7f GT:TPri_Highest RT:TPri_Highest RHI:TPri_Highest
static FAutoConsoleCommand SetupThreadConfigCmd(
	TEXT("SetThreadConfig"),
	TEXT("Sets a thread Priority and/or Affinity. A single arg of default resets all the thread Priorities and Affinities, otherwise [GT|RT|RHI|Task|TaskBP] both/either [TPri_type] [0x] sets the Priority and/or Affinity."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&SetupThreadConfig)
);

void InitThreadConfig()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitThreadConfig);
	GetDefaultThreadConfigs();

	if (GConfig)
	{
		FString PlatformVariant = FPlatformProperties::PlatformVariantName();
		TArray<FString> ThreadConfigEntries;

		if (PlatformVariant.IsEmpty())
		{
			GConfig->GetArray(TEXT("/Script/Engine.Engine"), TEXT("ThreadConfigs"), ThreadConfigEntries, GEngineIni);
		}
		else
		{
			GConfig->GetArray(TEXT("/Script/Engine.Engine"), *FString::Printf(TEXT("ThreadConfigs_%s"), *PlatformVariant), ThreadConfigEntries, GEngineIni);
		}

		auto IsCVarModified =
			[](const TCHAR* CVarName) -> bool
			{
				if (IConsoleObject* ConsoleObject = IConsoleManager::Get().FindConsoleObject(CVarName))
				{
					return (ConsoleObject->GetFlags() & EConsoleVariableFlags::ECVF_SetByMask) != 0;
				}

				return false;
			};

		// These cvars are not read until after the initial thread setup, so we need to reapply if they are different
		bool bRequiresRestart =
			IsCVarModified(TEXT("TaskGraph.UseDynamicPrioritization")) ||
			IsCVarModified(TEXT("TaskGraph.UseDynamicThreadCreation")) ||
			IsCVarModified(TEXT("TaskGraph.OversubscriptionRatio")) ||
			IsCVarModified(TEXT("TaskGraph.ThreadStackSize"));

		if (bRequiresRestart || !ThreadConfigEntries.IsEmpty())
		{
			SetupThreadConfig(ThreadConfigEntries);
		}
	}
}

void UEngine::SetPriorityAndAffinityOnGameThread()
{
	::SetPriorityAndAffinityOnGameThread();
}

#if !UE_BUILD_SHIPPING

static void PakFileTest(const TArray<FString>& Args)
{
	FString PakFilename = "D:\\work\\Dev-Core\\Samples\\Games\\ShooterGame\\Saved\\StagedBuilds\\Windows\\ShooterGame\\Content\\Paks\\test.pak";
	if (Args.Num() < 1)
	{
		UE_LOGF(LogConsoleResponse, Error, "Usage: PakFileTest path-to-pak-file");
	}
	else
	{
		PakFilename = Args[0];
	}
	if (!PakFilename.IsEmpty())
	{
		const FString MountPoint = FPaths::ProjectSavedDir() / TEXT("PakFileTest");
		FFileManagerGeneric::Get().DeleteDirectory(*MountPoint, false, true);

		FString MountCmd = FString::Printf(TEXT("mount %s %s"), *PakFilename, *MountPoint);
		GEngine->Exec(nullptr, *MountCmd);

		static TArray<FString> FileNames;
		check(!FileNames.Num()); // don't run this twice!
		IFileManager::Get().FindFilesRecursive(FileNames, *MountPoint, TEXT("*.*"), true, false);
		check(FileNames.Num());


		static FString ReloadTestFile;
		static int64 ReloadTestSize = -1;
		static uint32 ReloadTestCRC = 0;
		static FCriticalSection ReloadLock;
		static FThreadSafeCounter Processed;


		static TFunction<void()> Broadcast =
			[
			]
		()
		{
			FRandomStream RNG(FPlatformTime::Cycles());
			{
				int32 NumProc = Processed.Increment();
				if (NumProc % 1000 == 1 || NumProc == 11 || NumProc == 101 || NumProc == 501)
				{
					UE_LOGF(LogTemp, Display, "Processed %d files (Thread  %x)", NumProc - 1, FPlatformTLS::GetCurrentThreadId());
				}

				bool bMyReload = false;
				FString TestFile;
				int64 TestSize = 0;
				uint32 TestCRC = 0;
				if (RNG.GetFraction() > .75f)
				{
					FScopeLock Lock(&ReloadLock);
					if (ReloadTestSize != -1)
					{
						TestSize = ReloadTestSize;
						TestCRC = ReloadTestCRC;
						TestFile = ReloadTestFile;

						ReloadTestFile.Empty();
						ReloadTestSize = -1;
						ReloadTestCRC = 0;

						bMyReload = true;
					}
				}
				if (TestFile.IsEmpty())
				{
					TestFile = FileNames[RNG.RandRange(0, FileNames.Num() - 1)];
				}

				IAsyncReadFileHandle* IORequestHandle = FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*TestFile);
				check(IORequestHandle);
				IAsyncReadRequest* SizeReq = IORequestHandle->SizeRequest();
				if (!SizeReq->PollCompletion()) // this should already be done with pak files
				{
					//check(0);
					UE_LOGF(LogTemp, Display, "Had to wait for size!!! =  %ls", *TestFile);
					SizeReq->WaitCompletion();
				}
				if (bMyReload)
				{
					check(TestSize == SizeReq->GetSizeResults());
				}
				TestSize = SizeReq->GetSizeResults();
				SizeReq->WaitCompletion();
				delete SizeReq;

				check(TestSize >= 0);

				uint32 NewCRC = 0;
				bool bAbortAfterCancel = RNG.GetFraction() > .95f;
				if (TestSize > 0)
				{
					uint8* Memory = (uint8*)FMemory::Malloc(TestSize);

					TArray<int64> SpanOffsets;
					TArray<int64> SpanSizes;

					int64 CurrentOffset = 0;
					while (CurrentOffset < TestSize)
					{
						SpanOffsets.Add(CurrentOffset);
						int64 Span = int64(RNG.RandRange(int32(FMath::Min<int64>(8192, TestSize - CurrentOffset)), TestSize - CurrentOffset));
						SpanSizes.Add(Span);
						CurrentOffset += Span;
						check(CurrentOffset <= TestSize);
					}
					TArray<IAsyncReadRequest*> PrecacheReqs;
					if (RNG.GetFraction() > .75f)
					{
						for (int32 i = 0; i < SpanOffsets.Num() / 5; i++)
						{
							int32 Index = RNG.RandRange(0, SpanOffsets.Num() - 1);
							CurrentOffset = SpanOffsets[Index];
							int64 Span = SpanSizes[Index];

							PrecacheReqs.Add(IORequestHandle->ReadRequest(CurrentOffset, Span, AIOP_FLAG_PRECACHE | AIOP_MIN));
						}
					}
					while (SpanOffsets.Num())
					{
						int32 Index = RNG.RandRange(0, SpanOffsets.Num() - 1);
						CurrentOffset = SpanOffsets[Index];
						int64 Span = SpanSizes[Index];

						bool CallbackCalled = false;

						FAsyncFileCallBack AsyncFileCallBack =
							[&CallbackCalled](bool bWasCancelled, IAsyncReadRequest* Req)
						{
							CallbackCalled = true;
						};

						bool bUserMem = !!RNG.RandRange(0, 1);
						EAsyncIOPriorityAndFlags Pri = (EAsyncIOPriorityAndFlags)RNG.RandRange((int32)AIOP_Low, (int32)AIOP_CriticalPath);
						IAsyncReadRequest* ReadReq = IORequestHandle->ReadRequest(CurrentOffset, Span, Pri, &AsyncFileCallBack, bUserMem ? Memory + CurrentOffset : nullptr);

						bool bCancel = RNG.RandRange(0, 5) == 0;

						if (bCancel)
						{
							float s = float(RNG.RandRange(0, 5)) / 1000.0f;
							if (s >= .001f)
							{
								FPlatformProcess::Sleep(s);
							}
							ReadReq->Cancel();
						}

						switch (RNG.RandRange(0, 4))
						{
						default:
							ReadReq->WaitCompletion();
							break;
						case 1:
							while (!ReadReq->PollCompletion())
							{
								FPlatformProcess::SleepNoStats(0.016f);
							}
							break;
						case 2:
							while (!ReadReq->PollCompletion())
							{
								FPlatformProcess::SleepNoStats(0);
							}
							break;
						case 3:
							while (!ReadReq->WaitCompletion(.016f))
							{
							}
							break;
						case 4:
							// can't wait for the callback after we have canceled
							if (bCancel)
							{
								ReadReq->WaitCompletion();
							}
							else
							{
								while (!CallbackCalled) //-V654
								{
									FPlatformProcess::SleepNoStats(0);
								}
							}
							break;
						}

						if (!bUserMem)
						{
							uint8 *Mem = ReadReq->GetReadResults();
							check(Mem || bCancel);
							if (Mem)
							{
								FMemory::Memcpy(Memory + CurrentOffset, Mem, Span);
								bCancel = false; // we should have the memory anyway
								FMemory::Free(Mem);
							}
						}
						ReadReq->WaitCompletion();
						delete ReadReq;
						if (!bCancel)
						{
							SpanOffsets.RemoveAtSwap(Index);
							SpanSizes.RemoveAtSwap(Index);
						}
						else if (bAbortAfterCancel)
						{
							break;
						}
					}

					if (!bAbortAfterCancel)
					{
						NewCRC = FCrc::MemCrc32(Memory, TestSize, 0x56);
					}

					FMemory::Free(Memory);

					for (IAsyncReadRequest* Req : PrecacheReqs)
					{
						Req->Cancel();
						Req->WaitCompletion();
						delete Req;
					}

				}

				if (!bAbortAfterCancel)
				{
					if (bMyReload)
					{
						//UE_LOGF(LogTemp, Display, "Reload pass CRC = %x    %ls", NewCRC, *TestFile);
						check(NewCRC == TestCRC);
					}
					else
					{
						//UE_LOGF(LogTemp, Display, "CRC = %x    %ls", NewCRC, *TestFile);
					}
					if (RNG.GetFraction() > .75f)
					{
						FScopeLock Lock(&ReloadLock);
						if (ReloadTestSize == -1)
						{
							ReloadTestSize = TestSize;
							ReloadTestCRC = NewCRC;
							ReloadTestFile = TestFile;
						}
					}
				}

				delete IORequestHandle;
			}
			if (!IsEngineExitRequested())
			{
				switch (RNG.RandRange(0, 2))
				{
				case 1:
					if (ENamedThreads::bHasBackgroundThreads)
					{
						FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyBackgroundThreadNormalTask);
						return;
					}
					break;
				case 2:
					if (ENamedThreads::bHasHighPriorityThreads)
					{
						FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyHiPriThreadNormalTask);
						return;
					}
					break;
				}

				FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyThread);
			}
		};

		int32 NumThreads = 1; // careful, it is easy to deadlock, one should not wait in task graph tasks!
		check(NumThreads > 0);
		for (int32 i = 0; i < NumThreads; i++)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyThread);
		}
		if (ENamedThreads::bHasBackgroundThreads)
		{
			for (int32 i = 0; i < NumThreads; i++)
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyBackgroundThreadNormalTask);
			}
		}
		if (ENamedThreads::bHasHighPriorityThreads)
		{
			for (int32 i = 0; i < NumThreads; i++)
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady(Broadcast, TStatId(), NULL, ENamedThreads::AnyHiPriThreadNormalTask);
			}
		}
	}
}

static FAutoConsoleCommand PakFileTestCmd(
	TEXT("PakFileTest"),
	TEXT("Tests the low level filesystem by mounting a pak file and doing multithreaded loads on it forever. Arg should be a full path to a pak file."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&PakFileTest)
);

#endif // !UE_BUILD_SHIPPING

// AI
int32 UEngine::RenderStatAI(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	// Pick a larger font on console.
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GetSmallFont() : GetMediumFont();

	// gather numbers
	int32 NumAI = 0;
	int32 NumAIRendered = 0;
	for (FConstControllerIterator Iterator = World->GetControllerIterator(); Iterator; ++Iterator)
	{
		AController* Controller = Iterator->Get();
		if (Controller && !Cast<APlayerController>(Controller))
		{
			++NumAI;
			if (Controller->GetPawn() && Controller->GetPawn()->WasRecentlyRendered(0.08f))
			{
				++NumAIRendered;
			}
		}
	}


#define MAXDUDES 20
#define BADAMTOFDUDES 12
	FColor TotalColor = FColor::Green;
	if (NumAI > BADAMTOFDUDES)
	{
		float Scalar = 1.0f - FMath::Clamp<float>((float)NumAI / (float)MAXDUDES, 0.f, 1.f);

		TotalColor = FColor::MakeRedToGreenColorFromScalar(Scalar);
	}

	FColor RenderedColor = FColor::Green;
	if (NumAIRendered > BADAMTOFDUDES)
	{
		float Scalar = 1.0f - FMath::Clamp<float>((float)NumAIRendered / (float)MAXDUDES, 0.f, 1.f);

		RenderedColor = FColor::MakeRedToGreenColorFromScalar(Scalar);

	}

	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);
	Canvas->DrawShadowedString(
		X,
		Y,
		*FString::Printf(TEXT("%i AI"), NumAI),
		Font,
		TotalColor
	);
	Y += RowHeight;

	Canvas->DrawShadowedString(
		X,
		Y,
		*FString::Printf(TEXT("%i AI Rendered"), NumAIRendered),
		Font,
		RenderedColor
	);
	Y += RowHeight;
	return Y;
}

// Timecode
int32 UEngine::RenderStatTimecode(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GetSmallFont() : GetMediumFont();
	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);

	const UTimecodeProvider* Provider = GetTimecodeProvider();
	if (Provider)
	{
	ETimecodeProviderSynchronizationState State = Provider->GetSynchronizationState();
	FString ProviderName = Provider->GetName();
	float CharWidth, CharHeight;
	Font->GetCharSize(TEXT(' '), CharWidth, CharHeight);
	int32 NewX = X - Font->GetStringSize(*ProviderName) - (int32)CharWidth;
	switch (State)
	{
	case ETimecodeProviderSynchronizationState::Closed:
		Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s TC: Closed"), *ProviderName), Font, FColor::Red);
		break;
	case ETimecodeProviderSynchronizationState::Error:
		Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s TC: Error"), *ProviderName), Font, FColor::Red);
		break;
	case ETimecodeProviderSynchronizationState::Synchronized:
		Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s TC: Synchronized"), *ProviderName), Font, FColor::Green);
		break;
	case ETimecodeProviderSynchronizationState::Synchronizing:
		Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s TC: Synchronizing"), *ProviderName), Font, FColor::Yellow);
		break;
	default:
		check(false);
		break;
	}
	}
	Y += RowHeight;
	Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("TC: %s"), *FApp::GetTimecode().ToString()), Font, FColor::Green);
	Y += RowHeight;

	return Y;
}

// FRAMECOUNTER
int32 UEngine::RenderStatFrameCounter(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GetSmallFont() : GetMediumFont();
	const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight() * 1.1f);

	Canvas->DrawShadowedString(X, Y, *FString::Printf(TEXT("FC: %" UINT64_FMT), GFrameCounter), Font, FColor::Green);
	Y += RowHeight;

	return Y;
}

// SLATEBATCHES
#if STATS
int32 UEngine::RenderStatSlateBatches(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
	/* @todo Slate Rendering
	UFont* Font = SmallFont;

	const TArray<FBatchStats>& Stats = FSlateApplication::Get().GetRenderer()->GetBatchStats();

	// Start drawing the various counters.
	const int32 RowHeight = FMath::Trunc( Font->GetMaxCharHeight() * 1.1f );

	X = Viewport->GetSizeXY().X - 350;

	Canvas->DrawShadowedString(
	X,
	Y,
	TEXT("Slate Batches:"),
	Font, 
	FColor::Green );

	Y+=RowHeight;


	for( int32 I = 0; I < Stats.Num(); ++I )
	{
	const FBatchStats& Stat = Stats(I);

	// Draw a box representing the debug color of the batch
	DrawTriangle2D(Canvas, FVector2D(X,Y), FVector2D(0,0), FVector2D(X+10,Y), FVector2D(0,0), FVector2D(X+10,Y+7), FVector2D(0,0), Stat.BatchColor );
	DrawTriangle2D(Canvas, FVector2D(X,Y), FVector2D(0,0), FVector2D(X,Y+7), FVector2D(0,0), FVector2D(X+10,Y+7), FVector2D(0,0), Stat.BatchColor );

	Canvas->DrawShadowedString(
	X+15,
	Y,
	*FString::Printf(TEXT("Layer: %d, Elements: %d, Vertices: %d"), Stat.Layer, Stat.NumElementsInBatch, Stat.NumVertices ), 
	Font,
	FColor::Green );
	Y += RowHeight;
	}*/
	return Y;
}
#endif

#if !UE_BUILD_SHIPPING
bool UEngine::ToggleStatParticlePerf(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream)
{
#if WITH_PARTICLE_PERF_STATS
	FParticlePerfStatsManager::TogglePerfStatsRender(World);
#endif
	return false;
}

int32 UEngine::RenderStatParticlePerf(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
{
#if WITH_PARTICLE_PERF_STATS
	Y = FParticlePerfStatsManager::RenderStats(World, Viewport, Canvas, X, Y, ViewLocation, ViewRotation);
#endif
	return Y;
}
#endif

ERHIFeatureLevel::Type UEngine::GetDefaultWorldFeatureLevel() const
{
	return GMaxRHIFeatureLevel;
}

#if WITH_EDITOR
void UEngine::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!HasAnyFlags(RF_ArchetypeObject|RF_ClassDefaultObject))
	{
		FName PropertyName = PropertyChangedEvent.GetPropertyName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, TimecodeProviderClassName)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, bGenerateDefaultTimecode)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, GenerateDefaultTimecodeFrameRate)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, GenerateDefaultTimecodeFrameDelay))
		{
			// Only reload the timecode provider if the current was created by the engine
			UTimecodeProvider* CurrentTimecodeProvider = GetTimecodeProvider();
			if (CurrentTimecodeProvider == nullptr || CurrentTimecodeProvider->GetOuter() == this)
			{
				LoadTimecodeProvider(this);
			}
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, CustomTimeStepClassName))
		{
			// Only reload the custom time step if the current was created by the engine
			UEngineCustomTimeStep* CurrentCustomTimeStep = GetCustomTimeStep();
			if (CurrentCustomTimeStep == nullptr || CurrentCustomTimeStep->GetOuter() == this)
			{
				LoadCustomTimeStep(this);
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UEngine::GetPreviewPlatformName(FName& PlatformName) const
{
	// Support returning "Mobile" when started with -featureleveles31 etc

	static bool bHasCachedResult = false;
	static bool bCachedMobile = false;

	if (!bHasCachedResult)
	{
		bHasCachedResult = true;
		ERHIFeatureLevel::Type FeatureLevel;

		bCachedMobile = (RHIGetPreviewFeatureLevel(FeatureLevel) && FeatureLevel <= ERHIFeatureLevel::ES3_1);
	}

	if (bCachedMobile)
	{
		// the editor uses a real platform - this is unlikely tyo actually be called, but the code does technically handle using a group name as a platform name
		PlatformName = NAME_Mobile;
		return true;
	}
	else
	{
		return false;
	}
}

#endif

#undef LOCTEXT_NAMESPACE
