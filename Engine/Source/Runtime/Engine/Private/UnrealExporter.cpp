// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UExporter.cpp: Exporter class implementation.
=============================================================================*/

// Engine includes.
#include "UnrealExporter.h"
#include "UObject/UnrealType.h"
#include "Exporters/Exporter.h"
#include "Misc/AsciiSet.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/BufferArchive.h"
#include "UObject/UObjectIterator.h"
#include "Model.h"
#include "AssetExportTask.h"
#include "DataTableUtils.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/OverridableManager.h"
#include "UObject/OverriddenPropertySet.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Selection.h"
#else
#include "UObject/Package.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogExporter, Log, All);

namespace UnrealExporterPrivate
{
	FOverriddenPropertySet* GetOverriddenProperties(TNotNull<UObject*> Object)
	{
		if (FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization))
		{
			return FOverridableManager::Get().GetOverriddenProperties(Object);
		}
		return nullptr;
	}
}

FString UExporter::CurrentFilename(TEXT(""));

TSet< TWeakObjectPtr<UExporter> > UExporter::RegisteredExporters;

UExporter::UExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if ( HasAnyFlags(RF_ClassDefaultObject) )
	{
		TWeakObjectPtr<UExporter> DefaultExporterObj(this);
		TWeakObjectPtr<UExporter>* PreviousObj = RegisteredExporters.Find(DefaultExporterObj);
		if(!PreviousObj)
		{
			RegisteredExporters.Add(DefaultExporterObj);
		}
		else if(!PreviousObj->IsValid())
		{
			RegisteredExporters.Remove(*PreviousObj);
			RegisteredExporters.Add(DefaultExporterObj);
		}
	}
	BatchExportMode = false;
	CancelBatch = false;
	ShowExportOption = true;
}

void UExporter::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << SupportedClass << FormatExtension << FormatDescription;
	Ar << PreferredFormatIndex;
}

// Returns whether this exporter supports the specific object
bool UExporter::SupportsObject(UObject* Object) const
{
	return (SupportedClass && Object->IsA(SupportedClass));
}

bool UExporter::GetBatchMode() const
{
	return BatchExportMode;
}

void UExporter::SetBatchMode(bool InBatchExportMode)
{
	BatchExportMode = InBatchExportMode;
}

bool UExporter::GetCancelBatch() const
{
	return CancelBatch;
}

void UExporter::SetCancelBatch(bool InCancelBatch)
{
	CancelBatch = InCancelBatch;
}

bool UExporter::GetShowExportOption() const
{
	return ShowExportOption;
}

void UExporter::SetShowExportOption(bool InShowExportOption)
{
	ShowExportOption = InShowExportOption;
}

UExporter* UExporter::FindExporter( UObject* Object, const TCHAR* FileType )
{
	check(Object);

	if (Object->GetOutermost()->HasAnyPackageFlags(PKG_DisallowExport))
	{
		return NULL;
	}

	TMap<UClass*,UClass*> Exporters;

	for (TSet< TWeakObjectPtr<UExporter> >::TIterator It(RegisteredExporters); It; ++It)
	{
		UExporter* Default = It->Get();
		if(Default && !Default->GetClass()->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			check( Default->FormatExtension.Num() == Default->FormatDescription.Num() );
			if (Default->SupportsObject(Object))
			{
				for( int32 i=0; i<Default->FormatExtension.Num(); i++ )
				{
					const bool bIsFileType = (FCString::Stricmp( *Default->FormatExtension[i], FileType  ) == 0);
					const bool bIsWildCardType = ( FCString::Stricmp( *Default->FormatExtension[i], TEXT("*") )== 0 );
					if(	bIsFileType==true || bIsWildCardType==true )
					{
						Exporters.Add( Default->SupportedClass, Default->GetClass() );
					}
				}
			}
		}

	}

	UClass** E;
	auto TransientPackage = GetTransientPackage();
	for (UClass* TempClass = Object->GetClass(); TempClass != NULL; TempClass = TempClass->GetSuperClass())
	{
		const bool bFoundExporter = ((E = Exporters.Find( TempClass )) != NULL);

		if( bFoundExporter )
		{
			return NewObject<UExporter>(TransientPackage, *E);
		}
	}
		
	return NULL;
}


bool UExporter::ExportToArchive( UObject* Object, UExporter* InExporter, FArchive& Ar, const TCHAR* FileType, int32 FileIndex )
{
	check(Object);
	UExporter* Exporter = InExporter;
	if( !Exporter )
	{
		Exporter = FindExporter( Object, FileType );
	}
	if( !Exporter )
	{
		UE_LOGF(LogExporter, Warning, "No %ls exporter found for %ls", FileType, *Object->GetFullName() );
		return( false );
	}
	check( Object->IsA( Exporter->SupportedClass ) );
	return( Exporter->ExportBinary( Object, FileType, Ar, GWarn, FileIndex ) );
}


bool UExporter::ExportToOutputDevice(const FExportObjectInnerContext* Context, UObject* Object, UExporter* InExporter, FOutputDevice& Out, const TCHAR* FileType, int32 Indent, uint32 PortFlags, bool bInSelectedOnly, UObject* ExportRootScope)
{
	check(Object);
	UExporter* Exporter = InExporter;
	if( !Exporter )
	{
		Exporter = FindExporter( Object, FileType );
	}
	if( !Exporter )
	{
		UE_LOGF(LogExporter, Warning, "No %ls exporter found for %ls", FileType, *Object->GetFullName() );
		return false;
	}
	check(Object->IsA(Exporter->SupportedClass));
	int32 SavedIndent = Exporter->TextIndent;
	Exporter->TextIndent = Indent;
	Exporter->bSelectedOnly = bInSelectedOnly;
	Exporter->ExportRootScope = ExportRootScope;

	// this tells the lower-level code that determines whether property values are identical that
	// it should recursively compare subobjects properties as well
	if ( (PortFlags&PPF_SubobjectsOnly) == 0 )
	{
		PortFlags |= PPF_DeepComparison;

		// always export references to instanced subobjects that are constructed through non-default means, regardless of equality
		// this is needed to properly rebuild the outer object's instance graph during import after initializing from CDO/archetype
		// notes:
		// - a DSO reference can be uniquely resolved from its initialized value, since constructed DSOs are mapped by the initializer
		// - a DSO with an outer that's initialized from a native CDO will not need to fix up any references to it post-initialization
		// - a DSO overridden at edit time is considered unique when a deep comparison differs, and so any references must be exported
		// - references to non-default subobjects must also be exported as they can't be resolved through the initializer's DSO mapping
		PortFlags |= PPF_DeepCompareDSOsOnly;
	}

	if ( FCString::Stricmp(FileType, TEXT("COPY")) == 0 )
	{
		// some code which doesn't have access to the exporter's file type needs to handle copy/paste differently than exporting to file,
		// so set the export flag accordingly
		PortFlags |= PPF_Copy;
	}

	const bool bSuccess = Exporter->ExportText( Context, Object, FileType, Out, GWarn, PortFlags );
	Exporter->TextIndent = SavedIndent;

	return bSuccess;
}


int32 UExporter::ExportToFile( UObject* Object, UExporter* InExporter, const TCHAR* Filename, bool InSelectedOnly, bool NoReplaceIdentical, bool Prompt )
{
	UAssetExportTask* ExportTask = NewObject<UAssetExportTask>();
	FGCObjectScopeGuard ExportTaskGuard(ExportTask);
	ExportTask->Object = Object;
	ExportTask->Exporter = InExporter;
	ExportTask->Filename = Filename;
	ExportTask->bSelected = InSelectedOnly;
	ExportTask->bReplaceIdentical = !NoReplaceIdentical;
	ExportTask->bPrompt = Prompt;
	ExportTask->bUseFileArchive = false;
	ExportTask->bWriteEmptyFiles = false;
	ExportTask->bAutomated = false;
	return RunAssetExportTask(ExportTask) ? 1 : 0;
}


bool UExporter::RunAssetExportTask(class UAssetExportTask* Task)
{
	check(Task);

	CurrentFilename = Task->Filename;
	struct FScopedCleanup
	{
		~FScopedCleanup() { UExporter::CurrentFilename = TEXT(""); }
	};
	FScopedCleanup ScopedCleanup;

	UExporter*	Exporter	= Task->Exporter;
	FString		Extension	= FPaths::GetExtension(Task->Filename);

	// We were provided with an exporter, check to see if its compatible with the asset we want to export
	if (Exporter)
	{
		if (UObject* Object = Task->Object.Get())
		{
			if (!Object->IsA(Exporter->SupportedClass))
			{
				Task->Errors.Add(FString::Printf(TEXT("Chosen exporter '%s' does not support the exported object's class '%s'!"), *Exporter->GetName(), *Object->GetClass()->GetName()));
				UE_LOGF(LogExporter, Warning, "%ls", *Task->Errors.Last());
				return false;
			}
		}
	}
	else
	{
		// look for an exporter with all possible extensions, so an exporter can have something like *.xxx.yyy as an extension
		int32 SearchStart = 0;
		int32 DotLocation;
		while (!Exporter && (DotLocation = CurrentFilename.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart, SearchStart)) != INDEX_NONE)
		{
			// get everything after the current .
			Extension = CurrentFilename.Mid(DotLocation + 1);

			// try to find an exporter with it
			Exporter = FindExporter(Task->Object, *Extension );

			// skip past the dot in case we look again
			SearchStart = DotLocation + 1;
		}
	}

	if( !Exporter )
	{
		Task->Errors.Add(FString::Printf(TEXT("No %s exporter found for %s"), *Extension, *(Task->Object->GetFullName())));
		UE_LOGF(LogExporter, Warning, "%ls", *Task->Errors.Last());
		return false;
	}

	Exporter->ExportTask = Task;
	Exporter->bSelectedOnly = Task->bSelected;

	if (Exporter->ScriptRunAssetExportTask(Task))
	{
		return true;
	}
	else if( Exporter->bText )
	{
		TSharedPtr<FOutputDevice> TextBuffer;
		bool bIsFileDevice = false;
		FString TempFile = FPaths::GetPath(Task->Filename);
		if (Exporter->bForceFileOperations || Task->bUseFileArchive)
		{
			IFileManager::Get().MakeDirectory(*TempFile);

			TempFile += TEXT("/UnrealExportFile.tmp");
			TextBuffer = MakeShareable(new FOutputDeviceFile(*TempFile));
			if (TextBuffer.IsValid())
			{
				TextBuffer->SetSuppressEventTag(true);
				TextBuffer->SetAutoEmitLineTerminator(false);
				bIsFileDevice = true;
			}
		}

		if (!TextBuffer.IsValid())
		{
			if (Task->bUseFileArchive)
			{
				UE_LOGF(LogExporter, Warning, "Failed to create file output device... defaulting to string buffer");
			}
			TextBuffer = MakeShareable(new FStringOutputDevice());
		}
		const FExportObjectInnerContext Context(Task->IgnoreObjectList);
		ExportToOutputDevice(&Context, Task->Object, Exporter, *TextBuffer, *Extension, 0, PPF_ExportsNotFullyQualified, Task->bSelected);
		if (bIsFileDevice)
		{
			TextBuffer->TearDown();
			IFileManager::Get().Move(*Task->Filename, *TempFile, 1, 1);
		}
		else
		{
			FStringOutputDevice& StringBuffer = *((FStringOutputDevice*)TextBuffer.Get());
			if ( StringBuffer.Len() == 0 )
			{
				// non-fatal
				return true;
			}
			else
			{
				if ( !Task->bReplaceIdentical )
				{
					FString FileBytes;
					if
						(	FFileHelper::LoadFileToString(FileBytes, *Task->Filename)
						&&	FCString::Strcmp(*StringBuffer,*FileBytes)==0 )
					{
						UE_LOGF(LogExporter, Log, "Not replacing %ls because identical", *Task->Filename);
						return true;
					}
					if( Task->bPrompt )
					{
						if( !GWarn->YesNof(FText::Format(NSLOCTEXT("Core", "Overwrite", "The file '{0}' needs to be updated.  Do you want to overwrite the existing version?"), FText::FromString(  Task->Filename ) ) ) )
						{
							return true;
						}
					}
				}
				if(!FFileHelper::SaveStringToFile( StringBuffer, *Task->Filename ) )
				{
#if 0
					if(GWarn->YesNof(FText::Format(NSLOCTEXT("Core", "OverwriteReadOnly", "'{0}' is marked read-only.  Would you like to try to force overwriting it?"), FText::FromString(Task->Filename))))
					{
						IFileManager::Get().Delete( Task->Filename, 0, 1 );
						if(FFileHelper::SaveStringToFile( StringBuffer, *Task->Filename ) )
						{
							return true;
						}
					}
#endif
					UE_LOGF(LogExporter, Error, "%ls", *FString::Printf(TEXT("Error exporting %s: couldn't open file '%s'"), *(Task->Object->GetFullName()), *Task->Filename));
					return false;
				}
				return true;
			}
		}
	}
	else
	{
		const int32 FileCount = Exporter->GetFileCount(Task->Object);
		for( int32 i = 0; i < FileCount; i++ )
		{
			FBufferArchive64 Buffer;
			if(ExportToArchive(Task->Object, Exporter, Buffer, *Extension, i))
			{
				FString UniqueFilename = Exporter->GetUniqueFilename(Task->Object, *Task->Filename, i, FileCount);

				if(!Task->bReplaceIdentical)
				{
					TArray<uint8> FileBytes;

					if(	FFileHelper::LoadFileToArray( FileBytes, *UniqueFilename )
					&&	FileBytes.Num() == Buffer.Num()
					&&	FMemory::Memcmp( &FileBytes[ 0 ], &Buffer[ 0 ], Buffer.Num() ) == 0 )
					{
						UE_LOGF(LogExporter, Log,  "Not replacing %ls because identical", *UniqueFilename );
						return true;
					}
					if(Task->bPrompt)
					{
						if( !GWarn->YesNof( FText::Format( NSLOCTEXT("Core", "Overwrite", "The file '{0}' needs to be updated.  Do you want to overwrite the existing version?"), FText::FromString( UniqueFilename ) ) ) )
						{
							return true;
						}
					}
				}

				if (!Task->bWriteEmptyFiles && !Buffer.Num())
				{
					return true;
				}

				if( !FFileHelper::SaveArrayToFile( Buffer, *UniqueFilename ) )
				{
					Task->Errors.Add(FString::Printf(TEXT("Error exporting %s: couldn't open file '%s'"), *(Task->Object->GetFullName()), *UniqueFilename));
					UE_LOGF(LogExporter, Error, "%ls", *Task->Errors.Last());
					return false;
				}
			}
		}
		return true;
	}
	return false;
}

bool UExporter::RunAssetExportTasks(const TArray<UAssetExportTask*>& ExportTasks)
{
	bool bSuccess = true;
	for (UAssetExportTask* Task : ExportTasks)
	{
		if (!RunAssetExportTask(Task))
		{
			bSuccess = false;
		}
	}
	return bSuccess;
}

const bool UExporter::bEnableDebugBrackets = false;

void UExporter::EmitBeginObject( FOutputDevice& Ar, UObject* Obj, uint32 PortFlags )
{
	check(Obj);

	// figure out how to export
	bool bIsExportingDefaultObject = Obj->HasAnyFlags(RF_ClassDefaultObject) || Obj->GetArchetype()->HasAnyFlags(RF_ClassDefaultObject);

	// start outputting the string for the Begin Object line
	Ar.Logf(TEXT("%sBegin Object"), FCString::Spc(TextIndent));

	if (!(PortFlags & PPF_SeparateDefine))
	{
		Ar.Logf(TEXT(" Class=%s"), *Obj->GetClass()->GetPathName());
	}

	// always need a name, adding "" for space handling
	Ar.Logf(TEXT(" Name=\"%s\""), *Obj->GetName());

	if (!(PortFlags & PPF_SeparateDefine))
	{
		// do we want the archetype string?
		if (!bIsExportingDefaultObject)
		{
			UObject* Archetype = Obj->GetArchetype();
			// since we could have two object owners with the same name (like named Blueprints in different folders),
			// we need the fully qualified path for the archetype (so we don't get confused when unpacking this)
			Ar.Logf(TEXT(" Archetype=%s"), *FObjectPropertyBase::GetExportPath(Archetype, nullptr, /*ExportRootScope =*/nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified));
		}

		if (FOverriddenPropertySet* ObjectOverriddenProperties = UnrealExporterPrivate::GetOverriddenProperties(Obj))
		{
			const EOverriddenPropertyOperation Operation = ObjectOverriddenProperties->GetOverriddenPropertyOperation((FArchiveSerializedPropertyChain*)nullptr, (FProperty*)nullptr);
			Ar.Logf(TEXT(" OverriddenOperation=%s"), *GetOverriddenOperationString(Operation));
		}
	}

	// When exporting for diffs, export paths can cause false positives. since diff files don't get imported, we can
	// skip adding this info the file.
	if (!(PortFlags & PPF_ForDiff))
	{
		// Emit the object path
		Ar.Logf(TEXT(" ExportPath=%s"), *FObjectPropertyBase::GetExportPath(Obj, nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified));
	}
	// end in a return
	Ar.Logf(LINE_TERMINATOR);

	if ( bEnableDebugBrackets )
	{
		Ar.Logf(TEXT("%s{%s"), FCString::Spc(TextIndent), LINE_TERMINATOR);
	}
}


void UExporter::EmitEndObject( FOutputDevice& Ar )
{
	if ( bEnableDebugBrackets )
	{
		Ar.Logf(TEXT("%s}%s"), FCString::Spc(TextIndent), LINE_TERMINATOR);
	}
	Ar.Logf( TEXT("%sEnd Object\r\n"), FCString::Spc(TextIndent) );
}

FExportObjectInnerContext::FExportObjectInnerContext()
{
	// For each object . . .
	for (UObject* InnerObj : TObjectRange<UObject>(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage))
	{
		AddObjectToInnerMap(InnerObj);
	}
}


FExportObjectInnerContext::FExportObjectInnerContext(const TArray<UObject*>& ObjsToIgnore)
{
	// For each object . . .
	for (UObject* InnerObj : TObjectRange<UObject>(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage))
	{
		if (!ObjsToIgnore.Contains(InnerObj))
		{
			AddObjectToInnerMap(InnerObj);
		}
	}
}

void FExportObjectInnerContext::AddObjectToInnerMap(UObject* InObject)
{
	UObject* OuterObj = InObject->GetOuter();
	if (IsValid(OuterObj))
	{
		InnerList* Inners = ObjectToInnerMap.Find(OuterObj);
		if (Inners)
		{
			// Add object to existing inner list.
			Inners->Add(InObject);
		}
		else
		{
			// Create a new inner list for the outer object.
			InnerList& InnersForOuterObject = ObjectToInnerMap.Add(OuterObj, InnerList());
			InnersForOuterObject.Add(InObject);
		}
	}
}

bool FExportObjectInnerContext::IsObjectSelected(const UObject* InObj) const
{
	return InObj->IsSelected();
}


bool UExporter::IsObjectSelectedForExport(const FExportObjectInnerContext* Context, const UObject* Object)
{
	return Context
		? Context->IsObjectSelected(Object)
		: Object->IsSelected();
}


void UExporter::ExportObjectInner(const FExportObjectInnerContext* Context, UObject* Object, FOutputDevice& Ar, uint32 PortFlags)
{
	// indent all the text in here
	TextIndent += 3;

	FExportObjectInnerContext::InnerList TempInners;
	const FExportObjectInnerContext::InnerList* ContextInners = NULL;
	if (Context)
	{
		ContextInners = Context->ObjectToInnerMap.Find(Object);
	}
	else
	{
		// NOTE: We ignore inner objects that have been tagged for death
		GetObjectsWithOuter(Object, TempInners, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
	}
	FExportObjectInnerContext::InnerList const& UnsortedObjectInners = ContextInners ? *ContextInners : TempInners;

	FExportObjectInnerContext::InnerList SortedObjectInners;
	if (PortFlags & PPF_DebugDump)
	{
		SortedObjectInners = UnsortedObjectInners;
		// optionally sort inners, which can be useful when comparing/diffing debug dumps
		SortedObjectInners.Sort([](const UObject& A, const UObject& B) -> bool
		{
			return A.GetName() < B.GetName();
		});
	}

	FExportObjectInnerContext::InnerList const& ObjectInners = (PortFlags & PPF_DebugDump) ? SortedObjectInners : UnsortedObjectInners;

	if (!(PortFlags & PPF_SeparateDefine))
	{
		for (UObject* Obj : ObjectInners)
		{
			if (!Obj->HasAnyFlags(RF_TextExportTransient))
			{
				// export the object
				UExporter::ExportToOutputDevice( Context, Obj, NULL, Ar, (PortFlags & PPF_Copy) ? TEXT("Copy") : TEXT("T3D"), TextIndent, PortFlags | PPF_SeparateDeclare, false, ExportRootScope );
			}
		}
	}

	if (!(PortFlags & PPF_SeparateDeclare))
	{
		for (UObject* Obj : ObjectInners)
		{
			if (!Obj->HasAnyFlags(RF_TextExportTransient) && Obj->GetClass() != UModel::StaticClass())
			{
				// export the object
				UExporter::ExportToOutputDevice( Context, Obj, NULL, Ar, (PortFlags & PPF_Copy) ? TEXT("Copy") : TEXT("T3D"), TextIndent, PortFlags | PPF_SeparateDefine, false, ExportRootScope );

				// don't reexport below in ExportProperties
				Obj->Mark(OBJECTMARK_TagImp);
			}
		}

		// export the object's properties
		// Note: we use archetype as the object to diff properties against before they exported. When object is created, they should create from archetype
		// and using this system, it should recover all properties it needs to copy
		uint8 *CompareObject;
		if (Object->HasAnyFlags(RF_ClassDefaultObject))
		{
			CompareObject = (uint8*)Object;
		}
		else
		{
			CompareObject = (uint8*)Object->GetArchetype();
		}
		ExportProperties( Context, Ar, Object->GetClass(), (uint8*)Object, TextIndent, Object->GetClass(), CompareObject, Object, PortFlags, ExportRootScope );

		if (AActor* Actor = Cast<AActor>(Object))
		{
			// Todo PlacementMode consider removing that code when we it will replace the foliage
			// Export anything extra for the components. Used for instanced foliage.
			// This is done after the actor properties so these are set when regenerating the extra data objects.
			TArray<UActorComponent*> Components;
			Actor->GetComponents(Components);
			ExportComponentExtra(Context, Components, Ar, PortFlags);
		}
	}

	// remove indent
	TextIndent -= 3;
}

/** Shim ExportProperties when FOverridableSerializationLogic::ECapabilities::T3DSerialization capabilities enabled */
void ExportProperties_Overrides
(
	const FExportObjectInnerContext* Context,
	FOutputDevice&	Out,
	UStruct*		Struct,
	uint8*			ValuePtr,
	int32			Indent,
	UStruct*		DiffStruct,
	uint8*			Diff,
	UObject*		Parent,
	uint32			PortFlags,
	UObject*		ExportRootScope,
	FOverriddenPropertySet* OverriddenProperties,
	const FString&	PropertyPrefix = FString()
)
{
	check(Struct != NULL);

	for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->ShouldPort(PortFlags))
		{
			continue;
		}

		FString SanitizedPropertyName = Property->GetName();
		constexpr FAsciiSet Whitespace("\t ");
		constexpr FAsciiSet SpecialCharacters("=()[].\"\'");
		if (FAsciiSet::HasAny(SanitizedPropertyName, SpecialCharacters) || Whitespace.Contains(SanitizedPropertyName[0]))
		{
			// to increase frequency of forward compatibility, only sanitize property names that absolutely need it
			SanitizedPropertyName = FString::Format(TEXT("\"{0}\""), {Property->GetName().ReplaceCharWithEscapedChar()});
		}
		if (!PropertyPrefix.IsEmpty())
		{
			SanitizedPropertyName = PropertyPrefix + "." + SanitizedPropertyName;
		}

		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FMapProperty* MapProperty = CastField<FMapProperty>(Property);
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);

		FObjectPropertyBase *const ExportObjectProp = (Property->PropertyFlags & CPF_ExportObject) != 0 ? CastField<FObjectPropertyBase>(Property) : NULL;
		const uint32 ExportFlags = PortFlags | PPF_Delimited;
		auto HandleExportObject = [Context, &Out, Indent, Parent, PortFlags](const FProperty* Property, uint8* Data, FObjectPropertyBase*const InExportObjectProp)
		{
			if (InExportObjectProp)
			{
				UObject* Obj = InExportObjectProp->GetObjectPropertyValue(Data);
				check(!Obj || Obj->IsValidLowLevel());
				if (Obj && !Obj->HasAnyMarks(OBJECTMARK_TagImp))
				{
					// only export the BEGIN OBJECT block for a component if Parent is the component's Outer....when importing subobject definitions,
					// (i.e. BEGIN OBJECT), whichever BEGIN OBJECT block a component's BEGIN OBJECT block is located within is the object that will be
					// used as the Outer to create the component

					// Is this an array of components?
					if (Property->HasAnyPropertyFlags(CPF_InstancedReference))
					{
						if (Obj->GetOuter() == Parent)
						{
							// Don't export more than once.
							Obj->Mark(OBJECTMARK_TagImp);
							UExporter::ExportToOutputDevice(Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags);
						}
						else
						{
							// set the OBJECTMARK_TagExp flag so that the calling code knows we wanted to export this object
							Obj->Mark(OBJECTMARK_TagExp);
						}
					}
					else
					{
						// Don't export more than once.
						Obj->Mark(OBJECTMARK_TagImp);
						UExporter::ExportToOutputDevice(Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags);
					}
				}
			}
		};

		for (int32 PropertyArrayIndex=0; PropertyArrayIndex<Property->ArrayDim; PropertyArrayIndex++)
		{
			// Array special case
			if (ArrayProperty != nullptr)
			{
				// Export dynamic array.
				FProperty* InnerProp = ArrayProperty->Inner;
				FObjectPropertyBase*const ArrayExportObjectProp = (Property->PropertyFlags & CPF_ExportObject) != 0 ? CastField<FObjectPropertyBase>(InnerProp) : NULL;
				// This is used as the default value in the case of an array property that has
				// fewer elements than the exported object.
				uint8* StructDefaults = nullptr;
				FStructProperty* InnerStructProperty = CastField<FStructProperty>(InnerProp);
				if (InnerStructProperty != nullptr)
				{
					checkSlow(InnerStructProperty->Struct);
					StructDefaults = (uint8*)FMemory::Malloc(InnerStructProperty->Struct->GetStructureSize());
					InnerStructProperty->InitializeValue(StructDefaults);
				}
				ON_SCOPE_EXIT
				{
					if (StructDefaults)
					{
						InnerStructProperty->DestroyValue(StructDefaults);
						FMemory::Free(StructDefaults);
					}
				};

				void* Arr = Property->ContainerPtrToValuePtr<void>(ValuePtr, PropertyArrayIndex);
				FScriptArrayHelper ArrayHelper(ArrayProperty, Arr);

				void* DiffArr = nullptr;
				if (DiffStruct)
				{
					DiffArr = Property->ContainerPtrToValuePtrForDefaults<void>(DiffStruct, Diff, PropertyArrayIndex);
				}
				// we won't use this if DiffArr is NULL, but we have to set it up to something
				FScriptArrayHelper DiffArrayHelper(ArrayProperty, DiffArr);

				EOverriddenPropertyOperation Operation = EOverriddenPropertyOperation::None;
				FArchiveSerializedPropertyChain Chain;
				if (OverriddenProperties)
				{
					FOverridableTextPortPropertyPathScope ScopePath(Property);
					FPropertyVisitorPath* Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
					checkf(Path, TEXT("Expecting a path"));
					Chain = Path->ToSerializedPropertyChain();
					Operation = FOverridableSerializationLogic::GetOverriddenPropertyOperation(PortFlags, &Chain, nullptr, Arr, DiffArr);
				}

				if(!OverriddenProperties && !Property->HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic))
				{
					// If the current size of the array is 0 and the default one is not, add in an empty item so on import it will be empty
					if (ArrayHelper.Num() == 0 && DiffArrayHelper.Num() != 0)
					{
						Out.Logf(TEXT("%s%s=\r\n"), FCString::Spc(Indent), *SanitizedPropertyName);
						continue;
					}

					// If the array sizes are different, we will need to export each index so on import we maintain the size
					for (int32 DynamicArrayIndex = 0; DynamicArrayIndex < ArrayHelper.Num(); DynamicArrayIndex++)
					{
						FOverridableTextPortPropertyPathScope ScopePath(Property, DynamicArrayIndex, EPropertyVisitorInfoType::ContainerIndex);

						FString	Value;

						// compare each element's value manually so that elements which match the NULL value for the array's inner property type
						// but aren't in the diff array are still exported
						uint8* SourceData = ArrayHelper.GetRawPtr(DynamicArrayIndex);
						bool bHasDiffData = DiffArr && DynamicArrayIndex < DiffArrayHelper.Num();
						uint8* DiffData = bHasDiffData ? DiffArrayHelper.GetRawPtr(DynamicArrayIndex) : StructDefaults;

						// Make sure to export the last element even if it is default value if the default data doesn't have an element at that index (!bHasDiffData && (DynamicArrayIndex == ArrayHelper.Num()-1) && !bHasDiffData) 
						// because that will ensure the resulting imported array will be of proper size
						bool bExportItem = DiffData == NULL || (!bHasDiffData && (DynamicArrayIndex == ArrayHelper.Num()-1)) || (DiffData != SourceData && !InnerProp->Identical(SourceData, DiffData, ExportFlags));
						if (bExportItem)
						{
							InnerProp->ExportTextItem_Direct(Value, SourceData, DiffData, Parent, ExportFlags, ExportRootScope);
							HandleExportObject(InnerProp, ArrayHelper.GetRawPtr(DynamicArrayIndex), ArrayExportObjectProp);

							Out.Logf(TEXT("%s%s(%i)=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, DynamicArrayIndex, *Value);
						}
					}
					for (int32 DynamicArrayIndex = DiffArrayHelper.Num()-1; DynamicArrayIndex >= ArrayHelper.Num(); --DynamicArrayIndex)
					{
						Out.Logf(TEXT("%s%s.RemoveIndex(%d)\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, DynamicArrayIndex);
					}
					continue;
				}

				// Replaced operation goes through the generic property export code
				if (Operation == EOverriddenPropertyOperation::Modified)
				{
					if (const FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties->GetOverriddenPropertyNode(&Chain))
					{
						checkf(Arr && DiffArr, TEXT("Expecting a memory ptr to Array and its default"));

						const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(InnerProp);

						// Only array of instanced subobjects are handled here as like a set where the matching key is done using the archetype
						checkf(InnerObjectProperty&& InnerObjectProperty->HasAnyPropertyFlags(CPF_PersistentInstance), TEXT("Expecting only arrays of instanced subobjects"));

						auto FindObject = [InnerObjectProperty](const FOverriddenPropertyNodeID& ObjectToFind, FScriptArrayHelper& ArrayHelper) -> int32
						{
							const int32 ArrayNum = ArrayHelper.Num();
							for (int i = 0; i < ArrayNum; ++i)
							{
								if (UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i)))
								{
									if (ObjectToFind == FOverriddenPropertyNodeID(CurrentObject))
									{
										return i;
									}
								}
							}
							return INDEX_NONE;
						};

						TArray<int32> RemovedIndices;
						TArray<int32> AddedIndices;
						TArray<int32> ShadowIndices;
						bool bExternalized = false;

						if (FOverridableSerializationLogic::ShouldPropertyShadowSerializeSubObject(ArrayProperty))
						{
							const int32 ArrayNum = ArrayHelper.Num();
							for (int i = 0; i < ArrayNum; i++)
							{
								ShadowIndices.Add(i);
							}
						}

						for (const FOverriddenPropertyNode& SubNode: ArrayOverriddenPropertyNode->GetSubPropertyNodes())
						{
							switch (SubNode.GetOperation())
							{
							case EOverriddenPropertyOperation::Remove:
								{
									const int32 DefaultIndex = FindObject(SubNode.GetNodeID(), DiffArrayHelper);
									if (DefaultIndex != INDEX_NONE)
									{
										RemovedIndices.Add(DefaultIndex);
									}
									break;
								}
							case EOverriddenPropertyOperation::Add:
								{
									const int32 Index = FindObject(SubNode.GetNodeID(), ArrayHelper);
									if (Index != INDEX_NONE)
									{
										AddedIndices.Add(Index);
										ShadowIndices.Remove(Index);
									}
									break;
								}
							case EOverriddenPropertyOperation::Externalized:
								// nothing to do 
								bExternalized = true;
								break;
							default:
								checkf(false, TEXT("Unsupported operation type"));
								break;
							}
						}

						if(bExternalized)
						{
							Out.Logf(TEXT("%s%s<%s>=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *GetOverriddenOperationString(EOverriddenPropertyOperation::Externalized), TEXT("\"\""));
						}

						for (int32 i : RemovedIndices)
						{
							FString Value;
							InnerProp->ExportTextItem_Direct(Value, DiffArrayHelper.GetRawPtr(i), nullptr, Parent, ExportFlags, ExportRootScope);

							Out.Logf(TEXT("%s%s<%s>=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *GetOverriddenOperationString(EOverriddenPropertyOperation::Remove), *Value);
						}

						auto ExportArray = [&](TArray<int32> ExportIndices, const EOverriddenPropertyOperation Operation)
						{
							for (int32 i : ExportIndices)
						{
							FOverridableTextPortPropertyPathScope ScopePath(Property, i, EPropertyVisitorInfoType::ContainerIndex);

							FString Value;
							InnerProp->ExportTextItem_Direct(Value, ArrayHelper.GetRawPtr(i), nullptr, Parent, ExportFlags, ExportRootScope);
							HandleExportObject(InnerProp, ArrayHelper.GetRawPtr(i), ArrayExportObjectProp);

								Out.Logf(TEXT("%s%s<%s>=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *GetOverriddenOperationString(Operation), *Value);
						}
						};

						ExportArray(AddedIndices, EOverriddenPropertyOperation::Add);
						ExportArray(ShadowIndices, EOverriddenPropertyOperation::SubObjectsShadowing);
					}
					continue;
				}
			}

			// Map special case
			if (MapProperty != nullptr)
			{

				void* Map = Property->ContainerPtrToValuePtr<void>(ValuePtr, PropertyArrayIndex);
				void* DiffMap = nullptr;
				if (DiffStruct)
				{
					DiffMap = Property->ContainerPtrToValuePtrForDefaults<void>(DiffStruct, Diff, PropertyArrayIndex);
				}

				EOverriddenPropertyOperation Operation = EOverriddenPropertyOperation::None;
				FArchiveSerializedPropertyChain Chain;
				if (OverriddenProperties)
				{
					FOverridableTextPortPropertyPathScope ScopePath(Property);
					Operation = FOverridableSerializationLogic::GetOverriddenPropertyOperationForPortText(Map, DiffMap, PortFlags);
					FPropertyVisitorPath* Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
					checkf(Path, TEXT("Expecting a path"));
					Chain = Path->ToSerializedPropertyChain();
				}

				// Replaced operation goes through the generic property export code
				if (Operation == EOverriddenPropertyOperation::Modified)
				{
					if (const FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties->GetOverriddenPropertyNode(&Chain))
					{
						checkf(Map && DiffMap, TEXT("Expecting memory ptr to the map and its defaults"));

						FScriptMapHelper MapHelper(MapProperty, Map);
						FScriptMapHelper DiffMapHelper(MapProperty, DiffMap);

						TArray<int32> RemovedIndices;
						TArray<int32> ModifiedIndices;
						TArray<int32> AddedIndices;
						TArray<int32> ShadowedIndices;

						if (FOverridableSerializationLogic::ShouldPropertyShadowSerializeSubObject(MapProperty))
						{
							for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
							{
								ShadowedIndices.Add(It.GetInternalIndex());
							}
						}

						// Figure out the modifications of the map
						for (const FOverriddenPropertyNode& SubNode : MapOverriddenPropertyNode->GetSubPropertyNodes())
						{
							switch (SubNode.GetOperation())
							{
							case EOverriddenPropertyOperation::Remove:
								{
									const int32 InternalIndex = SubNode.GetNodeID().ToMapInternalIndex(DiffMapHelper);
									if (InternalIndex != INDEX_NONE)
									{
										RemovedIndices.Add(InternalIndex);
									}
									break;
								}
							case EOverriddenPropertyOperation::Add:
								{
									const int32 InternalIndex = SubNode.GetNodeID().ToMapInternalIndex(MapHelper);
									if (InternalIndex != INDEX_NONE)
									{
										AddedIndices.Add(InternalIndex);
										ShadowedIndices.Remove(InternalIndex);
									}
									break;
								}
							case EOverriddenPropertyOperation::Modified:
								{
									const int32 InternalIndex = SubNode.GetNodeID().ToMapInternalIndex(MapHelper);
									if (InternalIndex != INDEX_NONE)
									{
										ModifiedIndices.Add(InternalIndex);
										ShadowedIndices.Remove(InternalIndex);
									}
									break;
								}
							default:
								checkf(false, TEXT("Unsupported map operation"));
								break;
							}
						}

						auto ExportArray = [&](TArray<int32> ExportIndices, const EOverriddenPropertyOperation Operation)
						{
							for (int32 i : ExportIndices)
						{
								FOverridableTextPortPropertyPathScope ScopePath(Property, i, EPropertyVisitorInfoType::ContainerIndex);

							FString Key;
								MapHelper.KeyProp->ExportTextItem_Direct(Key, MapHelper.GetKeyPtr(i), /*DefaultValue*/nullptr, Parent, ExportFlags, ExportRootScope);

							FString Value;
								MapHelper.ValueProp->ExportTextItem_Direct(Value, MapHelper.GetValuePtr(i), /*DefaultValue*/nullptr, Parent, ExportFlags, ExportRootScope);

							Out.Logf(TEXT("%s%s<%s>=(%s,%s)\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *GetOverriddenOperationString(Operation), *Key, *Value);
							}
						};

						for (int32 i : RemovedIndices)
						{
							FString Value;
							DiffMapHelper.KeyProp->ExportTextItem_Direct(Value, DiffMapHelper.GetKeyPtr(i), nullptr, Parent, ExportFlags, ExportRootScope);

							Out.Logf(TEXT("%s%s<%s>=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *GetOverriddenOperationString(EOverriddenPropertyOperation::Remove), *Value);
						}

						ExportArray(ModifiedIndices, EOverriddenPropertyOperation::Modified);
						ExportArray(AddedIndices, EOverriddenPropertyOperation::Add);
						ExportArray(ShadowedIndices, EOverriddenPropertyOperation::SubObjectsShadowing);
					}

					continue;
				}
						}

			// Recursively call this method for struct in the overridable serialization case
			if (StructProperty != nullptr && Struct == DiffStruct && OverriddenProperties)
						{
				FOverridableTextPortPropertyPathScope ScopePath(StructProperty);

				ExportProperties_Overrides(
					Context, 
					Out, 
					StructProperty->Struct, 
					Property->ContainerPtrToValuePtr<uint8>(ValuePtr, PropertyArrayIndex), 
					Indent, 
					StructProperty->Struct,
					Diff ? Property->ContainerPtrToValuePtr<uint8>(Diff, PropertyArrayIndex) : nullptr,
					Parent,
					PortFlags,
					ExportRootScope,
					OverriddenProperties,
					SanitizedPropertyName);
					continue;
				}

			// Generic property export code
			FString	Value;
			const bool bStaticArray = Property->ArrayDim > 1;
			FOverridableTextPortPropertyPathScope ScopePath(Property, bStaticArray ? PropertyArrayIndex : INDEX_NONE, bStaticArray ? EPropertyVisitorInfoType::StaticArrayIndex : EPropertyVisitorInfoType::None);

			uint8* DiffData = (DiffStruct && Property->IsInContainer(DiffStruct->GetPropertiesSize())) ? Diff : NULL;
			if (Property->ExportText_InContainer(PropertyArrayIndex, Value, ValuePtr, DiffData, Parent, ExportFlags, ExportRootScope))
			{
				HandleExportObject(Property, Property->ContainerPtrToValuePtr<uint8>(ValuePtr, PropertyArrayIndex), ExportObjectProp);

				FString OverridableOperation;
				if (OverriddenProperties)
				{
					const EOverriddenPropertyOperation Operation = FOverridableSerializationLogic::GetOverriddenPropertyOperationForPortText(ValuePtr, DiffData, PortFlags);
					if (Operation!=EOverriddenPropertyOperation::None)
					{
						OverridableOperation = FString::Printf(TEXT("<%s>"), *GetOverriddenOperationString(Operation));
					}
				}

				if (!bStaticArray)
				{
					Out.Logf( TEXT("%s%s%s=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *OverridableOperation, *Value );
				}
				else
				{
					Out.Logf( TEXT("%s%s(%i)%s=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, PropertyArrayIndex, *OverridableOperation, *Value );
				}
			}
		}
	}
}

/** Shim ExportProperties when FOverridableSerializationLogic::ECapabilities::T3DSerialization capabilities disabled */
void ExportProperties_NoOverrides
(
	const FExportObjectInnerContext* Context,
	FOutputDevice&	Out,
	UClass*			ObjectClass,
	uint8*			Object,
	int32			Indent,
	UClass*			DiffClass,
	uint8*			Diff,
	UObject*		Parent,
	uint32			PortFlags,
	UObject*		ExportRootScope
)
{
	check(ObjectClass != NULL);
	
	for( FProperty* Property = ObjectClass->PropertyLink; Property; Property = Property->PropertyLinkNext )
	{
		if (!Property->ShouldPort(PortFlags))
			continue;

		FString SanitizedPropertyName = Property->GetName();
		constexpr FAsciiSet Whitespace("\t ");
		constexpr FAsciiSet SpecialCharacters("=()[].\"\'");
		if (FAsciiSet::HasAny(SanitizedPropertyName, SpecialCharacters) || Whitespace.Contains(SanitizedPropertyName[0]))
		{
			// to increase frequency of forward compatibility, only sanitize property names that absolutely need it
			SanitizedPropertyName = FString::Format(TEXT("\"{0}\""), {Property->GetName().ReplaceCharWithEscapedChar()});
		}

		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FObjectPropertyBase* ExportObjectProp = (Property->PropertyFlags & CPF_ExportObject) != 0 ? CastField<FObjectPropertyBase>(Property) : NULL;
		const uint32 ExportFlags = PortFlags | PPF_Delimited;

		if ( ArrayProperty != NULL )
		{
			// Export dynamic array.
			FProperty* InnerProp = ArrayProperty->Inner;
			ExportObjectProp = (Property->PropertyFlags & CPF_ExportObject) != 0 ? CastField<FObjectPropertyBase>(InnerProp) : NULL;
			// This is used as the default value in the case of an array property that has
			// fewer elements than the exported object.
			uint8* StructDefaults = NULL;
			FStructProperty* StructProperty = CastField<FStructProperty>(InnerProp);
			if ( StructProperty != NULL )
			{
				checkSlow(StructProperty->Struct);
				StructDefaults = (uint8*)FMemory::Malloc(StructProperty->Struct->GetStructureSize());
				StructProperty->InitializeValue(StructDefaults);
			}

			for( int32 PropertyArrayIndex=0; PropertyArrayIndex<Property->ArrayDim; PropertyArrayIndex++ )
			{
				void* Arr = Property->ContainerPtrToValuePtr<void>(Object, PropertyArrayIndex);
				FScriptArrayHelper ArrayHelper(ArrayProperty, Arr);

				void*	DiffArr = NULL;
				if( DiffClass )
				{
					DiffArr = Property->ContainerPtrToValuePtrForDefaults<void>(DiffClass, Diff, PropertyArrayIndex);
				}
				// we won't use this if DiffArr is NULL, but we have to set it up to something
				FScriptArrayHelper DiffArrayHelper(ArrayProperty, DiffArr);

				// If the current size of the array is 0 and the default one is not, add in an empty item so on import it will be empty
				if( ArrayHelper.Num() == 0 && DiffArrayHelper.Num() != 0 )
				{
					Out.Logf(TEXT("%s%s=\r\n"), FCString::Spc(Indent), *SanitizedPropertyName);
				}
				else
				{
					// If the array sizes are different, we will need to export each index so on import we maintain the size
					for (int32 DynamicArrayIndex = 0; DynamicArrayIndex < ArrayHelper.Num(); DynamicArrayIndex++)
					{
						FString	Value;

						// compare each element's value manually so that elements which match the NULL value for the array's inner property type
						// but aren't in the diff array are still exported
						uint8* SourceData = ArrayHelper.GetRawPtr(DynamicArrayIndex);
						bool bHasDiffData = DiffArr && DynamicArrayIndex < DiffArrayHelper.Num();
						uint8* DiffData = bHasDiffData ? DiffArrayHelper.GetRawPtr(DynamicArrayIndex) : StructDefaults;

						// Make sure to export the last element even if it is default value if the default data doesn't have an element at that index (!bHasDiffData && (DynamicArrayIndex == ArrayHelper.Num()-1) && !bHasDiffData) 
						// because that will ensure the resulting imported array will be of proper size
						bool bExportItem = DiffData == NULL || (!bHasDiffData && (DynamicArrayIndex == ArrayHelper.Num()-1)) || (DiffData != SourceData && !InnerProp->Identical(SourceData, DiffData, ExportFlags));
						if (bExportItem)
						{
							InnerProp->ExportTextItem_Direct(Value, SourceData, DiffData, Parent, ExportFlags, ExportRootScope);
							if (ExportObjectProp)
							{
								UObject* Obj = ExportObjectProp->GetObjectPropertyValue(ArrayHelper.GetRawPtr(DynamicArrayIndex));
								check(!Obj || Obj->IsValidLowLevel());
								if (Obj && !Obj->HasAnyMarks(OBJECTMARK_TagImp))
								{
									// only export the BEGIN OBJECT block for a component if Parent is the component's Outer....when importing subobject definitions,
									// (i.e. BEGIN OBJECT), whichever BEGIN OBJECT block a component's BEGIN OBJECT block is located within is the object that will be
									// used as the Outer to create the component

									// Is this an array of components?
									if (InnerProp->HasAnyPropertyFlags(CPF_InstancedReference))
									{
										if (Obj->GetOuter() == Parent)
										{
											// Don't export more than once.
											Obj->Mark(OBJECTMARK_TagImp);
											UExporter::ExportToOutputDevice(Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags);
										}
										else
										{
											// set the OBJECTMARK_TagExp flag so that the calling code knows we wanted to export this object
											Obj->Mark(OBJECTMARK_TagExp);
										}
									}
									else
									{
										// Don't export more than once.
										Obj->Mark(OBJECTMARK_TagImp);
										UExporter::ExportToOutputDevice(Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags);
									}
								}
							}

							Out.Logf(TEXT("%s%s(%i)=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, DynamicArrayIndex, *Value);
						}
					}
					for (int32 DynamicArrayIndex = DiffArrayHelper.Num()-1; DynamicArrayIndex >= ArrayHelper.Num(); --DynamicArrayIndex)
					{
						Out.Logf(TEXT("%s%s.RemoveIndex(%d)\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, DynamicArrayIndex);
					}
				}
			}
			if (StructDefaults)
			{
				StructProperty->DestroyValue(StructDefaults);
				FMemory::Free(StructDefaults);
			}
		}
		else
		{
			for( int32 PropertyArrayIndex=0; PropertyArrayIndex<Property->ArrayDim; PropertyArrayIndex++ )
			{
				FString	Value;
				// Export single element.

				uint8* DiffData = (DiffClass && Property->IsInContainer(DiffClass->GetPropertiesSize())) ? Diff : NULL;
				if( Property->ExportText_InContainer( PropertyArrayIndex, Value, Object, DiffData, Parent, ExportFlags, ExportRootScope ) )
				{
					if ( ExportObjectProp )
					{
						UObject* Obj = ExportObjectProp->GetObjectPropertyValue(Property->ContainerPtrToValuePtr<void>(Object, PropertyArrayIndex));
						if( Obj && !Obj->HasAnyMarks(OBJECTMARK_TagImp) )
						{
							// only export the BEGIN OBJECT block for a component if Parent is the component's Outer....when importing subobject definitions,
							// (i.e. BEGIN OBJECT), whichever BEGIN OBJECT block a component's BEGIN OBJECT block is located within is the object that will be
							// used as the Outer to create the component
							if ( Property->HasAnyPropertyFlags(CPF_InstancedReference) )
							{
								if ( Obj->GetOuter() == Parent )
								{
									// Don't export more than once.
									Obj->Mark(OBJECTMARK_TagImp);
									UExporter::ExportToOutputDevice( Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags );
								}
								else
								{
									// set the OBJECTMARK_TagExp flag so that the calling code knows we wanted to export this object
									Obj->Mark(OBJECTMARK_TagExp);
								}
							}
							else
							{
								// Don't export more than once.
								Obj->Mark(OBJECTMARK_TagImp);
								UExporter::ExportToOutputDevice( Context, Obj, NULL, Out, TEXT("T3D"), Indent, PortFlags );
							}
						}
					}

					if( Property->ArrayDim == 1 )
					{
						Out.Logf( TEXT("%s%s=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, *Value );
					}
					else
					{
						Out.Logf( TEXT("%s%s(%i)=%s\r\n"), FCString::Spc(Indent), *SanitizedPropertyName, PropertyArrayIndex, *Value );
					}
				}
			}
		}
	}

	// Allows to import/export C++ properties in case the automatic unreal script mesh wouldn't work.
	Parent->ExportCustomProperties(Out, Indent);
}

/**
 * Exports the property values for the specified object as text to the output device.
 *
 * @param	Context			Context from which the set of 'inner' objects is extracted.  If NULL, an object iterator will be used.
 * @param	Out				the output device to send the exported text to
 * @param	ObjectClass		the class of the object to dump properties for
 * @param	Object			the address of the object to dump properties for
 * @param	Indent			number of spaces to prepend to each line of output
 * @param	DiffClass		the class to use for comparing property values when delta export is desired.
 * @param	Diff			the address of the object to use for determining whether a property value should be exported.  If the value in Object matches the corresponding
 *							value in Diff, it is not exported.  Specify NULL to export all properties.
 * @param	Parent			the UObject corresponding to Object
 * @param	PortFlags		flags used for modifying the output and/or behavior of the export
 */
void ExportProperties
(
	const FExportObjectInnerContext* Context,
	FOutputDevice&	Out,
	UClass*			ObjectClass,
	uint8*			Object,
	int32			Indent,
	UClass*			DiffClass,
	uint8*			Diff,
	UObject*		Parent,
	uint32			PortFlags,
	UObject*		ExportRootScope
)
{
	if (FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization))
	{
		FOverriddenPropertySet* OverriddenProperties = Object ? UnrealExporterPrivate::GetOverriddenProperties((UObject*)Object) : nullptr;
		FEnableOverridableSerializationScope Scope(OverriddenProperties != nullptr, OverriddenProperties);

		ExportProperties_Overrides(Context, Out, ObjectClass, Object, Indent, DiffClass, Diff, Parent, PortFlags, ExportRootScope, OverriddenProperties);

		// Allows to import/export C++ properties in case the automatic unreal script mesh wouldn't work.
		Parent->ExportCustomProperties(Out, Indent);
	}
	else
	{
		ExportProperties_NoOverrides(Context, Out, ObjectClass, Object, Indent, DiffClass, Diff, Parent, PortFlags, ExportRootScope);
	}
}

/**
 * Debug spew for components
 * @param Object object to dump component spew for
 */
void DumpComponents(UObject *Object)
{
	for ( FThreadSafeObjectIterator It; It; ++It )
	{
		It->UnMark(EObjectMark(OBJECTMARK_TagImp | OBJECTMARK_TagExp));
	}

	if (FPlatformMisc::IsDebuggerPresent() )
	{
		// if we have a debugger attached, the watch window won't be able to display the full output if we attempt to log it as a single string
		// so pass in GLog instead so that each line is sent separately;  this causes the output to have an extra line break between each log statement,
		// but at least we'll be able to see the full output in the debugger's watch window
		UE_LOGF(LogExporter, Log, "Components for '%ls':", *Object->GetFullName());
		ExportProperties( NULL, *GLog, Object->GetClass(), (uint8*)Object, 0, NULL, NULL, Object, PPF_SubobjectsOnly );
		UE_LOGF(LogExporter, Log, "<--- DONE!");
	}
	else
	{
		FStringOutputDevice Output;
			Output.Logf(TEXT("Components for '%s':\r\n"), *Object->GetFullName());
			ExportProperties( NULL, Output, Object->GetClass(), (uint8*)Object, 2, NULL, NULL, Object, PPF_SubobjectsOnly );
			Output.Logf(TEXT("<--- DONE!\r\n"));
		UE_LOGF(LogExporter, Log, "%ls", *Output);
	}
}


FString DumpComponentsToString(UObject *Object)
{
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Output;
	Output.Logf(TEXT("Components for '%s':\r\n"), *Object->GetFullName());
	ExportProperties(NULL, Output, Object->GetClass(), (uint8*)Object, 2, NULL, NULL, Object, PPF_SubobjectsOnly);
	Output.Logf(TEXT("<--- DONE!\r\n"));
	return MoveTemp(Output);
}



void DumpObject(const TCHAR* Label, UObject* Object)
{
	FString const ExportedText = DumpObjectToString(Object);
	UE_LOGF(LogExporter, Display, "%ls", Label);
	UE_LOGF(LogExporter, Display, "%ls", *ExportedText);
}

FString DumpObjectToString(UObject* Object)
{
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context, Object, NULL, Archive, TEXT("copy"), 0, PPF_Copy | PPF_DebugDump, false);

	return MoveTemp(Archive);
}

#if WITH_EDITOR
FSelectedActorExportObjectInnerContext::FSelectedActorExportObjectInnerContext()
	: FExportObjectInnerContext(false) //call the empty version of the base class
{
	// For each selected actor...
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = (AActor*)*It;
		checkSlow(Actor->IsA(AActor::StaticClass()));
		AddSelectedActor(Actor);
	}
}

FSelectedActorExportObjectInnerContext::FSelectedActorExportObjectInnerContext(const TArray<AActor*> InSelectedActors)
	: FExportObjectInnerContext(false) //call the empty version of the base class
{
	// For each selected actor...
	for (AActor* Actor : InSelectedActors)
	{
		// Explicitly mark each actor as an inner of its outer.
		// This can be used (e.g. in ULevelExporterT3D`::ExportText()`)
		// to specify the precise order in which objects will be exported.
		AddObjectToInnerMap(Actor);
		AddSelectedActor(Actor);
	}
}

bool FSelectedActorExportObjectInnerContext::IsObjectSelected(const UObject* InObj) const
{
	const AActor* Actor = Cast<AActor>(InObj);
	return Actor && SelectedActors.Contains(Actor);
}

void FSelectedActorExportObjectInnerContext::AddSelectedActor(const AActor* InActor)
{
	SelectedActors.Add(InActor);

	ForEachObjectWithOuter(InActor, [this](UObject* InnerObj)
	{
		AddObjectToInnerMap(InnerObj);
	}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
}

#endif
