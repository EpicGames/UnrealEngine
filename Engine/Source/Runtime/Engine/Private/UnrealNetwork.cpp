// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/UnrealNetwork.h"
#include "UObject/CoreNet.h"

FPreReplayScrub                                      FNetworkReplayDelegates::OnPreScrub;
FReplayScrubTeardown                                 FNetworkReplayDelegates::OnScrubTeardown;
FOnWriteGameSpecificDemoHeader                       FNetworkReplayDelegates::OnWriteGameSpecificDemoHeader;
FOnProcessGameSpecificDemoHeader                     FNetworkReplayDelegates::OnProcessGameSpecificDemoHeader;
FOnWriteGameSpecificFrameData                        FNetworkReplayDelegates::OnWriteGameSpecificFrameData;
FOnProcessGameSpecificFrameData                      FNetworkReplayDelegates::OnProcessGameSpecificFrameData;
FGetOverridableVersionDataForDemoHeaderReadDelegate  FNetworkReplayDelegates::GetOverridableVersionDataForHeaderRead;
FGetOverridableVersionDataForDemoHeaderWriteDelegate FNetworkReplayDelegates::GetOverridableVersionDataForHeaderWrite;
FOnReplayStartedDelegate                             FNetworkReplayDelegates::OnReplayStarted;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOnReplayStartFailureDelegate                        FNetworkReplayDelegates::OnReplayStartFailure;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
FOnReplayPlaybackFailureDelegate                     FNetworkReplayDelegates::OnReplayPlaybackFailure;
FOnReplayScrubCompleteDelegate                       FNetworkReplayDelegates::OnReplayScrubComplete;
FOnReplayPlaybackCompleteDelegate                    FNetworkReplayDelegates::OnReplayPlaybackComplete;
FOnReplayRecordingStartAttemptDelegate               FNetworkReplayDelegates::OnReplayRecordingStartAttempt;
FOnReplayRecordingCompleteDelegate                   FNetworkReplayDelegates::OnReplayRecordingComplete;
FOnPauseChannelsChangedDelegate                      FNetworkReplayDelegates::OnPauseChannelsChanged;
FOnReplayIDChangedDelegate                           FNetworkReplayDelegates::OnReplayIDChanged;

// ----------------------------------------------------------------

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void RegisterReplicatedLifetimeProperty(
	const FProperty* ReplicatedProperty,
	TArray<FLifetimeProperty>& OutLifetimeProps,
	ELifetimeCondition InCondition,
	ELifetimeRepNotifyCondition InRepNotifyCondition)
{
	FDoRepLifetimeParams Params;
	Params.Condition = InCondition;
	Params.RepNotifyCondition = InRepNotifyCondition;
	RegisterReplicatedLifetimeProperty(ReplicatedProperty, OutLifetimeProps, Params);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void RegisterReplicatedLifetimeProperty(
	const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor,
	TArray<FLifetimeProperty>& OutLifetimeProps,
	const FDoRepLifetimeParams& Params)
{
	checkf(Params.Condition != COND_NetGroup, TEXT("Invalid lifetime condition for %s. COND_NetGroup can only be used with registered subobjects"), PropertyDescriptor.PropertyName);
	for (int32 i = 0; i < PropertyDescriptor.ArrayDim; i++)
	{
		const uint16 RepIndex = PropertyDescriptor.RepIndex + i;
		FLifetimeProperty* RegisteredPropertyPtr = OutLifetimeProps.FindByPredicate([&RepIndex](const FLifetimeProperty& Var) { return Var.RepIndex == RepIndex; });

		FLifetimeProperty LifetimeProp(RepIndex, Params.Condition, Params.RepNotifyCondition, Params.bIsPushBased);

		LifetimeProp.CreateAndRegisterReplicationFragmentFunction = Params.CreateAndRegisterReplicationFragmentFunction;

		if (RegisteredPropertyPtr)
		{
			// Disabled properties can be re-enabled via DOREPLIFETIME
			if (RegisteredPropertyPtr->Condition == COND_Never)
			{
				// Copy the new conditions since disabling a property doesn't set other conditions.
				(*RegisteredPropertyPtr) = LifetimeProp;
			}
			else
			{
				// Conditions should be identical when calling DOREPLIFETIME twice on the same variable.
				checkf((*RegisteredPropertyPtr) == LifetimeProp, TEXT("Property %s was registered twice with different conditions (old:%d) (new:%d)"), PropertyDescriptor.PropertyName, RegisteredPropertyPtr->Condition, Params.Condition);
			}
		}
		else
		{
			OutLifetimeProps.Add(LifetimeProp);
		}
	}
}

void RegisterReplicatedLifetimeProperty(
	const FProperty* ReplicatedProperty,
	TArray<FLifetimeProperty>& OutLifetimeProps,
	const FDoRepLifetimeParams& Params)
{
	if (ReplicatedProperty == nullptr)
	{
		check(false);
		return;
	}

	const FString ReplicatedPropertyName = ReplicatedProperty->GetName();
	NetworkingPrivate::FRepPropertyDescriptor PropDesc(*ReplicatedPropertyName, ReplicatedProperty->RepIndex, ReplicatedProperty->ArrayDim);
	RegisterReplicatedLifetimeProperty(PropDesc, OutLifetimeProps, Params);
}

void SetReplicatedPropertyToDisabled(const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor, TArray<FLifetimeProperty>& OutLifetimeProps)
{
	for (int32 i = 0; i < PropertyDescriptor.ArrayDim; i++)
	{
		const uint16 RepIndex = PropertyDescriptor.RepIndex + i;
		FLifetimeProperty* RegisteredPropertyPtr = OutLifetimeProps.FindByPredicate([&RepIndex](const FLifetimeProperty& Var) { return Var.RepIndex == RepIndex; });

		if (RegisteredPropertyPtr)
		{
			RegisteredPropertyPtr->Condition = COND_Never;
		}
		else
		{
			OutLifetimeProps.Add(FLifetimeProperty(RepIndex, COND_Never));
		}
	}
}

void SetReplicatedPropertyToDisabled(const FProperty* ReplicatedProperty, TArray<FLifetimeProperty>& OutLifetimeProps)
{
	const FString ReplicatedPropertyName = ReplicatedProperty->GetName();
	NetworkingPrivate::FRepPropertyDescriptor PropDesc(*ReplicatedPropertyName, ReplicatedProperty->RepIndex, ReplicatedProperty->ArrayDim);
	SetReplicatedPropertyToDisabled(PropDesc, OutLifetimeProps);
}

void DisableReplicatedLifetimeProperty(const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor, TArray<FLifetimeProperty>& OutLifetimeProps)
{
	SetReplicatedPropertyToDisabled(PropertyDescriptor, OutLifetimeProps);
}

void DisableReplicatedLifetimeProperty(const UClass* ThisClass, const UClass* PropertyClass, FName PropertyName, TArray< FLifetimeProperty >& OutLifetimeProps)
{
	const FProperty* ReplicatedProperty = GetReplicatedProperty(ThisClass, PropertyClass, PropertyName);
	if (!ReplicatedProperty)
	{
		return;
	}

	const FString ReplicatedPropertyName = ReplicatedProperty->GetName();
	NetworkingPrivate::FRepPropertyDescriptor PropDesc(*ReplicatedPropertyName, ReplicatedProperty->RepIndex, ReplicatedProperty->ArrayDim);
	SetReplicatedPropertyToDisabled(PropDesc, OutLifetimeProps);
}

void ResetReplicatedLifetimeProperty(
	const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor,
	ELifetimeCondition LifetimeCondition,
	TArray<FLifetimeProperty>& OutLifetimeProps)
{
	for (int32 i = 0; i < PropertyDescriptor.ArrayDim; i++)
	{
		uint16 RepIndex = PropertyDescriptor.RepIndex + i;
		FLifetimeProperty* RegisteredPropertyPtr = OutLifetimeProps.FindByPredicate([&RepIndex](const FLifetimeProperty& Var) { return Var.RepIndex == RepIndex; });

		// Set the new condition
		if (RegisteredPropertyPtr)
		{
			RegisteredPropertyPtr->Condition = LifetimeCondition;
		}
		else
		{
			OutLifetimeProps.Add(FLifetimeProperty(RepIndex, LifetimeCondition));
		}
	}
}

void ResetReplicatedLifetimeProperty(const UClass* ThisClass, const UClass* PropertyClass, FName PropertyName, ELifetimeCondition LifetimeCondition, TArray< FLifetimeProperty >& OutLifetimeProps)
{
	const FProperty* ReplicatedProperty = GetReplicatedProperty(ThisClass, PropertyClass, PropertyName);
	if (!ReplicatedProperty)
	{
		return;
	}

	const FString ReplicatedPropertyName = ReplicatedProperty->GetName();
	NetworkingPrivate::FRepPropertyDescriptor PropDesc(*ReplicatedPropertyName, ReplicatedProperty->RepIndex, ReplicatedProperty->ArrayDim);
	ResetReplicatedLifetimeProperty(PropDesc, LifetimeCondition, OutLifetimeProps);
}

void ResetReplicatedLifetimeProperty(const UClass* ThisClass, const UClass* PropertyClass, FName PropertyName, const FDoRepLifetimeParams& Params, TArray< FLifetimeProperty >& OutLifetimeProps)
{
	const FProperty* ReplicatedProperty = GetReplicatedProperty(ThisClass, PropertyClass, PropertyName);
	if (!ReplicatedProperty)
	{
		return;
	}

	const FString ReplicatedPropertyName = ReplicatedProperty->GetName();
	NetworkingPrivate::FRepPropertyDescriptor PropDesc(*ReplicatedPropertyName, ReplicatedProperty->RepIndex, ReplicatedProperty->ArrayDim);
	ResetReplicatedLifetimeProperty(PropDesc, Params, OutLifetimeProps);
}

void ResetReplicatedLifetimeProperty(const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor, const FDoRepLifetimeParams& Params, TArray< FLifetimeProperty >& OutLifetimeProps)
{
	for (int32 i = 0; i < PropertyDescriptor.ArrayDim; i++)
	{
		uint16 RepIndex = PropertyDescriptor.RepIndex + i;
		FLifetimeProperty* RegisteredPropertyPtr = OutLifetimeProps.FindByPredicate([&RepIndex](const FLifetimeProperty& Var) { return Var.RepIndex == RepIndex; });

		// Set the new condition
		if (RegisteredPropertyPtr)
		{
			RegisteredPropertyPtr->bIsPushBased = Params.bIsPushBased;
			RegisteredPropertyPtr->Condition = Params.Condition;
			RegisteredPropertyPtr->RepNotifyCondition = Params.RepNotifyCondition;
		}
		else
		{
			OutLifetimeProps.Add(FLifetimeProperty(RepIndex, Params.Condition, Params.RepNotifyCondition, Params.bIsPushBased));
		}
	}
}

void DisableAllReplicatedPropertiesOfClass(const NetworkingPrivate::FRepClassDescriptor& ClassDescriptor, EFieldIteratorFlags::SuperClassFlags SuperClassBehavior, TArray<FLifetimeProperty>& OutLifetimeProps)
{
	const int32 StartIndex = (EFieldIteratorFlags::IncludeSuper == SuperClassBehavior) ? 0 : ClassDescriptor.StartRepIndex;
	for (int32 RepIndex = StartIndex; RepIndex <= ClassDescriptor.EndRepIndex; ++RepIndex)
	{
		FLifetimeProperty* RegisteredPropertyPtr = OutLifetimeProps.FindByPredicate([&RepIndex](const FLifetimeProperty& Var) { return Var.RepIndex == RepIndex; });

		if (RegisteredPropertyPtr)
		{
			RegisteredPropertyPtr->Condition = COND_Never;
		}
		else
		{
			OutLifetimeProps.Add(FLifetimeProperty(RepIndex, COND_Never));
		}
	}
}

void DisableAllReplicatedPropertiesOfClass(const UClass* ThisClass, const UClass* ClassToDisable, EFieldIteratorFlags::SuperClassFlags SuperClassBehavior, TArray< FLifetimeProperty >& OutLifetimeProps)
{
	if (!ThisClass->IsChildOf(ClassToDisable))
	{
		ensureMsgf(false, TEXT("Attempting to disable replicated properties of '%s' but current class '%s' is not a child of '%s'"), *ClassToDisable->GetName(), *ThisClass->GetName(), *ClassToDisable->GetName());
		return;
	}

	for (TFieldIterator<FProperty> It(ClassToDisable, SuperClassBehavior); It; ++It)
	{
		const FProperty* Prop = *It;
		if (Prop && Prop->PropertyFlags & CPF_Net)
		{
			SetReplicatedPropertyToDisabled(Prop, OutLifetimeProps);
		}
	}
}
