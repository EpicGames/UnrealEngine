// Copyright Epic Games, Inc. All Rights Reserved.

#include "UpdateTextureShaders.h"
#include "Misc/DelayedAutoRegister.h"

IMPLEMENT_TYPE_LAYOUT(FUpdateTexture2DSubresourceCS);

IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(, ENGINE_API, FUpdateTexture3DSubresourceCS, SF_Compute);

#define IMPLEMENT_UPDATE_TEX2D_SUBRESOURCE_SHADER(ValueType)\
	typedef TUpdateTexture2DSubresourceCS<EUpdateTextureValueType::ValueType> FUpdateTexture2DSubresourceCS_##ValueType;\
	IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, ENGINE_API, FUpdateTexture2DSubresourceCS_##ValueType, SF_Compute);
IMPLEMENT_UPDATE_TEX2D_SUBRESOURCE_SHADER(Float);
IMPLEMENT_UPDATE_TEX2D_SUBRESOURCE_SHADER(Int32);
IMPLEMENT_UPDATE_TEX2D_SUBRESOURCE_SHADER(Uint32);

IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, ENGINE_API, TCopyDataCS<2>, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, ENGINE_API, TCopyDataCS<1>, SF_Compute);
