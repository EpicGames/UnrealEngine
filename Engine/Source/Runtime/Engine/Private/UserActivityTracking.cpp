// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserActivityTracking.h"
#include "Engine/EngineTypes.h"
#include "Misc/CoreDelegates.h"
#include "Stats/Stats.h"

FUserActivityTracking::FOnActivityChanged FUserActivityTracking::OnActivityChanged;

FUserActivity FUserActivityTracking::UserActivity;
EUserActivityContext FUserActivityTracking::ContextFilter = EUserActivityContext::Game;		// default to expecting (and firing delegates for) Game activities

void FUserActivityTracking::SetContextFilter(EUserActivityContext InContext)
{
	ContextFilter = InContext;
}

void FUserActivityTracking::SetActivity(const FUserActivity& InUserActivity)
{ 
	if (InUserActivity.Context == ContextFilter)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FUserActivityTracking_SetActivity);
		UserActivity = InUserActivity;
		OnActivityChanged.Broadcast(UserActivity);
		FCoreDelegates::UserActivityStringChanged.Broadcast(UserActivity.ActionName);
	}
}
