// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/UserDefinedEnum.h"
#include "Templates/SubclassOf.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedEnum)

#if WITH_EDITOR
#include "Kismet2/EnumEditorUtils.h"
#include "UObject/CookedMetaData.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#endif	// WITH_EDITOR

UUserDefinedEnum::UUserDefinedEnum(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UUserDefinedEnum::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.IsPersistent())
		{
			Names.MakeValuesSequentialFromZero();

			if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::StableUserDefinedEnumDisplayNames)
			{
				// Make sure DisplayNameMap is empty so we perform the meta-data upgrade in PostLoad
				DisplayNameMap.Reset();
			}
		}

		// Apply any transformations right away so that the enum is consistent at the end of its serialization during loading.
		if (!Ar.IsObjectReferenceCollector() && !Ar.IsCountingMemory() && !Ar.IsTransacting())
		{
			FEnumEditorUtils::UpdateAfterPathChanged(this);
			if (NumEnums() > 1 && DisplayNameMap.Num() == 0) // >1 because User Defined Enums always have a "MAX" entry
			{
				FEnumEditorUtils::UpgradeDisplayNamesFromMetaData(this);
			}
			FEnumEditorUtils::EnsureAllDisplayNamesExist(this);
		}
	}
#endif // WITH_EDITOR
}

FString UUserDefinedEnum::GenerateFullEnumName(const TCHAR* InEnumName) const
{
	check(CppForm == ECppForm::Namespaced);

	// This code used to compute full path but not use it
	return Super::GenerateFullEnumName(InEnumName);
}

#if WITH_EDITOR
bool UUserDefinedEnum::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	if (!FEnumEditorUtils::IsNameAvailebleForUserDefinedEnum(FName(NewName)))
	{
		UE_LOGF(LogClass, Warning, "UEnum::Rename: Name '%ls' is already used.", NewName);
		return false;
	}
	
	const bool bSucceeded = Super::Rename(NewName, NewOuter, Flags);

	if (bSucceeded)
	{
		FEnumEditorUtils::UpdateAfterPathChanged(this);
	}

	return bSucceeded;
}

void UUserDefinedEnum::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if(!bDuplicateForPIE)
	{
		FEnumEditorUtils::UpdateAfterPathChanged(this);
	}
}

bool UUserDefinedEnum::IsPostLoadThreadSafe() const
{
	return true;
}

void UUserDefinedEnum::PostLoad()
{
	Super::PostLoad();

	// Apply the transactional flag to user defined enums that were not created with it
	SetFlags(RF_Transactional);
}

void UUserDefinedEnum::PostEditUndo()
{
	Super::PostEditUndo();
	FEnumEditorUtils::PostEditUndo(this);
}

void UUserDefinedEnum::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Add the description to the tooltip
 	static const FName NAME_Tooltip("Tooltip");

	UPackage* Package = GetOutermost();
	check(Package);
	FMetaData& PackageMetaData = Package->GetMetaData();

	if (!EnumDescription.IsEmpty())
	{
		PackageMetaData.SetValue(this, NAME_Tooltip, *EnumDescription.ToString());
	}
	else
	{
		PackageMetaData.RemoveValue(this, NAME_Tooltip);
	}
}

void UUserDefinedEnum::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	FString DescriptionString;
	FTextStringHelper::WriteToBuffer(/*out*/ DescriptionString, EnumDescription);
	Context.AddTag(FAssetRegistryTag(GET_MEMBER_NAME_CHECKED(UUserDefinedEnum, EnumDescription), DescriptionString, FAssetRegistryTag::TT_Hidden));
}

void UUserDefinedEnum::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
	Super::PreSaveRoot(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking() && (ObjectSaveContext.GetSaveFlags() & SAVE_Optional))
	{
		UEnumCookedMetaData* CookedMetaData = NewCookedMetaData();
		CookedMetaData->CacheMetaData(this);

		if (!CookedMetaData->HasMetaData())
		{
			PurgeCookedMetaData();
		}
	}
	else
	{
		PurgeCookedMetaData();
	}
}

void UUserDefinedEnum::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

	PurgeCookedMetaData();
}

FString UUserDefinedEnum::GenerateNewEnumeratorName()
{
	FString EnumNameString;
	do 
	{
		EnumNameString = FString::Printf(TEXT("NewEnumerator%u"), UniqueNameIndex);
		++UniqueNameIndex;
	} 
	while (!FEnumEditorUtils::IsProperNameForUserDefinedEnumerator(this, EnumNameString));
	return EnumNameString;
}

#endif	// WITH_EDITOR

int64 UUserDefinedEnum::ResolveEnumerator(FArchive& Ar, int64 EnumeratorValue) const
{
#if WITH_EDITOR
	return FEnumEditorUtils::ResolveEnumerator(this, Ar, EnumeratorValue);
#else // WITH_EDITOR
	ensure(false);
	return EnumeratorValue;
#endif // WITH_EDITOR
}

FText UUserDefinedEnum::GetDisplayNameTextByIndex(int32 InIndex) const
{
	const FName EnumEntryName = *GetNameStringByIndex(InIndex);

	const FText* EnumEntryDisplayName = DisplayNameMap.Find(EnumEntryName);
	if (EnumEntryDisplayName)
	{
		return *EnumEntryDisplayName;
	}

	return Super::GetDisplayNameTextByIndex(InIndex);
}

FString UUserDefinedEnum::GetAuthoredNameStringByIndex(int32 InIndex) const
{
	const FName EnumEntryName = *GetNameStringByIndex(InIndex);

	if (const FText* EnumEntryDisplayName = DisplayNameMap.Find(EnumEntryName))
	{
		if (const FString* SourceString = FTextInspector::GetSourceString(*EnumEntryDisplayName))
		{
			return *SourceString;
		}
	}

	return Super::GetAuthoredNameStringByIndex(InIndex);
}

bool UUserDefinedEnum::SetEnums(TArray<TPair<FName, int64>>& InNames, ECppForm InCppForm, EUnderlyingType InUnderlyingType, EEnumFlags InFlags, EAddMaxKeyIfMissing AddMaxKeyIfMissing)
{
	ensure(AddMaxKeyIfMissing == UEnum::EAddMaxKeyIfMissing::Yes);
	if (Names.Num() > 0)
	{
		RemoveNamesFromPrimaryList();
	}

	ensureMsgf(InCppForm == ECppForm::EnumClass || InNames.Num() < std::numeric_limits<uint8>::max(), TEXT("The number of name exceed what can fit in a FByteProperty."));

	const int64 MaxEnumValue = (InNames.Num() == 0) ? 0 : GetMaxEnumValue(InNames, InFlags) + 1;

	// Add a placeholder to the end of Names for the max element we're going to generate so that we don't need to reallocate
	Names.Empty();
	Names.InitializeWithPlaceholder(InNames, MaxEnumValue);
	CppForm = InCppForm;
	UnderlyingType = InUnderlyingType;
	EnumFlags = InFlags;

	const FString BaseEnumPrefix = GenerateEnumPrefix();
	checkSlow(BaseEnumPrefix.Len());

	const int32 MaxTryNum = 1024;
	for (int32 TryNum = 0; TryNum < MaxTryNum; ++TryNum)
	{
		const FString EnumPrefix = (TryNum == 0) ? BaseEnumPrefix : FString::Printf(TEXT("%s_%d"), *BaseEnumPrefix, TryNum - 1);
		const FName MaxEnumItem = *GenerateFullEnumName(*(EnumPrefix + TEXT("_MAX")));
		const int64 MaxEnumItemIndex = GetValueByName(MaxEnumItem);
		if ((MaxEnumItemIndex == INDEX_NONE) && (LookupEnumName(GetPackage()->GetFName(), MaxEnumItem) == INDEX_NONE))
		{
			Names.UsePlaceholder(MaxEnumItem, MaxEnumValue);
			check(Names.Num() == (InNames.Num() + 1));
			AddNamesToPrimaryList();
			return true;
		}
	}

	Names.DiscardPlaceholder();
	check(Names.Num() == InNames.Num());
	UE_LOGF(LogClass, Error, "Unable to generate enum MAX entry due to name collision. Enum: %ls", *GetPathName());

	return false;
}

#if WITH_EDITORONLY_DATA
TSubclassOf<UEnumCookedMetaData> UUserDefinedEnum::GetCookedMetaDataClass() const
{
	return UEnumCookedMetaData::StaticClass();
}

UEnumCookedMetaData* UUserDefinedEnum::NewCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::NewCookedMetaData<UEnumCookedMetaData>(this, "CookedEnumMetaData", GetCookedMetaDataClass());
	}
	return CachedCookedMetaDataPtr;
}

const UEnumCookedMetaData* UUserDefinedEnum::FindCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::FindCookedMetaData<UEnumCookedMetaData>(this, TEXT("CookedEnumMetaData"));
	}
	return CachedCookedMetaDataPtr;
}

void UUserDefinedEnum::PurgeCookedMetaData()
{
	if (CachedCookedMetaDataPtr)
	{
		CookedMetaDataUtil::PurgeCookedMetaData<UEnumCookedMetaData>(CachedCookedMetaDataPtr);
	}
}
#endif // WITH_EDITORONLY_DATA

