// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Canvas.cpp: Unreal canvas rendering.
=============================================================================*/
#include "Engine/Canvas.h"
#include "CanvasBatchedElementRenderItem.h"
#include "Internationalization/IBreakIterator.h"
#include "SceneView.h"
#include "UnrealEngine.h"
#include "RenderingThread.h"
#include "UObject/Package.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineStats.h"
#include "Materials/MaterialInterface.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineFontServices.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/TextChar.h"
#include "Misc/CoreDelegates.h"
#include "RHIStaticStates.h"
#include "ClearQuad.h"
#include "MeshPassProcessor.h"
#include "Debug/ReporterGraph.h"
#include "Fonts/FontMeasure.h"
#include "EngineModule.h"
#include "CanvasRender.h"
#include "CanvasRendererItem.h"
#include "RenderGraphUtils.h"
#include "TextureResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Canvas)

DEFINE_LOG_CATEGORY_STATIC(LogCanvas, Log, All);


DEFINE_STAT(STAT_Canvas_FlushTime);
DEFINE_STAT(STAT_Canvas_DrawTextureTileTime);
DEFINE_STAT(STAT_Canvas_DrawMaterialTileTime);
DEFINE_STAT(STAT_Canvas_DrawStringTime);
DEFINE_STAT(STAT_Canvas_WordWrappingTime);
DEFINE_STAT(STAT_Canvas_GetBatchElementsTime);
DEFINE_STAT(STAT_Canvas_AddTileRenderTime);
DEFINE_STAT(STAT_Canvas_AddTriangleRenderTime);
DEFINE_STAT(STAT_Canvas_NumBatchesCreated);

namespace DisplayDebugManagerCvars
{
	static int32 StoredDebugMenuOffset = 0;
	static int32 LastKnownColumnCount = 0;
	static int32 MaxColumnsCreated = 0;

	static int32 ColumnOffsetIndex = 0;
	static FAutoConsoleVariableRef DebugManagerColumnOffsetIndex(
		TEXT("DebugManager.ColumnOffsetIndex"),
		ColumnOffsetIndex,
		TEXT("")
	);

	static FAutoConsoleCommandWithWorldAndArgs CmdDebugManagerCycleToNextColumn(
		TEXT("DebugManager.CycleToNextColumn"),
		TEXT("Will move the text over to the next column on the debugmanager"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* World)
			{
				DisplayDebugManagerCvars::ColumnOffsetIndex++;
				if (DisplayDebugManagerCvars::ColumnOffsetIndex > DisplayDebugManagerCvars::MaxColumnsCreated)
				{
					DisplayDebugManagerCvars::ColumnOffsetIndex = DisplayDebugManagerCvars::MaxColumnsCreated;
				}
			})
	);

	static FAutoConsoleCommandWithWorldAndArgs CmdDebugManagerCycleToPreviousColumn(
		TEXT("DebugManager.CycleToPreviousColumn"),
		TEXT("Will move the text over to the previous column on the debugmanager"),
		FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* World)
			{
				DisplayDebugManagerCvars::ColumnOffsetIndex--;
				if (DisplayDebugManagerCvars::ColumnOffsetIndex < 0)
				{
					DisplayDebugManagerCvars::ColumnOffsetIndex = 0;
				}
			})
	);
}

class FRDGCanvasRenderTarget final : public FRenderTarget
{
public:
	FRDGCanvasRenderTarget(FRDGTextureRef InRDGTexture)
		: RDGTexture(InRDGTexture)
	{}

	FIntPoint GetSizeXY() const override
	{
		return RDGTexture->Desc.Extent;
	}

	const FTextureRHIRef& GetRenderTargetTexture() const override
	{
		static FTextureRHIRef NullRef;
		return NullRef;
	}

	FRDGTextureRef GetRenderTargetTexture(FRDGBuilder&) const override
	{
		return RDGTexture;
	}

	FRDGTextureRef RDGTexture;
};

FCanvasWordWrapper::FCanvasWordWrapper()
	: GraphemeBreakIterator(FBreakIterator::CreateCharacterBoundaryIterator())
	, LineBreakIterator(FBreakIterator::CreateLineBreakIterator())
{
}

FTexture* GetTextureForCanvasItem(UTexture* RenderTexture)
{
	// Only assign a valid texture for the canvas item. Proxy textures, from the UTexture implementation, are not valid here.
	// This is because the proxy textures get destroyed on the render thread as soon as the build completes, which could happen
	// before the canvas item renders.
	if (RenderTexture && RenderTexture->GetResource() && !RenderTexture->GetResource()->IsProxy())
	{
		return RenderTexture->GetResource();
	}
	else
	{
		return GWhiteTexture;
	}
}

void FCanvasWordWrapper::Execute(FStringView InString, const FTextSizingParameters& InParameters, TArray<FWrappedStringElement>& OutStrings, FWrappedLineData* const OutWrappedLineData)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_WordWrappingTime);

	FWrappingState WrappingState(InString, InParameters, OutStrings, OutWrappedLineData);
	if (WrappingState.WrappedLineData)
	{
		WrappingState.WrappedLineData->Empty();
	}

	if (!WrappingState.String.IsEmpty())
	{
		GraphemeBreakIterator->SetStringRef(WrappingState.String);
		LineBreakIterator->SetStringRef(WrappingState.String);

		for(int32 i = 0; i < WrappingState.String.Len(); ++i) // Sanity check: Doesn't seem valid to have more lines than code units.
		{	
			if( !ProcessLine(WrappingState) )
			{
				break;
			}
		}
	}
}

bool FCanvasWordWrapper::ProcessLine(FWrappingState& WrappingState)
{
	bool bHasAddedLine = false;
	const int32 StringLength = WrappingState.String.Len();
	if (WrappingState.StartIndex < StringLength)
	{
		int32 BreakIndex = FindFirstMandatoryBreakBetween(WrappingState, StringLength);

		int32 NextStartIndex;
		if (BreakIndex == INDEX_NONE || !DoesSubstringFit(WrappingState, BreakIndex))
		{
			BreakIndex = INDEX_NONE;
			int32 WrapIndex = FindIndexAtOrAfterWrapWidth(WrappingState);

			if (WrapIndex == StringLength)
			{
				BreakIndex = WrapIndex;
			}

			if (BreakIndex <= WrappingState.StartIndex) // No mandatory break.
			{
				BreakIndex = FindLastBreakCandidateBetween(WrappingState.StartIndex, WrapIndex);
			}

			if (BreakIndex <= WrappingState.StartIndex) // No candidate break.
			{
				// Break after minimum length that would preserve the meaning/appearance.
				BreakIndex = FindEndOfLastWholeGraphemeCluster(WrappingState.StartIndex, WrapIndex);
			}

			if (BreakIndex <= WrappingState.StartIndex) // No complete grapheme cluster.
			{
				BreakIndex = WrapIndex; // Break at wrap.
			}

			// Index for the next search
			NextStartIndex = BreakIndex;
		}
		else
		{
			// Index for the next search
			NextStartIndex = BreakIndex;
			// The index is inclusive of the break - we don't want the break char in the string
			--BreakIndex;			
		}

		while (BreakIndex > 0 && FTextChar::IsWhitespace(WrappingState.String[BreakIndex - 1]))
		{
			--BreakIndex;
		}

		if (WrappingState.StartIndex <= BreakIndex)
		{
			AddLine(WrappingState, BreakIndex);

			bHasAddedLine = true;
		}

		while (NextStartIndex < StringLength && FTextChar::IsWhitespace(WrappingState.String[NextStartIndex]))
		{
			++NextStartIndex;
		}

		if (WrappingState.WrappedLineData)
		{
			WrappingState.WrappedLineData->Emplace(WrappingState.StartIndex, BreakIndex);
		}

		WrappingState.StartIndex = NextStartIndex;
	}
	return bHasAddedLine;
}

bool FCanvasWordWrapper::DoesSubstringFit(FWrappingState& WrappingState, const int32 EndIndex)
{
	FTextSizingParameters MeasureParameters(WrappingState.Parameters);
	int32 Unused;
	UCanvas::MeasureStringInternal( MeasureParameters, WrappingState.String.Mid(WrappingState.StartIndex, EndIndex - WrappingState.StartIndex), 0, UCanvas::ELastCharacterIndexFormat::Unused, Unused );
	return MeasureParameters.DrawXL <= WrappingState.Parameters.DrawXL;
}

int32 FCanvasWordWrapper::FindIndexAtOrAfterWrapWidth(FWrappingState& WrappingState)
{
	FTextSizingParameters MeasureParameters(WrappingState.Parameters);
	int32 Return = INDEX_NONE;
	UCanvas::MeasureStringInternal(MeasureParameters, WrappingState.String.RightChop(WrappingState.StartIndex), WrappingState.Parameters.DrawXL, UCanvas::ELastCharacterIndexFormat::CharacterAtOffset, Return);
	return WrappingState.StartIndex + Return;
}

void FCanvasWordWrapper::AddLine(FWrappingState& WrappingState, const int32 EndIndex)
{
	FTextSizingParameters MeasureParameters(WrappingState.Parameters);
	const FStringView Substring(WrappingState.String.Mid(WrappingState.StartIndex, EndIndex - WrappingState.StartIndex));
	UCanvas::CanvasStringSize(MeasureParameters, Substring);
	FWrappedStringElement Element(FString(Substring), 0.0f, 0.0f);
	Element.LineExtent.X = MeasureParameters.DrawXL;
	Element.LineExtent.Y = MeasureParameters.DrawYL;
	WrappingState.Results.Add(Element);
}

int32 FCanvasWordWrapper::FindFirstMandatoryBreakBetween(FWrappingState& WrappingState, const int32 WrapIndex)
{
	int32 BreakIndex = INDEX_NONE;
	for(int32 i = WrappingState.StartIndex + 1; i < WrapIndex; ++i)
	{
		const TCHAR Previous = WrappingState.String[i - 1];
		if( FChar::IsLinebreak(Previous) ) // Line break occurs *after* linebreak character.
		{
			const TCHAR* const Current = i < WrapIndex ? &WrappingState.String[i] : nullptr;
			if(	Previous != FChar::CarriageReturn || !(Current && *Current == FChar::LineFeed) ) // Line break cannot occur within CR LF pair.
			{
				BreakIndex = i;
				break;
			}
		}
	}
	// If we reached the end of the string we must also check that the last char is not a newline
	if( BreakIndex == INDEX_NONE )
	{
		const TCHAR Previous = WrappingState.String[WrapIndex - 1];
		if( FChar::IsLinebreak(Previous) ) // Line break occurs *after* linebreak character.
		{
			BreakIndex = WrapIndex;
		}
	}
	return BreakIndex;
}

int32 FCanvasWordWrapper::FindLastBreakCandidateBetween(const int32 InStartIndex, const int32 WrapIndex)
{
	int32 BreakIndex = LineBreakIterator->MoveToCandidateBefore(WrapIndex + 1);
	if(BreakIndex < InStartIndex)
	{
		BreakIndex = INDEX_NONE;
	}
	return BreakIndex;
}

int32 FCanvasWordWrapper::FindEndOfLastWholeGraphemeCluster(const int32 InStartIndex, const int32 WrapIndex)
{
	int32 BreakIndex = GraphemeBreakIterator->MoveToCandidateBefore(WrapIndex + 1);
	if(BreakIndex < InStartIndex)
	{
		BreakIndex = INDEX_NONE;
	}
	return BreakIndex;
}

FCanvas* FCanvas::Create(FRDGBuilder& GraphBuilder, FRDGTextureRef InRenderTarget, FHitProxyConsumer* InHitProxyConsumer, const FGameTime& Time, ERHIFeatureLevel::Type InFeatureLevel, float InDPIScale)
{
	return GraphBuilder.AllocObject<FCanvas>(
		GraphBuilder.AllocObject<FRDGCanvasRenderTarget>(InRenderTarget),
		InHitProxyConsumer, Time, InFeatureLevel, InDPIScale);
}

FCanvas::FCanvas(FRenderTarget* InRenderTarget, FHitProxyConsumer* InHitProxyConsumer, UWorld* InWorld, ERHIFeatureLevel::Type InFeatureLevel, ECanvasDrawMode InDrawMode, float InDPIScale)
:	ViewRect(0,0,0,0)
,	ScissorRect(0,0,0,0)
,	RenderTarget(InRenderTarget)
,	HitProxyConsumer(InHitProxyConsumer)
,	Scene(InWorld ? InWorld->Scene : NULL)
,	AllowedModes(0xFFFFFFFF)
,	bRenderTargetDirty(false)
,	FeatureLevel(InFeatureLevel)
,	bUseInternalTexture(false)
,	bWriteDestinationAlpha(false)
,	DrawMode(InDrawMode)
,	DPIScale(InDPIScale)
{
	Construct();

	if (InWorld)
	{
		Time = InWorld->GetTime();
	}
}

FCanvas::FCanvas(FRenderTarget* InRenderTarget,FHitProxyConsumer* InHitProxyConsumer, const FGameTime& InTime, ERHIFeatureLevel::Type InFeatureLevel, float InDPIScale)
:	ViewRect(0,0,0,0)
,	ScissorRect(0,0,0,0)
,	RenderTarget(InRenderTarget)
,	HitProxyConsumer(InHitProxyConsumer)
,	Scene(NULL)
,	AllowedModes(0xFFFFFFFF)
,	bRenderTargetDirty(false)
,	Time(InTime)
,	FeatureLevel(InFeatureLevel)
,	bUseInternalTexture(false)
,	bWriteDestinationAlpha(false)
,	DrawMode(CDM_DeferDrawing)
,	DPIScale(InDPIScale)
{
	Construct();
}

void FCanvas::Construct()
{
	bStereoRendering = false;
	bScaledToRenderTarget = false;

	const FIntPoint RenderTargetSizeXY = RenderTarget ? RenderTarget->GetSizeXY() : FIntPoint(1, 1);

	TransformStack.Emplace( 
		FMatrix( FScaleMatrix(GetDPIScale()) * CalcBaseTransform2D(RenderTargetSizeXY.X, RenderTargetSizeXY.Y) )
		);

	// init alpha to 1
	AlphaModulate=1.0;

	// make sure the LastElementIndex is invalid
	LastElementIndex = INDEX_NONE;

	// init sort key to 0
	PushDepthSortKey(0);
}

void FCanvas::SetBaseTransform(const FMatrix& Transform)
{
	// set the base transform
	if( TransformStack.Num() > 0 )
	{
		TransformStack[0].SetMatrix(Transform);
	}
	else
	{
		TransformStack.Emplace(Transform);
	}
}

FMatrix FCanvas::CalcBaseTransform2D(uint32 ViewSizeX, uint32 ViewSizeY)
{
	// Guard against division by zero.
	ViewSizeX = FMath::Max<uint32>(ViewSizeX, 1.f);
	ViewSizeY = FMath::Max<uint32>(ViewSizeY, 1.f);

	return AdjustProjectionMatrixForRHI(
		FTranslationMatrix(FVector(0, 0, 0)) *
		FMatrix(
			FPlane(	1.0f / (ViewSizeX / 2.0f),	0.0,										0.0f,	0.0f	),
			FPlane(	0.0f,						-1.0f / (ViewSizeY / 2.0f),					0.0f,	0.0f	),
			FPlane(	0.0f,						0.0f,										1.0f,	0.0f	),
			FPlane(	-1.0f,						1.0f,										0.0f,	1.0f	)
			)
		);
}


FMatrix FCanvas::CalcBaseTransform3D(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV, float NearPlane)
{
	FMatrix ViewMat(CalcViewMatrix(ViewSizeX,ViewSizeY,fFOV));
	FMatrix ProjMat(CalcProjectionMatrix(ViewSizeX,ViewSizeY,fFOV,NearPlane));
	return ViewMat * ProjMat;
}


FMatrix FCanvas::CalcViewMatrix(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV)
{
	// convert FOV to randians
	float FOVRad = fFOV * (float)UE_PI / 360.0f;
	// move camera back enough so that the canvas items being rendered are at the same screen extents as regular canvas 2d rendering	
	FTranslationMatrix CamOffsetMat(-FVector(0,0,-FMath::Tan(FOVRad)*ViewSizeX/2));
	// adjust so that canvas items render as if they start at [0,0] upper left corner of screen 
	// and extend to the lower right corner [ViewSizeX,ViewSizeY]. 
	FMatrix OrientCanvasMat(
		FPlane(	1.0f,				0.0f,				0.0f,	0.0f	),
		FPlane(	0.0f,				-1.0f,				0.0f,	0.0f	),
		FPlane(	0.0f,				0.0f,				1.0f,	0.0f	),
		FPlane(	ViewSizeX * -0.5f,	ViewSizeY * 0.5f,	0.0f, 1.0f		)
		);
	return 
		// also apply screen offset to align to pixel centers
		FTranslationMatrix(FVector(0, 0, 0)) * 
		OrientCanvasMat * 
		CamOffsetMat;
}


FMatrix FCanvas::CalcProjectionMatrix(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV, float NearPlane)
{
	// convert FOV to randians
	float FOVRad = fFOV * (float)UE_PI / 360.0f;
	// project based on the FOV and near plane given
	return AdjustProjectionMatrixForRHI(
		FReversedZPerspectiveMatrix(
			FOVRad,
			ViewSizeX,
			ViewSizeY,
			NearPlane
			)
		);
}

bool FCanvasBatchedElementRenderItem::Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas)
{
	checkSlow(Data);
	bool bDirty = false;

	if (Data->BatchedElements.HasPrimsToDraw())
	{
		bDirty = true;

		RenderContext.AddPass(
			RDG_EVENT_NAME("CanvasBatchedElements"),
			[LocalData = Data, DrawRenderState, Canvas](FRHICommandList& RHICmdList)
		{
			// current render target set for the canvas
			const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
			float Gamma = 1.0f / CanvasRenderTarget->GetDisplayGamma();

			// draw batched items
			LocalData->BatchedElements.Draw(
				RHICmdList,
				DrawRenderState,
				Canvas->GetFeatureLevel(),
				FBatchedElements::CreateProxySceneView(LocalData->Transform.GetMatrix(), FIntRect(0, 0, CanvasRenderTarget->GetSizeXY().X, CanvasRenderTarget->GetSizeXY().Y)),
				Canvas->IsHitTesting(),
				Gamma
			);
		});
	}

	if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
	{
		// delete data since we're done rendering it
		RenderContext.DeferredDelete(Data);
		Data = nullptr;
	}

	return bDirty;
}

bool FCanvasBatchedElementRenderItem::Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope)
{	
	checkSlow(Data);
	bool bDirty=false;
	if( Data->BatchedElements.HasPrimsToDraw() )
	{
		bDirty = true;

		// current render target set for the canvas  (eg. an FSlateTextureRenderTarget2DResource)
		const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
		const float Gamma = 1.0f / CanvasRenderTarget->GetDisplayGamma(); // GetDisplayGamma typically == 2.2
		const bool bAllowDeleteOnRender = Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender;
		
		// Render the batched elements.
		struct FBatchedDrawParameters
		{
			FRenderData* RenderData;
			uint32 bHitTesting : 1;
			uint32 ViewportSizeX;
			uint32 ViewportSizeY;
			float DisplayGamma;
			bool bAllowDeleteOnRender;
			ERHIFeatureLevel::Type FeatureLevel;
			EShaderPlatform ShaderPlatform;
		};
		// all the parameters needed for rendering
		FBatchedDrawParameters DrawParameters =
		{
			Data,
			(uint32)(Canvas->IsHitTesting() ? 1 : 0),
			(uint32)CanvasRenderTarget->GetSizeXY().X,
			(uint32)CanvasRenderTarget->GetSizeXY().Y,
			Gamma,
			bAllowDeleteOnRender,
			Canvas->GetFeatureLevel(),
			Canvas->GetShaderPlatform()
		};
		RenderScope.AddPass(
			TEXT("CanvasBatchedElements"),
			[DrawParameters](FRHICommandList& RHICmdList)
		{
			FSceneView SceneView = FBatchedElements::CreateProxySceneView(DrawParameters.RenderData->Transform.GetMatrix(), FIntRect(0, 0, DrawParameters.ViewportSizeX, DrawParameters.ViewportSizeY));

			FMeshPassProcessorRenderState DrawRenderState;

			// disable depth test & writes
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
			DrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());

			// draw batched items
			DrawParameters.RenderData->BatchedElements.Draw(
				RHICmdList,
				DrawRenderState,
				DrawParameters.FeatureLevel,
				SceneView,
				DrawParameters.bHitTesting,
				DrawParameters.DisplayGamma);

			if (DrawParameters.bAllowDeleteOnRender)
			{
				delete DrawParameters.RenderData;
			}
		});

		if (bAllowDeleteOnRender)
		{
			Data = nullptr;
		}
	}
	return bDirty;
}

FCanvasRenderContext::FCanvasRenderContext(FRDGBuilder& InGraphBuilder, const FRenderTarget* InRenderTarget, FIntRect InViewportRect, FIntRect InScissorRect, bool bScaledToRenderTarget, int16 InSliceIndex)
	: GraphBuilder(InGraphBuilder)
	, RenderTarget(InRenderTarget->GetRenderTargetTexture(InGraphBuilder))
	, ViewportRect(InViewportRect)
	, ScissorRect(InScissorRect)
	, SliceIndex(InSliceIndex)
{
	if (ViewportRect.Area() <= 0 || bScaledToRenderTarget)
	{
		ViewportRect = FIntRect(FIntPoint::ZeroValue, RenderTarget->Desc.Extent);
	}
}

FCanvasRenderThreadScope::FCanvasRenderThreadScope(const FCanvas& InCanvas)
	: Canvas(InCanvas)
{
	RenderCommands = new RenderCommandFunctionArray;
}

FCanvasRenderThreadScope::~FCanvasRenderThreadScope()
{
	RenderCommandFunctionArray* RenderCommandArray = RenderCommands;

	ENQUEUE_RENDER_COMMAND(DispatchCanvasRenderCommands)(
		[RenderCommandArray, RenderTarget = Canvas.RenderTarget, ViewRect = Canvas.ViewRect, ScissorRect = Canvas.ScissorRect, bScaledToRenderTarget = Canvas.bScaledToRenderTarget, SliceIndex = Canvas.SliceIndex](FRHICommandListImmediate& RHICmdList) mutable
	{
		GetRendererModule().InitializeSystemTextures(RHICmdList);
		FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("CanvasRenderThreadScope"));
		FCanvasRenderContext RenderContext(GraphBuilder, RenderTarget, ViewRect, ScissorRect, bScaledToRenderTarget, SliceIndex);

		for (uint32 Index = 0, Count = RenderCommandArray->Num(); Index < Count; ++Index)
		{
			(*RenderCommandArray)[Index](RenderContext);
		}

		// MSAA resolve as the last optional render pass
		if (RenderTarget->GetRenderTargetTexture()->GetDesc().IsMultisample())
		{
			FRenderTargetParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
			FRDGTextureRef ResolveTexture = RegisterExternalTexture(GraphBuilder, RenderTarget->GetShaderResourceTexture(), TEXT("MSAAResolveCanvasTexture"));
			PassParameters->RenderTargets[0] = FRenderTargetBinding(
				RenderTarget->GetRenderTargetTexture(GraphBuilder),
				ResolveTexture,
				ERenderTargetLoadAction::ELoad,
				/*MipIndex*/0,
				SliceIndex
			);

			GraphBuilder.AddPass(RDG_EVENT_NAME("MSAAResolveCanvas"), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
		}

		GraphBuilder.Execute();
		delete RenderCommandArray;
	});
}

FCanvas::FCanvasSortElement& FCanvas::GetSortElement(int32 DepthSortKey)
{
	// Optimization to resuse last index so that the more expensive Find() does not
	// need to be called as much.
	if (SortedElements.IsValidIndex(LastElementIndex))
	{
		FCanvasSortElement & LastElement = SortedElements[LastElementIndex];
		if (LastElement.DepthSortKey == DepthSortKey)
		{
			return LastElement;
		}
	}

	// find the FCanvasSortElement array entry based on the sortkey
	int32 ElementIdx = INDEX_NONE;
	int32* ElementIdxFromMap = SortedElementLookupMap.Find(DepthSortKey);
	if( ElementIdxFromMap )
	{
		ElementIdx = *ElementIdxFromMap;
		checkSlow( SortedElements.IsValidIndex(ElementIdx) );
	}	
	// if it doesn't exist then add a new entry (no duplicates allowed)
	else
	{
		SortedElements.Emplace(DepthSortKey);
		ElementIdx = SortedElements.Num()-1;
		// keep track of newly added array index for later lookup
		SortedElementLookupMap.Add( DepthSortKey, ElementIdx );
	}
	LastElementIndex = ElementIdx;
	return SortedElements[ElementIdx];
}


FBatchedElements* FCanvas::GetBatchedElements(EElementType InElementType, FBatchedElementParameters* InBatchedElementParameters, const FTexture* InTexture, ESimpleElementBlendMode InBlendMode, const FDepthFieldGlowInfo& GlowInfo, bool bApplyDPIScale)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_GetBatchElementsTime);

	// get sort element based on the current sort key from top of sort key stack
	FCanvasSortElement& SortElement = FCanvas::GetSortElement(TopDepthSortKey());
	// find a batch to use 
	FCanvasBatchedElementRenderItem* RenderBatch = NULL;
	// get the current transform entry from top of transform stack
	FTransformEntry FinalTransform = TransformStack.Top();

	if (!bApplyDPIScale && GetDPIScale() != 1.0f)
	{
		FinalTransform = FTransformEntry(FScaleMatrix(1/GetDPIScale()) * FinalTransform.GetMatrix());
	}

	// try to use the current top entry in the render batch array
	if( SortElement.RenderBatchArray.Num() > 0 )
	{
		checkSlow( SortElement.RenderBatchArray.Last() );
		RenderBatch = SortElement.RenderBatchArray.Last()->GetCanvasBatchedElementRenderItem();
	}	

	// if a matching entry for this batch doesn't exist then allocate a new entry
	if( RenderBatch == NULL ||		
		!RenderBatch->IsMatch(InBatchedElementParameters, InTexture, InBlendMode, InElementType, FinalTransform, GlowInfo) )
	{
		INC_DWORD_STAT(STAT_Canvas_NumBatchesCreated);

		RenderBatch = new FCanvasBatchedElementRenderItem( InBatchedElementParameters, InTexture, InBlendMode, InElementType, FinalTransform, GlowInfo);
		SortElement.RenderBatchArray.Add(RenderBatch);
	}
	return RenderBatch->GetBatchedElements();
}


void FCanvas::AddTileRenderItem(float X, float Y, float SizeX, float SizeY, float U, float V, float SizeU, float SizeV, const FMaterialRenderProxy* MaterialRenderProxy, FHitProxyId HitProxyId, bool bFreezeTime, FColor InColor)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_AddTileRenderTime);

	// get sort element based on the current sort key from top of sort key stack
	FCanvasSortElement& SortElement = FCanvas::GetSortElement(TopDepthSortKey());
	// find a batch to use 
	FCanvasTileRendererItem* RenderBatch = NULL;
	// get the current transform entry from top of transform stack
	const FTransformEntry& TopTransformEntry = TransformStack.Top();	

	// try to use the current top entry in the render batch array
	if( SortElement.RenderBatchArray.Num() > 0 )
	{
		checkSlow( SortElement.RenderBatchArray.Last() );
		RenderBatch = SortElement.RenderBatchArray.Last()->GetCanvasTileRendererItem();
	}	
	// if a matching entry for this batch doesn't exist then allocate a new entry
	if( RenderBatch == NULL ||		
		!RenderBatch->IsMatch(MaterialRenderProxy,TopTransformEntry) )
	{
		INC_DWORD_STAT(STAT_Canvas_NumBatchesCreated);

		RenderBatch = new FCanvasTileRendererItem(FeatureLevel, MaterialRenderProxy,TopTransformEntry,bFreezeTime );
		SortElement.RenderBatchArray.Add(RenderBatch);
	}
	// add the quad to the tile render batch
	RenderBatch->AddTile( X,Y,SizeX,SizeY,U,V,SizeU,SizeV,HitProxyId,InColor);
}

void FCanvas::AddTriangleRenderItem(const FCanvasUVTri& Tri, const FMaterialRenderProxy* MaterialRenderProxy, FHitProxyId HitProxyId, bool bFreezeTime)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_AddTriangleRenderTime);
	
	// get sort element based on the current sort key from top of sort key stack
	FCanvasSortElement& SortElement = FCanvas::GetSortElement(TopDepthSortKey());
	// find a batch to use 
	FCanvasTriangleRendererItem* RenderBatch = NULL;

	// get the current transform entry from top of transform stack
	const FTransformEntry& TopTransformEntry = TransformStack.Top();
	
	// try to use the current top entry in the render batch array
	if (SortElement.RenderBatchArray.Num() > 0)
	{
		checkSlow(SortElement.RenderBatchArray.Last());
		RenderBatch = SortElement.RenderBatchArray.Last()->GetCanvasTriangleRendererItem();
	}
	// if a matching entry for this batch doesn't exist then allocate a new entry
	if (RenderBatch == nullptr || !RenderBatch->IsMatch(MaterialRenderProxy, TopTransformEntry))
	{
		INC_DWORD_STAT(STAT_Canvas_NumBatchesCreated);
	
		RenderBatch = new FCanvasTriangleRendererItem(FeatureLevel, MaterialRenderProxy, TopTransformEntry, bFreezeTime);
		SortElement.RenderBatchArray.Add(RenderBatch);
	}
	// add the triangle to the triangle render batch
	RenderBatch->AddTriangle(Tri, HitProxyId);
}

FCanvas::~FCanvas()
{
	ClearBatchesToRender();
}


void FCanvas::Flush_RenderThread(FRHICommandListImmediate& RHICmdList, bool bForce)
{
	if (!(AllowedModes & Allow_Flush) && !bForce)
	{
		return;
	}

	// no need to set the render target if we aren't going to draw anything to it!
	if (SortedElements.Num() == 0)
	{
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);
	Flush_RenderThread(GraphBuilder, bForce);
	GraphBuilder.Execute();
}

void FCanvas::Flush_RenderThread(FRDGBuilder& GraphBuilder, bool bForce)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_FlushTime);

	if (!(AllowedModes&Allow_Flush) && !bForce)
	{
		return;
	}

	// current render target set for the canvas
	check(RenderTarget);

	// no need to set the render target if we aren't going to draw anything to it!
	if (SortedElements.Num() == 0)
	{
		return;
	}

	// Update the font cache with new text before elements are drawn
	FEngineFontServices::Get().UpdateCache();

	// FCanvasSortElement compare class
	struct FCompareFCanvasSortElement
	{
		FORCEINLINE bool operator()(const FCanvasSortElement& A, const FCanvasSortElement& B) const
		{
			return B.DepthSortKey < A.DepthSortKey;
		}
	};
	// sort the array of FCanvasSortElement entries so that higher sort keys render first (back-to-front)
	SortedElements.Sort(FCompareFCanvasSortElement());

	RDG_EVENT_SCOPE(GraphBuilder, "CanvasFlush");

	GetRendererModule().InitializeSystemTextures(GraphBuilder.RHICmdList);

	FCanvasRenderContext RenderContext(GraphBuilder, RenderTarget, ViewRect, ScissorRect, bScaledToRenderTarget, SliceIndex);

	// Set the RHI render target.
	if (IsUsingInternalTexture())
	{
		FRDGTextureClearInfo ClearInfo;
		if (RenderContext.GetSliceIndex() != -1)
		{
			ClearInfo.FirstSliceIndex = RenderContext.GetSliceIndex();
			ClearInfo.NumSlices = 1;
		}
		AddClearRenderTargetPass(GraphBuilder, RenderContext.GetRenderTarget(), ClearInfo);
	}

	// Disable depth test & writes
	FMeshPassProcessorRenderState DrawRenderState;
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

	if (ViewRect.Area() <= 0)
	{
		ViewRect = FIntRect(FIntPoint::ZeroValue, RenderTarget->GetSizeXY());
	}

	// iterate over the FCanvasSortElements in sorted order and render all the batched items for each entry
	for (int32 Idx = 0; Idx < SortedElements.Num(); Idx++)
	{
		FCanvasSortElement& SortElement = SortedElements[Idx];
		for (int32 BatchIdx = 0; BatchIdx < SortElement.RenderBatchArray.Num(); BatchIdx++)
		{
			FCanvasBaseRenderItem* RenderItem = SortElement.RenderBatchArray[BatchIdx];
			if (RenderItem)
			{
				// mark current render target as dirty since we are drawing to it
				bRenderTargetDirty |= RenderItem->Render_RenderThread(RenderContext, DrawRenderState, this);
				if (AllowedModes & Allow_DeleteOnRender)
				{
					RenderContext.DeferredDelete(RenderItem);
				}
			}
		}
		if (AllowedModes & Allow_DeleteOnRender)
		{
			SortElement.RenderBatchArray.Empty();
		}
	}
	if (AllowedModes & Allow_DeleteOnRender)
	{
		// empty the array of FCanvasSortElement entries after finished with rendering	
		SortedElements.Empty();
		SortedElementLookupMap.Empty();
		LastElementIndex = INDEX_NONE;
	}
}

void FCanvas::Flush_GameThread(bool bForce)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_FlushTime);

	if( !(AllowedModes&Allow_Flush) && !bForce ) 
	{ 
		return; 
	}

	// current render target set for the canvas
	check(RenderTarget);

	if (!(AllowedModes & Allow_DeleteOnRender))
	{
		ensureMsgf(false, TEXT("Flush_GameThread requires Allow_DeleteOnRender flag to be set."));
	}

	// no need to set the render target if we aren't going to draw anything to it!
	if (SortedElements.Num() == 0)
	{
		return;
	}

	// Update the font cache with new text before elements are drawn
	if (FEngineFontServices::IsInitialized())
	{
		FEngineFontServices::Get().UpdateCache();
	}

	// FCanvasSortElement compare class
	struct FCompareFCanvasSortElement
	{
		FORCEINLINE bool operator()( const FCanvasSortElement& A, const FCanvasSortElement& B ) const
		{
			return B.DepthSortKey < A.DepthSortKey;
		}
	};
	// sort the array of FCanvasSortElement entries so that higher sort keys render first (back-to-front)
	SortedElements.Sort(FCompareFCanvasSortElement());

	// Only create these render commands if we actually have something to draw.
	if(SortedElements.Num() > 0 && !GUsingNullRHI)
	{
		FCanvasRenderThreadScope RenderThreadScope(*this);

		// iterate over the FCanvasSortElements in sorted order and render all the batched items for each entry
		for( int32 Idx=0; Idx < SortedElements.Num(); Idx++ )
		{
			FCanvasSortElement& SortElement = SortedElements[Idx];
			for( int32 BatchIdx=0; BatchIdx < SortElement.RenderBatchArray.Num(); BatchIdx++ )
			{
				FCanvasBaseRenderItem* RenderItem = SortElement.RenderBatchArray[BatchIdx];
				if( RenderItem )
				{
					// mark current render target as dirty since we are drawing to it
					bRenderTargetDirty |= RenderItem->Render_GameThread(this, RenderThreadScope);

					RenderThreadScope.DeferredDelete(RenderItem);
				}
			}
			SortElement.RenderBatchArray.Empty();
		}
	}
	else
	{
		// iterate over the FCanvasSortElements in sorted order and delete all the batched items for each entry
		for (int32 Idx = 0; Idx < SortedElements.Num(); Idx++)
		{
			FCanvasSortElement& SortElement = SortedElements[Idx];
			for (int32 BatchIdx = 0; BatchIdx < SortElement.RenderBatchArray.Num(); BatchIdx++)
			{
				FCanvasBaseRenderItem* RenderItem = SortElement.RenderBatchArray[BatchIdx];
				if (RenderItem)
				{
					delete RenderItem;
				}
			}
			SortElement.RenderBatchArray.Empty();
		}
	}
	
	// empty the array of FCanvasSortElement entries after finished with rendering
	SortedElements.Empty();
	SortedElementLookupMap.Empty();
	LastElementIndex = INDEX_NONE;
}

void FCanvas::PushRelativeTransform(const FMatrix& Transform)
{
	int32 PreviousTopIndex = TransformStack.Num() - 1;
#if 0
	static bool DEBUG_NoRotation=1;
	if( DEBUG_NoRotation )
	{
		FMatrix TransformNoRotation(FMatrix::Identity);
		TransformNoRotation.SetOrigin(Transform.GetOrigin());
		TransformStack.AddItem( FTransformEntry(TransformNoRotation * TransformStack(PreviousTopIndex).GetMatrix()) );
	}
	else
#endif
	{
		TransformStack.Add( FTransformEntry(Transform * TransformStack[PreviousTopIndex].GetMatrix()) );
	}
}

void FCanvas::PushAbsoluteTransform(const FMatrix& Transform) 
{
	if(TransformStack.Num()>0)
	{
		TransformStack.Add(FTransformEntry(Transform * TransformStack[0].GetMatrix()));
	}
	else
	{
		TransformStack.Add(FTransformEntry(Transform));
	}
}

void FCanvas::PopTransform()
{
	TransformStack.Pop();
}

void FCanvas::SetHitProxy(HHitProxy* HitProxy)
{
	// Change the current hit proxy.
	CurrentHitProxy = HitProxy;

	if(HitProxyConsumer && HitProxy)
	{
		// Notify the hit proxy consumer of the new hit proxy.
		HitProxyConsumer->AddHitProxy(HitProxy);
	}
}

bool FCanvas::HasBatchesToRender() const
{
	for( int32 Idx=0; Idx < SortedElements.Num(); Idx++ )
	{
		const FCanvasSortElement& SortElement = SortedElements[Idx];
		for( int32 BatchIdx=0; BatchIdx < SortElement.RenderBatchArray.Num(); BatchIdx++ )
		{
			if( SortElement.RenderBatchArray[BatchIdx] )
			{
				return true;
			}
		}
	}
	return false;
}

void FCanvas::ClearBatchesToRender()
{
	// delete batches from elements entries
	for (FCanvasSortElement& SortElement : SortedElements)
	{
		for (FCanvasBaseRenderItem* RenderItem : SortElement.RenderBatchArray)
		{
			delete RenderItem;
		}
	}

	// empty the array of entries
	SortedElements.Empty();
	SortedElementLookupMap.Empty();
	LastElementIndex = INDEX_NONE;
}

void FCanvas::CopyTransformStack(const FCanvas& Copy)
{ 
	TransformStack = Copy.TransformStack;
}

void FCanvas::SetRenderTarget_GameThread(FRenderTarget* NewRenderTarget)
{
	if( RenderTarget != NewRenderTarget )
	{
		// flush whenever we swap render targets
		if( RenderTarget )
		{
			Flush_GameThread();
		}
		// Change the current render target.
		RenderTarget = NewRenderTarget;
	}
}

void FCanvas::SetRenderTargetRect( const FIntRect& InViewRect )
{
	ViewRect = InViewRect;
}

void FCanvas::SetRenderTargetScissorRect( const FIntRect& InScissorRect )
{
	ScissorRect = InScissorRect;
}

void FCanvas::Clear(const FLinearColor& ClearColor)
{
	FRenderTarget* CanvasRenderTarget = GetRenderTarget();
	ENQUEUE_RENDER_COMMAND(ClearCommand)(
		[ClearColor, CanvasRenderTarget, SliceIndex=SliceIndex](FRHICommandList& RHICmdList)
		{
			SCOPED_DRAW_EVENT(RHICmdList, CanvasClear);
			if (CanvasRenderTarget)
			{
				// possibility for the RTT to be null for nullrhi. In that case we do nothing.
				if (CanvasRenderTarget->GetRenderTargetTexture() && CanvasRenderTarget->GetRenderTargetTexture()->GetClearBinding() == FClearValueBinding(ClearColor))
				{
					// do fast clear
					FRHIRenderPassInfo RPInfo(CanvasRenderTarget->GetRenderTargetTexture(), ERenderTargetActions::Clear_Store, nullptr, /*MipIndex*/0, SliceIndex);
					RHICmdList.Transition(FRHITransitionInfo(CanvasRenderTarget->GetRenderTargetTexture(), ERHIAccess::Unknown, ERHIAccess::RTV));
					RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearCanvas"));
					RHICmdList.EndRenderPass();					
				}
				else if(CanvasRenderTarget->GetRenderTargetTexture())
				{
					FRHIRenderPassInfo RPInfo(CanvasRenderTarget->GetRenderTargetTexture(), ERenderTargetActions::DontLoad_Store, nullptr, /*MipIndex*/0, SliceIndex);
					RHICmdList.Transition(FRHITransitionInfo(CanvasRenderTarget->GetRenderTargetTexture(), ERHIAccess::Unknown, ERHIAccess::RTV));
					RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearCanvas"));
					RHICmdList.SetViewport(0, 0, 0.0f, CanvasRenderTarget->GetSizeXY().X, CanvasRenderTarget->GetSizeXY().Y, 1.0f);
					DrawClearQuad(RHICmdList, ClearColor);
					RHICmdList.EndRenderPass();
				}
			}
			else
			{
				ensureMsgf(0, TEXT("What is the current render target here?"));
				//RHICmdList.ClearColorTexture(CanvasRenderTarget->GetRenderTargetTexture(), ClearColor, FIntRect());
			}
		});
}

void FCanvas::DrawTile(double X, double Y, double SizeX, double SizeY, float U, float V, float SizeU, float SizeV, const FLinearColor& Color,	const FTexture* Texture, bool AlphaBlend)
{
	ESimpleElementBlendMode BlendMode = SE_BLEND_Opaque;
	if (AlphaBlend)
	{
		if (bUseInternalTexture)
		{
			BlendMode = SE_BLEND_TranslucentAlphaOnlyWriteAlpha;
		}
		else
		{
			// If alpha is required, and no internal texture used, replace SE_BLEND_Translucent with SE_BLEND_AlphaBlend for all tiles.
			BlendMode = (bWriteDestinationAlpha ? SE_BLEND_AlphaBlend : SE_BLEND_Translucent);
		}
	}

	DrawTile(X, Y, SizeX, SizeY, U, V, SizeU, SizeV, Color, Texture, BlendMode);
}

void FCanvas::DrawTile(double X, double Y, double SizeX, double SizeY, float U, float V, float SizeU, float SizeV, const FLinearColor& Color, const FTexture* Texture, ESimpleElementBlendMode BlendMode)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_DrawTextureTileTime);

	FCanvasTileItem TileItem(FVector2D(X, Y), Texture ? Texture : GWhiteTexture, FVector2D(SizeX, SizeY), FVector2D(U, V), FVector2D(SizeU, SizeV), Color);
	TileItem.BlendMode = BlendMode;
	DrawItem(TileItem);
}

int32 FCanvas::DrawShadowedString(double StartX, double StartY, FStringView Text, const UFont* Font, const FLinearColor& Color, const FLinearColor& ShadowColor)
{
	const float Z = 1.0f;
	FCanvasTextStringViewItem TextItem( FVector2D( StartX, StartY ), Text, Font, Color);
	// just render text in single pass for distance field drop shadow
	if (Font && Font->ImportOptions.bUseDistanceFieldAlpha)
	{	
		TextItem.BlendMode = SE_BLEND_MaskedDistanceFieldShadowed;
	}
	else
	{
		TextItem.EnableShadow( ShadowColor );
	}
	
	DrawItem( TextItem );
	return TextItem.DrawnSize.Y;	
}

void FCanvas::DrawNGon(const FVector2D& Center, const FColor& Color, int32 NumSides, float Radius)
{
	FCanvasNGonItem NGonItem(Center, FVector2D(Radius, Radius), FMath::Clamp(NumSides, 3, 255), Color);
	DrawItem(NGonItem);
}

int32 FCanvas::DrawShadowedText(double StartX, double StartY, const FText& Text, const UFont* Font, const FLinearColor& Color, const FLinearColor& ShadowColor)
{
	const float Z = 1.0f;
	FCanvasTextItem TextItem( FVector2D( StartX, StartY ), Text, Font, Color );
	// just render text in single pass for distance field drop shadow
	if (Font && Font->ImportOptions.bUseDistanceFieldAlpha)
	{	
		TextItem.BlendMode = SE_BLEND_MaskedDistanceFieldShadowed;
	}
	else
	{
		TextItem.EnableShadow( ShadowColor );
	}

	DrawItem( TextItem );
	return TextItem.DrawnSize.Y;	
}

void FCanvas::WrapString(FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData)
{
	if (!WordWrapper.IsValid())
	{
		WordWrapper = MakeShareable(new FCanvasWordWrapper());
	}
	UCanvas::WrapString( *WordWrapper, Parameters, InCurX, Text, out_Lines, OutWrappedLineData);
}

ENGINE_API void StringSize(const UFont* Font,int32& XL,int32& YL, FStringView Text)
{
	// this functionality has been moved to a static function in UIString
	FTextSizingParameters Parameters(Font,1.f,1.f);
	UCanvas::CanvasStringSize(Parameters, Text);
	XL = FMath::TruncToInt(Parameters.DrawXL);
	YL = FMath::TruncToInt(Parameters.DrawYL);
}

void UCanvas::MeasureStringInternal(FTextSizingParameters& Parameters, FStringView Text, const int32 StopAfterHorizontalOffset, const ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex)
{
	// initialize output so it always makes some sense
	OutLastCharacterIndex = INDEX_NONE;

	Parameters.DrawXL = 0.f;
	Parameters.DrawYL = 0.f;

	if (Parameters.DrawFont)
	{
		// we'll need to use scaling in multiple places, so create a variable to hold it so that if we modify
		// how the scale is calculated we only have to update these two lines
		const float ScaleX = Parameters.Scaling.X;
		const float ScaleY = Parameters.Scaling.Y;

		const float UserCharSpacingX = Parameters.SpacingAdjust.X * ScaleX;
		const float UserCharSpacingY = Parameters.SpacingAdjust.Y * ScaleY;

		const float MaxCharHeight = Parameters.DrawFont->GetMaxCharHeight() * ScaleY;
		// Advance by max height for consistent line spacing (plus any additional user-specified spacing adjustment).
		const float AdvanceHeight = MaxCharHeight + UserCharSpacingY;

		FVector2f MaxSize(ForceInitToZero);
		FVector2f CurrentSize(ForceInitToZero);

		const TCHAR* const BeginPos = Text.GetData();
		const TCHAR* const EndPos = BeginPos + Text.Len();
		const TCHAR* CurrentPos = BeginPos;
		const TCHAR* PrevPos = nullptr;
		for (; CurrentPos < EndPos && *CurrentPos; ++CurrentPos)
		{
			const TCHAR Ch = *CurrentPos;

			if (FChar::IsLinebreak(Ch))
			{
				CurrentSize.X = 0.f;
				CurrentSize.Y += AdvanceHeight;
				PrevPos = nullptr;
				continue;
			}

			float CharWidth, CharHeight;
			Parameters.DrawFont->GetCharSize(Ch, CharWidth, CharHeight);
			CharWidth *= ScaleX;
			CharHeight *= ScaleY;

			float CharKerning = 0.f;
			if (PrevPos)
			{
				CharKerning = Parameters.DrawFont->GetCharKerning(*PrevPos, Ch) * ScaleX;
			}

			const float AdvanceWidth = CharWidth + CharKerning + UserCharSpacingX;

			CurrentSize.X += AdvanceWidth;
			MaxSize.X = FMath::Max(MaxSize.X, CurrentSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, CurrentSize.Y + CharHeight + UserCharSpacingY);

			// Were we asked to stop measuring after the specified horizontal offset in pixels?
			if (StopAfterHorizontalOffset != INDEX_NONE)
			{
				if (CharIndexFormat == ELastCharacterIndexFormat::CharacterAtOffset)
				{
					// Round our test toward the character's center position
					const float CharHorizontalOffset = Parameters.DrawFont->GetCharHorizontalOffset(Ch) * ScaleX;
					const float TotalCharWidth = AdvanceWidth + CharHorizontalOffset;
					if (StopAfterHorizontalOffset < CurrentSize.X - TotalCharWidth / 2)
					{
						// We've reached the stopping point, so bail
						break;
					}
				}
				else if (CharIndexFormat == ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset)
				{
					if (StopAfterHorizontalOffset < CurrentSize.X)
					{
						--CurrentPos;
						// We've reached the stopping point, so bail
						break;
					}
				}
			}

			PrevPos = CurrentPos;
		}

		Parameters.DrawXL = MaxSize.X;
		Parameters.DrawYL = MaxSize.Y;

		if (CharIndexFormat == ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset)
		{
			// LastWholeCharacterBeforeOffset is an inclusive index
			if (CurrentPos < BeginPos)
			{
				CurrentPos = BeginPos;
			}
			else if (CurrentPos >= EndPos)
			{
				CurrentPos = EndPos - 1;
			}
		}

		OutLastCharacterIndex = CurrentPos - BeginPos;
	}
}

void UCanvas::MeasureStringInternal(FTextSizingParameters& Parameters, const TCHAR* const Text, const int32 TextLength, const int32 StopAfterHorizontalOffset, const ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex)
{
	MeasureStringInternal(Parameters, MakeStringView(Text, TextLength), StopAfterHorizontalOffset, CharIndexFormat, OutLastCharacterIndex);
}

void UCanvas::CanvasStringSize(FTextSizingParameters& Parameters, FStringView Text)
{
	int32 Unused;
	MeasureStringInternal(Parameters, Text, 0, ELastCharacterIndexFormat::Unused, Unused);
}

void UCanvas::WrapString( FCanvasWordWrapper& Wrapper, FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData)
{
	if (Parameters.DrawFont)
	{
		Wrapper.Execute(Text, Parameters, out_Lines, OutWrappedLineData);
	}
	else
	{
		UE_LOGF(LogCanvas, Warning, "UCanvas::WrapString() called with a NULL Font!");
	}
}

void UCanvas::WrapString( FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData)
{
	Canvas->WrapString(Parameters, InCurX, Text, out_Lines, OutWrappedLineData);
}

/*-----------------------------------------------------------------------------
	UCanvas object functions.
-----------------------------------------------------------------------------*/

UCanvas::UCanvas(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> WhiteSquareTexture;
		ConstructorHelpers::FObjectFinder<UPackage> WhiteSquareTextureP;
		ConstructorHelpers::FObjectFinder<UTexture2D> GradientTexture0;
		ConstructorHelpers::FObjectFinder<UPackage> GradientTexture0P;
		FConstructorStatics()
			: WhiteSquareTexture(TEXT("/Engine/EngineResources/WhiteSquareTexture"))
			, WhiteSquareTextureP(TEXT("/Engine/EngineResources/WhiteSquareTexture."))
			, GradientTexture0(TEXT("/Engine/EngineResources/GradientTexture0"))
			, GradientTexture0P(TEXT("/Engine/EngineResources/GradientTexture0."))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	DrawColor = FColor(127, 127, 127, 255);
	ColorModulate = FPlane(1.0f, 1.0f, 1.0f, 1.0f);

	DefaultTexture = ConstructorStatics.WhiteSquareTexture.Object;
	GradientTexture0 = ConstructorStatics.GradientTexture0.Object;
	HmdOrientation = FQuat::Identity;
	ViewProjectionMatrix = FMatrix::Identity;

	UnsafeSizeX = 0;
	UnsafeSizeY = 0;
	SafeZonePadX = 0;
	SafeZonePadY = 0;
	SafeZonePadEX = 0;
	SafeZonePadEY = 0;
	CachedDisplayWidth = 0;
	CachedDisplayHeight = 0;

	SceneView = nullptr;
	Canvas = nullptr;

	// only call once on construction.  Expensive on some platforms (occulus).
	// Init gets called every frame.	
	UpdateSafeZoneData();

	FCoreDelegates::OnSafeFrameChangedEvent.AddUObject(this, &UCanvas::UpdateSafeZoneData);
}

void UCanvas::Init(int32 InSizeX, int32 InSizeY, FSceneView* InSceneView, FCanvas* InCanvas)
{
	HmdOrientation = FQuat::Identity;
	SizeX = InSizeX;
	SizeY = InSizeY;
	UnsafeSizeX = SizeX;
	UnsafeSizeY = SizeY;
	SceneView = InSceneView;
	Canvas = InCanvas;
	
	Update();
}


void UCanvas::BeginDestroy()
{
	Super::BeginDestroy();
	FCoreDelegates::OnSafeFrameChangedEvent.RemoveAll(this);
}


void UCanvas::ApplySafeZoneTransform()
{
	// if there is no required safezone padding, then we can bail on the whole operation.
	// this is also basically punting on a bug we have on PC where GetDisplayMetrics isn't the right thing
	// to do, as it gives the monitor size, not the window size which we actually want.
	if (SafeZonePadX == 0 && SafeZonePadY == 0 && SafeZonePadEX == 0 && SafeZonePadEY == 0)
	{
		return;
	}

	//The basic idea behind this type of safezone handling is that we shrink the canvas to only the safe size, then we apply a transform
	//to place the canvas such that the all the safezone space is empty.  Another method that was attempted was to modify OrgXY and ClipXY,
	//however given the usage of those members inside and OUTSIDE of the canvas system it was deemed not safe at this time.  For example, adding 100
	//to OrgX, might have the effect of actually stretching out some HUD elements.	

	//We must account for the view's position and the existing origin when applying the safe zone.  A view might already be safe and not require or desire having its components
	//shoved around.  e.g.  The top viewport in horizontal splitscreen doesn't need its bottom adjusted as that is already in the middle of the screen.	
	int32 ViewOrigX = 0;
	int32 ViewOrigY = 0;

	int32 ViewMaxX = SizeX;
	int32 ViewMaxY = SizeY;

	//get the view's location and extents on the screen if we can
	if (SceneView)
	{
		ViewOrigX = SceneView->UnscaledViewRect.Min.X;
		ViewOrigY = SceneView->UnscaledViewRect.Min.Y;

		ViewMaxX = SceneView->UnscaledViewRect.Max.X;
		ViewMaxY = SceneView->UnscaledViewRect.Max.Y;
	}	

	//compute the absolute origin position on the screen.  We may not need to move the origin, or move it only a fraction of the full safe zone value to get it to be safe.
	int32 AbsOrgX = OrgX + ViewOrigX;
	int32 AbsOrgY = OrgY + ViewOrigY;

	//if the origin is already in the safe region, no need to adjust it any further.
	int32 OrgXPad = AbsOrgX >= SafeZonePadX ? 0 : SafeZonePadX - AbsOrgX;
	int32 OrgYPad = AbsOrgY >= SafeZonePadY ? 0 : SafeZonePadY - AbsOrgY;	

	//get the canvas's extents on the screen
	int32 AbsMaxX = AbsOrgX + SizeX;
	int32 AbsMaxY = AbsOrgY + SizeY;

	//get the distance from the canvas edge to the screen border
	int32 DistToXBorder = CachedDisplayWidth - AbsMaxX;
	int32 DistToYBorder = CachedDisplayHeight - AbsMaxY;

	//compute how much more we must move the canvas away from the border to meet safe zone requirements.
	int32 SizeXPad = AbsMaxX <= CachedDisplayWidth - SafeZonePadEX ? 0 : SafeZonePadEX - DistToXBorder;
	int32 SizeYPad = AbsMaxY <= CachedDisplayHeight - SafeZonePadEY ? 0 : SafeZonePadEY - DistToYBorder;

	int32 OrigClipOffsetX = SizeX - ClipX;
	int32 OrigClipOffsetY = SizeY - ClipY;

	//store the original size so we can restore it if the user wants to remove the safezone transform
	UnsafeSizeX = SizeX;
	UnsafeSizeY = SizeY;

	//set the size to the virtual safe size.
	SizeX = SizeX - SizeXPad - OrgXPad;
	SizeY = SizeY - SizeYPad - OrgYPad;

	//adjust clip to be within new bounds by the same absolute amount.  we aren't trying to preserve ratios at the moment.
	ClipX = SizeX - OrigClipOffsetX;
	ClipY = SizeY - OrigClipOffsetY;
	


	Canvas->PushRelativeTransform(FTranslationMatrix(FVector(OrgXPad, OrgYPad, 0)));
}

void UCanvas::PopSafeZoneTransform()
{
	// if there is no required safezone padding, then we can bail on the whole operation.
	// this is also basically punting on a bug we have on PC where GetDisplayMetrics isn't the right thing
	// to do, as it gives the monitor size, not the window size which we actually want.
	if (SafeZonePadX == 0 && SafeZonePadY == 0)
	{
		return;
	}

	Canvas->PopTransform();	

	//put our size and clip back to what they were before applying the safezone.
	int32 OrigClipOffsetX = SizeX - ClipX;
	int32 OrigClipOffsetY = SizeY - ClipY;

	SizeX = UnsafeSizeX;
	SizeY = UnsafeSizeY;
	
	ClipX = SizeX - OrigClipOffsetX;
	ClipY = SizeY - OrigClipOffsetY;
}

void UCanvas::UpdateSafeZoneData()
{
	if(GEngine && GEngine->IsStereoscopic3D())
	{
		FVector2D SafeRegionPercentage = GEngine->StereoRenderingDevice->GetTextSafeRegionBounds();

		CachedDisplayWidth = UnsafeSizeX;
		CachedDisplayHeight = UnsafeSizeY;

		SafeZonePadX = (CachedDisplayWidth - (CachedDisplayWidth * SafeRegionPercentage.X))/2.f;
		SafeZonePadY = (CachedDisplayHeight - (CachedDisplayHeight * SafeRegionPercentage.Y))/2.f;
		SafeZonePadEX = SafeZonePadX;
		SafeZonePadEY = SafeZonePadY;
	}
	else if(FSlateApplication::IsInitialized())
	{
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
 		CachedDisplayWidth = DisplayMetrics.PrimaryDisplayWidth;
 		CachedDisplayHeight = DisplayMetrics.PrimaryDisplayHeight;

#if PLATFORM_DESKTOP
		TSharedPtr<SWindow> Window = FSlateApplication::Get().GetActiveTopLevelWindow();
		if (Window.IsValid())
		{
			FVector2D WindowSize = Window->GetClientSizeInScreen();
			if (ISlateViewport* SlateViewport = Window->GetViewport().Get())
			{
				WindowSize = SlateViewport->GetSize();
			}

			CachedDisplayWidth = WindowSize.X;
			CachedDisplayHeight = WindowSize.Y;
		}
#endif

		const FVector2D EffectiveScreenSize = FVector2D(CachedDisplayWidth, CachedDisplayHeight);
		FMargin SafeZone;
		FSlateApplication::Get().GetSafeZoneSize(/*out*/ SafeZone, EffectiveScreenSize);

		SafeZonePadX = FMath::CeilToInt(SafeZone.Left);
		SafeZonePadY = FMath::CeilToInt(SafeZone.Top);
		SafeZonePadEX = FMath::CeilToInt(SafeZone.Right);
		SafeZonePadEY = FMath::CeilToInt(SafeZone.Bottom);
	}

}

void UCanvas::UpdateAllCanvasSafeZoneData()
{
	for (TObjectIterator<UCanvas> It; It; ++It)
	{
		It->UpdateSafeZoneData();
	}
}

void UCanvas::Update()
{
	// Reset canvas params.
	Reset();

	// Copy size parameters from viewport.
	ClipX = SizeX;
	ClipY = SizeY;

	if (Canvas)
	{
		ClipX /= Canvas->GetDPIScale();
		ClipY /= Canvas->GetDPIScale();

		Canvas->SetParentCanvasSize(FIntPoint(SizeX, SizeY));
	}
}

/*-----------------------------------------------------------------------------
	UCanvas scaled sprites.
-----------------------------------------------------------------------------*/

/** Set DrawColor with a FLinearColor and optional opacity override */
void UCanvas::SetLinearDrawColor(FLinearColor InColor, float OpacityOverride)
{
	DrawColor = InColor.ToFColor(true);

	if( OpacityOverride != -1 )
	{
		DrawColor.A = FMath::Clamp(FMath::TruncToInt(OpacityOverride * 255.0f),0,255);
	}
}

void UCanvas::SetDrawColor(uint8 R, uint8 G, uint8 B, uint8 A)
{
	DrawColor.R = R;
	DrawColor.G = G;
	DrawColor.B = B;
	DrawColor.A = A;
}

void UCanvas::SetDrawColor(FColor const& C)
{
	DrawColor = C;
}

/**
 * Translate EBlendMode into ESimpleElementBlendMode used by tiles
 * 
 * @param BlendMode Normal Unreal blend mode enum
 *
 * @return simple element rendering blend mode enum
 */
ESimpleElementBlendMode FCanvas::BlendToSimpleElementBlend(EBlendMode BlendMode)
{
	switch (BlendMode)
	{
		case BLEND_Opaque:
			return SE_BLEND_Opaque;
		case BLEND_Masked:
			return SE_BLEND_Masked;
		case BLEND_Additive:
			return SE_BLEND_Additive;
		case BLEND_Modulate:
			return SE_BLEND_Modulate;
		case BLEND_AlphaComposite:
			return SE_BLEND_AlphaComposite;
		case BLEND_AlphaHoldout:
			return SE_BLEND_AlphaHoldout;
		case BLEND_Translucent:
		default:
			return SE_BLEND_AlphaBlend; // Same as translucent, but also write a proper alpha value
	};
}


void UCanvas::DrawTile( UTexture* Tex, float X, float Y, float XL, float YL, float U, float V, float UL, float VL, EBlendMode BlendMode )
{
	if ( !Tex ) 
	{
		return;
	}
	float MyClipX = OrgX + ClipX;
	float MyClipY = OrgY + ClipY;
	float w = X + XL > MyClipX ? MyClipX - X : XL;
	float h = Y + YL > MyClipY ? MyClipY - Y : YL;
	if (XL > 0.f &&
		YL > 0.f)
	{
		// here we use the original size of the texture, not the current size (for instance, 
		// PVRTC textures, on some platforms anyway, need to be square), but the script code
		// was written using 0..TexSize coordinates, not 0..1, so to make the 0..1 coords
		// we divide by what the texture was when the script code was written
		// TEXTURE_TODO: This info is no longer stored outside of the Editor. Needed?
		float TexSurfaceWidth= Tex->GetSurfaceWidth();
		float TexSurfaceHeight = Tex->GetSurfaceHeight();

		FCanvasTileItem TileItem( FVector2D( X, Y ), GetTextureForCanvasItem(Tex),  FVector2D( w, h ),  
			FVector2D(U / TexSurfaceWidth, V / TexSurfaceHeight),
			FVector2D(U / TexSurfaceWidth + UL / TexSurfaceWidth * w / XL, V / TexSurfaceHeight + VL / TexSurfaceHeight * h / YL),
			DrawColor );
		TileItem.BlendMode = FCanvas::BlendToSimpleElementBlend( BlendMode );
		Canvas->DrawItem( TileItem );	
	}
}


void UCanvas::ClippedStrLen(const UFont* Font, float ScaleX, float ScaleY, int32& XL, int32& YL, FStringView Text)
{
	XL = 0;
	YL = 0;
	if (Font != NULL)
	{
		FTextSizingParameters Parameters(Font,ScaleX,ScaleY);
		CanvasStringSize(Parameters, Text);

		XL = FMath::TruncToInt(Parameters.DrawXL);
		YL = FMath::TruncToInt(Parameters.DrawYL);
	}
}

void VARARGS UCanvas::WrappedStrLenf( const UFont* Font, float ScaleX, float ScaleY, int32& XL, int32& YL, const TCHAR* Fmt, ... ) 
{
	TCHAR Text[4096];
	GET_TYPED_VARARGS( TCHAR, Text, UE_ARRAY_COUNT(Text), UE_ARRAY_COUNT(Text)-1, Fmt, Fmt );

	FFontRenderInfo Info;
	WrappedPrint( false, 0.0f, 0.0f, XL, YL, Font, ScaleX, ScaleY, false, false, Text, Info ); 
}

float UCanvas::DrawText(const UFont* InFont, const FText& InText, float X, float Y, float XScale, float YScale, const FFontRenderInfo& RenderInfo)
{
	return DrawText(InFont, InText.ToString(), X, Y, XScale, YScale, RenderInfo);
}

float UCanvas::DrawText(const UFont* InFont, FStringView InText, float X, float Y, float XScale, float YScale, const FFontRenderInfo& RenderInfo)
{
	ensure(InFont);
	int32		XL		= 0;
	int32		YL		= 0; 
	// need this call in any case to update YL and XL - one of them will be needed anyway
	WrappedPrint(RenderInfo.bClipText == false, X, Y, XL, YL, InFont, XScale, YScale, bCenterX, bCenterY, InText, RenderInfo);

	if (RenderInfo.bClipText)
	{
		FCanvasTextStringViewItem TextItem(FVector2D(FMath::TruncToFloat(OrgX + X), FMath::TruncToFloat(OrgY + Y)), InText, InFont, DrawColor);
		TextItem.Scale = FVector2D( XScale, YScale ), 
		TextItem.BlendMode = SE_BLEND_Translucent;
		TextItem.FontRenderInfo = RenderInfo;
		Canvas->DrawItem( TextItem );	
	}

	return (float)YL;
}

#if !PLATFORM_TCHAR_IS_UTF8CHAR

float UCanvas::DrawText(const UFont* InFont, FAnsiStringView InText, float X, float Y, float XScale, float YScale, const FFontRenderInfo& RenderInfo)
{
	return DrawText(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), X, Y, XScale, YScale, RenderInfo);
}

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

int32 UCanvas::WrappedPrint(bool Draw, float X, float Y, int32& out_XL, int32& out_YL, const UFont* Font, float ScaleX, float ScaleY, bool bCenterTextX, bool bCenterTextY, FStringView Text, const FFontRenderInfo& RenderInfo)
{
	if (ClipX < 0 || ClipY < 0)
	{
		return 0;
	}
	if (Font == NULL)
	{
		UE_LOGF(LogCanvas, Warning, "UCanvas::WrappedPrint() called with a NULL Font!");
		return 0;
	}

	FTextSizingParameters RenderParms(0.f, 0.f, ClipX - (OrgX + X), 0.f, Font );
	RenderParms.Scaling.X = ScaleX;
	RenderParms.Scaling.Y = ScaleY;
	TArray<FWrappedStringElement> WrappedStrings;
	WrapString(RenderParms, 0, Text, WrappedStrings);

	float DrawX = OrgX + X;
	float DrawY = OrgY + Y;
	if (bCenterTextY)
	{
		// Center text about DrawY
		float MeasuredHeight = 0.f;
		for (const FWrappedStringElement& WrappedString : WrappedStrings)
		{
			MeasuredHeight += WrappedString.LineExtent.Y;
		}
		DrawY -= (MeasuredHeight * 0.5f);
	}

	float XL = 0.f;
	float YL = 0.f;
	FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), Font, DrawColor );
	TextItem.Scale = FVector2D( ScaleX, ScaleY );
	TextItem.BlendMode = SE_BLEND_Translucent;
	TextItem.FontRenderInfo = RenderInfo;
	for (const FWrappedStringElement& WrappedString : WrappedStrings)
	{
		float LineDrawX = DrawX;
		float LineDrawY = DrawY;

		if (bCenterTextX)
		{
			// Center text about DrawX
			LineDrawX -= (WrappedString.LineExtent.X * 0.5f);
		}

		float LineXL = 0.0f;
		if( Draw )
		{
			TextItem.Text = FText::FromString(WrappedString.Value);
			Canvas->DrawItem( TextItem, LineDrawX, LineDrawY );
			LineXL = TextItem.DrawnSize.X;
		}
		else
		{
			int32 TempX;
			int32 TempY;
			ClippedStrLen(Font, ScaleX, ScaleY, TempX, TempY, WrappedString.Value);
			LineXL = TempX;
		}
		XL = FMath::Max<float>(XL, LineXL);
		DrawY += Font->GetMaxCharHeight() * ScaleY;
		YL += Font->GetMaxCharHeight() * ScaleY;
	}

	out_XL = FMath::TruncToInt(XL);
	out_YL = FMath::TruncToInt(YL);
	return WrappedStrings.Num();
}

void UCanvas::StrLen(const UFont* InFont, FStringView InText, float& XL, float& YL, bool bDPIAware)
{
	UCanvas::StrLen(InFont, InText, XL, YL, bDPIAware, Canvas);
}
void UCanvas::StrLen(const UFont* InFont, FStringView InText, double& XL, double& YL, bool bDPIAware)
{
	UCanvas::StrLen(InFont, InText, XL, YL, bDPIAware, Canvas);
}

#if !PLATFORM_TCHAR_IS_UTF8CHAR

void UCanvas::StrLen(const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, bool bDPIAware)
{
	return StrLen(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, bDPIAware);
}

void UCanvas::StrLen(const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, bool bDPIAware)
{
	return StrLen(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, bDPIAware);
}

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

void UCanvas::StrLen(const UFont* InFont, FStringView InText, float& XL, float& YL, bool bDPIAware, FCanvas* InCanvas)
{
	if (InFont == NULL)
	{
		UE_LOGF(LogScript, Warning, "No Font");
	}
	else
	{
		if (InFont->FontCacheType == EFontCacheType::Offline || bDPIAware == false)
		{
			FTextSizingParameters Parameters(InFont, 1.0f, 1.0f);
			UCanvas::CanvasStringSize(Parameters, InText);
			XL = Parameters.DrawXL;
			YL = Parameters.DrawYL;
		}
		else
		{
			const FSlateFontInfo LegacyFontInfo = InFont->GetLegacySlateFontInfo();
			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			const FVector2D MeasuredTextSize = FontMeasure->Measure(InText, LegacyFontInfo, InCanvas->GetDPIScale());
			XL = MeasuredTextSize.X;
			YL = MeasuredTextSize.Y;
		}
	}
}

void UCanvas::StrLen(const UFont* InFont, FStringView InText, double& XL, double& YL, bool bDPIAware, FCanvas* InCanvas)
{
	float XLF = static_cast<float>(XL);
	float YLF = static_cast<float>(YL);
	StrLen(InFont, InText, XLF, YLF, bDPIAware, InCanvas);
	XL = XLF;
	YL = YLF;
}

#if !PLATFORM_TCHAR_IS_UTF8CHAR

void UCanvas::StrLen(const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, bool bDPIAware, FCanvas* InCanvas)
{
	return StrLen(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, bDPIAware, InCanvas);
}

void UCanvas::StrLen(const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, bool bDPIAware, FCanvas* InCanvas)
{
	return StrLen(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, bDPIAware, InCanvas);
}

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

void UCanvas::TextSize(const UFont* InFont, FStringView InText, float& XL, float& YL, float ScaleX, float ScaleY)
{
	int32 XLi, YLi;

	if (!InFont)
	{
		UE_LOGF(LogCanvas, Log,  "TextSize: No font" ); 
		return;
	}

	ClippedStrLen(InFont, ScaleX, ScaleY, XLi, YLi, InText);

	XL = XLi;
	YL = YLi;
}
void UCanvas::TextSize(const UFont* InFont, FStringView InText, double& XL, double& YL, double ScaleX, double ScaleY)
{
	float XLF = static_cast<float>(XL);
	float YLF = static_cast<float>(YL);
	TextSize(InFont, InText, XLF, YLF, (float)ScaleX, (float)ScaleY);
	XL = XLF;
	YL = YLF;
}

#if !PLATFORM_TCHAR_IS_UTF8CHAR

void UCanvas::TextSize(const UFont* InFont, FAnsiStringView InText, float& XL, float& YL, float ScaleX, float ScaleY)
{
	return TextSize(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, ScaleX, ScaleY);
}

void UCanvas::TextSize(const UFont* InFont, FAnsiStringView InText, double& XL, double& YL, double ScaleX, double ScaleY)
{
	return TextSize(InFont, StringCast<TCHAR>(InText.GetData(), InText.Len()), XL, YL, ScaleX, ScaleY);
}

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR

FVector UCanvas::Project(FVector Location, bool bClampToZeroPlane) const 
{
	FPlane V(0,0,0,0);

	if (SceneView != NULL)
	{
		Location.DiagnosticCheckNaN();
		V = SceneView->Project(Location);
	}

	FVector resultVec(V);
	resultVec.X = (ClipX/2.f) + (resultVec.X*(ClipX/2.f));
	resultVec.Y *= -1.f * GProjectionSignY;
	resultVec.Y = (ClipY/2.f) + (resultVec.Y*(ClipY/2.f));

	// if behind the screen, clamp depth to the screen
	if (bClampToZeroPlane && V.W <= 0.0f) 
	{
		resultVec.Z = 0.0f;
	}
	return resultVec;
}

void UCanvas::Deproject(FVector2D ScreenPos, /*out*/ FVector& WorldOrigin, /*out*/ FVector& WorldDirection) const
{
	if (SceneView != NULL)
	{
		SceneView->DeprojectFVector2D(ScreenPos, /*out*/ WorldOrigin, /*out*/ WorldDirection);
	}
}


FFontRenderInfo UCanvas::CreateFontRenderInfo(bool bClipText, bool bEnableShadow, FLinearColor GlowColor, FVector2D GlowOuterRadius, FVector2D GlowInnerRadius)
{
	FFontRenderInfo Result;

	Result.bClipText = bClipText;
	Result.bEnableShadow = bEnableShadow;
	Result.GlowInfo.bEnableGlow = (GlowColor.A != 0.0);
	if (Result.GlowInfo.bEnableGlow)
	{
		Result.GlowInfo.GlowOuterRadius = GlowOuterRadius;
		Result.GlowInfo.GlowInnerRadius = GlowInnerRadius;
	}
	return Result;
}

void UCanvas::Reset(bool bKeepOrigin)
{
	if ( !bKeepOrigin )
	{
		OrgX        = GetDefault<UCanvas>(GetClass())->OrgX;
		OrgY        = GetDefault<UCanvas>(GetClass())->OrgY;
	}
	DrawColor   = GetDefault<UCanvas>(GetClass())->DrawColor;
	bCenterX    = false;
	bCenterY    = false;
	bNoSmooth   = false;
}

void UCanvas::SetClip(float X, float Y)
{
	ClipX = X;
	ClipY = Y;
}

FCanvasIcon UCanvas::MakeIcon(UTexture* Texture, float U, float V, float UL, float VL)
{
	FCanvasIcon Icon;
	if (Texture != NULL)
	{
		Icon.Texture = Texture;
		Icon.U = U;
		Icon.V = V;
		Icon.UL = (UL != 0.f ? UL : Texture->GetSurfaceWidth());
		Icon.VL = (VL != 0.f ? VL : Texture->GetSurfaceHeight());
	}
	return Icon;
}


void UCanvas::DrawScaledIcon(FCanvasIcon Icon, float X, float Y, FVector Scale)
{
	if (Icon.Texture != NULL)
	{
		// verify properties are valid
		if (Scale.Size() <= 0.f)
		{
			Scale.X = 1.f;
			Scale.Y = 1.f;
		}
		if (Icon.UL == 0.f)
		{
			Icon.UL = Icon.Texture->GetSurfaceWidth();
		}
		if (Icon.VL == 0.f)
		{
			Icon.VL = Icon.Texture->GetSurfaceHeight();
		}

		// and draw the texture
		DrawTile(Icon.Texture, X, Y, FMath::Abs(Icon.UL) * Scale.X, FMath::Abs(Icon.VL) * Scale.Y, Icon.U, Icon.V, Icon.UL, Icon.VL);
	}
}


void UCanvas::DrawIcon(FCanvasIcon Icon, float X, float Y, float Scale)
{
	if (Icon.Texture != NULL)
	{
		// verify properties are valid
		if (Scale <= 0.f)
		{
			Scale = 1.f;
		}
		if (Icon.UL == 0.f)
		{
			Icon.UL = Icon.Texture->GetSurfaceWidth();
		}
		if (Icon.VL == 0.f)
		{
			Icon.VL = Icon.Texture->GetSurfaceHeight();
		}
		
		// and draw the texture
		DrawTile(Icon.Texture, X, Y, FMath::Abs(Icon.UL) * Scale, FMath::Abs(Icon.VL) * Scale, Icon.U, Icon.V, Icon.UL, Icon.VL );
	}
}


void UCanvas::DrawDebugGraph(const FString& Title, float ValueX, float ValueY, float UL_X, float UL_Y, float W, float H, FVector2D RangeX, FVector2D RangeY)
{
	static const int32 GRAPH_ICONSIZE = 8;

	const int32 X = UL_X + ((RangeX.Y == RangeX.X) ? RangeX.X : (ValueX - RangeX.X) / (RangeX.Y - RangeX.X)) * W - GRAPH_ICONSIZE/2;
	const int32 Y = UL_Y + ((RangeY.Y == RangeY.X) ? RangeY.X : (ValueY - RangeY.X) / (RangeY.Y - RangeY.X)) * H - GRAPH_ICONSIZE/2;
	FCanvasBoxItem Box( FVector2D(  UL_X, UL_Y ), FVector2D(  W, H ) );
	DrawItem( Box );

	FCanvasTileItem Tile( FVector2D( X, Y ), GWhiteTexture, FVector2D( GRAPH_ICONSIZE, GRAPH_ICONSIZE ), FLinearColor::Yellow );
	DrawItem( Tile );

	FCanvasLineItem Line( FVector2D( UL_X, Y ), FVector2D(  UL_X+W, Y ) );
	Line.SetColor( FLinearColor( 0.5f, 0.5f, 0, 0.5f ) );
	DrawItem( Line );
	Line.Origin = FVector( X, UL_Y, 0.0f );
	Line.SetEndPos( FVector2D(  X, UL_Y+H ) );
	DrawItem( Line );

	FString ValText;		
	ValText = FString::Printf(TEXT("%f"), ValueX);
	FCanvasTextItem Text( FVector2D( X, UL_Y+H+16.0f ), FText::FromString( ValText ), GEngine->GetSmallFont(), FLinearColor::Yellow );
	DrawItem( Text );

	ValText = FString::Printf(TEXT("%f"), ValueY);
	Text.Text = FText::FromString( ValText );
	DrawItem( Text, FVector2D( UL_X+W+8.0f, Y ) );

	// title
	if (!Title.IsEmpty())
	{
		Text.Text = FText::FromString( Title );
		DrawItem( Text, FVector2D( UL_X, UL_Y-16.0f) );
	}	
}

void UCanvas::GetCenter(float& outX, float& outY) const
{
	outX = OrgX + ClipX/2;
	outY = OrgY + ClipY/2;
}
void UCanvas::GetCenter(double& outX, double& outY) const
{
	float X, Y;
	GetCenter(X, Y);
	outX = X;
	outY = Y;
}

void UCanvas::DrawItem( class FCanvasItem& Item )
{
	Canvas->DrawItem( Item );
}

void UCanvas::DrawItem( class FCanvasItem& Item , const FVector2D& InPosition )
{
	Canvas->DrawItem( Item, InPosition );
}
	
void UCanvas::DrawItem( class FCanvasItem& Item, float X, float Y )
{
	Canvas->DrawItem( Item, X, Y  );
}

void FCanvas::DrawItem(FCanvasItem& Item)
{

	Item.Draw(this);

	if (DrawMode == CDM_ImmediateDrawing)
	{
		Flush_GameThread();
	}
}

void FCanvas::DrawItem(FCanvasItem& Item, const FVector2D& InPosition)
{
	Item.Draw(this , InPosition);

	if (DrawMode == CDM_ImmediateDrawing)
	{
		Flush_GameThread();
	}
}

void FCanvas::DrawItem(FCanvasItem& Item, float X, float Y)
{
	Item.Draw(this, X, Y);

	if (DrawMode == CDM_ImmediateDrawing)
	{
		Flush_GameThread();
	}
}

EShaderPlatform FCanvas::GetShaderPlatform() const
{
	return GShaderPlatformForFeatureLevel[FeatureLevel];
}

void FCanvas::SetSliceIndex(int16 InSliceIndex)
{
	SliceIndex = InSliceIndex;
}

void FScreenMessageWriter::DrawLine(const FText& Message, int32 X, const FLinearColor& Color)
{
	Canvas.DrawShadowedText((float)X, (float)Y, Message, GetStatsFont(), Color);
	EmptyLine();
}

void UCanvas::SetView(FSceneView* InView)
{
	SceneView = InView;
	if (InView)
	{
		if (GEngine->StereoRenderingDevice.IsValid() && IStereoRendering::IsStereoEyeView(*InView))
		{
			// @todo - this function no longer exists as its effects are handled elsewhere. Keeping this here to document what changed.
			// Maybe the positive side of this 'if' should be removed and we always call ViewmMatrices.GetWorldToClip()?
			//GEngine->StereoRenderingDevice->InitCanvasFromView(InView, this);
		}
		else
		{
			ViewProjectionMatrix = InView->ViewMatrices.GetWorldToClip();
		}
	}
	else
	{
		ViewProjectionMatrix.SetIdentity();
	}
}

TWeakObjectPtr<class UReporterGraph> UCanvas::GetReporterGraph()
{
	if (!ReporterGraph)
	{
		ReporterGraph = NewObject<UReporterGraph>(this);
	}

	return ReporterGraph;
}

void UCanvas::K2_DrawLine(FVector2D ScreenPositionA, FVector2D ScreenPositionB, float Thickness, FLinearColor RenderColor)
{
	if (FMath::Square(ScreenPositionB.X - ScreenPositionA.X) + FMath::Square(ScreenPositionB.Y - ScreenPositionA.Y) > 0 && Canvas)
	{
		FCanvasLineItem LineItem(ScreenPositionA, ScreenPositionB);
		LineItem.LineThickness = Thickness;
		LineItem.SetColor(RenderColor);
		DrawItem(LineItem);
	}
}

void UCanvas::K2_DrawTexture(UTexture* RenderTexture, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize, FLinearColor RenderColor, EBlendMode BlendMode, float Rotation, FVector2D PivotPoint)
{
	if (ScreenSize.X > 0.0f && ScreenSize.Y > 0.0f && Canvas)
	{
		FCanvasTileItem TileItem(ScreenPosition, GetTextureForCanvasItem(RenderTexture), ScreenSize, CoordinatePosition, CoordinatePosition + CoordinateSize, RenderColor);
		TileItem.Rotation = FRotator(0, Rotation, 0);
		TileItem.PivotPoint = PivotPoint;
		TileItem.BlendMode = FCanvas::BlendToSimpleElementBlend(BlendMode);
		DrawItem(TileItem);
	}
}

void UCanvas::K2_DrawMaterial(UMaterialInterface* RenderMaterial, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize, float Rotation, FVector2D PivotPoint)
{
	if (RenderMaterial 
		&& ScreenSize.X > 0.0f 
		&& ScreenSize.Y > 0.0f 
		// Canvas can be NULL if the user tried to draw after EndDrawCanvasToRenderTarget
		&& Canvas)
	{
		// This is a user-facing function, so we'd rather make sure that shaders are ready by the time we render, in order to ensure we don't draw with a fallback material :
		RenderMaterial->EnsureIsComplete();

		// TODO: need to pass a correct vertex declaration for non-MVF platforms
		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			// Should be moved earlier
			FPSOPrecacheParams PSOPrecacheParams;
			PSOPrecacheParams.bCanvasMaterial = true;
			PSOPrecacheParams.BasePassPixelFormat = (Canvas->GetRenderTarget() && Canvas->GetRenderTarget()->GetRenderTargetTexture()) ? Canvas->GetRenderTarget()->GetRenderTargetTexture()->GetDesc().Format : PF_Unknown;
			RenderMaterial->PrecachePSOs(&FLocalVertexFactory::StaticType, PSOPrecacheParams);
		}

		FCanvasTileItem TileItem(ScreenPosition, RenderMaterial->GetRenderProxy(), ScreenSize, CoordinatePosition, CoordinatePosition + CoordinateSize);
		TileItem.Rotation = FRotator(0, Rotation, 0);
		TileItem.PivotPoint = PivotPoint;
		TileItem.SetColor(DrawColor);
		DrawItem(TileItem);
	}
}


void UCanvas::K2_DrawText(UFont* RenderFont, const FString& RenderText, FVector2D ScreenPosition, FVector2D Scale, FLinearColor RenderColor, float Kerning, FLinearColor ShadowColor, FVector2D ShadowOffset, bool bCentreX, bool bCentreY, bool bOutlined, FLinearColor OutlineColor)
{
	if (!RenderText.IsEmpty() && Canvas)
	{
		FCanvasTextItem TextItem(ScreenPosition, FText::FromString(RenderText), RenderFont, RenderColor);
		TextItem.HorizSpacingAdjust = Kerning;
		TextItem.ShadowColor = ShadowColor;
		TextItem.ShadowOffset = ShadowOffset;
		TextItem.bCentreX = bCentreX;
		TextItem.bCentreY = bCentreY;
		TextItem.Scale = Scale;
		TextItem.bOutlined = bOutlined;
		TextItem.OutlineColor = OutlineColor;
		DrawItem(TextItem);
	}
}

void UCanvas::K2_DrawBorder(UTexture* BorderTexture, UTexture* BackgroundTexture, UTexture* LeftBorderTexture, UTexture* RightBorderTexture, UTexture* TopBorderTexture, UTexture* BottomBorderTexture, FVector2D ScreenPosition, FVector2D ScreenSize, FVector2D CoordinatePosition, FVector2D CoordinateSize, FLinearColor RenderColor, FVector2D BorderScale, FVector2D BackgroundScale, float Rotation, FVector2D PivotPoint, FVector2D CornerSize)
{
	if (ScreenSize.X > 0.0f && ScreenSize.Y > 0.0f && BorderTexture && BackgroundTexture && LeftBorderTexture && RightBorderTexture && TopBorderTexture && BottomBorderTexture && Canvas)
	{
		FCanvasBorderItem BorderItem(ScreenPosition, BorderTexture->GetResource(), BackgroundTexture->GetResource(), LeftBorderTexture->GetResource(), RightBorderTexture->GetResource(), TopBorderTexture->GetResource(), BottomBorderTexture->GetResource(), ScreenSize, RenderColor);
		BorderItem.BorderScale = BorderScale;
		BorderItem.BackgroundScale = BackgroundScale;
		BorderItem.BorderUV0 = CoordinatePosition;
		BorderItem.BorderUV1 = CoordinatePosition + CoordinateSize;
		BorderItem.Rotation = FRotator(0, Rotation, 0);
		BorderItem.PivotPoint = PivotPoint;
		BorderItem.CornerSize = CornerSize;
		DrawItem(BorderItem);
	}
}

void UCanvas::K2_DrawBox(FVector2D ScreenPosition, FVector2D ScreenSize, float Thickness, FLinearColor RenderColor)
{
	if (ScreenSize.X > 0.0f && ScreenSize.Y > 0.0f && Canvas)
	{
		FCanvasBoxItem BoxItem(ScreenPosition, ScreenSize);
		BoxItem.LineThickness = Thickness;
		BoxItem.SetColor(RenderColor);
		DrawItem(BoxItem);
	}
}

void UCanvas::K2_DrawTriangle(UTexture* RenderTexture, TArray<FCanvasUVTri> Triangles)
{
	if (Triangles.Num() > 0 && Canvas)
	{
		FCanvasTriangleItem TriangleItem(FVector2D::ZeroVector, FVector2D::ZeroVector, FVector2D::ZeroVector, GetTextureForCanvasItem(RenderTexture));
		TriangleItem.TriangleList = MoveTemp(Triangles);
		DrawItem(TriangleItem);
	}
}

void UCanvas::K2_DrawMaterialTriangle(UMaterialInterface* RenderMaterial, TArray<FCanvasUVTri> Triangles)
{
	if (RenderMaterial && Triangles.Num() > 0 && Canvas)
	{
		// This is a user-facing function, so we'd rather make sure that shaders are ready by the time we render, in order to ensure we don't draw with a fallback material :
		RenderMaterial->EnsureIsComplete();

		FCanvasTriangleItem TriangleItem(FVector2D::ZeroVector, FVector2D::ZeroVector, FVector2D::ZeroVector, NULL);
		TriangleItem.MaterialRenderProxy = RenderMaterial->GetRenderProxy();
		TriangleItem.TriangleList = MoveTemp(Triangles);
		DrawItem(TriangleItem);
	}
}

void UCanvas::K2_DrawPolygon(UTexture* RenderTexture, FVector2D ScreenPosition, FVector2D Radius, int32 NumberOfSides, FLinearColor RenderColor)
{
	if (Radius.X > 0.0f && Radius.Y > 0.0f && NumberOfSides >= 3 && Canvas)
	{
		FCanvasNGonItem NGonItem(ScreenPosition, Radius, NumberOfSides, GetTextureForCanvasItem(RenderTexture), RenderColor);
		DrawItem(NGonItem);
	}
}

FVector UCanvas::K2_Project(FVector WorldLocation)
{
	return Project(WorldLocation);
}

void UCanvas::K2_Deproject(FVector2D ScreenPosition, FVector& WorldOrigin, FVector& WorldDirection)
{
	Deproject(ScreenPosition, WorldOrigin, WorldDirection);
}

FVector2D UCanvas::K2_StrLen(UFont* RenderFont, const FString& RenderText)
{
	if (!RenderText.IsEmpty())
	{
		FVector2f OutTextSize;
		StrLen(RenderFont, RenderText, OutTextSize.X, OutTextSize.Y);
		return (FVector2D)OutTextSize;
	}

	return FVector2D::ZeroVector;
}

FVector2D UCanvas::K2_TextSize(UFont* RenderFont, const FString& RenderText, FVector2D Scale)
{
	if (!RenderText.IsEmpty())
	{
		FVector2D OutTextSize;
		TextSize(RenderFont, RenderText, OutTextSize.X, OutTextSize.Y, Scale.X, Scale.Y);
		return OutTextSize;
	}
	
	return FVector2D::ZeroVector;
}

void FDisplayDebugManager::DrawString(const FString& InDebugString, const float& OptionalXOffset)
{
	if (Canvas)
	{
		const float TextScale = GetTextScale();
		DebugTextItem.Scale = FVector2D(TextScale, TextScale);

		DebugTextItem.Text = FText::FromString(InDebugString);
		Canvas->DrawItem(DebugTextItem, FVector2D((CurrentPos.X + OptionalXOffset) - DisplayDebugManagerCvars::StoredDebugMenuOffset, CurrentPos.Y));

		const float EndOfStringPosX = CurrentPos.X + OptionalXOffset + (DebugTextItem.DrawnSize.X / Canvas->GetDPIScale());

		NextColumXPos = FMath::Max(NextColumXPos, EndOfStringPosX);
		CurrentPos.Y += FMath::Max(GetYStep(), (DebugTextItem.DrawnSize.Y / Canvas->GetDPIScale()));
		AddColumnIfNeeded();
	}
}

float FDisplayDebugManager::GetTextScale() const
{
	return Canvas ? FMath::Max(Canvas->SizeX / (1920.f * Canvas->GetDPIScale()), 1.f) : 1.f;
}

void FDisplayDebugManager::AddColumnIfNeeded()
{
	if (Canvas)
	{
		const float YStep = GetYStep();
		if ((CurrentPos.Y + YStep) > Canvas->ClipY)
		{
			CurrentPos.Y = InitialPos.Y;
			CurrentPos.X = NextColumXPos + YStep * 2.f;

			// Checking and storing the max amount of columns created
			if (ColumnCount == 0)
			{
				DisplayDebugManagerCvars::MaxColumnsCreated = DisplayDebugManagerCvars::LastKnownColumnCount;
			}
			ColumnCount++;
			DisplayDebugManagerCvars::LastKnownColumnCount = ColumnCount;


			// Storing the targeted column offset
			if (DisplayDebugManagerCvars::ColumnOffsetIndex > 0)
			{
				if (ColumnCount == DisplayDebugManagerCvars::ColumnOffsetIndex)
				{
					DisplayDebugManagerCvars::StoredDebugMenuOffset = CurrentPos.X;
				}
			}
			else
			{
				DisplayDebugManagerCvars::StoredDebugMenuOffset = 0;
			}
		}
	}
}