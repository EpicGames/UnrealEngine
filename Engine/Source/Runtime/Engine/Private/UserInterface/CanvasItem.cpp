// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Canvas.cpp: Unreal canvas rendering.
=============================================================================*/

#include "CanvasItem.h"
#include "BatchedElements.h"
#include "EngineStats.h"
#include "Materials/Material.h"
#include "CanvasRendererItem.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineFontServices.h"
#include "Math/RotationMatrix.h"
#include "TextureResource.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Selection.h"
#else
#include "Engine/Engine.h"
#endif

DECLARE_CYCLE_STAT(TEXT("CanvasTileTextureItem Time"),STAT_Canvas_TileTextureItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasTileMaterialItem Time"),STAT_Canvas_TileMaterialItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasTextItem Time"),STAT_Canvas_TextItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasLineItem Time"),STAT_Canvas_LineItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasBoxItem Time"),STAT_Canvas_BoxItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasDashedBoxItem Time"),STAT_Canvas_DashedBoxItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasTriTextureItem Time"),STAT_Canvas_TriTextureItemTime,STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasTriMaterialItem Time"), STAT_Canvas_TriMaterialItemTime, STATGROUP_Canvas);
DECLARE_CYCLE_STAT(TEXT("CanvasBorderItem Time"),STAT_Canvas_BorderItemTime,STATGROUP_Canvas);


#if WITH_EDITOR

FCanvasItemTestbed::LineVars FCanvasItemTestbed::TestLine;
bool FCanvasItemTestbed::bTestState = false;
bool FCanvasItemTestbed::bShowTestbed = false;

bool FCanvasItemTestbed::bShowLines = false;
bool FCanvasItemTestbed::bShowBoxes = false;
bool FCanvasItemTestbed::bShowTris = true;
bool FCanvasItemTestbed::bShowText = false;
bool FCanvasItemTestbed::bShowTiles = false;

FCanvasItemTestbed::FCanvasItemTestbed()
{
	TestMaterial=nullptr;
}

void FCanvasItemTestbed::Draw( class FViewport* Viewport, class FCanvas* Canvas )
{
	bTestState = !bTestState;

	if( !bShowTestbed )
	{
		return;
	}

	if (TestMaterial == nullptr)
	{
		TestMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Game/NewMaterial.NewMaterial"));
	}
	
	// A little ott for a testbed - but I wanted to draw several lines to ensure it worked :)
	if( TestLine.bTestSet == false )
	{
		TestLine.bTestSet = true;
		TestLine.LineStart.X = FMath::FRandRange( 0.0f, Viewport->GetSizeXY().X );
		TestLine.LineStart.Y = FMath::FRandRange( 0.0f, Viewport->GetSizeXY().Y );
		TestLine.LineEnd.X = FMath::FRandRange( 0.0f, Viewport->GetSizeXY().X );
		TestLine.LineEnd.Y  = FMath::FRandRange( 0.0f, Viewport->GetSizeXY().Y );
		TestLine.LineMove.X = FMath::FRandRange( 0.0f, 32.0f );
		TestLine.LineMove.Y = FMath::FRandRange( 0.0f, 32.0f );
		TestLine.LineMove2.X = FMath::FRandRange( 0.0f, 32.0f );
		TestLine.LineMove2.Y = FMath::FRandRange( 0.0f, 32.0f );
	}
	else
	{
		TestLine.LineStart += TestLine.LineMove;
		TestLine.LineEnd += TestLine.LineMove2;
		if( TestLine.LineStart.X < 0 )
		{
			TestLine.LineMove.X = -TestLine.LineMove.X;
		}
		if( TestLine.LineStart.Y < 0 )
		{
			TestLine.LineMove.Y = -TestLine.LineMove.Y;
		}
		if( TestLine.LineEnd.X < 0 )
		{
			TestLine.LineMove2.X = -TestLine.LineMove2.X;
		}
		if( TestLine.LineEnd.Y < 0 )
		{
			TestLine.LineMove2.Y = -TestLine.LineMove2.Y;
		}
		if( TestLine.LineStart.X > Viewport->GetSizeXY().X )
		{
			TestLine.LineMove.X = -TestLine.LineMove.X;
		}
		if( TestLine.LineStart.Y > Viewport->GetSizeXY().Y )
		{
			TestLine.LineMove.Y = -TestLine.LineMove.Y;
		}
		if( TestLine.LineEnd.X > Viewport->GetSizeXY().X )
		{
			TestLine.LineMove2.X = -TestLine.LineMove2.X;
		}
		if( TestLine.LineEnd.Y > Viewport->GetSizeXY().Y )
		{
			TestLine.LineMove2.Y = -TestLine.LineMove2.Y;
		}
	}
	
	// Text
	if( bShowText == true )
	{
		float CenterX = Canvas->GetViewRect().Width() / 2.0f;
		float YTest = 16.0f;
		FCanvasTextItem TextItem(FVector2D(CenterX, YTest), FText::FromString(TEXT("String Here")), GEngine->GetSmallFont(), FLinearColor::Red);
		TextItem.Draw(Canvas);

		// Shadowed text
		TextItem.Position.Y += TextItem.DrawnSize.Y;
		TextItem.Scale.X = 2.0f;
		TextItem.EnableShadow(FLinearColor::Green, FVector2D(2.0f, 2.0f));
		TextItem.Text = FText::FromString(TEXT("Scaled String here"));
		TextItem.Draw(Canvas);
		TextItem.DisableShadow();

		TextItem.Position.Y += TextItem.DrawnSize.Y;;
		TextItem.Text = FText::FromString(TEXT("Centered String Here"));
		TextItem.Scale.X = 1.0f;
		TextItem.bCentreX = true;
		TextItem.Draw(Canvas);

		// Outlined text
		TextItem.Position.Y += TextItem.DrawnSize.Y;
		TextItem.Text = FText::FromString(TEXT("Scaled Centred String here"));
		TextItem.OutlineColor = FLinearColor::Black;
		TextItem.bOutlined = true;
		TextItem.Scale = FVector2D(2.0f, 2.0f);
		TextItem.SetColor(FLinearColor::Green);
		TextItem.Text = FText::FromString(TEXT("Scaled Centred Outlined String here"));
		TextItem.Draw(Canvas);
	}
	
	// a line
	if( bShowLines == true )
	{
		FCanvasLineItem LineItem(TestLine.LineStart, TestLine.LineEnd);
		LineItem.Draw(Canvas);
	}

// 	some boxes
	if( bShowBoxes == true )
	{
		FCanvasBoxItem BoxItem(FVector2D(88.0f, 88.0f), FVector2D(188.0f, 188.0f));
		BoxItem.SetColor(FLinearColor::Yellow);
		BoxItem.Draw(Canvas);

		BoxItem.SetColor(FLinearColor::Red);
		BoxItem.Position = FVector2D(256.0f, 256.0f);
		BoxItem.Draw(Canvas);

		BoxItem.SetColor(FLinearColor::Blue);
		BoxItem.Position = FVector2D(6.0f, 6.0f);
		BoxItem.Size = FVector2D(48.0f, 96.0f);
		BoxItem.Draw(Canvas);

	}
	
	if (bShowTris == true)
	{
		// Triangle
		//FCanvasTriangleItem TriItem(FVector2D(32.0f, 32.0f), FVector2D(64.0f, 32.0f), FVector2D(64.0f, 64.0f), GWhiteTexture);
		//TriItem.Draw(Canvas);

		// Triangle list
		TArray< FCanvasUVTri >	TriangleList;
		FCanvasUVTri SingleTri;
		SingleTri.V0_Pos = FVector2D(128.0f, 128.0f);
		SingleTri.V1_Pos = FVector2D(248.0f, 108.0f);
		SingleTri.V2_Pos = FVector2D(100.0f, 348.0f);
		TriangleList.Add(SingleTri);
		SingleTri.V0_Pos = FVector2D(348.0f, 128.0f);
		SingleTri.V1_Pos = FVector2D(448.0f, 148.0f);
		SingleTri.V2_Pos = FVector2D(438.0f, 308.0f);
		TriangleList.Add(SingleTri);

		//FCanvasTriangleItem TriItemList(TriangleList, GWhiteTexture);
		//TriItemList.SetColor(FLinearColor::Red);
		//TriItemList.Draw( Canvas );

		if (TestMaterial)
		{
			FCanvasTileItem TileItemMat(FVector2D(256.0f, 256.0f), TestMaterial->GetRenderProxy(), FVector2D(128.0f, 128.0f));
			//TileItemMat.Draw(Canvas);

			FCanvasTriangleItem TriItem(FVector2D(512.0f, 256.0f), FVector2D(512.0f, 256.0f), FVector2D(640.0f, 384.0f), FVector2D::ZeroVector, FVector2D(1.0f, 0.0f), FVector2D(1.0f, 1.0f), nullptr);
			TriItem.MaterialRenderProxy = TestMaterial->GetRenderProxy();
			//TriItem.Draw(Canvas);

			SingleTri.V0_Pos = FVector2D(228.0f, 228.0f);
			SingleTri.V1_Pos = FVector2D(348.0f, 208.0f);
			SingleTri.V2_Pos = FVector2D(200.0f, 448.0f);
			SingleTri.V0_UV = FVector2D(0.0f, 0.0f);
			SingleTri.V1_UV = FVector2D(1.0f, 0.0f);
			SingleTri.V2_UV = FVector2D(1.0f, 1.0f);
			TriangleList.Add(SingleTri);
			SingleTri.V0_Pos = FVector2D(448.0f, 228.0f);
			SingleTri.V1_Pos = FVector2D(548.0f, 248.0f);
			SingleTri.V2_Pos = FVector2D(538.0f, 408.0f);
			SingleTri.V0_UV = FVector2D(0.0f, 1.0f);
			SingleTri.V1_UV = FVector2D(0.0f, 0.0f);
			SingleTri.V2_UV = FVector2D(1.0f, 0.0f);
			TriangleList.Add(SingleTri);
			FCanvasTriangleItem TriItemList(TriangleList, nullptr);
			TriItemList.MaterialRenderProxy = TestMaterial->GetRenderProxy();
			TriItemList.Draw(Canvas);
		}
	}

// 	FCanvasNGonItem NGon( FVector2D( 256.0f, 256.0f ), FVector2D( 256.0f, 256.0f ), 6, GWhiteTexture, FLinearColor::White );
// 	NGon.Draw( Canvas );
// 
// 	FCanvasNGonItem NGon2( FVector2D( 488, 666.0f ), FVector2D( 256.0f, 256.0f ), 16, GWhiteTexture, FLinearColor::Green );
// 	NGon2.Draw( Canvas );

	// Texture
	UTexture* SelectedTexture = GEditor->GetSelectedObjects()->GetTop<UTexture>();	
	if( ( SelectedTexture ) && ( bShowTiles == true ))
	{
		// Plain tex
		FCanvasTileItem TileItem( FVector2D( 128.0f,128.0f ), SelectedTexture->GetResource(), FLinearColor::White );
		TileItem.Draw( Canvas );
		TileItem.Size = FVector2D( 32.0f,32.0f );
		TileItem.Position = FVector2D( 16.0f,16.0f );
		TileItem.Draw( Canvas );

		// UV 
		TileItem.Size = FVector2D( 64.0f,64.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( 1.0f, 1.0f );
		TileItem.Position = FVector2D( 256.0f,16.0f );
		TileItem.Draw( Canvas );

		// UV 
		TileItem.Size = FVector2D( 64.0f,64.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( 1.0f, -1.0f );
		TileItem.Position = FVector2D( 356.0f,16.0f );
		TileItem.Draw( Canvas );

		// UV 
		TileItem.Size = FVector2D( 64.0f,64.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( -1.0f, 1.0f );
		TileItem.Position = FVector2D( 456.0f,16.0f );
		TileItem.Draw( Canvas );

		// UV 
		TileItem.Size = FVector2D( 64.0f,64.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( -1.0f, -1.0f );
		TileItem.Position = FVector2D( 556.0f,16.0f );
		TileItem.Draw( Canvas );

		// Rotate top/left pivot
		TileItem.Size = FVector2D( 96.0f,96.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( 1.0f, 1.0f );
		TileItem.Position = FVector2D( 400.0f,264.0f );
		TileItem.Rotation.Yaw = TestLine.Testangle;
		TileItem.Draw( Canvas );

		// Rotate center pivot
		TileItem.Size = FVector2D( 128.0f, 128.0f );
		TileItem.UV0 = FVector2D( 0.0f, 0.0f );
		TileItem.UV1 = FVector2D( 1.0f, 1.0f );
		TileItem.Position = FVector2D( 600.0f,264.0f );
		TileItem.Rotation.Yaw = 360.0f - TestLine.Testangle;
		TileItem.PivotPoint = FVector2D( 0.5f, 0.5f );
		TileItem.Draw( Canvas );

		TestLine.Testangle = FMath::Fmod( TestLine.Testangle + 2.0f, 360.0f );
// 
// 		// textured tri
// 		FCanvasTriangleItem TriItemTex(  FVector2D( 48.0f, 48.0f ), FVector2D( 148.0f, 48.0f ), FVector2D( 48.0f, 148.0f ), FVector2D( 0.0f, 0.0f ), FVector2D( 1.0f, 0.0f ), FVector2D( 0.0f, 1.0f ), SelectedTexture->Resource  );
// 		TriItem.Texture = GWhiteTexture;
// 		TriItemTex.Draw( Canvas );
// 
// 		// moving tri (only 1 point moves !)
// 		TriItemTex.Position = TestLine.LineStart;
// 		TriItemTex.Draw( Canvas );
	}
}

#endif // WITH_EDITOR


FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FTexture* InTexture, const FLinearColor& InColor)
	: FCanvasItem(InPosition)
	, Z(1.0f)
	, UV0(0.0f, 0.0f)
	, UV1(1.0f, 1.0f)
	, Texture(InTexture)
	, MaterialRenderProxy(nullptr)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	SetColor(InColor);
	// Ensure texture is valid.
	check(InTexture);
	Size.X = InTexture->GetSizeX();
	Size.Y = InTexture->GetSizeY();
}

FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InSize, const FLinearColor& InColor)
	: FCanvasItem(InPosition)
	, Size(InSize)
	, Z(1.0f)
	, UV0(0.0f, 0.0f)
	, UV1(1.0f, 1.0f)
	, Texture(InTexture)
	, MaterialRenderProxy(nullptr)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	SetColor(InColor);
	// Ensure texture is valid
	check(InTexture);
}

FCanvasTileItem::FCanvasTileItem( const FVector2D& InPosition, const FVector2D& InSize, const FLinearColor& InColor )
	: FCanvasItem( InPosition )
	, Size( InSize )
	, Z( 1.0f )
	, UV0( 0.0f, 0.0f )
	, UV1( 1.0f, 1.0f )
	, Texture( GWhiteTexture )
	, MaterialRenderProxy( nullptr )
	, Rotation( ForceInitToZero )
	, PivotPoint( FVector2D::ZeroVector )
{		
	SetColor( InColor );
}

FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InUV0, const FVector2D& InUV1, const FLinearColor& InColor)
	: FCanvasItem(InPosition)
	, Z(1.0f)
	, UV0(InUV0)
	, UV1(InUV1)
	, Texture(InTexture)
	, MaterialRenderProxy(nullptr)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	SetColor(InColor);
	// Ensure texture is valid.
	check(InTexture);

	Size.X = InTexture->GetSizeX();
	Size.Y = InTexture->GetSizeY();
}

FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InSize, const FVector2D& InUV0, const FVector2D& InUV1, const FLinearColor& InColor)
	: FCanvasItem(InPosition)
	, Size(InSize)
	, Z(1.0f)
	, UV0(InUV0)
	, UV1(InUV1)
	, Texture(InTexture)
	, MaterialRenderProxy(nullptr)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	SetColor(InColor);
	// Ensure texture is valid.
	check(InTexture != nullptr);
}

FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InSize)
	: FCanvasItem(InPosition)
	, Size(InSize)
	, Z(1.0f)
	, UV0(0.0f, 0.0f)
	, UV1(1.0f, 1.0f)
	, Texture(nullptr)
	, MaterialRenderProxy(InMaterialRenderProxy)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	// Ensure specify Texture or material, but not both.
	check(InMaterialRenderProxy);
}

FCanvasTileItem::FCanvasTileItem(const FVector2D& InPosition, const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InSize, const FVector2D& InUV0, const FVector2D& InUV1)
	: FCanvasItem(InPosition)
	, Size(InSize)
	, Z(1.0f)
	, UV0(InUV0)
	, UV1(InUV1)
	, Texture(nullptr)
	, MaterialRenderProxy(InMaterialRenderProxy)
	, Rotation(ForceInitToZero)
	, PivotPoint(FVector2D::ZeroVector)
{
	// Ensure material proxy is valid.
	check(InMaterialRenderProxy);
}

void FCanvasTileItem::Draw( class FCanvas* InCanvas )
{		
	// Rotate the canvas if the item has rotation
	if( Rotation.IsZero() == false )
	{
		FVector AnchorPos( Size.X * PivotPoint.X, Size.Y * PivotPoint.Y, 0.0f );
		FRotationMatrix RotMatrix( Rotation );
		FMatrix TransformMatrix;
		TransformMatrix = FTranslationMatrix(-AnchorPos) * RotMatrix * FTranslationMatrix(AnchorPos);

		FVector TestPos( Position.X, Position.Y, 0.0f );
		// translate the matrix back to origin, apply the rotation matrix, then transform back to the current position
		FMatrix FinalTransform = FTranslationMatrix(-TestPos) * TransformMatrix * FTranslationMatrix( TestPos );

		InCanvas->PushRelativeTransform(FinalTransform);
	}

	// Draw the item
	if( Texture )
	{
		SCOPE_CYCLE_COUNTER(STAT_Canvas_TileTextureItemTime);

		FLinearColor ActualColor = Color;
		ActualColor.A *= InCanvas->AlphaModulate;
		const FTexture* FinalTexture = Texture;
		FBatchedElements* BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParameters, FinalTexture, BlendMode);
		FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

		// Correct for Depth. This only works because we won't be applying a transform later--otherwise we'd have to adjust the transform instead.
		float Left, Top, Right, Bottom;
		Left =		Position.X * Z;
		Top =		Position.Y * Z;
		Right =		( Position.X + Size.X ) * Z;
		Bottom =	( Position.Y + Size.Y ) * Z;		

		int32 V00 = BatchedElements->AddVertexf(
			FVector4f( Left, Top, 0.0f, Z ),
			FVector2f( UV0.X, UV0.Y ),
			ActualColor,
			HitProxyId );
		int32 V10 = BatchedElements->AddVertexf(
			FVector4f( Right, Top, 0.0f, Z ),
			FVector2f( UV1.X, UV0.Y ),
			ActualColor,
			HitProxyId );
		int32 V01 = BatchedElements->AddVertexf(
			FVector4f( Left, Bottom, 0.0f, Z ),
			FVector2f( UV0.X, UV1.Y ),		
			ActualColor,
			HitProxyId );
		int32 V11 = BatchedElements->AddVertexf(
			FVector4f(Right,	Bottom,	0.0f, Z ),
			FVector2f( UV1.X, UV1.Y ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, FinalTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, FinalTexture, BlendMode );
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_Canvas_TileMaterialItemTime);
		RenderMaterialTile( InCanvas, Position );
	}

	// Restore the canvas transform if we rotated it.
	if( Rotation.IsZero() == false )
	{
		InCanvas->PopTransform();
	}
	

}


void FCanvasTileItem::RenderMaterialTile( class FCanvas* InCanvas, const FVector2D& InPosition )
{
	// get sort element based on the current sort key from top of sort key stack
	FCanvas::FCanvasSortElement& SortElement = InCanvas->GetSortElement(InCanvas->TopDepthSortKey());
	// find a batch to use 
	FCanvasTileRendererItem* RenderBatch = nullptr;
	// get the current transform entry from top of transform stack
	const FCanvas::FTransformEntry& TopTransformEntry = InCanvas->GetTransformStack().Top();	

	// try to use the current top entry in the render batch array
	if( SortElement.RenderBatchArray.Num() > 0 )
	{
		checkSlow( SortElement.RenderBatchArray.Last() );
		RenderBatch = SortElement.RenderBatchArray.Last()->GetCanvasTileRendererItem();
	}	
	// if a matching entry for this batch doesn't exist then allocate a new entry
	if( RenderBatch == nullptr ||		
		!RenderBatch->IsMatch(MaterialRenderProxy,TopTransformEntry) )
	{
		INC_DWORD_STAT(STAT_Canvas_NumBatchesCreated);

		RenderBatch = new FCanvasTileRendererItem(InCanvas->GetFeatureLevel(), MaterialRenderProxy,TopTransformEntry,bFreezeTime );
		SortElement.RenderBatchArray.Add(RenderBatch);
	}
	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();
	// add the quad to the tile render batch
	RenderBatch->AddTile( InPosition.X, InPosition.Y ,Size.X, Size.Y, UV0.X, UV0.Y, UV1.X-UV0.X, UV1.Y-UV0.Y, HitProxyId, Color.ToFColor(true));
}


void FCanvasBorderItem::Draw( class FCanvas* InCanvas )
{		
	// Rotate the canvas if the item has rotation
	if( Rotation.IsZero() == false )
	{
		FVector AnchorPos( Size.X * PivotPoint.X, Size.Y * PivotPoint.Y, 0.0f );
		FRotationMatrix RotMatrix( Rotation );
		FMatrix TransformMatrix;
		TransformMatrix = FTranslationMatrix(-AnchorPos) * RotMatrix * FTranslationMatrix(AnchorPos);

		FVector TestPos( Position.X, Position.Y, 0.0f );
		// translate the matrix back to origin, apply the rotation matrix, then transform back to the current position
		FMatrix FinalTransform = FTranslationMatrix(-TestPos) * TransformMatrix * FTranslationMatrix( TestPos );

		InCanvas->PushRelativeTransform(FinalTransform);
	}

	// Draw the item
	if( BorderTexture )
	{
		SCOPE_CYCLE_COUNTER(STAT_Canvas_BorderItemTime);

		FLinearColor ActualColor = Color;
		ActualColor.A *= InCanvas->AlphaModulate;
		const FTexture* const CornersTexture = BorderTexture;
		const FTexture* const BackTexture = BackgroundTexture ? BackgroundTexture : GWhiteTexture;
		const FTexture* const LeftTexture = BorderLeftTexture ? BorderLeftTexture : GWhiteTexture;
		const FTexture* const RightTexture = BorderRightTexture ? BorderRightTexture : GWhiteTexture;
		const FTexture* const TopTexture = BorderTopTexture ? BorderTopTexture : GWhiteTexture;
		const FTexture* const BottomTexture = BorderBottomTexture ? BorderBottomTexture : GWhiteTexture;
		FBatchedElements* BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParameters, CornersTexture, BlendMode);
		FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

		// Correct for Depth. This only works because we won't be applying a transform later--otherwise we'd have to adjust the transform instead.
		float Left, Top, Right, Bottom;
		Left =		Position.X * Z;
		Top =		Position.Y * Z;
		Right =		( Position.X + Size.X ) * Z;
		Bottom =	( Position.Y + Size.Y ) * Z;		

		const float BorderLeftDrawSizeX = LeftTexture->GetSizeX()*BorderScale.X;
		const float BorderLeftDrawSizeY = LeftTexture->GetSizeY()*BorderScale.Y;
		const float BorderTopDrawSizeX = TopTexture->GetSizeX()*BorderScale.X;
		const float BorderTopDrawSizeY = TopTexture->GetSizeY()*BorderScale.Y;
		const float BorderRightDrawSizeX = RightTexture->GetSizeX()*BorderScale.X;
		const float BorderRightDrawSizeY = RightTexture->GetSizeY()*BorderScale.Y;
		const float BorderBottomDrawSizeX = BottomTexture->GetSizeX()*BorderScale.X;
		const float BorderBottomDrawSizeY = BottomTexture->GetSizeY()*BorderScale.Y;

		const float BackgroundTilingX = (Right-Left)/(BackTexture->GetSizeX()*BackgroundScale.X);
		const float BackgroundTilingY = (Bottom-Top)/(BackTexture->GetSizeY()*BackgroundScale.Y);

		const int32 NumElements = 9; // for 1 background + 4 corners + 4 borders
		BatchedElements->ReserveVertices(4 * NumElements); // 4 verts each

		//Draw background
		int32 V00 = BatchedElements->AddVertexf(
			FVector4f( Left + BorderLeftDrawSizeX, Top + BorderTopDrawSizeY, 0.0f, Z ),
			FVector2f( 0, 0 ),
			ActualColor,
			HitProxyId );
		int32 V10 = BatchedElements->AddVertexf(
			FVector4f( Right - BorderRightDrawSizeX, Top + BorderTopDrawSizeY, 0.0f, Z ),
			FVector2f( BackgroundTilingX, 0 ),
			ActualColor,
			HitProxyId );
		int32 V01 = BatchedElements->AddVertexf(
			FVector4f( Left + BorderLeftDrawSizeX, Bottom - BorderBottomDrawSizeY, 0.0f, Z ),
			FVector2f( 0, BackgroundTilingY ),		
			ActualColor,
			HitProxyId );
		int32 V11 = BatchedElements->AddVertexf(
			FVector4f(Right - BorderRightDrawSizeX, Bottom - BorderBottomDrawSizeY, 0.0f, Z ),
			FVector2f( BackgroundTilingX, BackgroundTilingY ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, BackTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, BackTexture, BlendMode );


		const float BorderTextureWidth = BorderTexture->GetSizeX() * (BorderUV1.X - BorderUV0.X);
		const float BorderTextureHeight = BorderTexture->GetSizeY() * (BorderUV1.Y - BorderUV0.Y);
		const float CornerDrawWidth = BorderTextureWidth * CornerSize.X * BorderScale.X;
		const float CornerDrawHeight = BorderTextureHeight * CornerSize.Y * BorderScale.Y;

		//Top Left Corner
		V00 = BatchedElements->AddVertexf(
			FVector4f( Left, Top, 0.0f, Z ),
			FVector2f( BorderUV0.X, BorderUV0.Y ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Left + CornerDrawWidth, Top, 0.0f, Z ),
			FVector2f( BorderUV1.X*CornerSize.X, BorderUV0.Y ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Left, Top + CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV0.X, BorderUV1.Y*CornerSize.Y ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Left + CornerDrawWidth, Top + CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X*CornerSize.X, BorderUV1.Y*CornerSize.Y ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, CornersTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, CornersTexture, BlendMode );

		// Top Right Corner
		V00 = BatchedElements->AddVertexf(
			FVector4f( Right - CornerDrawWidth, Top, 0.0f, Z ),
			FVector2f( BorderUV1.X - (BorderUV1.X - BorderUV0.X)*CornerSize.X, BorderUV0.Y ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Right, Top, 0.0f, Z ),
			FVector2f( BorderUV1.X, BorderUV0.Y ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Right - CornerDrawWidth, Top + CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X - (BorderUV1.X - BorderUV0.X)*CornerSize.X, BorderUV1.Y*CornerSize.Y ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Right, Top + CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X, BorderUV1.Y*CornerSize.Y ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, CornersTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, CornersTexture, BlendMode );

		//Left Bottom Corner
		V00 = BatchedElements->AddVertexf(
			FVector4f( Left, Bottom - CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV0.X, BorderUV1.Y - (BorderUV1.Y - BorderUV0.Y)*CornerSize.Y),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Left + CornerDrawWidth, Bottom - CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X*CornerSize.X, BorderUV1.Y - (BorderUV1.Y - BorderUV0.Y)*CornerSize.Y ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Left, Bottom, 0.0f, Z ),
			FVector2f( BorderUV0.X, BorderUV1.Y ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Left + CornerDrawWidth, Bottom, 0.0f, Z ),
			FVector2f( BorderUV1.X*CornerSize.X, BorderUV1.Y),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, CornersTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, CornersTexture, BlendMode );

		// Right Bottom Corner
		V00 = BatchedElements->AddVertexf(
			FVector4f( Right - CornerDrawWidth, Bottom - CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X - (BorderUV1.X - BorderUV0.X)*CornerSize.X, BorderUV1.Y - (BorderUV1.Y - BorderUV0.Y)*CornerSize.Y ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Right, Bottom - CornerDrawHeight, 0.0f, Z ),
			FVector2f( BorderUV1.X, BorderUV1.Y - (BorderUV1.Y - BorderUV0.Y)*CornerSize.Y ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Right - CornerDrawWidth, Bottom, 0.0f, Z ),
			FVector2f( BorderUV1.X - (BorderUV1.X - BorderUV0.X)*CornerSize.X, BorderUV1.Y ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Right, Bottom, 0.0f, Z ),
			FVector2f( BorderUV1.X, BorderUV1.Y ),
			ActualColor,
			HitProxyId);
			
		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, CornersTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, CornersTexture, BlendMode );

		const float BorderLeft = Left + CornerDrawWidth;
		const float BorderRight = Right - CornerDrawWidth;
		const float BorderTop = Top + CornerDrawHeight;
		const float BorderBottom = Bottom - CornerDrawHeight;

		//Top Frame Border
		const float TopFrameTilingX = (BorderRight-BorderLeft)/BorderTopDrawSizeX;

		V00 = BatchedElements->AddVertexf(
			FVector4f( BorderLeft, Top, 0.0f, Z ),
			FVector2f( 0, 0 ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( BorderRight, Top, 0.0f, Z ),
			FVector2f( TopFrameTilingX , 0 ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( BorderLeft, Top + BorderTopDrawSizeY, 0.0f, Z ),
			FVector2f( 0, 1.0f ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( BorderRight, Top + BorderTopDrawSizeY, 0.0f, Z ),
			FVector2f( TopFrameTilingX, 1.0f ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, TopTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, TopTexture, BlendMode );

		//Bottom Frame Border
		const float BottomFrameTilingX = (BorderRight-BorderLeft)/BorderBottomDrawSizeX;

		V00 = BatchedElements->AddVertexf(
			FVector4f( BorderLeft, Bottom - BorderBottomDrawSizeY, 0.0f, Z ),
			FVector2f( 0, 0 ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( BorderRight, Bottom - BorderBottomDrawSizeY, 0.0f, Z ),
			FVector2f( BottomFrameTilingX, 0 ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( BorderLeft, Bottom, 0.0f, Z ),
			FVector2f( 0, 1.0f ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( BorderRight, Bottom, 0.0f, Z ),
			FVector2f( BottomFrameTilingX, 1.0f ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, BottomTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, BottomTexture, BlendMode );


		//Left Frame Border
		const float LeftFrameTilingY = (BorderBottom-BorderTop) / BorderLeftDrawSizeY;

		V00 = BatchedElements->AddVertexf(
			FVector4f( Left, BorderTop, 0.0f, Z ),
			FVector2f( 0, 0 ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Left +  BorderLeftDrawSizeX , BorderTop, 0.0f, Z ),
			FVector2f( 1.0f, 0 ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Left, BorderBottom, 0.0f, Z ),
			FVector2f( 0, LeftFrameTilingY ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Left + BorderLeftDrawSizeX, BorderBottom, 0.0f, Z ),
			FVector2f( 1.0f, LeftFrameTilingY ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, LeftTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, LeftTexture, BlendMode );


		//Right Frame Border
		const float RightFrameTilingY = (BorderBottom-BorderTop)/BorderRightDrawSizeY;

		V00 = BatchedElements->AddVertexf(
			FVector4f( Right - BorderRightDrawSizeX, BorderTop, 0.0f, Z ),
			FVector2f( 0, 0 ),
			ActualColor,
			HitProxyId );
		V10 = BatchedElements->AddVertexf(
			FVector4f( Right, BorderTop, 0.0f, Z ),
			FVector2f( 1.0f , 0 ),
			ActualColor,
			HitProxyId );
		V01 = BatchedElements->AddVertexf(
			FVector4f( Right - BorderRightDrawSizeX, BorderBottom, 0.0f, Z ),
			FVector2f( 0, RightFrameTilingY ),		
			ActualColor,
			HitProxyId );
		V11 = BatchedElements->AddVertexf(
			FVector4f( Right, BorderBottom, 0.0f, Z ),
			FVector2f( 1.0f, RightFrameTilingY ),
			ActualColor,
			HitProxyId);

		BatchedElements->AddTriangleExtensive( V00, V10, V11, BatchedElementParameters, RightTexture, BlendMode );
		BatchedElements->AddTriangleExtensive( V00, V11, V01, BatchedElementParameters, RightTexture, BlendMode );

	}

	// Restore the canvas transform if we rotated it.
	if( Rotation.IsZero() == false )
	{
		InCanvas->PopTransform();
	}

}

FCanvasTextItemBase::FCanvasTextItemBase( const FVector2D& InPosition, const FLinearColor& InColor )
	: FCanvasItem( InPosition )
	, HorizSpacingAdjust( 0.0f )
	, Depth( 1.0f )
	, ShadowColor( FLinearColor::Black )
	, ShadowOffset( FVector2D::ZeroVector )
	, DrawnSize( FVector2D::ZeroVector )
	, bCentreX( false )
	, bCentreY( false )
	, bOutlined( false )
	, OutlineColor( FLinearColor::Black )
	, bDontCorrectStereoscopic( true )
	, TileItem( InPosition, FVector2D::ZeroVector, InColor )
{
	SetColor( InColor );
	Scale.Set( 1.0f, 1.0f );
	BlendMode = SE_BLEND_Translucent;
}

FCanvasTextItemBase::~FCanvasTextItemBase() = default;

void FCanvasTextItemBase::Draw( class FCanvas* InCanvas )
{	
	SCOPE_CYCLE_COUNTER(STAT_Canvas_TextItemTime);

	if (InCanvas == nullptr || !HasValidText())
	{
		return;
	}

	bool bHasShadow = FontRenderInfo.bEnableShadow;
	if( bHasShadow && ShadowOffset.SizeSquared() == 0.0f )
	{
		// EnableShadow will set a default ShadowOffset value
		EnableShadow( FLinearColor::Black );
	}

	const bool bWriteAlpha = InCanvas->IsWriteDestinationAlphaSet();
	BlendMode = GetTextBlendMode( bHasShadow );
	if (InCanvas->IsUsingInternalTexture() || bWriteAlpha)
	{
		BlendMode = SE_BLEND_TranslucentAlphaOnlyWriteAlpha;
	}

	const float DPIScale = InCanvas->GetDPIScale();

	FVector2D DrawPos( Position.X * DPIScale, Position.Y * DPIScale);

	// If we are centering the string or we want to fix stereoscopic rendering issues we need to measure the string
	if( ( bCentreX || bCentreY ) || ( !bDontCorrectStereoscopic ) )
	{
		const FVector2D MeasuredTextSize = GetTextSize(DPIScale);
		
		// Calculate the offset if we are centering
		if( bCentreX || bCentreY )
		{		
			// Note we drop the fraction after the length divide or we can end up with coords on 1/2 pixel boundaries
			if( bCentreX )
			{
				DrawPos.X -= (int)( MeasuredTextSize.X / 2 );
			}
			if( bCentreY )
			{
				DrawPos.Y -= (int)( MeasuredTextSize.Y / 2 );
			}
		}

		// Check if we want to correct the stereo3d issues - if we do, render the correction now
		const bool CorrectStereo = !bDontCorrectStereoscopic  && GEngine->IsStereoscopic3D();
		if( CorrectStereo )
		{
			const FVector2D StereoOutlineBoxSize( 2.0f, 2.0f );
			TileItem.MaterialRenderProxy = GEngine->RemoveSurfaceMaterial->GetRenderProxy();
			TileItem.Position = DrawPos - StereoOutlineBoxSize;
			const FVector2D CorrectionSize = MeasuredTextSize + StereoOutlineBoxSize + StereoOutlineBoxSize;
			TileItem.Size = CorrectionSize;
			TileItem.bFreezeTime = true;
			TileItem.Draw( InCanvas );
		}		
	}
	
	TArray<FTextEffect, TFixedAllocator<6>> TextEffects;

	const auto ModulateColor = [InCanvas, this](FLinearColor InColor)
	{
		InColor.A = Color.A;
		// Copy the Alpha from the shadow otherwise if we fade the text the shadow wont fade - which is almost certainly not what we will want.
		InColor.A *= InCanvas->AlphaModulate;
		return InColor;
	};

	// If we have a shadow - draw it now
	if (bHasShadow)
	{
		TextEffects.Add({ FVector2f{ShadowOffset * DPIScale}, ModulateColor(ShadowColor) });
	}

	if( bOutlined )
	{
		const FLinearColor DrawColor = ModulateColor(OutlineColor);
		TextEffects.Add({ FVector2f(-1.0f, -1.0f), OutlineColor });
		TextEffects.Add({ FVector2f(-1.0f, 1.0f), OutlineColor });
		TextEffects.Add({ FVector2f(1.0f, 1.0f), OutlineColor });
		TextEffects.Add({ FVector2f(1.0f, -1.0f), OutlineColor });
	}

	const FLinearColor DrawColor = ModulateColor(Color);
	DrawStringInternal( InCanvas, DrawPos, DrawColor, TextEffects);
}

EFontCacheType FCanvasSimpleTextItem::GetFontCacheType() const
{
	return Font->FontCacheType;
}

// Declaration is pure-virtual, but we provide an implementation to allow checking the validity of the font in derived classes.
bool FCanvasSimpleTextItem::HasValidText() const
{
	return Font != nullptr;
}

ESimpleElementBlendMode FCanvasSimpleTextItem::GetTextBlendMode( const bool bHasShadow ) const
{
	ESimpleElementBlendMode BlendModeToUse = BlendMode;
	if (Font->ImportOptions.bUseDistanceFieldAlpha)
	{
		// convert blend mode to distance field type
		switch(BlendMode)
		{
		case SE_BLEND_Translucent:
			BlendModeToUse = (bHasShadow) ? SE_BLEND_TranslucentDistanceFieldShadowed : SE_BLEND_TranslucentDistanceField;
			break;
		case SE_BLEND_Masked:
			BlendModeToUse = (bHasShadow) ? SE_BLEND_MaskedDistanceFieldShadowed : SE_BLEND_MaskedDistanceField;
			break;
		}
	}
	if (GetFontCacheType() == EFontCacheType::Runtime)
	{
		// The runtime font cache uses an alpha-only texture, so we have to force this blend mode so we use the correct shader
		check(BlendModeToUse == SE_BLEND_Translucent || BlendModeToUse == SE_BLEND_TranslucentAlphaOnly || BlendModeToUse == SE_BLEND_TranslucentAlphaOnlyWriteAlpha);
		BlendModeToUse = SE_BLEND_TranslucentAlphaOnly;
	}
	return BlendModeToUse;
}

FVector2D FCanvasSimpleTextItem::GetTextSizeInternal(FStringView Text, float DPIScale) const
{
	FVector2D MeasuredTextSize = FVector2D::ZeroVector;
	switch( GetFontCacheType() )
	{
	case EFontCacheType::Offline:
		{
			FTextSizingParameters Parameters( Font, Scale.X ,Scale.Y );
			UCanvas::CanvasStringSize( Parameters, Text);
			MeasuredTextSize.X = Parameters.DrawXL;
			MeasuredTextSize.Y = Parameters.DrawYL;
		}
		break;

	case EFontCacheType::Runtime:
		{
			const FSlateFontInfo LegacyFontInfo = (SlateFontInfo.IsSet()) ? SlateFontInfo.GetValue() : Font->GetLegacySlateFontInfo();
			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			MeasuredTextSize = FontMeasure->Measure(Text, LegacyFontInfo, DPIScale) * Scale;
		}
		break;

	default:
		break;
	}
	return MeasuredTextSize;
}

void FCanvasSimpleTextItem::DrawStringInternal(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& InColor, TArrayView<FTextEffect> TextEffects)
{
	switch(GetFontCacheType())
	{
	case EFontCacheType::Offline:
		DrawStringInternal_OfflineCache(InCanvas, Text, DrawPos, InColor, TextEffects);
		break;

	case EFontCacheType::Runtime:
		DrawStringInternal_RuntimeCache(InCanvas, Text, DrawPos, InColor, TextEffects);
		break;

	default:
		break;
	}
}

void FCanvasSimpleTextItem::DrawStringInternal_OfflineCache(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& InColor, TArrayView<FTextEffect> TextEffects)
{
	QUICK_SCOPE_CYCLE_COUNTER(DrawStringInternal_OfflineCache);

	DrawnSize = FVector2D::ZeroVector;

	// Nothing to do if no text
	if (Text.IsEmpty())
	{
		return;
	}

	FVector2D CurrentPos = FVector2D(ForceInitToZero);
	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();
	FTexture* LastTexture = nullptr;
	UTexture* Texture = nullptr;
	FVector2D InvTextureSize(1.0f,1.0f);

	const float CharIncrement = ( (float)Font->Kerning + HorizSpacingAdjust ) * Scale.X;

	// Draw all characters in string.
	const int32 TextLen = Text.Len();
	for( int32 i=0; i < TextLen; i++ )
	{
		int32 Ch = (int32)Font->RemapChar(Text[i]);

		// Skip invalid characters.
		if (!Font->Characters.IsValidIndex(Ch))
		{
			continue;
		}

		const FFontCharacter& Char = Font->Characters[Ch];

		if (DrawnSize.Y == 0)
		{
			// We have a valid character so initialize vertical DrawnSize
			DrawnSize.Y = Font->GetMaxCharHeight() * Scale.Y;
		}

		if (FChar::IsLinebreak(Text[i]))
		{
			// Set current character offset to the beginning of next line.
			CurrentPos.X = 0.0f;
			CurrentPos.Y += Font->GetMaxCharHeight() * Scale.Y;

			// Increase the vertical DrawnSize
			DrawnSize.Y += Font->GetMaxCharHeight() * Scale.Y;

			// Don't draw newline character
			continue;
		}

		if( Font->Textures.IsValidIndex(Char.TextureIndex) && 
			(Texture=Font->Textures[Char.TextureIndex])!=nullptr && 
			Texture->GetResource() != nullptr
#if WITH_EDITOR
			&& !Texture->IsCompiling()
#endif
			)
		{
			if( LastTexture != Texture->GetResource() || BatchedElements == nullptr )
			{
				FBatchedElementParameters* BatchedElementParams = nullptr;
				BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParams, Texture->GetResource(), BlendMode, FontRenderInfo.GlowInfo, false);
				check(BatchedElements != nullptr);
				// Trade-off to use memory for performance by pre-allocating more reserved space 
				// for the triangles/vertices of the batched elements used to render the text tiles
				// Only reserve initial batch, allow growth afterwards in case there are multiple repeated calls.
				// Reserving exactly the added amount each time would essentially force an alloc each time on subsequent calls.
				BatchedElements->ReserveTriangles(TextLen*2,Texture->GetResource(),BlendMode);
				BatchedElements->ReserveVertices(TextLen*4);

				InvTextureSize.X = 1.0f / Texture->GetSurfaceWidth();
				InvTextureSize.Y = 1.0f / Texture->GetSurfaceHeight();
			}
			LastTexture = Texture->GetResource();

			float SizeX			= Char.USize * Scale.X;
			const float SizeY	= Char.VSize * Scale.Y;
			const float U		= Char.StartU * InvTextureSize.X;
			const float V		= Char.StartV * InvTextureSize.Y;
			const float SizeU	= Char.USize * InvTextureSize.X;
			const float SizeV	= Char.VSize * InvTextureSize.Y;				

			const auto AddTriangles = [&](const FVector2f& Offset, const FLinearColor& InColor)
			{
				const float X = CurrentPos.X + (DrawPos.X + Offset.X);
				const float Y = CurrentPos.Y + (DrawPos.Y + Offset.Y) + Char.VerticalOffset * Scale.Y;

				const float Left = X * Depth;
				const float Top = Y * Depth;
				const float Right = (X + SizeX) * Depth;
				const float Bottom = (Y + SizeY) * Depth;

				int32 V00 = BatchedElements->AddVertexf(
					FVector4f(Left, Top, 0.f, Depth),
					FVector2f(U, V),
					InColor,
					HitProxyId);
				int32 V10 = BatchedElements->AddVertexf(
					FVector4f(Right, Top, 0.0f, Depth),
					FVector2f(U + SizeU, V),
					InColor,
					HitProxyId);
				int32 V01 = BatchedElements->AddVertexf(
					FVector4f(Left, Bottom, 0.0f, Depth),
					FVector2f(U, V + SizeV),
					InColor,
					HitProxyId);
				int32 V11 = BatchedElements->AddVertexf(
					FVector4f(Right, Bottom, 0.0f, Depth),
					FVector2f(U + SizeU, V + SizeV),
					InColor,
					HitProxyId);

				BatchedElements->AddTriangle(V00, V10, V11, Texture->GetResource(), BlendMode, FontRenderInfo.GlowInfo);
				BatchedElements->AddTriangle(V00, V11, V01, Texture->GetResource(), BlendMode, FontRenderInfo.GlowInfo);
			};

			// Render all the effects first that appear under the text
			for (const FTextEffect& TextEffect : TextEffects)
			{
				AddTriangles(TextEffect.Offset, TextEffect.Color);
			}

			// The draw the main text
			AddTriangles(FVector2f::ZeroVector, InColor);

			// if we have another non-whitespace character to render, add the font's kerning.
			if (Text.IsValidIndex(i+1) && !FChar::IsWhitespace(Text[i+1]) )
			{
				SizeX += CharIncrement;
			}

			// Update the current rendering position
			CurrentPos.X += SizeX;

			// Increase the Horizontal DrawnSize
			if (CurrentPos.X > DrawnSize.X)
			{
				DrawnSize.X = CurrentPos.X;
			}
		}
	}
}

void FCanvasSimpleTextItem::DrawStringInternal_RuntimeCache(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& InColor, TArrayView<FTextEffect> TextEffects)
{
	QUICK_SCOPE_CYCLE_COUNTER(DrawStringInternal_RuntimeCache);

	const float DPIScale = InCanvas->GetDPIScale();

	DrawnSize = FVector2D::ZeroVector;

	// Nothing to do if no text
	if (Text.IsEmpty())
	{
		return;
	}

	TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();
	if( !FontCache.IsValid() )
	{
		return;
	}

	const float FontScale = DPIScale;
	const FSlateFontInfo LegacyFontInfo = (SlateFontInfo.IsSet()) ? SlateFontInfo.GetValue() : Font->GetLegacySlateFontInfo();
	FCharacterList& CharacterList = FontCache->GetCharacterList( LegacyFontInfo, FontScale );

	const FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

	uint32 FontTextureIndex = 0;
	FTextureResource* FontTexture = nullptr;
	ESimpleElementBlendMode FontTextureBlendMode = BlendMode;
	FLinearColor FontColor = InColor;

	float InvTextureSizeX = 0;
	float InvTextureSizeY = 0;

	FVector2D TopLeft(0,0);

	const float PosX = TopLeft.X * DPIScale;
	float PosY = TopLeft.Y * DPIScale;

	const float ScaledHorizSpacingAdjust = HorizSpacingAdjust * Scale.X;
	const float ScaledMaxHeight = CharacterList.GetMaxHeight() * Scale.Y;

	float LineX = PosX;

	TCHAR PreviousChar = 0;
	const int32 TextLen = Text.Len();
	for (int32 CharIndex = 0; CharIndex < TextLen; ++CharIndex)
	{
		const TCHAR CurrentChar = Text[CharIndex];

		if (DrawnSize.Y == 0)
		{
			// We have a valid character so initialize vertical DrawnSize
			DrawnSize.Y = ScaledMaxHeight;
		}

		const bool IsNewline = (CurrentChar == '\n');

		if (IsNewline)
		{
			// Move down: we are drawing the next line.
			PosY += ScaledMaxHeight;
			// Carriage return 
			LineX = PosX;
			// Increase the vertical DrawnSize
			DrawnSize.Y += ScaledMaxHeight;
		}
		else
		{
			const FCharacterEntry& Entry = CharacterList.GetCharacter(CurrentChar, LegacyFontInfo.FontFallback);

			if (Entry.Valid && (FontTexture == nullptr || Entry.TextureIndex != FontTextureIndex))
			{
				// Font has a new texture for this glyph. Refresh the batch we use and the index we are currently using
				FontTextureIndex = Entry.TextureIndex;

				ISlateFontTexture* SlateFontTexture = FontCache->GetFontTexture(FontTextureIndex);
				check(SlateFontTexture);

				FontTexture = SlateFontTexture->GetEngineTexture();
				check(FontTexture);

				const ESlateFontAtlasContentType ContentType = SlateFontTexture->GetContentType();
				FontTextureBlendMode = ContentType == ESlateFontAtlasContentType::Color ? SE_BLEND_Translucent : BlendMode;
				FontColor = ContentType == ESlateFontAtlasContentType::Color ? FLinearColor::White : InColor;

				FBatchedElementParameters* BatchedElementParams = nullptr;
				BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParams, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo, false);
				check(BatchedElements);

				// Trade-off to use memory for performance by pre-allocating more reserved space 
				// for the triangles/vertices of the batched elements used to render the text tiles.
				// Only reserve initial batch, allow growth afterwards in case there are multiple repeated calls.
				// Reserving exactly the added amount each time would essentially force an alloc each time on subsequent calls.
				{
					const int32 NumTris = 2 * (TextEffects.Num() + 1);
					const int32 NumVerts = 4 * (TextEffects.Num() + 1);
					BatchedElements->ReserveTriangles(TextLen * NumTris, FontTexture, FontTextureBlendMode);
					BatchedElements->ReserveVertices(TextLen * NumVerts);
				}

				InvTextureSizeX = 1.0f / FontTexture->GetSizeX();
				InvTextureSizeY = 1.0f / FontTexture->GetSizeY();
			}

			const bool bIsWhitespace = !Entry.Valid || FChar::IsWhitespace(CurrentChar);

			int32 Kerning = 0;
			if (!bIsWhitespace)
			{
				if (CharIndex > 0)
				{
					// We can't store a pointer to the previous entry as calls to CharacterList.GetCharacter can invalidate previous pointers.
					const FCharacterEntry& PreviousEntry = CharacterList.GetCharacter(PreviousChar, LegacyFontInfo.FontFallback);
					Kerning = CharacterList.GetKerning(PreviousEntry, Entry) * Scale.X;
				}
			}

			LineX += Kerning;

			if (!bIsWhitespace)
			{
				const float InvBitmapRenderScale = 1.0f / Entry.BitmapRenderScale;
				
				const float U = Entry.StartU * InvTextureSizeX;
				const float V = Entry.StartV * InvTextureSizeY;
				const float SizeX = Entry.USize * Scale.X * Entry.BitmapRenderScale;
				const float SizeY = Entry.VSize * Scale.Y * Entry.BitmapRenderScale;
				const float SizeU = Entry.USize * InvTextureSizeX;
				const float SizeV = Entry.VSize * InvTextureSizeY;

				const auto AddTriangles = [&](const FVector2f& Offset, const FLinearColor& InColor)
				{
					// Note PosX,PosY is the upper left corner of the bounding box representing the string.  This computes the Y position of the baseline where text will sit
					const float X = (DrawPos.X + Offset.X) + LineX + (Entry.HorizontalOffset * Scale.X);
					const float Y = (DrawPos.Y + Offset.Y) + PosY - (Entry.VerticalOffset * Scale.Y) + (((Entry.GlobalDescender * Scale.Y) + ScaledMaxHeight) * InvBitmapRenderScale);

					const float Left = X * Depth;
					const float Top = Y * Depth;
					const float Right = (X + SizeX) * Depth;
					const float Bottom = (Y + SizeY) * Depth;

					int32 V00 = BatchedElements->AddVertexf(
						FVector4f(Left, Top, 0.f, Depth),
						FVector2f(U, V),
						InColor,
						HitProxyId);
					int32 V10 = BatchedElements->AddVertexf(
						FVector4f(Right, Top, 0.0f, Depth),
						FVector2f(U + SizeU, V),
						InColor,
						HitProxyId);
					int32 V01 = BatchedElements->AddVertexf(
						FVector4f(Left, Bottom, 0.0f, Depth),
						FVector2f(U, V + SizeV),
						InColor,
						HitProxyId);
					int32 V11 = BatchedElements->AddVertexf(
						FVector4f(Right, Bottom, 0.0f, Depth),
						FVector2f(U + SizeU, V + SizeV),
						InColor,
						HitProxyId);

					BatchedElements->AddTriangle(V00, V10, V11, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo);
					BatchedElements->AddTriangle(V00, V11, V01, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo);
				};

				// Render all the effects first that appear under the text
				for (const FTextEffect& TextEffect : TextEffects)
				{
					AddTriangles(TextEffect.Offset, TextEffect.Color);
				}

				// The draw the main text
				AddTriangles(FVector2f::ZeroVector, FontColor);
			}

			LineX += Entry.XAdvance * Scale.X;
			LineX += ScaledHorizSpacingAdjust;

			// Increase the Horizontal DrawnSize
			if (LineX > DrawnSize.X)
			{
				DrawnSize.X = LineX;
			}

			PreviousChar = CurrentChar;
		}
	}
}

FCanvasTextItem::FCanvasTextItem(const FVector2D& InPosition, const FText& InText, const UFont* InFont, const FLinearColor& InColor)
	: FCanvasSimpleTextItem(InPosition, InFont, InColor)
	, Text(InText)
{
}

FCanvasTextItem::FCanvasTextItem(const FVector2D& InPosition, const FText& InText, const FSlateFontInfo& InFontInfo, const FLinearColor& InColor)
	: FCanvasSimpleTextItem(InPosition, InFontInfo, InColor)
	, Text(InText)
{
}

FCanvasTextItem::~FCanvasTextItem() = default;

bool FCanvasTextItem::HasValidText() const
{
	return FCanvasSimpleTextItem::HasValidText() && !Text.IsEmpty();
}

FVector2D FCanvasTextItem::GetTextSize(float DPIScale) const
{
	return GetTextSizeInternal(Text.ToString(), DPIScale);
}

void FCanvasTextItem::DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects)
{
	FCanvasSimpleTextItem::DrawStringInternal(InCanvas, Text.ToString(), DrawPos, DrawColor, TextEffects);
}

void FCanvasTextItem::DrawStringInternal_OfflineCache(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects)
{
	FCanvasSimpleTextItem::DrawStringInternal_OfflineCache(InCanvas, Text.ToString(), DrawPos, DrawColor, TextEffects);
}

void FCanvasTextItem::DrawStringInternal_RuntimeCache(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects)
{
	FCanvasSimpleTextItem::DrawStringInternal_RuntimeCache(InCanvas, Text.ToString(), DrawPos, DrawColor, TextEffects);
}

bool FCanvasTextStringViewItem::HasValidText() const
{
	return FCanvasSimpleTextItem::HasValidText() && !Text.IsEmpty();
}

FVector2D FCanvasTextStringViewItem::GetTextSize(float DPIScale) const
{
	return GetTextSizeInternal(Text, DPIScale);
}

void FCanvasTextStringViewItem::DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects)
{
	return FCanvasSimpleTextItem::DrawStringInternal(InCanvas, Text, DrawPos, DrawColor, TextEffects);
}

bool FCanvasShapedTextItem::HasValidText() const
{
	return ShapedGlyphSequence.IsValid() && ShapedGlyphSequence->GetGlyphsToRender().Num();
}

ESimpleElementBlendMode FCanvasShapedTextItem::GetTextBlendMode( const bool bHasShadow ) const
{
	ESimpleElementBlendMode BlendModeToUse = BlendMode;

	// The runtime font cache uses an alpha-only texture, so we have to force this blend mode so we use the correct shader
	check(BlendModeToUse == SE_BLEND_Translucent || BlendModeToUse == SE_BLEND_TranslucentAlphaOnly || BlendModeToUse == SE_BLEND_TranslucentAlphaOnlyWriteAlpha);
	BlendModeToUse = SE_BLEND_TranslucentAlphaOnly;

	return BlendModeToUse;
}

FVector2D FCanvasShapedTextItem::GetTextSize(float DPIScale) const
{
	return FVector2D(ShapedGlyphSequence->GetMeasuredWidth(), ShapedGlyphSequence->GetMaxTextHeight());
}

void FCanvasShapedTextItem::DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& InColor, TArrayView<FTextEffect> TextEffects)
{
	DrawnSize = FVector2D::ZeroVector;

	TSharedPtr<FSlateFontCache> FontCache = FEngineFontServices::Get().GetFontCache();
	if( !FontCache.IsValid() )
	{
		return;
	}

	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

	uint32 FontTextureIndex = 0;
	FTextureResource* FontTexture = nullptr;
	ESimpleElementBlendMode FontTextureBlendMode = BlendMode;
	FLinearColor FontColor = InColor;

	float InvTextureSizeX = 0;
	float InvTextureSizeY = 0;

	FVector2D TopLeft(0,0);

	const float PosX = TopLeft.X;
	float PosY = TopLeft.Y;

	const float ScaledHorizSpacingAdjust = HorizSpacingAdjust * Scale.X;
	const float ScaledMaxHeight = ShapedGlyphSequence->GetMaxTextHeight() * Scale.Y;
	const float ScaledBaseline = ShapedGlyphSequence->GetTextBaseline() * Scale.Y;

	float LineX = PosX;
	
	for( const auto& GlyphToRender : ShapedGlyphSequence->GetGlyphsToRender() )
	{
		if (DrawnSize.Y == 0)
		{
			// We have a valid glyph so initialize vertical DrawnSize
			DrawnSize.Y = ScaledMaxHeight;
		}

		if( GlyphToRender.bIsVisible )
		{
			const FShapedGlyphFontAtlasData GlyphAtlasData = FontCache->GetShapedGlyphFontAtlasData(GlyphToRender,FFontOutlineSettings::NoOutline);

			if (GlyphAtlasData.Valid)
			{
				if( FontTexture == nullptr || GlyphAtlasData.TextureIndex != FontTextureIndex )
				{
					// Font has a new texture for this glyph. Refresh the batch we use and the index we are currently using
					FontTextureIndex = GlyphAtlasData.TextureIndex;

					ISlateFontTexture* SlateFontTexture = FontCache->GetFontTexture(FontTextureIndex);
					check(SlateFontTexture);

					FontTexture = SlateFontTexture->GetEngineTexture();
					check(FontTexture);

					const ESlateFontAtlasContentType ContentType = SlateFontTexture->GetContentType();
					FontTextureBlendMode = ContentType == ESlateFontAtlasContentType::Color ? SE_BLEND_Translucent : BlendMode;
					FontColor = ContentType == ESlateFontAtlasContentType::Color ? FLinearColor::White : InColor;

					FBatchedElementParameters* BatchedElementParams = nullptr;
					BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParams, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo);
					check(BatchedElements);

					const int32 NumGlyphs = ShapedGlyphSequence->GetGlyphsToRender().Num();
					BatchedElements->ReserveVertices(4 * NumGlyphs);
					BatchedElements->ReserveTriangles(2 * NumGlyphs, FontTexture, FontTextureBlendMode);

					InvTextureSizeX = 1.0f/FontTexture->GetSizeX();
					InvTextureSizeY = 1.0f/FontTexture->GetSizeY();
				}

				const float BitmapRenderScale = GlyphToRender.GetBitmapRenderScale();
				const float InvBitmapRenderScale = 1.0f / BitmapRenderScale;

				const float U = GlyphAtlasData.StartU * InvTextureSizeX;
				const float V = GlyphAtlasData.StartV * InvTextureSizeY;
				const float SizeX = GlyphAtlasData.USize * Scale.X * BitmapRenderScale;
				const float SizeY = GlyphAtlasData.VSize * Scale.Y * BitmapRenderScale;
				const float SizeU = GlyphAtlasData.USize * InvTextureSizeX;
				const float SizeV = GlyphAtlasData.VSize * InvTextureSizeY;

				const auto AddTriangles = [&](const FVector2f& Offset, const FLinearColor& InColor)
				{
					// Note PosX,PosY is the upper left corner of the bounding box representing the string.  This computes the Y position of the baseline where text will sit
					const float X = (DrawPos.X + Offset.X) + LineX + (GlyphAtlasData.HorizontalOffset * Scale.X) + (GlyphToRender.XOffset * Scale.X);
					const float Y = (DrawPos.Y + Offset.Y) + PosY - (GlyphAtlasData.VerticalOffset * Scale.Y) + (GlyphToRender.YOffset * Scale.Y) + ((ScaledBaseline + ScaledMaxHeight) * InvBitmapRenderScale);

					const float Left = X * Depth;
					const float Top = Y * Depth;
					const float Right = (X + SizeX) * Depth;
					const float Bottom = (Y + SizeY) * Depth;

					int32 V00 = BatchedElements->AddVertexf(
						FVector4f(Left, Top, 0.f, Depth),
						FVector2f(U, V),
						InColor,
						HitProxyId);
					int32 V10 = BatchedElements->AddVertexf(
						FVector4f(Right, Top, 0.0f, Depth),
						FVector2f(U + SizeU, V),
						InColor,
						HitProxyId);
					int32 V01 = BatchedElements->AddVertexf(
						FVector4f(Left, Bottom, 0.0f, Depth),
						FVector2f(U, V + SizeV),
						InColor,
						HitProxyId);
					int32 V11 = BatchedElements->AddVertexf(
						FVector4f(Right, Bottom, 0.0f, Depth),
						FVector2f(U + SizeU, V + SizeV),
						InColor,
						HitProxyId);

					BatchedElements->AddTriangle(V00, V10, V11, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo);
					BatchedElements->AddTriangle(V00, V11, V01, FontTexture, FontTextureBlendMode, FontRenderInfo.GlowInfo);
				};

				// Render all the effects first that appear under the text
				for (const FTextEffect& TextEffect : TextEffects)
				{
					AddTriangles(TextEffect.Offset, TextEffect.Color);
				}

				// The draw the main text
				AddTriangles(FVector2f::ZeroVector, FontColor);
			}
		}

		LineX += GlyphToRender.XAdvance * Scale.X;
		LineX += ScaledHorizSpacingAdjust;

		// Increase the Horizontal DrawnSize
		if (LineX > DrawnSize.X)
		{
			DrawnSize.X = LineX;
		}
	}
}


void FCanvasLineItem::Draw( class FCanvas* InCanvas )
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_LineItemTime);

	FBatchedElements* BatchedElements = InCanvas->GetBatchedElements( FCanvas::ET_Line );
	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();
	BatchedElements->AddLine( Origin, EndPos, Color, HitProxyId, LineThickness );
}


void FCanvasBoxItem::Draw( class FCanvas* InCanvas )
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_BoxItemTime);

	SetupBox();

	FBatchedElements* BatchedElements = InCanvas->GetBatchedElements( FCanvas::ET_Line );
	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();
	
	// Draw the 4 edges
	for (int32 iEdge = 0; iEdge < Corners.Num() ; iEdge++)
	{
		int32 NextCorner = ( ( iEdge + 1 ) % Corners.Num() );
		BatchedElements->AddLine( Corners[ iEdge ], Corners[ NextCorner ] , Color, HitProxyId, LineThickness );	
	}
}

void FCanvasBoxItem::SetupBox()
{
	Corners.Empty();
	// Top
	Corners.AddUnique( FVector( Position.X , Position.Y, 0.0f ) );
	// Right
	Corners.AddUnique( FVector( Position.X + Size.X , Position.Y, 0.0f ) );
	// Bottom
	Corners.AddUnique( FVector( Position.X + Size.X , Position.Y + Size.Y, 0.0f ) );
	// Left
	Corners.AddUnique( FVector( Position.X, Position.Y + Size.Y, 0.0f ) );
}

void FCanvasDashedBoxItem::Draw(class FCanvas* InCanvas)
{
	SCOPE_CYCLE_COUNTER(STAT_Canvas_DashedBoxItemTime);

	if (DashLength <= 0.0f || DashGap <= 0.0f)
	{
		return;
	}
	
	FBatchedElements* BatchedElements = InCanvas->GetBatchedElements( FCanvas::ET_Line );
	FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

	TArray<FVector, TInlineAllocator<4>> Corners;
	// Top
	Corners.AddUnique( FVector( Position.X , Position.Y, 0.0f ) );
	// Right
	Corners.AddUnique( FVector( Position.X + Size.X , Position.Y, 0.0f ) );
	// Bottom
	Corners.AddUnique( FVector( Position.X + Size.X , Position.Y + Size.Y, 0.0f ) );
	// Left
	Corners.AddUnique( FVector( Position.X, Position.Y + Size.Y, 0.0f ) );
	
	if (Corners.Num() < 2)
	{
		return;
	}
	
	FVector CurrentPosition = Corners[0];
	bool bIsDash = true;
	float NextDistance = DashLength;
	
	// Draw the four dashed edges
	for (int32 EdgeIndex = 0; EdgeIndex < Corners.Num() ; EdgeIndex++)
	{
		int32 NextCornerIndex = ( ( EdgeIndex + 1 ) % Corners.Num() );
		
		bool bMoveNext = false;
		
		while (!bMoveNext)
		{
			FVector ToCornerDelta = Corners[NextCornerIndex] - CurrentPosition;
			bool bFlipDash = false;
		
			FVector NextPosition;
			if (ToCornerDelta.SquaredLength() < NextDistance * NextDistance)
			{
				NextPosition = Corners[NextCornerIndex];
				NextDistance = NextDistance - ToCornerDelta.Length();
				bMoveNext = true;
			}
			else
			{
				NextPosition = CurrentPosition + ToCornerDelta.GetSafeNormal() * NextDistance;
				bFlipDash = true;
				NextDistance = bIsDash ? DashGap : DashLength;
			}
		
			if (bIsDash)
			{
				BatchedElements->AddLine( CurrentPosition, NextPosition, Color, HitProxyId, DashThickness );
			}
		
			CurrentPosition = NextPosition;
			
			if (bFlipDash)
			{
				bIsDash = !bIsDash;
			}
		}
	}
}

void FCanvasTriangleItem::Draw( class FCanvas* InCanvas )
{
	if (MaterialRenderProxy == nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_Canvas_TriTextureItemTime);
		FBatchedElements* BatchedElements = InCanvas->GetBatchedElements(FCanvas::ET_Triangle, BatchedElementParameters, Texture, BlendMode);

		FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

		const int32 NumTriangles = TriangleList.Num();
		BatchedElements->ReserveVertices(3 * NumTriangles);
		if (BatchedElementParameters == nullptr)
		{
			BatchedElements->ReserveTriangles(NumTriangles, Texture, BlendMode);
		}

		for (int32 i = 0; i < NumTriangles; i++)
		{
			const FCanvasUVTri& Tri = TriangleList[i];
			int32 V0 = BatchedElements->AddVertexf(FVector4f(Tri.V0_Pos.X, Tri.V0_Pos.Y, 0, 1), FVector2f{Tri.V0_UV}, Tri.V0_Color, HitProxyId);
			int32 V1 = BatchedElements->AddVertexf(FVector4f(Tri.V1_Pos.X, Tri.V1_Pos.Y, 0, 1), FVector2f{Tri.V1_UV}, Tri.V1_Color, HitProxyId);
			int32 V2 = BatchedElements->AddVertexf(FVector4f(Tri.V2_Pos.X, Tri.V2_Pos.Y, 0, 1), FVector2f{Tri.V2_UV}, Tri.V2_Color, HitProxyId);

			if (BatchedElementParameters)
			{
				BatchedElements->AddTriangle(V0, V1, V2, BatchedElementParameters, BlendMode);
			}
			else
			{
				check(Texture);
				BatchedElements->AddTriangle(V0, V1, V2, Texture, BlendMode);
			}
		}
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_Canvas_TriMaterialItemTime);

		// get sort element based on the current sort key from top of sort key stack
		FCanvas::FCanvasSortElement& SortElement = InCanvas->GetSortElement(InCanvas->TopDepthSortKey());

		// find a batch to use 
		FCanvasTriangleRendererItem* RenderBatch = nullptr;

		// get the current transform entry from top of transform stack
		const FCanvas::FTransformEntry& TopTransformEntry = InCanvas->GetTransformStack().Top();

		// try to use the current top entry in the render batch array
		if (SortElement.RenderBatchArray.Num() > 0)
		{
			checkSlow(SortElement.RenderBatchArray.Last());
			RenderBatch = SortElement.RenderBatchArray.Last()->GetCanvasTriangleRendererItem();
		}

		// if a matching entry for this batch doesn't exist then allocate a new entry
		if (RenderBatch == nullptr || !RenderBatch->IsMatch(MaterialRenderProxy, TopTransformEntry))
		{
			INC_DWORD_STAT(STAT_Canvas_NumBatchesCreated);

			RenderBatch = new FCanvasTriangleRendererItem(InCanvas->GetFeatureLevel(), MaterialRenderProxy, TopTransformEntry, bFreezeTime);
			SortElement.RenderBatchArray.Add(RenderBatch);
		}

		FHitProxyId HitProxyId = InCanvas->GetHitProxyId();

		// add the triangles to the triangle render batch
		const int32 NumTriangles = TriangleList.Num();
		RenderBatch->ReserveTriangles(NumTriangles);
		for (int32 i = 0; i < NumTriangles; i++)
		{
			RenderBatch->AddTriangle(TriangleList[i], HitProxyId);
		}
	}	
}

void FCanvasTriangleItem::SetColor( const FLinearColor& InColor )
{
	for(int32 i=0; i<TriangleList.Num(); i++)
	{
		TriangleList[i].V0_Color = InColor;
		TriangleList[i].V1_Color = InColor;
		TriangleList[i].V2_Color = InColor;
	}	
}


void FCanvasNGonItem::Draw( class FCanvas* InCanvas )
{
	TriListItem->BlendMode = BlendMode;
	TriListItem->Draw( InCanvas );
}

void FCanvasNGonItem::SetColor( const FLinearColor& InColor )
{
	TriListItem->SetColor( InColor );
}



FCanvasNGonItem::FCanvasNGonItem( const FVector2D& InPosition, const FVector2D& InRadius, int32 InNumSides, const FLinearColor& InColor )
	: FCanvasItem( InPosition )
	, TriListItem( nullptr )
	, Texture( GWhiteTexture )
{
	Color = InColor;
	check( InNumSides >= 3 );
	TriangleList.SetNum( InNumSides );
	SetupPosition( InPosition, InRadius );
}
