// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Interaction.cpp: See .UC for for info
=============================================================================*/

#include "Engine/Console.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "Misc/PackageName.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Texture2D.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SViewport.h"
#include "Engine/LevelScriptActor.h"
#include "Misc/DefaultValueHelper.h"
#include "GameFramework/InputSettings.h"
#include "Stats/StatsData.h"
#include "Misc/TextFilter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Console)

static const uint32 MAX_AUTOCOMPLETION_LINES = 20;

static const FName NAME_Typing = FName("Typing");
static const FName NAME_Open = FName("Open");

UConsole::FRegisterConsoleAutoCompleteEntries UConsole::RegisterConsoleAutoCompleteEntries;
UConsole::FOnConsoleActivationStateChanged UConsole::OnConsoleActivationStateChanged;

static TAutoConsoleVariable<int32> CVarCustomConsolePosEnabled(
	TEXT("console.position.enable"),
	0,
	TEXT("Enable custom console positioning \n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarConsoleXPos(
	TEXT("console.position.x"),
	0,
	TEXT("Console X offset from left border \n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarConsoleYPos(
	TEXT("console.position.y"),
	0,
	TEXT("Console Y offset from bottom border \n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarConsoleLegacySearch(
	TEXT("console.searchmode.legacy"),
	false,
	TEXT("Use the legacy search behaviour for console commands \n"),
	ECVF_Default);


namespace ConsoleDefs
{
	/** Colors */
	static const FColor BorderColor(140, 140, 140);
	static const FColor CursorColor(255, 255, 255);
	static const FColor AutocompleteBackgroundColor(0, 0, 0);
	static const FColor CursorLineColor(0, 50, 0);
	static const int32 AutocompleteGap = 6;

	/** Text that appears before the user's typed input string that acts as a visual cue for the editable area */
	static const FString LeadingInputText(" > ");
}

class FConsoleVariableAutoCompleteVisitor
{
public:
	// @param Name must not be 0
	// @param CVar must not be 0
	static void OnConsoleVariable(const TCHAR* Name, IConsoleObject* CVar, TArray<struct FAutoCompleteCommand>* Sink)
	{
		if (!CVar->IsEnabled())
		{
			return;
		}

		const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

		// can be optimized
		int32 NewIdx = Sink->AddDefaulted();
		FAutoCompleteCommand& Cmd = (*Sink)[NewIdx];
		Cmd.Command = Name;

		if (ConsoleSettings->bDisplayHelpInAutoComplete)
		{
			TArray<FString> Lines;
			FString(CVar->GetHelp()).ParseIntoArrayLines(Lines, true);
			if (Lines.Num())
			{
				Cmd.Desc = Lines[0];
			}
		}

		IConsoleVariable* CVariable = CVar->AsVariable();
		if (CVariable)
		{
			if (CVar->TestFlags(ECVF_ReadOnly))
			{
				Cmd.Color = ConsoleSettings->AutoCompleteFadedColor;
			}
			else
			{
				Cmd.Color = ConsoleSettings->AutoCompleteCVarColor;
			}
		}
		else
		{
			Cmd.Color = ConsoleSettings->AutoCompleteCommandColor;
		}
	}
};

UConsole::UConsole(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ConsoleSettings(GetDefault<UConsoleSettings>())
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> BlackTexture;
		ConstructorHelpers::FObjectFinder<UTexture2D> WhiteSquareTexture;
		FConstructorStatics()
			: BlackTexture(TEXT("/Engine/EngineResources/Black"))
			, WhiteSquareTexture(TEXT("/Engine/EngineResources/WhiteSquareTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	DefaultTexture_Black = ConstructorStatics.BlackTexture.Object;
	DefaultTexture_White = ConstructorStatics.WhiteSquareTexture.Object;
}

UConsole::~UConsole()
{
	// At shutdown, GLog may already be null
	if (GLog != nullptr)
	{
		GLog->RemoveOutputDevice(this);
	}

	FEngineShowFlags::OnCustomShowFlagRegistered.RemoveAll(this);
}

void UConsole::PostInitProperties()
{
#if WITH_EDITOR
	// Re-load config properties when in editor to preserve command history
	// between PIE sessions. Can't use perobjectconfig because the console 
	// name changes each PIE session.
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		LoadConfig();
	}
#endif
	Super::PostInitProperties();

	FEngineShowFlags::OnCustomShowFlagRegistered.AddUObject(this, &UConsole::InvalidateAutocomplete);
}

void UConsole::InvalidateAutocomplete()
{
	bIsRuntimeAutoCompleteUpToDate = false;
}

void UConsole::BuildRuntimeAutoCompleteList(bool bForce)
{
	LLM_SCOPE(ELLMTag::EngineMisc);

#if ALLOW_CONSOLE
	if (!bForce)
	{
		// unless forced delay updating until needed
		bIsRuntimeAutoCompleteUpToDate = false;
		return;
	}

	// clear the existing tree
	//@todo - probably only need to rebuild the tree + partial command list on level load
	for (int32 Idx = 0; Idx < AutoCompleteTree.ChildNodes.Num(); Idx++)
	{
		FAutoCompleteNode* Node = AutoCompleteTree.ChildNodes[Idx];
		delete Node;
	}

	AutoCompleteTree.ChildNodes.Reset();

	// copy the manual list first
	AutoCompleteList.Reset();
	AutoCompleteList.AddDefaulted(ConsoleSettings->ManualAutoCompleteList.Num());
	for (int32 Idx = 0; Idx < ConsoleSettings->ManualAutoCompleteList.Num(); Idx++)
	{
		AutoCompleteList[Idx] = ConsoleSettings->ManualAutoCompleteList[Idx];
		AutoCompleteList[Idx].Color = ConsoleSettings->AutoCompleteCommandColor;
	}

	// systems that have registered to want to introduce entries
	RegisterConsoleAutoCompleteEntries.Broadcast(AutoCompleteList);

	// console variables
	{
		IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(
			FConsoleObjectVisitor::CreateStatic(
				&FConsoleVariableAutoCompleteVisitor::OnConsoleVariable,
				&AutoCompleteList));
	}

	// iterate through script exec functions and append to the list
	for (TObjectIterator<UFunction> It; It; ++It)
	{
		UFunction* Func = *It;

		// Determine whether or not this is a level script event that we can call (must be defined in the level script actor and not in parent, and has no return value)
		const UClass* FuncOuter = Cast<UClass>(Func->GetOuter());
		const bool bIsLevelScriptFunction = FuncOuter
			&& (FuncOuter->IsChildOf(ALevelScriptActor::StaticClass()))
			&& (FuncOuter != ALevelScriptActor::StaticClass())
			&& (Func->ReturnValueOffset == MAX_uint16)
			&& (Func->GetSuperFunction() == nullptr);

		// exec functions that either have no parent, level script events, or are in the global state (filtering some unnecessary dupes)
		if ((Func->HasAnyFunctionFlags(FUNC_Exec) && (Func->GetSuperFunction() == nullptr || FuncOuter))
			|| bIsLevelScriptFunction)
		{
			FString FuncName = Func->GetName();
			if (FDefaultValueHelper::HasWhitespaces(FuncName))
			{
				FuncName = FString::Printf(TEXT("\"%s\""), *FuncName);
			}
			if (bIsLevelScriptFunction)
			{
				FuncName = FString("ce ") + FuncName;
			}

			int32 Idx = 0;
			for (; Idx < AutoCompleteList.Num(); ++Idx)
			{
				if (AutoCompleteList[Idx].Command == FuncName)
				{
					break;
				}
			}

			const int32 NewIdx = (Idx < AutoCompleteList.Num()) ? Idx : AutoCompleteList.AddDefaulted();
			AutoCompleteList[NewIdx].Command = FuncName;
			AutoCompleteList[NewIdx].Color = ConsoleSettings->AutoCompleteCommandColor;

			FString Desc;

			// build a help string
			// append each property (and it's type) to the help string
			for (TFieldIterator<FProperty> PropIt(Func); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
			{
				FProperty* Prop = *PropIt;
				Desc += FString::Printf(TEXT("%s[%s] "), *Prop->GetName(), *Prop->GetCPPType());
			}
			AutoCompleteList[NewIdx].Desc = Desc + AutoCompleteList[NewIdx].Desc;
		}
	}

	// enumerate maps
	{
		auto FindPackagesInDirectory = [](TArray<FString>& OutPackages, const FString& InPath)
		{
			FString PackagePath;
			if (FPackageName::TryConvertFilenameToLongPackageName(InPath, PackagePath))
			{
				if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::LoadModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
				{
					TArray<FAssetData> Assets;
					AssetRegistryModule->Get().GetAssetsByPath(FName(*PackagePath), Assets, true);

					for (const FAssetData& Asset : Assets)
					{
						if (!!(Asset.PackageFlags & PKG_ContainsMap) && Asset.IsUAsset())
						{
							OutPackages.AddUnique(Asset.AssetName.ToString());
						}
					}
				}
			}
			TArray<FString> Filenames;
			FPackageName::FindPackagesInDirectory(Filenames, InPath);

			for (const FString& Filename : Filenames)
			{
				const int32 NameStartIdx = Filename.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				const int32 ExtIdx = Filename.Find(*FPackageName::GetMapPackageExtension(), ESearchCase::IgnoreCase, ESearchDir::FromEnd);

				if (NameStartIdx != INDEX_NONE && ExtIdx != INDEX_NONE)
				{
					OutPackages.AddUnique(Filename.Mid(NameStartIdx + 1, ExtIdx - NameStartIdx - 1));
				}
			}
		};

		TArray<FString> Packages;
		for (const FString& MapPath : ConsoleSettings->AutoCompleteMapPaths)
		{
			FindPackagesInDirectory(Packages, FString::Printf(TEXT("%s%s"), *FPaths::ProjectDir(), *MapPath));
		}

		FindPackagesInDirectory(Packages, FPaths::GameUserDeveloperDir());

		for (const FString& MapName : Packages)
		{
			int32 NewIdx = 0;
			// put _P maps at the front so that they match early, since those are generally the maps we want to actually open
			if (MapName.EndsWith(TEXT("_P")))
			{
				AutoCompleteList.InsertDefaulted(0, 3);
			}
			else
			{
				NewIdx = AutoCompleteList.AddDefaulted(3);
			}

			AutoCompleteList[NewIdx].Command = FString::Printf(TEXT("open %s"), *MapName);
			AutoCompleteList[NewIdx].Color = ConsoleSettings->AutoCompleteCommandColor;
			AutoCompleteList[NewIdx + 1].Command = FString::Printf(TEXT("travel %s"), *MapName);
			AutoCompleteList[NewIdx + 1].Color = ConsoleSettings->AutoCompleteCommandColor;
			AutoCompleteList[NewIdx + 2].Command = FString::Printf(TEXT("servertravel %s"), *MapName);
			AutoCompleteList[NewIdx + 2].Color = ConsoleSettings->AutoCompleteCommandColor;
		}
	}

	// misc commands
	{
		const int32 NewIdx = AutoCompleteList.AddDefaulted();
		AutoCompleteList[NewIdx].Command = FString("open 127.0.0.1");
		AutoCompleteList[NewIdx].Desc = FString("(opens connection to localhost)");
		AutoCompleteList[NewIdx].Color = ConsoleSettings->AutoCompleteCommandColor;
	}

#if STATS
	// stat commands
	{
		const TSet<FName>& StatGroupNames = FStatGroupGameThreadNotifier::Get().StatGroupNames;
		for (const FName& StatGroupName : StatGroupNames)
		{
			FString Command = FString("Stat ");
			Command += StatGroupName.ToString().RightChop(sizeof("STATGROUP_") - 1);

			int32 Idx = 0;
			for (; Idx < AutoCompleteList.Num(); ++Idx)
			{
				if (AutoCompleteList[Idx].Command == Command)
				{
					break;
				}
			}

			Idx = (Idx < AutoCompleteList.Num()) ? Idx : AutoCompleteList.AddDefaulted();
			AutoCompleteList[Idx].Command = Command;
			AutoCompleteList[Idx].Color = ConsoleSettings->AutoCompleteCommandColor;
		}
	}
#endif

	// Add all showflag commands.
	{
		struct FIterSink
		{
			FIterSink(TArray<FAutoCompleteCommand>& InAutoCompleteList)
				: AutoCompleteList(InAutoCompleteList)
			{
			}

			bool HandleShowFlag(uint32 InIndex, const FString& InName)
			{
				// Get localized name.
				FText LocName;
				FEngineShowFlags::FindShowFlagDisplayName(InName, LocName);

				int32 NewIdx = AutoCompleteList.AddDefaulted();
				AutoCompleteList[NewIdx].Command = TEXT("show ") + InName;
				AutoCompleteList[NewIdx].Desc = FString::Printf(TEXT("(toggles the %s showflag)"), *LocName.ToString());
				AutoCompleteList[NewIdx].Color = GetDefault<UConsoleSettings>()->AutoCompleteCommandColor;

				return true;
			}
			
			bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
			{
				return HandleShowFlag(InIndex, InName);
			}

			bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
			{
				return HandleShowFlag(InIndex, InName);
			}

			TArray<FAutoCompleteCommand>& AutoCompleteList;
		};

		FIterSink Sink(AutoCompleteList);
		FEngineShowFlags::IterateAllFlags(Sink);
	}

	// Add any commands from UConsole subclasses
	AugmentRuntimeAutoCompleteList(AutoCompleteList);

	AutoCompleteList.Shrink();

	// build the magic tree!
	for (int32 ListIdx = 0; ListIdx < AutoCompleteList.Num(); ListIdx++)
	{
		FString Command = AutoCompleteList[ListIdx].Command.ToLower();
		FAutoCompleteNode* Node = &AutoCompleteTree;
		for (int32 Depth = 0; Depth < Command.Len(); Depth++)
		{
			int32 Char = Command[Depth];
			int32 FoundNodeIdx = INDEX_NONE;
			TArray<FAutoCompleteNode*>& NodeList = Node->ChildNodes;
			for (int32 NodeIdx = 0; NodeIdx < NodeList.Num(); NodeIdx++)
			{
				if (NodeList[NodeIdx]->IndexChar == Char)
				{
					FoundNodeIdx = NodeIdx;
					Node = NodeList[FoundNodeIdx];
					NodeList[FoundNodeIdx]->AutoCompleteListIndices.Add(ListIdx);
#if UE_ENABLE_ARRAY_SLACK_TRACKING
					// Disable array slack tracking for Console system related allocations.  This needs to be called any time a reallocation occurs,
					// which in practical terms means any "Add" (although it's cheap if there wasn't a recent reallocation, a couple memory reads).
					// The auto-complete code (triggered when bringing up a console window to issue the slack report command) generates over a million
					// allocations, with over 10 MB cumulative slack memory, which ends up at the top of any slack report, and is debug related code
					// that we don't care about tracking.  If anyone wants to look at it for some reason, just remove these lines.
					NodeList[FoundNodeIdx]->AutoCompleteListIndices.GetAllocatorInstance().DisableSlackTracking();
#endif
					break;
				}
			}
			if (FoundNodeIdx == INDEX_NONE)
			{
				FAutoCompleteNode* NewNode = new FAutoCompleteNode(Char);
				NewNode->AutoCompleteListIndices.Add(ListIdx);
				Node->ChildNodes.Add(NewNode);
#if UE_ENABLE_ARRAY_SLACK_TRACKING
				NewNode->AutoCompleteListIndices.GetAllocatorInstance().DisableSlackTracking();
				Node->ChildNodes.GetAllocatorInstance().DisableSlackTracking();
#endif
				Node = NewNode;
			}
		}
	}
	bIsRuntimeAutoCompleteUpToDate = true;
	//PrintNode(&AutoCompleteTree);
#endif
}

void UConsole::AugmentRuntimeAutoCompleteList(TArray<FAutoCompleteCommand>& List)
{
	// Implement in subclasses as necessary
}

typedef TTextFilter< const FAutoCompleteCommand& > FCheatTextFilter;

void CommandToStringArray(const FAutoCompleteCommand& Command, OUT TArray< FString >& StringArray)
{
	StringArray.Add(Command.Command);
}

void UConsole::UpdateCompleteIndices(bool bIsSearchNarrowing /*= false*/)
{
	if (!bIsRuntimeAutoCompleteUpToDate)
	{
		BuildRuntimeAutoCompleteList(true);
	}

	// If we've just narrowed our search by appending characters to the end, yet we've already determined that
	// no console commands match our entry, then early out.  It's impossible to discover any new suggestions
	// by appending text.
	if (bIsSearchNarrowing && AutoComplete.IsEmpty())
	{
		return;
	}

	AutoComplete.Empty();
	AutoCompleteIndex = 0;
	AutoCompleteCursor = 0;

	if (CVarConsoleLegacySearch.GetValueOnAnyThread())
	{
		// use the old autocomplete behaviour
		FAutoCompleteNode* Node = &AutoCompleteTree;
		FString LowerTypedStr = TypedStr.ToLower();
		int32 EndIdx = -1;
		for (int32 Idx = 0; Idx < TypedStr.Len(); Idx++)
		{
			int32 Char = LowerTypedStr[Idx];
			bool bFoundMatch = false;
			int32 BranchCnt = 0;
			for (int32 CharIdx = 0; CharIdx < Node->ChildNodes.Num(); CharIdx++)
			{
				BranchCnt += Node->ChildNodes[CharIdx]->ChildNodes.Num();
				if (Node->ChildNodes[CharIdx]->IndexChar == Char)
				{
					bFoundMatch = true;
					Node = Node->ChildNodes[CharIdx];
					break;
				}
			}
			if (!bFoundMatch)
			{
				if (!bAutoCompleteLocked && BranchCnt > 0)
				{
					// we're off the grid!
					return;
				}
				else
				{
					if (Idx < TypedStr.Len())
					{
						// if the first non-matching character is a space we might be adding parameters, stay on the last node we found so users can see the parameter info
						if (TypedStr[Idx] == TCHAR(' '))
						{
							EndIdx = Idx;
							break;
						}
						// there is more text behind the auto completed text, we don't need auto completion
						return;
					}
					else
					{
						break;
					}
				}
			}
		}
		if (Node != &AutoCompleteTree)
		{
			const TArray<int32>& Leaf = Node->AutoCompleteListIndices;

			for (uint32 i = 0, Num = (uint32)Leaf.Num(); i < Num; ++i)
			{
				// if we're adding parameters we want to make sure that we only display exact matches
				// ie Typing "Foo 5" should still show info for "Foo" but not for "FooBar"
				if (EndIdx < 0 || AutoCompleteList[Leaf[i]].Command.Len() == EndIdx)
				{
					AutoComplete.Add(AutoCompleteList[Leaf[i]]);
				}
			}
			AutoComplete.Sort();
		}
	}
	else if (!TypedStr.IsEmpty())
	{
		// search for any substring, not just the prefix
		static FCheatTextFilter Filter(FCheatTextFilter::FItemToStringArray::CreateStatic(&CommandToStringArray));
		Filter.SetRawFilterText(FText::FromString(TypedStr));

		for (const FAutoCompleteCommand& Command : AutoCompleteList)
		{
			if (Filter.PassesFilter(Command))
			{
				AutoComplete.Add(Command);
			}
		}

		AutoComplete.Sort([this](const FAutoCompleteCommand& A, const FAutoCompleteCommand& B)
			{
				// Give preference to commands that actually begin with what the user's typed so far
				if (A.Command.StartsWith(TypedStr))
				{
					if (!B.Command.StartsWith(TypedStr))
					{
						return true;
					}
				}
				else
				{
					if (B.Command.StartsWith(TypedStr))
					{
						return false;
					}
				}

				return A < B;
			});
	}
}

void UConsole::SetAutoCompleteFromHistory()
{
	AutoCompleteIndex = 0;
	AutoCompleteCursor = 0;
	AutoComplete.Empty();

	for (int32 i = HistoryBuffer.Num() - 1; i >= 0; --i)
	{
		FAutoCompleteCommand Cmd;

		Cmd.Command = HistoryBuffer[i];
		Cmd.Color = ConsoleSettings->HistoryColor;
		Cmd.SetHistory();

		AutoComplete.Add(Cmd);
	}
}

void UConsole::SetInputText(const FString& Text)
{
	TypedStr = Text;
}

void UConsole::SetCursorPos(int32 Position)
{
	TypedStrPos = Position;
}

void UConsole::ConsoleCommand(const FString& Command)
{
	// insert into history buffer
	{
		HistoryBuffer.Remove(Command);
		HistoryBuffer.Add(Command);

		NormalizeHistoryBuffer();
	}

	// Save the command history to the INI.
	SaveConfig();

	OutputText(FString::Printf(TEXT("\n>>> %s <<<"), *Command));

	UGameInstance* GameInstance = GetOuterUGameViewportClient()->GetGameInstance();
	if (ConsoleTargetPlayer != nullptr)
	{
		// If there is a console target player, execute the command in the player's context.
		ConsoleTargetPlayer->PlayerController->ConsoleCommand(Command);
	}
	else if (GameInstance && GameInstance->GetFirstLocalPlayerController())
	{
		// If there are any players, execute the command in the first local player's context.
		APlayerController* PC = GameInstance->GetFirstLocalPlayerController();
		PC->ConsoleCommand(Command);
	}
	else
	{
		// Otherwise, execute the command in the context of the viewport.
		GetOuterUGameViewportClient()->ConsoleCommand(Command);
	}
}


void UConsole::ClearOutput()
{
	SBHead = 0;
	Scrollback.Empty();
}


void UConsole::OutputTextLine(const FString& Text)
{
	// If we are full, delete the first line
	if (Scrollback.Num() >= ConsoleSettings->MaxScrollbackSize)
	{
		Scrollback.RemoveAt(0, Scrollback.Num() - ConsoleSettings->MaxScrollbackSize + 1);
		SBHead = ConsoleSettings->MaxScrollbackSize;
	}
	else
	{
		SBHead++;
	}

	// Add the line
	Scrollback.Add(Text);

	if (Selection.bActive)
	{
		Selection.Offset += TextH;
	}
}


void UConsole::OutputText(const FString& Text)
{
	FString RemainingText = Text;
	int32 StringLength = Text.Len();
	while (StringLength > 0)
	{
		// Find the number of characters in the next line of text.
		int32 LineLength = RemainingText.Find(TEXT("\n"), ESearchCase::CaseSensitive);
		if (LineLength == -1)
		{
			// There aren't any more newlines in the string, assume there's a newline at the end of the string.
			LineLength = StringLength;
		}

		// Output the line to the console.
		OutputTextLine(RemainingText.Left(LineLength));

		// Remove the line from the string.
		RemainingText = (RemainingText).Mid(LineLength + 1, MAX_int32);
		StringLength -= LineLength + 1;
	}
}


void UConsole::StartTyping(const FString& Text)
{
	static const FName TypingName = FName("Typing");
	FakeGotoState(TypingName);
	SetInputText(Text);
	SetCursorPos(Text.Len());
}


void UConsole::FlushPlayerInput()
{
	APlayerController* PC = nullptr;
	if (ConsoleTargetPlayer)
	{
		PC = ConsoleTargetPlayer->PlayerController;
	}
	else
	{
		UWorld* World = GetOuterUGameViewportClient()->GetWorld();
		if (ULocalPlayer * LocalPlayer = GEngine->GetFirstGamePlayer(World))
		{
			PC = LocalPlayer->PlayerController;
		}
	}

	if (PC && PC->PlayerInput)
	{
		PC->PlayerInput->FlushPressedKeys();
	}
}

bool UConsole::ProcessControlKey(FKey Key, EInputEvent Event)
{
#if PLATFORM_MAC
	if (Key == EKeys::LeftCommand || Key == EKeys::RightCommand)
#else
	if (Key == EKeys::LeftControl || Key == EKeys::RightControl)
#endif
	{
		if (Event == IE_Released)
		{
			bCtrl = false;
		}
		else if (Event == IE_Pressed)
		{
			bCtrl = true;
		}

		return true;
	}
	else if (bCtrl && Event == IE_Pressed)
	{
		if (Key == EKeys::V)
		{
			// paste
			FString ClipboardContent;
			FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
			AppendInputText(ClipboardContent);
			return true;
		}
		else if (Key == EKeys::C)
		{
			// copy
			FPlatformApplicationMisc::ClipboardCopy(*TypedStr);
			return true;
		}
		else if (Key == EKeys::X)
		{
			// cut
			if (!TypedStr.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*TypedStr);
				SetInputText(TEXT(""));
				SetCursorPos(0);
			}
			return true;
		}
	}

	return false;
}

bool UConsole::ProcessShiftKey(FKey Key, EInputEvent Event)
{
	if (Key == EKeys::LeftShift || Key == EKeys::RightShift)
	{
		if (Event == IE_Released)
		{
			bShift = false;
		}
		else if (Event == IE_Pressed)
		{
			bShift = true;
		}

		return true;
	}

	return false;
}


void UConsole::AppendInputText(const FString& Text)
{
	// If we already have text, and our insertion point is at the end, then we're appending this text.
	// Later, we'll pass this hint along to the autocomplete code, who uses it to potentially bypass a LOT of expensive code.
	const bool bIsAppending = (TypedStrPos > 0 && TypedStrPos == TypedStr.Len());

	FString TextMod = Text;
	int32 CurrentPos = 0;

	while (CurrentPos < TextMod.Len())
	{
		const int32 Character = static_cast<int32>(TextMod[CurrentPos]);

		// Omit all control characters
		if (Character < 0x20)
		{
			TextMod.RemoveAt(CurrentPos, EAllowShrinking::No);	// No reallocs
		}
		else
		{
			++CurrentPos;
		}
	}

	SetInputText(
		FString::Format(TEXT("{0}{1}{2}"), { FStringView(TypedStr).Left(TypedStrPos), TextMod, FStringView(TypedStr).Right(TypedStr.Len() - TypedStrPos) }));

	SetCursorPos(TypedStrPos + TextMod.Len());

	UpdateCompleteIndices(bIsAppending);
	UpdatePrecompletedInputLine();
}

bool UConsole::InputChar_Typing(FInputDeviceId DeviceId, const FString& Unicode)
{
	if (bCaptureKeyInput)
	{
		return true;
	}

	AppendInputText(Unicode);

	return true;
}

bool UConsole::InputKey_InputLine(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed, bool bGamepad)
{
	if (Event == IE_Pressed)
	{
		bCaptureKeyInput = false;
	}

	// cycle between console states
	bool bModifierDown = bCtrl;
	FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
	bModifierDown |= KeyState.IsAltDown() || KeyState.IsCommandDown() || KeyState.IsShiftDown() || KeyState.IsControlDown();
	if (GetDefault<UInputSettings>()->ConsoleKeys.Contains(Key) && Event == IE_Pressed && !bModifierDown)
	{
		if (ConsoleState == NAME_Typing)
		{
			FakeGotoState(NAME_Open);
			bCaptureKeyInput = true;
		}
		else if (ConsoleState == NAME_Open)
		{
			FakeGotoState(NAME_None);
			bCaptureKeyInput = true;
		}
		else if (ConsoleState == NAME_None)
		{
			FakeGotoState(NAME_Typing);
			bCaptureKeyInput = true;
		}
		return true;
	}

	auto DecrementCursor = [this]()
	{
		if (AutoCompleteCursor > 0)
		{
			// move cursor within displayed region
			--AutoCompleteCursor;
		}
		else
		{
			// can we scroll?
			if (AutoCompleteIndex > 0)
			{
				--AutoCompleteIndex;
			}
			else
			{
				// wrap around
				AutoCompleteIndex = FMath::Max(0, AutoComplete.Num() - (int32)MAX_AUTOCOMPLETION_LINES - 1);
				if (AutoComplete.Num() <= MAX_AUTOCOMPLETION_LINES)
				{
					AutoCompleteCursor = AutoComplete.Num() + AutoCompleteCursor - 1;
				}
				else
				{
					// skip the "x more matches" line when wrapping
					AutoCompleteIndex++;
					AutoCompleteCursor = MAX_AUTOCOMPLETION_LINES + AutoCompleteCursor - 1;
				}
			}
			bAutoCompleteLocked = false;
		}
	};

	auto IncrementCursor = [this]()
	{
		if (AutoCompleteCursor + 1 < FMath::Min((int32)MAX_AUTOCOMPLETION_LINES, AutoComplete.Num()))
		{
			// move cursor within displayed region
			++AutoCompleteCursor;
		}
		else
		{
			// can be negative
			int32 ScrollRegionSize = AutoComplete.Num() - (int32)MAX_AUTOCOMPLETION_LINES;

			// can we scroll?
			if (AutoCompleteIndex < ScrollRegionSize)
			{
				++AutoCompleteIndex;
			}
			else
			{
				// wrap around
				AutoCompleteIndex = AutoCompleteCursor = 0;
			}
		}
	};

	auto FindWordBreak = [](const FString& Str, uint32 StartPos, ESearchDir::Type Direction)
	{
		// find the nearest '.' or ' '
		int32 SpacePos = Str.Find(TEXT(" "), ESearchCase::CaseSensitive, Direction, StartPos);
		int32 PeriodPos = Str.Find(TEXT("."), ESearchCase::CaseSensitive, Direction, StartPos);
		if (Direction == ESearchDir::FromEnd)
		{
			return FMath::Max(SpacePos, PeriodPos);
		}
		else
		{ 
			int32 Result = SpacePos < 0 ? PeriodPos : (PeriodPos < 0 ? SpacePos : FMath::Min(SpacePos, PeriodPos));
			Result = Result == INDEX_NONE ? Str.Len() : Result;
			return Result;
		}
	};

	// if user input is open
	if (ConsoleState != NAME_None)
	{
		if (ProcessControlKey(Key, Event))
		{
			return true;
		}
		else if (ProcessShiftKey(Key, Event))
		{
			return true;
		}
		else if (bGamepad)
		{
			return false;
		}
		else if (Key == EKeys::Escape && Event == IE_Released)
		{
			if (!TypedStr.IsEmpty())
			{
				SetInputText("");
				SetCursorPos(0);

				AutoCompleteIndex = 0;
				AutoCompleteCursor = 0;
				PrecompletedInputLine = FString();
				LastAutoCompletedCommand = nullptr;
				AutoComplete.Empty();
				bAutoCompleteLocked = false;

				return true;
			}
			else
			{
				FakeGotoState(NAME_None);
			}

			return true;
		}
		else if (Key == EKeys::Enter && Event == IE_Released)
		{
			if (!TypedStr.IsEmpty())
			{
				// Make a local copy of the string.
				FString Temp = TypedStr;

				SetInputText(TEXT(""));
				SetCursorPos(0);

				ConsoleCommand(Temp);

				//OutputText( Localize("Errors","Exec","Core") );

				OutputText(TEXT(""));

				if (ConsoleState == NAME_Typing)
				{
					// close after each command when in typing mode (single line)
					FakeGotoState(NAME_None);
				}

				UpdateCompleteIndices();
			}
			else
			{
				FakeGotoState(NAME_None);
			}

			// A command was executed and/or the console closed, discard the most recent autocomplete info
			PrecompletedInputLine = FString();
			LastAutoCompletedCommand = nullptr;

			return true;
		}
		else if (Event != IE_Pressed && Event != IE_Repeat)
		{
			if (!bGamepad)
			{
				return	Key != EKeys::LeftMouseButton
					&&	Key != EKeys::MiddleMouseButton
					&&	Key != EKeys::RightMouseButton;
			}
			return false;
		}
		else if (Key == EKeys::Up || (Key == EKeys::Tab && bShift))
		{
			if (!bCtrl)
			{
				if (AutoComplete.Num())
				{

					if (Key == EKeys::Tab)
					{
						bCaptureKeyInput = true;
					}

					if (ConsoleSettings->bOrderTopToBottom)
					{
						DecrementCursor();
					}
					else
					{
						IncrementCursor();
					}
				}
				else
				{
					SetAutoCompleteFromHistory();
				}
				SetInputLineFromAutoComplete();
			}
		}
		else if (Key == EKeys::Down || (Key == EKeys::Tab && !bShift))
		{
			if (!bCtrl)
			{
				if (AutoComplete.Num())
				{
					bool bScroll = AutoComplete.Num() > 1;

					if (Key == EKeys::Tab)
					{
						bCaptureKeyInput = true;

						// If this is a repeated tab press, we want to scroll. Otherwise complete the current command
						bScroll = bScroll && LastAutoCompletedCommand == TypedStr;
					}

					if (bScroll)
					{
						if (ConsoleSettings->bOrderTopToBottom)
						{
							IncrementCursor();
						}
						else
						{
							DecrementCursor();
						}
					}
				}
				else
				{
					SetAutoCompleteFromHistory();
				}

				SetInputLineFromAutoComplete();
			}
			return true;
		}
		else if (Key == EKeys::BackSpace)
		{
			if (TypedStrPos > 0)
			{
				int32 NewPos;
				if (bCtrl)
				{
					NewPos = FMath::Max(0, FindWordBreak(TypedStr, TypedStrPos, ESearchDir::FromEnd));
				}
				else
				{
					NewPos = TypedStrPos - 1;
				}

				SetInputText(
					FString::Format(TEXT("{0}{1}"), { FStringView(TypedStr).Left(NewPos), FStringView(TypedStr).Right(TypedStr.Len() - TypedStrPos) }));

				SetCursorPos(NewPos);

				// Update autocomplete
				UpdateCompleteIndices();
				UpdatePrecompletedInputLine();

				// unlock auto-complete (@todo - track the lock position so we don't bother unlocking under bogus cases)
				bAutoCompleteLocked = false;
			}
			bCaptureKeyInput = true;

			return true;
		}
		else if (Key == EKeys::Delete)
		{
			if (TypedStrPos < TypedStr.Len())
			{
				int32 RightStart;
				if (bCtrl)
				{
					RightStart = FindWordBreak(TypedStr, TypedStrPos + 1, ESearchDir::FromStart);
				}
				else
				{
					RightStart = TypedStrPos + 1;
				}

				SetInputText(
					FString::Format(TEXT("{0}{1}"), { FStringView(TypedStr).Left(TypedStrPos), FStringView(TypedStr).Right(TypedStr.Len() - RightStart) }));

				// Update autocomplete
				UpdateCompleteIndices();
				UpdatePrecompletedInputLine();
			}
			return true;
		}
		else if (Key == EKeys::Left)
		{
			int32 NewPos;
			if (bCtrl)
			{
				NewPos = FMath::Min(FindWordBreak(TypedStr, FMath::Max(0, TypedStrPos - 1), ESearchDir::FromEnd) + 1, TypedStr.Len());
			}
			else
			{
				NewPos = FMath::Max(0, TypedStrPos - 1);
			}
			SetCursorPos(NewPos);
			return true;
		}
		else if (Key == EKeys::Right)
		{
			int32 NewPos;
			if (bCtrl)
			{
				NewPos = FindWordBreak(TypedStr, FMath::Min(TypedStrPos + 1, TypedStr.Len()), ESearchDir::FromStart);
			}
			else
			{
				NewPos = FMath::Min(TypedStr.Len(), TypedStrPos + 1);
			}
			SetCursorPos(NewPos);
			return true;
		}
		else if (Key == EKeys::Home)
		{
			SetCursorPos(0);
			return true;
		}
		else if (Key == EKeys::End)
		{
			SetCursorPos(TypedStr.Len());
			return true;
		}
	}

	return false;
}

void UConsole::SetInputLineFromAutoComplete()
{
	if (AutoComplete.Num() > 0)
	{
		const int32 Index = AutoCompleteIndex + (AutoCompleteCursor >= 0 ? AutoCompleteCursor : 0);
		const FAutoCompleteCommand& Cmd = AutoComplete[Index];

		TypedStr = Cmd.Command;
		SetCursorPos(TypedStr.Len());
		bAutoCompleteLocked = true;

		PrecompletedInputLine = Cmd.Command;
		LastAutoCompletedCommand = Cmd.Command;
	}
}

void UConsole::UpdatePrecompletedInputLine()
{
	// Set the full command text for the user input if they were to autocomplete it with tab
	if (AutoComplete.Num() > 0)
	{
		const int32 Index = AutoCompleteIndex + (AutoCompleteCursor >= 0 ? AutoCompleteCursor : 0);
		const FAutoCompleteCommand& Cmd = AutoComplete[Index];
		PrecompletedInputLine = Cmd.Command;
	}
	else
	{
		// Input buffer cleared, or the user is typing some nonexistent command
		PrecompletedInputLine = FString();
	}
}

void UConsole::NormalizeHistoryBuffer()
{
	const uint32 Count = MAX_HISTORY_ENTRIES;

	check(Count > 0);

	if ((uint32)HistoryBuffer.Num() > Count)
	{
		uint32 ShrinkCount = (uint32)(HistoryBuffer.Num() - Count);
		HistoryBuffer.RemoveAt(0, ShrinkCount);
	}
}

void UConsole::PostRender_Console_Typing(UCanvas* Canvas)
{
	float ClipX = Canvas->ClipX;
	float ClipY = Canvas->ClipY;
	float PosLeft = 0;

	if (CVarCustomConsolePosEnabled.GetValueOnAnyThread())
	{
		PosLeft = (float)CVarConsoleXPos.GetValueOnAnyThread();
		float BottomOffset = (float)CVarConsoleYPos.GetValueOnAnyThread();
		ClipY = ClipY - BottomOffset;
	}

	PostRender_InputLine(Canvas, FIntPoint(PosLeft, ClipY));
}

void UConsole::BeginState_Typing(FName PreviousStateName)
{
	if (PreviousStateName == NAME_None)
	{
		FlushPlayerInput();
	}
	bCaptureKeyInput = true;
}

void UConsole::EndState_Typing(FName NextStateName)
{
	bAutoCompleteLocked = false;
}

bool UConsole::InputChar_Open(FInputDeviceId DeviceId, const FString& Unicode)
{
	if (bCaptureKeyInput)
	{
		return true;
	}

	AppendInputText(Unicode);

	return true;
}

bool UConsole::InputKey_Open(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed, bool bGamepad)
{
	if (Key == EKeys::LeftMouseButton)
	{
		if (Event == IE_Pressed)
		{
			Selection.MousePosDown = Selection.MousePos;
			Selection.bActive = true;
			Selection.bMade = false;
			Selection.Offset = 0.0f;
			return true;
		}
		else if (Event == IE_Released)
		{
			Selection.MousePosUp = Selection.MousePos;
			Selection.bCapture = true;
			Selection.bMade = true;
			return true;
		}
	}

	if (Key == EKeys::PageUp || Key == EKeys::MouseScrollUp)
	{
		if (SBPos < Scrollback.Num() - 1)
		{
			if (bCtrl)
				SBPos += 5;
			else
				SBPos++;

			if (SBPos >= Scrollback.Num())
				SBPos = Scrollback.Num() - 1;
		}

		return true;
	}
	else if (Key == EKeys::PageDown || Key == EKeys::MouseScrollDown)
	{
		if (SBPos > 0)
		{
			if (bCtrl)
				SBPos -= 5;
			else
				SBPos--;

			if (SBPos < 0)
				SBPos = 0;
		}

		return true;
	}

	return false;
}

void UConsole::PostRender_Console_Open(UCanvas* Canvas)
{
	// the height of the buffer will be 75% of the height of the screen
	float Height = FMath::FloorToFloat(Canvas->ClipY * 0.75f);

	// shrink for TVs
	float ClipX = Canvas->ClipX;
	float PosTop = 0;
	float PosLeft = 0;

	if (CVarCustomConsolePosEnabled.GetValueOnAnyThread())
	{
		PosLeft = (float)CVarConsoleXPos.GetValueOnAnyThread();
		float BottomOffset = (float)CVarConsoleYPos.GetValueOnAnyThread();
		Height = Canvas->ClipY - BottomOffset;
	}

	UFont* Font = GEngine->GetSmallFont();

	const float DPIScale = Canvas->GetDPIScale();

	// determine the height of the text
	Canvas->StrLen(Font, TEXT("M"), TextW, TextH, bDPIAwareStringMeasurement);
	TextW /= DPIScale;
	TextH /= DPIScale;

	float y = Height - TextH;

	if (Scrollback.Num())
	{
		FLinearColor BackgroundColorBlack = ConsoleDefs::AutocompleteBackgroundColor.ReinterpretAsLinear();
		FLinearColor BackgroundColorWhite = FLinearColor::White;
		BackgroundColorBlack.A = ConsoleSettings->BackgroundOpacityPercentage / 100.0f;
		BackgroundColorWhite.A = ConsoleSettings->BackgroundOpacityPercentage / 100.0f;

		FCanvasTextItem ConsoleText(FVector2D(PosLeft, PosTop + Height - 5 - TextH), FText::FromString(TEXT("")), Font, ConsoleSettings->InputColor);

		FCanvasTileItem ConsoleTileBlack(FVector2D(PosLeft, 0.0f), DefaultTexture_Black->GetResource(), FVector2D(ClipX, Height + PosTop - TextH), BackgroundColorBlack);
		FCanvasTileItem ConsoleTileWhite(FVector2D(PosLeft, 0.0f), DefaultTexture_White->GetResource(), FVector2D(ClipX, Height + PosTop - TextH), BackgroundColorWhite);
		ConsoleTileBlack.BlendMode = SE_BLEND_AlphaBlend;		// Preserve alpha to allow single-pass composite
		ConsoleTileWhite.BlendMode = SE_BLEND_AlphaBlend;		// Preserve alpha to allow single-pass composite

		// figure out which element of the scrollback buffer should appear first (at the bottom of the scrollback area)
		int32 idx = SBHead - SBPos - 1;

		TArray<FString> SelectedLines;

		// while we have enough room to draw another line and there are more lines to draw
		while (y > -TextH && idx >= 0)
		{
			float PenX;
			float PenY;
			float PenZ = 0.1f;
			PenX = PosLeft;
			PenY = PosTop + y;

			// adjust the location for any word wrapping due to long text lines
			if (idx < Scrollback.Num())
			{
				float ScrollLineXL, ScrollLineYL;
				Canvas->StrLen(Font, Scrollback[idx], ScrollLineXL, ScrollLineYL, bDPIAwareStringMeasurement);
				ScrollLineXL /= DPIScale;
				ScrollLineYL /= DPIScale;
				if (ScrollLineYL > TextH)
				{
					y -= (ScrollLineYL - TextH);
					PenX = PosLeft;
					PenY = PosTop + y;
				}

				DrawLine(Canvas, Font, ConsoleTileBlack, ConsoleTileWhite, ConsoleText, Scrollback[idx], PenX, PenY, PosLeft, SelectedLines);
			}
			idx--;
			y -= TextH;
		}

		if (Selection.bCapture && !SelectedLines.IsEmpty())
		{
			Algo::Reverse(SelectedLines);
			const FString SelectedText = FString::Join(SelectedLines, LINE_TERMINATOR);

			if (SelectedText.Len())
			{
				FPlatformApplicationMisc::ClipboardCopy(*SelectedText);
			}

			Selection.bCapture = false;		// Finished copy to clipboard when mouse is released
		}
	}

	{
		// Draw any remaining empty scrollback area to the top of viewport

		FLinearColor BackgroundColor = ConsoleDefs::AutocompleteBackgroundColor.ReinterpretAsLinear();
		BackgroundColor.A = ConsoleSettings->BackgroundOpacityPercentage / 100.0f;

		FCanvasTileItem ConsoleTile(FVector2D(PosLeft, 0.0f), DefaultTexture_Black->GetResource(), FVector2D(ClipX, y + TextH), BackgroundColor);
		ConsoleTile.BlendMode = SE_BLEND_AlphaBlend;			// Preserve alpha to allow single-pass composite

		Canvas->DrawItem(ConsoleTile);
	}

	PostRender_InputLine(Canvas, FIntPoint(PosLeft, PosTop + Height + 6 + TextH));
}

void UConsole::DrawLine(
	UCanvas* Canvas, UFont* Font, FCanvasTileItem& TileBlack, FCanvasTileItem& TileWhite,
	FCanvasTextItem ConsoleText, const FString& Line, float PenX, float PenY, float PosLeft,
	TArray<FString>& SelectedLines)
{
	float ClipX = Canvas->ClipX;
	bool bLineSelected = false;				// Partial line is selected
	bool bLineSelectedWhole = false;		// Whole line is selected
	bool bLineUnselectedWhole = false;		// Whole line is unselected
	bool bLineSelectedDown = false;			// Partial line selected, on mouse down (selection begin)
	bool bLineSelectedUp = false;			// Partial line selected, on mouse up (selection end)
	bool bLineSelectTopToBottom = true;		// Drag selecting down or up
	float PosInLineDown = 0.0f;				// Partial line selected (pixels)
	float PosInLineUp = 0.0f;				// Partial line selected (pixels)
	int32 PosInStringDown = 0;				// Partial line selected (characters)
	int32 PosInStringUp = 0;				// Partial line selected (characters)

	if (Selection.bActive)
	{
		// Check if selection range overlaps current line
		float PenY0 = PenY;
		float PenY1 = PenY + TextH;
		float PosDownY0 = Selection.MousePosDown.Y;
		float PosUpY1 = Selection.bMade ? Selection.MousePosUp.Y : Selection.MousePos.Y;

		// Adjust for new lines added since selection was initiated
		PosDownY0 -= Selection.Offset;
		PosUpY1 -= Selection.Offset;

		bLineSelectedDown = (PosDownY0 >= PenY0) && (PosDownY0 <= PenY1);
		if (bLineSelectedDown)
		{
			// Partial line selection (mouse down)
			GetPosInTextLine(Canvas, Font, Line, Selection.MousePosDown.X, PosInLineDown, PosInStringDown);
		}

		bLineSelectedUp = (PosUpY1 >= PenY0) && (PosUpY1 <= PenY1);
		if (bLineSelectedUp)
		{
			// Partial line selection (mouse up)
			float MousePosUpX = Selection.bMade ? Selection.MousePosUp.X : Selection.MousePos.X;
			GetPosInTextLine(Canvas, Font, Line, MousePosUpX, PosInLineUp, PosInStringUp);
		}

		if (PosDownY0 > PosUpY1)
		{
			// Swap to keep Y-range in order, to allow drag selecting down or up
			Swap(PosDownY0, PosUpY1);

			bLineSelectTopToBottom = false;
			bLineSelectedDown = !bLineSelectedDown;
			bLineSelectedUp = !bLineSelectedUp;
		}

		auto Overlaps = [](float PenY0, float PenY1, float PosDownY0, float PosUpY1)
		{
			return (PosUpY1 >= PenY0) && (PosDownY0 <= PenY1);
		};

		bLineSelected = Overlaps(PenY0, PenY1, PosDownY0, PosUpY1);
		bLineSelectedWhole = (PosUpY1 > PenY1) && (PosDownY0 < PenY0);
		bLineUnselectedWhole = bLineSelectedUp && (PosInLineUp <= 0.0f);
	}

	if (!bLineUnselectedWhole && !bLineSelectedWhole && bLineSelectedDown && bLineSelectedUp)
	{
		// Draw three segments: Black White Black

		if (PosInLineDown > PosInLineUp)
		{
			Swap(PosInLineDown, PosInLineUp);
			Swap(PosInStringDown, PosInStringUp);
		}

		{
			// white text on black background (from left to start of selected text)
			TileBlack.Position = FVector2D(PosLeft, PenY);
			TileBlack.Size = FVector2D(PosInLineDown, TextH);
			Canvas->DrawItem(TileBlack);

			ConsoleText.SetColor(FLinearColor::White);
			FString Segment = Line.Mid(0, PosInStringDown + 1);

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX, PenY);
		}
		{
			// black text on white background (from start of selected text)
			TileWhite.Position = FVector2D(PosInLineDown, PenY);
			TileWhite.Size = FVector2D(PosInLineUp - PosInLineDown, TextH);
			Canvas->DrawItem(TileWhite);

			ConsoleText.SetColor(FLinearColor::Black);
			int32 PosCharDown = (PosInStringDown > 0) ? PosInStringDown + 1 : 0;
			FString Segment = Line.Mid(PosCharDown, PosInStringUp - PosCharDown + 1);

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX + PosInLineDown, PenY);
		}
		{
			// white text on black background (from start of unselected text)
			TileBlack.Position = FVector2D(PosInLineUp, PenY);
			TileBlack.Size = FVector2D(ClipX - PosInLineUp, TextH);
			Canvas->DrawItem(TileBlack);

			ConsoleText.SetColor(FLinearColor::White);
			int32 PosCharUp = (PosInLineUp > 0) ? PosInStringUp + 1 : 0;
			FString Segment = Line.Mid(PosCharUp, Line.Len());

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX + PosInLineUp, PenY);
		}
		if (Selection.bCapture)
		{
			int32 PosCharDown = PosInStringDown + 1;
			FString Segment = Line.Mid(PosCharDown, PosInStringUp - PosCharDown + 1);
			SelectedLines.Add(Segment);
		}
	}
	else if (!bLineUnselectedWhole && !bLineSelectedWhole && bLineSelectedDown)
	{
		// Draw three segments: Black White Black

		float LineWidth = GetPosTextLineEnd(Canvas, Font, Line);

		if (!bLineSelectTopToBottom)
		{
			Swap(PosInLineDown, PosInLineUp);
			Swap(PosInStringDown, PosInStringUp);
		}

		{
			// white text on black background (from left to start of selected text)
			TileBlack.Position = FVector2D(PosLeft, PenY);
			TileBlack.Size = FVector2D(PosInLineDown, TextH);
			Canvas->DrawItem(TileBlack);

			ConsoleText.SetColor(FLinearColor::White);
			FString Segment = Line.Mid(0, PosInStringDown + 1);

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX, PenY);
		}
		{
			// black text on white background (from start of selected text)
			TileWhite.Position = FVector2D(PosInLineDown, PenY);
			TileWhite.Size = FVector2D(LineWidth - PosInLineDown, TextH);
			Canvas->DrawItem(TileWhite);

			ConsoleText.SetColor(FLinearColor::Black);
			int32 PosCharDown = (PosInStringDown > 0) ? PosInStringDown + 1 : 0;
			FString Segment = Line.Mid(PosCharDown, Line.Len());

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX + PosInLineDown, PenY);
		}
		{
			// remaining black background (from end of text to end of line)
			TileBlack.Position = FVector2D(LineWidth, PenY);
			TileBlack.Size = FVector2D(ClipX - LineWidth, TextH);
			Canvas->DrawItem(TileBlack);
		}
		if (Selection.bCapture)
		{
			int32 PosCharDown = (PosInStringDown > 0) ? PosInStringDown + 1 : 0;
			FString Segment = Line.Mid(PosCharDown, Line.Len());
			SelectedLines.Add(Segment);
		}
	}
	else if (!bLineSelectedWhole && bLineSelectedUp)
	{
		// Draw two segments: White Black

		if (!bLineSelectTopToBottom)
		{
			Swap(PosInLineDown, PosInLineUp);
			Swap(PosInStringDown, PosInStringUp);
		}

		{
			// black text on white background (from left to start of unselected text)
			TileWhite.Position = FVector2D(PosLeft, PenY);
			TileWhite.Size = FVector2D(PosInLineUp, TextH);
			Canvas->DrawItem(TileWhite);

			ConsoleText.SetColor(FLinearColor::Black);
			int32 PosCharUp = (PosInLineUp > 0.0f) ? PosInStringUp + 1 : 0;
			FString Segment = Line.Mid(0, PosCharUp);
			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX, PenY);
		}
		{
			// white text on black background (from start of unselected text)
			TileBlack.Position = FVector2D(PosInLineUp, PenY);
			TileBlack.Size = FVector2D(ClipX - PosInLineUp, TextH);
			Canvas->DrawItem(TileBlack);

			ConsoleText.SetColor(FLinearColor::White);
			int32 PosCharUp = (PosInLineUp > 0.0f) ? PosInStringUp + 1 : 0;
			FString Segment = Line.Mid(PosCharUp, Line.Len());

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX + PosInLineUp, PenY);
		}
		if (Selection.bCapture)
		{
			int32 PosCharUp = (PosInLineUp > 0.0f) ? PosInStringUp + 1 : 0;
			FString Segment = Line.Mid(0, PosCharUp);
			if (!Segment.IsEmpty())
			{
				SelectedLines.Add(Segment);
			}
		}
	}
	else if (bLineSelected)
	{
		// Draw two segments: White Black

		float LineWidth = GetPosTextLineEnd(Canvas, Font, Line);

		{
			// black text on white background (from left to end of text)
			TileWhite.Position = FVector2D(PosLeft, PenY);
			TileWhite.Size = FVector2D(LineWidth, TextH);
			Canvas->DrawItem(TileWhite);

			ConsoleText.SetColor(FLinearColor::Black);
			FString Segment = Line.Mid(0, Line.Len());

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX, PenY);
		}
		{
			// remaining black background (from end of text to end of line)
			TileBlack.Position = FVector2D(LineWidth, PenY);
			TileBlack.Size = FVector2D(ClipX - LineWidth, TextH);
			Canvas->DrawItem(TileBlack);
		}
		if (Selection.bCapture)
		{
			SelectedLines.Add(Line);
		}
	}
	else
	{
		// Draw two segments: Black Black

		float LineWidth = GetPosTextLineEnd(Canvas, Font, Line);

		{
			TileBlack.Position = FVector2D(PosLeft, PenY);
			TileBlack.Size = FVector2D(LineWidth, TextH);
			Canvas->DrawItem(TileBlack);

			ConsoleText.SetColor(FLinearColor::White);
			FString Segment = Line.Mid(0, Line.Len());

			ConsoleText.Text = FText::FromString(Segment);
			Canvas->DrawItem(ConsoleText, PenX, PenY);
		}
		{
			// remaining black background (from end of text to end of line)
			TileBlack.Position = FVector2D(LineWidth, PenY);
			TileBlack.Size = FVector2D(ClipX - LineWidth, TextH);
			Canvas->DrawItem(TileBlack);
		}
	}
}

void UConsole::GetPosInTextLine(UCanvas* Canvas, UFont* Font, const FString& Line, float MousePosX, float& PosInLine, int32& PosInString)
{
	// add up character widths to selection position in X, since we are not using a monospace font
	float CharPos = 0.0f;
	for (int32 i = 0; i < Line.Len(); ++i)
	{
		FString LineChar1 = FString::Printf(TEXT("%c"), Line[i]);
		float CharW1, CharH1;
		Canvas->StrLen(Font, LineChar1, CharW1, CharH1, bDPIAwareStringMeasurement);
		CharPos += CharW1;

		if (CharPos > MousePosX)
		{
			break;
		}
		PosInLine = CharPos;
		PosInString = i;
	}
}

float UConsole::GetPosTextLineEnd(UCanvas* Canvas, UFont* Font, const FString& Line)
{
	// add up character widths of whole line in X, since we are not using a monospace font
	float CharPos = 0.0f;
	for (int32 i = 0; i < Line.Len(); ++i)
	{
		FString LineChar1 = FString::Printf(TEXT("%c"), Line[i]);
		float CharW1, CharH1;
		Canvas->StrLen(Font, LineChar1, CharW1, CharH1, bDPIAwareStringMeasurement);

		CharPos += CharW1;
	}

	return CharPos;
}

void UConsole::BeginState_Open(FName PreviousStateName)
{
	bCaptureKeyInput = true;
	//	HistoryCur = HistoryTop;

	SBPos = 0;
	bCtrl = false;
	bShift = false;

	if (PreviousStateName == NAME_None)
	{
		FlushPlayerInput();
	}

	// Ensure we receive IE_Pressed with LeftMouseButton
	PreviousMouseCaptureMode = GetOuterUGameViewportClient()->GetMouseCaptureMode();
	GetOuterUGameViewportClient()->SetMouseCaptureMode(EMouseCaptureMode::CaptureDuringMouseDown);
	Selection.bActive = false;
	Selection.bCapture = false;
	Selection.bMade = false;
	Selection.Offset = 0.0f;
}

void UConsole::EndState_Open(FName NextStateName)
{
	// Restore mouse capture for non-open states (only if nothing else modified it while the console was open otherwise, let's keep the new mode)
	if (GetOuterUGameViewportClient()->GetMouseCaptureMode() == EMouseCaptureMode::CaptureDuringMouseDown)
	{
		GetOuterUGameViewportClient()->SetMouseCaptureMode(PreviousMouseCaptureMode);
	}
}

bool UConsole::InputChar(FInputDeviceId DeviceId, const FString& Unicode)
{
	if (ConsoleState == NAME_Typing)
	{
		return InputChar_Typing(DeviceId, Unicode);
	}
	if (ConsoleState == NAME_Open)
	{
		return InputChar_Open(DeviceId, Unicode);
	}
	return bCaptureKeyInput;
}

bool UConsole::InputKey(FInputDeviceId DeviceId, FKey Key, EInputEvent Event, float AmountDepressed, bool bGamepad)
{
	bool bWasConsumed = InputKey_InputLine(DeviceId, Key, Event, AmountDepressed, bGamepad);

	if (!bWasConsumed)
	{
		if (ConsoleState == NAME_Typing)
		{
			// if the console is open we don't want any other one to consume the input
			return true;
		}
		if (ConsoleState == NAME_Open)
		{
			bWasConsumed = InputKey_Open(DeviceId, Key, Event, AmountDepressed, bGamepad);
			// if the console is open we don't want any other one to consume the input
			return true;
		}
	}

	return bWasConsumed;
}

void UConsole::MouseMove(FViewport* Viewport, int32 X, int32 Y)
{
	Selection.MousePos = FVector2f(X, Y);
}

void UConsole::CapturedMouseMove(FViewport* InViewport, int32 X, int32 Y)
{
	Selection.MousePos = FVector2f(X, Y);
}

void UConsole::PostRender_Console(UCanvas* Canvas)
{
	if (ConsoleState == NAME_None)
	{
		return;
	}

	Canvas->ApplySafeZoneTransform();

	if (ConsoleState == NAME_Typing)
	{
		PostRender_Console_Typing(Canvas);
	}
	else if (ConsoleState == NAME_Open)
	{
		PostRender_Console_Open(Canvas);
	}

	Canvas->PopSafeZoneTransform();
}

void UConsole::PostRender_InputLine(UCanvas* Canvas, FIntPoint UserInputLinePos)
{
	float xl, yl;

	const FString TypedInputText = FString::Printf(TEXT("%s%s"), *ConsoleDefs::LeadingInputText, *TypedStr);
	const FString PrecompletedInputText = !PrecompletedInputLine.IsEmpty() ? PrecompletedInputLine.RightChop(TypedStr.Len()) : FString();

	const float DPIScale = Canvas->GetDPIScale();

	// use the smallest font
	UFont* Font = GEngine->GetSmallFont();
	// determine the size of the input line
	Canvas->StrLen(Font, TypedInputText, xl, yl, bDPIAwareStringMeasurement);
	xl /= DPIScale;
	yl /= DPIScale;

	float ClipX = Canvas->ClipX;
	float ClipY = Canvas->ClipY;

	// Background
	FLinearColor BackgroundColor = ConsoleDefs::AutocompleteBackgroundColor.ReinterpretAsLinear();
	BackgroundColor.A = ConsoleSettings->BackgroundOpacityPercentage / 100.0f;
	FCanvasTileItem ConsoleTile(FVector2D(UserInputLinePos.X, UserInputLinePos.Y - 6 - yl), DefaultTexture_Black->GetResource(), FVector2D(ClipX, yl + 6), FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f), BackgroundColor);

	// Preserve alpha to allow single-pass composite
	ConsoleTile.BlendMode = SE_BLEND_AlphaBlend;

	Canvas->DrawItem(ConsoleTile);

	// Separator line
	ConsoleTile.SetColor(ConsoleDefs::BorderColor);
	ConsoleTile.Texture = DefaultTexture_White->GetResource();
	ConsoleTile.Size = FVector2D(ClipX, 2.0f);
	Canvas->DrawItem(ConsoleTile);

	// Currently typed string
	FText Str = FText::FromString(TypedInputText);
	FCanvasTextItem ConsoleText(FVector2D(UserInputLinePos.X, UserInputLinePos.Y - 3 - yl), Str, Font, ConsoleSettings->InputColor);
	ConsoleText.EnableShadow(FLinearColor::Black);
	Canvas->DrawItem(ConsoleText);

	// Precompleted remainder of the typed string (faded out)
	if (!PrecompletedInputText.IsEmpty())
	{
		ConsoleText.SetColor(ConsoleSettings->AutoCompleteFadedColor);
		ConsoleText.Text = FText::FromString(PrecompletedInputText);
		Canvas->DrawItem(ConsoleText, UserInputLinePos.X + xl, UserInputLinePos.Y - 3 - yl);
	}

	// Draw the autocomplete elements
	if (AutoComplete.Num() > 0)
	{
		int32 StartIdx = AutoCompleteIndex;
		if (StartIdx < 0)
		{
			StartIdx = FMath::Max(0, AutoComplete.Num() + StartIdx);
		}

		Canvas->StrLen(Font, *ConsoleDefs::LeadingInputText, xl, yl, bDPIAwareStringMeasurement);
		xl /= DPIScale;
		yl /= DPIScale;

		float y = UserInputLinePos.Y - 6.0f - (yl * 2.0f);

		// Set the background color/texture of the auto-complete section
		FLinearColor AutoCompleteBackgroundColor = ConsoleDefs::AutocompleteBackgroundColor;
		AutoCompleteBackgroundColor.A = ConsoleSettings->BackgroundOpacityPercentage / 100.0f;
		ConsoleTile.SetColor(AutoCompleteBackgroundColor);
		ConsoleTile.Texture = DefaultTexture_White->GetResource();

		// wasteful memory allocations but when typing in a console command this is fine
		TArray<const FAutoCompleteCommand*> AutoCompleteElements;
		// to avoid memory many allocations
		AutoCompleteElements.Empty(MAX_AUTOCOMPLETION_LINES + 1);

		float MaxLeftWidth = 0;
		float MaxRightWidth = 0;
		for (int32 MatchIdx = 0; MatchIdx < MAX_AUTOCOMPLETION_LINES && MatchIdx < AutoComplete.Num(); MatchIdx++)
		{
			const FAutoCompleteCommand& Cmd = AutoComplete[StartIdx + MatchIdx];
			AutoCompleteElements.Add(&Cmd);

			// Find the longest command and the longest description for left-justification of the descriptions
			float CmdLenX, CmdLenY;
			Canvas->StrLen(Font, Cmd.GetLeft(), CmdLenX, CmdLenY, bDPIAwareStringMeasurement);
			CmdLenX /= DPIScale;
			CmdLenY /= DPIScale;
			MaxLeftWidth = FMath::Max(MaxLeftWidth, CmdLenX);
			if (!Cmd.Desc.IsEmpty())
			{
				float DescLenX, DescLenY;
				Canvas->StrLen(Font, Cmd.GetRight(), DescLenX, DescLenY, bDPIAwareStringMeasurement);
				DescLenX /= DPIScale;
				DescLenY /= DPIScale;
				MaxRightWidth = FMath::Max(MaxRightWidth, ConsoleDefs::AutocompleteGap + DescLenX);
			}
		}

		// Display a message if there were more matches
		if (AutoComplete.Num() > MAX_AUTOCOMPLETION_LINES)
		{
			static FAutoCompleteCommand MoreMatchesLine;

			MoreMatchesLine.Command = FString::Printf(TEXT("[%i more matches]"), (AutoComplete.Num() - MAX_AUTOCOMPLETION_LINES));
			MoreMatchesLine.Color = ConsoleSettings->AutoCompleteFadedColor;
			AutoCompleteElements.Add(&MoreMatchesLine);
		}

		// background rectangle behind auto completion
		float MaxWidth = (MaxLeftWidth + MaxRightWidth);
		float Height = AutoCompleteElements.Num() * yl;
		int32 Border = 4;

		// dark inner part
		ConsoleTile.Size = FVector2D(MaxWidth + 2 * Border, Height + 2 * Border);
		ConsoleTile.SetColor(AutoCompleteBackgroundColor);

		// Preserve alpha to allow single-pass composite
		ConsoleTile.BlendMode = SE_BLEND_AlphaBlend;

		Canvas->DrawItem(ConsoleTile, UserInputLinePos.X + xl - Border, y + yl - Height - Border);

		// white border
		FCanvasBoxItem ConsoleOutline(ConsoleTile.Position, FVector2D(MaxWidth + 2 * Border, Height + 2 * Border));
		ConsoleOutline.SetColor(ConsoleDefs::BorderColor);
		ConsoleOutline.BlendMode = SE_BLEND_Opaque;
		Canvas->DrawItem(ConsoleOutline, UserInputLinePos.X + xl - Border, y + yl - Height - Border);

		// auto completion elements
		auto DrawElement = [&](const FAutoCompleteCommand& AutoCompleteElement, int32 i, int32 Num)
		{
			const bool bCursorLineColor = (i == AutoCompleteCursor);
			const bool bMoreMatches = (Num > MAX_AUTOCOMPLETION_LINES && i == Num - 1);
			const bool bHistory = AutoCompleteElement.IsHistory();
			int CmdXOffset = 0;

			FColor LeftC = AutoCompleteElement.Color;
			FColor RightC = ConsoleSettings->AutoCompleteFadedColor;

			if (bCursorLineColor)
			{
				ConsoleTile.Size = FVector2D(MaxWidth, yl);
				ConsoleTile.SetColor(ConsoleDefs::CursorLineColor);
				ConsoleTile.BlendMode = SE_BLEND_Opaque;
				Canvas->DrawItem(ConsoleTile, UserInputLinePos.X + xl, y);
				LeftC = ConsoleDefs::CursorColor;
			}

			if (bHistory)
			{
				// > HistoryElement has the strings swapped so we need to swap the colors
				Swap(LeftC, RightC);
			}

			float CommandWidth, CommandHeight;
			Canvas->StrLen(Font, AutoCompleteElement.Command, CommandWidth, CommandHeight, bDPIAwareStringMeasurement);
			CommandWidth /= DPIScale;
			CommandHeight /= DPIScale;

			if (bMoreMatches)
			{
				// Center the "x more matches" line, unless that would put it further right than the descriptions
				CmdXOffset = FMath::Min((MaxWidth / 2) - (CommandWidth / 2), MaxLeftWidth + ConsoleDefs::AutocompleteGap);
			}

			ConsoleText.SetColor(LeftC);
			ConsoleText.Text = FText::FromString(AutoCompleteElement.GetLeft());
			Canvas->DrawItem(ConsoleText, UserInputLinePos.X + CmdXOffset + xl, y);

			float DescriptionWidth, DescriptionHeight;
			Canvas->StrLen(Font, AutoCompleteElement.GetRight(), DescriptionWidth, DescriptionHeight, bDPIAwareStringMeasurement);
			DescriptionWidth /= DPIScale;
			DescriptionHeight /= DPIScale;
			
			float DescriptionX = UserInputLinePos.X + xl + ConsoleDefs::AutocompleteGap;
			float DescriptionOverflow = DescriptionX + MaxLeftWidth + DescriptionWidth - Canvas->SizeX;

			if (DescriptionOverflow > 0)
			{
				// Horizontal overflow due to low resolution or an overly long description; forgo justification
				DescriptionX = FMath::Max(DescriptionX + CommandWidth, DescriptionX + MaxLeftWidth - DescriptionOverflow);
			}
			else
			{
				DescriptionX += MaxLeftWidth;
			}

			ConsoleText.SetColor(RightC);
			ConsoleText.Text = FText::FromString(AutoCompleteElement.GetRight());
			Canvas->DrawItem(ConsoleText, DescriptionX, y);
			y -= yl;
		};

		if (ConsoleSettings->bOrderTopToBottom)
		{
			for (int32 Num = AutoCompleteElements.Num(), i = Num - 1; i >= 0; --i)
			{
				DrawElement(*AutoCompleteElements[i], i, Num);
			}
		}
		else
		{
			for (int32 Num = AutoCompleteElements.Num(), i = 0; i < Num; ++i)
			{
				DrawElement(*AutoCompleteElements[i], i, Num);
			}
		}
	}

	// determine the cursor position
	const FString TypedInputTextUpToCursor = FString::Printf(TEXT("%s%s"), *ConsoleDefs::LeadingInputText, *TypedStr.Left(TypedStrPos));
	Canvas->StrLen(Font, TypedInputTextUpToCursor, xl, yl, bDPIAwareStringMeasurement);
	xl /= DPIScale;
	yl /= DPIScale;
	// draw the cursor
	ConsoleText.SetColor(ConsoleDefs::CursorColor);
	ConsoleText.Text = FText::FromString(FString("_"));
	Canvas->DrawItem(ConsoleText, UserInputLinePos.X + xl, UserInputLinePos.Y - 1.0f - yl);

}

bool UConsole::ConsoleActive() const
{
	return (ConsoleState != NAME_None);
}

void UConsole::FakeGotoState(FName NextStateName)
{
	if (ConsoleState == NAME_Typing)
	{
		EndState_Typing(NextStateName);
	}
	else if (ConsoleState == NAME_Open)
	{
		EndState_Open(NextStateName);
	}
	if (NextStateName == NAME_Typing)
	{
		BeginState_Typing(ConsoleState);

		// Console has opened
		OnConsoleActivationStateChanged.Broadcast(true);

		// Save the currently focused widget so that we can restore to it once the console is closed
		PreviousFocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();

		FSlateApplication::Get().ResetToDefaultPointerInputSettings();
		FSlateApplication::Get().SetKeyboardFocus(GetOuterUGameViewportClient()->GetGameViewportWidget());
	}
	else if (NextStateName == NAME_Open)
	{
		BeginState_Open(ConsoleState);
		FSlateApplication::Get().ResetToDefaultPointerInputSettings();

		// Console has opened
		OnConsoleActivationStateChanged.Broadcast(true);
	}
	else if (NextStateName == NAME_None)
	{
		// We need to force the console state name change now otherwise inside the call 
		// to SetKeyboardFocus the console is still considered active
		ConsoleState = NAME_None;
		bCtrl = false;
		bShift = false;

		TSharedPtr<SWidget> WidgetToFocus;
		if (PreviousFocusedWidget.IsValid())
		{
			// Restore focus to whatever was the focus before the console was opened.
			WidgetToFocus = PreviousFocusedWidget.Pin();
		}
		else
		{
			// Since the viewport may not be the current focus, we need to re-focus whatever the current focus is,
			// in order to ensure it gets a chance to reapply any custom input settings
			WidgetToFocus = FSlateApplication::Get().GetKeyboardFocusedWidget();
		}

		if (WidgetToFocus.IsValid())
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
			FSlateApplication::Get().SetKeyboardFocus(WidgetToFocus, EFocusCause::Mouse);
		}

		// Console has closed
		OnConsoleActivationStateChanged.Broadcast(false);
	}
	ConsoleState = NextStateName;
}

void UConsole::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	// e.g. UE_LOGF(LogConsoleResponse, Display, "Test");
	static const FName ConsoleResponseLog = FName("LogConsoleResponse");

	if (Category == ConsoleResponseLog)
	{
		// log all LogConsoleResponse
		OutputText(V);
	}
	else
	{
		static const TConsoleVariableData<int32>* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("con.MinLogVerbosity"));

		if (CVar)
		{
			int MinVerbosity = CVar->GetValueOnAnyThread();

			if ((int)Verbosity <= MinVerbosity)
			{
				// log all that is >= the specified verbosity
				OutputText(V);
			}
		}
	}
}

