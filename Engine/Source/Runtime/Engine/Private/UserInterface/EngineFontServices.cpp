// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineFontServices.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"

FEngineFontServices* FEngineFontServices::Instance = nullptr;

FEngineFontServices::FEngineFontServices()
{
	check(IsInGameThread());

	if (FSlateApplication::IsInitialized())
	{
		if (const FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
		{
			SlateFontServices = Renderer->GetFontServices();
		}
	}
}

FEngineFontServices::~FEngineFontServices()
{
	check(IsInGameThread());

	SlateFontServices.Reset();
}

void FEngineFontServices::Create()
{
	check(!Instance);
	Instance = new FEngineFontServices();
}

void FEngineFontServices::Destroy()
{
	check(Instance);
	delete Instance;
	Instance = nullptr;
}

bool FEngineFontServices::IsInitialized()
{
	return Instance != nullptr;
}

FEngineFontServices& FEngineFontServices::Get()
{
	check(Instance);
	return *Instance;
}

TSharedPtr<FSlateFontCache> FEngineFontServices::GetFontCache()
{
	if (SlateFontServices.IsValid())
	{
		return SlateFontServices->GetFontCache();
	}

	return nullptr;
}

TSharedPtr<FSlateFontMeasure> FEngineFontServices::GetFontMeasure()
{
	if (SlateFontServices.IsValid())
	{
		return SlateFontServices->GetFontMeasureService();
	}

	return nullptr;
}

void FEngineFontServices::UpdateCache()
{
	if (SlateFontServices.IsValid())
	{
		TSharedRef<FSlateFontCache> FontCache = SlateFontServices->GetFontCache();
		FontCache->UpdateCache();
	}
}

FOnReleaseFontResources& FEngineFontServices::OnReleaseResources()
{
	if (SlateFontServices.IsValid())
	{
		return SlateFontServices->OnReleaseResources();
	}

	static FOnReleaseFontResources DummyDelegate;
	return DummyDelegate;
}
