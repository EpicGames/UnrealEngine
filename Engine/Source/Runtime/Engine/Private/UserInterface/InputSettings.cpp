// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InputSettings.cpp: Project configurable input settings
=============================================================================*/

#include "GameFramework/InputSettings.h"
#include "Engine/PlatformSettingsManager.h"
#include "Misc/CommandLine.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectIterator.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputSettings)

#if WITH_EDITOR
#include "Editor.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

namespace UE::Input
{
	static int32 DeviceMappingPolicy = 1;
	static FAutoConsoleVariableRef CVarDeviceMappingPolicy(TEXT("input.DeviceMappingPolicy"),
	DeviceMappingPolicy,
	TEXT("The policy defining the behavior of a newly connected input device. See EInputDeviceMappingPolicy\n")
	TEXT("0 - UseManagedPlatformLogin\n")
	TEXT("1 - PrimaryUserSharesKeyboardAndFirstGamepad (PC Default)\n")
	TEXT("2 - CreateUniquePlatformUserForEachDevice\n")
	TEXT("3 - MapAllDevicesToPrimaryUser\n"),
	ECVF_Default);
};

UInputSettings::UInputSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCaptureMouseOnLaunch(true)
	, bEnableLegacyInputScales(true)
	, bEnableMotionControls(true)
	, bFilterInputByPlatformUser(false)
	, bShouldFlushPressedKeysOnViewportFocusLost(true)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, bEnableDynamicComponentInputBinding(true)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, bEnabledLegacyMappingDeprecationWarnings(true)
	, bEnablePreferredInputAPIPreferences(false)
	, DefaultViewportMouseCaptureMode(EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown)
	, DefaultViewportMouseLockMode(EMouseLockMode::LockOnCapture)
	, DefaultPlayerInputClass(UPlayerInput::StaticClass())
	, DefaultInputComponentClass(UInputComponent::StaticClass())
{
	PlatformSettings.Initialize(UInputPlatformSettings::StaticClass());
}

void UInputSettings::RemoveInvalidKeys()
{
	TArray<int32> InvalidIndices;
	int32 CurrentIndex = 0;
	//detect invalid keys and add them to the array for removal
	for (const FInputActionKeyMapping& KeyMapping : ActionMappings)
	{
		if (!(KeyMapping.Key.IsValid() || (KeyMapping.Key.GetFName() == TEXT("None"))))
		{
			UE_LOGF(LogInput, Warning, "Action %ls uses invalid key %ls.", *KeyMapping.ActionName.ToString(), *KeyMapping.Key.ToString());
			InvalidIndices.Add(CurrentIndex);
		}
		CurrentIndex++;
	}

	if (InvalidIndices.Num())
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("RemoveInvalidKeys")))
		{
			//now remove them
			for (int32 i = InvalidIndices.Num() - 1; i >= 0; --i)
			{
				int32 IndexToRemove = InvalidIndices[i];
				ActionMappings[IndexToRemove].Key = FName();
			}
			//if there were any indices to remove, save the new values
			SaveConfig();
			TryUpdateDefaultConfigFile();
		}
		else
		{
			UE_LOGF(LogInput, Warning, "Use -RemoveInvalidKeys to remove instances of these keys from the action mapping.");
		}
	}
}


void UInputSettings::PostInitProperties()
{
	Super::PostInitProperties();

	PopulateAxisConfigs();
	AddInternationalConsoleKey();

	for (const FInputActionKeyMapping& KeyMapping : ActionMappings)
	{
		if (KeyMapping.Key.IsDeprecated())
		{
			UE_LOGF(LogInput, Warning, "Action %ls uses deprecated key %ls.", *KeyMapping.ActionName.ToString(), *KeyMapping.Key.ToString());
		}
	}

	for (const FInputAxisKeyMapping& KeyMapping : AxisMappings)
	{
		if (KeyMapping.Key.IsDeprecated())
		{
			UE_LOGF(LogInput, Warning, "Axis %ls uses deprecated key %ls.", *KeyMapping.AxisName.ToString(), *KeyMapping.Key.ToString());
		}
	}

	FPlatformApplicationMisc::EnableMotionData(bEnableMotionControls);
}

void UInputSettings::PopulateAxisConfigs()
{
	TMap<FName, int32> UniqueAxisConfigNames;
	for (int32 Index = 0; Index < AxisConfig.Num(); ++Index)
	{
		UniqueAxisConfigNames.Add(AxisConfig[Index].AxisKeyName, Index);
	}

	for (int32 Index = AxisConfig.Num() - 1; Index >= 0; --Index)
	{
		const int32 UniqueAxisIndex = UniqueAxisConfigNames.FindChecked(AxisConfig[Index].AxisKeyName);
		if (UniqueAxisIndex != Index)
		{
			AxisConfig.RemoveAtSwap(Index);
		}
	}

#if WITH_EDITOR
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);
	for (const FKey& Key : AllKeys)
	{
		if (Key.IsAxis1D() && !UniqueAxisConfigNames.Contains(Key.GetFName()))
		{
			FInputAxisConfigEntry NewAxisConfigEntry;
			NewAxisConfigEntry.AxisKeyName = Key.GetFName();
			NewAxisConfigEntry.AxisProperties.DeadZone = 0.f; // Override the default so that we keep existing behavior
			AxisConfig.Add(NewAxisConfigEntry);
		}
	}
#endif
}

void UInputSettings::AddInternationalConsoleKey()
{
#if PLATFORM_WINDOWS
	// If the console key is set to the default we'll see about adding the keyboard default
	if (ConsoleKeys.Num() >= 1 && ConsoleKeys[0] == EKeys::Tilde)
	{
		CHAR KeyboardLayoutName[KL_NAMELENGTH];

		// We need to use this call because it gives the actual layout, rather than the locale
		if (GetKeyboardLayoutNameA(KeyboardLayoutName))
		{
			FKey DefaultConsoleKey = EKeys::Tilde;

			const int32 KeyboardLayoutId = strtol(KeyboardLayoutName, nullptr, 16);

			switch (PRIMARYLANGID(LOWORD(KeyboardLayoutId)))
			{
			case LANG_FRENCH:
			case LANG_HUNGARIAN:
				DefaultConsoleKey = FInputKeyManager::Get().GetKeyFromCodes(VK_OEM_7, 0);
				break;

			case LANG_GERMAN:
				DefaultConsoleKey = FInputKeyManager::Get().GetKeyFromCodes(VK_OEM_5, 0);
				break;

			case LANG_ITALIAN:
				DefaultConsoleKey = EKeys::Backslash;
				break;

			case LANG_SPANISH:
				DefaultConsoleKey = FInputKeyManager::Get().GetKeyFromCodes(VK_OEM_5, 0);
				break;

			case LANG_SLOVAK:
			case LANG_SWEDISH:
				DefaultConsoleKey = EKeys::Section;
				break;

			case LANG_JAPANESE:
			case LANG_RUSSIAN:
				DefaultConsoleKey = FInputKeyManager::Get().GetKeyFromCodes(VK_OEM_3, 0);
				break;
			}

			if (DefaultConsoleKey != EKeys::Tilde && DefaultConsoleKey.IsValid())
			{
				ConsoleKeys.AddUnique(DefaultConsoleKey);
			}
		}
	}
#endif
}

void UInputSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);
#if WITH_EDITOR
	PopulateAxisConfigs();
#endif
	AddInternationalConsoleKey();
}

#if WITH_EDITOR

void UInputSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, ActionMappings) || MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, AxisMappings) ||
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, AxisConfig) || MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, SpeechMappings))
	{
		ForceRebuildKeymaps();
		FEditorDelegates::OnActionAxisMappingsChanged.Broadcast();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, bEnableGestureRecognizer))
	{
		FEditorDelegates::OnEnableGestureRecognizerChanged.Broadcast();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputSettings, bEnableMotionControls))
	{
		FPlatformApplicationMisc::EnableMotionData(bEnableMotionControls);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif

void UInputSettings::SaveKeyMappings()
{
	ActionMappings.Sort();
	AxisMappings.Sort();
	SaveConfig();
}

UInputSettings* UInputSettings::GetInputSettings()
{
	return GetMutableDefault<UInputSettings>();
}

void UInputSettings::AddActionMapping(const FInputActionKeyMapping& KeyMapping, const bool bForceRebuildKeymaps)
{
	ActionMappings.AddUnique(KeyMapping);
	if (bForceRebuildKeymaps)
	{
		ForceRebuildKeymaps();
	}
}

void UInputSettings::GetActionMappingByName(const FName InActionName, TArray<FInputActionKeyMapping>& OutMappings) const
{
	if (InActionName.IsValid())
	{
		for (int32 ActionIndex = ActionMappings.Num() - 1; ActionIndex >= 0; --ActionIndex)
		{
			if (ActionMappings[ActionIndex].ActionName == InActionName)
			{
				OutMappings.Add(ActionMappings[ActionIndex]);
				// we don't break because the mapping may have been in the array twice
			}
		}
	}
}

void UInputSettings::RemoveActionMapping(const FInputActionKeyMapping& KeyMapping, const bool bForceRebuildKeymaps)
{
	for (int32 ActionIndex = ActionMappings.Num() - 1; ActionIndex >= 0; --ActionIndex)
	{
		if (ActionMappings[ActionIndex] == KeyMapping)
		{
			ActionMappings.RemoveAt(ActionIndex);
			// we don't break because the mapping may have been in the array twice
		}
	}
	if (bForceRebuildKeymaps)
	{
		ForceRebuildKeymaps();
	}
}

void UInputSettings::AddAxisMapping(const FInputAxisKeyMapping& KeyMapping, const bool bForceRebuildKeymaps)
{
	AxisMappings.AddUnique(KeyMapping);
	if (bForceRebuildKeymaps)
	{
		ForceRebuildKeymaps();
	}
}

void UInputSettings::GetAxisMappingByName(const FName InAxisName, TArray<FInputAxisKeyMapping>& OutMappings) const
{
	if (InAxisName.IsValid())
	{
		for (int32 AxisIndex = AxisMappings.Num() - 1; AxisIndex >= 0; --AxisIndex)
		{
			if (AxisMappings[AxisIndex].AxisName == InAxisName)
			{
				OutMappings.Add(AxisMappings[AxisIndex]);
				// we don't break because the mapping may have been in the array twice
			}
		}
	}
}

void UInputSettings::RemoveAxisMapping(const FInputAxisKeyMapping& InKeyMapping, const bool bForceRebuildKeymaps)
{
	for (int32 AxisIndex = AxisMappings.Num() - 1; AxisIndex >= 0; --AxisIndex)
	{
		const FInputAxisKeyMapping& KeyMapping = AxisMappings[AxisIndex];
		if (KeyMapping.AxisName == InKeyMapping.AxisName
			&& KeyMapping.Key == InKeyMapping.Key)
		{
			AxisMappings.RemoveAt(AxisIndex);
			// we don't break because the mapping may have been in the array twice
		}
	}
	if (bForceRebuildKeymaps)
	{
		ForceRebuildKeymaps();
	}
}

void UInputSettings::GetActionNames(TArray<FName>& ActionNames) const
{
	ActionNames.Reset();

	for (const FInputActionKeyMapping& ActionMapping : ActionMappings)
	{
		ActionNames.AddUnique(ActionMapping.ActionName);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const FInputActionSpeechMapping& SpeechMapping : SpeechMappings)
	{
		ActionNames.AddUnique(SpeechMapping.GetActionName());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UInputSettings::IsActionMappingDeprecated(const FName ActionName) const
{
	return DeprecatedActionAndAxisNames.Contains(ActionName);
}

void UInputSettings::SetActionMappingDeprecated(const FName ActionName, const bool bIsDeprecated)
{
	if (bIsDeprecated)
	{
		DeprecatedActionAndAxisNames.Add(ActionName);
	}
	else
	{
		DeprecatedActionAndAxisNames.Remove(ActionName);
	}
}

void UInputSettings::GetAxisNames(TArray<FName>& AxisNames) const
{
	AxisNames.Reset();

	for (const FInputAxisKeyMapping& AxisMapping : AxisMappings)
	{
		AxisNames.AddUnique(AxisMapping.AxisName);
	}
}

#if WITH_EDITOR
const TArray<FName>& UInputSettings::GetAllActionAndAxisNames()
{
	static TArray<FName> OutNames;
	OutNames.Reset();

	const UInputSettings* InputSettings = GetDefault<UInputSettings>();
	check(InputSettings);

	TArray<FName> ActionNames;
	InputSettings->GetActionNames(ActionNames);
	OutNames.Append(ActionNames);
	
	InputSettings->GetAxisNames(ActionNames);
	OutNames.Append(ActionNames);
	
	return OutNames;
}
#endif // WITH_EDITOR

void UInputSettings::ForceRebuildKeymaps()
{
	for (TObjectIterator<UPlayerInput> It; It; ++It)
	{
		It->ForceRebuildingKeyMaps(true);
	}
}


FName UInputSettings::GetUniqueActionName(const FName BaseActionMappingName)
{
	static int32 NewMappingCount = 0;
	FName NewActionMappingName;
	bool bFoundUniqueName;

	do
	{
		// Create a numbered name and check whether it's already been used
		NewActionMappingName = FName(BaseActionMappingName, ++NewMappingCount);
		bFoundUniqueName = true;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bFoundUniqueName = !(DoesActionExist(NewActionMappingName) || DoesSpeechExist(NewActionMappingName));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	} while (!bFoundUniqueName);

	return NewActionMappingName;
}

FName UInputSettings::GetUniqueAxisName(const FName BaseAxisMappingName)
{
	static int32 NewMappingCount = 0;
	FName NewAxisMappingName;
	bool bFoundUniqueName;

	do
	{
		// Create a numbered name and check whether it's already been used
		NewAxisMappingName = FName(BaseAxisMappingName, ++NewMappingCount);
		bFoundUniqueName = true;
		for (int32 Index = 0; Index < AxisMappings.Num(); ++Index)
		{
			if (AxisMappings[Index].AxisName == NewAxisMappingName)
			{
				bFoundUniqueName = false;
				break;
			}
		}
	} while (!bFoundUniqueName);

	return NewAxisMappingName;
}

void UInputSettings::AddActionMapping(FInputActionKeyMapping& NewMapping)
{
	ActionMappings.Add(NewMapping);
}

void UInputSettings::AddAxisMapping(FInputAxisKeyMapping& NewMapping)
{
	AxisMappings.Add(NewMapping);
}

/** Ask for all the action mappings */
const TArray <FInputActionKeyMapping>& UInputSettings::GetActionMappings() const
{
	return ActionMappings;
}

/** Ask for all the axis mappings */
const TArray <FInputAxisKeyMapping>& UInputSettings::GetAxisMappings() const
{
	return AxisMappings;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TArray <FInputActionSpeechMapping>& UInputSettings::GetSpeechMappings() const
{
	return SpeechMappings;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FMatchMappingByName
{
	FMatchMappingByName(const FName InName)
		: Name(InName)
	{
	}

	bool operator() (const FInputActionKeyMapping& ActionMapping)
	{
		return ActionMapping.ActionName == Name;
	}

	bool operator() (const FInputAxisKeyMapping& AxisMapping)
	{
		return AxisMapping.AxisName == Name;
	}

	bool operator() (const FInputActionSpeechMapping& SpeechMapping)
	{
		return SpeechMapping.GetActionName() == Name;
	}

	FName Name;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Finds unique action name based on existing action names */
bool UInputSettings::DoesActionExist(const FName InActionName)
{
	return (ActionMappings.FindByPredicate(FMatchMappingByName(InActionName)) != nullptr);
}

/** Finds unique axis name based on existing action names */
bool UInputSettings::DoesAxisExist(const FName InAxisName)
{
	return (AxisMappings.FindByPredicate(FMatchMappingByName(InAxisName)) != nullptr);
}

/** Finds unique axis name based on existing action names */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UInputSettings::DoesSpeechExist(const FName InSpeechName)
{
	return (SpeechMappings.FindByPredicate(FMatchMappingByName(InSpeechName)) != nullptr);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Get the member name for the details panel */
const FName UInputSettings::GetActionMappingsPropertyName()
{
	static const FName ActionMappingsName = GET_MEMBER_NAME_CHECKED(UInputSettings, ActionMappings);
	return ActionMappingsName;
}

/** Get the member name for the details panel */
const FName UInputSettings::GetAxisMappingsPropertyName()
{
	static const FName AxisMappingsName = GET_MEMBER_NAME_CHECKED(UInputSettings, AxisMappings);
	return AxisMappingsName;
}

UClass* UInputSettings::GetDefaultPlayerInputClass()
{
	TSoftClassPtr<UPlayerInput> Class = UInputSettings::GetInputSettings()->DefaultPlayerInputClass;
	ensureMsgf(Class.IsValid(), TEXT("Invalid PlayerInput class in Input Settings. Manual reset required."));
	return Class.IsValid() ? Class.Get() : UPlayerInput::StaticClass();
}

UClass* UInputSettings::GetDefaultInputComponentClass()
{
	TSoftClassPtr<UInputComponent> Class = UInputSettings::GetInputSettings()->DefaultInputComponentClass;
	ensureMsgf(Class.IsValid(), TEXT("Invalid InputComponent class in Input Settings. Manual reset required."));
	return Class.IsValid() ? Class.Get() : UInputComponent::StaticClass();
}

void UInputSettings::SetDefaultPlayerInputClass(TSubclassOf<UPlayerInput> NewDefaultPlayerInputClass)
{
	if(ensure(NewDefaultPlayerInputClass))
	{
		UInputSettings* InputSettings = Cast<UInputSettings>(UInputSettings::StaticClass()->GetDefaultObject());
		InputSettings->DefaultPlayerInputClass = NewDefaultPlayerInputClass;
	}
}

void UInputSettings::SetDefaultInputComponentClass(TSubclassOf<UInputComponent> NewDefaultInputComponentClass)
{
	if(ensure(NewDefaultInputComponentClass))
	{
		UInputSettings* InputSettings = Cast<UInputSettings>(UInputSettings::StaticClass()->GetDefaultObject());
		InputSettings->DefaultInputComponentClass = NewDefaultInputComponentClass;
	}
}

/////////////////////////////////////////////////////////////
// FHardwareDeviceIdentifier

FHardwareDeviceIdentifier FHardwareDeviceIdentifier::Invalid =
	{
		/* .InputClassName= */ NAME_None,
		/* .HardwareDeviceIdentifier= */ NAME_None,
		/* .PrimaryDeviceType= */ EHardwareDevicePrimaryType::Unspecified,
		/* .SupportedFeaturesMask= */ EHardwareDeviceSupportedFeatures::Type::Unspecified
	};

FHardwareDeviceIdentifier FHardwareDeviceIdentifier::DefaultKeyboardAndMouse = 
	{
		/* .InputClassName= */ TEXT("DefaultKeyboardAndMouse"),
		/* .HardwareDeviceIdentifier= */ TEXT("KBM"),
		/* .PrimaryDeviceType= */ EHardwareDevicePrimaryType::KeyboardAndMouse,
		/* .SupportedFeaturesMask= */ EHardwareDeviceSupportedFeatures::Type::Keypress | EHardwareDeviceSupportedFeatures::Type::Pointer
	};

FHardwareDeviceIdentifier FHardwareDeviceIdentifier::DefaultGamepad = 
	{
	/* .InputClassName= */ TEXT("DefaultGamepad"),
	/* .HardwareDeviceIdentifier= */ TEXT("Gamepad"),
	/* .PrimaryDeviceType= */ EHardwareDevicePrimaryType::Gamepad,
	/* .SupportedFeaturesMask= */  EHardwareDeviceSupportedFeatures::Type::Gamepad
};

FHardwareDeviceIdentifier FHardwareDeviceIdentifier::DefaultMobileTouch = 
	{
	/* .InputClassName= */ TEXT("DefaultMobileTouch"),
	/* .HardwareDeviceIdentifier= */ TEXT("MobileTouch"),
	/* .PrimaryDeviceType= */ EHardwareDevicePrimaryType::Touch,
	/* .SupportedFeaturesMask= */ EHardwareDeviceSupportedFeatures::Type::Touch
};

// Default to the invalid hardware device identifier
FHardwareDeviceIdentifier::FHardwareDeviceIdentifier()
	: InputClassName(Invalid.InputClassName)
	, HardwareDeviceIdentifier(Invalid.HardwareDeviceIdentifier)
	, PrimaryDeviceType(EHardwareDevicePrimaryType::Unspecified)
	, SupportedFeaturesMask(EHardwareDeviceSupportedFeatures::Type::Unspecified)
{
	
}

FHardwareDeviceIdentifier::FHardwareDeviceIdentifier(const FName InClassName, const FName InHardwareDeviceIdentifier, EHardwareDevicePrimaryType InPrimaryType, EHardwareDeviceSupportedFeatures::Type Flags)
	: InputClassName(InClassName)
	, HardwareDeviceIdentifier(InHardwareDeviceIdentifier)
	, PrimaryDeviceType(InPrimaryType)
	, SupportedFeaturesMask(Flags)
{

}

uint32 GetTypeHash(const FHardwareDeviceIdentifier& InDevice)
{
	return HashCombine(GetTypeHash(InDevice.InputClassName), GetTypeHash(InDevice.HardwareDeviceIdentifier));
}

FArchive& operator<<(FArchive& Ar, FHardwareDeviceIdentifier& InDevice)
{
	Ar << InDevice.InputClassName;
	Ar << InDevice.HardwareDeviceIdentifier;
	Ar << InDevice.PrimaryDeviceType;
	Ar << InDevice.SupportedFeaturesMask;
	
	return Ar;
}

bool FHardwareDeviceIdentifier::HasAnySupportedFeatures(const EHardwareDeviceSupportedFeatures::Type FlagsToCheck) const
{
	return (SupportedFeaturesMask & static_cast<int32>(FlagsToCheck)) != 0;
}

bool FHardwareDeviceIdentifier::HasAllSupportedFeatures(const EHardwareDeviceSupportedFeatures::Type FlagsToCheck) const
{
	return ((SupportedFeaturesMask & FlagsToCheck) == FlagsToCheck);
}

bool FHardwareDeviceIdentifier::IsValid() const
{
	return InputClassName.IsValid() && HardwareDeviceIdentifier.IsValid();
}

FString FHardwareDeviceIdentifier::ToString() const
{
	return FString::Printf(TEXT("%s::%s"), *InputClassName.ToString(), *HardwareDeviceIdentifier.ToString());
}

bool FHardwareDeviceIdentifier::operator==(const FHardwareDeviceIdentifier& Other) const
{
	return
		Other.InputClassName == InputClassName &&
		Other.HardwareDeviceIdentifier == HardwareDeviceIdentifier &&
		Other.SupportedFeaturesMask == SupportedFeaturesMask &&
		Other.PrimaryDeviceType == PrimaryDeviceType;	
}

bool FHardwareDeviceIdentifier::operator!=(const FHardwareDeviceIdentifier& Other) const
{
	return !(Other == *this);
}

//////////////////////////////////////////////////////////////////
// UInputPlatformSettings

UInputPlatformSettings::UInputPlatformSettings()
	: MaxPlatformUserCount(8)
	, DeviceMappingPolicy(EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad)
	, MaxTriggerFeedbackPosition(8)
	, MaxTriggerFeedbackStrength(8)
	, MaxTriggerVibrationTriggerPosition(9)
	, MaxTriggerVibrationFrequency(255)
	, MaxTriggerVibrationAmplitude(8)
{
	// Add the default invalid and KBM hardware device ID's here so that they will
	// be found if no custom devices exist when you call GetHardwareDeviceForClassName
	HardwareDevices.AddUnique(FHardwareDeviceIdentifier::Invalid);
	HardwareDevices.AddUnique(FHardwareDeviceIdentifier::DefaultKeyboardAndMouse);
	HardwareDevices.AddUnique(FHardwareDeviceIdentifier::DefaultGamepad);
	HardwareDevices.AddUnique(FHardwareDeviceIdentifier::DefaultMobileTouch);
}

UInputPlatformSettings* UInputPlatformSettings::Get()
{
	return UPlatformSettingsManager::Get().GetSettingsForPlatform<UInputPlatformSettings>();
}

const FHardwareDeviceIdentifier* UInputPlatformSettings::GetHardwareDeviceForClassName(const FName InHardwareDeviceIdentifier) const
{
	return HardwareDevices.FindByPredicate(
		[InHardwareDeviceIdentifier](const FHardwareDeviceIdentifier& Hardware)
		{
			return Hardware.HardwareDeviceIdentifier == InHardwareDeviceIdentifier;
		});
}

void UInputPlatformSettings::AddHardwareDeviceIdentifier(const FHardwareDeviceIdentifier& InHardwareDevice)
{
	if (ensure(InHardwareDevice.IsValid()))
	{
		HardwareDevices.AddUnique(InHardwareDevice);
	}
}

const TArray<FHardwareDeviceIdentifier>& UInputPlatformSettings::GetHardwareDevices() const
{
	return HardwareDevices;
}

#if WITH_EDITOR
void UInputPlatformSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
	const FName MemberPropertyName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();
	
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UInputPlatformSettings, DeviceMappingPolicy))
	{
		IPlatformInputDeviceMapper::Get().HandleInputDevicePolicyChanged(DeviceMappingPolicy);
	}
}

const TArray<FName>& UInputPlatformSettings::GetAllHardwareDeviceNames()
{
	static TArray<FName> HardwareDevices;
	HardwareDevices.Reset();

	// Add the keyboard and mouse by default for everything
	HardwareDevices.Add(FHardwareDeviceIdentifier::DefaultKeyboardAndMouse.HardwareDeviceIdentifier);

	// Get every known platform's InputPlatformSettings and compile a list of them
	TArray<TObjectPtr<UPlatformSettings>> AllInputSettings = UPlatformSettingsManager::Get().GetAllPlatformSettings<UInputPlatformSettings>();

	for (const TObjectPtr<UPlatformSettings>& Setting : AllInputSettings)
	{
		if (const UInputPlatformSettings* InputSetting = Cast<UInputPlatformSettings>(Setting))
		{
			for (const FHardwareDeviceIdentifier& Device : InputSetting->HardwareDevices)
			{
				HardwareDevices.AddUnique(Device.HardwareDeviceIdentifier);
			}
		}
	}

	return HardwareDevices;
}

TArray<FHardwareDeviceIdentifier> UInputPlatformSettings::GetAllHardwareDeviceIdentifiers()
{
	TArray<FHardwareDeviceIdentifier> Result;

	// Always include the built-in defaults.
	Result.AddUnique(FHardwareDeviceIdentifier::DefaultKeyboardAndMouse);
	Result.AddUnique(FHardwareDeviceIdentifier::DefaultGamepad);
	Result.AddUnique(FHardwareDeviceIdentifier::DefaultMobileTouch);

	TArray<TObjectPtr<UPlatformSettings>> AllInputSettings = UPlatformSettingsManager::Get().GetAllPlatformSettings<UInputPlatformSettings>();

	for (const TObjectPtr<UPlatformSettings>& Setting : AllInputSettings)
	{
		if (const UInputPlatformSettings* InputSetting = Cast<UInputPlatformSettings>(Setting))
		{
			for (const FHardwareDeviceIdentifier& Device : InputSetting->HardwareDevices)
			{
				Result.AddUnique(Device);
			}
		}
	}

	return Result;
}
#endif	// WITH_EDITOR
