// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PlayerInput.cpp: Unreal input system.
=============================================================================*/

#include "GameFramework/PlayerInput.h"
#include "GameFramework/PlayerInputDebugging.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Engine/World.h"
#include "Misc/App.h"
#include "GlobalRenderResources.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Canvas.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/InputSettings.h"

#if UE_ENABLE_DEBUG_DRAWING
#include "Engine/GameViewportClient.h"	// For grabbing some info about mouse capture for the debug display
#endif	// UE_ENABLE_DEBUG_DRAWING

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlayerInput)

DECLARE_CYCLE_STAT(TEXT("    PC Gesture Recognition"), STAT_PC_GestureRecognition, STATGROUP_PlayerController);

DEFINE_LOG_CATEGORY(LogPlayerInput);

bool bExecutingBindCommand = false;

/** for debug rendering */
static float DebugUnSmoothedMouseX = 0.f;
static float DebugSmoothedMouseX = 0.f;
static float DebugUnSmoothedMouseY = 0.f;
static float DebugSmoothedMouseY = 0.f;
static float DebugSmoothedMouseSensitivity = 8.f;

const TArray<FInputActionKeyMapping> UPlayerInput::NoKeyMappings;
const TArray<FInputAxisKeyMapping> UPlayerInput::NoAxisMappings;
TArray<FInputActionKeyMapping> UPlayerInput::EngineDefinedActionMappings;
TArray<FInputAxisKeyMapping> UPlayerInput::EngineDefinedAxisMappings;

namespace UE
{
	namespace Input
	{
		static bool AxisEventsCanBeConsumed = true;
		static FAutoConsoleVariableRef CVarShouldOnlyTriggerLastActionInChord(TEXT("Input.AxisEventsCanBeConsumed"),
			AxisEventsCanBeConsumed,
			TEXT("If true and all FKey's for a given Axis Event are consumed, then the axis delegate will not fire."));

		static bool bAutoReconcilePressedEventsOnFirstRepeat = true;
		static FAutoConsoleVariableRef CVarAutoReconcilePressedEventsOnFirstRepeat(TEXT("Input.AutoReconcilePressedEventsOnFirstRepeat"),
			bAutoReconcilePressedEventsOnFirstRepeat,
			TEXT("If true, then we will automatically mark a IE_Pressed event if we receive an IE_Repeat event but have not received a pressed event first.\nNote: This option will be removed in a future update."));

		static bool bClearAxisValueIfConsumed = true;
		static FAutoConsoleVariableRef CVarClearAxisValueIfConsumed(TEXT("Input.ClearAxisValueIfConsumed"),
			bClearAxisValueIfConsumed,
			TEXT("If true, we will clear the value of any FInputAxisKeyBinding whose FKey has been previously consumed.\nNote: This option will be removed in a future update."));
			
		static bool bMaintainLegacyDelegateExecutionOrder = false;
		static FAutoConsoleVariableRef CVarMaintainLegacyDelegateExecutionOrder(TEXT("Input.bMaintainLegacyDelegateExecutionOrder"),
			bMaintainLegacyDelegateExecutionOrder,
			TEXT("If true, legacy input delegates will not fire after processing each UInputComponent, but instead wait until every UInputComponent has been processed."));

		const TCHAR* LexToString(const EInputEvent Event)
		{
			switch (Event)
			{
			case IE_Pressed: return TEXT("IE_Pressed");
			case IE_Released: return TEXT("IE_Released");
			case IE_Repeat: return TEXT("IE_Repeat");
			case IE_DoubleClick: return TEXT("IE_DoubleClick");
			case IE_Axis: return TEXT("IE_Axis");
			case IE_MAX: return TEXT("IE_MAX");
			default: return TEXT("Unknown");
			}
		}
	}
}

/** Runtime struct that gathers up the different kinds of delegates that might be issued */
struct FDelegateDispatchDetails
{
	uint32 EventIndex;
	uint32 FoundIndex;

	FInputActionUnifiedDelegate ActionDelegate;
	const FInputActionBinding* SourceAction;
	FInputChord Chord;
	TEnumAsByte<EInputEvent> KeyEvent;

	FInputTouchUnifiedDelegate TouchDelegate;
	FVector TouchLocation;
	ETouchIndex::Type FingerIndex;

	FInputGestureUnifiedDelegate GestureDelegate;
	float GestureValue;

	FDelegateDispatchDetails(const uint32 InEventIndex, const uint32 InFoundIndex, const FInputChord& InChord, FInputActionUnifiedDelegate InDelegate, const EInputEvent InKeyEvent, const FInputActionBinding* InSourceAction = NULL)
		: EventIndex(InEventIndex)
		, FoundIndex(InFoundIndex)
		, ActionDelegate(MoveTemp(InDelegate))
		, SourceAction(InSourceAction)
		, Chord(InChord)
		, KeyEvent(InKeyEvent)
	{}

	FDelegateDispatchDetails(const uint32 InEventIndex, const uint32 InFoundIndex, FInputTouchUnifiedDelegate InDelegate, const FVector InLocation, const ETouchIndex::Type InFingerIndex)
		: EventIndex(InEventIndex)
		, FoundIndex(InFoundIndex)
		, SourceAction(nullptr)
		, TouchDelegate(MoveTemp(InDelegate))
		, TouchLocation(InLocation)
		, FingerIndex(InFingerIndex)
	{}

	FDelegateDispatchDetails(const uint32 InEventIndex, const uint32 InFoundIndex, FInputGestureUnifiedDelegate InDelegate, const float InValue)
		: EventIndex(InEventIndex)
		, FoundIndex(InFoundIndex)
		, SourceAction(nullptr)
		, GestureDelegate(MoveTemp(InDelegate))
		, GestureValue(InValue)
	{}
};

UPlayerInput::UPlayerInput()
	: Super()
{
	SetFlags(RF_Transactional);
	MouseSamplingTotal = +0.0083f;
	MouseSamples = 1;
}

void UPlayerInput::PostInitProperties()
{
	Super::PostInitProperties();
	ForceRebuildingKeyMaps(true);
}

void UPlayerInput::FlushPressedKeys()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPlayerInput::FlushPressedKeys);

	UE_LOGF(LogPlayerInput, Verbose, "[%s] Flush pressed keys on player input object %ls", __func__, *GetNameSafe(this));
	
	APlayerController* PlayerController = GetOuterAPlayerController();
	ULocalPlayer* LocalPlayer = PlayerController ? Cast<ULocalPlayer>(PlayerController->Player) : nullptr;
	if (LocalPlayer != nullptr)
	{
		TArray<FKey> PressedKeys;

		for (TMap<FKey,FKeyState>::TIterator It(KeyStateMap); It; ++It)
		{
			const FKeyState& KeyState = It.Value();
			if (KeyState.bDown)
			{
				PressedKeys.Add(It.Key());
			}
		}

		// we may have gotten here as a result of executing an input bind.  in order to ensure that the simulated IE_Released events
		// we're about to fire are actually propagated to the game, we need to clear the bExecutingBindCommand flag
		if (PressedKeys.Num() > 0)
		{
			bExecutingBindCommand = false;
			
			for (int32 KeyIndex = 0; KeyIndex < PressedKeys.Num(); KeyIndex++)
			{
				FKey& Key = PressedKeys[KeyIndex];
				
				FInputKeyEventArgs Params = FInputKeyEventArgs::CreateSimulated(Key, IE_Released, 0.0f, 1);

				InputKey(Params);
			}
		}
	}

	UWorld* World = GetWorld();
	check(World);
	float TimeSeconds = World->GetRealTimeSeconds();
	for (TMap<FKey,FKeyState>::TIterator It(KeyStateMap); It; ++It)
	{
		FKeyState& KeyState = It.Value();
		KeyState.RawValue = FVector(0.f, 0.f, 0.f);
		KeyState.bDown = false;
		KeyState.bDownPrevious = false;
		KeyState.LastUpDownTransitionTime = TimeSeconds;
		// Flag each key as having been flushed this frame so that we can correctly evaluate it next time
		KeyState.bWasJustFlushed = true;
	}

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
	UE::Input::FPlayerInputDebugging::BroadcastPlayerInputFlushed(this);
#endif
}

void UPlayerInput::FlushPressedActionBindingKeys(FName ActionName)
{
	//need an action name and a local player to move forward
	APlayerController* PlayerController = (ActionName != NAME_None) ? GetOuterAPlayerController() : nullptr;
	ULocalPlayer* LocalPlayer = PlayerController ? Cast<ULocalPlayer>(PlayerController->Player) : nullptr;
	if (!LocalPlayer)
	{
		return;
	}

	//there can't be more than 32 keys...
	TArray<FKey, TInlineAllocator<32>> AssociatedPressedKeys;

	//grab the action key details
	if (const FActionKeyDetails* KeyDetails = ActionKeyMap.Find(ActionName))
	{
		//go through the details
		for (const FInputActionKeyMapping& KeyMapping : KeyDetails->Actions)
		{
			//grab out key state
			if (const FKeyState* KeyState = KeyStateMap.Find(KeyMapping.Key))
			{
				//if the key is down, add it to the associated keys array
				if (KeyState->bDown)
				{
					AssociatedPressedKeys.AddUnique(KeyMapping.Key);
				}
			}
		}
	}

	//if there are no keys, nothing to do here
	if (AssociatedPressedKeys.Num() > 0)
	{
		// we may have gotten here as a result of executing an input bind.  in order to ensure that the simulated IE_Released events
		// we're about to fire are actually propagated to the game, we need to clear the bExecutingBindCommand flag
		bExecutingBindCommand = false;
		
		//go through all the keys, releasing them
		for (const FKey& Key : AssociatedPressedKeys)
		{
			InputKey(FInputKeyEventArgs::CreateSimulated(Key, IE_Released, 0.0f));
		}

		UWorld* World = GetWorld();
		check(World);
		float TimeSeconds = World->GetRealTimeSeconds();

		//go through the details
		for (const FKey& Key : AssociatedPressedKeys)
		{
			//grab out key state
			if (FKeyState* KeyState = KeyStateMap.Find(Key))
			{
				KeyState->RawValue = FVector(0.f, 0.f, 0.f);
				KeyState->bDown = false;
				KeyState->bDownPrevious = false;
				KeyState->LastUpDownTransitionTime = TimeSeconds;
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UPlayerInput::InputKey(const FInputKeyParams& Params)
{
	if (Params.Delta.Y != 0.0 || Params.Delta.Z != 0.0)
	{
		UE_LOGF(LogPlayerInput, Warning, "Call to the deprecated version of UPlayerInput::InputKey will no longer consider the YZ components of the input delta. Use paired key axis instead.");
	}
	
	FInputKeyEventArgs NewArgs(
		/*Viewport*/ nullptr,
		Params.InputDevice,
		Params.Key,
		/*Delta*/Params.Delta.X,
		Params.DeltaTime,
		Params.NumSamples,
		/*timestamp*/0u);

	NewArgs.Event = Params.Event;
	
	return InputKey(NewArgs);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPlayerInput::InputKey(const FInputKeyEventArgs& Params)
{
	UE_LOGF(LogPlayerInput, VeryVerbose, "[%s] Frame: %llu %ls (outer: %ls) received input : Key: %ls Value: %.4f Event:  %ls",
		__func__,
		GFrameCounter,
		*GetNameSafe(this),
		*GetNameSafe(GetOuter()),
		*Params.Key.GetFName().ToString(),
		Params.AmountDepressed,
		UE::Input::LexToString(Params.Event));

	// MouseX and MouseY should not be treated as analog if there are no samples, as they need their EventAccumulator to be incremented 
	// in the case of a IE_Pressed or IE_Released
	const bool bTreatAsAnalog =
		Params.Key.IsAnalog() &&
		((Params.Key != EKeys::MouseX && Params.Key != EKeys::MouseY) || Params.NumSamples > 0);
	
	if(bTreatAsAnalog)
	{
		ensure((Params.Key != EKeys::MouseX && Params.Key != EKeys::MouseY) || Params.NumSamples > 0);

		auto TestEventEdges = [this, &Params](FKeyState& TestKeyState, float EdgeValue)
		{
			// look for event edges
			if (EdgeValue == 0.f && Params.AmountDepressed != 0.0f)
			{
				TestKeyState.EventAccumulator[IE_Pressed].Add(++EventCount);
			}
			else if (EdgeValue != 0.f && Params.AmountDepressed == 0.0f)
			{
				TestKeyState.EventAccumulator[IE_Released].Add(++EventCount);
			}
			else
			{
				TestKeyState.EventAccumulator[IE_Repeat].Add(++EventCount);
			}
		};

		{
			// first event associated with this key, add it to the map
			FKeyState& KeyState = KeyStateMap.FindOrAdd(Params.Key);

			TestEventEdges(KeyState, KeyState.Value.X);

			// accumulate deltas until processed next
			KeyState.SampleCountAccumulator += Params.NumSamples;
			KeyState.RawValueAccumulator.X += Params.AmountDepressed;
		}

		// Mirror the key press to any associated paired axis
		FKey PairedKey = Params.Key.GetPairedAxisKey();
		if (PairedKey.IsValid())
		{
			//UE_LOGF(LogTemp, Warning, "Spotted axis %ls which is paired to %ls", *Key.GetDisplayName().ToString(), Key.GetPairedAxisKey().IsValid() ? *Key.GetPairedAxisKey().GetDisplayName().ToString() : TEXT("NONE"));

			EPairedAxis PairedAxis = Params.Key.GetPairedAxis();
			FKeyState& PairedKeyState = KeyStateMap.FindOrAdd(PairedKey);

			// The FindOrAdd can invalidate KeyState, so we must look it back up
			FKeyState& KeyState = KeyStateMap.FindChecked(Params.Key);

			// Update accumulator for the appropriate axis
			if (PairedAxis == EPairedAxis::X)
			{
				PairedKeyState.RawValueAccumulator.X = KeyState.RawValueAccumulator.X;
				PairedKeyState.PairSampledAxes |= 0b001;
			}
			else if (PairedAxis == EPairedAxis::Y)
			{
				PairedKeyState.RawValueAccumulator.Y = KeyState.RawValueAccumulator.X;
				PairedKeyState.PairSampledAxes |= 0b010;
			}
			else if (PairedAxis == EPairedAxis::Z)
			{
				PairedKeyState.RawValueAccumulator.Z = KeyState.RawValueAccumulator.X;
				PairedKeyState.PairSampledAxes |= 0b100;
			}
			else
			{
				checkf(false, TEXT("Key %s has paired axis key %s but no valid paired axis!"), *Params.Key.GetFName().ToString(), *Params.Key.GetPairedAxisKey().GetFName().ToString());
			}

			PairedKeyState.SampleCountAccumulator = FMath::Max(PairedKeyState.SampleCountAccumulator, KeyState.SampleCountAccumulator);	// Take max count of each contributing axis.

			// TODO: Will trigger multiple times for the paired axis key. Not desirable.
			TestEventEdges(PairedKeyState, PairedKeyState.Value.Size());

			PairedKeyState.bWasJustFlushed = false;
		}

		// Clear the flushed flag now that we've received a real OS event for this key.
		// This mirrors the same clear in the non-analog path (see below).
		{
			FKeyState& KeyState = KeyStateMap.FindChecked(Params.Key);
			KeyState.bWasJustFlushed = false;
		}

	#if !UE_BUILD_SHIPPING
		CurrentEvent		= IE_Axis;

		if(Params.Event == IE_Pressed)
		{
			const FString Command = GetBind(Params.Key);
			if(Command.Len())
			{
				UWorld* World = GetWorld();
				check(World);
				ExecInputCommands( World, *Command,*GLog);
				return true;
			}
		}
	#endif

		return false;
	}
	// Non-analog key
	else
	{
		FKeyState* ExistingKeyState = KeyStateMap.Find(Params.Key);
		
		// first event associated with this key, add it to the map
		FKeyState& KeyState = !ExistingKeyState ? KeyStateMap.Add(Params.Key) : *ExistingKeyState;

		UWorld* World = GetWorld();
		check(World);

		const bool bIsFirstEventForKey = (ExistingKeyState == nullptr) || KeyState.bWasJustFlushed;

		const float WorldRealTimeSeconds = World->GetRealTimeSeconds();

		// If this is the first key press for us and it is a repeat, then we have missed the initial IE_Pressed event.
		// This can happen if you are holding down a key between level transitions and the player controller gets recreated,
		// which means that we will be using a new UPlayerInput object and the KeyState map is emptied.
		// This can ALSO happen if "FlushPressedKeys" gets called, like when we change player controller input modes, and you keep holding down the key 
		// in between those transitions
		if (UE::Input::bAutoReconcilePressedEventsOnFirstRepeat && bIsFirstEventForKey && Params.Event == IE_Repeat && KeyState.EventAccumulator[IE_Pressed].IsEmpty())
		{
			// Mark as having received the IE_Pressed event already, so that we can correctly evaluate the 
			// state of the IE_Repeat event. It is impossible to get a IE_Repeat with an initial IE_Pressed somewhere
			//
			// Without this, the key will incorrectly evaluate as being released/pressed
			// every frame, even if you are just holding it
			KeyState.RawValueAccumulator.X = Params.AmountDepressed;
			KeyState.EventAccumulator[IE_Pressed].Add(++EventCount);
			KeyState.LastUpDownTransitionTime = WorldRealTimeSeconds;
			KeyState.SampleCountAccumulator++;

			// We can assume that if we are getting a "Repeat event" that this key was already down the previous frame and it is down this frame
			KeyState.bDown = true;
			KeyState.bDownPrevious = true;
		}

		switch(Params.Event)
		{
		case IE_Pressed:
		case IE_Repeat:
			KeyState.RawValueAccumulator.X = Params.AmountDepressed;
			KeyState.EventAccumulator[Params.Event].Add(++EventCount);
			if (KeyState.bDownPrevious == false)
			{
				// check for doubleclick
				// note, a tripleclick will currently count as a 2nd double click.
				if ((WorldRealTimeSeconds - KeyState.LastUpDownTransitionTime) < GetDefault<UInputSettings>()->DoubleClickTime)
				{
					KeyState.EventAccumulator[IE_DoubleClick].Add(++EventCount);
				}

				// just went down
				KeyState.LastUpDownTransitionTime = WorldRealTimeSeconds;
			}
			break;
		case IE_Released:
			KeyState.RawValueAccumulator.X = 0.f;
			KeyState.EventAccumulator[IE_Released].Add(++EventCount);
			break;
		case IE_DoubleClick:
			KeyState.RawValueAccumulator.X = Params.AmountDepressed;
			KeyState.EventAccumulator[IE_Pressed].Add(++EventCount);
			KeyState.EventAccumulator[IE_DoubleClick].Add(++EventCount);
			break;
		}
		KeyState.SampleCountAccumulator++;

		// We have now processed this key's state, so we can clear its "just flushed" flag and treat it normally
		KeyState.bWasJustFlushed = false;

	#if !UE_BUILD_SHIPPING
		CurrentEvent		= Params.Event;

		const FString Command = GetBind(Params.Key);
		if(Command.Len())
		{
			return ExecInputCommands(World, *Command,*GLog);
		}
	#endif

		if(Params.Event == IE_Pressed)
		{
			return IsKeyHandledByAction( Params.Key);
		}

		return true;
	}
}

bool UPlayerInput::InputTouch(
		const FInputDeviceId DeviceId,
		const uint32 Handle,
		const ETouchType::Type Type,
		const FVector2D& TouchLocation,
		const float Force,
		const uint32 TouchpadIndex,
		const uint64 Timestamp)
{
	checkf(TouchpadIndex == 0, TEXT("We currently assume one touchpad in UPlayerInput::InputTouch. If this triggers, add support for multiple pads"));

	return InputTouch(FTouchId(DeviceId, static_cast<ETouchIndex::Type>(Handle)), Type, TouchLocation, Force, Timestamp);
}

bool UPlayerInput::InputTouch(const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force,
	const uint64 Timestamp)
{
	using namespace UE;

	// if touch disabled from the command line, consume the input and do nothing
	static bool bTouchDisabled = FParse::Param(FCommandLine::Get(), TEXT("DisableTouch"));
	if (bTouchDisabled)
	{
		return true;
	}

	// get the current state of each finger
	// if the index is out of bounds, we can't handle it
	const ETouchIndex::Type TouchIndex = TouchId.GetIndex();
	if (TouchIndex >= EKeys::NUM_TOUCH_KEYS)
	{
		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// update position
	Touches[TouchIndex].X = TouchLocation.X;
	Touches[TouchIndex].Y = TouchLocation.Y;

	// update touched/untouched flag
	// > 0 means that it's currently held down (anything besides an end message is held down)
	Touches[TouchIndex].Z = (Type == ETouchType::Ended) ? 0.0f : Force;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TouchStates[TouchIndex].DeviceId = TouchId.GetDeviceId();
	TouchStates[TouchIndex].ViewportRelativeLocation = FVector2f(TouchLocation);
	TouchStates[TouchIndex].Pressure = (Type == ETouchType::Ended) ? 0.0f : Force;

	// hook up KeyState for InputComponent
	FKeyState& KeyState = KeyStateMap.FindOrAdd(EKeys::TouchKeys[TouchIndex]);
	FTouchState& TouchState = TouchStates[TouchIndex];
	switch(Type)
	{
	case ETouchType::Began:
		KeyState.EventAccumulator[IE_Pressed].Add(++EventCount);
		// store current touch location paired with event id
		TouchEventLocations.Add(EventCount, FVector(FVector2D(TouchState.ViewportRelativeLocation), TouchState.Pressure));
		if (KeyState.bDownPrevious == false)
		{
			UWorld* World = GetWorld();
			check(World);

			// check for doubleclick
			// note, a tripleclick will currently count as a 2nd double click.
			const float WorldRealTimeSeconds = World->GetRealTimeSeconds();
			if ((WorldRealTimeSeconds - KeyState.LastUpDownTransitionTime) < GetDefault<UInputSettings>()->DoubleClickTime)
			{
				KeyState.EventAccumulator[IE_DoubleClick].Add(++EventCount);
				// store current touch location paired with event id
				TouchEventLocations.Add(EventCount, FVector(FVector2D(TouchState.ViewportRelativeLocation), TouchState.Pressure));
			}

			// just went down
			KeyState.LastUpDownTransitionTime = WorldRealTimeSeconds;
		}
		break;

	case ETouchType::Ended:
		KeyState.EventAccumulator[IE_Released].Add(++EventCount);
		// store current touch location paired with event id
		TouchEventLocations.Add(EventCount, FVector(FVector2D(TouchState.ViewportRelativeLocation), TouchState.Pressure));
		break;

	default:
		KeyState.EventAccumulator[IE_Repeat].Add(++EventCount);
		// store current touch location paired with event id
		TouchEventLocations.Add(EventCount, FVector(FVector2D(TouchState.ViewportRelativeLocation), TouchState.Pressure));
		break;
	}

	// accumulate deltas until processed next
	KeyState.SampleCountAccumulator++;
	KeyState.RawValueAccumulator = KeyState.Value = KeyState.RawValue = FVector(TouchLocation.X, TouchLocation.Y, 0);

	// We have now processed this touch's state, so we can clear its "just flushed" flag and treat it normally.
	// Without this, EnhancedInput mappings flagged bShouldBeIgnored during a flush stay stuck because their
	// lifecycle clear depends on bWasJustFlushed reaching false (mirrors the InputKey/InputAxis paths).
	KeyState.bWasJustFlushed = false;

	// for now, if we have a player, assume it sucks up all touch input
	// @todo: Here we could look for game touch zones, etc, and only return true if some non-Slate in-game zone used the touch
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UPlayerInput::InputMotion(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration)
{
	// No way to get the input device from deprecated code here
	const FInputDeviceId DeviceId = INPUTDEVICEID_NONE;

	// A timestamp of 0 is invalid because we don't have any way to accurately get one here from deprecated callsites.
	constexpr uint64 Timestamp = 0u;

	return InputMotion(DeviceId, Tilt, RotationRate, Gravity, Acceleration, Timestamp);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPlayerInput::InputMotion(
	const FInputDeviceId DeviceId,
	const FVector& InTilt,
	const FVector& InRotationRate,
	const FVector& InGravity,
	const FVector& InAcceleration,
	const uint64 Timestamp)
{
	FKeyState& KeyStateTilt = KeyStateMap.FindOrAdd(EKeys::Tilt);
	KeyStateTilt.RawValueAccumulator += InTilt;
	KeyStateTilt.EventAccumulator[IE_Repeat].Add(++EventCount);
	KeyStateTilt.SampleCountAccumulator++;

	FKeyState& KeyStateRotationRate = KeyStateMap.FindOrAdd(EKeys::RotationRate);
	KeyStateRotationRate.RawValueAccumulator += InRotationRate;
	KeyStateRotationRate.EventAccumulator[IE_Repeat].Add(++EventCount);
	KeyStateRotationRate.SampleCountAccumulator++;

	FKeyState& KeyStateGravity = KeyStateMap.FindOrAdd(EKeys::Gravity);
	KeyStateGravity.RawValueAccumulator += InGravity;
	KeyStateGravity.EventAccumulator[IE_Repeat].Add(++EventCount);
	KeyStateGravity.SampleCountAccumulator++;

	FKeyState& KeyStateAcceleration = KeyStateMap.FindOrAdd(EKeys::Acceleration);
	KeyStateAcceleration.RawValueAccumulator += InAcceleration;
	KeyStateAcceleration.EventAccumulator[IE_Repeat].Add(++EventCount);
	KeyStateAcceleration.SampleCountAccumulator++;

	// for now, if we have a player, assume it sucks up all motion input
	// @todo: Here we could look for game kismet, etc, and only return true if some non-Slate in-game handler used the touch
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UPlayerInput::InputGesture(const FKey Gesture, const EInputEvent Event, const float Value)
{
	// No way to get the input device from deprecated code here
	const FInputDeviceId DeviceId = INPUTDEVICEID_NONE;

	// A timestamp of 0 is invalid because we don't have any way to accurately get one here from deprecated callsites.
	constexpr uint64 Timestamp = 0u;

	return InputGesture(DeviceId, Gesture, Event, Value, Timestamp);	
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UPlayerInput::InputGesture(
		const FInputDeviceId DeviceId,
		const FKey Gesture,
		const EInputEvent Event,
		const float Value,
		const uint64 Timestamp)
{
	FKeyState& KeyState = KeyStateMap.FindOrAdd(Gesture);

	KeyState.Value.X = KeyState.RawValue.X = KeyState.RawValueAccumulator.X = Value;
	KeyState.EventAccumulator[Event].Add(++EventCount);

	return true;
}

bool UPlayerInput::InputGesture(const FInputKeyEventArgs& Params)
{
	FKeyState& KeyState = KeyStateMap.FindOrAdd(Params.Key);

	KeyState.Value.X = KeyState.RawValue.X = KeyState.RawValueAccumulator.X = Params.AmountDepressed2D.X;
	KeyState.Value.Y = KeyState.RawValue.Y = KeyState.RawValueAccumulator.Y = Params.AmountDepressed2D.Y;
	KeyState.EventAccumulator[Params.Event].Add(++EventCount);

	UE_LOGF(LogPlayerInput, VeryVerbose, "[%s] Frame: %llu %ls (outer: %ls) received gesture: Key: %ls Value: %ls Event:  %ls",
		__func__,
		GFrameCounter,
		*GetNameSafe(this),
		*GetNameSafe(GetOuter()),
		*Params.Key.GetFName().ToString(),
		*Params.AmountDepressed2D.ToString(),
		UE::Input::LexToString(Params.Event));

	return true;
}

void UPlayerInput::UpdatePinchStartDistance()
{
	GestureRecognizer.SetAnchorDistanceSquared(FVector2D(TouchStates[ETouchIndex::Touch1].ViewportRelativeLocation),
		FVector2D(TouchStates[ETouchIndex::Touch2].ViewportRelativeLocation));
}

bool UPlayerInput::GetAxisProperties(const FKey AxisKey, FInputAxisProperties& OutAxisProperties)
{
	ConditionalInitAxisProperties();

	const FInputAxisProperties* const AxisProps = AxisProperties.Find(AxisKey);
	if (AxisProps)
	{
		OutAxisProperties = *AxisProps;
		return true;
	}

	return false;
}

void UPlayerInput::SetAxisProperties(const FKey AxisKey, const FInputAxisProperties& InAxisProperties)
{
	for (FInputAxisConfigEntry& AxisConfigEntry : AxisConfig)
	{
		if (AxisConfigEntry.AxisKeyName == AxisKey)
		{
			AxisConfigEntry.AxisProperties = InAxisProperties;
		}
	}

	AxisProperties.Reset();
}

float UPlayerInput::GetMouseSensitivityX()
{
	FInputAxisProperties MouseAxisProps;
	if (GetAxisProperties(EKeys::MouseX, MouseAxisProps))
	{
		return MouseAxisProps.Sensitivity;
	}

	return 1.f;
}

float UPlayerInput::GetMouseSensitivityY()
{
	FInputAxisProperties MouseAxisProps;
	if (GetAxisProperties(EKeys::MouseY, MouseAxisProps))
	{
		return MouseAxisProps.Sensitivity;
	}

	return 1.f;
}

void UPlayerInput::SetMouseSensitivity(const float SensitivityX, const float SensitivityY)
{
	FInputAxisProperties MouseAxisProps;
	if (GetAxisProperties(EKeys::MouseX, MouseAxisProps))
	{
		MouseAxisProps.Sensitivity = SensitivityX;
		SetAxisProperties(EKeys::MouseX, MouseAxisProps);
	}
	if (GetAxisProperties(EKeys::MouseY, MouseAxisProps))
	{
		MouseAxisProps.Sensitivity = SensitivityY;
		SetAxisProperties(EKeys::MouseY, MouseAxisProps);
	}
}

bool UPlayerInput::GetInvertAxis(const FName AxisName)
{
	ConditionalBuildKeyMappings();

	bool bAxisInverted = false;

	FAxisKeyDetails* KeyDetails = AxisKeyMap.Find(AxisName);
	if (KeyDetails)
	{
		bAxisInverted = KeyDetails->bInverted;
	}
	return bAxisInverted;
}

void UPlayerInput::InvertAxis(const FName AxisName)
{
	bool bInverted = true;

	if (AxisKeyMap.Num() > 0)
	{
		FAxisKeyDetails* KeyDetails = AxisKeyMap.Find(AxisName);
		if (KeyDetails)
		{
			bInverted = (KeyDetails->bInverted = !KeyDetails->bInverted);
		}
		if (bInverted)
		{
			InvertedAxis.AddUnique(AxisName);
		}
		else
		{
			for (int32 InvertIndex = InvertedAxis.Num() - 1; InvertIndex >= 0; --InvertIndex)
			{
				if (InvertedAxis[InvertIndex] == AxisName)
				{
					InvertedAxis.RemoveAtSwap(InvertIndex, EAllowShrinking::No);
				}
			}
		}
	}
	else
	{
		bool bFound = false;
		for (int32 InvertIndex = InvertedAxis.Num() - 1; InvertIndex >= 0; --InvertIndex)
		{
			if (InvertedAxis[InvertIndex] == AxisName)
			{
				bFound = true;
				InvertedAxis.RemoveAtSwap(InvertIndex, EAllowShrinking::No);
			}
		}
		if (!bFound)
		{
			InvertedAxis.Add(AxisName);
		}
	}
	SaveConfig();
}

bool UPlayerInput::GetInvertAxisKey(const FKey AxisKey)
{
	bool bAxisInverted = false;
	FInputAxisProperties AxisKeyProperties;
	if (GetAxisProperties(AxisKey, AxisKeyProperties))
	{
		bAxisInverted = AxisKeyProperties.bInvert;
	}

	return bAxisInverted;
}

void UPlayerInput::InvertAxisKey(const FKey AxisKey)
{
	ConditionalInitAxisProperties();

	FInputAxisProperties AxisKeyProperties;
	if (GetAxisProperties(AxisKey, AxisKeyProperties))
	{
		AxisKeyProperties.bInvert = !AxisKeyProperties.bInvert;
		SetAxisProperties(AxisKey, AxisKeyProperties);
	}
}

struct FAxisDelegate
{
	UObject* Obj;
	UFunction* Func;

	bool operator==(FAxisDelegate const& Other) const
	{
		return (Obj == Other.Obj) && (Func == Other.Func);
	}

	friend inline uint32 GetTypeHash(FAxisDelegate const& D)
	{
		return uint32((PTRINT)D.Obj + (PTRINT)D.Func);
	}
};

void UPlayerInput::AddActionMapping(const FInputActionKeyMapping& KeyMapping)
{
	ActionMappings.AddUnique(KeyMapping);
	ActionKeyMap.Reset();
	bKeyMapsBuilt = false;
}

void UPlayerInput::RemoveActionMapping(const FInputActionKeyMapping& KeyMapping)
{
	for (int32 ActionIndex = ActionMappings.Num() - 1; ActionIndex >= 0; --ActionIndex)
	{
		if (ActionMappings[ActionIndex] == KeyMapping)
		{
			ActionMappings.RemoveAtSwap(ActionIndex, EAllowShrinking::No);
			ActionKeyMap.Reset();
			bKeyMapsBuilt = false;
			// we don't break because the mapping may have been in the array twice
		}
	}
}

void UPlayerInput::AddAxisMapping(const FInputAxisKeyMapping& KeyMapping)
{
	AxisMappings.AddUnique(KeyMapping);
	AxisKeyMap.Reset();
	bKeyMapsBuilt = false;
}

void UPlayerInput::RemoveAxisMapping(const FInputAxisKeyMapping& InKeyMapping)
{
	for (int32 AxisIndex = AxisMappings.Num() - 1; AxisIndex >= 0; --AxisIndex)
	{
		const FInputAxisKeyMapping& KeyMapping = AxisMappings[AxisIndex];
		if (KeyMapping.AxisName == InKeyMapping.AxisName
			&& KeyMapping.Key == InKeyMapping.Key)
		{
			AxisMappings.RemoveAtSwap(AxisIndex, EAllowShrinking::No);
			AxisKeyMap.Reset();
			bKeyMapsBuilt = false;
			// we don't break because the mapping may have been in the array twice
		}
	}
}

void UPlayerInput::AddEngineDefinedActionMapping(const FInputActionKeyMapping& ActionMapping)
{
	EngineDefinedActionMappings.AddUnique(ActionMapping);
	for (TObjectIterator<UPlayerInput> It; It; ++It)
	{
		It->ActionKeyMap.Reset();
		It->bKeyMapsBuilt = false;
	}
}

void UPlayerInput::AddEngineDefinedAxisMapping(const FInputAxisKeyMapping& AxisMapping)
{
	EngineDefinedAxisMappings.AddUnique(AxisMapping);
	for (TObjectIterator<UPlayerInput> It; It; ++It)
	{
		It->AxisKeyMap.Reset();
		It->bKeyMapsBuilt = false;
	}
}

void UPlayerInput::ForceRebuildingKeyMaps(const bool bRestoreDefaults)
{
	if (bRestoreDefaults)
	{
		const UInputSettings* InputSettings = GetDefault<UInputSettings>();
		if (InputSettings)
		{
			AxisConfig = InputSettings->AxisConfig;
			AxisMappings = InputSettings->GetAxisMappings();
			ActionMappings = InputSettings->GetActionMappings();

			// Speech mappings are deprecated in 5.7
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			//append on speech action mappings
			const TArray<FInputActionSpeechMapping>& SpeechMappings = InputSettings->GetSpeechMappings();
			for (const FInputActionSpeechMapping& SpeechMapping : SpeechMappings)
			{
				FInputActionKeyMapping& ConvertedSpeechToActionMap = ActionMappings.AddDefaulted_GetRef();
				ConvertedSpeechToActionMap.ActionName = SpeechMapping.GetActionName();
				ConvertedSpeechToActionMap.Key = SpeechMapping.GetKeyName();
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	ActionKeyMap.Reset();
	AxisKeyMap.Reset();
	AxisProperties.Reset();
	bKeyMapsBuilt = false;
}

APlayerController* UPlayerInput::GetOuterAPlayerController() const
{
	return Cast<APlayerController>(GetOuter());
}

ULocalPlayer* UPlayerInput::GetOwningLocalPlayer() const
{
	if (const APlayerController* PC = GetOuterAPlayerController())
	{
		return PC->GetLocalPlayer();
	}

	return nullptr;
}

void UPlayerInput::ConditionalBuildKeyMappings_Internal() const
{
	if (ActionKeyMap.Num() == 0)
	{
		struct
		{
			void Build(const TArray<FInputActionKeyMapping>& Mappings, TMap<FName, FActionKeyDetails>& KeyMap)
			{
				for (const FInputActionKeyMapping& ActionMapping : Mappings)
				{
					TArray<FInputActionKeyMapping>& KeyMappings = KeyMap.FindOrAdd(ActionMapping.ActionName).Actions;
					KeyMappings.AddUnique(ActionMapping);
				}
			}
		} ActionMappingsUtility;

		ActionMappingsUtility.Build(ActionMappings, ActionKeyMap);
		ActionMappingsUtility.Build(EngineDefinedActionMappings, ActionKeyMap);

		KeyMapBuildIndex++;
	}

	if (AxisKeyMap.Num() == 0)
	{
		struct
		{
			void Build(const TArray<FInputAxisKeyMapping>& Mappings, TMap<FName, FAxisKeyDetails>& AxisMap)
			{
				for (const FInputAxisKeyMapping& AxisMapping : Mappings)
				{
					bool bAdd = true;
					FAxisKeyDetails& KeyDetails = AxisMap.FindOrAdd(AxisMapping.AxisName);
					for (const FInputAxisKeyMapping& KeyMapping : KeyDetails.KeyMappings)
					{
						if (KeyMapping.Key == AxisMapping.Key)
						{
							UE_LOGF(LogInput, Error, "Duplicate mapping of key %ls for axis %ls", *KeyMapping.Key.ToString(), *AxisMapping.AxisName.ToString());
							bAdd = false;
							break;
						}
					}
					if (bAdd)
					{
						KeyDetails.KeyMappings.Add(AxisMapping);
					}
				}
			}
		} AxisMappingsUtility;

		AxisMappingsUtility.Build(AxisMappings, AxisKeyMap);
		AxisMappingsUtility.Build(EngineDefinedAxisMappings, AxisKeyMap);

		// Apply the axis inversions
		for (int32 InvertedAxisIndex = 0; InvertedAxisIndex < InvertedAxis.Num(); ++InvertedAxisIndex)
		{
			FAxisKeyDetails* KeyDetails = AxisKeyMap.Find(InvertedAxis[InvertedAxisIndex]);
			if (KeyDetails)
			{
				KeyDetails->bInverted = true;
			}
		}
	}

	bKeyMapsBuilt = true;
}

void UPlayerInput::GetChordsForKeyMapping(const FInputActionKeyMapping& KeyMapping, const FInputActionBinding& ActionBinding, const bool bGamePaused, TArray<FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume, const FKeyState* KeyState)
{
	bool bConsumeInput = false;

	checkSlow(!EventIndices.Num());

	// test modifier conditions and ignore the event if they failed
	if (	(KeyMapping.bAlt == false || IsAltPressed())
		&&	(KeyMapping.bCtrl == false || IsCtrlPressed())
		&&	(KeyMapping.bShift == false || IsShiftPressed())
		&&	(KeyMapping.bCmd == false || IsCmdPressed())
		&& 	KeyEventOccurred(KeyMapping.Key, ActionBinding.KeyEvent, EventIndices, KeyState))
	{
		bool bAddDelegate = true;

		// look through the found chords and determine if this is masked (or masks) anything in the array
		const FInputChord Chord(KeyMapping.Key, KeyMapping.bShift, KeyMapping.bCtrl, KeyMapping.bAlt, KeyMapping.bCmd);
		for (int32 ChordIndex = FoundChords.Num() - 1; ChordIndex >= 0; --ChordIndex)
		{
			FInputChord::ERelationshipType ChordRelationship = Chord.GetRelationship(FoundChords[ChordIndex].Chord);

			if (ChordRelationship == FInputChord::ERelationshipType::Masks)
			{
				// If we mask the found one, then remove it from the list
				FoundChords.RemoveAtSwap(ChordIndex, EAllowShrinking::No);
			}
			else if (ChordRelationship == FInputChord::ERelationshipType::Masked)
			{
				bAddDelegate = false;
				break;
			}
		}

		if (bAddDelegate)
		{
			check(EventIndices.Num() > 0);
			FDelegateDispatchDetails FoundChord(
										EventIndices[0]
										, FoundChords.Num()
										, Chord
										, ((!bGamePaused || ActionBinding.bExecuteWhenPaused) ? ActionBinding.ActionDelegate : FInputActionUnifiedDelegate())
										, ActionBinding.KeyEvent
										, &ActionBinding);

			const int32 LastEventIndex = EventIndices.Num() - 1;
			for (int32 EventsIndex = 0; EventsIndex < LastEventIndex; ++EventsIndex)
			{
				FoundChord.EventIndex = EventIndices[EventsIndex];
				FoundChords.Emplace(FoundChord);
			}

			FoundChord.EventIndex = EventIndices[LastEventIndex];
			FoundChords.Emplace(MoveTemp(FoundChord));

			bConsumeInput = true;
		}
	}
	if (ActionBinding.bConsumeInput && (bConsumeInput || !(KeyMapping.bAlt || KeyMapping.bCtrl || KeyMapping.bShift || KeyMapping.bCmd || ActionBinding.KeyEvent == EInputEvent::IE_DoubleClick)))
	{
		KeysToConsume.AddUnique(KeyMapping.Key);
	}

	EventIndices.Reset();
}

void UPlayerInput::GetChordsForAction(const FInputActionBinding& ActionBinding, const bool bGamePaused, TArray<FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume)
{
	ConditionalBuildKeyMappings();

	FActionKeyDetails* KeyDetails = ActionKeyMap.Find(ActionBinding.GetActionName());
	if (KeyDetails)
	{
		for (const FInputActionKeyMapping& KeyMapping : KeyDetails->Actions)
		{
			if (KeyMapping.Key == EKeys::AnyKey)
			{
				for (TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
				{
					const FKey& Key = KeyStatePair.Key;
					if (!Key.IsAnalog() && !KeyStatePair.Value.bConsumed)
					{
						FInputActionKeyMapping SubKeyMapping(KeyMapping);
						SubKeyMapping.Key = Key;
						GetChordsForKeyMapping(SubKeyMapping, ActionBinding, bGamePaused, FoundChords, KeysToConsume, &KeyStatePair.Value);
					}
				}
			}
			else
			{
				FKeyState* KeyState = KeyStateMap.Find(KeyMapping.Key);
				if (!IsKeyConsumed(KeyMapping.Key, KeyState))
				{
					GetChordsForKeyMapping(KeyMapping, ActionBinding, bGamePaused, FoundChords, KeysToConsume, KeyState);
				}
			}
		}
	}
}

void UPlayerInput::GetChordForKey(const FInputKeyBinding& KeyBinding, const bool bGamePaused, TArray<FDelegateDispatchDetails>& FoundChords, TArray<FKey>& KeysToConsume, const FKeyState* KeyState)
{
	bool bConsumeInput = false;

	if (KeyBinding.Chord.Key == EKeys::AnyKey)
	{
		for (TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
		{
			const FKey& Key = KeyStatePair.Key;
			if (!Key.IsAnalog() && !KeyStatePair.Value.bConsumed)
			{
				FInputKeyBinding SubKeyBinding(KeyBinding);
				SubKeyBinding.Chord.Key = Key;
				GetChordForKey(SubKeyBinding, bGamePaused, FoundChords, KeysToConsume, &KeyStatePair.Value);
			}
		}
	}
	else
	{
		if (KeyState == nullptr)
		{
			KeyState = KeyStateMap.Find(KeyBinding.Chord.Key);
		}
		if ( !IsKeyConsumed(KeyBinding.Chord.Key, KeyState) )
		{
			checkSlow(!EventIndices.Num());

			// test modifier conditions and ignore the event if they failed
			if (	(KeyBinding.Chord.bAlt == false || IsAltPressed())
				&&	(KeyBinding.Chord.bCtrl == false || IsCtrlPressed())
				&&	(KeyBinding.Chord.bShift == false || IsShiftPressed())
				&&	(KeyBinding.Chord.bCmd == false || IsCmdPressed())
				&& 	KeyEventOccurred(KeyBinding.Chord.Key, KeyBinding.KeyEvent, EventIndices, KeyState))
			{
				bool bAddDelegate = true;

				// look through the found chords and determine if this is masked (or masks) anything in the array
				for (int32 ChordIndex = FoundChords.Num() - 1; ChordIndex >= 0; --ChordIndex)
				{
					FInputChord::ERelationshipType ChordRelationship = KeyBinding.Chord.GetRelationship(FoundChords[ChordIndex].Chord);

					if (ChordRelationship == FInputChord::ERelationshipType::Masks)
					{
						// If we mask the found one, then remove it from the list
						FoundChords.RemoveAtSwap(ChordIndex, EAllowShrinking::No);
					}
					else if (ChordRelationship == FInputChord::ERelationshipType::Masked)
					{
						bAddDelegate = false;
						break;
					}
				}

				if (bAddDelegate)
				{
					check(EventIndices.Num() > 0);
					FDelegateDispatchDetails FoundChord(
												EventIndices[0]
												, FoundChords.Num()
												, KeyBinding.Chord
												, ((!bGamePaused || KeyBinding.bExecuteWhenPaused) ? KeyBinding.KeyDelegate : FInputActionUnifiedDelegate())
												, KeyBinding.KeyEvent);

					const int32 LastEventIndex = EventIndices.Num() - 1;
					for (int32 EventsIndex = 0; EventsIndex < LastEventIndex; ++EventsIndex)
					{
						FoundChord.EventIndex = EventIndices[EventsIndex];
						FoundChords.Emplace(FoundChord);
					}

					FoundChord.EventIndex = EventIndices[LastEventIndex];
					FoundChords.Emplace(MoveTemp(FoundChord));

					bConsumeInput = true;
				}

				EventIndices.Reset();
			}
		}
	}
	if (KeyBinding.bConsumeInput && (bConsumeInput || !(KeyBinding.Chord.bAlt || KeyBinding.Chord.bCtrl || KeyBinding.Chord.bShift || KeyBinding.Chord.bCmd || KeyBinding.KeyEvent == EInputEvent::IE_DoubleClick)))
	{
		KeysToConsume.AddUnique(KeyBinding.Chord.Key);
	}
}

float UPlayerInput::DetermineAxisValue(const FInputAxisBinding& AxisBinding, const bool bGamePaused, TArray<FKey>& KeysToConsume, OUT bool& bHadAnyNonConsumedKeys) const
{
	ConditionalBuildKeyMappings();

	float AxisValue = 0.0f;
	bHadAnyNonConsumedKeys = false;

	FAxisKeyDetails* KeyDetails = AxisKeyMap.Find(AxisBinding.AxisName);
	if (KeyDetails)
	{
		for (int32 AxisIndex = 0; AxisIndex < KeyDetails->KeyMappings.Num(); ++AxisIndex)
		{
			const FInputAxisKeyMapping& KeyMapping = (KeyDetails->KeyMappings)[AxisIndex];
			if ( !IsKeyConsumed(KeyMapping.Key) )
			{
				if (!bGamePaused || AxisBinding.bExecuteWhenPaused)
				{
					AxisValue += GetKeyValue(KeyMapping.Key) * KeyMapping.Scale;
				}

				if (AxisBinding.bConsumeInput)
				{
					KeysToConsume.AddUnique(KeyMapping.Key);
				}
				bHadAnyNonConsumedKeys = true;
			}
		}

		if (KeyDetails->bInverted)
		{
			AxisValue *= -1.f;
		}
	}

	return AxisValue;
}

void UPlayerInput::ProcessNonAxesKeys(FKey InKey, FKeyState* KeyState)
{
	if (InKey.IsAxis2D() || InKey.IsAxis3D())
	{
		KeyState->Value = MassageVectorAxisInput(InKey, KeyState->RawValue);
	}
	else
	{
		KeyState->Value.X = MassageAxisInput(InKey, KeyState->RawValue.X);
	}

	int32 const PressDelta = KeyState->EventCounts[IE_Pressed].Num() - KeyState->EventCounts[IE_Released].Num();
	if (PressDelta < 0)
	{
		// If this is negative, we definitely released
		KeyState->bDown = false;
	}
	else if (PressDelta > 0)
	{
		// If this is positive, we definitely pressed
		KeyState->bDown = true;
	}
	else
	{
		// If this is 0, we maintain state
		KeyState->bDown = KeyState->bDownPrevious;
	}
}

void UPlayerInput::ProcessInputStack(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPlayerInput::ProcessInputStack);
	
	ConditionalBuildKeyMappings();

	static TArray<TPair<FKey, FKeyState*>> KeysWithEvents;

	// @todo, if input is coming in asynchronously, we should buffer anything that comes in during playerinput() and not include it
	// in this processing batch

	APlayerController* PlayerController = GetOuterAPlayerController();
	if (PlayerController)
	{
		PlayerController->PreProcessInput(DeltaTime, bGamePaused);
	}
	
	// Evaluate the current state of the key map. This will take the event accumulators and put them into the actual values of the key state map.
	EvaluateKeyMapState(DeltaTime, bGamePaused, OUT KeysWithEvents);
	
	// Determine potential delegates
	EvaluateInputDelegates(InputComponentStack, DeltaTime, bGamePaused, KeysWithEvents);

	if (PlayerController)
	{
		PlayerController->PostProcessInput(DeltaTime, bGamePaused);
	}
	
	FinishProcessingPlayerInput();

	TouchEventLocations.Reset();
	KeysWithEvents.Reset();
}

void UPlayerInput::EvaluateKeyMapState(const float DeltaTime, const bool bGamePaused, OUT TArray<TPair<FKey, FKeyState*>>& KeysWithEvents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPlayerInput::EvaluateKeyMapState);
	
	// Must be called non-recursively on the game thread
	check(IsInGameThread() && !KeysWithEvents.Num());
	
	// copy data from accumulators to the real values
	for (TMap<FKey,FKeyState>::TIterator It(KeyStateMap); It; ++It)
	{
		bool bKeyHasEvents = false;
		FKeyState* const KeyState = &It.Value();
		const FKey& Key = It.Key();

		for (uint8 EventIndex = 0; EventIndex < IE_MAX; ++EventIndex)
		{
			KeyState->EventCounts[EventIndex].Reset();
			Exchange(KeyState->EventCounts[EventIndex], KeyState->EventAccumulator[EventIndex]);

			if (!bKeyHasEvents && KeyState->EventCounts[EventIndex].Num() > 0)
			{
				KeysWithEvents.Emplace(Key, KeyState);
				bKeyHasEvents = true;
			}
		}

		if ( (KeyState->SampleCountAccumulator > 0) || Key.ShouldUpdateAxisWithoutSamples() )
		{
			if (KeyState->PairSampledAxes)
			{
				// Paired keys sample only the axes that have changed, leaving unaltered axes in their previous state
				for (int32 Axis = 0; Axis < 3; ++Axis)
				{
					if (KeyState->PairSampledAxes & (1 << Axis))
					{
						KeyState->RawValue[Axis] = KeyState->RawValueAccumulator[Axis];
					}
				}
			}
			else
			{
				// Unpaired keys just take the whole accumulator
				KeyState->RawValue = KeyState->RawValueAccumulator;
			}

			// if we had no samples, we'll assume the state hasn't changed
			// except for some axes, where no samples means the mouse stopped moving
			if (KeyState->SampleCountAccumulator == 0)
			{
				KeyState->EventCounts[IE_Released].Add(++EventCount);
				if (!bKeyHasEvents)
				{
					KeysWithEvents.Emplace(Key, KeyState);
					bKeyHasEvents = true;
				}
			}
		}

		//@todo there must be some way of directly getting the mouse sampling rate from directinput
		if ( ( Key == EKeys::MouseX ) && KeyState->RawValue.X != 0.f )
		{
		 	// calculate sampling time
		 	// make sure not first non-zero sample
		 	if ( SmoothedMouse[0] != 0 )
		 	{
		 		// not first non-zero
		 		MouseSamplingTotal += UE_REAL_TO_FLOAT(FApp::GetDeltaTime());
		 		MouseSamples += KeyState->SampleCountAccumulator;
		 	}
		}

		// will just copy for non-axes
		ProcessNonAxesKeys(Key, KeyState);

		// reset the accumulators
		KeyState->RawValueAccumulator = FVector(0.f, 0.f, 0.f);
		KeyState->SampleCountAccumulator = 0;
		KeyState->PairSampledAxes = 0;
	}
	EventCount = 0;
}

namespace UE::Input
{

	// We collect axis contributions by delegate, so we can sum up
	// contributions from multiple bindings.
	struct FAxisDelegateDetails
	{
		FInputAxisUnifiedDelegate Delegate;
		float Value;

		FAxisDelegateDetails(FInputAxisUnifiedDelegate InDelegate, const float InValue)
			: Delegate(MoveTemp(InDelegate))
			, Value(InValue)
		{
		}
	};
	
	struct FVectorAxisDelegateDetails
	{
		FInputVectorAxisUnifiedDelegate Delegate;
		FVector Value;

		FVectorAxisDelegateDetails(FInputVectorAxisUnifiedDelegate InDelegate, const FVector InValue)
			: Delegate(MoveTemp(InDelegate))
			, Value(InValue)
		{
		}
	};

	struct FDelegateDispatchDetailsSorter
	{
		bool operator()( const FDelegateDispatchDetails& A, const FDelegateDispatchDetails& B ) const
		{
			return (A.EventIndex == B.EventIndex ? A.FoundIndex < B.FoundIndex : A.EventIndex < B.EventIndex);
		}
	};

	static TArray<FAxisDelegateDetails> AxisDelegates;
	static TArray<FVectorAxisDelegateDetails> VectorAxisDelegates;
	static TArray<FDelegateDispatchDetails> NonAxisDelegates;
	
	/**
	* Sorts the AxisDelegates, VectorAxisDelegates, and NonAxisDelegates arrays on the order that they occured
	* and executes them
	*/
	void SortAndExecuteDelegates()
	{
		// Execute the delegates for this component 
    	NonAxisDelegates.Sort(FDelegateDispatchDetailsSorter());
    	for (const FDelegateDispatchDetails& Details : NonAxisDelegates)
    	{
    		if (Details.ActionDelegate.IsBound())
    		{
    			Details.ActionDelegate.Execute(Details.Chord.Key);
    		}
    		else if (Details.TouchDelegate.IsBound())
    		{
    			Details.TouchDelegate.Execute(Details.FingerIndex, Details.TouchLocation);
    		}
    		else if (Details.GestureDelegate.IsBound())
    		{
    			Details.GestureDelegate.Execute(Details.GestureValue);
    		}
    	}
    	// Now dispatch delegates for summed axes
    	for (const FAxisDelegateDetails& Details : AxisDelegates)
    	{
    		if (Details.Delegate.IsBound())
    		{
    			Details.Delegate.Execute(Details.Value);
    		}
    	}
    	for (const FVectorAxisDelegateDetails& Details : VectorAxisDelegates)
    	{
    		if (Details.Delegate.IsBound())
    		{
    			Details.Delegate.Execute(Details.Value);
    		}
    	}
    	
    	AxisDelegates.Reset();
    	VectorAxisDelegates.Reset();
    	NonAxisDelegates.Reset();
	}
	
	
};

void UPlayerInput::EvaluateInputDelegates(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPlayerInput::EvaluateInputDelegates);

	using namespace UE::Input;

	PrepareInputDelegatesForEvaluation(InputComponentStack, DeltaTime, bGamePaused, KeysWithEvents);
	
	// must be called non-recursively and on the game thread
	check(IsInGameThread() && !AxisDelegates.Num() && !VectorAxisDelegates.Num() && !NonAxisDelegates.Num() && !EventIndices.Num());

	int32 StackIndex = InputComponentStack.Num() - 1;

	// Walk the stack, top to bottom
	for ( ; StackIndex >= 0; --StackIndex)
	{
		UInputComponent* const IC = InputComponentStack[StackIndex];
		if (IsValid(IC))
		{
			const bool bDoesComponentBlockFurtherInput = EvaluateInputComponentDelegates(IC, KeysWithEvents, DeltaTime, bGamePaused);
			if (bDoesComponentBlockFurtherInput)
			{
				// stop traversing the stack, all input has been consumed by this InputComponent
                --StackIndex;
				break;
			}
		}
	}
	
	// If the stack index is >=0, then an input component has "blocked" the input.
	// Set any remaining component delegate values to zero. Otherwise, these delegate values
	// have already been set and processed.
	for ( ; StackIndex >= 0; --StackIndex)
	{
		if (UInputComponent* IC = InputComponentStack[StackIndex])
		{
			EvaluateBlockedInputComponent(IC);
		}
	}
	
	SortAndExecuteDelegates();
}

bool UPlayerInput::EvaluateInputComponentDelegates(UInputComponent* const IC, const TArray<TPair<FKey, FKeyState*>>& KeysWithEvents, const float DeltaTime, const bool bGamePaused)
{
	if (!IsValid(IC))
	{
		return false;
	}

	using namespace UE::Input;

	static TArray<FKey> KeysToConsume;
	static TArray<FDelegateDispatchDetails> FoundChords;
	static TArray<TSharedPtr<FInputActionBinding>> PotentialActions;
	
	check(!KeysToConsume.Num() && !FoundChords.Num() && !EventIndices.Num() && !PotentialActions.Num());

	IC->ConditionalBuildKeyMap(this);

	for (const TPair<FKey,FKeyState*>& KeyWithEvent : KeysWithEvents)
	{
		if (!KeyWithEvent.Value->bConsumed)
		{
			FGetActionsBoundToKey::Get(IC, this, KeyWithEvent.Key, PotentialActions);
		}
	}

	for (const TSharedPtr<FInputActionBinding>& ActionBinding : PotentialActions)
	{
		GetChordsForAction(*ActionBinding.Get(), bGamePaused, FoundChords, KeysToConsume);
	}

	PotentialActions.Reset();

	for (const FInputKeyBinding& KeyBinding : IC->KeyBindings)
	{
		GetChordForKey(KeyBinding, bGamePaused, FoundChords, KeysToConsume);
	}

	FoundChords.Sort(FDelegateDispatchDetailsSorter());

	for (int32 ChordIndex=0; ChordIndex < FoundChords.Num(); ++ChordIndex)
	{
		const FDelegateDispatchDetails& FoundChord = FoundChords[ChordIndex];
		bool bFireDelegate = true;
		// If this is a paired action (implements both pressed and released) then we ensure that only one chord is
		// handling the pairing
		if (FoundChord.SourceAction && FoundChord.SourceAction->IsPaired())
		{
			FActionKeyDetails& KeyDetails = ActionKeyMap.FindChecked(FoundChord.SourceAction->GetActionName());
			if (!KeyDetails.CapturingChord.Key.IsValid() || KeyDetails.CapturingChord == FoundChord.Chord || !IsPressed(KeyDetails.CapturingChord.Key))
			{
				if (FoundChord.SourceAction->KeyEvent == IE_Pressed)
				{
					KeyDetails.CapturingChord = FoundChord.Chord;
				}
				else
				{
					KeyDetails.CapturingChord.Key = EKeys::Invalid;
				}
			}
			else
			{
				bFireDelegate = false;
			}
		}

		if (bFireDelegate && FoundChords[ChordIndex].ActionDelegate.IsBound())
		{
			FoundChords[ChordIndex].FoundIndex = NonAxisDelegates.Num();
			NonAxisDelegates.Add(FoundChords[ChordIndex]);

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
			UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
				this,
				{
					.InputComponent = IC,
					.ListeningObject = FoundChords[ChordIndex].ActionDelegate.GetUObject(),
					.ActionName = FoundChords[ChordIndex].SourceAction ? FoundChords[ChordIndex].SourceAction->GetActionName().ToString() : TEXT("Unknown"),
					.InputValue = FoundChords[ChordIndex].KeyEvent == IE_Pressed ? FVector(1.0,0.0,0.0) : FVector(0.0,1.0,0.0),
				});
#endif
		}
	}

	for (int32 TouchBindingIndex=0; TouchBindingIndex<IC->TouchBindings.Num(); ++TouchBindingIndex)
	{
		FInputTouchBinding& TB = IC->TouchBindings[TouchBindingIndex];

		for (int32 TouchIndex = 0; TouchIndex < EKeys::NUM_TOUCH_KEYS; TouchIndex++)
		{
			const FKey& TouchKey = EKeys::TouchKeys[TouchIndex];
			FKeyState* KeyState = KeyStateMap.Find(TouchKey);
			if (KeyEventOccurred(TouchKey, TB.KeyEvent, EventIndices, KeyState) && !IsKeyConsumed(TouchKey, KeyState))
			{
				if (TB.bExecuteWhenPaused || !bGamePaused)
				{
					check(EventIndices.Num() > 0);
					FVector* TouchedLocation = TouchEventLocations.Find(EventIndices[0]);
					FDelegateDispatchDetails TouchInfo(EventIndices[0], NonAxisDelegates.Num(), TB.TouchDelegate, TouchedLocation != nullptr ? *TouchedLocation : FVector(-1.0f, -1.0f, 0.0f), (ETouchIndex::Type)TouchIndex);
					NonAxisDelegates.Add(TouchInfo);
#if UE_PLAYER_INPUT_INCLUDE_DEBUG
					UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
						this,
						{
							.InputComponent = IC,
							.ListeningObject = TB.TouchDelegate.GetUObject(),
							.ActionName = TouchInfo.SourceAction ? TouchInfo.SourceAction->GetActionName().ToString() : TEXT("Unknown"),
							.InputValue = TouchInfo.TouchLocation,
						});
#endif
					
					for (int32 EventsIndex = 1; EventsIndex < EventIndices.Num(); ++EventsIndex)
					{
						TouchInfo.EventIndex = EventIndices[EventsIndex];
						TouchedLocation = TouchEventLocations.Find(TouchInfo.EventIndex);
						TouchInfo.TouchLocation = TouchedLocation != nullptr ? *TouchedLocation : FVector(-1.0f, -1.0f, 0.0f);
						NonAxisDelegates.Add(TouchInfo);

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
						UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
							this,
							{
								.InputComponent = IC,
								.ListeningObject = TB.TouchDelegate.GetUObject(),
								.ActionName = TouchInfo.SourceAction ? TouchInfo.SourceAction->GetActionName().ToString() : TEXT("Unknown"),
								.InputValue = TouchInfo.TouchLocation,
							});
#endif
					}
				}
				if (TB.bConsumeInput)
				{
					KeysToConsume.AddUnique(TouchKey);
				}
			}
		}

		EventIndices.Reset();
	}

	// look for any gestures that happened
	for (FInputGestureBinding& GB : IC->GestureBindings)
	{
		FKeyState* KeyState = KeyStateMap.Find(GB.GestureKey);
		// treat gestures as fire-and-forget, so by convention we assume if they happen, it was a "pressed" event
		if (KeyEventOccurred(GB.GestureKey, IE_Pressed, EventIndices, KeyState) && !IsKeyConsumed(GB.GestureKey, KeyState))
		{
			if (KeyState)
			{
				check(EventIndices.Num() > 0);
				FDelegateDispatchDetails GestureInfo(EventIndices[0], NonAxisDelegates.Num(), GB.GestureDelegate, KeyState->Value.X);
				NonAxisDelegates.Emplace(MoveTemp(GestureInfo));

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
				UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
					this,
					{
						.InputComponent = IC,
						.ListeningObject = GB.GestureDelegate.GetUObject(),
						.ActionName = GestureInfo.SourceAction ? GestureInfo.SourceAction->GetActionName().ToString() : TEXT("Unknown"),
						.InputValue = GestureInfo.TouchLocation,
					});
#endif

				if (GB.bConsumeInput)
				{
					KeysToConsume.AddUnique(GB.GestureKey);
				}
			}
		}

		EventIndices.Reset();
	}

	// A flag that is set in DetermineAxisValue. If false, then all keys related to the axis binding have been consumed.
	bool bHadAnyKeys = false;
	
	// Run though game axis bindings and accumulate axis values
	for (FInputAxisBinding& AB : IC->AxisBindings)
	{
		AB.AxisValue = DetermineAxisValue(AB, bGamePaused, KeysToConsume, bHadAnyKeys); 
		
		if ((bHadAnyKeys || !UE::Input::AxisEventsCanBeConsumed) && AB.AxisDelegate.IsBound())
		{
			AxisDelegates.Emplace(FAxisDelegateDetails(AB.AxisDelegate, AB.AxisValue));
#if UE_PLAYER_INPUT_INCLUDE_DEBUG
			UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
				this,
				{
					.InputComponent = IC,
					.ActionName = AB.AxisName.ToString(),
					.InputValue = FVector(AB.AxisValue, 0.0, 0.0),
				});
#endif
		}
	}
	for (FInputAxisKeyBinding& AxisKeyBinding : IC->AxisKeyBindings)
	{
		if (!IsKeyConsumed(AxisKeyBinding.AxisKey))
		{
			if (!bGamePaused || AxisKeyBinding.bExecuteWhenPaused)
			{
				AxisKeyBinding.AxisValue = GetKeyValue(AxisKeyBinding.AxisKey);
			}
			else
			{
				AxisKeyBinding.AxisValue = 0.f;
			}

			if (AxisKeyBinding.bConsumeInput)
			{
				KeysToConsume.AddUnique(AxisKeyBinding.AxisKey);
			}
		}
		else if(UE::Input::bClearAxisValueIfConsumed)
		{
			AxisKeyBinding.AxisValue = 0.f;
		}

		if (AxisKeyBinding.AxisDelegate.IsBound())
		{
			AxisDelegates.Emplace(FAxisDelegateDetails(AxisKeyBinding.AxisDelegate, AxisKeyBinding.AxisValue));
#if UE_PLAYER_INPUT_INCLUDE_DEBUG
			UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
				this,
				{
					.InputComponent = IC,
					.ListeningObject = AxisKeyBinding.AxisDelegate.GetUObject(),
					.ActionName = AxisKeyBinding.AxisKey.GetFName().ToString(),
					.InputValue = FVector(AxisKeyBinding.AxisValue, 0.0, 0.0),
				});
#endif
		}
	}
	for (FInputVectorAxisBinding& VectorAxisBinding : IC->VectorAxisBindings)
	{
		if (!IsKeyConsumed(VectorAxisBinding.AxisKey))
		{
			if (!bGamePaused || VectorAxisBinding.bExecuteWhenPaused)
			{
				VectorAxisBinding.AxisValue = GetProcessedVectorKeyValue(VectorAxisBinding.AxisKey);
			}
			else
			{
				VectorAxisBinding.AxisValue = FVector::ZeroVector;
			}

			if (VectorAxisBinding.bConsumeInput)
			{
				KeysToConsume.AddUnique(VectorAxisBinding.AxisKey);
			}
		}

		if (VectorAxisBinding.AxisDelegate.IsBound())
		{
			VectorAxisDelegates.Emplace(FVectorAxisDelegateDetails(VectorAxisBinding.AxisDelegate, VectorAxisBinding.AxisValue));
#if UE_PLAYER_INPUT_INCLUDE_DEBUG
			UE::Input::FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(
				this,
				{
					.InputComponent = IC,
					.ListeningObject = VectorAxisBinding.AxisDelegate.GetUObject(),
					.ActionName = VectorAxisBinding.AxisKey.GetFName().ToString(),
					.InputValue = VectorAxisBinding.AxisValue,
				});
#endif
		}
	}

	const bool bComponentBlocksInput = IC->bBlockInput; 

	if (!bComponentBlocksInput)
	{
		// we do this after finishing the whole component, so we don't consume a key while there might be more bindings to it
		for (int32 KeyIndex = 0; KeyIndex < KeysToConsume.Num(); ++KeyIndex)
		{
			if (FKeyState* KeyState = KeyStateMap.Find(KeysToConsume[KeyIndex]))
			{
				KeyState->bConsumed = true;	
			}
		}
	}

	KeysToConsume.Reset();
	FoundChords.Reset();
	
	// This will make the input delegates execute after each component is processed, instead of waiting until all the components are done
	// This will make it so that the order of execution on the delegates is actually set by the UInputComponent priority level. 
	if (!UE::Input::bMaintainLegacyDelegateExecutionOrder)
	{
		// Execute the delegates for this component 
    	SortAndExecuteDelegates();
	}	

	return bComponentBlocksInput;	
}

void UPlayerInput::EvaluateBlockedInputComponent(UInputComponent* InputComponent)
{
	if (!InputComponent)
	{
		return;
	}
	
	for (FInputAxisBinding& AxisBinding : InputComponent->AxisBindings)
	{
		AxisBinding.AxisValue = 0.f;
	}
	for (FInputAxisKeyBinding& AxisKeyBinding : InputComponent->AxisKeyBindings)
	{
		AxisKeyBinding.AxisValue = 0.f;
	}
	for (FInputVectorAxisBinding& VectorAxisBinding : InputComponent->VectorAxisBindings)
	{
		VectorAxisBinding.AxisValue = FVector::ZeroVector;
	}
}

void UPlayerInput::DiscardPlayerInput()
{
	// discard any accumulated player input that hasn't yet been processed
	for (TMap<FKey, FKeyState>::TIterator It(KeyStateMap); It; ++It)
	{
		FKeyState& KeyState = It.Value();
		KeyState.RawValueAccumulator = FVector(0.f,0.f,0.f);
		KeyState.SampleCountAccumulator = 0;
		KeyState.PairSampledAxes = 0;
		for (uint8 EventIndex = 0; EventIndex < IE_MAX; ++EventIndex)
		{
			KeyState.EventAccumulator[EventIndex].Reset();
		}
	}
	EventCount = 0;
}

void UPlayerInput::FinishProcessingPlayerInput()
{
	// finished processing input for this frame, clean up for next update
	for (TMap<FKey,FKeyState>::TIterator It(KeyStateMap); It; ++It)
	{
		FKeyState& KeyState = It.Value();
		KeyState.bDownPrevious = KeyState.bDown;
		KeyState.bConsumed = false;
	}
}

void UPlayerInput::ClearSmoothing()
{
	for ( int32 i=0; i<2; i++ )
	{
		ZeroTime[i] = 0.f;
		SmoothedMouse[i] = 0.f;
	}

	const UPlayerInput* DefaultPlayerInput = GetDefault<UPlayerInput>();

   	MouseSamplingTotal = DefaultPlayerInput->MouseSamplingTotal;
	MouseSamples = DefaultPlayerInput->MouseSamples;
}


float UPlayerInput::SmoothMouse(float aMouse, uint8& SampleCount, int32 Index)
{
	check(Index >= 0);
	check(Index < UE_ARRAY_COUNT(ZeroTime));

	UWorld* World = GetWorld();
	if (World)
	{
		check(World->GetWorldSettings());
		const float EffectiveTimeDilation = World->GetWorldSettings()->GetEffectiveTimeDilation();
		if (EffectiveTimeDilation != LastTimeDilation)
		{
			LastTimeDilation = EffectiveTimeDilation;
			ClearSmoothing();
		}
	}

	const float DeltaTime = UE_REAL_TO_FLOAT(FApp::GetDeltaTime());

	if (DeltaTime < 0.25f)
	{
		check(MouseSamples > 0);

		// this is seconds/sample
		const float MouseSamplingTime = MouseSamplingTotal / MouseSamples;
		check(MouseSamplingTime > 0.0f);

		if (aMouse == 0)
		{
			// no mouse movement received
			ZeroTime[Index] += DeltaTime;		// increment length of time we've been at zero
			if ( ZeroTime[Index] < MouseSamplingTime )
			{
				// zero mouse movement is possibly because less than the mouse sampling interval has passed
				aMouse = SmoothedMouse[Index] * DeltaTime/MouseSamplingTime;
			}
			else
			{
				SmoothedMouse[Index] = 0;
			}
		}
		else
		{
			ZeroTime[Index] = 0;
			if (SmoothedMouse[Index] != 0)
			{
				// SampleCount may be zero because the Mouse keys are flagged as UpdateAxisWithoutSamples, which means
				// we will process them even if they do not have any samples.
				//
				// For that reason, in order to guard against divide by zero errors, we will just use 1 sample count when
				// calculating any smoothing if there aren't any this frame.
				const uint8 NonZeroSampleCount = FMath::Max<uint8>(1, SampleCount);
				ensureMsgf(SampleCount > 0, TEXT("The sample count was zero when smoothing the mouse. A value of 1 will be used in its place"));
			
				// this isn't the first tick with non-zero mouse movement
				if ( DeltaTime < MouseSamplingTime * (SampleCount + 1) )
				{
					// smooth mouse movement so samples/tick is constant
					aMouse = aMouse * DeltaTime / (MouseSamplingTime * NonZeroSampleCount);
				}
				else
				{
					// fewer samples, so going slow
					// use number of samples we should have had for sample count
					SampleCount = (uint8)(DeltaTime / MouseSamplingTime);
				}
			}
			
			if (SampleCount == 0)
			{
				SmoothedMouse[Index] = 0;
			}
			else
			{
				SmoothedMouse[Index] = aMouse / SampleCount;
			}
		}
	}
	else
	{
		// if we had an abnormally long frame, clear everything so it doesn't distort the results
		ClearSmoothing();
	}

	SampleCount = 0;

	return aMouse;
}

void UPlayerInput::DisplayDebug(class UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	if (Canvas)
	{
		FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;

#if UE_ENABLE_DEBUG_DRAWING
		// Show the UI mode of the owning player controller if there is one. This is a very common issue with folks
		// debugging input. Often times users will set the UI mode to "UI Only", and then never set it back so the 
		// game can consume input again. 
		if (const APlayerController* PlayerController = GetOuterAPlayerController())
		{		
			DisplayDebugManager.SetDrawColor(FColor::Orange);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t Player Controller Input Settings:  %s"), *GetNameSafe(PlayerController)));
			
			// Display whether input is enabled on the player controller or not
			const bool bIsInputEnabled = PlayerController->InputEnabled();
			DisplayDebugManager.SetDrawColor(bIsInputEnabled ? FColor::White : FColor::Red);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t PlayerController bIsInputEnabled: %s"), bIsInputEnabled ? TEXT("True") : TEXT("False")));

			DisplayDebugManager.SetDrawColor(FColor::White);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("\t PlayerController Input Mode: %s"), *PlayerController->GetCurrentInputModeDebugString()));

			const ULocalPlayer* LP = GetOwningLocalPlayer();

			// Display some info about the viewport, all of these settings can affect the input behavior of the mouse
			if (LP && LP->ViewportClient)
			{
				// Show the capture mode of the viewport
				DisplayDebugManager.DrawString(FString::Printf(TEXT("\t ViewportClient MouseCaptureMode: %s"), *UEnum::GetValueAsString(TEXT("Engine.EMouseCaptureMode"), LP->ViewportClient->GetMouseCaptureMode())));
				DisplayDebugManager.DrawString(FString::Printf(TEXT("\t ViewportClient MouseLockMode: %s"), *UEnum::GetValueAsString(TEXT("Engine.EMouseLockMode"), LP->ViewportClient->GetMouseLockMode())));
				
				const bool bIsIgnoringInput = LP->ViewportClient->IgnoreInput();
				DisplayDebugManager.SetDrawColor(bIsIgnoringInput ? FColor::Red : FColor::White);
				DisplayDebugManager.DrawString(FString::Printf(TEXT("\t ViewportClient bIsIgnoringInput: %s"), bIsIgnoringInput ? TEXT("True") : TEXT("False")));
				
				const bool bIsUsingMouseForTouch = LP->ViewportClient->GetUseMouseForTouch();
				DisplayDebugManager.SetDrawColor(bIsUsingMouseForTouch ? FColor::Green : FColor::White);
				DisplayDebugManager.DrawString(FString::Printf(TEXT("\t ViewportClient bIsUsingMouseForTouch: %s"), bIsUsingMouseForTouch ? TEXT("True") : TEXT("False")));
			}
		}
#endif // #if UE_ENABLE_DEBUG_DRAWING

		DisplayDebugManager.SetDrawColor(FColor::Red);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("INPUT %s"), *GetName()));

		UWorld* World = GetWorld();
		check(World);
		const float WorldRealTimeSeconds = World->GetRealTimeSeconds();
		for (TMap<FKey,FKeyState>::TIterator It(KeyStateMap); It; ++It)
		{
			FKeyState const* const KeyState = &It.Value();
			FKey const Key = It.Key();

			auto GetKeyStateString = [Key, KeyState]() -> FString
			{
				if (Key.IsAxis3D())
				{
					return FString::Printf(TEXT(" %s: X: %.2f Y: %.2f Z: %.2f (raw X: %.2f Y: %.2f Z: %.2f)"), *(Key.GetDisplayName().ToString()),
						KeyState->Value.X, KeyState->Value.Y, KeyState->Value.Z,
						KeyState->RawValue.X, KeyState->RawValue.Y, KeyState->RawValue.Z);
				}
				else if (Key.IsAxis2D())
				{
					return FString::Printf(TEXT(" %s: X: %.2f Y: %.2f (raw X: %.2f Y: %.2f)"), *(Key.GetDisplayName().ToString()),
						KeyState->Value.X, KeyState->Value.Y,
						KeyState->RawValue.X, KeyState->RawValue.Y);
				}
				return FString::Printf(TEXT(" %s: %.2f (raw %.2f)"), *(Key.GetDisplayName().ToString()), KeyState->Value.X, KeyState->RawValue.X);
			};

			// used cached mouse values, since they were flushed already
			
			FString Str = GetKeyStateString();

			if ( KeyState->bDown || (KeyState->Value.X != 0.f) )
			{
				if (!Key.IsAxis1D())
				{
					Str += FString::Printf(TEXT(" time: %.2f"), WorldRealTimeSeconds - KeyState->LastUpDownTransitionTime);
				}
				DisplayDebugManager.SetDrawColor(FColor(180, 255, 180));
				DisplayDebugManager.DrawString(Str);
			}
			else
			{
				DisplayDebugManager.SetDrawColor(FColor(180, 180, 180));
				DisplayDebugManager.DrawString(Str);
			}
		}

		float const DetectedMouseSampleHz = MouseSamples / MouseSamplingTotal;

		DisplayDebugManager.SetDrawColor(FColor::White);
		DisplayDebugManager.DrawString(FString::Printf(TEXT("MouseSampleRate: %.2f"), DetectedMouseSampleHz));
		DisplayDebugManager.DrawString(FString::Printf(TEXT("MouseX ZeroTime: %.2f, Smoothed: %.2f"), ZeroTime[0], SmoothedMouse[0]));
		DisplayDebugManager.DrawString(FString::Printf(TEXT("MouseY ZeroTime: %.2f, Smoothed: %.2f"), ZeroTime[1], SmoothedMouse[1]));

		if ( (ZeroTime[0] > 2.f && ZeroTime[1] > 2.f) && GetDefault<UInputSettings>()->bEnableMouseSmoothing )
		{
			// return to center of screen
			DebugSmoothedMouseX = DebugUnSmoothedMouseX = Canvas->SizeX * 0.5f;
			DebugSmoothedMouseY = DebugUnSmoothedMouseY = Canvas->SizeY * 0.5f;
		}
		else
		{
			float const CenterX = Canvas->SizeX * 0.5f;
			float const CenterY = Canvas->SizeY * 0.5f;

			static const float DEBUGSMOOTHMOUSE_REGIONSIZE = 700;

			// clamp mouse smoothing debug cursor
			DebugSmoothedMouseX = FMath::Clamp(DebugSmoothedMouseX, CenterX-DEBUGSMOOTHMOUSE_REGIONSIZE, CenterX+DEBUGSMOOTHMOUSE_REGIONSIZE);
			DebugSmoothedMouseY = FMath::Clamp(DebugSmoothedMouseY, CenterY-DEBUGSMOOTHMOUSE_REGIONSIZE, CenterY+DEBUGSMOOTHMOUSE_REGIONSIZE);
			DebugUnSmoothedMouseX = FMath::Clamp(DebugUnSmoothedMouseX, CenterX-DEBUGSMOOTHMOUSE_REGIONSIZE, CenterX+DEBUGSMOOTHMOUSE_REGIONSIZE);
			DebugUnSmoothedMouseY = FMath::Clamp(DebugUnSmoothedMouseY, CenterY-DEBUGSMOOTHMOUSE_REGIONSIZE, CenterY+DEBUGSMOOTHMOUSE_REGIONSIZE);
		}

		// draw YELLOW box for SMOOTHED mouse loc
		FCanvasTileItem TileItem( FVector2D(DebugSmoothedMouseX, DebugSmoothedMouseY), GWhiteTexture, FVector2D(8.0f,8.0f), FLinearColor::Yellow );
		Canvas->DrawItem( TileItem );

		// draw WHITE box for UNSMOOTHED mouse loc
		TileItem.SetColor( FLinearColor::White );
		TileItem.Size = FVector2D( 5.0f, 5.0f );
		TileItem.Position = FVector2D(DebugUnSmoothedMouseX, DebugUnSmoothedMouseY);
		Canvas->DrawItem( TileItem );
	}
}

bool UPlayerInput::WasJustPressed( FKey InKey ) const
{
	if (InKey == EKeys::AnyKey)
	{
		// Is there any key that has just been pressed
		for (const TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
		{
			if (!KeyStatePair.Key.IsAnalog() && KeyStatePair.Value.EventCounts[IE_Pressed].Num() > 0)
			{
				return true;
			}
		}
	}
	else if (FKeyState const* const KeyState = KeyStateMap.Find(InKey))
	{
		return (KeyState->EventCounts[IE_Pressed].Num() > 0);
	}

	return false;
}


bool UPlayerInput::WasJustReleased( FKey InKey ) const
{
	if (InKey == EKeys::AnyKey)
	{
		// Is there any key that has just been released
		for (const TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
		{
			if (!KeyStatePair.Key.IsAnalog() && KeyStatePair.Value.EventCounts[IE_Released].Num() > 0)
			{
				return true;
			}
		}
	}
	else if (FKeyState const* const KeyState = KeyStateMap.Find(InKey))
	{
		return (KeyState->EventCounts[IE_Released].Num() > 0);
	}

	return false;
}

float UPlayerInput::GetTimeDown( FKey InKey ) const
{
	UE_CLOGF(InKey == EKeys::AnyKey, LogInput, Warning, "GetTimeDown cannot return a meaningful result for AnyKey");
	UWorld* World = GetWorld();
	float DownTime = 0.f;
	if( World )
	{
		FKeyState const* const KeyState = KeyStateMap.Find(InKey);
		if (KeyState && KeyState->bDown)
		{
			DownTime = (World->GetRealTimeSeconds() - KeyState->LastUpDownTransitionTime);
		}
	}

	return DownTime;
}

bool UPlayerInput::IsKeyConsumed( FKey InKey, const FKeyState* KeyState ) const
{
	if (InKey == EKeys::AnyKey)
	{
		// Is there any key that is consumed
		for (const TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
		{
			if (KeyStatePair.Value.bConsumed)
			{
				return true;
			}
		}
	}
	else
	{
		if (KeyState == nullptr)
		{
			KeyState = KeyStateMap.Find(InKey);
		}
		if (KeyState)
		{
			return KeyState->bConsumed;
		}
	}

	return false;
}

float UPlayerInput::GetKeyValue( FKey InKey ) const
{
	UE_CLOGF(InKey == EKeys::AnyKey, LogInput, Warning, "GetKeyValue cannot return a meaningful result for AnyKey");
	FKeyState const* const KeyState = KeyStateMap.Find(InKey);
	return KeyState ? KeyState->Value.X : 0.f;
}

float UPlayerInput::GetRawKeyValue( FKey InKey ) const
{
	UE_CLOGF(InKey == EKeys::AnyKey, LogInput, Warning, "GetRawKeyValue cannot return a meaningful result for AnyKey");
	FKeyState const* const KeyState = KeyStateMap.Find(InKey);
	return KeyState ? KeyState->RawValue.X : 0.f;
}

FVector UPlayerInput::GetProcessedVectorKeyValue(FKey InKey) const
{
	UE_CLOGF(InKey == EKeys::AnyKey, LogInput, Warning, "GetProcessedVectorKeyValue cannot return a meaningful result for AnyKey");
	FKeyState const* const KeyState = KeyStateMap.Find(InKey);
	return KeyState ? KeyState->Value : FVector(0, 0, 0);
}

FVector UPlayerInput::GetRawVectorKeyValue(FKey InKey) const
{
	UE_CLOGF(InKey == EKeys::AnyKey, LogInput, Warning, "GetRawVectorKeyValue cannot return a meaningful result for AnyKey");
	FKeyState const* const KeyState = KeyStateMap.Find(InKey);
	return KeyState ? KeyState->RawValue : FVector(0, 0, 0);
}

bool UPlayerInput::IsPressed( FKey InKey ) const
{
	if (InKey == EKeys::AnyKey)
	{
		// Is there any key that is down
		for (const TPair<FKey, FKeyState>& KeyStatePair : KeyStateMap)
		{
			if (KeyStatePair.Key.IsDigital() && KeyStatePair.Value.bDown)
			{
				return true;
			}
		}
	}
	else if (FKeyState const* const KeyState = KeyStateMap.Find(InKey))
	{
		return KeyState->bDown;
	}

	return false;
}

FVector UPlayerInput::MassageVectorAxisInput(FKey Key, FVector RawValue)
{
	FVector NewVal = RawValue;

	ConditionalInitAxisProperties();

	// no massaging for buttons atm, might want to support it for things like pressure-sensitivity at some point

	FInputAxisProperties const* const AxisProps = AxisProperties.Find(Key);
	if (AxisProps)
	{
		// deal with axis deadzone
		if (AxisProps->DeadZone > 0.f)
		{
			auto DeadZoneLambda = [AxisProps](const float AxisVal)
			{
				// We need to translate and scale the input to the +/- 1 range after removing the dead zone.
				if (AxisVal > 0)
				{
					return FMath::Max(0.f, AxisVal - AxisProps->DeadZone) / (1.f - AxisProps->DeadZone);
				}
				else
				{
					return -FMath::Max(0.f, -AxisVal - AxisProps->DeadZone) / (1.f - AxisProps->DeadZone);
				}
			};

			NewVal.X = DeadZoneLambda(NewVal.X);
			NewVal.Y = DeadZoneLambda(NewVal.Y);
			NewVal.Z = DeadZoneLambda(NewVal.Z);
		}

		// apply any exponent curvature while we're in the [0..1] range
		if (AxisProps->Exponent != 1.f)
		{
			auto ExponentLambda = [AxisProps](const float AxisVal)
			{
				return FMath::Sign(AxisVal) * FMath::Pow(FMath::Abs(AxisVal), AxisProps->Exponent);
			};

			NewVal.X = ExponentLambda(NewVal.X);
			NewVal.Y = ExponentLambda(NewVal.Y);
			NewVal.Z = ExponentLambda(NewVal.Z);
		}

		// now apply any scaling (sensitivity)
		NewVal *= AxisProps->Sensitivity;

		if (AxisProps->bInvert)
		{
			NewVal *= -1.f;
		}
	}

	return NewVal;
}

float UPlayerInput::MassageAxisInput(FKey Key, float RawValue)
{
	float NewVal = RawValue;

	ConditionalInitAxisProperties();

	// no massaging for buttons atm, might want to support it for things like pressure-sensitivity at some point

	FInputAxisProperties const* const AxisProps = AxisProperties.Find(Key);
	if (AxisProps)
	{
		// deal with axis deadzone
		if (AxisProps->DeadZone > 0.f)
		{
			// We need to translate and scale the input to the +/- 1 range after removing the dead zone.
			if( NewVal > 0 )
			{
				NewVal = FMath::Max( 0.f, NewVal - AxisProps->DeadZone ) / (1.f - AxisProps->DeadZone);
			}
			else
			{
				NewVal = -FMath::Max( 0.f, -NewVal - AxisProps->DeadZone ) / (1.f - AxisProps->DeadZone);
			}
		}

		// apply any exponent curvature while we're in the [0..1] range
		if (AxisProps->Exponent != 1.f)
		{
			NewVal = FMath::Sign(NewVal) * FMath::Pow(FMath::Abs(NewVal), AxisProps->Exponent);
		}

		// now apply any scaling (sensitivity)
		NewVal *= AxisProps->Sensitivity;

		if (AxisProps->bInvert)
		{
			NewVal *= -1.f;
		}
	}

	// special handling for mouse input
	if ( (Key == EKeys::MouseX) || (Key == EKeys::MouseY) )
	{
		const UInputSettings* DefaultInputSettings = GetDefault<UInputSettings>();

		// Take FOV into account (lower FOV == less sensitivity).
		APlayerController const* const PlayerController = GetOuterAPlayerController();
		float const FOVScale = (DefaultInputSettings->bEnableFOVScaling && PlayerController && PlayerController->PlayerCameraManager) ? (DefaultInputSettings->FOVScale*PlayerController->PlayerCameraManager->GetFOVAngle()) : 1.0f;
		NewVal *= FOVScale;

		// debug
		if (Key == EKeys::MouseX)
		{
			DebugUnSmoothedMouseX += NewVal * DebugSmoothedMouseSensitivity;
		}
		else
		{
			DebugUnSmoothedMouseY += -NewVal * DebugSmoothedMouseSensitivity;
		}

		// mouse smoothing
		if (DefaultInputSettings->bEnableMouseSmoothing)
		{
			FKeyState* const KeyState = KeyStateMap.Find(Key);
			if (KeyState)
			{
				NewVal = SmoothMouse( NewVal, KeyState->SampleCountAccumulator, (Key == EKeys::MouseX ? 0 : 1) );
			}
		}

		// debug
		if (Key == EKeys::MouseX)
		{
			DebugSmoothedMouseX += NewVal * DebugSmoothedMouseSensitivity;
		}
		else
		{
			DebugSmoothedMouseY += -NewVal * DebugSmoothedMouseSensitivity;
		}
	}

	return NewVal;
}

void UPlayerInput::Tick(float DeltaTime)
{
	ConditionalInitAxisProperties();

	if (GetDefault<UInputSettings>()->bEnableGestureRecognizer)
	{
		// Skip the software gesture recognizer when native platform gesture recognizers are active.
		if (!FGenericPlatformApplicationMisc::AreNativeGesturesEnabled())
		{
			SCOPE_CYCLE_COUNTER(STAT_PC_GestureRecognition);
			GestureRecognizer.DetectGestures(TouchStates, this, DeltaTime);
		}
	}
}

void UPlayerInput::ConsumeKey(FKey Key)
{
	FKeyState& KeyState = KeyStateMap.FindOrAdd(Key);
	KeyState.bConsumed = true;
}

bool UPlayerInput::KeyEventOccurred(FKey Key, EInputEvent Event, TArray<uint32>& InEventIndices, const FKeyState* KeyState) const
{
	if (KeyState == nullptr)
	{
		KeyState = KeyStateMap.Find(Key);
	}
	if (KeyState && KeyState->EventCounts[Event].Num() > 0)
	{
		InEventIndices = KeyState->EventCounts[Event];
		return true;
	}

	return false;
}

bool UPlayerInput::IsAltPressed() const
{
	return IsPressed(EKeys::LeftAlt) || IsPressed(EKeys::RightAlt);
}

bool UPlayerInput::IsCtrlPressed() const
{
#if PLATFORM_MAC
	return IsPressed(EKeys::LeftCommand) || IsPressed(EKeys::RightCommand);
#else
	return IsPressed(EKeys::LeftControl) || IsPressed(EKeys::RightControl);
#endif
}

bool UPlayerInput::IsShiftPressed() const
{
	return IsPressed(EKeys::LeftShift) || IsPressed(EKeys::RightShift);
}

bool UPlayerInput::IsCmdPressed() const
{
#if PLATFORM_MAC
	return IsPressed(EKeys::LeftControl) || IsPressed(EKeys::RightControl);
#else
	return IsPressed(EKeys::LeftCommand) || IsPressed(EKeys::RightCommand);
#endif
}

void UPlayerInput::ConditionalInitAxisProperties()
{
	// Initialize AxisProperties map if needed.
	if ( AxisProperties.Num() == 0 )
	{
		// move stuff from config structure to our runtime structure
		for (const FInputAxisConfigEntry& AxisConfigEntry : AxisConfig)
		{
			const FKey AxisKey = AxisConfigEntry.AxisKeyName;
			if (AxisKey.IsValid())
			{
				AxisProperties.Add(AxisKey, AxisConfigEntry.AxisProperties);
			}
		}
	}
}

bool UPlayerInput::IsKeyHandledByAction( FKey Key ) const
{
	for (const FInputActionKeyMapping& Mapping : ActionMappings)
	{
		if( (Mapping.Key == Key || Mapping.Key == EKeys::AnyKey) &&
			(Mapping.bAlt == false || IsAltPressed()) &&
			(Mapping.bCtrl == false || IsCtrlPressed()) &&
			(Mapping.bShift == false || IsShiftPressed()) &&
			(Mapping.bCmd == false || IsCmdPressed()) )
		{
			return true;
		}
	}

	return false;
}

void UPlayerInput::GetActionsBoundToKey(UInputComponent* InputComponent, FKey Key, TArray<TSharedPtr<FInputActionBinding>>& Actions)
{
	InputComponent->ConditionalBuildKeyMap(this);
	FGetActionsBoundToKey::Get(InputComponent, this, Key, Actions);
}

const TArray<FInputActionKeyMapping>& UPlayerInput::GetKeysForAction(const FName ActionName) const
{
	ConditionalBuildKeyMappings();

	const FActionKeyDetails* KeyDetails = ActionKeyMap.Find(ActionName);
	if (KeyDetails)
	{
		return KeyDetails->Actions;
	}

	return UPlayerInput::NoKeyMappings;
}

const TArray<FInputAxisKeyMapping>& UPlayerInput::GetKeysForAxis(const FName AxisName) const
{
	ConditionalBuildKeyMappings();

	const FAxisKeyDetails* KeyDetails = AxisKeyMap.Find(AxisName);
	if (KeyDetails)
	{
		return KeyDetails->KeyMappings;
	}

	return UPlayerInput::NoAxisMappings;
}

#if !UE_BUILD_SHIPPING
bool UPlayerInput::ExecInputCommands( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	bool bResult = false;

	int32 CmdLen = FCString::Strlen(Cmd);
	TCHAR* Line = (TCHAR*)FMemory::Malloc(sizeof(TCHAR)*(CmdLen+1));

	while( FParse::Line( &Cmd, Line, CmdLen+1) )		// The FParse::Line function expects the full array size, including the NULL character.
	{
		const TCHAR* Str = Line;
		if(CurrentEvent == IE_Pressed || (CurrentEvent == IE_Released && FParse::Command(&Str,TEXT("OnRelease"))))
		{
			APlayerController*	Actor = Cast<APlayerController>(GetOuter());
			UPlayer* Player = Actor ? ToRawPtr(Actor->Player) : NULL;
			if(ProcessConsoleExec(Str,Ar,this))
			{
				bResult = true;
				continue;
			}
			else if(Actor && Exec(Actor->GetWorld(), Str,Ar))
			{
				bResult = true;
				continue;
			}
			else if(Player && Player->Exec( Actor->GetWorld(), Str,Ar))
			{
				bResult = true;
				continue;
			}
		}
		else
		{
			bResult |= Exec(InWorld, Str,Ar);
		}
	}

	FMemory::Free(Line);

	return bResult;
}

bool UPlayerInput::Exec(UWorld* InWorld, const TCHAR* Str,FOutputDevice& Ar)
{
	TCHAR Temp[256];

	if( FParse::Command( &Str, TEXT("KEYBINDING") ) && FParse::Token( Str, Temp, UE_ARRAY_COUNT(Temp), 0 ) )
	{
		FKey const Key(Temp);
		if (Key.IsValid())
		{
			for(uint32 BindIndex = 0;BindIndex < (uint32)DebugExecBindings.Num();BindIndex++)
			{
				if (DebugExecBindings[BindIndex].Key == Key)
				{
					Ar.Logf(TEXT("%s"),*DebugExecBindings[BindIndex].Command);
					break;
				}
			}
		}

		return 1;
	}
	else if( !bExecutingBindCommand && FParse::Token( Str, Temp, UE_ARRAY_COUNT(Temp), 0 ) )
	{
		FKey const Key(Temp);
		if(Key.IsValid())
		{
			for(int32 BindIndex = DebugExecBindings.Num() - 1; BindIndex >= 0; BindIndex--)
			{
				if(DebugExecBindings[BindIndex].Key == Key)
				{
					bExecutingBindCommand = true;
					bool bResult = ExecInputCommands(GetWorld(), *DebugExecBindings[BindIndex].Command,Ar);
					bExecutingBindCommand = false;
					return bResult;
				}
			}
		}
	}

	return 0;
}

FString UPlayerInput::GetBind(FKey Key)
{
	static bool bDebugExecBindingsAllowed = !FParse::Param( FCommandLine::Get(), TEXT("NoDebugExecBindings") );

	if ( bDebugExecBindingsAllowed )
	{
		const bool bControlPressed = IsCtrlPressed();
		const bool bShiftPressed = IsShiftPressed();
		const bool bAltPressed = IsAltPressed();
		const bool bCmdPressed = IsCmdPressed();

		for ( int32 BindIndex = DebugExecBindings.Num() - 1; BindIndex >= 0; BindIndex-- )
		{
			const FKeyBind&	Bind = DebugExecBindings[BindIndex];
			if ( Bind.Key == Key && !Bind.bDisabled )
			{
				// if the modifier key pressed [or this key-bind doesn't require that key], and the key-bind isn't
				// configured to ignore they modifier key, we've found a match.
				if ((!Bind.Control || bControlPressed) && (!Bind.Shift || bShiftPressed) && (!Bind.Alt || bAltPressed) && (!Bind.Cmd || bCmdPressed)
					&&	(!Bind.bIgnoreCtrl || !bControlPressed) && (!Bind.bIgnoreShift || !bShiftPressed) && (!Bind.bIgnoreAlt || !bAltPressed) && (!Bind.bIgnoreCmd || !bCmdPressed))
				{
					return DebugExecBindings[BindIndex].Command;
				}
			}
		}
	}

	return TEXT("");
}

FKeyBind UPlayerInput::GetExecBind(FString const& ExecCommand)
{
	FKeyBind Binding;
	for( auto InputBindingIt = DebugExecBindings.CreateConstIterator(); InputBindingIt; ++InputBindingIt )
	{
		if(InputBindingIt->Command == ExecCommand)
		{
			Binding = *InputBindingIt;
			break;
		}
	}
	return Binding;
}
#endif

void UPlayerInput::SetBind(FName BindName, const FString& Command)
{
#if !UE_BUILD_SHIPPING

	// Split modfiiers; eg controL+alt+j into [control, alt, j]
	TArray<FString> Tokens;
	BindName.ToString().ParseIntoArray(Tokens, TEXT("+"), true);

	FKey BindKey;
	FKeyBind NewBind;
	for (const FString& Token : Tokens)
	{
		if (Token.Equals(TEXT("ctrl"), ESearchCase::IgnoreCase)
			|| Token.Equals(TEXT("control"), ESearchCase::IgnoreCase))
		{
			NewBind.Control = true;
		}
		else if (Token.Equals(TEXT("alt"), ESearchCase::IgnoreCase))
		{
			NewBind.Alt = true;
		}
		else if (Token.Equals(TEXT("shift"), ESearchCase::IgnoreCase))
		{
			NewBind.Shift = true;
		}
		else if (Token.Equals(TEXT("cmd"), ESearchCase::IgnoreCase))
		{
			NewBind.Cmd = true;
		}
		else if (Token.Len() > 0)
		{
			BindKey = FKey(FName(Token));
		}
	}
	if (BindKey.IsValid())
	{
		FString CommandMod = Command;
		if ( CommandMod.Left(1) == TEXT("\"") && CommandMod.Right(1) == ("\"") )
		{
			CommandMod.MidInline(1, CommandMod.Len() - 2, EAllowShrinking::No);
		}

		for(int32 BindIndex = DebugExecBindings.Num()-1;BindIndex >= 0;BindIndex--)
		{
			// if it matches an existing binding (except for command), then overwrite the old one.
			const FKeyBind& ExistingBinding = DebugExecBindings[BindIndex];
			if (ExistingBinding.Key == BindKey
				&& ExistingBinding.Alt == NewBind.Alt
				&& ExistingBinding.Cmd == NewBind.Cmd
				&& ExistingBinding.Shift == NewBind.Shift
				&& ExistingBinding.Control ==  NewBind.Control)
			{
				DebugExecBindings[BindIndex].Command = CommandMod;
				SaveConfig();
				return;
			}
		}

		NewBind.Key = BindKey;
		NewBind.Command = CommandMod;
		DebugExecBindings.Add(NewBind);
		SaveConfig();
	}
#endif
}

class UWorld* UPlayerInput::GetWorld() const
{
	if (APlayerController* PC = GetOuterAPlayerController())
	{
		return PC->GetWorld();	
	}
	
	return Super::GetWorld();
}

