// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/PlayerInputDebugging.h"

#if UE_PLAYER_INPUT_INCLUDE_DEBUG

#include "GameFramework/PlayerInput.h"

namespace UE::Input
{
	
FPlayerInputDebugging::FOnPlayerInputFlushed FPlayerInputDebugging::OnPlayerInputFlushed;
FPlayerInputDebugging::FOnPlayerInputDelegateExecuted FPlayerInputDebugging::OnPlayerInputEventExecuted;

void FPlayerInputDebugging::BroadcastPlayerInputDelegateExecuted(const UPlayerInput* PlayerInput, const FPlayerInputDebuggingArgs& EventData)
{
	if (OnPlayerInputEventExecuted.IsBound())
	{
		OnPlayerInputEventExecuted.Broadcast(PlayerInput, EventData);
	}
}

void FPlayerInputDebugging::BroadcastPlayerInputFlushed(const UPlayerInput* PlayerInput)
{
	if (OnPlayerInputFlushed.IsBound())
	{
		OnPlayerInputFlushed.Broadcast(PlayerInput);
	}
}
}

#endif	// #if UE_PLAYER_INPUT_INCLUDE_DEBUG
