// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VisualizeRT.cpp: Implements the VisualizeRT Slate window
=============================================================================*/

#include "Framework/Views/TableViewMetadata.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Views/SListView.h"
#include "VisualizeTexture.h"

#define ENABLE_IMAGES	0

static FName ColumnName("Name");
static FName ColumnWidth("Width");
static FName ColumnHeight("Height");
static FName ColumnDepth("Depth");
static FName ColumnFormat("Format");
static FName ColumnDimensions("Dim");
static FName ColumnSize("Size");
static FName ColumnType("Type");
static FName ColumnNumber("Number");


struct FRTInfo : public FRefCountedObject
{
	FString Dimensions;
	FString Width;
	FString Height;
	FString Depth;
	FString Format;
	FString Number;
	FString Name;
	FString Size;
	FString Type;
#if ENABLE_IMAGES
	TSharedPtr<FSlateBrush> Image;
#endif

	bool Parse(const FString& In)
	{
		FString Text = In;
		Depth = TEXT("-");
		Type = TEXT("-");

		// Current Desc Info format: (DIM W[xH[xD]] FMT[ RT]) NUM NAME SIZEkB
		if ( !Text.IsEmpty() && Text.Split(TEXT(" "), &Dimensions, &Text, ESearchCase::CaseSensitive))
		{
			Dimensions.MidInline(1, MAX_int32, EAllowShrinking::No);
			if (Dimensions.StartsWith(TEXT("Cube")))
			{
				if (!Text.Split(TEXT(" "), &Width, &Text, ESearchCase::CaseSensitive))
				{
					return false;
				}

				Height = Width;
			}
			else
			{
				if (Text.Split(TEXT("x"), &Width, &Text))
				{
					if (Dimensions.Find(TEXT("3")) == 0)
					{
						if (!Text.Split(TEXT("x"), &Height, &Text))
						{
							return false;
						}

						if (!Text.Split(TEXT(" "), &Depth, &Text, ESearchCase::CaseSensitive))
						{
							return false;
						}
					}
					else
					{
						if (!Text.Split(TEXT(" "), &Height, &Text, ESearchCase::CaseSensitive))
						{
							return false;
						}
					}
				}
			}

			// Might or not have Type
			if (Text.Split(TEXT(") "), &Format, &Text, ESearchCase::CaseSensitive))
			{
				if (Text.Split(TEXT(" "), &Number, &Text, ESearchCase::CaseSensitive))
				{
					if (Text.Split(TEXT(" "), &Name, &Size, ESearchCase::CaseSensitive))
					{
						// Get the type if found
						int32 Found = Format.Find(TEXT(" "), ESearchCase::CaseSensitive);
						if (Found > 0)
						{
							Type = Format.Mid(Found + 1);
							Format.MidInline(0, Found, EAllowShrinking::No);
						}
						return true;
					}
				}
			}
		}

		return false;
	}
};

struct FReferenceToRenderer
{
	IRendererModule& RendererModule;
	FReferenceToRenderer() : RendererModule(FModuleManager::LoadModuleChecked<IRendererModule>(TEXT("Renderer")))
	{
	}
};

class SVisualizeRTWidget : public SCompoundWidget, public FReferenceToRenderer
{
public:
	SLATE_BEGIN_ARGS(SVisualizeRTWidget)
	{
	}

	SLATE_END_ARGS()

	TArray< TSharedPtr<FRTInfo> > ListItems;
	FRTInfo* Selected;

	void Construct(const FArguments& InArgs )
	{
		ChildSlot
		[
#if ENABLE_IMAGES
			SNew(SSplitter)
			+ SSplitter::Slot()
			[
#endif
				// Info table (left)
				SNew( SListView< TSharedPtr<FRTInfo> > )
				//.ItemHeight(24)
				.ListItemsSource( &ListItems )
				.OnGenerateRow( this, &SVisualizeRTWidget::OnGenerateWidgetForList )
				.OnSelectionChanged(this, &SVisualizeRTWidget::OnSelectionChanged)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(ColumnNumber).DefaultLabel(NSLOCTEXT("VisualizeRT", "NumberColumnHeader", "-")).FillWidth(1)
					+ SHeaderRow::Column(ColumnName).DefaultLabel(NSLOCTEXT("VisualizeRT", "NameColumnHeader", "Name")).FillWidth(12)
					+ SHeaderRow::Column(ColumnWidth).DefaultLabel(NSLOCTEXT("VisualizeRT", "WidthColumnHeader", "Width")).HAlignCell(HAlign_Right).FillWidth(2)
					+ SHeaderRow::Column(ColumnHeight).DefaultLabel(NSLOCTEXT("VisualizeRT", "HeightColumnHeader", "Height")).HAlignCell(HAlign_Right).FillWidth(2)
					+ SHeaderRow::Column(ColumnDepth).DefaultLabel(NSLOCTEXT("VisualizeRT", "DepthColumnHeader", "Depth")).HAlignCell(HAlign_Right).FillWidth(2)
					+ SHeaderRow::Column(ColumnFormat).DefaultLabel(NSLOCTEXT("VisualizeRT", "FormatColumnHeader", "Format")).FillWidth(6)
					+ SHeaderRow::Column(ColumnType).DefaultLabel(NSLOCTEXT("VisualizeRT", "TypeColumnHeader", "Type")).FillWidth(5)
					+ SHeaderRow::Column(ColumnDimensions).DefaultLabel(NSLOCTEXT("VisualizeRT", "DimensionsColumnHeader", "Dimensions")).FillWidth(3)
					+ SHeaderRow::Column(ColumnSize).DefaultLabel(NSLOCTEXT("VisualizeRT", "SizeColumnHeader", "Size (kb)")).HAlignCell(HAlign_Right).FillWidth(3)
				)
#if ENABLE_IMAGES
			]
			+ SSplitter::Slot()
			[
				// Image (right)
				SNew(SSplitter).Orientation(Orient_Vertical)
				+ SSplitter::Slot().Value(1)
				[
					// Control checkboxes (top)
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: R G B A")))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: Mip")))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: Face")))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: Luminance")))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: Color Multiplier")))
					]
					+SVerticalBox::Slot()
					[
						SNew(STextBlock).Text(FText::FromString(TEXT("TODO: Alpha Multiplier")))
					]
				]
				+ SSplitter::Slot().Value(4)
				[
					// Actual image (bottom)
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot().Padding(24)
					[
						SNew(SImage).Image(this, &SVisualizeRTWidget::GetRTImage)
					]
				]
			]
#endif
		];

		// Get List from Renderer
		TArray<FString> VisTextureEntries;
		GVisualizeTexture.GetTextureInfos_GameThread(VisTextureEntries);

		uint32 TotalSize = 0;
		for (int32 Index = 0; Index < VisTextureEntries.Num(); ++Index)
		{
			TSharedPtr<FRTInfo> RTInfo = MakeShareable(new FRTInfo);
			if (RTInfo->Parse(VisTextureEntries[Index]))
			{
#if ENABLE_IMAGES
				FColor Color(0x80808080 + (Index << 9) +  ((Index&3) << 2));
				Color.A = 255;
				RTInfo->Image = MakeShareable(new FSlateColorBrush(Color));
#endif
				TotalSize += FCString::Atoi(*RTInfo->Size);
				ListItems.Add(RTInfo);
			}
		}

		TSharedPtr<FRTInfo> RTTotal = MakeShareable(new FRTInfo);
		RTTotal->Depth = TEXT("-");
		RTTotal->Dimensions = TEXT("-");
		RTTotal->Format = TEXT("-");
		RTTotal->Height = TEXT("-");
		RTTotal->Name = TEXT("TOTAL");
		RTTotal->Number = TEXT("-");
		RTTotal->Type = TEXT("-");
		RTTotal->Width = TEXT("-");
		RTTotal->Size = FString::Printf(TEXT("%d"), TotalSize);
		ListItems.Add(RTTotal);

		Selected = nullptr;
		OnSelectionChanged(ListItems[0], ESelectInfo::Direct);
	}

	void OnSelectionChanged(TSharedPtr<FRTInfo> Selection, ESelectInfo::Type SelectInfo)
	{
		if( Selection.IsValid() )
		{
			Selected = Selection.Get();
			if(Selection->Number != TEXT("-"))
			{
				FString Cmd = Selected->Number;
				RendererModule.ExecVisualizeTextureCmd(Cmd);
			}
		}
	}

#if ENABLE_IMAGES
	const FSlateBrush* GetRTImage()
	{
		return Selected->Image.Get();
	}
#endif

	struct FRow : public SMultiColumnTableRow< TSharedPtr< FRTInfo > >
	{
		SLATE_BEGIN_ARGS( FRow ){}
		SLATE_END_ARGS()

		TSharedPtr<FRTInfo> Info;

		void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FRTInfo> InItem )
		{
			Info = InItem;
			SMultiColumnTableRow< TSharedPtr<FRTInfo> >::Construct( FSuperRowType::FArguments(), InOwnerTable );
		}

		TSharedRef<SWidget> GenerateWidgetForColumn( const FName& Column )
		{
			FText Label;
			if ( Column == ColumnName )
			{
				Label = FText::FromString(Info->Name);
			}
			else if ( Column == ColumnWidth )
			{
				Label = FText::FromString(Info->Width);
			}
			else if ( Column == ColumnDepth )
			{
				Label = FText::FromString(Info->Depth);
			}
			else if ( Column == ColumnHeight )
			{
				Label = FText::FromString(Info->Height);
			}
			else if ( Column == ColumnFormat )
			{
				Label = FText::FromString(Info->Format);
			}
			else if ( Column == ColumnDimensions )
			{
				Label = FText::FromString(Info->Dimensions);
			}
			else if ( Column == ColumnSize )
			{
				Label = FText::FromString(Info->Size);
			}
			else if ( Column == ColumnType )
			{
				Label = FText::FromString(Info->Type);
			}
			else if ( Column == ColumnNumber )
			{
				Label = FText::FromString(Info->Number);
			}

			if (!Label.IsEmpty())
			{
				return SNew( SHorizontalBox )
					+SHorizontalBox::Slot().Padding(2)
					[
						SNew( STextBlock ).Text( Label )
					];
			}
			else
			{
				return SNew( STextBlock ).Text( NSLOCTEXT("VisualizeRenderTargets", "UnknownColumnName", "N/A") );
			}
		}
	};

	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRTInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew( FRow, OwnerTable, Item );
	}
};

bool HandleVisualizeRT()
{
	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title( NSLOCTEXT("VisualizeRT", "Title", "Visualize Render Targets") )
		.ClientSize( FVector2D(1024, 640) )
		.AutoCenter(EAutoCenter::None)
		.SupportsMaximize(true)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::UserSized);
	Window = FSlateApplication::Get().AddWindow( Window.ToSharedRef() );
	Window->SetContent(SNew(SVisualizeRTWidget));

	return true;
}
