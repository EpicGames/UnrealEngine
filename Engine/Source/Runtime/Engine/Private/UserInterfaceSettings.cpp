// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/UserInterfaceSettings.h"

#include "Logging/MessageLog.h"
#include "Fonts/FontCache.h"

#include "Engine/DPICustomScalingRule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserInterfaceSettings)

#define LOCTEXT_NAMESPACE "Engine"

UUserInterfaceSettings::UUserInterfaceSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderFocusRule(ERenderFocusRule::NavigationOnly)
	, ApplicationScale(1)
	, bLoadWidgetsOnDedicatedServer(true)
	, bAuthorizeAutomaticWidgetVariableCreation(true)
#if WITH_EDITORONLY_DATA
	, CustomFontDPI(FontConstants::RenderDPI)
	, FontDPIPreset(ConvertToEFontDPI(CustomFontDPI))
	, bUseCustomFontDPI(false)
#endif
	, bEnableDistanceFieldFontRasterization(false)
{
	SectionName = TEXT("UI");
}

void UUserInterfaceSettings::PostInitProperties()
{
	Super::PostInitProperties();

	if ( !DefaultCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::Default, DefaultCursor_DEPRECATED);
		DefaultCursor_DEPRECATED.Reset();
	}

	if ( !TextEditBeamCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::TextEditBeam, TextEditBeamCursor_DEPRECATED);
		TextEditBeamCursor_DEPRECATED.Reset();
	}

	if ( !CrosshairsCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::Crosshairs, CrosshairsCursor_DEPRECATED);
		CrosshairsCursor_DEPRECATED.Reset();
	}

	if ( !HandCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::Hand, HandCursor_DEPRECATED);
		HandCursor_DEPRECATED.Reset();
	}

	if ( !GrabHandCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::GrabHand, GrabHandCursor_DEPRECATED);
		GrabHandCursor_DEPRECATED.Reset();
	}

	if ( !GrabHandClosedCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::GrabHandClosed, GrabHandClosedCursor_DEPRECATED);
		GrabHandClosedCursor_DEPRECATED.Reset();
	}

	if ( !SlashedCircleCursor_DEPRECATED.IsNull() )
	{
		SoftwareCursors.Add(EMouseCursor::SlashedCircle, SlashedCircleCursor_DEPRECATED);
		SlashedCircleCursor_DEPRECATED.Reset();
	}

	// Allow the assets to be replaced in the editor, but make sure they're part of the root set in cooked games
#if WITH_EDITOR
	if ( IsTemplate() == false )
	{
		ForceLoadResources();
	}
#else
	ForceLoadResources();
#endif
}

float UUserInterfaceSettings::GetDPIScaleBasedOnSize(FIntPoint Size) const
{
	float Scale = 1.0f;

	if (LastViewportSize.IsSet() && Size == LastViewportSize.GetValue())
	{
		Scale = CalculatedScale;
	}
	else
	{
		bool bOutError;
		Scale = CalculateScale(Size, bOutError);
		if (!bOutError)
		{
			CalculatedScale = Scale;
#if !WITH_EDITOR
			// Only cache the value outside the editor.
			LastViewportSize = Size;
#endif
		}
	}

	return FMath::Max(Scale * ApplicationScale, 0.01f);
}

float UUserInterfaceSettings::CalculateScale(FIntPoint Size, bool& bError) const
{
	bError = false;

	if (UIScaleRule == EUIScalingRule::Custom)
	{
		if (CustomScalingRuleClassInstance == nullptr)
		{
			CustomScalingRuleClassInstance = CustomScalingRuleClass.TryLoadClass<UDPICustomScalingRule>();

			if (CustomScalingRuleClassInstance == nullptr)
			{
				FMessageLog("MapCheck").Error()
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("CustomScalingRule_NotFound", "Project Settings - User Interface Custom Scaling Rule '{0}' could not be found."), FText::FromString(CustomScalingRuleClass.ToString()))));
				bError = true;
				return 1;
			}
		}

		if (CustomScalingRule == nullptr)
		{
			CustomScalingRule = CustomScalingRuleClassInstance->GetDefaultObject<UDPICustomScalingRule>();
		}

		return CustomScalingRule->GetDPIScaleBasedOnSize(Size);
	}
	else
	{
		int32 EvalPoint = 0;
		switch (UIScaleRule)
		{
		case EUIScalingRule::ShortestSide:
			EvalPoint = FMath::Min(Size.X, Size.Y);
			break;
		case EUIScalingRule::LongestSide:
			EvalPoint = FMath::Max(Size.X, Size.Y);
			break;
		case EUIScalingRule::Horizontal:
			EvalPoint = Size.X;
			break;
		case EUIScalingRule::Vertical:
			EvalPoint = Size.Y;
			break;
		case EUIScalingRule::ScaleToFit:
			return DesignScreenSize.X > 0 && DesignScreenSize.Y > 0 ? FMath::Min((float)(Size.X) / DesignScreenSize.X, (float)(Size.Y) / DesignScreenSize.Y) : 1.f;
		}

		const FRichCurve* DPICurve = UIScaleCurve.GetRichCurveConst();
		return DPICurve->Eval((float)EvalPoint, 1.0f);
	}
}

uint32 UUserInterfaceSettings::GetFontDisplayDPI() const
{
	if (bUseCustomFontDPI)
	{
		return CustomFontDPI;
	}
	return ConvertToFontDPI(FontDPIPreset);
}

constexpr uint32 UUserInterfaceSettings::ConvertToFontDPI(EFontDPI inFontDPIEntry)
{
	switch (inFontDPIEntry)
	{
	case EFontDPI::Unreal:
		return 96;
	case EFontDPI::Standard:
	case EFontDPI::Custom:
	default:
		return 72;
	}
}

float UUserInterfaceSettings::ConvertFontSizeFromNativeToDisplay(float NativeFontSize) const
{
	const float FontDisplayDPI = static_cast<float>(GetFontDisplayDPI());
	const float DisplayedSize = NativeFontSize * static_cast<float>(FontConstants::RenderDPI) / FontDisplayDPI;
	return DisplayedSize;
}

float UUserInterfaceSettings::ConvertFontSizeFromDisplayToNative(float DisplayFontSize) const
{
	const float FontDisplayDPI = static_cast<float>(GetFontDisplayDPI());
	const float NativeSize = DisplayFontSize * FontDisplayDPI / static_cast<float>(FontConstants::RenderDPI);
	const float RoundedSize = FMath::GridSnap(NativeSize, 0.01f);
	return RoundedSize;
}

#if WITH_EDITOR
FText UUserInterfaceSettings::GetFontDPIDisplayString() const
{
	const uint32 DisplayDPI = GetFontDisplayDPI();

	//Wether the Display DPI is from a custom value or preset, try to find the matching preset.
	const EFontDPI Preset = ConvertToEFontDPI(GetFontDisplayDPI());
	if (Preset != EFontDPI::Custom)
	{
		const UEnum* Enum = StaticEnum<EFontDPI>();
		return Enum->GetDisplayNameTextByValue(static_cast<int64>(Preset));
	}

	//If no preset found, just use the value.
	FFormatNamedArguments Args;
	Args.Add(TEXT("DPI"), FText::AsNumber(DisplayDPI));
	return FText::Format(LOCTEXT("DPI","{DPI} DPI"), Args);
}

constexpr EFontDPI UUserInterfaceSettings::ConvertToEFontDPI(uint32 inFontDPI)
{
	switch (inFontDPI)
	{
		case 72: return EFontDPI::Standard;
		case 96: return EFontDPI::Unreal;
		default: return EFontDPI::Custom;
	}
}

void UUserInterfaceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Look for changed properties
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UUserInterfaceSettings, CustomScalingRuleClass))
	{
		CustomScalingRuleClassInstance = nullptr;
		CustomScalingRule = nullptr;
	}

	LastViewportSize.Reset();
}

#endif

void UUserInterfaceSettings::ForceLoadResources(bool bForceLoadEverything) const
{
	bool bShouldLoadCursors = true;

	if (IsRunningCommandlet())
	{
		bShouldLoadCursors = false;
	}
	else if (IsRunningDedicatedServer())
	{
		bShouldLoadCursors = bLoadWidgetsOnDedicatedServer;
	}

	if (bShouldLoadCursors || bForceLoadEverything)
	{
		SCOPED_BOOT_TIMING("UUserInterfaceSettings::ForceLoadResources");

		TArray<UObject*> LoadedClasses;
		for ( auto& Entry : SoftwareCursors )
		{
			LoadedClasses.Add(Entry.Value.TryLoad());
		}

		for (int32 i = 0; i < LoadedClasses.Num(); ++i)
		{
			UObject* Cursor = LoadedClasses[i];
			if (Cursor)
			{
#if !WITH_EDITOR
				// Add to root in case this was loaded after disregard for GC closes
				Cursor->AddToRoot();
#endif
				CursorClasses.Add(Cursor);
			}
			else
			{
				UE_LOGF(LogLoad, Warning, "UUserInterfaceSettings::ForceLoadResources: Failed to load cursor resource %d.", i);
			}
		}

		CustomScalingRuleClassInstance = CustomScalingRuleClass.TryLoadClass<UDPICustomScalingRule>();
	}
}

bool UUserInterfaceSettings::ShouldDisplayDistanceFieldFontSettings()
{
	return IsSlateSdfTextFeatureEnabled();
}

#undef LOCTEXT_NAMESPACE
