// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlockCodingHelpers.h"
#include "VirtualTextureUploadCache.h"

#define MAKE_565(r,g,b) (((r>>3) << 11) | ((g>>2) << 5) | (b>>3))
#define MAKE_8888(r,g,b,a) ((a << 24) | (b << 16) | (g << 8) | r)

#pragma pack(push, 1)

struct DXT1Block
{
	uint16_t color0;
	uint16_t color1;
	uint32_t bits;
};

struct AlphaBlock
{
	uint8_t alpha0;
	uint8_t alpha1;
	uint16_t bits0;
	uint32_t bits1;
};

struct BC4Block
{
	AlphaBlock gray;
};

struct DXT5Block
{
	BC4Block alpha;
	DXT1Block color;
};

struct BC5Block
{
	AlphaBlock x;
	AlphaBlock y;
};

struct BC6Block
{
	uint64_t bits[2];
};

struct BC7Block
{
	uint32_t bits[4];
};

struct ASTCBlock
{
	uint32_t voidExtentConfig[2];
	uint16_t colors[4];
};

static_assert(sizeof(ASTCBlock) == 4 * sizeof(uint32_t), "Astc block size mismatch");
static_assert(sizeof(BC5Block) == 16, "BC5 block size mismatch");
static_assert(sizeof(DXT5Block) == 16, "DXT5 block size mismatch");
static_assert(sizeof(BC4Block) == 8, "BC4 block size mismatch");
static_assert(sizeof(DXT1Block) == 8, "DXT1 block size mismatch");

struct FloatPixel
{
	float r;
	float g;
	float b;
	float a;
};

struct Float16Pixel
{
	uint16_t r;
	uint16_t g;
	uint16_t b;
	uint16_t a;
};

#pragma pack(pop)

DXT1Block MakeDXT1(const uint8 *Rgb)
{
	DXT1Block Result;
	Result.color0 = MAKE_565(Rgb[0], Rgb[1], Rgb[2]);
	Result.color1 = 0xFFFF;// MAKE_565(rgb[0], rgb[1], rgb[2]);
	Result.bits = 0;
	return Result;
}

BC4Block MakeBC4(const uint8 *R)
{
	BC4Block Result;
	Result.gray.alpha0 = R[0];
	Result.gray.alpha1 = R[0];
	Result.gray.bits0 = 0;
	Result.gray.bits1 = 0;
	return Result;
}

DXT5Block MakeDXT5(const uint8 *Rgba)
{
	DXT5Block Result;
	Result.color = MakeDXT1(Rgba);
	Result.alpha = MakeBC4(Rgba + 3);
	return Result;
}

BC5Block MakeBC5(const uint8 *Rg)
{
	BC5Block res;
	res.x.alpha0 = Rg[0];
	res.x.alpha1 = Rg[0];
	res.x.bits0 = 0;
	res.x.bits1 = 0;
	res.y.alpha0 = Rg[1];
	res.y.alpha1 = Rg[1];
	res.y.bits0 = 0;
	res.y.bits1 = 0;
	return res;
}

BC7Block MakeBC7(const uint8 *rgb)
{
	BC7Block res;
	res.bits[0] = (1 << 6);
	res.bits[0] |= (rgb[0] >> 1) << 7;
	res.bits[0] |= (rgb[0] >> 1) << 14;
	res.bits[0] |= (rgb[1] >> 1) << 21;
	res.bits[0] |= ((rgb[1] >> 1) & 15) << 28;
	res.bits[1] = rgb[1] >> 5;
	res.bits[1] |= (rgb[2] >> 1) << 3;
	res.bits[1] |= (rgb[2] >> 1) << 10;
	res.bits[1] |= (255 >> 1) << 17;
	res.bits[1] |= (255 >> 1) << 24;
	res.bits[1] |= 1 << 31;
	res.bits[2] = 0;
	res.bits[3] = 0;
	return res;
}

ASTCBlock MakeASTC(const uint8 *rgb, bool sRGB)
{
	// create a 2D void-extend block (see spec c.2.23)

#define UNORM16(x) (x << 8) | x
#define UNORM16_SRGB(x) (x << 8) | 0x80

	ASTCBlock res;
	res.voidExtentConfig[0] = 0xFFFFFDFC;
	res.voidExtentConfig[1] = 0xFFFFFFFF;

	res.colors[0] = sRGB ? UNORM16_SRGB(rgb[0]) : UNORM16(rgb[0]); // B
	res.colors[1] = sRGB ? UNORM16_SRGB(rgb[1]) : UNORM16(rgb[1]); // G
	res.colors[2] = sRGB ? UNORM16_SRGB(rgb[2]) : UNORM16(rgb[2]); // R
	res.colors[3] = sRGB ? UNORM16_SRGB(0xFF) : UNORM16(0xFF);   // A

#undef UNORM16
#undef UNORM16_SRGB

	return res;
}

uint16_t Float32ToFloat16(float value)
{
	// Let's follow the strict aliasing rules (kinda, this still violates them, but it should be supported by all compilers)
	union FloatInt { float f; unsigned int i; } fi;
	fi.f = value;
	unsigned int float32 = fi.i;

	uint16_t float16 = (float32 >> 31) << 5;
	unsigned short tmp = (float32 >> 23) & 0xff;
	tmp = (tmp - 0x70) & ((unsigned int)((int)(0x70 - tmp) >> 4) >> 27);
	float16 = (float16 | tmp) << 10;
	float16 |= (float32 >> 13) & 0x3ff;
	return float16;
}

Float16Pixel MakeFloat16(const uint8 *rgb)
{
	Float16Pixel res;
	res.r = Float32ToFloat16(rgb[0] / 255.0f);
	res.g = Float32ToFloat16(rgb[1] / 255.0f);
	res.b = Float32ToFloat16(rgb[2] / 255.0f);
	res.a = Float32ToFloat16(1.0f);
	return res;
}


BC6Block MakeBC6(const uint8 *rgb)
{
	Float16Pixel half = MakeFloat16(rgb);
	BC6Block res;

	// Quantize
	half.r = (unsigned int)half.r * 64 / 31;
	half.g = (unsigned int)half.g * 64 / 31;
	half.b = (unsigned int)half.b * 64 / 31;
	half.a = (unsigned int)half.a * 64 / 31;

	// Syntax from LSB to MSB for Mode 3 (easiest mode, only endpoints and indices, no partition)
	// Mode (5 bit), endpoint0 r (10 bit), endpoint0 g (10 bit), endpoint0 b (10 bit), endpoint1 r (10 bit), endpoint1 g (10 bit), endpoint1 b (10 bit), indices (63 bits)
	res.bits[0] = (half.b >> 7);
	res.bits[0] <<= 10;
	res.bits[0] |= (half.g >> 6);
	res.bits[0] <<= 10;
	res.bits[0] |= (half.r >> 6);
	res.bits[0] <<= 10;
	res.bits[0] |= (half.b >> 6);
	res.bits[0] <<= 10;
	res.bits[0] |= (half.g >> 6);
	res.bits[0] <<= 10;
	res.bits[0] |= (half.r >> 6);
	res.bits[0] <<= 5;
	res.bits[0] |= 3;

	res.bits[1] = (half.b >> 9) & 1;
	return res;
}

FloatPixel MakeFloat(const uint8 *rgb)
{
	FloatPixel res;
	res.r = rgb[0] / 255.0f;
	res.g = rgb[1] / 255.0f;
	res.b = rgb[2] / 255.0f;
	res.a = 1.0f;
	return res;
}


uint32 MakeRGBA(const uint8 *Rgba)
{
	uint32 Res;
	Res = MAKE_8888(Rgba[0], Rgba[1], Rgba[2], Rgba[3]);
	return Res;
}

uint32 MakeBGRA(const uint8 *Rgba)
{
	uint32 Res;
	Res = MAKE_8888(Rgba[2], Rgba[1], Rgba[0], Rgba[3]);
	return Res;
}

template<class PixelType> inline void PatchLine(PixelType *data, const PixelType &value, int items)
{
	for (int i = 0; i < items; i++)
	{
		data[i] = value;
	}
}

template<class PixelType> inline void PatchBlock(PixelType *data, const PixelType &value, int Width, int Height, int Stride)
{
	for (int y = 0; y < Height; ++y)
	{
		PatchLine((PixelType*)((uint8*)data + Stride * y), value, Width);
	}
}

/**
This patches a borderWidth pixels wide border around the given image to the 'value'
*/
template<class PixelType> void Patch(PixelType *data, PixelType value, int width, int height, int borderWidth, int Stride)
{
	for (int i = 0; i < borderWidth; i++)
	{
		// Patch top and bottom 'borderWidth' rows
		PatchLine((PixelType*)((uint8*)data + i * Stride), value, width);
		PatchLine((PixelType*)((uint8*)data + (height - 1 - i) * Stride), value, width);
	}

	for (int i = borderWidth; i < (height - borderWidth); i++)
	{
		PixelType *scanline = (PixelType*)((uint8*)data + i * Stride);
		// Patch left and right 'borderWidth' columns
		PatchLine(scanline, value, borderWidth);
		PatchLine(scanline + (width - borderWidth), value, borderWidth);
	}
}

#define NUM_LEVEL_COLORS 14

unsigned char MipColors[NUM_LEVEL_COLORS][3] =
{
	{ 255, 255, 255 },
	{ 255, 255, 000 },
	{ 000, 255, 255 },
	{ 000, 255, 000 },
	{ 255, 000, 255 },
	{ 255, 000, 000 },
	{ 000, 000, 255 },
	{ 128, 128, 128 },
	{ 128, 128, 000 },
	{ 000, 128, 128 },
	{ 000, 128, 000 },
	{ 128, 000, 128 },
	{ 128, 000, 000 },
	{ 000, 000, 128 }
};

unsigned char MipGreys[NUM_LEVEL_COLORS][1] =
{
	{ 255 }, // gray scale that looks good (mostly the smaller levels)
	{ 220 },
	{ 200 },
	{ 180 },
	{ 160 },
	{ 140 },
	{ 120 },
	{ 100 },
	{ 80 },
	{ 60 },
	{ 40 },
	{ 20 },
	{ 10 },
	{ 0 }
};

void BakeDebugInfo(const FVTUploadTileBuffer& TileBuffer, int32 Width, int32 Height, int32 Border, EPixelFormat Format, int32 MipLevel)
{
	unsigned char *Color = MipColors[MipLevel];
	unsigned char *Grey = MipGreys[MipLevel];

	switch (Format)
	{
	case PF_B8G8R8A8:
	{
		uint32_t Pixel = MakeBGRA(Color);
		Patch((uint32_t *)TileBuffer.Memory, Pixel, Width, Height, Border, TileBuffer.Stride);
		break;
	}
	case PF_R8G8B8A8:
	case PF_R8G8B8A8_SNORM:
	case PF_R8G8B8A8_UINT:
	{
		uint32_t Pixel = MakeRGBA(Color);
		Patch((uint32_t *)TileBuffer.Memory, Pixel, Width, Height, Border, TileBuffer.Stride);
		break;
	}
	case PF_G8:
	{
		const uint8_t Pixel = *Grey;
		Patch((uint8*)TileBuffer.Memory, Pixel, Width, Height, Border, TileBuffer.Stride);
		break;
	}
	case PF_A32B32G32R32F:
	{
		FloatPixel Fpixel = MakeFloat(Color);
		Patch((FloatPixel *)TileBuffer.Memory, Fpixel, Width, Height, Border, TileBuffer.Stride);
		break;
	}
	case PF_A16B16G16R16:
	{
		Float16Pixel Fpixel = MakeFloat16(Color);
		Patch((Float16Pixel *)TileBuffer.Memory, Fpixel, Width, Height, Border, TileBuffer.Stride);
		break;
	}
	case PF_DXT1:
	{
		DXT1Block DXT1pixel = MakeDXT1(Color);
		Patch((DXT1Block *)TileBuffer.Memory, DXT1pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_DXT5:
	{
		DXT5Block DXT5pixel = MakeDXT5(Color);
		Patch((DXT5Block *)TileBuffer.Memory, DXT5pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC4:
	{
		BC4Block BC4pixel = MakeBC4(Grey);
		Patch((BC4Block *)TileBuffer.Memory, BC4pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC5:
	{
		// This will actually code to a flat normal?
		BC5Block BC5pixel = MakeBC5(Color);

		Patch((BC5Block *)TileBuffer.Memory, BC5pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC6H:
	{
		BC6Block BC6pixel = MakeBC6(Color);
		Patch((BC6Block *)TileBuffer.Memory, BC6pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC7:
	{
		BC7Block BC7pixel = MakeBC7(Color);
		Patch((BC7Block *)TileBuffer.Memory, BC7pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_ASTC_4x4:
	{
		ASTCBlock AstcPixel = MakeASTC(Color, false);
		Patch((ASTCBlock *)TileBuffer.Memory, AstcPixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), FMath::DivideAndRoundUp(Border, 4), TileBuffer.Stride);
		break;
	}
	case PF_ASTC_8x8:
	{
		ASTCBlock AstcPixel = MakeASTC(Color, false);
		Patch((ASTCBlock *)TileBuffer.Memory, AstcPixel, FMath::DivideAndRoundUp(Width, 8), FMath::DivideAndRoundUp(Height, 8), FMath::DivideAndRoundUp(Border, 8), TileBuffer.Stride);
		break;
	}
	default:
		// Not really an error... we just don't draw debug tiles then...
		break;
	}
}

bool UniformColorPixels(const FVTUploadTileBuffer& TileBuffer, int32 Width, int32 Height, EPixelFormat Format, const uint8 *Color)
{
	switch (Format)
	{
	case PF_B8G8R8A8:
	{
		uint32_t Pixel = MakeBGRA(Color);
		PatchBlock((uint32_t *)TileBuffer.Memory, Pixel, Width, Height, TileBuffer.Stride);
		break;
	}
	case PF_R8G8B8A8:
	case PF_R8G8B8A8_SNORM:
	case PF_R8G8B8A8_UINT:
	{
		uint32_t Pixel = MakeRGBA(Color);
		PatchBlock((uint32_t *)TileBuffer.Memory, Pixel, Width, Height, TileBuffer.Stride);
		break;
	}
	case PF_A32B32G32R32F:
	{
		FloatPixel Fpixel = MakeFloat(Color);
		PatchBlock((FloatPixel *)TileBuffer.Memory, Fpixel, Width, Height, TileBuffer.Stride);
		break;
	}
	case PF_A16B16G16R16:
	{
		Float16Pixel Fpixel = MakeFloat16(Color);
		PatchBlock((Float16Pixel *)TileBuffer.Memory, Fpixel, Width, Height, TileBuffer.Stride);
		break;
	}
	case PF_DXT1:
	{
		DXT1Block DXT1pixel = MakeDXT1(Color);
		PatchBlock((DXT1Block *)TileBuffer.Memory, DXT1pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_DXT5:
	{
		DXT5Block DXT5pixel = MakeDXT5(Color);
		PatchBlock((DXT5Block *)TileBuffer.Memory, DXT5pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC4:
	{
		BC4Block BC4pixel = MakeBC4(Color);
		PatchBlock((BC4Block *)TileBuffer.Memory, BC4pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC5:
	{
		BC5Block BC5pixel = MakeBC5(Color);
		PatchBlock((BC5Block *)TileBuffer.Memory, BC5pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC6H:
	{
		BC6Block BC6pixel = MakeBC6(Color);
		PatchBlock((BC6Block *)TileBuffer.Memory, BC6pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_BC7:
	{
		BC7Block BC7pixel = MakeBC7(Color);
		PatchBlock((BC7Block *)TileBuffer.Memory, BC7pixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_ASTC_4x4:
	{
		ASTCBlock AstcPixel = MakeASTC(Color, false);
		PatchBlock((ASTCBlock *)TileBuffer.Memory, AstcPixel, FMath::DivideAndRoundUp(Width, 4), FMath::DivideAndRoundUp(Height, 4), TileBuffer.Stride);
		break;
	}
	case PF_ASTC_8x8:
	{
		ASTCBlock AstcPixel = MakeASTC(Color, false);
		PatchBlock((ASTCBlock *)TileBuffer.Memory, AstcPixel, FMath::DivideAndRoundUp(Width, 8), FMath::DivideAndRoundUp(Height, 8), TileBuffer.Stride);
		break;
	}
	default:
		return false;
	}

	return true;
}