// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"

struct FVTUploadTileBuffer;

/*
 * Bake a colored border of width pixels around the specified image.
 * If the pixel format is block compressed the width, height and borders will be rounded up to the nearest block size
 * 
 * If baking borders is not supported for this pixel format no borders will be baked.
 */
void BakeDebugInfo(const FVTUploadTileBuffer& TileBuffer, int32 Width, int32 Height, int32 Border, EPixelFormat Format, int32 MipLevel);

/*
* Fill a block with uniformely colored data. 
* If the pixel format is block compressed the width, height will be rounded up to the nearest block size
* - If the pixel format is compressed some loss may occur if the color can't be exactly represented
* - If the pixel format is floating point the color will be scaled by 1/255
* Not all formats may be supported we will return false in this case and fill the block with zeros
*/
bool UniformColorPixels(const FVTUploadTileBuffer& TileBuffer, int32 Width, int32 Height, EPixelFormat Format, const uint8 *Color);