// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/LightmapVirtualTexture.h"
#include "Async/TaskGraphInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightmapVirtualTexture)

ULightMapVirtualTexture2D::ULightMapVirtualTexture2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VirtualTextureStreaming = true;
	SetLayerForType(ELightMapVirtualTextureType::LightmapLayer0, 0u);
	SetLayerForType(ELightMapVirtualTextureType::LightmapLayer1, 1u);
}

void ULightMapVirtualTexture2D::SetLayerForType(ELightMapVirtualTextureType InType, uint8 InLayer)
{
	const int TypeIndex = (int)InType;
	while (TypeIndex >= TypeToLayer.Num())
	{
		TypeToLayer.Add(-1);
	}
	TypeToLayer[TypeIndex] = InLayer;
}

uint32 ULightMapVirtualTexture2D::GetLayerForType(ELightMapVirtualTextureType InType) const
{
	const int TypeIndex = (int)InType;
	return (TypeIndex >= TypeToLayer.Num()) ? ~0u : (uint32)TypeToLayer[TypeIndex];
}

