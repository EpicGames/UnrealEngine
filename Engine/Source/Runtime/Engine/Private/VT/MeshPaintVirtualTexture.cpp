// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/MeshPaintVirtualTexture.h"

#include "ComponentRecreateRenderStateContext.h"
#include "Components/PrimitiveComponent.h"
#include "EngineModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "RendererInterface.h"
#include "RenderUtils.h"
#include "ShaderPlatformCachedIniValue.h"
#include "TextureResource.h"
#include "VirtualTexturing.h"
#include "VT/VirtualTextureBuildSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPaintVirtualTexture)

static TAutoConsoleVariable<bool> CVarMeshPaintVirtualTextureSupport(
	TEXT("r.MeshPaintVirtualTexture.Support"),
	true,
	TEXT("Build time support mesh painting with virtual textures"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarMeshPaintVirtualTextureEnable(
	TEXT("r.MeshPaintVirtualTexture.Enable"),
	true,
	TEXT("Run time enable mesh painting with virtual textures"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarMeshPaintVirtualTextureUseCompression(
	TEXT("r.MeshPaintVirtualTexture.UseCompression"),
	true,
	TEXT("Whether mesh paint textures use a compressed format"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMeshPaintVirtualTextureTileSize(
	TEXT("r.MeshPaintVirtualTexture.TileSize"),
	32,
	TEXT("Virtual texture tile size for mesh paint textures"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMeshPaintVirtualTextureTileBorderSize(
	TEXT("r.MeshPaintVirtualTexture.TileBorderSize"),
	2,
	TEXT("Virtual texture tile border size for mesh paint textures"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMeshPaintVirtualTextureTexelsPerVertex(
	TEXT("r.MeshPaintVirtualTexture.DefaultTexelsPerVertex"),
	4,
	TEXT("Default ratio of texels to vertices when creating a texture for a mesh"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMeshPaintVirtualTextureMaxTextureSize(
	TEXT("r.MeshPaintVirtualTexture.MaxTextureSize"),
	4096,
	TEXT("Maximum allowed size for mesh paint textures"),
	ECVF_Default);

namespace MeshPaintVirtualTexture
{
	bool IsSupported(EShaderPlatform InShaderPlatform)
	{
		static FShaderPlatformCachedIniValue<bool> CPlatformVarMeshPaintVirtualTextureSupport(CVarMeshPaintVirtualTextureSupport.AsVariable());
		return CPlatformVarMeshPaintVirtualTextureSupport.Get(InShaderPlatform) && UseVirtualTexturing(InShaderPlatform);
	}

	bool IsSupported(ITargetPlatform const* InTargetPlatform)
	{
		if (InTargetPlatform != nullptr)
		{
			TArray<FName> DesiredShaderFormats;
			InTargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);
			for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
			{
				const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
				if (IsSupported(ShaderPlatform))
				{
					return true;
				}
			}
		}
		return false;
	}

	static bool IsEnabled()
	{
		return CVarMeshPaintVirtualTextureEnable.GetValueOnAnyThread();
	}

	static bool UseCompressedTextureFormat()
	{
		return CVarMeshPaintVirtualTextureUseCompression.GetValueOnAnyThread();
	}

	static uint32 GetTileSize()
	{
		return FVirtualTextureBuildSettings::ClampAndAlignTileSize(CVarMeshPaintVirtualTextureTileSize.GetValueOnAnyThread());
	}

	static uint32 GetTileBorderSize()
	{
		return FVirtualTextureBuildSettings::ClampAndAlignTileBorderSize(CVarMeshPaintVirtualTextureTileBorderSize.GetValueOnAnyThread());
	}

	static uint32 GetMaximumTextureSize()
	{
		// Maximum texture size needs to be at least one tile and power of two aligned.
		return FMath::RoundUpToPowerOfTwo(FMath::Max((int32)GetTileSize(), CVarMeshPaintVirtualTextureMaxTextureSize.GetValueOnGameThread()));
	}

	uint32 GetAlignedTextureSize(int32 InSize)
	{
		// Must be power of 2 aligned.
		const uint32 SizePow2Aligned = FMath::RoundUpToPowerOfTwo((uint32)FMath::Max(InSize, 0));
		// Must be at least the size of a tile and clamped to the maximum (both of which are also power of two aligned).
		return FMath::Clamp(SizePow2Aligned, GetTileSize(), GetMaximumTextureSize());
	}

	uint32 GetDefaultTextureSize(int32 InNumVertices)
	{
		const int32 NumTexels = FMath::Max(InNumVertices * CVarMeshPaintVirtualTextureTexelsPerVertex.GetValueOnGameThread(), 0);
		const int32 TextureSize = FMath::CeilToInt(FMath::Sqrt((float)NumTexels));
		return GetAlignedTextureSize(TextureSize);
	}

	/** 
	 * Fill out the scene uniforms from an allocated VT. 
	 * We expect the result to be constant for all allocated VTs (so that they can share one uniform buffer).
	 * Note that there are valid cases when it will change over time (but we always use the latest).
	 * For example, when the virtual texture pools are resized, all VTs reallocate and change to a new value here.
	 */
	static void GetSceneUniformParams(IAllocatedVirtualTexture* InAllocatedVT, FUniformParams& OutParams)
	{
		OutParams.PageTableTexture = InAllocatedVT->GetPageTableTexture(0);
		OutParams.PhysicalTexture = InAllocatedVT->GetPhysicalTexture(0);

		// Packed uniform layout should match shader unpacking in VTUniform_Unpack().
		// This packing code is copied from FAllocatedVirtualTexture::GetPackedUniform()

		const uint32 vPageSize = InAllocatedVT->GetVirtualTileSize();
		const uint32 PageBorderSize = InAllocatedVT->GetTileBorderSize();
		const float RcpPhysicalTextureSize = 1.0f / float(InAllocatedVT->GetPhysicalTextureSize(0));
		const uint32 pPageSize = vPageSize + PageBorderSize * 2u;

		OutParams.PackedUniform.X = GetDefaultFallbackColor();
		OutParams.PackedUniform.Y = FMath::AsUInt((float)vPageSize * RcpPhysicalTextureSize);
		OutParams.PackedUniform.Z = FMath::AsUInt((float)PageBorderSize * RcpPhysicalTextureSize);
		
		OutParams.PackedUniform.W = FMath::AsUInt((float)pPageSize * RcpPhysicalTextureSize);
	}

	/** A global set to track allocated mesh paint virtual textures. */
	TSet<IAllocatedVirtualTexture*> AllocatedVTs;
	/** The global scene uniform params cached from the last allocated VT. */
	FUniformParams Params;
	/** Mutex for the global state. The state is all accessed only on the render thread timeline, but the Params are accessed from render worker threads. */
	UE::FRecursiveMutex Mutex;

	/** Add an allocated VT to our global allocated set. */
	static void AddAllocatedVT(IAllocatedVirtualTexture* InAllocatedVT)
	{
		UE::TScopeLock Lock(Mutex);
		
		bool bAlreadyInSet = false;
		AllocatedVTs.Add(InAllocatedVT, &bAlreadyInSet);
		// This may fire in future if we allow components to share the same virtual texture.
		// If that happens we could change to store in a map against a ref count.
		ensure(!bAlreadyInSet);

		// Update the cached uniform params.
		GetSceneUniformParams(InAllocatedVT, Params);
	}

	/** Remove an allocated VT from our global allocated set. */
	static void RemoveAllocatedVT(const FVirtualTextureProducerHandle& InHandle, void* InBaton)
	{
		UE::TScopeLock Lock(Mutex);

		const int32 Removed = AllocatedVTs.Remove((IAllocatedVirtualTexture*)InBaton);
		ensure(Removed == 1);

		// Clear the cached uniform params if we have removed all the allocated VTs.
		if (AllocatedVTs.Num() == 0)
		{
			Params = {};
		}
	}

	/** 
	 * Call this on texture resource creation.
	 * This will acquire the virtual texture and store to our global set.
	 */
	static IAllocatedVirtualTexture* AcquireAllocatedVT_RenderThread(FVirtualTexture2DResource* VTResource)
	{
		IAllocatedVirtualTexture* AllocatedVT = VTResource->GetAllocatedVT();
		if (AllocatedVT == nullptr)
		{
			AllocatedVT = VTResource->AcquireAllocatedVT();
			if (ensure(AllocatedVT))
			{
				// Add virtual texture to the global set.
				AddAllocatedVT(AllocatedVT);
				// Queue on-destruction callback for removal from the global set.
				GetRendererModule().AddVirtualTextureProducerDestroyedCallback(AllocatedVT->GetProducerHandle(0), &RemoveAllocatedVT, AllocatedVT);
			}
		}

		return AllocatedVT;
	}

	static void AcquireAllocatedVT(FTextureResource* Resource)
	{
		FVirtualTexture2DResource* VTResource = Resource != nullptr ? Resource->GetVirtualTexture2DResource() : nullptr;
		if (VTResource)
		{
			ENQUEUE_RENDER_COMMAND(AcquireVT)([VTResource](FRHICommandListImmediate& RHICmdList)
			{
				AcquireAllocatedVT_RenderThread(VTResource);
			});
		}	
	}

	FUniformParams GetUniformParams()
	{
		if (MeshPaintVirtualTexture::IsEnabled())
		{
			UE::TScopeLock Lock(Mutex);
			return Params;
		}
		return {};
	}

	FUintVector2 GetTextureDescriptor(FTextureResource* InTextureResource, uint32 InOptionalCoordinateIndex)
	{
		FUintVector2 Descriptor(0, 0);

		if (InTextureResource == nullptr)
		{
			return Descriptor;
		}

		if (!MeshPaintVirtualTexture::IsEnabled())
		{
			return Descriptor;
		}
		
		FVirtualTexture2DResource* VTTextureResource = InTextureResource->GetVirtualTexture2DResource();
		if (VTTextureResource == nullptr)
		{
			return Descriptor;
		}

		IAllocatedVirtualTexture* AllocatedVT = AcquireAllocatedVT_RenderThread(VTTextureResource);
		if (AllocatedVT == nullptr)
		{
			return Descriptor;
		}

		const uint32 vPageX = AllocatedVT->GetVirtualPageX();
		const uint32 vPageY = AllocatedVT->GetVirtualPageY();
		const uint32 WidthInPages = AllocatedVT->GetWidthInTiles();
		const uint32 HeightInPages = AllocatedVT->GetHeightInTiles();
		const uint32 vPageTableMipBias = FMath::FloorLog2(AllocatedVT->GetVirtualTileSize());
		const uint32 SpaceID = AllocatedVT->GetSpaceID();
		const uint32 MaxLevel = AllocatedVT->GetMaxLevel();
		const uint32 PageTable32Bit = AllocatedVT->GetPageTableFormat() == EVTPageTableFormat::UInt32 ? 1 : 0;

		// Descriptor layout should match shader unpacking in VTPageTableUniform_Unpack().
		// We optionally pack a coordinate index which is used for debug views.
		Descriptor.X = vPageX | (vPageY << 12) | (vPageTableMipBias << 24) | (SpaceID << 28);
		Descriptor.Y = WidthInPages | (HeightInPages << 12) | (MaxLevel << 24) | PageTable32Bit << 28 | (InOptionalCoordinateIndex << 30);
		
		return Descriptor;
	}
}


UMeshPaintVirtualTexture::UMeshPaintVirtualTexture(const FObjectInitializer& ObjectInitializer)
	: UTexture2D(ObjectInitializer)
{
	VirtualTextureStreaming = true;

	// Mesh paint textures usually don't tile. We should default to clamping borders during texture build.
	AddressX = TA_Clamp;
	AddressY = TA_Clamp;

#if WITH_EDITORONLY_DATA
	CompressionNone = !MeshPaintVirtualTexture::UseCompressedTextureFormat();

	// Force alpha channel so that we the platform format is consistent for all content.
	CompressionForceAlpha = true;
#endif
}

void UMeshPaintVirtualTexture::GetVirtualTextureBuildSettings(FVirtualTextureBuildSettings& OutSettings) const
{
	// Use the specific tile size for mesh painting textures.
	// This is typically different from the default virtual texture tile size. That's good because it keeps the mesh painting virtual texture pools isolated.
	// It's also better to have a small tile size since texture painting is typically low frequency and needs relatively small textures. Small tiles also give improved mipping and less wastage at distance.
	OutSettings.TileSize = MeshPaintVirtualTexture::GetTileSize();
	OutSettings.TileBorderSize = MeshPaintVirtualTexture::GetTileBorderSize();
}

void UMeshPaintVirtualTexture::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
	Super::UpdateResourceWithParams(InFlags);

	// We get here on virtual texture pool recreation, and on texture compilation in editor.
	// In those cases we need to reacquire the virtual texture, and notify our component.
	MeshPaintVirtualTexture::AcquireAllocatedVT(GetResource());

	// We assume a 1-1 mapping of component and texture here.
	// If in future we want to share a painted texture across components then we will need a way to track the set of components to dirty.
	if (UPrimitiveComponent* PrimitiveComponent = OwningComponent.Get())
	{
		PrimitiveComponent->MarkRenderStateDirty();
	}

	// Notify that we are done updating resource
	OnResourceUpdated.Broadcast();
}

#if WITH_EDITOR

void UMeshPaintVirtualTexture::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	// Even though we skip the cook of this object for non VT platforms in URuntimeVirtualTexture::Serialize()
	// we still load the object at cook time and kick off the DDC build. This will trigger an error in the texture DDC code.
	// Either we need to make the DDC code more robust for non VT platforms or we can skip the process here...
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return;
	}

	Super::BeginCacheForCookedPlatformData(TargetPlatform);
}

bool UMeshPaintVirtualTexture::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return true;
	}

	return Super::IsCachedCookedPlatformDataLoaded(TargetPlatform);
}

void UMeshPaintVirtualTexture::ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return;
	}

	Super::ClearCachedCookedPlatformData(TargetPlatform);
}

#endif
