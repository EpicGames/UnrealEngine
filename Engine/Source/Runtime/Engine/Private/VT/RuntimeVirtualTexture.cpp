// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTexture.h"

#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "EngineModule.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "Shader/ShaderTypes.h"
#include "ShaderPlatformCachedIniValue.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"
#include "VT/RuntimeVirtualTextureNotify.h"
#include "VT/UploadingVirtualTexture.h"
#include "VT/VirtualTexture.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VT/VirtualTextureLevelRedirector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeVirtualTexture)

static TAutoConsoleVariable<bool> CVarRVTEnableBaseColor(TEXT("r.VT.RVT.EnableBaseColor"), true, TEXT("Enable 'Base Color' RVT Material Type"), ECVF_ReadOnly);
static TAutoConsoleVariable<bool> CVarRVTEnableBaseColorRoughness(TEXT("r.VT.RVT.EnableBaseColorRoughness"), true, TEXT("Enable 'Base Color, Normal, Roughness' RVT Material Type"), ECVF_ReadOnly);
static TAutoConsoleVariable<bool> CVarRVTEnableBaseColorSpecular(TEXT("r.VT.RVT.EnableBaseColorSpecular"), true, TEXT("Enable 'Base Color, Normal, Roughness, Specular' family of RVT Material Types"), ECVF_ReadOnly);
static TAutoConsoleVariable<bool> CVarRVTEnableMask4(TEXT("r.VT.RVT.EnableMask4"), true, TEXT("Enable 'Mask4' RVT Material Type"), ECVF_ReadOnly);
static TAutoConsoleVariable<bool> CVarRVTEnableWorldHeight(TEXT("r.VT.RVT.EnableWorldHeight"), true, TEXT("Enable 'World Height' RVT Material Type"), ECVF_ReadOnly);
static TAutoConsoleVariable<bool> CVarRVTEnableDisplacement(TEXT("r.VT.RVT.EnableDisplacement"), true, TEXT("Enable 'Displacement' RVT Material Type"), ECVF_ReadOnly);

namespace
{
	/** Null producer to use as placeholder when no producer has been set on a URuntimeVirtualTexture */
	class FNullVirtualTextureProducer : public IVirtualTexture
	{
	public:
		/** Get a producer description to use with the null producer. */
		static void GetNullProducerDescription(FVTProducerDescription& OutDesc)
		{
			OutDesc.Dimensions = 2;
			OutDesc.TileSize = 4;
			OutDesc.TileBorderSize = 0;
			OutDesc.BlockWidthInTiles = 1;
			OutDesc.BlockHeightInTiles = 1;
			OutDesc.MaxLevel = 1;
			OutDesc.DepthInTiles = 1;
			OutDesc.WidthInBlocks = 1;
			OutDesc.HeightInBlocks = 1;
			OutDesc.NumTextureLayers = 0;
			OutDesc.NumPhysicalGroups = 0;
		}

		//~ Begin IVirtualTexture Interface.
		virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
		{
			return false;
		}

		virtual FVTRequestPageResult RequestPageData(
			FRHICommandListBase& RHICmdList,
			const FVirtualTextureProducerHandle& ProducerHandle,
			uint8 LayerMask,
			uint8 vLevel,
			uint64 vAddress,
			EVTRequestPagePriority Priority
		) override
		{
			return FVTRequestPageResult();
		}

		virtual IVirtualTextureFinalizer* ProducePageData(
			FRHICommandListBase& RHICmdList,
			ERHIFeatureLevel::Type FeatureLevel,
			EVTProducePageFlags Flags,
			const FVirtualTextureProducerHandle& ProducerHandle,
			uint8 LayerMask,
			uint8 vLevel,
			uint64 vAddress,
			uint64 RequestHandle,
			const FVTProduceTargetLayer* TargetLayers
		) override 
		{
			return nullptr; 
		}
		//~ End IVirtualTexture Interface.
	};
}


/**
 * Helpers for setting up adaptive virtual texture PageTable and AdaptiveGrid sizes.
 * We could expose these values directly to the user but it would require detailed knowledge of the underlying system to choose good settings.
 */
namespace
{
	/* First TileCount at which we start using adaptive virtual texture. */
	static const int32 GMinAdaptiveSize = 11;
	/* Maximum TileCount for adaptive virtual texture. */
	static const int32 GMaxAdaptiveSize = 20;

	/** Convert TileCount to PageTable size. This setup could be tweaked in future if necessary. */
	static int32 GetPageTableTileCountLog2(int32 TileCountLog2)
	{
		check(TileCountLog2 <= GMaxAdaptiveSize);
		if (TileCountLog2 < GMinAdaptiveSize) return TileCountLog2;

		if (TileCountLog2 <= 14) return 10;
		if (TileCountLog2 <= 17) return 11;
		return 12;
	}

	/** Convert TileCount to Adaptive Grid size. This setup could be tweaked in future if necessary. */
	static int32 GetAdaptiveGridSizeLog2(int32 TileCountLog2)
	{
		const int32 AdaptiveLevels = TileCountLog2 - GetPageTableTileCountLog2(TileCountLog2);
		const int32 BaseAdaptiveLevels = 2; // Base value ensures that each allocated virtual texture takes a maximum of 1/16 of the full adaptive page table.
		return AdaptiveLevels + BaseAdaptiveLevels;
	}
};


/** 
 * Container for render thread resources created for a URuntimeVirtualTexture object. 
 * Any access to the resources should be on the render thread only so that access is serialized with the Init()/Release() render thread tasks.
 */
class FRuntimeVirtualTextureRenderResource
{
public:
	FRuntimeVirtualTextureRenderResource()
		: AllocatedVirtualTexture(nullptr)
		, AdaptiveVirtualTexture(nullptr)
	{
	}

	/** Getter for the virtual texture producer. */
	FVirtualTextureProducerHandle GetProducerHandle() const
	{
		return ProducerHandle;
	}

	/** Getter for the virtual texture allocation. */
	IAllocatedVirtualTexture* GetAllocatedVirtualTexture() const 
	{
		checkSlow(IsInParallelRenderingThread());
		return AdaptiveVirtualTexture != nullptr ? AdaptiveVirtualTexture->GetAllocatedVirtualTexture() : AllocatedVirtualTexture;
	}

	/** Structure for passing data to Init() that isn't in the producer description. */
	struct FResourceInitDesc
	{
		bool bSinglePhysicalSpace = false;
		bool bPrivateSpace = false;
		bool bAdaptive = false;
	};

	/** Queues up render thread work to create resources and also releases any old resources. */
	void Init(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc, FResourceInitDesc const& InInitDesc)
	{
		FRuntimeVirtualTextureRenderResource* Resource = this;
		
		ENQUEUE_RENDER_COMMAND(FRuntimeVirtualTextureRenderResource_Init)(
			[Resource, InProducer, InProducerDesc, InInitDesc](FRHICommandList& RHICmdList)
		{
			FVirtualTextureProducerHandle OldProducerHandle = Resource->ProducerHandle;
			ReleaseVirtualTexture(Resource->AllocatedVirtualTexture);
			ReleaseVirtualTexture(Resource->AdaptiveVirtualTexture);
			Resource->ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(InProducerDesc, InProducer);
			
			FAllocatedVTDescription AllocatedVTDesc;
			FAdaptiveVTDescription AdaptiveVTDesc;
			const bool bAdaptive = FillVTDescriptions(Resource->ProducerHandle, InProducerDesc, InInitDesc, AllocatedVTDesc, AdaptiveVTDesc);

			// Only one or none of these should be allocated...
			Resource->AllocatedVirtualTexture = !bAdaptive ? AllocateVirtualTexture(RHICmdList, AllocatedVTDesc) : nullptr;
			Resource->AdaptiveVirtualTexture = bAdaptive ? AllocateAdaptiveVirtualTexture(RHICmdList, AllocatedVTDesc, AdaptiveVTDesc) : nullptr;

			// Release old producer after new one is created so that any destroy callbacks can access the new producer
			GetRendererModule().ReleaseVirtualTextureProducer(OldProducerHandle);
		});
	}

	/** Queues up render thread work to release resources. */
	void Release()
	{
		FVirtualTextureProducerHandle ProducerHandleToRelease = ProducerHandle;
		ProducerHandle = FVirtualTextureProducerHandle();
		IAdaptiveVirtualTexture* AdaptiveVirtualTextureToRelease = AdaptiveVirtualTexture;
		AdaptiveVirtualTexture = nullptr;
		IAllocatedVirtualTexture* AllocatedVirtualTextureToRelease = AllocatedVirtualTexture;
		AllocatedVirtualTexture = nullptr;

		ENQUEUE_RENDER_COMMAND(FRuntimeVirtualTextureRenderResource_Release)(
			[ProducerHandleToRelease, AdaptiveVirtualTextureToRelease, AllocatedVirtualTextureToRelease](FRHICommandList& RHICmdList)
		{
			ReleaseVirtualTexture(AllocatedVirtualTextureToRelease);
			ReleaseVirtualTexture(AdaptiveVirtualTextureToRelease);
			GetRendererModule().ReleaseVirtualTextureProducer(ProducerHandleToRelease);
		});
	}

protected:
	/** Fill the allocated VT descriptions from the producer description. Returns true if this should be allocated as an adaptive virtual texture. */
	static bool FillVTDescriptions(
		FVirtualTextureProducerHandle const& InProducerHandle, FVTProducerDescription const& InProducerDesc, FResourceInitDesc const& InInitDesc, 
		FAllocatedVTDescription& OutAllocatedVTDesc, FAdaptiveVTDescription& OutAdaptiveVTDesc)
	{
		const int32 MinTileCountLog2 = FMath::CeilLogTwo(FMath::Min(InProducerDesc.BlockWidthInTiles, InProducerDesc.BlockHeightInTiles));
		const int32 MaxTileCountLog2 = FMath::CeilLogTwo(FMath::Max(InProducerDesc.BlockWidthInTiles, InProducerDesc.BlockHeightInTiles));
		const bool bAdaptive = InInitDesc.bAdaptive && MaxTileCountLog2 >= GMinAdaptiveSize;
		const int32 PageTableSizeLog2 = bAdaptive ? GetPageTableTileCountLog2(MaxTileCountLog2) : MaxTileCountLog2;
		const int32 AdaptiveGridSizeLog2 = bAdaptive ? GetAdaptiveGridSizeLog2(MaxTileCountLog2) : 0;
		const int32 MaxPrivateSpaceSize = FMath::Clamp<int32>(1 << PageTableSizeLog2, VIRTUALTEXTURE_MIN_PAGETABLE_SIZE, VIRTUALTEXTURE_MAX_PAGETABLE_SIZE);
		const bool bPrivateSpace = bAdaptive || InInitDesc.bPrivateSpace;

		OutAllocatedVTDesc.Dimensions = InProducerDesc.Dimensions;
		OutAllocatedVTDesc.TileSize = InProducerDesc.TileSize;
		OutAllocatedVTDesc.TileBorderSize = InProducerDesc.TileBorderSize;
		OutAllocatedVTDesc.NumTextureLayers = InProducerDesc.NumTextureLayers;
		OutAllocatedVTDesc.bPrivateSpace = bPrivateSpace;
		OutAllocatedVTDesc.bShareDuplicateLayers = InInitDesc.bSinglePhysicalSpace;
		OutAllocatedVTDesc.MaxSpaceSize = bPrivateSpace ? MaxPrivateSpaceSize : 0;

		for (uint32 LayerIndex = 0u; LayerIndex < InProducerDesc.NumTextureLayers; ++LayerIndex)
		{
			OutAllocatedVTDesc.ProducerHandle[LayerIndex] = InProducerHandle;
			OutAllocatedVTDesc.ProducerLayerIndex[LayerIndex] = LayerIndex;
		}

		OutAdaptiveVTDesc.TileCountX = InProducerDesc.BlockWidthInTiles;
		OutAdaptiveVTDesc.TileCountY = InProducerDesc.BlockHeightInTiles;
		OutAdaptiveVTDesc.MaxAdaptiveLevel = FMath::Max(MaxTileCountLog2 - AdaptiveGridSizeLog2, 0);

		return bAdaptive;
	}

	/** Allocate in the virtual texture system. */
	static IAllocatedVirtualTexture* AllocateVirtualTexture(FRHICommandListBase& RHICmdList, FAllocatedVTDescription const& InAllocatedVTDesc)
	{
		// Check for NumLayers avoids allocating for the null producer
		IAllocatedVirtualTexture* OutAllocatedVirtualTexture = nullptr;
		if (InAllocatedVTDesc.NumTextureLayers > 0)
		{ 
			OutAllocatedVirtualTexture = GetRendererModule().AllocateVirtualTexture(RHICmdList, InAllocatedVTDesc);
		}
		return OutAllocatedVirtualTexture;
	}

	/** Release our virtual texture allocation. */
	static void ReleaseVirtualTexture(IAllocatedVirtualTexture* InAllocatedVirtualTexture)
	{
		if (InAllocatedVirtualTexture != nullptr)
		{
			GetRendererModule().DestroyVirtualTexture(InAllocatedVirtualTexture);
		}
	}

	/** Allocate an adaptive virtual texture in the virtual texture system. */
	static IAdaptiveVirtualTexture* AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, FAllocatedVTDescription const& InAllocatedVTDesc, FAdaptiveVTDescription const& InAdaptiveVTDesc)
	{
		// Check for NumLayers avoids allocating for the null producer
		IAdaptiveVirtualTexture* OutAdaptiveVirtualTexture = nullptr;
		if (InAllocatedVTDesc.NumTextureLayers > 0)
		{
			OutAdaptiveVirtualTexture = GetRendererModule().AllocateAdaptiveVirtualTexture(RHICmdList, InAdaptiveVTDesc, InAllocatedVTDesc);
		}
		return OutAdaptiveVirtualTexture;
	}

	/** Release our adaptive virtual texture allocation. */
	static void ReleaseVirtualTexture(IAdaptiveVirtualTexture* InAdaptiveVirtualTexture)
	{
		if (InAdaptiveVirtualTexture != nullptr)
		{
			GetRendererModule().DestroyAdaptiveVirtualTexture(InAdaptiveVirtualTexture);
		}
	}

private:
	FVirtualTextureProducerHandle ProducerHandle;
	IAllocatedVirtualTexture* AllocatedVirtualTexture;
	IAdaptiveVirtualTexture* AdaptiveVirtualTexture;
};


URuntimeVirtualTexture::URuntimeVirtualTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CustomMaterialData(FVector4f::Zero())
{
	// Initialize the RHI resources with a null producer
	if (!HasAnyFlags(RF_ClassDefaultObject) && FApp::CanEverRender())
	{
		Resource = new FRuntimeVirtualTextureRenderResource;
		InitNullResource();
	}
}

URuntimeVirtualTexture::~URuntimeVirtualTexture()
{
	if (Resource)
	{
		Resource->Release();
		delete Resource;
	}
}

int32 URuntimeVirtualTexture::GetMaxTileCountLog2(bool InAdaptive) 
{
	return InAdaptive ? GMaxAdaptiveSize : VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE;
}

int32 URuntimeVirtualTexture::GetPageTableSize() const
{
	const int32 ClampedTileCountLog2 = FMath::Clamp(TileCount, 0, GetMaxTileCountLog2(bAdaptive));
	const int32 PageTableSizeLog2 = bAdaptive ? GetPageTableTileCountLog2(ClampedTileCountLog2) : ClampedTileCountLog2;
	return 1 << PageTableSizeLog2;
}

void URuntimeVirtualTexture::GetProducerDescription(FVTProducerDescription& OutDesc, FInitSettings const& InitSettings, FTransform const& VolumeToWorld) const
{
	OutDesc.Name = GetFName();
	OutDesc.FullNameHash = GetTypeHash(GetName().ToLower());
	OutDesc.Dimensions = 2;
	OutDesc.DepthInTiles = 1;
	OutDesc.WidthInBlocks = 1;
	OutDesc.HeightInBlocks = 1;

	// Apply LODGroup TileSize bias here.
	const int32 TileSizeBias = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetTextureLODGroup(LODGroup).VirtualTextureTileSizeBias;
	OutDesc.TileSize = GetClampedTileSize(TileSize + TileSizeBias);

	OutDesc.TileBorderSize = GetTileBorderSize();

	// Apply TileCount bias here.
	const int32 TileCountBiasFromLodGroup = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetTextureLODGroup(LODGroup).VirtualTextureTileCountBias;
	const int32 MaxSizeInTiles = GetClampedTileCount(TileCount + TileCountBiasFromLodGroup + InitSettings.TileCountBias, bAdaptive);

	// Set width and height to best match the runtime virtual texture volume's aspect ratio.
	const FVector VolumeSize = VolumeToWorld.GetScale3D();
	const FVector::FReal VolumeSizeX = FMath::Max<FVector::FReal>(FMath::Abs(VolumeSize.X), 0.0001f);
	const FVector::FReal VolumeSizeY = FMath::Max<FVector::FReal>(FMath::Abs(VolumeSize.Y), 0.0001f);
	const float AspectRatioLog2 = FMath::Log2(VolumeSizeX / VolumeSizeY);

	uint32 WidthInTiles, HeightInTiles;
	if (AspectRatioLog2 >= 0.f)
	{
		WidthInTiles = MaxSizeInTiles;
		HeightInTiles = FMath::Max(WidthInTiles >> FMath::RoundToInt(AspectRatioLog2), 1u);
	}
	else
	{
		HeightInTiles = MaxSizeInTiles;
		WidthInTiles = FMath::Max(HeightInTiles >> FMath::RoundToInt(-AspectRatioLog2), 1u);
	}

	OutDesc.BlockWidthInTiles = WidthInTiles;
	OutDesc.BlockHeightInTiles = HeightInTiles;
	OutDesc.MaxLevel = FMath::Max((int32)FMath::CeilLogTwo(FMath::Max(OutDesc.BlockWidthInTiles, OutDesc.BlockHeightInTiles)) - GetRemoveLowMips(), 0);

	OutDesc.bContinuousUpdate = bContinuousUpdate;
	
	// Disable splitting physical pools to give 16bit page tables when r.VT.PageTableMode=2.
	// This is only useful for virtual texture spaces with many small allocated page tables, and that isn't typically the case for RVT.
	OutDesc.bRequiresSinglePhysicalPool = true;

	OutDesc.NumTextureLayers = GetLayerCount();
	OutDesc.NumPhysicalGroups = bSinglePhysicalSpace ? 1 : GetLayerCount();

	for (int32 Layer = 0; Layer < OutDesc.NumTextureLayers; Layer++)
	{
		OutDesc.LayerFormat[Layer] = GetLayerFormat(Layer);
		OutDesc.PhysicalGroupIndex[Layer] = bSinglePhysicalSpace ? 0 : Layer;
		OutDesc.bIsLayerSRGB[Layer] = IsLayerSRGB(Layer);
	}

	OutDesc.Priority = GetPriority();
}

int32 URuntimeVirtualTexture::GetLayerCount(ERuntimeVirtualTextureMaterialType InMaterialType)
{
	switch (InMaterialType)
	{
	case ERuntimeVirtualTextureMaterialType::BaseColor:
	case ERuntimeVirtualTextureMaterialType::Mask4:
	case ERuntimeVirtualTextureMaterialType::WorldHeight:
	case ERuntimeVirtualTextureMaterialType::Displacement:
		return 1;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
		return 2;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
		return 3;
	default:
		break;
	}

	// Implement logic for any missing material types
	check(false);
	return 1;
}

int32 URuntimeVirtualTexture::GetLayerCount() const
{
	return GetLayerCount(MaterialType);
}

static EPixelFormat PlatformCompressedRVTFormat(EPixelFormat Format)
{
	if (GPixelFormats[Format].Supported)
	{
		return Format;
	}
	else if (GPixelFormats[PF_ETC2_RGB].Supported)
	{
		switch(Format)
		{
		case PF_DXT1:
			Format = PF_ETC2_RGB;
			break;
		case PF_DXT5:
			Format = PF_ETC2_RGBA;
			break;
		case PF_BC4:
			Format = PF_ETC2_R11_EAC;
			break;
		case PF_BC5:
			Format = PF_ETC2_RG11_EAC;
			break;
		default:
			check(false);
		};
	}

	return Format;
}

static EPixelFormat PlatformLQCompressedFormat(bool bRequireAlpha)
{
	const EPixelFormat LQFormat = bRequireAlpha ? PF_B5G5R5A1_UNORM : PF_R5G6B5_UNORM;
	const EPixelFormat HQFormat = bRequireAlpha ? PF_DXT5 : PF_DXT1;
	bool bLQFormatSupported = GPixelFormats[PF_B5G5R5A1_UNORM].Supported && GPixelFormats[PF_R5G6B5_UNORM].Supported;
	return bLQFormatSupported? LQFormat : HQFormat;
}

EPixelFormat URuntimeVirtualTexture::GetLayerFormat(int32 LayerIndex) const
{
	if (LayerIndex == 0)
	{
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT1) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
			return bCompressTextures ? (bUseLowQualityCompression? PlatformLQCompressedFormat(false) : PlatformCompressedRVTFormat(PF_DXT1)) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT5) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::Mask4:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT5) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::WorldHeight:
			return PF_G16;
		case ERuntimeVirtualTextureMaterialType::Displacement:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_BC4) : PF_G16;
		default:
			break;
		}
	}
	else if (LayerIndex == 1)
	{
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
			return bCompressTextures ? (bUseLowQualityCompression ? PlatformLQCompressedFormat(false) : PlatformCompressedRVTFormat(PF_DXT5)) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT5) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_BC5) : PF_B8G8R8A8;
		default:
			break;
		}
	}
	else if (LayerIndex == 2)
	{
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT1) : PF_B8G8R8A8;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			return bCompressTextures ? PlatformCompressedRVTFormat(PF_DXT5) : PF_B8G8R8A8;
		default:
			break;
		}
	}

	// Implement logic for any missing material types
	check(false);
	return PF_B8G8R8A8;
}

bool URuntimeVirtualTexture::IsLayerSRGB(int32 LayerIndex) const
{
	switch (MaterialType)
	{
	case ERuntimeVirtualTextureMaterialType::BaseColor:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
		// Only BaseColor layer is sRGB.
		return LayerIndex == 0;
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
		// This format encodes/decodes sRGB on write/read.
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
		// These formats have YCoCg packing which can't use sRGB.
	case ERuntimeVirtualTextureMaterialType::Mask4:
	case ERuntimeVirtualTextureMaterialType::WorldHeight:
	case ERuntimeVirtualTextureMaterialType::Displacement:
		// These formats require linear encoding.
		return false;
	default:
		break;
	}

	// Implement logic for any missing material types
	check(false);
	return false;
}

bool URuntimeVirtualTexture::IsLayerYCoCg(int32 LayerIndex) const
{
	switch (MaterialType)
	{
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
		return LayerIndex == 0;
	default:
		break;
	}
	return false;
}

FVirtualTextureProducerHandle URuntimeVirtualTexture::GetProducerHandle() const
{
	return Resource ? Resource->GetProducerHandle() : FVirtualTextureProducerHandle();
}

IAllocatedVirtualTexture* URuntimeVirtualTexture::GetAllocatedVirtualTexture() const
{
	return Resource ? Resource->GetAllocatedVirtualTexture() : nullptr;
}

FVector4 URuntimeVirtualTexture::GetUniformParameter(int32 Index) const
{
	switch (Index)
	{
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0: return WorldToUVTransformParameters[0];
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1: return WorldToUVTransformParameters[1];
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2: return WorldToUVTransformParameters[2];
	case ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack: return WorldHeightUnpackParameter;
	default:
		break;
	}
	
	check(0);
	return FVector4(ForceInitToZero);
}

UE::Shader::EValueType URuntimeVirtualTexture::GetUniformParameterType(int32 Index)
{
	switch (Index)
	{
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform0: return UE::Shader::EValueType::Double3;
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform1: return UE::Shader::EValueType::Float3;
	case ERuntimeVirtualTextureShaderUniform_WorldToUVTransform2: return UE::Shader::EValueType::Float3;
	case ERuntimeVirtualTextureShaderUniform_WorldHeightUnpack: return UE::Shader::EValueType::Float2;
	default:
		break;
	}

	check(0);
	return UE::Shader::EValueType::Float4;
}

EVTProducerPriority URuntimeVirtualTexture::GetPriority() const
{
	return bUseCustomPriority ? CustomPriority : GetDefaultPriority();
}

EVTProducerPriority URuntimeVirtualTexture::GetDefaultPriority() const
{
	switch (MaterialType)
	{
	case ERuntimeVirtualTextureMaterialType::BaseColor:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
	case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
		return EVTProducerPriority::High;
	case ERuntimeVirtualTextureMaterialType::Mask4:
		return EVTProducerPriority::BelowNormal;
	case ERuntimeVirtualTextureMaterialType::WorldHeight:
		return EVTProducerPriority::Low;
	case ERuntimeVirtualTextureMaterialType::Displacement:
		return EVTProducerPriority::Lower;
	default:
		checkNoEntry();
	}

	return EVTProducerPriority::Normal;
}

void URuntimeVirtualTexture::Initialize(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc, FTransform const& InVolumeToWorld, FBox const& InWorldBounds, FOnReInitDelegate InDelegate)
{
	OnReInitDelegate.ExecuteIfBound();
	OnReInitDelegate = InDelegate;

	WorldToUVTransformParameters[0] = InVolumeToWorld.GetTranslation();
	WorldToUVTransformParameters[1] = InVolumeToWorld.GetUnitAxis(EAxis::X) * 1.f / InVolumeToWorld.GetScale3D().X;
	WorldToUVTransformParameters[2] = InVolumeToWorld.GetUnitAxis(EAxis::Y) * 1.f / InVolumeToWorld.GetScale3D().Y;

	const FVector::FReal HeightRange = FMath::Max<FVector::FReal>(InWorldBounds.Max.Z - InWorldBounds.Min.Z, 1.f);
	WorldHeightUnpackParameter = FVector4(HeightRange, InWorldBounds.Min.Z, 0.f, 0.f);

	InitResource(InProducer, InProducerDesc);
}

void URuntimeVirtualTexture::Initialize(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc, FTransform const& InVolumeToWorld, FBox const& InWorldBounds)
{
	Initialize(InProducer, InProducerDesc, InVolumeToWorld, InWorldBounds, {});
}

void URuntimeVirtualTexture::Release()
{
	OnReInitDelegate = {};

	InitNullResource();
}

void URuntimeVirtualTexture::InitResource(IVirtualTexture* InProducer, FVTProducerDescription const& InProducerDesc)
{
	FRuntimeVirtualTextureRenderResource::FResourceInitDesc InitDesc;
	InitDesc.bSinglePhysicalSpace = bSinglePhysicalSpace;
	InitDesc.bPrivateSpace = bPrivateSpace;
	InitDesc.bAdaptive = bAdaptive;

	Resource->Init(InProducer, InProducerDesc, InitDesc);
}

void URuntimeVirtualTexture::InitNullResource()
{
	if (FApp::CanEverRender() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		FNullVirtualTextureProducer* Producer = new FNullVirtualTextureProducer;
		FVTProducerDescription ProducerDesc;
		FNullVirtualTextureProducer::GetNullProducerDescription(ProducerDesc);
		FRuntimeVirtualTextureRenderResource::FResourceInitDesc InitDesc;
		Resource->Init(Producer, ProducerDesc, InitDesc);
	}
}

void URuntimeVirtualTexture::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void URuntimeVirtualTexture::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag("Size", FString::FromInt(GetSize()), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("TileCount", FString::FromInt(GetTileCount()), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("TileSize", FString::FromInt(GetTileSize()), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("TileBorderSize", FString::FromInt(GetTileBorderSize()), FAssetRegistryTag::TT_Numerical));
}

void URuntimeVirtualTexture::PostLoad()
{
	// Convert Size_DEPRECATED to TileCount
	if (Size_DEPRECATED >= 0)
	{
		int32 OldSize = 1 << FMath::Clamp(Size_DEPRECATED + 10, 10, 18);
		int32 TileCountFromSize = FMath::Max(OldSize / GetTileSize(), 1);
		TileCount = FMath::FloorLog2(TileCountFromSize);
		Size_DEPRECATED = -1;
	}

	// Remove StreamingTexture_DEPRECATED
	StreamingTexture_DEPRECATED = nullptr;

	Super::PostLoad();
}

#if WITH_EDITOR

void URuntimeVirtualTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Adjust TileCount when adaptive setting is toggled
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URuntimeVirtualTexture, bAdaptive))
	{
		TileCount = FMath::Min(TileCount, GetMaxTileCountLog2(bAdaptive));
	}

	RuntimeVirtualTexture::NotifyComponents(this);
	RuntimeVirtualTexture::NotifyPrimitives(this);
}

#endif

namespace RuntimeVirtualTexture
{
	IVirtualTexture* CreateStreamingTextureProducer(
		UVirtualTexture2D* InStreamingTexture,
		FVTProducerDescription const& InOwnerProducerDesc,
		FVTProducerDescription& OutStreamingProducerDesc)
	{
		OutStreamingProducerDesc = InOwnerProducerDesc;

		if(InStreamingTexture != nullptr)
		{
			OutStreamingProducerDesc.Name = InStreamingTexture->GetFName();

			FTexturePlatformData* StreamingTextureData = InStreamingTexture->GetPlatformData();
			if(ensure(StreamingTextureData != nullptr))
			{
				FVirtualTextureBuiltData* VTData = StreamingTextureData->VTData;
				if(ensure(VTData != nullptr))
				{
					ensure(InOwnerProducerDesc.TileSize == VTData->TileSize);
					ensure(InOwnerProducerDesc.TileBorderSize == VTData->TileBorderSize);
					ensure(VTData->GetNumMips() > 0);

					// Note that streaming data may have mips added/removed during cook.
					const uint32 BlockWidthInTiles = VTData->GetWidthInTiles();
					const uint32 BlockHeightInTiles = VTData->GetHeightInTiles();
					const uint32 NumLevels = FMath::CeilLogTwo(FMath::Max(BlockWidthInTiles, BlockHeightInTiles));
					const uint32 NumOwnerLevels = FMath::CeilLogTwo(FMath::Max(InOwnerProducerDesc.BlockWidthInTiles, InOwnerProducerDesc.BlockHeightInTiles));

					// Clamp the streaming texture size to the runtime virtual texture.
					const uint32 FirstMipToUse = NumLevels > NumOwnerLevels ? NumLevels - NumOwnerLevels : 0;

					OutStreamingProducerDesc.BlockWidthInTiles = BlockWidthInTiles >> FirstMipToUse;
					OutStreamingProducerDesc.BlockHeightInTiles = BlockHeightInTiles >> FirstMipToUse;
					OutStreamingProducerDesc.MaxLevel = NumLevels - FirstMipToUse;
					
					for (uint32 LayerIndex = 0u; LayerIndex < VTData->GetNumLayers(); ++LayerIndex)
					{
						OutStreamingProducerDesc.LayerFallbackColor[LayerIndex] = VTData->LayerFallbackColors[LayerIndex];
					}

					return new FUploadingVirtualTexture(InStreamingTexture->GetFName(), VTData, FirstMipToUse);
				}
			}
		}

		return nullptr;
	}

	IVirtualTexture* BindStreamingTextureProducer(
		IVirtualTexture* InProducer,
		IVirtualTexture* InStreamingProducer,
		int32 InTransitionLevel)
	{
		return (InStreamingProducer == nullptr) ? InProducer : new FVirtualTextureLevelRedirector(InProducer, InStreamingProducer, InTransitionLevel);
	}

	bool IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType InMaterialType, EShaderPlatform InPlatform)
	{
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableBaseColor(CVarRVTEnableBaseColor.AsVariable());
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableBaseColorRoughness(CVarRVTEnableBaseColorRoughness.AsVariable());
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableBaseColorSpecular(CVarRVTEnableBaseColorSpecular.AsVariable());
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableMask4(CVarRVTEnableMask4.AsVariable());
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableWorldHeight(CVarRVTEnableWorldHeight.AsVariable());
		static FShaderPlatformCachedIniValue<bool> CPlatformVarRVTEnableDisplacement(CVarRVTEnableDisplacement.AsVariable());

		switch (InMaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor:
			return CPlatformVarRVTEnableBaseColor.Get(InPlatform);
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
			return CPlatformVarRVTEnableBaseColorRoughness.Get(InPlatform);
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			return CPlatformVarRVTEnableBaseColorSpecular.Get(InPlatform);
		case ERuntimeVirtualTextureMaterialType::Mask4:
			return CPlatformVarRVTEnableMask4.Get(InPlatform);
		case ERuntimeVirtualTextureMaterialType::WorldHeight:
			return CPlatformVarRVTEnableWorldHeight.Get(InPlatform);
		case ERuntimeVirtualTextureMaterialType::Displacement:
			return CPlatformVarRVTEnableDisplacement.Get(InPlatform);
		default:
			checkNoEntry();
		}

		return false;
	}

	bool IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType MaterialType)
	{
		return IsMaterialTypeSupported(MaterialType, GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel));
	}
}

