// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTextureNotify.h"

#include "Components/PrimitiveComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "UObject/UObjectIterator.h"

namespace RuntimeVirtualTexture
{
#if WITH_EDITOR

	void NotifyComponents(URuntimeVirtualTexture const* VirtualTexture)
	{
		for (TObjectIterator<URuntimeVirtualTextureComponent> It; It; ++It)
		{
			if (It->GetVirtualTexture() == VirtualTexture)
			{
				It->MarkRenderStateDirty();
			}
		}
	}

	void NotifyPrimitives(URuntimeVirtualTexture const* VirtualTexture)
	{
		for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
		{
			for (URuntimeVirtualTexture* ItVirtualTexture : It->GetRuntimeVirtualTextures())
			{
				if (ItVirtualTexture == VirtualTexture)
				{
					It->MarkRenderStateDirty();
					break;
				}
			}
		}
	}

#endif
}
