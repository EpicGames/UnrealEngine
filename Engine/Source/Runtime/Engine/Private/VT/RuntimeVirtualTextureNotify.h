// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class URuntimeVirtualTexture;

namespace RuntimeVirtualTexture
{
#if WITH_EDITOR
	/**
	 * Find any URuntimeVirtualTextureComponent that reference this virtual texture and mark them dirty.
	 * We need to do this after editing the URuntimeVirtualTexture settings.
	 */
	void NotifyComponents(URuntimeVirtualTexture const* VirtualTexture);

	/**
	 * Find any primitive components that render to this virtual texture and mark them dirty.
	 * We need to do this after editing the URuntimeVirtualTexture settings.
	 */
	void NotifyPrimitives(URuntimeVirtualTexture const* VirtualTexture);
#endif
}