// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTextureVolume.h"

#include "Async/TaskGraphInterfaces.h"
#include "Components/BoxComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimeVirtualTextureVolume)

ARuntimeVirtualTextureVolume::ARuntimeVirtualTextureVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = VirtualTextureComponent = CreateDefaultSubobject<URuntimeVirtualTextureComponent>(TEXT("VirtualTextureComponent"));

#if WITH_EDITORONLY_DATA
	// Add box for visualization of bounds
	Box = CreateDefaultSubobject<UBoxComponent>(TEXT("Bounds"));
	Box->SetBoxExtent(FVector(.5f, .5f, .5f), false);
	Box->SetRelativeTransform(FTransform(FVector(.5f, .5f, .5f)));
	Box->bDrawOnlyIfSelected = true;
	Box->SetIsVisualizationComponent(true);
	Box->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	Box->SetCanEverAffectNavigation(false);
	Box->CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
	Box->SetGenerateOverlapEvents(false);
	Box->SetupAttachment(VirtualTextureComponent);

	bIsSpatiallyLoaded = false;
#endif
}

void ARuntimeVirtualTextureVolume::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixupRuntimeVirtualTextureVolume)
	{
		// Fix old transforms (which required additional maths wherever they were referenced).
		const FTransform TransformFix(FRotator(0, 0, 0), FVector(-.5f, -.5f, -1.f), FVector(1.f, 1.f, 2.f));
		VirtualTextureComponent->SetRelativeTransform(TransformFix * VirtualTextureComponent->GetRelativeTransform());
	}
}

