// Copyright Epic Games, Inc. All Rights Reserved.

#include "UploadingVirtualTexture.h"
#include "HAL/PlatformFile.h"
#include "VirtualTextureChunkManager.h"
#include "FileCache/FileCache.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VirtualTextureChunkDDCCache.h"
#include "ProfilingDebugging/IoStoreTrace.h"

DECLARE_MEMORY_STAT(TEXT("File Cache Size"), STAT_FileCacheSize, STATGROUP_VirtualTextureMemory);
DECLARE_MEMORY_STAT(TEXT("Total Header Size"), STAT_TotalHeaderSize, STATGROUP_VirtualTextureMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Disk Size (MB)"), STAT_TotalDiskSize, STATGROUP_VirtualTextureMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Tile Headers"), STAT_NumTileHeaders, STATGROUP_VirtualTextureMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Codecs"), STAT_NumCodecs, STATGROUP_VirtualTextureMemory);
DECLARE_FLOAT_COUNTER_STAT(TEXT("IO Requests Completed (MB)"), STAT_IORequestsComplete, STATGROUP_VirtualTexturing);

static TAutoConsoleVariable<int32> CVarVTCodecAgeThreshold(
	TEXT("r.VT.CodecAgeThreshold"),
	120,
	TEXT("Mininum number of frames VT codec must be unused before possibly being retired"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVTCodecNumThreshold(
	TEXT("r.VT.CodecNumThreshold"),
	100,
	TEXT("Once number of VT codecs exceeds this number, attempt to retire codecs that haven't been recently used"),
	ECVF_RenderThreadSafe);

int32 GVirtualTextureIOPriority_NormalPagePri = (int32)AIOP_Low;
static FAutoConsoleVariableRef CVarVirtualTextureIOPriority_NormalPagePri(
	TEXT("r.VT.IOPriority_NormalPagePri"),
	GVirtualTextureIOPriority_NormalPagePri,
	TEXT("Priority of default priority VT I/O requests"),
	ECVF_Default
);

int32 GVirtualTextureIOPriority_HighPagePri = (int32)AIOP_BelowNormal;
static FAutoConsoleVariableRef CVarVirtualTextureIOPriority_HighPagePri(
	TEXT("r.VT.IOPriority_HighPagePri"),
	GVirtualTextureIOPriority_HighPagePri,
	TEXT("Priority of high priority VT I/O requests"),
	ECVF_Default
);

static EAsyncIOPriorityAndFlags GetAsyncIOPriority(EVTRequestPagePriority Priority)
{
	switch (Priority)
	{
	case EVTRequestPagePriority::High: return (EAsyncIOPriorityAndFlags)GVirtualTextureIOPriority_HighPagePri | EAsyncIOPriorityAndFlags::AIOP_FLAG_DONTCACHE;
	case EVTRequestPagePriority::Normal: return (EAsyncIOPriorityAndFlags)GVirtualTextureIOPriority_NormalPagePri | EAsyncIOPriorityAndFlags::AIOP_FLAG_DONTCACHE;
	default: check(false); return (EAsyncIOPriorityAndFlags)GVirtualTextureIOPriority_NormalPagePri | EAsyncIOPriorityAndFlags::AIOP_FLAG_DONTCACHE;
	}
}

FVirtualTextureCodec* FVirtualTextureCodec::ListHead = nullptr;
FVirtualTextureCodec FVirtualTextureCodec::ListTail;
uint32 FVirtualTextureCodec::NumCodecs = 0u;

FUploadingVirtualTexture::FUploadingVirtualTexture(const FName& InName, FVirtualTextureBuiltData* InData, int32 InFirstMipToUse)
	: Name(InName)
	, Data(InData)
	, FirstMipOffset(InFirstMipToUse)
{
	HandlePerChunk.AddDefaulted(InData->Chunks.Num());
	CodecPerChunk.AddDefaulted(InData->Chunks.Num());
	InvalidChunks.Init(false, InData->Chunks.Num());

	StreamingManager = &IStreamingManager::Get().GetVirtualTextureStreamingManager();

	INC_MEMORY_STAT_BY(STAT_TotalHeaderSize, InData->GetMemoryFootprint());
	INC_DWORD_STAT_BY(STAT_TotalDiskSize, InData->GetDiskMemoryFootprint() / (1024 * 1024));
	INC_DWORD_STAT_BY(STAT_NumTileHeaders, InData->GetNumTileHeaders());
}

FUploadingVirtualTexture::~FUploadingVirtualTexture()
{
	check(Data);
	DEC_MEMORY_STAT_BY(STAT_TotalHeaderSize, Data->GetMemoryFootprint());
	DEC_DWORD_STAT_BY(STAT_TotalDiskSize, Data->GetDiskMemoryFootprint() / (1024 * 1024));
	DEC_DWORD_STAT_BY(STAT_NumTileHeaders, Data->GetNumTileHeaders());

	for (TUniquePtr<FVirtualTextureCodec>& Codec : CodecPerChunk)
	{
		if (Codec)
		{
			Codec->Unlink();
			Codec.Reset();
		}
	}

#if WITH_EDITOR
	// Need to flush DDC tasks in case they reference any chunks from this.
	GetVirtualTextureChunkDDCCache()->WaitFlushRequests_AnyThread();
#endif
}

uint32 FUploadingVirtualTexture::GetLocalMipBias(uint8 vLevel, uint32 vAddress) const
{
	vLevel += FirstMipOffset;

	const uint32 NumMips = Data->NumMips;
	uint32 Current_vLevel = vLevel;
	uint32 Current_vAddress = vAddress;
	while (Current_vLevel < NumMips)
	{
		if (!Data->IsValidAddress(Current_vLevel, Current_vAddress))
		{
			// vAddress is out-of-bounds for the given producer
			Current_vLevel = NumMips - 1u;
			break;
		}

		const uint32 TileOffset = Data->GetTileOffset(Current_vLevel, Current_vAddress, 0);
		if (TileOffset != ~0u)
		{
			break;
		}

		Current_vLevel++;
		Current_vAddress >>= 2;
	}

	return Current_vLevel - vLevel;
}

FVTRequestPageResult FUploadingVirtualTexture::RequestPageData(FRHICommandListBase& RHICmdList, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress, EVTRequestPagePriority Priority)
{
	vLevel += FirstMipOffset;

	check(vAddress <= MAX_uint32); // Not supporting 64 bit vAddress here. Only currently supported for adaptive runtime virtual texture.
	return StreamingManager->RequestTile(RHICmdList, this, ProducerHandle, LayerMask, vLevel, (uint32)vAddress, Priority);
}

IVirtualTextureFinalizer* FUploadingVirtualTexture::ProducePageData(FRHICommandListBase& RHICmdList,
	ERHIFeatureLevel::Type FeatureLevel,
	EVTProducePageFlags Flags,
	const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress,
	uint64 RequestHandle,
	const FVTProduceTargetLayer* TargetLayers)
{
	INC_DWORD_STAT(STAT_VTP_NumUploads);

	const uint32 SkipBorderSize = (Flags & EVTProducePageFlags::SkipPageBorders) != EVTProducePageFlags::None ? Data->TileBorderSize : 0;
	return StreamingManager->ProduceTile(RHICmdList, SkipBorderSize, Data->GetNumLayers(), LayerMask, RequestHandle, TargetLayers);
}

void FUploadingVirtualTexture::GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const
{
	StreamingManager->GatherProducePageDataTasks(ProducerHandle, InOutTasks);
}

void FUploadingVirtualTexture::GatherProducePageDataTasks(uint64 RequestHandle, FGraphEventArray& InOutTasks) const
{
	StreamingManager->GatherProducePageDataTasks(RequestHandle, InOutTasks);
}

void FVirtualTextureCodec::RetireOldCodecs()
{
	const uint32 AgeThreshold = CVarVTCodecAgeThreshold.GetValueOnRenderThread();
	const uint32 NumThreshold = CVarVTCodecNumThreshold.GetValueOnRenderThread();
	const uint32 CurrentFrame = GFrameNumberRenderThread;

	FVirtualTextureCodec::TIterator It(ListHead);
	while (It && NumCodecs > NumThreshold)
	{
		FVirtualTextureCodec& Codec = *It;
		It.Next();

		bool bRetiredCodec = false;
		// Can't retire codec if it's not even finished loading yet
		if (Codec.Owner && Codec.IsIdle())
		{
			check(CurrentFrame >= Codec.LastFrameUsed);
			const uint32 Age = CurrentFrame - Codec.LastFrameUsed;
			if (Age > AgeThreshold)
			{
				Codec.Unlink();
				Codec.Owner->CodecPerChunk[Codec.ChunkIndex].Reset();
				bRetiredCodec = true;
			}
		}

		if (!bRetiredCodec)
		{
			// List is kept sorted, so once we find a codec too new to retire, don't need to check any further
			break;
		}
	}
}

void FVirtualTextureCodec::Init(IMemoryReadStreamRef& HeaderData)
{
	const FVirtualTextureBuiltData* VTData = Owner->GetVTData();
	const FVirtualTextureDataChunk& Chunk = VTData->Chunks[ChunkIndex];
	const uint32 NumLayers = VTData->GetNumLayers();

	TArray<uint8, TInlineAllocator<16u * 1024>> TempBuffer;

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		const uint32 CodecPayloadOffset = Chunk.CodecPayloadOffset[LayerIndex];
		const uint32 NextOffset = (LayerIndex + 1u) < NumLayers ? Chunk.CodecPayloadOffset[LayerIndex + 1] : Chunk.CodecPayloadSize;
		const uint32 CodecPayloadSize = NextOffset - CodecPayloadOffset;
		
		const uint8* CodecPayload = nullptr;
		if (CodecPayloadSize > 0u)
		{
			int64 PayloadReadSize = 0;
			CodecPayload = (uint8*)HeaderData->Read(PayloadReadSize, CodecPayloadOffset, CodecPayloadSize);
			if (PayloadReadSize < (int64)CodecPayloadSize)
			{
				// Generally this should not be needed, since payload is at start of file, should generally not cross a file cache page boundary
				TempBuffer.SetNumUninitialized(CodecPayloadSize);
				HeaderData->CopyTo(TempBuffer.GetData(), CodecPayloadOffset, CodecPayloadSize);
				CodecPayload = TempBuffer.GetData();
			}
		}
	}
}

void FVirtualTextureCodec::LinkGlobalHead()
{
	if (!ListHead)
	{
		ListTail.LinkHead(ListHead);
	}
	LinkHead(ListHead);
}

void FVirtualTextureCodec::LinkGlobalTail()
{
	if (!ListHead)
	{
		ListTail.LinkHead(ListHead);
	}
	LinkBefore(&ListTail);
}

FVirtualTextureCodec::~FVirtualTextureCodec()
{
	if (Owner)
	{
		checkf(IsCreationComplete(), TEXT("Codec is being released before its construction task hasn't finished."));
		checkf(AllTranscodeTasksComplete(), TEXT("Codec is being released while there are tasks that still reference it."));

		check(!IsLinked());

		check(NumCodecs > 0u);
		--NumCodecs;
		DEC_DWORD_STAT(STAT_NumCodecs);
	}
}

struct FCreateCodecTask
{
	IMemoryReadStreamRef HeaderData;
	FVirtualTextureCodec* Codec;

	FCreateCodecTask(const IMemoryReadStreamRef& InHeader, FVirtualTextureCodec* InCodec)
		: HeaderData(InHeader)
		, Codec(InCodec)
	{}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Codec->Init(HeaderData);
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyNormalThreadNormalTask; }

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FCreateCodecTask, STATGROUP_VTP);
	}
};

FVTCodecAndStatus FUploadingVirtualTexture::GetCodecForChunk(FGraphEventArray& OutCompletionEvents, uint32 ChunkIndex, EVTRequestPagePriority Priority)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUploadingVirtualTexture::GetCodecForChunk);

	// CodecPayloadSize includes the header that is always present even if there is no payload.
 	const FVirtualTextureDataChunk& Chunk = Data->Chunks[ChunkIndex];
 	if (Chunk.CodecPayloadSize <= sizeof(FVirtualTextureChunkHeader))
	{
		// Chunk has no codec
		return EVTRequestPageStatus::Available;
	}

	TUniquePtr<FVirtualTextureCodec>& Codec = CodecPerChunk[ChunkIndex];
	if (Codec)
	{
		const bool bIsComplete = !Codec->CompletedEvent || Codec->CompletedEvent->IsComplete();
		if (!bIsComplete)
		{
			OutCompletionEvents.Add(Codec->CompletedEvent);
		}
		// Update LastFrameUsed and re-link to the back of the list
		Codec->Unlink();
		Codec->LinkGlobalTail();
		Codec->LastFrameUsed = GFrameNumberRenderThread;
		return FVTCodecAndStatus(bIsComplete ? EVTRequestPageStatus::Available : EVTRequestPageStatus::Pending, Codec.Get());
	}

	FGraphEventArray ReadDataCompletionEvents;
	const FVTDataAndStatus HeaderResult = ReadData(ReadDataCompletionEvents, ChunkIndex, 0, Chunk.CodecPayloadSize, Priority);
	if (!VTRequestPageStatus_HasData(HeaderResult.Status))
	{
		// GetData may fail if file cache is saturated
		return HeaderResult.Status;
	}

	INC_DWORD_STAT(STAT_NumCodecs);
	++FVirtualTextureCodec::NumCodecs;
	Codec.Reset(new FVirtualTextureCodec());
	Codec->LinkGlobalTail();
	Codec->Owner = this;
	Codec->ChunkIndex = ChunkIndex;
	Codec->LastFrameUsed = GFrameNumberRenderThread;
	Codec->CompletedEvent = TGraphTask<FCreateCodecTask>::CreateTask(&ReadDataCompletionEvents).ConstructAndDispatchWhenReady(HeaderResult.Data, Codec.Get());
	OutCompletionEvents.Add(Codec->CompletedEvent);
	return FVTCodecAndStatus(EVTRequestPageStatus::Pending, Codec.Get());
}

FVTDataAndStatus FUploadingVirtualTexture::ReadData(FGraphEventArray& OutCompletionEvents, uint32 ChunkIndex, size_t Offset, size_t Size, EVTRequestPagePriority Priority)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUploadingVirtualTexture::ReadData);
	TRACE_IOSTORE_METADATA_SCOPE_TAG(Name);

	FVirtualTextureDataChunk& Chunk = Data->Chunks[ChunkIndex];
	FByteBulkData& BulkData = Chunk.BulkData;
#if WITH_EDITOR
	// It could be that the bulkdata has no file associated yet (aka lightmaps have been build but not saved to disk yet) but still contains valid data
	// Streaming is done from memory
	if (BulkData.IsBulkDataLoaded() && BulkData.GetBulkDataSize() > 0)
	{
		ensure(Size <= (size_t)BulkData.GetBulkDataSize());
		const uint8 *P = (uint8*)BulkData.LockReadOnly() + Offset;
		IMemoryReadStreamRef Buffer = IMemoryReadStream::CreateFromCopy(P, Size);
		BulkData.Unlock();
		return FVTDataAndStatus(EVTRequestPageStatus::Available, Buffer);
	}
#endif // WITH_EDITOR

	TUniquePtr<IFileCacheHandle>& Handle = HandlePerChunk[ChunkIndex];
	if (!Handle)
	{
		enum class EChunkSource
		{
			File,
			BulkData,
			Invalid
		};
		EChunkSource ChunkSource = EChunkSource::Invalid;
		FString ChunkFileName;
		int64 ChunkOffsetInFile = 0;
#if WITH_EDITOR
		// If the bulkdata has a file associated with it, we stream directly from it.
		// This only happens for lightmaps atm
		if (BulkData.CanLoadFromDisk())
		{
			ensure(Size <= (size_t)BulkData.GetBulkDataSize());

			ChunkOffsetInFile = BulkData.GetBulkDataOffsetInFile();
			ChunkSource = EChunkSource::BulkData;
		}
		// Else it should be VT data that is injected into the DDC (and stream from VT DDC cache)
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_VTP_MakeChunkAvailable);
			check(Chunk.DerivedDataKey.IsEmpty() == false);
			FString ChunkFileNameDDC;

			// If request is flagged as high priority, we will block here until DDC cache is populated
			// This way we can service these high priority tasks immediately
			// Would be better to have DDC cache return a task event handle, which could be used to chain a subsequent read operation,
			// but that would be more complicated, and this should generally not be a critical runtime path
			const bool bAsyncDDC = (Priority == EVTRequestPagePriority::Normal);
			const bool Available = GetVirtualTextureChunkDDCCache()->MakeChunkAvailable(&Data->Chunks[ChunkIndex], bAsyncDDC, ChunkFileNameDDC, ChunkOffsetInFile);
			if (!Available)
			{
				return EVTRequestPageStatus::Saturated;
			}
			ChunkFileName = ChunkFileNameDDC;
			ChunkSource = EChunkSource::File;
		}
#else // WITH_EDITOR
		ChunkOffsetInFile = BulkData.GetBulkDataOffsetInFile();
		if (BulkData.GetBulkDataSize() == 0)
		{
			if (!InvalidChunks[ChunkIndex])
			{
				UE_LOGF(LogConsoleResponse, Display, "BulkData for chunk %d in file '%ls' is empty.", ChunkIndex, *BulkData.GetDebugName());
				InvalidChunks[ChunkIndex] = true;
			}
			return EVTRequestPageStatus::Invalid;
		}
		ChunkSource = EChunkSource::BulkData;
#endif // !WITH_EDITOR
		// If we have a valid file name then we can create a file handle directly to it.
		// If we do not then pass in the BulkData object which will create the IAsyncReadFileHandle for us.
		switch (ChunkSource)
		{
		case EChunkSource::File:
			Handle.Reset(IFileCacheHandle::CreateFileCacheHandle(*ChunkFileName, ChunkOffsetInFile));
			break;
		case EChunkSource::BulkData:
			Handle.Reset(IFileCacheHandle::CreateFileCacheHandle(BulkData.OpenAsyncReadHandle(), ChunkOffsetInFile));
			break;
		default:
			check(false);
			return EVTRequestPageStatus::Invalid;
		}
		
		// Don't expect CreateFileCacheHandle() to fail, async files should never fail to open
		if (!ensure(Handle.IsValid()))
		{
			if (!InvalidChunks[ChunkIndex])
			{
				UE_LOGF(LogConsoleResponse, Display, "Could not create a file cache for '%ls'.", *ChunkFileName);
				InvalidChunks[ChunkIndex] = true;
			}
			return EVTRequestPageStatus::Invalid;
		}
		SET_MEMORY_STAT(STAT_FileCacheSize, IFileCacheHandle::GetFileCacheSize());
	}

	IMemoryReadStreamRef ReadData = Handle->ReadData(OutCompletionEvents, Offset, Size, GetAsyncIOPriority(Priority));
	if (!ReadData)
	{
		return EVTRequestPageStatus::Saturated;
	}

	const float SizeMB = (float)Size / (float)(1024 * 1024);
	INC_FLOAT_STAT_BY(STAT_IORequestsComplete, SizeMB);
	
	return FVTDataAndStatus(EVTRequestPageStatus::Pending, ReadData);
}

void FUploadingVirtualTexture::DumpToConsole(bool verbose)
{
	UE_LOGF(LogConsoleResponse, Display, "Uploading virtual texture");
	UE_LOGF(LogConsoleResponse, Display, "FirstMipOffset: %i", FirstMipOffset);
	UE_LOGF(LogConsoleResponse, Display, "Current Size: %i x %i", Data->Width >> FirstMipOffset, Data->Height >> FirstMipOffset);
	UE_LOGF(LogConsoleResponse, Display, "Cooked Size: %i x %i", Data->Width, Data->Height);
	UE_LOGF(LogConsoleResponse, Display, "Cooked Tiles: %i x %i", Data->GetWidthInTiles(), Data->GetHeightInTiles());
	UE_LOGF(LogConsoleResponse, Display, "Tile Size: %i", Data->TileSize);
	UE_LOGF(LogConsoleResponse, Display, "Tile Border: %i", Data->TileBorderSize);
	UE_LOGF(LogConsoleResponse, Display, "Chunks: %i", Data->Chunks.Num());
	UE_LOGF(LogConsoleResponse, Display, "Layers: %i", Data->GetNumLayers());

	TSet<FString> BulkDataFiles;

	for (const auto& Chunk : Data->Chunks)
	{
#if WITH_EDITORONLY_DATA
		if (Chunk.DerivedDataKey.IsEmpty() == false)
		{
			BulkDataFiles.Add(Chunk.DerivedDataKey);
		}
		else
#endif
			BulkDataFiles.Add(Chunk.BulkData.GetDebugName());
	}

	for (const auto& FileName : BulkDataFiles)
	{
		UE_LOGF(LogConsoleResponse, Display, "Bulk data file / DDC entry: %ls", *FileName);
	}

#if 0
	if (verbose)
	{
		for (int32 L = 0; L < Data->Tiles.Num(); L++)
		{
			for (int32 T = 0; T < Data->Tiles[L].Num(); T++)
			{
				// TODO - missing VirtualAddress, GetPhysicalAddress

				FVirtualTextureTileInfo &Tile = Data->Tiles[L][T];
				uint32 lX = FMath::ReverseMortonCode2(T);
				uint32 lY = FMath::ReverseMortonCode2(T >> 1);

				uint32 SpaceAddress = VirtualAddress + T;
				uint32 vX = FMath::ReverseMortonCode2(SpaceAddress);
				uint32 vY = FMath::ReverseMortonCode2(SpaceAddress >> 1);


				// Check if the tile is resident if so print physical info as well
				uint32 pAddr = Space->GetPhysicalAddress(L, SpaceAddress);
				if (pAddr != ~0u)
				{
					uint32 pX = FMath::ReverseMortonCode2(pAddr);
					uint32 pY = FMath::ReverseMortonCode2(pAddr >> 1);
					UE_LOGF(LogConsoleResponse, Display, "Tile: Level %i, lAddr %i (%i,%i), vAddr %i (%i,%i), pAddr %i (%i,%i), Chunk %i, Offset %i, Size %i %i %i %i",
						L, T, lX, lY,
						SpaceAddress, vX, vY,
						pAddr, pX, pY,
						Tile.Chunk, Tile.Offset, Tile.Size[0], (NumLayers > 1) ? Tile.Size[1] : 0, (NumLayers > 2) ? Tile.Size[2] : 0, (NumLayers > 3) ? Tile.Size[3] : 0);
				}
				else
				{
					if (Tile.Chunk < 0 || Tile.Size.Num() == 0)
					{
						UE_LOGF(LogConsoleResponse, Display, "Tile: Level %i, lAddr %i (%i,%i), vAddr %i (%i,%i) - No Data Associated",
							L, T, lX, lY,
							SpaceAddress, vX, vY);
					}
					else
					{
						UE_LOGF(LogConsoleResponse, Display, "Tile: Level %i, lAddr %i (%i,%i), vAddr %i (%i,%i), Chunk %i, Offset %i, Size %i %i %i %i",
							L, T, lX, lY,
							SpaceAddress, vX, vY,
							Tile.Chunk, Tile.Offset, Tile.Size[0], (NumLayers > 1) ? Tile.Size[1] : 0, (NumLayers > 2) ? Tile.Size[2] : 0, (NumLayers > 3) ? Tile.Size[3] : 0);
					}
				}
			}
		}
	}
#endif
}
