// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VirtualTextureBuiltData.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/MemoryReadStream.h"
#include "Containers/BitArray.h"
#include "Containers/List.h"
#include "Templates/UniquePtr.h"
#include "Async/TaskGraphInterfaces.h"
#include "VirtualTexturing.h"

#include <atomic>

class FUploadingVirtualTexture;
class IFileCacheHandle;

class FVirtualTextureCodec : public TIntrusiveLinkedList<FVirtualTextureCodec>
{
	friend struct FTranscodeTask;
public:
	static void RetireOldCodecs();

	~FVirtualTextureCodec();
	void Init(IMemoryReadStreamRef& HeaderData);

	// Tracks completion of FCreateCodecTask.
	inline bool IsCreationComplete() const { return !CompletedEvent || CompletedEvent->IsComplete(); }

	inline bool IsIdle() const { return IsCreationComplete() && AllTranscodeTasksComplete(); }

	void LinkGlobalHead();
	void LinkGlobalTail();

	static FVirtualTextureCodec* ListHead;
	static FVirtualTextureCodec ListTail;
	static uint32 NumCodecs;

	FGraphEventRef CompletedEvent;

	class FUploadingVirtualTexture* Owner;
	void* Contexts[VIRTUALTEXTURE_DATA_MAXLAYERS] = { nullptr };
	uint32 ChunkIndex = 0u;
	uint32 LastFrameUsed = 0u;

protected:

	// This codec may be used by transcode tasks (see FTranscodeTask),
	// which must be all complete before the codec is released by RetireOldTasks().
	inline bool AllTranscodeTasksComplete() const { return TaskRefs.load() == 0; }
	inline void BeginTranscodeTask() const { TaskRefs.fetch_add(1); }
	inline void EndTranscodeTask() const { TaskRefs.fetch_sub(1); }

	mutable std::atomic<uint32> TaskRefs {};
};

struct FVTCodecAndStatus
{
	FVTCodecAndStatus(EVTRequestPageStatus InStatus, const FVirtualTextureCodec* InCodec = nullptr) : Codec(InCodec), Status(InStatus) {}
	const FVirtualTextureCodec* Codec;
	EVTRequestPageStatus Status;
};

struct FVTDataAndStatus
{
	FVTDataAndStatus(EVTRequestPageStatus InStatus, const IMemoryReadStreamRef& InData = nullptr) : Data(InData), Status(InStatus) {}
	IMemoryReadStreamRef Data;
	EVTRequestPageStatus Status;
};

// IVirtualTexture implementation that is uploading from CPU to GPU and gets its data from a FChunkProvider
class FUploadingVirtualTexture : public IVirtualTexture
{
public:
	FUploadingVirtualTexture(const FName& InName, FVirtualTextureBuiltData* InData, int32 FirstMipToUse);
	virtual ~FUploadingVirtualTexture();

	// IVirtualTexture interface
	virtual uint32 GetLocalMipBias(uint8 vLevel, uint32 vAddress) const override;
	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override { return true; }
	virtual FVTRequestPageResult RequestPageData(FRHICommandListBase& RHICmdList, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress, EVTRequestPagePriority Priority) override;
	virtual IVirtualTextureFinalizer* ProducePageData(FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers) override;
	virtual void GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const override;
	virtual void GatherProducePageDataTasks(uint64 RequestHandle, FGraphEventArray& InOutTasks) const override;
	virtual void DumpToConsole(bool verbose) override;
	// End IVirtualTexture interface

	inline const FName& GetName() const { return Name; }
	inline const FVirtualTextureBuiltData* GetVTData() const { return Data; }

	// gets the codec for the given chunk, data is not valid until returned OutCompletionEvents are complete
	FVTCodecAndStatus GetCodecForChunk(FGraphEventArray& OutCompletionEvents, uint32 ChunkIndex, EVTRequestPagePriority Priority);

	// read a portion of a chunk
	FVTDataAndStatus ReadData(FGraphEventArray& OutCompletionEvents, uint32 ChunkIndex, size_t Offset, size_t Size, EVTRequestPagePriority Priority);

private:
	friend class FVirtualTextureCodec;

	FName Name;
	FVirtualTextureBuiltData* Data;
	TArray< TUniquePtr<IFileCacheHandle> > HandlePerChunk;
	TArray< TUniquePtr<FVirtualTextureCodec> > CodecPerChunk;
	TBitArray<> InvalidChunks; // marks chunks that failed to load
	int32 FirstMipOffset;

	struct FVirtualTextureChunkStreamingManager* StreamingManager;
};
