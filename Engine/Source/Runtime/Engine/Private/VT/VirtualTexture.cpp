// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTexture.h"

#include "EngineModule.h"
#include "FileCache/FileCache.h"
#include "Interfaces/ITargetPlatform.h"
#include "RenderUtils.h"
#include "RendererInterface.h"
#include "Serialization/CustomVersion.h"
#include "VT/VirtualTextureBuiltData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTexture)

static FAutoConsoleCommand GVTFlushAndEvictFileCacheCommand(
	TEXT("r.VT.FlushAndEvictFileCache"),
	TEXT("Flush both the virtual texture physical page cache and disk file cache"),
	FConsoleCommandDelegate::CreateStatic(
		[]()
	{
		IFileCacheHandle::EvictAll();
		GetRendererModule().FlushVirtualTextureCache();
	})
);

// Deprecated serialization code
namespace
{
	// Version used to serialize dummy data for UVirtualTexture
	struct FVirtualTextureBuiltDataCustomVersion
	{
		static const FGuid Key;
		enum Type {
			Initial,
			MipTails,			// Added support for mipmap tails
			ActualSize,			// Actual size is explicitly stored in the file instead of derived based on tile size
			MacroBlocks,		// Refactor of the Build data to be stored in macroblocks
			SplitDDC,			// Store chunks separated in the DDC
			NoMipTails,			// Removed support for mip tails
			Latest = NoMipTails	// Always update this to be equal to the latest version
		};
	};
	const FGuid FVirtualTextureBuiltDataCustomVersion::Key(0x804E3F75, 0x70884B49, 0xA4D68C06, 0x3C7EB6DC);
	FCustomVersionRegistration GVTDummyRegisterVersion(FVirtualTextureBuiltDataCustomVersion::Key, FVirtualTextureBuiltDataCustomVersion::Latest, TEXT("VirtualTextureBuiltDataVersion"));

	// Types needed for dummy serialize of UVirtualTexture
	struct FLegacyMipTail
	{
		int32 SizeX;
		int32 SizeY;
		int32 SizeZ;
		TArray<uint8> Data;

		friend FArchive& operator<<(FArchive& Ar, FLegacyMipTail& A)
		{
			Ar << A.SizeX;
			Ar << A.SizeY;
			Ar << A.SizeZ;
			Ar << A.Data;
			return Ar;
		}
	};

	struct FLegacyTileInfo
	{
		int16 Chunk = -1; // The index of the chunk the tile is stored in	
		int32 Offset = -1;
		TArray<int32, TFixedAllocator<4>> Size; // 'VIRTUALTEXTURE_DATA_MAXLAYERS' was 4 at the time this was deprecated

		friend FArchive& operator<<(FArchive& Ar, FLegacyTileInfo& A)
		{
			Ar << A.Chunk;
			Ar << A.Offset;
			Ar << A.Size;
			return Ar;
		}
	};
}

UVirtualTexture::UVirtualTexture(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer) 
{
}

void UVirtualTexture::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FVirtualTextureBuiltDataCustomVersion::Key);
	const int32 Version = Ar.CustomVer(FVirtualTextureBuiltDataCustomVersion::Key);

	Super::Serialize(Ar);

	// Dummy serialize contents of FVirtualTextureBuiltData, as it existed when UVirtualTexture was deprecated
	{
		int32 TileWidth = 0;
		int32 TileHeight = 0;
		Ar << TileWidth;
		Ar << TileHeight;

		int32 NumTilesX = 0;
		int32 NumTilesY = 0;
		Ar << NumTilesX;
		Ar << NumTilesY;

		int32 Border = 0;
		Ar << Border;

		TArray<TArray<FLegacyTileInfo>, TFixedAllocator<VIRTUALTEXTURE_DATA_MAXMIPS>> Tiles;
		Ar << Tiles;
	}

	if (Version >= FVirtualTextureBuiltDataCustomVersion::ActualSize)
	{
		int32 Width = 0;
		int32 Height = 0;
		Ar << Width;
		Ar << Height;
	}

	int32 NumLayers = 0;
	Ar << NumLayers;
	if (Ar.IsLoading())
	{
		for (int32 Layer = 0; Layer < NumLayers; Layer++)
		{
			FString PixelFormatString;
			Ar << PixelFormatString;
		}
	}

	if (Version >= FVirtualTextureBuiltDataCustomVersion::MipTails && Version < FVirtualTextureBuiltDataCustomVersion::NoMipTails)
	{
		TArray<TArray<FLegacyMipTail>, TFixedAllocator<4>> DummyMipTails;
		Ar << DummyMipTails;
	}

	int32 NumChunks = 0;
	Ar << NumChunks;

	for (int32 ChunkId = 0; ChunkId < NumChunks; ChunkId++)
	{
		// Serialize the chunk header
		bool bCooked = Ar.IsCooking();
		if (Version >= FVirtualTextureBuiltDataCustomVersion::SplitDDC)
		{
			Ar << bCooked;
		}

		FByteBulkData BulkData;
		BulkData.Serialize(Ar, this, ChunkId, false);

#if WITH_EDITORONLY_DATA
		if (!bCooked && Version >= FVirtualTextureBuiltDataCustomVersion::SplitDDC)
		{
			FString DerivedDataKey;
			Ar << DerivedDataKey;
		}
#endif // #if WITH_EDITORONLY_DATA
	}

	// serialize the chunk header sizes
	if (Version >= FVirtualTextureBuiltDataCustomVersion::MacroBlocks)
	{
		int NumChunkHeaderSizes = 0;
		Ar << NumChunkHeaderSizes;

		for (int32 ChunkId = 0; ChunkId < NumChunkHeaderSizes; ChunkId++)
		{
			uint32 ChunkHeaderSize = 0u;
			Ar << ChunkHeaderSize;
		}
	}
}

ULightMapVirtualTexture::ULightMapVirtualTexture(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer) 
{
}

URuntimeVirtualTextureStreamingProxy::URuntimeVirtualTextureStreamingProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


UVirtualTexture2D::UVirtualTexture2D(const FObjectInitializer& ObjectInitializer)
	: UTexture2D(ObjectInitializer)
	, bSinglePhysicalSpace(false)
{
}

#if WITH_EDITOR

void UVirtualTexture2D::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	// Even though we skip the cook of this object for non VT platforms in URuntimeVirtualTexture::Serialize()
	// we still load the object at cook time and kick off the DDC build. This will trigger an error in the texture DDC code.
	// Either we need to make the DDC code more robust for non VT platforms or we can skip the process here...
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return;
	}

	Super::BeginCacheForCookedPlatformData(TargetPlatform);
}

bool UVirtualTexture2D::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return true;
	}

	return Super::IsCachedCookedPlatformDataLoaded(TargetPlatform);
}

void UVirtualTexture2D::ClearCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	if (!UseVirtualTexturing(TargetPlatform))
	{
		return;
	}

	Super::ClearCachedCookedPlatformData(TargetPlatform);
}

#endif

