// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureAdapter.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "GlobalShader.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"
#include "VirtualTextureEnum.h"
#include "VirtualTexturing.h"
#include "VT/CopyCompressShader.h"
#include "VT/VirtualTextureBuildSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureAdapter)

IMPLEMENT_GLOBAL_SHADER(FCopyCompressCS, "/Engine/Private/VirtualTextureAdapter.usf", "CopyCompressCS", SF_Compute);

namespace VirtualTextureAdapter
{
	/** Final copy to output RDG parameters. */
	BEGIN_SHADER_PARAMETER_STRUCT(FCopyToOutputParameters, )
		RDG_TEXTURE_ACCESS(Input, ERHIAccess::CopySrc)
		RDG_TEXTURE_ACCESS(Output, ERHIAccess::CopyDest)
	END_SHADER_PARAMETER_STRUCT()


	/** Get the final virtual texture format from the wrapped texture format. */
	static EPixelFormat GetFinalFormat(EPixelFormat InSourcePixelFormat, EPixelFormat InFinalPixelFormat)
	{
		EPixelFormat FinalPixelFormat = InFinalPixelFormat != PF_Unknown ? InFinalPixelFormat : InSourcePixelFormat;

		// Can't override some formats.
		if (IsBlockCompressedFormat(InSourcePixelFormat) || IsInteger(InSourcePixelFormat) || IsInteger(FinalPixelFormat))
		{
			return InSourcePixelFormat;
		}

		return FinalPixelFormat;
	}

	/** Get the intermediate texture format that we use for transient intermediate targets. */
	static EPixelFormat GetIntermediateFormat(EPixelFormat InSourceFormat, EPixelFormat InDestFormat)
	{
		if (IsBlockCompressedFormat(InSourceFormat))
		{
			return IsHDR(InSourceFormat) ? PF_FloatRGBA : PF_R8G8B8A8;
		}

		return InSourceFormat;
	}

	/** Copy from source to destination. Handles downsample and optional texture compression. */
	static void RenderTile(
		FRDGBuilder& GraphBuilder, 
		FShaderResourceViewRHIRef& InSourceSRV,
		IPooledRenderTarget* InDestRenderTarget, 
		EPixelFormat InSourceFormat,
		EPixelFormat InIntermediateFormat,
		EPixelFormat InDestFormat,
		uint32 InLevel, 
		FBox2f const& InUVRange, 
		FIntRect const& InDestRect)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		// Need a compute shader copy on the first mip level when:
		// * Source/Dest formats are incompatible, or
		// * Source UV extends outside texture (because of tile borders). Then we need correct clamping behavior.
		// For other cases we can just rely on the final RHICopyTexture (which can also handle the direct copy of compressed formats).
		const bool bCanUseRHICopyTexture = InSourceFormat == InDestFormat || IsBlockCompressedFormat(InDestFormat);
		const bool bCopyRequiresClamping = InUVRange.Min.X < 0 || InUVRange.Min.Y < 0 || InUVRange.Max.X > 1 || InUVRange.Max.Y > 1;
		const bool bUseCopyStep = InLevel == 0 && (!bCanUseRHICopyTexture || bCopyRequiresClamping);
		// Need downsampling steps if we are copying to higher mip level.
		// Note that if source texture has mips then we should have passed in the correct SRV and modified InLevel to avoid the downsampling.
		const bool bUseDownsampleStep = InLevel > 0 && !bUseCopyStep;
		// Need a compresssion step if we have a compressed format that isn't simply copied with the final RHICopyTexture.
		const bool bUseCompressionStep = IsBlockCompressedFormat(InDestFormat) && (InSourceFormat != InDestFormat || bUseCopyStep || bUseDownsampleStep);

		const uint32 FinalTexelCountX = InDestRect.Max.X - InDestRect.Min.X;
		const uint32 FinalTexelCountY = InDestRect.Max.Y - InDestRect.Min.Y;

		const bool bSourceSrgb = InSourceSRV->GetDesc().Texture.SRV.GetViewInfo(InSourceSRV->GetTexture()).bSRGB;
		const bool bDestSrgb = EnumHasAnyFlags(InDestRenderTarget->GetRHI()->GetDesc().Flags, ETextureCreateFlags::SRGB);
		const ETextureCreateFlags TextureCreateFlagsSrgb = bSourceSrgb ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None;
		const bool bFinalPassConvertToSrgb = !bSourceSrgb && bDestSrgb;

		FRDGTextureRef CurrentOutput = nullptr;
		FBox2f CurrentUVRange(InUVRange);
		bool bIsDone = false;

		if (bUseCopyStep)
		{
			const bool bIsFinalPass = !bUseCompressionStep;
			if (bIsFinalPass)
			{
				CurrentOutput = GraphBuilder.RegisterExternalTexture(InDestRenderTarget, ERDGTextureFlags::None);
			}
			else
			{
				const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(FIntPoint(FinalTexelCountX, FinalTexelCountY), InIntermediateFormat, FClearValueBinding::None, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | TextureCreateFlagsSrgb));
				CurrentOutput = GraphBuilder.CreateTexture(Desc, TEXT("VirtualTextureAdapter.Downsample"));
			}

			FCopyCompressCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyCompressCS::FSourceTextureSelector>(true);
			PermutationVector.Set<FCopyCompressCS::FDestSrgb>(bIsFinalPass ? bFinalPassConvertToSrgb : false);
			PermutationVector.Set<FCopyCompressCS::FCompressionFormatDim>(0);
			TShaderMapRef<FCopyCompressCS> Shader(GlobalShaderMap, PermutationVector);

			FCopyCompressCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyCompressCS::FParameters>();
			Parameters->SourceTextureA = InSourceSRV;
			Parameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->DestTexture = GraphBuilder.CreateUAV(CurrentOutput);
			Parameters->SourceUV = CurrentUVRange.Min;
			Parameters->TexelSize = (CurrentUVRange.Max - CurrentUVRange.Min) / FVector2f(FinalTexelCountX, FinalTexelCountY);
			Parameters->TexelOffsets = FVector2f(1.0f, 0.0f);
			Parameters->DestRect = bIsFinalPass ? FIntVector4(InDestRect.Min.X, InDestRect.Min.Y, InDestRect.Max.X, InDestRect.Max.Y) : FIntVector4(0, 0, FinalTexelCountX, FinalTexelCountY);

			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(FIntPoint(FinalTexelCountX, FinalTexelCountY), FCopyCompressCS::GroupSize);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("VirtualTextureAdapterDownsample"),
				Shader, Parameters, GroupCount);

			CurrentUVRange = FBox2f(FVector2f(0, 0), FVector2f(1, 1));
			bIsDone = bIsFinalPass;
		}

		if (bUseDownsampleStep)
		{
			for (uint32 LevelIndex = 0; LevelIndex < InLevel; ++LevelIndex)
			{
				FRDGTextureRef LastOutput = CurrentOutput;
				const bool bUseSourceTextureA = LastOutput == nullptr;
			
				const uint32 DownsampleInputSizeX = FinalTexelCountX << (InLevel - LevelIndex);
				const uint32 DownsampleInputSizeY = FinalTexelCountY << (InLevel - LevelIndex);
				const uint32 DownsampleOutputSizeX = FinalTexelCountX << (InLevel - LevelIndex - 1);
				const uint32 DownsampleOutputSizeY = FinalTexelCountY << (InLevel - LevelIndex - 1);

				const bool bIsFinalPass = (LevelIndex == InLevel - 1) && !bUseCompressionStep;
				if (bIsFinalPass)
				{
					CurrentOutput = GraphBuilder.RegisterExternalTexture(InDestRenderTarget, ERDGTextureFlags::None);
				}
				else
				{
					const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(FIntPoint(DownsampleOutputSizeX, DownsampleOutputSizeY), InIntermediateFormat, FClearValueBinding::None, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | TextureCreateFlagsSrgb));
					CurrentOutput = GraphBuilder.CreateTexture(Desc, TEXT("VirtualTextureAdapter.Downsample"));
				}

				FCopyCompressCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCopyCompressCS::FSourceTextureSelector>(bUseSourceTextureA);
				PermutationVector.Set<FCopyCompressCS::FDestSrgb>(bIsFinalPass ? bFinalPassConvertToSrgb : false);
				PermutationVector.Set<FCopyCompressCS::FCompressionFormatDim>(0);
				TShaderMapRef<FCopyCompressCS> Shader(GlobalShaderMap, PermutationVector);

				FCopyCompressCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyCompressCS::FParameters>();
				Parameters->SourceTextureA = bUseSourceTextureA ? InSourceSRV : nullptr;
				Parameters->SourceTextureB = bUseSourceTextureA ? nullptr : GraphBuilder.CreateSRV(FRDGTextureSRVDesc(LastOutput));
				Parameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				Parameters->DestTexture = GraphBuilder.CreateUAV(CurrentOutput);
				Parameters->SourceUV = CurrentUVRange.Min;
				Parameters->TexelSize = (CurrentUVRange.Max - CurrentUVRange.Min) / FVector2f(DownsampleInputSizeX, DownsampleInputSizeY);
				Parameters->TexelOffsets = FVector2f(2.0f, 0.5f);
				Parameters->DestRect = bIsFinalPass ? FIntVector4(InDestRect.Min.X, InDestRect.Min.Y, InDestRect.Max.X, InDestRect.Max.Y) : FIntVector4(0, 0, DownsampleOutputSizeX, DownsampleOutputSizeY);

				const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(FIntPoint(DownsampleOutputSizeX, DownsampleOutputSizeY), FCopyCompressCS::GroupSize);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("VirtualTextureAdapterDownsample"),
					Shader, Parameters, GroupCount);

				CurrentUVRange = FBox2f(FVector2f(0, 0), FVector2f(1, 1));
				bIsDone = bIsFinalPass;
			}
		}

		if (bUseCompressionStep)
		{
			const int32 CompressionPermutation = FCopyCompressCS::GetCompressionPermutation(InDestFormat);
			check(CompressionPermutation > 0)
			
			FRDGTextureRef LastOutput = CurrentOutput;
			const bool bUseSourceTextureA = LastOutput == nullptr;

			FCopyCompressCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyCompressCS::FSourceTextureSelector>(bUseSourceTextureA);
			PermutationVector.Set<FCopyCompressCS::FDestSrgb>(bFinalPassConvertToSrgb);
			PermutationVector.Set<FCopyCompressCS::FCompressionFormatDim>(CompressionPermutation);
			TShaderMapRef<FCopyCompressCS> Shader(GlobalShaderMap, PermutationVector);

			const EPixelFormat AliasFormat64bit = PF_R32G32_UINT;
			const EPixelFormat AliasFormat128bit = PF_R32G32B32A32_UINT;
			const bool bAliasTo64bit = InDestFormat == PF_DXT1 || InDestFormat == PF_BC4;
			const EPixelFormat AliasFormat = bAliasTo64bit ? AliasFormat64bit : AliasFormat128bit;
				
			const bool bDirectAliasing = GRHISupportsUAVFormatAliasing;
			if (bDirectAliasing)
			{
				CurrentOutput = GraphBuilder.RegisterExternalTexture(InDestRenderTarget, ERDGTextureFlags::None);
			}
			else
			{
				const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(FIntPoint(FinalTexelCountX, FinalTexelCountY) / 4, AliasFormat, FClearValueBinding::None, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV));
				CurrentOutput = GraphBuilder.CreateTexture(Desc, TEXT("VirtualTextureAdapter.Compress"));
			}

			FCopyCompressCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyCompressCS::FParameters>();
			Parameters->SourceTextureA = bUseSourceTextureA ? InSourceSRV : nullptr;
			Parameters->SourceTextureB = bUseSourceTextureA ? nullptr : GraphBuilder.CreateSRV(FRDGTextureSRVDesc(LastOutput));
			Parameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->DestCompressTexture_64bit = bAliasTo64bit ? GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CurrentOutput, 0, AliasFormat64bit)) : nullptr;
			Parameters->DestCompressTexture_128bit = bAliasTo64bit ? nullptr : GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CurrentOutput, 0, AliasFormat128bit));
			Parameters->SourceUV = CurrentUVRange.Min;
			Parameters->TexelSize = (CurrentUVRange.Max - CurrentUVRange.Min) / FVector2f(FinalTexelCountX, FinalTexelCountY);
			Parameters->TexelOffsets = FVector2f(4.0f, 0.5f);
			Parameters->DestRect = bDirectAliasing ? FIntVector4(InDestRect.Min.X, InDestRect.Min.Y, InDestRect.Max.X, InDestRect.Max.Y) / 4 : FIntVector4(0, 0, FinalTexelCountX, FinalTexelCountY);

			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(FIntPoint(FinalTexelCountX, FinalTexelCountY) / 4, FCopyCompressCS::GroupSize);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("VirtualTextureAdapterCompress"),
				Shader, Parameters, GroupCount);

			bIsDone = bDirectAliasing;
		}

		// Final copy if we didn't already directly write to the physical texture.
		if (!bIsDone)
		{
			FRDGTextureRef LastOutput = CurrentOutput;
			CurrentOutput = GraphBuilder.RegisterExternalTexture(InDestRenderTarget, ERDGTextureFlags::None);

			FRHICopyTextureInfo CopyInfo;
			CopyInfo.DestPosition = FIntVector(InDestRect.Min.X, InDestRect.Min.Y, 0);
			CopyInfo.Size = FIntVector(FinalTexelCountX, FinalTexelCountY, 0);

			const bool bUseSourceTextureA = LastOutput == nullptr;
			if (bUseSourceTextureA)
			{
				// Copying directly from the source.
				FIntPoint SourceTextureSize = InSourceSRV->GetTexture()->GetDesc().Extent;
				uint32 SourceTextureMip = InSourceSRV->GetDesc().Texture.SRV.MipRange.First;
				int32 SourcePositionX = FMath::FloorToInt(InUVRange.Min.X * (float)((uint32)SourceTextureSize.X >> SourceTextureMip));
				int32 SourcePositionY = FMath::FloorToInt(InUVRange.Min.Y * (float)((uint32)SourceTextureSize.Y >> SourceTextureMip));
				CopyInfo.SourcePosition = FIntVector(SourcePositionX, SourcePositionY, 0);
				CopyInfo.SourceMipIndex = SourceTextureMip;
			}

			if (bUseCompressionStep)
			{
				// Take aliased format size difference into account.
				CopyInfo.SourcePosition = CopyInfo.SourcePosition / 4;
				CopyInfo.Size = CopyInfo.Size / 4;
			}

			FCopyToOutputParameters* Parameters = GraphBuilder.AllocParameters<FCopyToOutputParameters>();
			Parameters->Input = LastOutput;
			Parameters->Output = CurrentOutput;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("VirtualTextureAdapterCopyToOutput"),
				Parameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[bUseSourceTextureA, InputTextureA = InSourceSRV, InputTextureB = LastOutput, OutputTexture = CurrentOutput, CopyInfo](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FRHITexture* InputTexture = bUseSourceTextureA ? InputTextureA->GetTexture() : InputTextureB->GetRHI();
				RHICmdList.CopyTexture(InputTexture, OutputTexture->GetRHI(), CopyInfo);
			});
 		}
	}
}

/** IVirtualTextureFinalizer implementation that renders the virtual texture pages on demand. */
class FVirtualTextureAdapterFinalizer : public IVirtualTextureFinalizer
{
public:
	FVirtualTextureAdapterFinalizer(FRHICommandListBase& RHICmdList, FRHITexture* InSourceTexture, FVTProducerDescription const& InProducerDesc)
		: SourceTexture(InSourceTexture)
		, ProducerDesc(InProducerDesc)
	{
		SourceFormat = SourceTexture->GetDesc().Format;
		DestFormat = ProducerDesc.LayerFormat[0];
		IntermediateFormat = VirtualTextureAdapter::GetIntermediateFormat(SourceFormat, DestFormat);

		// Store off SRVs which we use to access individual source mips.
		SourceSRVs.Reserve(SourceTexture->GetNumMips());
		for (uint32 MipIndex = 0; MipIndex < SourceTexture->GetNumMips(); ++MipIndex)
		{
			SourceSRVs.Add(RHICmdList.CreateShaderResourceView(
				SourceTexture,
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SourceTexture)
					.SetMipRange(MipIndex, 1)));
		}
	}

	virtual ~FVirtualTextureAdapterFinalizer() = default;

	/** A description for a single tile to render. */
	struct FTileEntry
	{
		FVTProduceTargetLayer Target;
		uint64 vAddress = 0;
		uint8 vLevel = 0;
	};

	void AddTile(FTileEntry const& InEntry)
	{
		TilesToRender.Add(InEntry);
	}

protected:
	/** Source RHI texture to copy into virtual texture pages. */
	FRHITexture* SourceTexture = nullptr;
	/** Producer description of our virtual texture. */
	const FVTProducerDescription ProducerDesc;
	/** Format of SourceTexture. */
	EPixelFormat SourceFormat;
	/** Destination format for tile generation. */
	EPixelFormat DestFormat;
	/** Pixel format used for intermedite downsample buffers. */
	EPixelFormat IntermediateFormat;
	/** SRVs of source RHI texture to copy into virtual texture pages. */
	TArray<FShaderResourceViewRHIRef> SourceSRVs;
	/** Array of tiles in the queue to finalize. */
	TArray<FTileEntry> TilesToRender;

	//~ Begin IVirtualTextureFinalizer Interface.
	virtual void Finalize(FRDGBuilder& GraphBuilder) override 
	{
		for (FTileEntry const& Tile : TilesToRender)
		{
			IPooledRenderTarget* DestRenderTarget = Tile.Target.PooledRenderTarget;

			const float X = (float)FMath::ReverseMortonCode2_64(Tile.vAddress);
			const float Y = (float)FMath::ReverseMortonCode2_64(Tile.vAddress >> 1);
			const float DivisorX = (float)ProducerDesc.BlockWidthInTiles / (float)(1 << Tile.vLevel);
			const float DivisorY = (float)ProducerDesc.BlockHeightInTiles / (float)(1 << Tile.vLevel);

			const FVector2f UV(X / DivisorX, Y / DivisorY);
			const FVector2f UVSize(1.f / DivisorX, 1.f / DivisorY);
			const FVector2f UVBorder = UVSize * ((float)ProducerDesc.TileBorderSize / (float)ProducerDesc.TileSize);
			const FBox2f UVRange(UV - UVBorder, UV + UVSize + UVBorder);

			const int32 TileSize = ProducerDesc.TileSize + 2 * ProducerDesc.TileBorderSize;
			const FIntPoint DestinationPos(Tile.Target.pPageLocation.X * TileSize, Tile.Target.pPageLocation.Y * TileSize);
			const FIntRect DestRect(DestinationPos, DestinationPos + FIntPoint(TileSize, TileSize));
		
			const bool bHasSRVForLevel = SourceSRVs.IsValidIndex(Tile.vLevel);
			const int32 SRVIndex = bHasSRVForLevel ? Tile.vLevel : 0;
			const int32 vLevel = bHasSRVForLevel ? 0 : Tile.vLevel;

			VirtualTextureAdapter::RenderTile(GraphBuilder, SourceSRVs[SRVIndex], DestRenderTarget, SourceFormat, IntermediateFormat, DestFormat, vLevel, UVRange, DestRect);
		}

		TilesToRender.Reset();
	}
	//~ End IVirtualTextureFinalizer Interface.
};

/** IVirtualTexture implementation that is handling runtime rendered page data requests. */
class FVirtualTextureAdapterProducer : public IVirtualTexture
{
public:
	FVirtualTextureAdapterProducer(FRHICommandListBase& RHICmdList, FRHITexture* InSourceTexture, int32 InUnstreamedMipCount, FVTProducerDescription const& InProducerDesc)
		: Finalizer(RHICmdList, InSourceTexture, InProducerDesc)
		, UnstreamedMipCount(InUnstreamedMipCount)
	{}

	virtual ~FVirtualTextureAdapterProducer() = default;

	//~ Begin IVirtualTexture Interface.
	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
	{
		return false;
	}

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override
	{
		EVTRequestPageStatus Status = vLevel < UnstreamedMipCount ? EVTRequestPageStatus::Invalid : EVTRequestPageStatus::Available;
		return FVTRequestPageResult(Status, 0u);
	}

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override
	{
		FVirtualTextureAdapterFinalizer::FTileEntry Tile;
		Tile.Target = TargetLayers[0];
		Tile.vAddress = vAddress;
		Tile.vLevel = vLevel;
		Finalizer.AddTile(Tile);
		return &Finalizer;
	}
	//~ End IVirtualTexture Interface.

private:
	/** The Finalizer object to write the virtual texture pages. */
	FVirtualTextureAdapterFinalizer Finalizer;
	/** The number of mips that are not streamed which is also the number of mips that we can't produce. */
	int32 UnstreamedMipCount = 0;

};

/** Implementation of FVirtualTextureResource that instantiates an FVirtualTextureAdapterProducer. */
class FVirtualTextureAdapterRenderResource : public FVirtualTexture2DResource
{
	FTextureResource* SourceResource = nullptr;
	EPixelFormat Format = PF_Unknown;
	uint32 TileSize = 0;
	uint32 TileBorderSize = 0;
	uint32 NumTilesX = 0;
	uint32 NumTilesY = 0;
	uint32 MaxLevel = 0;
	uint32 NumSourceMips = 1;

public:
	FVirtualTextureAdapterRenderResource(UVirtualTextureAdapter const* InOwner, UTexture* InTexture, int32 InTileSize, int32 InTileBorderSize, EPixelFormat InFinalPixelFormat, bool bInRequiresSinglePhysicalPool)
	{
		TextureName = InOwner->GetFName();
		PackageName = InOwner->GetOutermost()->GetFName();

		SourceResource = InTexture->GetResource();

		EPixelFormat SourcePixelFormat = PF_Unknown;
		if (UTexture2D* Texture2D = Cast<UTexture2D>(InTexture))
		{
			SourcePixelFormat = Texture2D->GetPixelFormat(0);
			NumSourceMips = Texture2D->GetNumMips();
			bSRGB = Texture2D->SRGB;
		}
		else if (UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(InTexture))
		{
			SourcePixelFormat = RenderTarget2D->GetFormat();
			bSRGB = RenderTarget2D->SRGB;
		}
		Format = VirtualTextureAdapter::GetFinalFormat(SourcePixelFormat, InFinalPixelFormat);
		bRequiresSinglePhysicalPool = bInRequiresSinglePhysicalPool;

		TileSize = InTileSize;
		TileBorderSize = InTileBorderSize;
		NumTilesX = FMath::DivideAndRoundUp((uint32)InTexture->GetSurfaceWidth(), TileSize);
		NumTilesY = FMath::DivideAndRoundUp((uint32)InTexture->GetSurfaceHeight(), TileSize);
		MaxLevel = FMath::CeilLogTwo(FMath::Max(NumTilesX, NumTilesY));
	}

	//~ Begin FVirtualTexture2DResource Interface.
	virtual uint32 GetNumLayers() const override { return 1; }
	virtual EPixelFormat GetFormat(uint32 LayerIndex) const override { return Format; }
	virtual uint32 GetTileSize() const override { return TileSize; }
	virtual uint32 GetBorderSize() const override { return TileBorderSize; }
	virtual uint32 GetNumTilesX() const override { return NumTilesX; }
	virtual uint32 GetNumTilesY() const override { return NumTilesY; }
	virtual uint32 GetNumMips() const override { return MaxLevel + 1; }
	virtual FIntPoint GetSizeInBlocks() const override { return 1; }
	//~ End FVirtualTexture2DResource Interface.

	//~ Begin FRenderResource Interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureAdapterRenderResource::InitRHI);

		FSamplerStateInitializerRHI SamplerStateInitializer;
		SamplerStateInitializer.Filter = SF_Bilinear;
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		const int32 UnstreamedMipCount = NumSourceMips - SourceResource->GetCurrentMipCount();

		FVTProducerDescription ProducerDesc;
		ProducerDesc.Name = TextureName;
		ProducerDesc.FullNameHash = GetTypeHash(TextureName);
		ProducerDesc.bContinuousUpdate = false;
		ProducerDesc.bRequiresSinglePhysicalPool = bRequiresSinglePhysicalPool;
		ProducerDesc.Dimensions = 2;
		ProducerDesc.TileSize = TileSize;
		ProducerDesc.TileBorderSize = TileBorderSize;
		ProducerDesc.BlockWidthInTiles = NumTilesX;
		ProducerDesc.BlockHeightInTiles = NumTilesY;
		ProducerDesc.DepthInTiles = 1u;
		ProducerDesc.MaxLevel = MaxLevel;
		ProducerDesc.NumTextureLayers = 1;
		ProducerDesc.NumPhysicalGroups = 1;
		ProducerDesc.LayerFormat[0] = Format;
		// TODO [jonathan.bard] : For now, let's use the normal priority, but that could come from the texture asset and/or the texture group too : 
		ProducerDesc.Priority = EVTProducerPriority::Normal;

		FVirtualTextureAdapterProducer* VirtualTexture = new FVirtualTextureAdapterProducer(RHICmdList, SourceResource->GetTexture2DRHI(), UnstreamedMipCount, ProducerDesc);
		ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(RHICmdList, ProducerDesc, VirtualTexture);
	}
	//~ End FRenderResource Interface.
};

UVirtualTextureAdapter::UVirtualTextureAdapter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FTextureResource* UVirtualTextureAdapter::CreateResource()
{
	// Only support 2D texture or render target.
	if (Cast<UTexture2D>(Texture) == nullptr && Cast<UTextureRenderTarget2D>(Texture) == nullptr)
	{
		return nullptr;
	}
	// Can only wrap regular textures.
	if (Texture->VirtualTextureStreaming)
	{
		return nullptr;
	}

	FVirtualTextureBuildSettings DefaultSettings;
	DefaultSettings.Init();

	const uint32 FinalTileSize = bUseDefaultTileSizes ? DefaultSettings.TileSize : FVirtualTextureBuildSettings::ClampAndAlignTileSize(TileSize);
	const uint32 FinalTileBorderSize = bUseDefaultTileSizes ? DefaultSettings.TileBorderSize : FVirtualTextureBuildSettings::ClampAndAlignTileBorderSize(TileBorderSize);
	
	const EPixelFormat FinalPixelFormat = OverrideWithTextureFormat ? OverrideWithTextureFormat->GetPixelFormat(0) : PF_Unknown;
	const bool bRequiresSinglePhysicalPool = OverrideWithTextureFormat ? OverrideWithTextureFormat->IsVirtualTexturedWithSinglePhysicalPool() : false;

	return new FVirtualTextureAdapterRenderResource(this, Texture, FinalTileSize, FinalTileBorderSize, FinalPixelFormat, bRequiresSinglePhysicalPool);
}


ETextureClass UVirtualTextureAdapter::GetTextureClass() const
{
	return ETextureClass::TwoD;
}

EMaterialValueType UVirtualTextureAdapter::GetMaterialType() const
{
	return MCT_TextureVirtual;
}

float UVirtualTextureAdapter::GetSurfaceWidth() const
{
	return Texture ? Texture->GetSurfaceWidth() : 0;
}

float UVirtualTextureAdapter::GetSurfaceHeight() const
{
	return Texture ? Texture->GetSurfaceHeight() : 0;
}

void UVirtualTextureAdapter::Flush(FBox2f const& UVRect)
{
	FTextureResource* Resource = GetResource();
	if (Resource == nullptr)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(VirtualTextureAdapterFlush)([Resource, UVRect](FRHICommandListBase&)
	{
		FVirtualTexture2DResource* VTResource = Resource->GetVirtualTexture2DResource();
		IAllocatedVirtualTexture* AllocatedVT = VTResource ? VTResource->AcquireAllocatedVT() : nullptr;

		if (AllocatedVT != nullptr)
		{
			GetRendererModule().FlushVirtualTextureCache(AllocatedVT, UVRect.Min, UVRect.Max);
		}
	});
}
