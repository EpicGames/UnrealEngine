// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureBuildSettings.h"

#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureBuildSettings)

static TAutoConsoleVariable<int32> CVarVTTileSize(
	TEXT("r.VT.TileSize"),
	128,
	TEXT("Size in pixels to use for virtual texture tiles (rounded to next power-of-2)")
);

static TAutoConsoleVariable<int32> CVarVTTileBorderSize(
	TEXT("r.VT.TileBorderSize"),
	4,
	TEXT("Size in pixels to use for virtual texture tiles borders (rounded to multiple-of-2)")
);

int32 FVirtualTextureBuildSettings::ClampAndAlignTileSize(int32 InTileSize)
{
	return FMath::RoundUpToPowerOfTwo(FMath::Clamp<uint32>(InTileSize, 16u, 1024u));
}

int32 FVirtualTextureBuildSettings::ClampAndAlignTileBorderSize(int32 InTileBorderSize)
{
	// Round tile border size up to multiple of 2 to ensure that block compressed formats will be OK.
	return (FMath::Clamp<uint32>(InTileBorderSize, 0u, 8u) + 1u) & ~1u;
}

void FVirtualTextureBuildSettings::Init()
{
	TileSize = ClampAndAlignTileSize(CVarVTTileSize.GetValueOnAnyThread());
	TileBorderSize = ClampAndAlignTileBorderSize(CVarVTTileBorderSize.GetValueOnAnyThread());
}
