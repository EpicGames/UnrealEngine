// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureBuilder.h"

#include "RenderUtils.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "TextureCompiler.h"
#include "UObject/UObjectIterator.h"
#include "VT/VirtualTexture.h"

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Misc/ConfigCacheIni.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureBuilder)

UVirtualTextureBuilder::UVirtualTextureBuilder(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
}

UVirtualTextureBuilder::~UVirtualTextureBuilder()
{
}

void UVirtualTextureBuilder::Serialize(FArchive& Ar)
{
#if WITH_EDITOR	
	if (Ar.IsCooking() && Ar.IsSaving())
	{
		UVirtualTexture2D* TextureBackup = Texture;
		UVirtualTexture2D* TextureMobileBackup = TextureMobile;
		
		// Clear Texture during cook for target platforms that don't support virtual texturing.
		if (!UseVirtualTexturing(Ar.CookingTarget()))
		{
			Texture = nullptr;
			TextureMobile = nullptr;
		}

		int32 OverrideSVTTexture = Ar.CookingTarget()->GetConfigSystem()->GetIntOrDefault(TEXT("/Script/Engine.RendererSettings"), TEXT("OverrideSVTTexture"), -1, GEngineIni);

		if (OverrideSVTTexture >= 0)
		{
			if (OverrideSVTTexture == 0)
			{
				TextureMobile = Texture;
			}
			else
			{
				Texture = TextureMobile;
			}
		}

		// Selectivly serialize VirtualTexture for platforms that support Deferred or/and Mobile rendering
		if (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::DeferredRendering) && bSeparateTextureForMobile)
		{
			Texture = nullptr;
		}
				
		if (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::MobileRendering))
		{
			TextureMobile = nullptr;
		}

		Super::Serialize(Ar);

		Texture = TextureBackup;
		TextureMobile = TextureMobileBackup;
	}
	else
#endif
	{
		Super::Serialize(Ar);
	}
}

void UVirtualTextureBuilder::PostLoad()
{
	Super::PostLoad();

	// Discard one of the VTs on a cooked platform that support both rendering modes
	if (FPlatformProperties::RequiresCookedData())
	{
		if (GetFeatureLevelShadingPath(GMaxRHIFeatureLevel) == EShadingPath::Mobile && bSeparateTextureForMobile)
		{
			Texture = nullptr;
		}
		else
		{
			TextureMobile = nullptr;
		}
	}
}

UVirtualTexture2D* UVirtualTextureBuilder::GetVirtualTexture(EShadingPath ShadingPath) const
{
	if (ShadingPath == EShadingPath::Mobile && bSeparateTextureForMobile)
	{
		return TextureMobile;
	}

	return Texture;
}

#if WITH_EDITOR

static void BuildVirtualTexture2D(UVirtualTexture2D* Texture, FVirtualTextureBuildDesc const& BuildDesc, bool bWaitForCompilation)
{
	Texture->VirtualTextureStreaming = true;
	Texture->LODGroup = BuildDesc.LODGroup;

	Texture->Settings.Init();
	Texture->Settings.TileSize = BuildDesc.TileSize;
	Texture->Settings.TileBorderSize = BuildDesc.TileBorderSize;
	Texture->LossyCompressionAmount = BuildDesc.LossyCompressionAmount;

	Texture->bSinglePhysicalSpace = BuildDesc.bSinglePhysicalSpace;

	for (int32 Layer = 0; Layer < BuildDesc.LayerCount; Layer++)
	{
		Texture->SetLayerFormatSettings(Layer, BuildDesc.LayerFormatSettings[Layer]);
	}

	Texture->Source.InitLayered(BuildDesc.InSizeX, BuildDesc.InSizeY, 1, BuildDesc.LayerCount, 1, BuildDesc.LayerFormats.GetData(), BuildDesc.InData);
	Texture->PostEditChange();

	if (bWaitForCompilation)
	{
		FTextureCompilingManager::Get().FinishCompilation({ Texture });
	}
}


void UVirtualTextureBuilder::BuildTexture(EShadingPath ShadingPath, FVirtualTextureBuildDesc const& BuildDesc, bool bWaitForCompilation)
{
	if (!bSeparateTextureForMobile)
	{
		// Always clear mobile specific VT if property is switched off
		TextureMobile = nullptr;
	}
	
	if (ShadingPath == EShadingPath::Mobile)
	{
		if (!bSeparateTextureForMobile)
		{
			return;
		}
		
		BuildHash = BuildDesc.BuildHash;
		TextureMobile = NewObject<UVirtualTexture2D>(this, TEXT("TextureMobile"));
		BuildVirtualTexture2D(TextureMobile, BuildDesc, bWaitForCompilation);
	}
	else
	{
		BuildHash = BuildDesc.BuildHash;
		Texture = NewObject<UVirtualTexture2D>(this, TEXT("Texture"));
		BuildVirtualTexture2D(Texture, BuildDesc, bWaitForCompilation);
	}

	// Reinit associated runtime virtual textures.
	for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->GetStreamingTexture() == this)
		{
			It->MarkRenderStateDirty();
		}
	}
}

#endif

