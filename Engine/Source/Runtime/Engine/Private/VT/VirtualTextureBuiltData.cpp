// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureBuiltData.h"

#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "Engine/Texture.h"
#include "EngineLogs.h"
#include "Misc/Compression.h"

#if WITH_EDITOR

void FVirtualTextureTileOffsetData::Init(uint32 InWidth, uint32 InHeight)
{
	Width = FMath::Max(InWidth, 1u);
	Height = FMath::Max(InHeight, 1u);

	const uint32 SizePadded = FMath::RoundUpToPowerOfTwo(FMath::Max(Width, Height));
	MaxAddress = SizePadded * SizePadded;

	Addresses.Empty();
	Offsets.Empty();
	TileStates.Empty(MaxAddress);
	TileStates.Add(false, MaxAddress);
}

void FVirtualTextureTileOffsetData::AddTile(uint32 InAddress)
{
	TileStates[InAddress] = true;
}

void FVirtualTextureTileOffsetData::Finalize()
{
	uint32 Offset = 0;
	uint32 StartAddress = 0;
	bool bCurrentState = TileStates[0];

	Addresses.Add(StartAddress);
	Offsets.Add(bCurrentState ? Offset : ~0u);

	for (uint32 Address = 1; Address < MaxAddress; ++Address)
	{
		const bool bState = TileStates[Address];
		if (bState != bCurrentState)
		{
			Offset += bCurrentState ? Address - StartAddress : 0;

			StartAddress = Address;
			bCurrentState = bState;

			Addresses.Add(StartAddress);
			Offsets.Add(bCurrentState ? Offset : ~0u);
		}
	}

	TileStates.Empty();
}

#endif // WITH_EDITOR

uint32 FVirtualTextureTileOffsetData::GetTileOffset(uint32 InAddress) const
{
	const int32 BlockIndex = Algo::UpperBound(Addresses, InAddress) - 1;
	const uint32 BaseOffset = Offsets[BlockIndex];
	if (BaseOffset == ~0u)
	{
		// Address is in empty space.
		return ~0u;
	}
	const uint32 BaseAddress = Addresses[BlockIndex];
	const uint32 LocalOffset = InAddress - BaseAddress;
	return BaseOffset + LocalOffset;
}

uint64 FVirtualTextureBuiltData::GetDiskMemoryFootprint() const
{
	uint64 result = 0;
	for (int32 ChunkId = 0; ChunkId < Chunks.Num(); ChunkId++)
	{
		result += Chunks[ChunkId].SizeInBytes;
	}
	return result;
}

uint32 FVirtualTextureBuiltData::GetMemoryFootprint() const
{
	uint32 TotalSize = sizeof(*this);

	TotalSize += Chunks.GetAllocatedSize();
	for (const FVirtualTextureDataChunk& Chunk : Chunks)
	{
		TotalSize += Chunk.GetMemoryFootprint();
	}

	TotalSize += GetTileMemoryFootprint();

	return TotalSize;
}

uint32 FVirtualTextureBuiltData::GetTileMemoryFootprint() const
{
	uint64 TotalSize = 0;

	// Legacy tile offsets are used if tiles are compressed.
	TotalSize += TileOffsetInChunk.GetAllocatedSize();
	TotalSize += TileIndexPerChunk.GetAllocatedSize();
	TotalSize += TileIndexPerMip.GetAllocatedSize();

	// Implicit tile offsets are used if tiles are uncompressed.
	TotalSize += ChunkIndexPerMip.GetAllocatedSize();
	TotalSize += BaseOffsetPerMip.GetAllocatedSize();
	for (FVirtualTextureTileOffsetData const& Data : TileOffsetData)
	{
		TotalSize += Data.Addresses.GetAllocatedSize();
		TotalSize += Data.Offsets.GetAllocatedSize();
	}

	return IntCastChecked<uint32>( TotalSize );
}

uint32 FVirtualTextureBuiltData::GetNumTileHeaders() const
{
	return TileOffsetInChunk.Num();
}

bool FVirtualTextureBuiltData::IsLegacyData() const
{
	return TileOffsetInChunk.Num() > 0;
}

uint32 FVirtualTextureBuiltData::GetTileIndex_Legacy(uint8 vLevel, uint32 vAddress) const
{
	check(vLevel < NumMips);
	const uint32 TileIndex = TileIndexPerMip[vLevel] + vAddress * NumLayers;
	if (TileIndex >= TileIndexPerMip[vLevel + 1])
	{
		// vAddress is out of bounds for this texture/mip level
		return ~0u;
	}
	return TileIndex;
}

uint32 FVirtualTextureBuiltData::GetTileOffset_Legacy(uint32 ChunkIndex, uint32 TileIndex) const
{
	check(TileIndex >= TileIndexPerChunk[ChunkIndex]);
	if (TileIndex < TileIndexPerChunk[ChunkIndex + 1])
	{
		return TileOffsetInChunk[TileIndex];
	}

	// If TileIndex is past the end of chunk, return the size of chunk
	// This allows us to determine size of region by asking for start/end offsets
	return Chunks[ChunkIndex].SizeInBytes;
}

bool FVirtualTextureBuiltData::IsValidAddress(uint32 vLevel, uint32 vAddress)
{
	bool bIsValid = false;

	if (IsLegacyData())
	{
		bIsValid = GetTileIndex_Legacy(vLevel, vAddress) != ~0u;
	}
	else
	{
		if (TileOffsetData.IsValidIndex(vLevel))
		{
			const uint32 X = FMath::ReverseMortonCode2(vAddress);
			const uint32 Y = FMath::ReverseMortonCode2(vAddress >> 1);
			bIsValid = X < TileOffsetData[vLevel].Width && Y < TileOffsetData[vLevel].Height;
		}
	}

	return bIsValid;
}

int32 FVirtualTextureBuiltData::GetChunkIndex(uint8 vLevel) const
{
	return ChunkIndexPerMip.IsValidIndex(vLevel) ? ChunkIndexPerMip[vLevel] : -1;
}

uint32 FVirtualTextureBuiltData::GetTileOffset(uint32 vLevel, uint32 vAddress, uint32 LayerIndex) const
{
	uint32 Offset = ~0u;

	if (IsLegacyData())
	{
		const uint32 TileIndex = GetTileIndex_Legacy(vLevel, vAddress);
		if (TileIndex != ~0u)
		{
			// If size of the tile is 0 we return ~0u to indicate that there is no data present.
			const int32 ChunkIndex = GetChunkIndex(vLevel);
			const uint32 TileOffset = GetTileOffset_Legacy(ChunkIndex, TileIndex);
			const uint32 NextTileOffset = GetTileOffset_Legacy(ChunkIndex, TileIndex + GetNumLayers());
			if (TileOffset != NextTileOffset)
			{
				Offset = GetTileOffset_Legacy(ChunkIndex, TileIndex + LayerIndex);
			}
		}
	}
	else
	{
		if (BaseOffsetPerMip.IsValidIndex(vLevel) && TileOffsetData.IsValidIndex(vLevel))
		{
			// If the tile offset is ~0u there is no data present so we return ~0u to indicate that.
			const uint32 BaseOffset = BaseOffsetPerMip[vLevel];
			const uint32 TileOffset = TileOffsetData[vLevel].GetTileOffset(vAddress);
			if (BaseOffset != ~0u && TileOffset != ~0u)
			{
				const uint32 TileDataSize = TileDataOffsetPerLayer.Last();
				const uint32 LayerDataOffset = LayerIndex == 0 ? 0 : TileDataOffsetPerLayer[LayerIndex - 1];
				
				int64 Offset64 = BaseOffset + (int64) TileOffset * TileDataSize + LayerDataOffset;
				Offset = IntCastChecked<uint32>( Offset64 );
			}
		}
	}

	return Offset;
}

void FVirtualTextureBuiltData::Serialize(FArchive& Ar, UObject* Owner, int32 FirstMipToSerialize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureDataChunk::Serialize);

	check(FirstMipToSerialize == 0 || Ar.IsSaving());
	const bool bStripMips = (FirstMipToSerialize > 0);
	uint32 NumChunksToStrip = 0u;

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	Ar << NumLayers;
	Ar << WidthInBlocks;
	Ar << HeightInBlocks;
	Ar << TileSize;
	Ar << TileBorderSize;
	Ar << TileDataOffsetPerLayer;

	if (!bStripMips)
	{
		Ar << NumMips;
		Ar << Width;
		Ar << Height;
		Ar << ChunkIndexPerMip;
		Ar << BaseOffsetPerMip;
		Ar << TileOffsetData;
		Ar << TileIndexPerChunk;
		Ar << TileIndexPerMip;
		Ar << TileOffsetInChunk;
	}
	else
	{
		check((uint32)FirstMipToSerialize < NumMips);
		NumChunksToStrip = ChunkIndexPerMip[FirstMipToSerialize];

		uint32 NumMipsToSerialize = NumMips - FirstMipToSerialize;
		uint32 WidthToSerialize = Width >> FirstMipToSerialize;
		uint32 HeightToSerialize = Height >> FirstMipToSerialize;

		TArray<uint32> StrippedChunkIndexPerMip;
		TArray<uint32> StrippedBaseOffsetPerMip;
		TArray<FVirtualTextureTileOffsetData> StrippedTileOffsetData;
		TArray<uint32> StrippedTileIndexPerChunk;
		TArray<uint32> StrippedTileIndexPerMip;
		TArray<uint32> StrippedTileOffsetInChunk;

		StrippedChunkIndexPerMip.Reserve(ChunkIndexPerMip.Num() - FirstMipToSerialize);
		for (int32 i = FirstMipToSerialize; i < ChunkIndexPerMip.Num(); ++i)
		{
			check(ChunkIndexPerMip[i] >= NumChunksToStrip);
			StrippedChunkIndexPerMip.Add(ChunkIndexPerMip[i] - NumChunksToStrip);
		}
		
		StrippedBaseOffsetPerMip.Reserve(BaseOffsetPerMip.Num() - FirstMipToSerialize);
		for (int32 i = FirstMipToSerialize; i < BaseOffsetPerMip.Num(); ++i)
		{
			StrippedBaseOffsetPerMip.Add(BaseOffsetPerMip[i]);
		}

		StrippedTileOffsetData.Reserve(TileOffsetData.Num() - FirstMipToSerialize);
		for (int32 i = FirstMipToSerialize; i < TileOffsetData.Num(); ++i)
		{
			StrippedTileOffsetData.Add(TileOffsetData[i]);
		}

		const bool bHasLegacyData = TileOffsetInChunk.Num() > 0;
		if (bHasLegacyData)
		{
			const uint32 NumTilesToStrip = TileIndexPerMip[FirstMipToSerialize];
			check(NumTilesToStrip < (uint32)TileOffsetInChunk.Num());

			StrippedTileIndexPerChunk.Reserve(TileIndexPerChunk.Num() - NumChunksToStrip);
			for (int32 i = NumChunksToStrip; i < TileIndexPerChunk.Num(); ++i)
			{
				// Since we can only exclude data by chunk, it's possible that the first chunk we need to include will contain some initial tiles from a mip that's been excluded
				StrippedTileIndexPerChunk.Add(TileIndexPerChunk[i] - FMath::Min(NumTilesToStrip, TileIndexPerChunk[i]));
			}

			StrippedTileIndexPerMip.Reserve(TileIndexPerMip.Num() - FirstMipToSerialize);
			for (int32 i = FirstMipToSerialize; i < TileIndexPerMip.Num(); ++i)
			{
				check(TileIndexPerMip[i] >= NumTilesToStrip);
				StrippedTileIndexPerMip.Add(TileIndexPerMip[i] - NumTilesToStrip);
			}

			StrippedTileOffsetInChunk.Reserve(TileOffsetInChunk.Num() - NumTilesToStrip);
			for (int32 i = NumTilesToStrip; i < TileOffsetInChunk.Num(); ++i)
			{
				// offsets within each chunk are unchanged...we are removing chunks that are no longer referenced, but not truncating any existing chunks
				StrippedTileOffsetInChunk.Add(TileOffsetInChunk[i]);
			}
		}

		Ar << NumMipsToSerialize;
		Ar << WidthToSerialize;
		Ar << HeightToSerialize;
		Ar << StrippedChunkIndexPerMip;
		Ar << StrippedBaseOffsetPerMip;
		Ar << StrippedTileOffsetData;
		Ar << StrippedTileIndexPerChunk;
		Ar << StrippedTileIndexPerMip;
		Ar << StrippedTileOffsetInChunk;
	}

	// Serialize the layer pixel formats.
	// Pixel formats are serialized as strings to protect against enum changes
	const UEnum* PixelFormatEnum = UTexture::GetPixelFormatEnum();
	if (Ar.IsLoading())
	{
		checkf(NumLayers <= VIRTUALTEXTURE_DATA_MAXLAYERS, TEXT("Trying to load FVirtualTextureBuiltData with %d layers, only %d layers supported"),
			NumLayers, VIRTUALTEXTURE_DATA_MAXLAYERS);
		for (uint32 Layer = 0; Layer < NumLayers; Layer++)
		{
			FString PixelFormatString;
			Ar << PixelFormatString;
			LayerTypes[Layer] = (EPixelFormat)PixelFormatEnum->GetValueByName(*PixelFormatString);
		}
	}
	else if (Ar.IsSaving())
	{
		for (uint32 Layer = 0; Layer < NumLayers; Layer++)
		{
			FString PixelFormatString = PixelFormatEnum->GetNameByValue(LayerTypes[Layer]).GetPlainNameString();
			Ar << PixelFormatString;
		}
	}
	
 	for (uint32 Layer = 0; Layer < NumLayers; Layer++)
 	{
 		Ar << LayerFallbackColors[Layer];
 	}

	// Serialize the chunks
	int32 NumChunksToSerialize = Chunks.Num() - NumChunksToStrip;
	Ar << NumChunksToSerialize;

	if (Ar.IsLoading())
	{
		Chunks.Reserve(NumChunksToSerialize);	// Allocates the exact memory we need as relying on SetNum would
												// result in allocating additional slack.
		Chunks.SetNum(NumChunksToSerialize);
	}

	int32 SerialzeChunkId = 0;
	for (int32 ChunkId = NumChunksToStrip; ChunkId < Chunks.Num(); ChunkId++)
	{
		FVirtualTextureDataChunk& Chunk = Chunks[ChunkId];

		Ar << Chunk.BulkDataHash;
		Ar << Chunk.SizeInBytes;
		Ar << Chunk.CodecPayloadSize;
		for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
		{
			Ar << Chunk.CodecType[LayerIndex];
			Ar << Chunk.CodecPayloadOffset[LayerIndex];
		}

		Chunk.BulkData.Serialize(Ar, Owner, SerialzeChunkId, false);

#if WITH_EDITORONLY_DATA
		if (!bCooked)
		{
			Ar << Chunk.DerivedDataKey;
			if (Ar.IsLoading() && !Ar.IsCooking())
			{
				Chunk.ShortenKey(Chunk.DerivedDataKey, Chunk.ShortDerivedDataKey);
			}
		}

		// Streaming chunks are saved with a size of 0 because they are stored separately.
		// IsBulkDataLoaded() returns true for this empty bulk data. Remove the empty
		// bulk data to allow unloaded streaming chunks to be detected.
		if (Chunk.BulkData.GetBulkDataSize() == 0 && !Chunk.DerivedDataKey.IsEmpty())
		{
			Chunk.BulkData.RemoveBulkData();
		}
#endif // WITH_EDITORONLY_DATA

		SerialzeChunkId++;
	}
}

struct FBulkDataLockedScope
{
	~FBulkDataLockedScope()
	{
		if (BulkData)
		{
			BulkData->Unlock();
		}
	}

	const FByteBulkData* BulkData = nullptr;
};

bool FVirtualTextureBuiltData::ValidateData(FStringView const& InDDCDebugContext, bool bValidateCompression) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureBuiltData::ValidateCompression);

	const uint32 TilePixelSize = GetPhysicalTileSize();
	TArray<uint8> UncompressedResult;
	FSharedBuffer ChunkDataDDC;

	const FString TextureName(InDDCDebugContext);

	bool bResult = true;
	for (int32 ChunkIndex = 0; bResult && ChunkIndex < Chunks.Num(); ++ChunkIndex)
	{
		const FVirtualTextureDataChunk& Chunk = Chunks[ChunkIndex];
		FBulkDataLockedScope BulkDataLockedScope;

		const void* ChunkData = nullptr;
		int64 ChunkDataSize = 0;
		if (Chunk.BulkData.GetBulkDataSize() > 0)
		{
			ChunkDataSize = Chunk.BulkData.GetBulkDataSize();
			ChunkData = Chunk.BulkData.LockReadOnly();
			BulkDataLockedScope.BulkData = &Chunk.BulkData;
		}
#if WITH_EDITORONLY_DATA
		else
		{
			ChunkDataDDC.Reset();
			using namespace UE::DerivedData;
			FRequestOwner BlockingOwner(EPriority::Blocking);
			GetCache().GetValue({{{InDDCDebugContext}, ConvertLegacyCacheKey(Chunk.DerivedDataKey)}}, BlockingOwner, [&ChunkDataDDC](FCacheGetValueResponse&& Response)
			{
				ChunkDataDDC = Response.Value.GetData().Decompress();
			});
			BlockingOwner.Wait();

			if (ChunkDataDDC.IsNull())
			{
				UE_LOGF(LogTexture, Log, "Virtual Texture %ls failed to retrieve DDC data (%ls) for chunk %d", *TextureName, *Chunk.DerivedDataKey, ChunkIndex);
				bResult = false;
				break;
			}

			ChunkData = ChunkDataDDC.GetData();
			ChunkDataSize = ChunkDataDDC.GetSize();
		}
#endif // WITH_EDITORONLY_DATA

		if (!ChunkData || ChunkDataSize < sizeof(FVirtualTextureChunkHeader))
		{
			UE_LOGF(LogTexture, Error, "Virtual Texture %ls has invalid size %lld for chunk %d", *TextureName, ChunkDataSize, ChunkIndex);
			bResult = false;
			break;
		}

		FSHAHash Hash;
		FSHA1::HashBuffer(ChunkData, ChunkDataSize, Hash.Hash);
		if (Hash != Chunk.BulkDataHash)
		{
			UE_LOGF(LogTexture, Error, "Virtual Texture %ls has invalid hash for chunk %d", *TextureName, ChunkIndex);
			bResult = false;
			break;
		}

		if (bValidateCompression && IsLegacyData())
		{
			uint32 TileIndex = TileIndexPerChunk[ChunkIndex];
			while (bResult && TileIndex < TileIndexPerChunk[ChunkIndex + 1])
			{
				for (uint32 LayerIndex = 0u; LayerIndex < GetNumLayers(); ++LayerIndex)
				{
					const EVirtualTextureCodec VTCodec = Chunk.CodecType[LayerIndex];
					const EPixelFormat LayerFormat = LayerTypes[LayerIndex];
					const uint32 TileWidthInBlocks = FMath::DivideAndRoundUp(TilePixelSize, (uint32)GPixelFormats[LayerFormat].BlockSizeX);
					const uint32 TileHeightInBlocks = FMath::DivideAndRoundUp(TilePixelSize, (uint32)GPixelFormats[LayerFormat].BlockSizeY);
					const uint32 PackedStride = TileWidthInBlocks * GPixelFormats[LayerFormat].BlockBytes;
					const size_t PackedOutputSize = PackedStride * TileHeightInBlocks;

					if (VTCodec == EVirtualTextureCodec::ZippedGPU_DEPRECATED)
					{
						const uint32 TileOffset = GetTileOffset_Legacy(ChunkIndex, TileIndex);
						const uint32 NextTileOffset = GetTileOffset_Legacy(ChunkIndex, TileIndex + 1);
						check(NextTileOffset >= TileOffset);
						if (NextTileOffset > TileOffset)
						{
							const uint32 CompressedTileSize = NextTileOffset - TileOffset;

							UncompressedResult.SetNumUninitialized(PackedOutputSize, EAllowShrinking::No);
							const bool bUncompressResult = FCompression::UncompressMemory(NAME_Zlib, UncompressedResult.GetData(), PackedOutputSize, static_cast<const uint8*>(ChunkData) + TileOffset, CompressedTileSize);
							if (!bUncompressResult)
							{
								UE_LOGF(LogTexture, Error, "Virtual Texture %ls failed to validate compression for chunk %d", *TextureName, ChunkIndex);
								bResult = false;
								break;
							}
						}
					}
					++TileIndex;
				}
			}
		}
	}

	return bResult;
}

#if WITH_EDITORONLY_DATA

bool FVirtualTextureDataChunk::ShortenKey(const FString& CacheKey, FString& Result)
{
#define MAX_BACKEND_KEY_LENGTH (120)

	Result = FString(CacheKey);
	if (Result.Len() <= MAX_BACKEND_KEY_LENGTH)
	{
		return false;
	}

	FSHA1 HashState;
	int32 Length = Result.Len();
	HashState.Update((const uint8*)&Length, sizeof(int32));

	auto ResultSrc = StringCast<UCS2CHAR>(*Result);
	uint32 CRCofPayload(FCrc::MemCrc32(ResultSrc.Get(), Length * sizeof(UCS2CHAR)));

	HashState.Update((const uint8*)&CRCofPayload, sizeof(uint32));
	HashState.Update((const uint8*)ResultSrc.Get(), Length * sizeof(UCS2CHAR));

	HashState.Final();
	uint8 Hash[FSHA1::DigestSize];
	HashState.GetHash(Hash);
	FString HashString = BytesToHex(Hash, FSHA1::DigestSize);

	int32 HashStringSize = HashString.Len();
	int32 OriginalPart = MAX_BACKEND_KEY_LENGTH - HashStringSize - 2;
	Result = Result.Left(OriginalPart) + TEXT("__") + HashString;
	check(Result.Len() == MAX_BACKEND_KEY_LENGTH && Result.Len() > 0);
	return true;
}

int64 FVirtualTextureDataChunk::StoreInDerivedDataCache(const FStringView InKey, const FStringView InName, const bool bInReplaceExisting)
{
	using namespace UE;
	using namespace UE::DerivedData;

	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureDataChunk::StoreInDerivedDataCache);

	const int64 BulkDataSizeInBytes = BulkData.GetBulkDataSize();
	check(BulkDataSizeInBytes > 0);

	const FSharedString Name = InName;
	const FCacheKey Key = ConvertLegacyCacheKey(InKey);
	FValue Value = FValue::Compress(FSharedBuffer::MakeView(BulkData.Lock(LOCK_READ_ONLY), BulkDataSizeInBytes));
	BulkData.Unlock();

	FRequestOwner AsyncOwner(EPriority::Normal);
	const ECachePolicy Policy = bInReplaceExisting ? ECachePolicy::Store : ECachePolicy::Default;
	GetCache().PutValue({{Name, Key, MoveTemp(Value), Policy}}, AsyncOwner);
	AsyncOwner.KeepAlive();

	DerivedDataKey = InKey;
	ShortenKey(DerivedDataKey, ShortDerivedDataKey);

	DerivedData = FDerivedData(Name, Key);

	// remove the actual bulkdata so when we serialize the owning FVirtualTextureBuiltData, this is actually serializing only the meta data
	BulkData.RemoveBulkData();
	return BulkDataSizeInBytes;
}

#endif // WITH_EDITORONLY_DATA
