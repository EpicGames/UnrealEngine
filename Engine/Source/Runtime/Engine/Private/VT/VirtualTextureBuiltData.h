// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "Serialization/BulkData.h"
#include "Serialization/DerivedData.h"
#include "Engine/Texture.h"
#include "HAL/ThreadSafeBool.h"

// This header contains the all the structs and classes pertaining to the
// virtual texture on-disc file format.

/** Max number of layers that can be stored in a VT asset, may be lower than number of VT layers that can be stored in page table */
#define VIRTUALTEXTURE_DATA_MAXLAYERS 8u

/** Max number of mips that can be stored in a VT asset */
#define VIRTUALTEXTURE_DATA_MAXMIPS 16u

enum class EVirtualTextureCodec : uint8
{
	Black,			//Special case codec, always outputs black pixels 0,0,0,0
	OpaqueBlack,	//Special case codec, always outputs opaque black pixels 0,0,0,255
	White,			//Special case codec, always outputs white pixels 255,255,255,255
	Flat,			//Special case codec, always outputs 128,125,255,255 (flat normal map)
	RawGPU,			//Uncompressed data in an GPU-ready format (e.g R8G8B8A8, BC7, ASTC, ...)
	ZippedGPU_DEPRECATED,		//Same as RawGPU but with the data zipped
	Crunch_DEPRECATED,			//Use the Crunch library to compress data
	Max,			// Add new codecs before this entry
};

struct FVirtualTextureChunkHeader
{
	uint32 Unused; // Chunk header is currently not used, but keeping in place for future expansion
};

struct FVirtualTextureDataChunk
{
	/** Reference to the data for the chunk if it can be streamed. */
	UE::FDerivedData DerivedData;
	/** Stores the data for the chunk when it is loaded. */
	FByteBulkData BulkData;
	FSHAHash BulkDataHash;
	uint32 SizeInBytes;
	uint32 CodecPayloadSize;
	uint32 CodecPayloadOffset[VIRTUALTEXTURE_DATA_MAXLAYERS];
	EVirtualTextureCodec CodecType[VIRTUALTEXTURE_DATA_MAXLAYERS];

	inline FVirtualTextureDataChunk()
		: SizeInBytes(0u)
		, CodecPayloadSize(0u)
	{
		FMemory::Memzero(CodecPayloadOffset);
		FMemory::Memzero(CodecType);
	}

	inline uint32 GetMemoryFootprint() const
	{
		// Don't include editor-only data
		static constexpr uint32 MemoryFootprint = uint32(sizeof(BulkData) + sizeof(SizeInBytes) + sizeof(CodecPayloadOffset) + sizeof(CodecPayloadSize) + sizeof(CodecType));
		return MemoryFootprint;
	}

#if WITH_EDITORONLY_DATA
	/** Key if stored in the derived data cache. */
	FString DerivedDataKey;

	// Cached short key for VT DDC cache (not serialized)
	FString ShortDerivedDataKey;
	bool ShortenKey(const FString& CacheKey, FString& Result);
	FThreadSafeBool bFileAvailableInVTDDCDache;
	bool bCorruptDataLoadedFromDDC = false;
	int64 StoreInDerivedDataCache(FStringView InKey, FStringView InName, bool bReplaceExisting);
#endif // WITH_EDITORONLY_DATA
};


/** 
 * Compact structure to find tile offsets within a sparse chunk. 
 * The tiles are stored in Morton order inside the chunks. But for UDIM textures some areas of a mip level may be empty and so some tiles don't exist.
 * We could store one chunk offset per tile, but that uses a large amount of memory.
 * So we need a fast, but memory efficient way to get from tile index to chunk location.
 * This structure splits the space into contiguous blocks of empty or non-empty tiles.
 * If a block is non-empty it has an offset into the chunk, and if it is empty it has a special "empty" offset.
 * Within a non-empty block, all tiles exist and are contiguous.
 * Because the tiles are in Morton order and UDIMs are aligned on power of 2 boundaries the sequences are not usually very fragmented, so textures can be described with a low number of blocks.
 * Lookup of tile index is done at the cost of a binary search of Morton address through the stored block addresses.
 */
struct FVirtualTextureTileOffsetData
{
#if WITH_EDITOR
	/* Call at start of building the data before any calls to AddTile(). The entire area is initialized as empty. */
	void Init(uint32 InWidth, uint32 InHeight);
	/* Call to mark a tile as non-empty. */
	void AddTile(uint32 InAddress);
	/* Call after all calls to AddTile() to build the final blocks and offsets. */
	void Finalize();
#endif // WITH_EDITOR

	/** Call at runtime to get the final tile offset in the chunk. */
	uint32 GetTileOffset(uint32 InAddress) const;

	/** Serialization helper. */
	friend FArchive& operator<<(FArchive& Ar, FVirtualTextureTileOffsetData& TileOffsetData)
	{
		Ar << TileOffsetData.Width;
		Ar << TileOffsetData.Height;
		Ar << TileOffsetData.MaxAddress;
		Ar << TileOffsetData.Addresses;
		Ar << TileOffsetData.Offsets;
		return Ar;
	}

	uint32 Width = 0;
	uint32 Height = 0;
	/** Upper bound Morton address for managed area. */
	uint32 MaxAddress = 0;
	/** Sorted list of contiguous tile block addresses. */
	TArray<uint32> Addresses;
	/** Offset for each block in Addresses. An empty block is marked with ~0u. */
	TArray<uint32> Offsets;
	
#if WITH_EDITOR
	/** Tile state scratch buffer. Created in Init() and destroyed in Finalize(). */
	TBitArray<> TileStates;
#endif
};


struct FVirtualTextureBuiltData
{
	uint32 NumLayers;
	uint32 NumMips;
	uint32 Width; // Width of the texture in pixels. Note the physical width may be larger due to tiling
	uint32 Height; // Height of the texture in pixels. Note the physical height may be larger due to tiling
	uint32 WidthInBlocks; // Number of UDIM blocks that make up the texture, used to compute UV scaling factor
	uint32 HeightInBlocks;
	uint32 TileSize;  // Tile size excluding borders
	uint32 TileBorderSize; // A BorderSize pixel border will be added around all tiles

	/**
	 * The pixel format output of the data for each layer. The actual data
	 * may still be compressed but will decompress to this pixel format (e.g. zipped DXT5 data).
	 */
	TEnumAsByte<EPixelFormat> LayerTypes[VIRTUALTEXTURE_DATA_MAXLAYERS];

	/**
	 * The fallback color to use for each layer. This color is used whenever we need to sample
	 * the VT but have not yet streamed the root pages.
	 */
	FLinearColor LayerFallbackColors[VIRTUALTEXTURE_DATA_MAXLAYERS];

	/**
	 * Tile data is packed into separate chunks, typically there is 1 mip level in each chunk for high resolution mips.
	 * After a certain threshold, all remaining low resolution mips will be packed into one final chunk.
	 */
	TArray<FVirtualTextureDataChunk> Chunks;

	/** 
	 * Per layer tile data offset in bytes. 
	 * This is empty if compression (zlib etc) is used, since in that case offsets will vary per tile.
	 * Each value is the sum of the packed tile data sizes for the current and the preceding layers.
	 */
	TArray<uint32> TileDataOffsetPerLayer;

	/** Chunk index that contains the mip. */
	TArray<uint32> ChunkIndexPerMip;

	/** Base offset in chunk for the mip. */
	TArray<uint32> BaseOffsetPerMip;

	/** Holds the tile to chunk offset lookup data. One array entry per mip level. */
	TArray<FVirtualTextureTileOffsetData> TileOffsetData;

	/** Index of the first tile within each chunk */
	TArray<uint32> TileIndexPerChunk;

	/** Index of the first tile within each mip level */
	TArray<uint32> TileIndexPerMip;

	/**
	 * Info for the tiles organized per level. Within a level tile info is organized in Morton order.
	 * This is in Morton order which can waste a lot of space in this array for non-square images
	 * e.g.:
	 * - An 8x1 tile image will allocate 8x4 indexes in this array.
	 * - An 1x8 tile image will allocate 8x8 indexes in this array.
	 */
	TArray<uint32> TileOffsetInChunk;

	FVirtualTextureBuiltData()
		: NumLayers(0u)
		, NumMips(0u)
		, Width(0u)
		, Height(0u)
		, WidthInBlocks(0u)
		, HeightInBlocks(0u)
		, TileSize(0u)
		, TileBorderSize(0u)
	{
		FMemory::Memzero(LayerTypes);
		FMemory::Memzero(LayerFallbackColors);
	}

	inline bool IsInitialized() const { return TileSize != 0u; }
	inline uint32 GetNumMips() const { return NumMips; }
	inline uint32 GetNumLayers() const { return NumLayers; }
	inline uint32 GetPhysicalTileSize() const { return TileSize + TileBorderSize * 2u; }
	inline uint32 GetWidthInTiles() const { return FMath::DivideAndRoundUp(Width, TileSize); }
	inline uint32 GetHeightInTiles() const { return FMath::DivideAndRoundUp(Height, TileSize); }

	uint32 GetMemoryFootprint() const;
	uint32 GetTileMemoryFootprint() const;
	uint64 GetDiskMemoryFootprint() const;
	uint32 GetNumTileHeaders() const;
	
	void Serialize(FArchive& Ar, UObject* Owner, int32 FirstMipToSerialize);

	/** Returns false if the address isn't inside the texture bounds. */
	bool IsValidAddress(uint32 vLevel, uint32 vAddress);

	/** Return the index of the chunk that contains the given mip level. Returns -1 if the mip level doesn't exist. */
	int32 GetChunkIndex(uint8 vLevel) const;

	/** Return the byte offset of a tile within a chunk. Returns ~0u if the tile doesn't exist. */
	uint32 GetTileOffset(uint32 vLevel, uint32 vAddress, uint32 LayerIndex) const;

	/**
	* Validates the VT data
	* If bValidateCompression is true, attempts to decompress every VT tile, to verify data is valid. This will be very slow
	*/
	bool ValidateData(FStringView const& InDDCDebugContext, bool bValidateCompression) const;

private:
	bool IsLegacyData() const;
	uint32 GetTileIndex_Legacy(uint8 vLevel, uint32 vAddress) const;
	uint32 GetTileOffset_Legacy(uint32 ChunkIndex, uint32 TileIndex) const;
};
