// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureChunkDDCCache.h"

#if WITH_EDITOR

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Async/AsyncWork.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "VirtualTextureBuiltData.h"
#include "Misc/ConfigCacheIni.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVTDiskCache, Log, All);
DEFINE_LOG_CATEGORY(LogVTDiskCache);

struct FVirtualTextureFileHeader
{
	static const uint32 CurrentMagic = 0x4558ACDF;
	static const uint32 CurrentVersion = 2u;

	uint32 Magic = 0u;
	uint32 Version = 0u;
	uint32 HashSize = 0u;
	FSHAHash Hash;

	friend FArchive& operator<<(FArchive& Ar, FVirtualTextureFileHeader& Ref)
	{
		return Ar << Ref.Magic << Ref.Version << Ref.HashSize << Ref.Hash;
	}
};
static_assert(sizeof(FVirtualTextureFileHeader) == sizeof(uint32) * 3 + sizeof(FSHAHash), "Bad packing");

class FAsyncFillCacheWorker : public FNonAbandonableTask
{
public:
	FString TempFilename;
	FString	FinalFilename;
	FVirtualTextureDataChunk* Chunk;

	FAsyncFillCacheWorker(const FString& InTempFilename, const FString& InFinalFilename, FVirtualTextureDataChunk* InChunk)
		: TempFilename(InTempFilename)
		, FinalFilename(InFinalFilename)
		, Chunk(InChunk)
	{
	}

	void DoWork()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncFillCacheWorker::DoWork);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		// Limit size of valid hash
		static const uint32 kMaxHashSize = 32 * 1024;

		//The file might be resident but this is the first request to it, flag as available
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*FinalFilename, 0));
			if (Ar)
			{
				FVirtualTextureFileHeader Header;
				*Ar << Header;
				if (Header.Magic == FVirtualTextureFileHeader::CurrentMagic &&
					Header.Version == FVirtualTextureFileHeader::CurrentVersion &&
					Header.HashSize <= kMaxHashSize)
				{
					TArray64<uint8> FileContents;
					FileContents.AddUninitialized(Header.HashSize);
					Ar->Serialize(FileContents.GetData(), Header.HashSize);

					FSHAHash FileHash;
					FSHA1::HashBuffer(FileContents.GetData(), FileContents.Num(), FileHash.Hash);
					if (FileHash == Header.Hash)
					{
						Ar.Reset(); // Close the file before marking the chunk as available
						Chunk->bFileAvailableInVTDDCDache = true;
						return;
					}
					else
					{
						UE_LOGF(LogVTDiskCache, Log, "Found invalid existing VT DDC cache %ls, mismatched hash, deleting", *FinalFilename);
					}
				}
				else
				{
					UE_LOGF(LogVTDiskCache, Log, "Found invalid existing VT DDC cache %ls, Magic: %d Version: %d HashSize: %d, deleting", *FinalFilename, Header.Magic, Header.Version, Header.HashSize);
				}

				Ar.Reset();
				const bool bDeleteResult = PlatformFile.DeleteFile(*FinalFilename);
				ensureMsgf(bDeleteResult, TEXT("Failed to delete invalid VT DDC file %s"), *FinalFilename);
			}
		}

		// Fetch data from DDC
		using namespace UE::DerivedData;
		FSharedBuffer Results;
		FRequestOwner BlockingOwner(EPriority::Blocking);
		GetCache().GetValue({{{FinalFilename}, ConvertLegacyCacheKey(Chunk->DerivedDataKey)}}, BlockingOwner, [&Results](FCacheGetValueResponse&& Response)
		{
			Results = Response.Value.GetData().Decompress();
		});
		BlockingOwner.Wait();

		if (Results.IsNull())
		{
			UE_LOGF(LogVTDiskCache, Log, "Failed to fetch data from DDC (key: %ls)", *Chunk->DerivedDataKey);
			return;
		}

		if (Results.GetSize() <= sizeof(FVirtualTextureChunkHeader))
		{
			UE_LOG(LogVTDiskCache, Error, TEXT("VT DDC data is too small %" UINT64_FMT " (key: %s)"), Results.GetSize(), *Chunk->DerivedDataKey);
			return;
		}

		const void* ChunkData = Results.GetData();
		const uint64 ChunkDataSize = Results.GetSize();

		FSHAHash Hash;
		FSHA1::HashBuffer(ChunkData, ChunkDataSize, Hash.Hash);
		if (Hash != Chunk->BulkDataHash)
		{
			UE_LOGF(LogVTDiskCache, Error, "VT DDC data is corrupt (key: %ls)", *Chunk->DerivedDataKey);
			return;
		}

		// Write to Disk
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*TempFilename, 0));
			if (!Ar)
			{
				UE_LOGF(LogVTDiskCache, Error, "Failed to write to %ls", *TempFilename);
				return;
			}

			FVirtualTextureFileHeader Header;
			Header.Magic = FVirtualTextureFileHeader::CurrentMagic;
			Header.Version = FVirtualTextureFileHeader::CurrentVersion;
			Header.HashSize = FMath::Min<uint32>(ChunkDataSize, kMaxHashSize);
			FSHA1::HashBuffer(ChunkData, Header.HashSize, Header.Hash.Hash);
			*Ar << Header;
			const int64 ArchiveOffset = Ar->Tell();
			check(ArchiveOffset == sizeof(FVirtualTextureFileHeader));
			Ar->Serialize(const_cast<void*>(ChunkData), ChunkDataSize);
		}

		if (PlatformFile.MoveFile(*FinalFilename, *TempFilename))
		{
			// File is now available
			Chunk->bFileAvailableInVTDDCDache = true;
		}
		else
		{
			// Failed to move file, was the final file somehow created by a different process?
			PlatformFile.DeleteFile(*TempFilename);
			Chunk->bFileAvailableInVTDDCDache = PlatformFile.FileExists(*FinalFilename);
			check(Chunk->bFileAvailableInVTDDCDache);
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		return TStatId();
	}
};

FVirtualTextureChunkDDCCache* GetVirtualTextureChunkDDCCache()
{
	static FVirtualTextureChunkDDCCache DDCCache;
	return &DDCCache;
}

void FVirtualTextureChunkDDCCache::Initialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureChunkDDCCache::Initialize);

	// setup the cache folder
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	check(GConfig);
	GConfig->GetString(TEXT("VirtualTextureChunkDDCCache"), TEXT("Path"), AbsoluteCachePath, GEngineIni);
	AbsoluteCachePath = FPaths::ConvertRelativePathToFull(AbsoluteCachePath);
	if (PlatformFile.DirectoryExists(*AbsoluteCachePath) == false)
	{
		PlatformFile.CreateDirectoryTree(*AbsoluteCachePath);
	}

	// test if the folder is accessible
	FString TempFilename = AbsoluteCachePath / FGuid::NewGuid().ToString() + ".tmp";
	FFileHelper::SaveStringToFile(FString("TEST"), *TempFilename);
	int32 TestFileSize = IFileManager::Get().FileSize(*TempFilename);
	if (TestFileSize < 4)
	{
		UE_LOGF(LogVTDiskCache, Warning, "Fail to write to %ls, derived data cache to this directory will be read only.", *AbsoluteCachePath);
	}
	if (TestFileSize >= 0)
	{
		IFileManager::Get().Delete(*TempFilename, false, false, true);
	}

	// Delete any old files in the cache directory
	check(GConfig);
	int32 UnusedFileAge = 17;
	GConfig->GetInt(TEXT("VirtualTextureChunkDDCCache"), TEXT("UnusedFileAge"), UnusedFileAge, GEngineIni);
	const FTimespan UnusedFileTime = FTimespan(UnusedFileAge, 0, 0, 0);

	// find all files in the directory
	IFileManager::Get().IterateDirectoryStatRecursively(*AbsoluteCachePath, [UnusedFileTime](const TCHAR* FileName, const FFileStatData& Stat)
	{
		if (!Stat.bIsDirectory && (Stat.AccessTime != FDateTime::MinValue() || Stat.ModificationTime != FDateTime::MinValue()))
		{
			const FTimespan TimeSinceLastAccess = FDateTime::UtcNow() - Stat.AccessTime;
			const FTimespan TimeSinceLastModification = FDateTime::UtcNow() - Stat.ModificationTime;
			if (TimeSinceLastAccess >= UnusedFileTime && TimeSinceLastModification >= UnusedFileTime)
			{
				// Delete the file
				const bool Result = IFileManager::Get().Delete(FileName, false, true, true);
				UE_LOGF(LogVTDiskCache, Log, "Deleted old VT cache file %ls", FileName);
			}
		}

		return true;
	});
}

void FVirtualTextureChunkDDCCache::ShutDown()
{
	WaitFlushRequests_AnyThread();
}

void FVirtualTextureChunkDDCCache::UpdateRequests()
{
	check(IsInRenderingThread());

	// Needs a lock because these arrays are accessed by rare calls to WaitFlushRequests_AnyThread().
	// All other access is render thread.
	FScopeLock Lock(&ActiveTasksLock);

	// Remove completed chunks from array.
	ActiveChunks.RemoveAllSwap([](auto Chunk) -> bool {return Chunk->bFileAvailableInVTDDCDache == true; }, EAllowShrinking::No);

	// Remove completed tasks from array.
	for (int32 TaskIndex = 0; TaskIndex < ActiveTasks.Num(); ++TaskIndex)
	{
		FAsyncTask<FAsyncFillCacheWorker>* Task = ActiveTasks[TaskIndex];
		if (Task->IsWorkDone())
		{
			Task->EnsureCompletion();
			delete Task;
			ActiveTasks.RemoveAtSwap(TaskIndex--, EAllowShrinking::No);
		}
	}
}

void FVirtualTextureChunkDDCCache::WaitFlushRequests_AnyThread()
{
	TArray<FAsyncTask<FAsyncFillCacheWorker>*> ActiveTasksCopy;
	{
		FScopeLock Lock(&ActiveTasksLock);
		ActiveChunks.Empty();
		ActiveTasksCopy = MoveTemp(ActiveTasks);
	}
	for (int32 TaskIndex = 0; TaskIndex < ActiveTasksCopy.Num(); ++TaskIndex)
	{
		FAsyncTask<FAsyncFillCacheWorker>* Task = ActiveTasksCopy[TaskIndex];
		Task->EnsureCompletion();
		delete Task;
	}
}

bool FVirtualTextureChunkDDCCache::MakeChunkAvailable(struct FVirtualTextureDataChunk* Chunk, bool bAsync, FString& OutChunkFileName, int64& OutOffsetInFile)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureChunkDDCCache::MakeChunkAvailable);

	check(IsInParallelRenderingThread());

	const FString CachedFilePath = AbsoluteCachePath / Chunk->ShortDerivedDataKey;
	const FString TempFilePath = AbsoluteCachePath / FGuid::NewGuid().ToString() + ".tmp";

	if (Chunk->bCorruptDataLoadedFromDDC)
	{
		// We determined data loaded from DDC was corrupt...this means any file saved to VT DDC cache is also corrupt and can no longer be used
		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*CachedFilePath);
		Chunk->bCorruptDataLoadedFromDDC = false;
		Chunk->bFileAvailableInVTDDCDache = false;
	}

	// File already available? 
	if (Chunk->bFileAvailableInVTDDCDache)
	{
		OutChunkFileName = CachedFilePath;
		OutOffsetInFile = sizeof(FVirtualTextureFileHeader);
		return true;
	}

	{
		// Are we already processing this chunk ?
		FScopeLock Lock(&ActiveTasksLock);
		const int32 ChunkInProgressIdx = ActiveChunks.Find(Chunk);
		if (ChunkInProgressIdx != -1)
		{
			return false;
		}
	}
	
	// start filling it to the cache
	if (bAsync)
	{
		FAsyncTask<FAsyncFillCacheWorker>* Task = (new FAsyncTask<FAsyncFillCacheWorker>(TempFilePath, CachedFilePath, Chunk));
		
		{
			FScopeLock Lock(&ActiveTasksLock);
			ActiveTasks.Add(Task);
			ActiveChunks.Add(Chunk);
		}
		
		Task->StartBackgroundTask();
	}
	else
	{
		FAsyncFillCacheWorker SyncWorker(TempFilePath, CachedFilePath, Chunk);
		SyncWorker.DoWork();
		if (Chunk->bFileAvailableInVTDDCDache)
		{
			OutChunkFileName = CachedFilePath;
			OutOffsetInFile = sizeof(FVirtualTextureFileHeader);
			return true;
		}
	}

	return false;
}

void FVirtualTextureChunkDDCCache::MakeChunkAvailable_Concurrent(struct FVirtualTextureDataChunk* Chunk)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureChunkDDCCache::MakeChunkAvailable_Concurrent);

	const FString CachedFilePath = AbsoluteCachePath / Chunk->ShortDerivedDataKey;
	const FString TempFilePath = AbsoluteCachePath / FGuid::NewGuid().ToString() + ".tmp";

	if (Chunk->bCorruptDataLoadedFromDDC)
	{
		// We determined data loaded from DDC was corrupt...this means any file saved to VT DDC cache is also corrupt and can no longer be used
		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*CachedFilePath);
		Chunk->bCorruptDataLoadedFromDDC = false;
		Chunk->bFileAvailableInVTDDCDache = false;
	}

	// File already available? 
	if (Chunk->bFileAvailableInVTDDCDache)
	{
		return;
	}

	FAsyncFillCacheWorker SyncWorker(TempFilePath, CachedFilePath, Chunk);
	SyncWorker.DoWork();
}

#endif
