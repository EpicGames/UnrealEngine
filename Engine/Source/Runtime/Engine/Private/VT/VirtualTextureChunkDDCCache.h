// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

class FAsyncFillCacheWorker;
template<typename TTask> class FAsyncTask;
struct FVirtualTextureDataChunk;

class FVirtualTextureChunkDDCCache
{
public:
	void Initialize();
	void ShutDown();

	void UpdateRequests();
	void WaitFlushRequests_AnyThread();

	bool MakeChunkAvailable(FVirtualTextureDataChunk* Chunk, bool bAsync, FString& OutChunkFileName, int64& OutOffsetInFile);

	void MakeChunkAvailable_Concurrent(FVirtualTextureDataChunk* Chunk);
private:
	FString AbsoluteCachePath;
	TArray<FVirtualTextureDataChunk*> ActiveChunks;
	TArray<FAsyncTask<FAsyncFillCacheWorker>*> ActiveTasks;
	FCriticalSection ActiveTasksLock;
};

FVirtualTextureChunkDDCCache* GetVirtualTextureChunkDDCCache();

#endif