// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureChunkManager.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "UploadingVirtualTexture.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VT/VirtualTextureTranscodeCache.h"
#include "VT/VirtualTextureUploadCache.h"

#if WITH_EDITOR
#include "VirtualTextureChunkDDCCache.h"
#endif

FVirtualTextureChunkStreamingManager::FVirtualTextureChunkStreamingManager()
{
#if WITH_EDITOR
	GetVirtualTextureChunkDDCCache()->Initialize();
#endif
	BeginInitResource(&UploadCache);
}

FVirtualTextureChunkStreamingManager::~FVirtualTextureChunkStreamingManager()
{
	UploadCache.ReleaseResource(); // Must be called from rendering thread
#if WITH_EDITOR
	GetVirtualTextureChunkDDCCache()->ShutDown();
#endif
}

void FVirtualTextureChunkStreamingManager::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything /*= false*/)
{
	// It's OK to execute update if virtual texturing is disabled but we early out anyway to avoid an unnecessary render thread task.
	if (UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		FVirtualTextureChunkStreamingManager* StreamingManager = this;

		ENQUEUE_RENDER_COMMAND(UpdateVirtualTextureStreaming)(
			[StreamingManager](FRHICommandListImmediate& RHICmdList)
			{
#if WITH_EDITOR
				GetVirtualTextureChunkDDCCache()->UpdateRequests();
#endif // WITH_EDITOR
				StreamingManager->TranscodeCache.RetireOldTasks(RHICmdList, StreamingManager->UploadCache);
				StreamingManager->UploadCache.UpdateFreeList(RHICmdList);

				FVirtualTextureCodec::RetireOldCodecs();
			});
	}
}

int32 FVirtualTextureChunkStreamingManager::BlockTillAllRequestsFinished(float TimeLimit /*= 0.0f*/, bool bLogResults /*= false*/)
{
	int32 Result = 0;
	return Result;
}

void FVirtualTextureChunkStreamingManager::CancelForcedResources()
{
}

FVTRequestPageResult FVirtualTextureChunkStreamingManager::RequestTile(FRHICommandListBase& RHICmdList, FUploadingVirtualTexture* VTexture, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint32 vAddress, EVTRequestPagePriority Priority)
{
	SCOPE_CYCLE_COUNTER(STAT_VTP_RequestTile);

	const FVirtualTextureBuiltData* VTData = VTexture->GetVTData();
	const uint32 TileOffset = VTData->GetTileOffset(vLevel, vAddress, 0);
	if (TileOffset == ~0u)
	{
		UE_LOGF(LogConsoleResponse, Verbose, "vAddr %i@%i has an invalid tile (-1).", vAddress, vLevel);
	
		return EVTRequestPageStatus::Invalid;
	}

	const int32 ChunkIndex = VTData->GetChunkIndex(vLevel);
	check(ChunkIndex >= 0);

	// tile is being transcoded/is done transcoding
	const FVTTranscodeKey TranscodeKey = FVirtualTextureTranscodeCache::GetKey(ProducerHandle, LayerMask, vLevel, vAddress);
	const FVTTranscodeTileHandleAndStatus TranscodeTaskResult = TranscodeCache.FindTask(TranscodeKey);
	if (TranscodeTaskResult.Handle.IsValid())
	{
		const EVTRequestPageStatus Status = TranscodeTaskResult.IsComplete ? EVTRequestPageStatus::Available : EVTRequestPageStatus::Pending;
		return FVTRequestPageResult(Status, TranscodeTaskResult.Handle.PackedData);
	}

	// we limit the number of pending upload tiles in order to limit the memory required to store all the staging buffers
	// Never throttle high priority requests
	if (!UploadCache.IsInMemoryBudget() && Priority != EVTRequestPagePriority::High)
	{
		INC_DWORD_STAT(STAT_VTP_NumTranscodeDropped);
		return EVTRequestPageStatus::Saturated;
	}

	FGraphEventArray GraphCompletionEvents;
	const FVTCodecAndStatus CodecResult = VTexture->GetCodecForChunk(GraphCompletionEvents, ChunkIndex, Priority);
	if (!VTRequestPageStatus_HasData(CodecResult.Status))
	{
		// May fail to get codec if the file cache is saturated
		return CodecResult.Status;
	}

	uint32 MinLayerIndex = ~0u;
	uint32 MaxLayerIndex = 0u;
	for (uint32 LayerIndex = 0u; LayerIndex < VTData->GetNumLayers(); ++LayerIndex)
	{
		if (LayerMask & (1u << LayerIndex))
		{
			MinLayerIndex = FMath::Min(MinLayerIndex, LayerIndex);
			MaxLayerIndex = LayerIndex;
		}
	}

	// make a single read request that covers region of all requested tiles
	const uint32 OffsetStart = VTData->GetTileOffset(vLevel, vAddress, MinLayerIndex);
	const uint32 OffsetEnd = VTData->GetTileOffset(vLevel, vAddress, MaxLayerIndex + 1u);
	const uint32 RequestSize = OffsetEnd - OffsetStart;

	const FVTDataAndStatus TileDataResult = VTexture->ReadData(GraphCompletionEvents, ChunkIndex, OffsetStart, RequestSize, Priority);
	if (!VTRequestPageStatus_HasData(TileDataResult.Status))
	{
		return TileDataResult.Status;
	}
	check(TileDataResult.Data);

	FVTTranscodeParams TranscodeParams;
	TranscodeParams.Data = TileDataResult.Data;
	TranscodeParams.VTData = VTData;
	TranscodeParams.ChunkIndex = ChunkIndex;
	TranscodeParams.vAddress = vAddress;
	TranscodeParams.vLevel = vLevel;
	TranscodeParams.LayerMask = LayerMask;
	TranscodeParams.Codec = CodecResult.Codec;
	TranscodeParams.Name = VTexture->GetName();
	const FVTTranscodeTileHandle TranscodeHandle = TranscodeCache.SubmitTask(RHICmdList, UploadCache, TranscodeKey, ProducerHandle, TranscodeParams, &GraphCompletionEvents);
	return FVTRequestPageResult(EVTRequestPageStatus::Pending, TranscodeHandle.PackedData);
}

IVirtualTextureFinalizer* FVirtualTextureChunkStreamingManager::ProduceTile(FRHICommandListBase& RHICmdList, uint32 SkipBorderSize, uint8 NumLayers, uint8 LayerMask, uint64 RequestHandle, const FVTProduceTargetLayer* TargetLayers)
{
	SCOPE_CYCLE_COUNTER(STAT_VTP_ProduceTile);

	const FVTUploadTileHandle* StageTileHandles = TranscodeCache.AcquireTaskResult(FVTTranscodeTileHandle(RequestHandle));
	for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
	{
		if (LayerMask & (1u << LayerIndex))
		{
			const FVTProduceTargetLayer& Target = TargetLayers[LayerIndex];
			UploadCache.SubmitTile(RHICmdList, StageTileHandles[LayerIndex], Target, SkipBorderSize);
		}
	}

	return &UploadCache;
}

void FVirtualTextureChunkStreamingManager::GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const
{
	TranscodeCache.GatherProducePageDataTasks(ProducerHandle, InOutTasks);
}

void FVirtualTextureChunkStreamingManager::GatherProducePageDataTasks(uint64 RequestHandle, FGraphEventArray& InOutTasks) const
{
	FGraphEventRef Event = TranscodeCache.GetTaskEvent(FVTTranscodeTileHandle(RequestHandle));
	if (Event)
	{
		InOutTasks.Emplace(MoveTemp(Event));
	}
}
