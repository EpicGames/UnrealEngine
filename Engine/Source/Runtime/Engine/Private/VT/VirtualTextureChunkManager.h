// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "RHIFwd.h"
#include "ContentStreaming.h"
#include "VirtualTextureUploadCache.h"
#include "VirtualTextureTranscodeCache.h"

class FUploadingVirtualTexture;
struct FVTRequestPageResult;
class ULevel;
union FVirtualTextureProducerHandle;
struct FVTProduceTargetLayer;

DECLARE_STATS_GROUP(TEXT("Virtual Texturing Paging"), STATGROUP_VTP, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("RequestTile"), STAT_VTP_RequestTile, STATGROUP_VTP);
DECLARE_CYCLE_STAT(TEXT("ProduceTile"), STAT_VTP_ProduceTile, STATGROUP_VTP);
DECLARE_CYCLE_STAT(TEXT("StageTile"), STAT_VTP_StageTile, STATGROUP_VTP);

DECLARE_CYCLE_STAT(TEXT("stage upload"), STAT_VTP_StageUpload, STATGROUP_VTP);
DECLARE_CYCLE_STAT(TEXT("flush upload"), STAT_VTP_FlushUpload, STATGROUP_VTP);
DECLARE_CYCLE_STAT(TEXT("VT DDC Cache probing"), STAT_VTP_MakeChunkAvailable, STATGROUP_VTP);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num generates"), STAT_VTP_NumGenerate, STATGROUP_VTP);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num transcodes"), STAT_VTP_NumTranscode, STATGROUP_VTP);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num transcodes dropped"), STAT_VTP_NumTranscodeDropped, STATGROUP_VTP);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num transcodes retired"), STAT_VTP_NumTranscodeRetired, STATGROUP_VTP);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Intraframe upload flushes"), STAT_VTP_NumIntraFrameFlush, STATGROUP_VTP);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num uploads"), STAT_VTP_NumUploads, STATGROUP_VTP);

struct FVirtualTextureChunkStreamingManager final  : public IStreamingManager
{
public:
	FVirtualTextureChunkStreamingManager();
	virtual ~FVirtualTextureChunkStreamingManager();

	// IStreamingManager interface
	ENGINE_API void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override;
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override;
	virtual void CancelForcedResources() override;
	virtual void NotifyLevelChange() override {}
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) override {}
	virtual void AddLevel(class ULevel* Level) override {}
	virtual void RemoveLevel(class ULevel* Level) override {}
	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) override {}
	// End IStreamingManager interface

	FVTRequestPageResult RequestTile(FRHICommandListBase& RHICmdList, FUploadingVirtualTexture* VTexture, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint32 vAddress, EVTRequestPagePriority Priority);
	IVirtualTextureFinalizer* ProduceTile(FRHICommandListBase& RHICmdList, uint32 SkipBorderSize, uint8 NumLayers, uint8 LayerMask, uint64 RequestHandle, const FVTProduceTargetLayer* TargetLayers);
	void GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const;
	void GatherProducePageDataTasks(uint64 RequestHandle, FGraphEventArray& InOutTasks) const;

private:
	FVirtualTextureUploadCache UploadCache;
	FVirtualTextureTranscodeCache TranscodeCache;
};
