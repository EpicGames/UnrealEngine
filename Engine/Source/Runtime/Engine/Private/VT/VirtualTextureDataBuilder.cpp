// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "VirtualTextureDataBuilder.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleManager.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopedSlowTask.h"
#include "TextureBuildUtilities.h"
#include "TextureDerivedDataTask.h"
#include "ImageCoreUtils.h"
#include "VirtualTexturing.h"

// Debugging aid to dump tiles to disc as png files
#define SAVE_TILES 0
#define SAVE_CHUNKS 0

#if SAVE_TILES || SAVE_CHUNKS
#include "ImageUtils.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#endif

static TAutoConsoleVariable<int32> CVarVTParallelTileCompression(
	TEXT("r.VT.ParallelTileCompression"),
	1,
	TEXT("Enables parallel compression of macro tiles")
);

/*
 * Just a simple helper struct wrapping a pointer to an image in some source format.
 * This class does not own the memory it simply wraps some useful functionality around it
 * This functionality should ideally be part of something like FImage but it's not.
 */
struct FPixelDataRectangle
{
	ETextureSourceFormat Format;
	int64 Width;
	int64 Height;
	uint8 *Data;

	FPixelDataRectangle(ETextureSourceFormat SetFormat, int32 SetWidth, int32 SetHeight, uint8* SetData) :
		Format(SetFormat),
		Width(SetWidth),
		Height(SetHeight),
		Data(SetData)
	{}

	/*
	 * Copies a Width x Height rectangle located at SourceX, SourceY in the source image to location DestX,DestY in this image.
	 * If the requested rectangle is outside the source image it will be clipped to the source and the smaller clipped rectangle will be copied instead.
	 */
	void CopyRectangle(int32 DestX, int32 DestY, FPixelDataRectangle &Source, int32 SourceX, int32 SourceY, int32 RectWidth, int32 RectHeight)
	{
		checkf(Format == Source.Format, TEXT("Formats need to match"));
		checkf(DestX >= 0 && DestX < Width, TEXT("Destination location out of bounds"));
		checkf(DestY >= 0 && DestY < Height, TEXT("Destination location out of bounds"));

		int64 PixelSize = FTextureSource::GetBytesPerPixel(Source.Format);
		int64 SrcScanlineSize = Source.Width * PixelSize;
		int64 DstScanlineSize = Width * PixelSize;

		// Handle source position having negative coordinates in source image
		if (SourceX < 0)
		{
			DestX = DestX - SourceX;
			RectWidth = RectWidth + SourceX;
			SourceX = 0;
		}


		if (SourceY < 0)
		{
			DestY = DestY - SourceY;
			RectHeight = RectHeight + SourceY;
			SourceY = 0;
		}

		// Handle source position our width being beyond the boundaries of the source image
		int32 ClampedWidth = FMath::Max(FMath::Min(RectWidth, Source.Width - SourceX),0);
		int32 ClampedHeight = FMath::Max(FMath::Min(RectHeight, Source.Height - SourceY),0);
		int32 ClampedScanlineSize = ClampedWidth * PixelSize;

		// Copy the data a scan line at a time

		uint8 *DstScanline = Data + DestX * PixelSize + DestY * DstScanlineSize;
		const uint8 *SrcScanline = Source.Data + SourceX * PixelSize + SourceY * SrcScanlineSize;

		for (int Y = 0; Y < ClampedHeight; Y++)
		{
			FMemory::Memcpy(DstScanline, SrcScanline, ClampedScanlineSize);
			DstScanline += DstScanlineSize;
			SrcScanline += SrcScanlineSize;
		}
	}

	static int32 ApplyBorderMode(int32 x, int32 Width, TextureAddress Mode)
	{
		switch (Mode)
		{
			case TA_Wrap:
			{
				// Make sure it's a proper module for negative numbers ....
				int x_Wrap = x % Width;
				return x_Wrap + ((x_Wrap < 0) ? Width : 0);
			}
			case TA_Clamp:
				return FMath::Max(FMath::Min(x, Width-1), 0);
			case TA_Mirror:
			{
				int32 DoubleWidth = Width + Width;
				int32 DoubleWrap = x % DoubleWidth;
				DoubleWrap += ((DoubleWrap < 0) ? DoubleWidth : 0);
				return (DoubleWrap < Width) ? DoubleWrap : ((Width-1) - (DoubleWrap - Width));
			}
		}
		check(0);
		return x;
	}

	/*
	* Copies a Width x Height rectangle located at SourceX, SourceY in the source image to location DestX,DestY in this image.
	* If the requested rectangle is outside the source image it will be clipped to the source and the smaller clipped rectangle will be copied instead.
	*/
	void CopyRectangleBordered(int32 DestX, int32 DestY, FPixelDataRectangle &Source, int32 SourceX, int32 SourceY, int32 RectWidth, int32 RectHeight, TextureAddress BorderX, TextureAddress BorderY)
	{
		checkf(Format == Source.Format, TEXT("Formats need to match"));
		checkf(DestX >= 0 && DestX < Width, TEXT("Destination location out of bounds"));
		checkf(DestY >= 0 && DestY < Height, TEXT("Destination location out of bounds"));

		// Fast copy of regular pixels
		CopyRectangle(DestX, DestY, Source, SourceX, SourceY, RectWidth, RectHeight);

		int64 pixelSize = FTextureSource::GetBytesPerPixel(Format);

		// Special case the out of bounds pixels loop over all oob pixels and get the properly adjusted values
		if (SourceX < 0 ||
			SourceY < 0 ||
			SourceX + RectWidth > Source.Width ||
			SourceY + RectHeight > Source.Height)
		{
			// Top border and adjacent corners
			for (int32 y = SourceY; y < 0; y++)
			{
				for (int32 x = SourceX; x < SourceX + RectWidth; x++)
				{
					int32 xb = ApplyBorderMode(x, Source.Width, BorderX);
					int32 yb = ApplyBorderMode(y, Source.Height, BorderY);
					SetPixel(x - SourceX + DestX, y - SourceY + DestY, Source.GetPixel(xb, yb, pixelSize), pixelSize);
				}
			}

			// Bottom border and adjacent corners
			for (int32 y = Source.Height; y < SourceY + RectHeight; y++)
			{
				for (int32 x = SourceX; x < SourceX + RectWidth; x++)
				{
					int32 xb = ApplyBorderMode(x, Source.Width, BorderX);
					int32 yb = ApplyBorderMode(y, Source.Height, BorderY);
					SetPixel(x - SourceX + DestX, y - SourceY + DestY, Source.GetPixel(xb, yb, pixelSize), pixelSize);
				}
			}

			// Left border (w.o. corners)
			for (int32 x = SourceX; x < 0; x++)
			{
				//for (int32 y = SourceY; y < SourceY + RectHeight; y++)
				for (int32 y = FMath::Max(0, SourceY); y < FMath::Min(SourceY + RectHeight, Source.Height); y++)
				{
					int32 xb = ApplyBorderMode(x, Source.Width, BorderX);
					int32 yb = ApplyBorderMode(y, Source.Height, BorderY);
					SetPixel(x - SourceX + DestX, y - SourceY + DestY, Source.GetPixel(xb, yb, pixelSize), pixelSize);
				}
			}

			// Right border (w.o. corners)
			for (int32 x = Source.Width; x < SourceX + RectWidth; x++)
			{
				//for (int32 y = SourceY; y < SourceY + RectHeight; y++)
				for (int32 y = FMath::Max(0, SourceY); y < FMath::Min(SourceY + RectHeight, Source.Height); y++)
				{
					int32 xb = ApplyBorderMode(x, Source.Width, BorderX);
					int32 yb = ApplyBorderMode(y, Source.Height, BorderY);
					SetPixel(x - SourceX + DestX, y - SourceY + DestY, Source.GetPixel(xb, yb, pixelSize), pixelSize);
				}
			}
		}
	}

	void Clear()
	{
		FMemory::Memzero(Data, FTextureSource::GetBytesPerPixel(Format) * Width * Height);
	}

	inline void SetPixel(int32 x, int32 y, void *Value, int64 PixelSize)
	{
		void *DestPixelData = GetPixel(x, y, PixelSize);
		FMemory::Memcpy(DestPixelData, Value, PixelSize);
	}

	inline void *GetPixel(int32 x, int32 y, int64 PixelSize) const
	{
		check(x >= 0);
		check(y >= 0);
		check(x < Width);
		check(y < Height);
		return Data + (((y * Width) + x) * PixelSize);
	}
	
#if SAVE_TILES
	FImageView GetImageView() const 
	{
		ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(Format);
		return FImageView(Data,Width,Height,RawFormat);
	}

	void Save(FString BaseFileName, IImageWrapperModule* ImageWrapperModule)
	{
		FImageView Image = GetImageView();

		if ( ! FImageUtils::SaveImageAutoFormat(*BaseFileName,Image) )
		{
			UE_LOGF(LogVirtualTexturing,Warning,"Couldn't save to : %ls",*BaseFileName);
		}
	}
#endif

};

#define TEXTURE_COMPRESSOR_MODULENAME "TextureCompressor"

FVirtualTextureDataBuilder::FVirtualTextureDataBuilder(FVirtualTextureBuiltData &SetOutData, const FString& InDebugTexturePathName, ITextureCompressorModule *InCompressor, IImageWrapperModule* InImageWrapper)
	: OutData(SetOutData)
	, DebugTexturePathName(InDebugTexturePathName)
{
	Compressor = InCompressor ? InCompressor : &FModuleManager::LoadModuleChecked<ITextureCompressorModule>(TEXTURE_COMPRESSOR_MODULENAME);
	ImageWrapper = InImageWrapper ? InImageWrapper : &FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
}

FVirtualTextureDataBuilder::~FVirtualTextureDataBuilder()
{
}

bool FVirtualTextureBuilderDerivedInfo::InitializeFromBuildSettings(const FTextureSourceData& InSourceData, const FTextureBuildSettings* InSettingsPerLayer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.Initialize);

	const int32 NumLayers = InSourceData.Layers.Num();
	checkf(NumLayers <= (int32)VIRTUALTEXTURE_DATA_MAXLAYERS, TEXT("The maximum amount of layers is exceeded."));
	checkf(NumLayers > 0, TEXT("No layers to build."));

	const FTextureBuildSettings& BuildSettingsLayer0 = InSettingsPerLayer[0];
	const int32 TileSize = BuildSettingsLayer0.VirtualTextureTileSize;

	if (BuildSettingsLayer0.PowerOfTwoMode == ETexturePowerOfTwoSetting::ResizeToSpecificResolution)
	{
		// do not allow to set target width or height smaller than VT tile size
		if ((BuildSettingsLayer0.ResizeDuringBuildX && BuildSettingsLayer0.ResizeDuringBuildX < TileSize) ||
			(BuildSettingsLayer0.ResizeDuringBuildY && BuildSettingsLayer0.ResizeDuringBuildY < TileSize))
		{
			// will need to adjust miptail block calculations for this to work

			UE_LOGF(LogVirtualTexturing, Warning, "InitializeFromBuildSettings failed : Explicit resize to smaller than tile size (%d) not supported (%d x %d) [%ls]",
				TileSize, BuildSettingsLayer0.ResizeDuringBuildX, BuildSettingsLayer0.ResizeDuringBuildY, *InSourceData.TextureFullName);

			return false;
		}

		for (const auto& SourceBlock : InSourceData.Blocks)
		{
			// if any of block sizes is not power of two any of them is smaller than VT tile size
			if (!FMath::IsPowerOfTwo(SourceBlock.SizeX) || !FMath::IsPowerOfTwo(SourceBlock.SizeY) || SourceBlock.SizeX < TileSize || SourceBlock.SizeY < TileSize)
			{
				// then both target resize width & height must be set
				if (BuildSettingsLayer0.ResizeDuringBuildX == 0 || BuildSettingsLayer0.ResizeDuringBuildY == 0)
				{
					UE_LOGF(LogVirtualTexturing, Warning, "InitializeFromBuildSettings failed : Both resized width and height (%d x %d) must be set if any block is smaller than tile size (%d) [%ls]",
						BuildSettingsLayer0.ResizeDuringBuildX, BuildSettingsLayer0.ResizeDuringBuildY, TileSize, *InSourceData.TextureFullName);
					return false;
				}
			}
		}
	}

	int32 BlockSizeZ; // not needed here
	UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(
		InSourceData.BlockSizeX, InSourceData.BlockSizeY, 1,
		false,
		(ETexturePowerOfTwoSetting::Type)BuildSettingsLayer0.PowerOfTwoMode,
		BuildSettingsLayer0.ResizeDuringBuildX, BuildSettingsLayer0.ResizeDuringBuildY,
		BlockSizeX, BlockSizeY, BlockSizeZ);

	check(InSettingsPerLayer[0].MaxTextureResolution >= (uint32)TileSize);

	// Clamp BlockSizeX and BlockSizeY to MaxTextureResolution, but don't change aspect ratio
	//	(this is not right if MaxTextureResolution is not power of two)
	const uint32 ClampBlockSize = InSettingsPerLayer[0].MaxTextureResolution;
	if (FMath::Max<uint32>(BlockSizeX, BlockSizeY) > ClampBlockSize)
	{
		const int32 ClampedBlockSizeX = BlockSizeX >= BlockSizeY ? ClampBlockSize : FMath::Max(ClampBlockSize * BlockSizeX / BlockSizeY, 1u);
		const int32 ClampedBlockSizeY = BlockSizeY >= BlockSizeX ? ClampBlockSize : FMath::Max(ClampBlockSize * BlockSizeY / BlockSizeX, 1u);
		BlockSizeX = ClampedBlockSizeX;
		BlockSizeY = ClampedBlockSizeY;
	}

	// We require VT blocks (UDIM pages) to be PoT, but multi block textures may have full logical dimension that's not PoT
	if ( ! FMath::IsPowerOfTwo(BlockSizeX) || ! FMath::IsPowerOfTwo(BlockSizeY) )
	{
		UE_LOGF(LogVirtualTexturing,Warning,"InitializeFromBuildSettings failed : Block dimensions not power of 2 (%d x %d) [%ls]",
			BlockSizeX,BlockSizeY,*InSourceData.TextureFullName);

		return false;
	}

	// Ensure block size is at least 1 tile.
	BlockSizeScaleX = 1;
	BlockSizeScaleY = 1;
	while (BlockSizeX < TileSize) { BlockSizeX *= 2; BlockSizeScaleX *= 2; }
	while (BlockSizeY < TileSize) { BlockSizeY *= 2; BlockSizeScaleY *= 2; }

	SizeInBlocksX = InSourceData.SizeInBlocksX;
	SizeInBlocksY = InSourceData.SizeInBlocksY;

	const int64 FullSizeX = (int64)BlockSizeX * SizeInBlocksX;
	const int64 FullSizeY = (int64)BlockSizeY * SizeInBlocksY;

	// make sure virtual texture dimensions are valid for runtime usage
	// this should match calculation in FVirtualTextureAllocator::Alloc in VirtualTextureAllocator.cpp file
	const int64 WidthInTiles = FullSizeX / TileSize;
	const int64 HeightInTiles = FullSizeY / TileSize;
	const uint64 MaxSizeInTiles = FMath::Max(WidthInTiles, HeightInTiles);
	const int64 vLogMaxSize = FMath::CeilLogTwo64(MaxSizeInTiles);
	if (vLogMaxSize > VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE)
	{
		// max VT size on pixels that runtime supports is TileSize<<MAX_PAGETABLE_SIZE pixels for max dimension
		// for 128 tile size that is 128<<12 = 524288 pixels
		UE_LOGF(LogVirtualTexturing, Warning,
			"InitializeFromBuildSettings failed: VT dimensions (%lld x %lld) are too large - too many tiles (%lld x %lld) [%ls]",
			FullSizeX, FullSizeY,
			WidthInTiles, HeightInTiles,
			*InSourceData.TextureFullName);
		return false;
	}

	// total dimensions (of virtual canvas of UDIM blocks) must fit in INT32 on each axis
	//	  there is a limit of 16 bits of the tile index for the U32 morton code, maybe that's stricter?
	//	  that's something like 128*65536 maximum virtual dimension?
	//	  in practice that's hard to hit because the total pixel count will limit you first
	const int64 VTCanvasMaxDimension = TileSize * 65536; // must fit in INT32_MAX
	if (FullSizeX > VTCanvasMaxDimension || FullSizeY > VTCanvasMaxDimension )
	{
		UE_LOGF(LogVirtualTexturing,Warning,"InitializeFromBuildSettings failed : dimensions exceed VTCanvasMaxDimension "
			"(%d x %d = %lld) (%d x %d = %lld) [%ls]",
			BlockSizeX,SizeInBlocksX,FullSizeX,
			BlockSizeY,SizeInBlocksY,FullSizeY,
			*InSourceData.TextureFullName);

		return false;
	}

	SizeX = (int32)FullSizeX;
	SizeY = (int32)FullSizeY;
	
	// there is no strict limit on total pixel count
	//	but output must fit in 4 GB
	//	so as a sanity check, test if pixel count is over 4G
	// see FImageCoreUtils::IsImageImportPossible
	//	(this is sort of the wrong check, it really depends on output pixel format)
	int64 NumBlocks = InSourceData.Blocks.Num();
	int64 TotalPixels = (int64) BlockSizeX * BlockSizeY * NumBlocks;
	if ( TotalPixels > (1ULL<<32) )
	{
		UE_LOGF(LogVirtualTexturing,Warning,"InitializeFromBuildSettings failed : total pixel count over 4G "
			"(%d x %d = %lld) [%ls]",
			SizeX,SizeY,(int64)SizeX * SizeY,
			*InSourceData.TextureFullName);

		return false;
	}

	const uint32 Size = FMath::Max(SizeX, SizeY);

	// Mip down to 1x1 pixels
	NumMips = FMath::FloorLog2(Size) + 1;

	return true;
}

bool FVirtualTextureDataBuilder::Build(FTextureSourceData& InSourceData, FTextureSourceData& InCompositeSourceData, const FTextureBuildSettings* InSettingsPerLayer, bool bAllowAsync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.Build);

	const int32 NumBlocks = InSourceData.Blocks.Num();

	const int32 NumLayers = InSourceData.Layers.Num();
	checkf(NumLayers <= (int32)VIRTUALTEXTURE_DATA_MAXLAYERS, TEXT("The maximum amount of layers is exceeded."));
	checkf(NumLayers > 0, TEXT("No layers to build."));

	SettingsPerLayer.AddUninitialized(NumLayers);
	FMemory::Memcpy(&SettingsPerLayer[0], InSettingsPerLayer, sizeof(FTextureBuildSettings) * NumLayers);
	const FTextureBuildSettings& BuildSettingsLayer0 = SettingsPerLayer[0];
	const int32 TileSize = BuildSettingsLayer0.VirtualTextureTileSize;

	if ( ! DerivedInfo.InitializeFromBuildSettings(InSourceData, InSettingsPerLayer) )
	{
		return false;
	}

	//NOTE: OutData may point to a previously build data so it is important to
	//properly initialize all fields and not assume this is a freshly constructed object

	OutData.TileBorderSize = BuildSettingsLayer0.VirtualTextureBorderSize;
	OutData.TileSize = TileSize;
	OutData.NumLayers = NumLayers;
	OutData.Width = DerivedInfo.SizeX;
	OutData.Height = DerivedInfo.SizeY;
	OutData.WidthInBlocks = InSourceData.SizeInBlocksX;
	OutData.HeightInBlocks = InSourceData.SizeInBlocksY;

	OutData.TileDataOffsetPerLayer.Empty();
	OutData.ChunkIndexPerMip.Empty();
	OutData.BaseOffsetPerMip.Empty();
	OutData.TileOffsetData.Empty();

	OutData.TileIndexPerChunk.Empty();
	OutData.TileIndexPerMip.Empty();
	OutData.TileOffsetInChunk.Empty();

	OutData.Chunks.Empty();
	OutData.NumMips = DerivedInfo.NumMips;

	// override async compression if requested
	bAllowAsync = bAllowAsync && CVarVTParallelTileCompression.GetValueOnAnyThread();

	LayerPayload.SetNum(NumLayers);

	{
		FScopedSlowTask BuildTask(NumLayers * NumBlocks);
		
		// Process source texture layer by layer
		// Layer blocks will be freed from inside of BuildLayerBlocks() as soon as they are done
		for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			const FTextureBuildSettings& BuildSettingsForLayer = SettingsPerLayer[LayerIndex];
			FVirtualTextureSourceLayerData LayerData;

			// Specify the format we are processing to in this step :
			LayerData.ImageFormat = UE::TextureBuildUtilities::GetVirtualTextureBuildIntermediateFormat(BuildSettingsForLayer);
			LayerData.GammaSpace = BuildSettingsForLayer.GetDestGammaSpace();
			// Gamma correction can either be applied in step 1 or step 2 of the VT build
			//	depending on whether the Intermediate format is U8 or not
			if (!ERawImageFormat::GetFormatNeedsGammaSpace(LayerData.ImageFormat))
			{
				LayerData.GammaSpace = EGammaSpace::Linear;
			}

			LayerData.FormatName = FImageCoreUtils::ConvertToUncompressedTextureFormatName(LayerData.ImageFormat);
			LayerData.PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(LayerData.ImageFormat);
			LayerData.SourceFormat = FImageCoreUtils::ConvertToTextureSourceFormat(LayerData.ImageFormat);

			// Don't want platform specific swizzling for VT tile data, this tends to add extra padding for textures with odd dimensions
			// (VT physical tiles generally not power-of-2 after adding border)
			// Must match TextureDerivedData.cpp
			LayerData.TextureFormatName = UE::TextureBuildUtilities::TextureFormatRemovePlatformPrefixFromName(BuildSettingsForLayer.TextureFormatName);
			
			if ( BuildSettingsForLayer.bKnowAlphaTransparency )
			{
				// bKnowAlphaTransparency includes all Force actions
				LayerData.bHasAlpha = BuildSettingsForLayer.bHasTransparentAlpha;
			}
			else if ( BuildSettingsForLayer.bForceNoAlphaChannel ) // note the order of operations! ( ForceNo takes precedence )
			{
				LayerData.bHasAlpha = false;
			}
			else if ( BuildSettingsForLayer.bForceAlphaChannel )
			{
				LayerData.bHasAlpha = true;
			}
			else
			{
				// alpha detection was not previously done
				//	must do it now on all blocks
				// (this is hard to hit; bKnowAlphaTransparency is almost always true now)

				for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
				{
					const TArray<FImage>& SourceMips = InSourceData.Blocks[BlockIndex].MipsPerLayer[LayerIndex];
					if (!SourceMips.IsEmpty())
					{
						LayerData.bHasAlpha = FImageCore::DetectAlphaChannel(SourceMips[0]);
						if (LayerData.bHasAlpha)
						{
							break;
						}
					}
				}
			}

			// Building happens in following order: [Layers] -> [Blocks (with creating mips)] -> [Tiles] -> [Mips]
			BuildLayerBlocks(BuildTask, LayerIndex, LayerData, InSourceData, InCompositeSourceData, bAllowAsync);
		}
	}

	if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
	{
		return false;
	}

	// Rearrange compressed VT tiles into chunks for output
	// Chunks contain multiple tiles: [Tiles] -> [Mips] -> [Layers]
	bool ok = BuildChunks();

	// Release memory used during build process
	LayerPayload.Empty();

	return ok;
}

bool FVirtualTextureDataBuilder::BuildChunks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.BuildChunks);

	static const uint32 MinSizePerChunk = 1024u; // Each chunk will contain a mip level of at least this size (MinSizePerChunk x MinSizePerChunk)
	const uint32 NumLayers = LayerPayload.Num();
	const int32 TileSize = SettingsPerLayer[0].VirtualTextureTileSize;
	const uint32 MinSizePerChunkInTiles = FMath::DivideAndRoundUp<uint32>(MinSizePerChunk, TileSize);
	const uint32 MinTilesPerChunk = MinSizePerChunkInTiles * MinSizePerChunkInTiles;
	const int32 BlockSizeInTilesX = FMath::DivideAndRoundUp(DerivedInfo.BlockSizeX, TileSize);
	const int32 BlockSizeInTilesY = FMath::DivideAndRoundUp(DerivedInfo.BlockSizeY, TileSize);

	uint32 MipWidthInTiles = FMath::DivideAndRoundUp(DerivedInfo.SizeX, TileSize);
	uint32 MipHeightInTiles = FMath::DivideAndRoundUp(DerivedInfo.SizeY, TileSize);
	int64 NumTiles64 = 0;

	check(MipWidthInTiles <= (1<<16));
	check(MipHeightInTiles <= (1<<16));

	for (uint32 Mip = 0; Mip < OutData.NumMips; ++Mip)
	{
		const uint32 MaxTileInMip = FMath::MortonCode2(MipWidthInTiles - 1) | (FMath::MortonCode2(MipHeightInTiles - 1) << 1);
		NumTiles64 += (MaxTileInMip + 1u);
		MipWidthInTiles = FMath::DivideAndRoundUp(MipWidthInTiles, 2u);
		MipHeightInTiles = FMath::DivideAndRoundUp(MipHeightInTiles, 2u);
	}
	
	if ( NumTiles64 > INT32_MAX )
	{
		return false;
	}

	uint32 NumTiles = (uint32)NumTiles64;
	FScopedSlowTask BuildTask(NumTiles);

	TArray<FVTSourceTileEntry> TilesInChunk;
	TilesInChunk.Reserve(NumTiles);

	// Loop over Tiles in Morton order, and assemble the tiles into chunks
	// This only moves memory into chunks, packing tile payload into same
	// order as older version of code to maintain identical output
	{
		uint32 TileIndex = 0u;
		bool bInFinalChunk = false;

		OutData.ChunkIndexPerMip.Reserve(OutData.NumMips);
		OutData.BaseOffsetPerMip.Init(~0u, OutData.NumMips);
		OutData.TileOffsetData.Reserve(OutData.NumMips);

		OutData.TileOffsetInChunk.Init(~0u, NumTiles * NumLayers);
		OutData.TileIndexPerChunk.Reserve(OutData.NumMips + 1);
		OutData.TileIndexPerMip.Reserve(OutData.NumMips + 1);

		OutData.TileIndexPerChunk.Add(TileIndex);

		MipWidthInTiles = FMath::DivideAndRoundUp(DerivedInfo.SizeX, TileSize);
		MipHeightInTiles = FMath::DivideAndRoundUp(DerivedInfo.SizeY, TileSize);
		for (uint32 Mip = 0; Mip < OutData.NumMips; ++Mip)
		{
			FVirtualTextureTileOffsetData& OffsetData = OutData.TileOffsetData.AddDefaulted_GetRef();
			OffsetData.Init(MipWidthInTiles, MipHeightInTiles);

			OutData.ChunkIndexPerMip.Add(OutData.Chunks.Num());
			OutData.TileIndexPerMip.Add(TileIndex);

			const int32 MipBlockSizeInTilesX = FMath::Max(BlockSizeInTilesX >> Mip, 1);
			const int32 MipBlockSizeInTilesY = FMath::Max(BlockSizeInTilesY >> Mip, 1);
			const uint32 MaxTileInMip = FMath::MortonCode2(MipWidthInTiles - 1) | (FMath::MortonCode2(MipHeightInTiles - 1) << 1);

			for (uint32 TileIndexInMip = 0u; TileIndexInMip <= MaxTileInMip; ++TileIndexInMip)
			{
				BuildTask.EnterProgressFrame();

				const uint32 TileX = FMath::ReverseMortonCode2(TileIndexInMip);
				const uint32 TileY = FMath::ReverseMortonCode2(TileIndexInMip >> 1);
				if (TileX < MipWidthInTiles && TileY < MipHeightInTiles)
				{
					const int32 BlockX = TileX / MipBlockSizeInTilesX;
					const int32 BlockY = TileY / MipBlockSizeInTilesY;

					int32 BlockIndex = FindSourceBlockIndex(Mip, BlockX, BlockY);
					if (BlockIndex != INDEX_NONE)
					{
						FVTBlockPayload& Block = LayerPayload[0].Blocks[BlockIndex];
						FVTSourceTileEntry* TileEntry = new(TilesInChunk) FVTSourceTileEntry;
						TileEntry->BlockIndex = BlockIndex;
						TileEntry->TileIndex = TileIndex;
						TileEntry->MipIndex = Mip;
						TileEntry->MipIndexInBlock = Mip - Block.MipBias;
						TileEntry->TileX = TileX;
						TileEntry->TileY = TileY;
						OffsetData.AddTile(TileIndexInMip);
					}
				}
				TileIndex += NumLayers;
			}

			OffsetData.Finalize();

			if (!bInFinalChunk && TilesInChunk.Num() >= (int32)MinTilesPerChunk)
			{
				OutData.TileIndexPerChunk.Add(TileIndex);
				if ( ! BuildPagesForChunk(TilesInChunk) )
					return false;
				TilesInChunk.Reset();
			}
			else
			{
				bInFinalChunk = true;
			}

			MipWidthInTiles = FMath::DivideAndRoundUp(MipWidthInTiles, 2u);
			MipHeightInTiles = FMath::DivideAndRoundUp(MipHeightInTiles, 2u);
		}

		check(TileIndex == NumTiles * NumLayers);
		OutData.TileIndexPerChunk.Add(TileIndex);
		OutData.TileIndexPerMip.Add(TileIndex);

		if (TilesInChunk.Num() > 0)
		{
			if ( ! BuildPagesForChunk(TilesInChunk) )
				return false;
		}

		check(OutData.BaseOffsetPerMip.Num() == OutData.NumMips);
	}

	// Use compact tile offsets if we have fixed tile sizes on every layer (raw GPU codecs).
	// Otherwise use legacy data.
	const bool bUseLegacyData = OutData.TileDataOffsetPerLayer.Num() != NumLayers;
	if (bUseLegacyData)
	{
		// Using legacy data from now on so remove the compact data.
		OutData.TileOffsetData.Empty();

		// Patch holes left in offset array
		for (int32 ChunkIndex = 0; ChunkIndex < OutData.Chunks.Num(); ++ChunkIndex)
		{
			uint32 CurrentOffset = OutData.Chunks[ChunkIndex].SizeInBytes;
			for (int32 TileIndex = OutData.TileIndexPerChunk[ChunkIndex + 1] - 1u; TileIndex >= (int32)OutData.TileIndexPerChunk[ChunkIndex]; --TileIndex)
			{
				const uint32 TileOffset = OutData.TileOffsetInChunk[TileIndex];
				if (TileOffset > CurrentOffset)
				{
					check(TileOffset == ~0u);
					OutData.TileOffsetInChunk[TileIndex] = CurrentOffset;
				}
				else
				{
					CurrentOffset = TileOffset;
				}
			}
		}

		for (int32 TileIndex = 0u; TileIndex < OutData.TileOffsetInChunk.Num(); ++TileIndex)
		{
			const uint32 TileOffset = OutData.TileOffsetInChunk[TileIndex];
			check(TileOffset != ~0u);
		}
	}
	else
	{
		// We can remove legacy data and only reference the compact data from now on.
		OutData.TileIndexPerChunk.Empty();
		OutData.TileIndexPerMip.Empty();
		OutData.TileOffsetInChunk.Empty();
	}

	return true;
}

static const TCHAR* GetSafePixelFormatName(EPixelFormat Format)
{
	if (Format >= PF_MAX)
	{
		return TEXT("INVALID");
	}
	else
	{
		return GPixelFormats[Format].Name;
	}
}

void FVirtualTextureDataBuilder::BuildBlockTiles(uint32 LayerIndex, uint32 BlockIndex, FVTBlockPayload& Block, const FVirtualTextureSourceLayerData& LayerData, bool bAllowAsync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.BuildBlockTiles);

	const FTextureBuildSettings& BuildSettingsLayer0 = SettingsPerLayer[0];
	const FTextureBuildSettings& BuildSettingsForLayer = SettingsPerLayer[LayerIndex];

	const int32 TileSize = BuildSettingsLayer0.VirtualTextureTileSize;
	const int32 BorderSize = BuildSettingsLayer0.VirtualTextureBorderSize;
	const int32 PhysicalTileSize = TileSize + BorderSize * 2;

	FThreadSafeBool bCompressionError = false;
	EPixelFormat CompressedFormat = PF_Unknown;

	{
		// Create settings for building the tile. These should be simple, "clean" settings
		// just compressing the style to a GPU format not adding things like colour correction, ... 
		// as these settings were already baked into the SourcePixels.

		// TBSettings starts default constructed (no processing)
		//	then we copy over just the compression options without the color-change processing
		FTextureBuildSettings TBSettings;
		TBSettings.MaxTextureResolution = FTextureBuildSettings::MaxTextureResolutionDefault;
		TBSettings.TextureFormatName = LayerData.TextureFormatName;
		TBSettings.BaseTextureFormatName = TBSettings.TextureFormatName; // VTs never have platform prefix
		TBSettings.bSRGB = BuildSettingsForLayer.bSRGB;
		TBSettings.bUseLegacyGamma = BuildSettingsForLayer.bUseLegacyGamma;
		TBSettings.MipGenSettings = TMGS_NoMipmaps;

		// LayerData.bHasAlpha was set for the layer if there is alpha anywhere (or ForceAlpha and not ForceNoAlpha)
		// force each tile to make the same choice about whether it has alpha or not, do not DetectAlpha per tile
		// make sure either ForceAlpha or ForceNoAlpha is on for tiles
		TBSettings.bForceAlphaChannel = LayerData.bHasAlpha;
		TBSettings.bForceNoAlphaChannel = !LayerData.bHasAlpha;

		TBSettings.bHDRSource = BuildSettingsForLayer.bHDRSource;
		TBSettings.bVirtualStreamable = true;

		// Encode speed must be resolved before we get here.
		TBSettings.LossyCompressionAmount = BuildSettingsForLayer.LossyCompressionAmount;
		TBSettings.CompressionQuality = BuildSettingsForLayer.CompressionQuality;
		TBSettings.OodleEncodeEffort = BuildSettingsForLayer.OodleEncodeEffort;
		TBSettings.OodleUniversalTiling = BuildSettingsForLayer.OodleUniversalTiling;
		TBSettings.bOodleUsesRDO = BuildSettingsForLayer.bOodleUsesRDO;
		TBSettings.OodleRDO = BuildSettingsForLayer.OodleRDO;
		TBSettings.bOodlePreserveExtremes = BuildSettingsForLayer.bOodlePreserveExtremes;
		TBSettings.OodleTextureSdkVersion = BuildSettingsForLayer.OodleTextureSdkVersion;

		TBSettings.bKnowAlphaTransparency = BuildSettingsForLayer.bKnowAlphaTransparency;
		TBSettings.bHasTransparentAlpha = BuildSettingsForLayer.bHasTransparentAlpha;

		check(TBSettings.GetDestGammaSpace() == BuildSettingsForLayer.GetDestGammaSpace());

		// Mip levels start at Block.MipBias in case provided Texture is smaller than block size used in VT
		for (int32 MipIndex = 0; MipIndex < Block.NumMips; ++MipIndex)
		{
			int32 Mip = MipIndex + Block.MipBias;
			const int32 MipBlockSizeX = FMath::Max(Block.SizeX >> MipIndex, 1);
			const int32 MipBlockSizeY = FMath::Max(Block.SizeY >> MipIndex, 1);
			const int32 MipBlockSizeInTilesX = FMath::DivideAndRoundUp(MipBlockSizeX, TileSize);
			const int32 MipBlockSizeInTilesY = FMath::DivideAndRoundUp(MipBlockSizeY, TileSize);
			const int32 NumTiles = MipBlockSizeInTilesY * MipBlockSizeInTilesX;

			if (MipIndex == 0)
			{
				Block.Tiles.SetNum(NumTiles);
			}

			// ParallelFor is on TaskGraph for VT tiles
			//	TextureFormats should disable their own internal use of TaskGraph for VT tiles if necessary
			const bool bIsSingleThreaded = !bAllowAsync;

			UE::Tasks::FCancellationToken* CancellationToken = UE::Tasks::FCancellationTokenScope::GetCurrentCancellationToken();

			// Build all tiles for this mip level
			ParallelFor(TEXT("Texture.VT.BuildTiles.PF"), NumTiles, 1, [&](int32 TileIndex)
			{
				if (CancellationToken && CancellationToken->IsCanceled())
				{
					return;
				}

				int32 TileY = TileIndex / MipBlockSizeInTilesX;
				int32 TileX = TileIndex % MipBlockSizeInTilesX;

				Block.Tiles[TileIndex].Mips.SetNum(Block.NumMips);

				const FImage& SourceMip = Block.Mips[MipIndex];
				check(SourceMip.Format == LayerData.ImageFormat);
				check(SourceMip.GammaSpace == LayerData.GammaSpace);

				FPixelDataRectangle SourceData(LayerData.SourceFormat,
					SourceMip.SizeX,
					SourceMip.SizeY,
					const_cast<uint8*>(SourceMip.RawData.GetData()));

				TArray<FImage> TileImages;
				FImage* TileImage = new(TileImages) FImage(PhysicalTileSize, PhysicalTileSize, LayerData.ImageFormat, LayerData.GammaSpace);
				FPixelDataRectangle TileData(LayerData.SourceFormat, PhysicalTileSize, PhysicalTileSize, TileImage->RawData.GetData());

				TileData.Clear();
				TileData.CopyRectangleBordered(0, 0, SourceData,
					TileX * TileSize - BorderSize,
					TileY * TileSize - BorderSize,
					PhysicalTileSize,
					PhysicalTileSize,
					(TextureAddress)BuildSettingsLayer0.VirtualAddressingModeX,
					(TextureAddress)BuildSettingsLayer0.VirtualAddressingModeY);

#if SAVE_TILES
				{
					FString DebugName = FPaths::MakeValidFileName(*DebugTexturePathName, TEXT('_'));
					FString BasePath = FPaths::ProjectUserDir();
					FString TileFileName = BasePath / FString::Format(TEXT("{0}_L{1}_B{2}_M{3}_X{4}_Y{5}.png"), TArray<FStringFormatArg>({ *DebugName, LayerIndex, BlockIndex, Mip, TileX, TileY }));
					TileData.Save(TileFileName, ImageWrapper);
				}
#endif // SAVE_TILES

				// give each tile a unique DebugTexturePathName for DebugDump option :
				FString DebugTilePathName = FString::Printf(TEXT("%s_L%d_VT%04d_B%d_M%d_X%d_Y%d"), *DebugTexturePathName, LayerIndex, TileIndex, BlockIndex, Mip, TileX, TileY);

				TArray<FCompressedImage2D> CompressedMip;
				TArray<FImage> EmptyList;
				uint32 NumMipsInTail, ExtData;
				// this is the Build for Tiles to do the encode to GPU formats, with no processing
				bool bResult = Compressor->BuildTexture(TileImages, EmptyList, TBSettings, DebugTilePathName, CompressedMip, NumMipsInTail, ExtData, nullptr);

				if (CancellationToken && CancellationToken->IsCanceled())
				{
					return;
				}

				if (!ensure(bResult))
				{
					bCompressionError = true;
				}

				check(CompressedMip.Num() == 1);
				checkf(CompressedFormat == PF_Unknown || CompressedFormat == CompressedMip[0].PixelFormat,
					TEXT("CompressedFormat: %s (%d), CompressedMip[0].PixelFormat: %s (%d)"),
					GetSafePixelFormatName(CompressedFormat), (int32)CompressedFormat,
					GetSafePixelFormatName((EPixelFormat)CompressedMip[0].PixelFormat), (int32)CompressedMip[0].PixelFormat);

				FVTTileMipPayload& MipPayload = Block.Tiles[TileIndex].Mips[MipIndex];
				MipPayload.Payload = MoveTemp(CompressedMip[0].RawData);
				MipPayload.CompressedFormat = (EPixelFormat)CompressedMip[0].PixelFormat;
			},
			(bIsSingleThreaded ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None));
		}
	}
}

void FVirtualTextureDataBuilder::BuildLayerBlocks(FSlowTask& BuildTask, uint32 LayerIndex, const FVirtualTextureSourceLayerData& LayerData, FTextureSourceData& SourceData, FTextureSourceData& CompositeSourceData, bool bAllowAsync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.BuildLayerBlocks);

	const int32 TileSize = SettingsPerLayer[0].VirtualTextureTileSize;
	const int32 NumLayers = SourceData.Layers.Num();
	const int32 NumBlocks = SourceData.Blocks.Num();

	// Miptail
	TArray<FImage> MiptailInputImages;
	FPixelDataRectangle MiptailPixelData{ LayerData.SourceFormat, 0, 0, 0 };
	const uint32 BlockSize = FMath::Min(DerivedInfo.BlockSizeX, DerivedInfo.BlockSizeY);
	const uint32 BlockSizeInTiles = FMath::DivideAndRoundUp<uint32>(BlockSize, TileSize);
	const uint32 MaxMipInBlock = FMath::CeilLogTwo(BlockSizeInTiles);
	const uint32 MipWidthInBlock = FMath::Max<uint32>(DerivedInfo.BlockSizeX >> MaxMipInBlock, 1);
	const uint32 MipHeightInBlock = FMath::Max<uint32>(DerivedInfo.BlockSizeY >> MaxMipInBlock, 1);
	const uint32 MipInputSizeX = FMath::RoundUpToPowerOfTwo(DerivedInfo.SizeInBlocksX * MipWidthInBlock);
	const uint32 MipInputSizeY = FMath::RoundUpToPowerOfTwo(DerivedInfo.SizeInBlocksY * MipHeightInBlock);

	// If we have more than 1 block and we can produce more mips than each block has
	// then need to create miptail that contains mips made from multiple blocks2
	// be aware of mip limit (OutData.NumMips) - no need for miptail block if all mips are already used
	const bool bNeedsMiptailBlock = (NumBlocks > 1) && (OutData.NumMips > (MaxMipInBlock + 1));

	if (bNeedsMiptailBlock)
	{
		MiptailInputImages.Reset(1);
		FImage* MiptailInputImage = new(MiptailInputImages) FImage();
		MiptailInputImage->Init(MipInputSizeX, MipInputSizeY, LayerData.ImageFormat, LayerData.GammaSpace);
		MiptailPixelData = FPixelDataRectangle(LayerData.SourceFormat, MipInputSizeX, MipInputSizeY, MiptailInputImage->RawData.GetData());
		MiptailPixelData.Clear();
	}

	LayerPayload[LayerIndex].Blocks.SetNum(NumBlocks + (bNeedsMiptailBlock ? 1 : 0));

	// Process source texture block by block from same layer
	// Each block is released as soon as possible at end of each iteration
	for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
	{
		BuildTask.EnterProgressFrame();

		FTextureSourceBlockData& SourceBlockData = SourceData.Blocks[BlockIndex];

		// Current lock + mips that will be compressed to tiles
		FVTBlockPayload& BlockData = LayerPayload[LayerIndex].Blocks[BlockIndex];

		BlockData.BlockX = SourceBlockData.BlockX;
		// UE applies a (1-y) transform to imported UVs, so apply a similar transform to UDIM block locations here
		// This ensures that UDIM tiles will appear in the correct location when sampled with transformed UVs
		BlockData.BlockY = (DerivedInfo.SizeInBlocksY - SourceBlockData.BlockY) % DerivedInfo.SizeInBlocksY;
		BlockData.NumMips = SourceBlockData.NumMips;
		BlockData.NumSlices = SourceBlockData.NumSlices;
		BlockData.MipBias = SourceBlockData.MipBias;
		BlockData.SizeX = 0u;
		BlockData.SizeY = 0u;

		const FTextureBuildSettings& BuildSettingsForLayer = SettingsPerLayer[LayerIndex];

		TArray<FImage>& SourceMips = SourceBlockData.MipsPerLayer[LayerIndex];
		TArray<FImage> EmptyImageArray;
		TArray<FImage>* CompositeSourceMips = &EmptyImageArray;
		if (CompositeSourceData.Blocks.Num() > 0)
		{
			CompositeSourceMips = &CompositeSourceData.Blocks[BlockIndex].MipsPerLayer[LayerIndex];
		}

		// Adjust the build settings to generate an uncompressed texture with mips but leave other settings
		// like color correction, ... in place

		// TBSettings starts with the full Texture settings, so we get all options
		//  then we change FormatName to be == Source format, so no Compression is done
		FTextureBuildSettings TBSettings = SettingsPerLayer[0];
		//TBSettings.MaxTextureResolution = FTextureBuildSettings::MaxTextureResolutionDefault;
		TBSettings.TextureFormatName = LayerData.FormatName;
		TBSettings.BaseTextureFormatName = LayerData.FormatName; // VTs never have platform prefix

		if (LayerIndex != 0)
		{
			// @todo Oodle : this looks fragile
			//	some of the processing options are copied from BuildSettingsForLayer
			//	but some are NOT
			//	it seems semi-random
			//  In the common case of NumLayers==1 , then it doesn't matter
			//	so this would be rarely observed

			// TBSettings was set from Layer 0, copy in some settings from this Layer ?
			TBSettings.bSRGB = BuildSettingsForLayer.bSRGB;
			TBSettings.bUseLegacyGamma = BuildSettingsForLayer.bUseLegacyGamma;
			TBSettings.bForceAlphaChannel = BuildSettingsForLayer.bForceAlphaChannel;
			TBSettings.bForceNoAlphaChannel = BuildSettingsForLayer.bForceNoAlphaChannel;
			TBSettings.bHDRSource = BuildSettingsForLayer.bHDRSource;
			TBSettings.bApplyYCoCgBlockScale = BuildSettingsForLayer.bApplyYCoCgBlockScale;
			TBSettings.bReplicateRed = BuildSettingsForLayer.bReplicateRed;
			TBSettings.bReplicateAlpha = BuildSettingsForLayer.bReplicateAlpha;
		}

		// Make sure the output of the texture builder is in the same gamma space as we expect it.
		check(TBSettings.GetDestGammaSpace() == BuildSettingsForLayer.GetDestGammaSpace());

		// Leave original mip settings alone unless it's none at which point we will just generate them using a simple average
		if (TBSettings.MipGenSettings == TMGS_NoMipmaps)
		{
			TBSettings.MipGenSettings = TMGS_SimpleAverage;
		}

		// in case image is smaller than tile size, we need PowerOfTwoMode adjusted size to correctly calculate MipBias & LocalBlockSizeScaleX/Y in a while loop below
		int32 AdjustedSizeX, AdjustedSizeY, AdjustedSizeZ;
		UE::TextureBuildUtilities::GetPowerOfTwoTargetTextureSize(
			SourceMips[0].SizeX, SourceMips[0].SizeY, 1,
			false,
			(ETexturePowerOfTwoSetting::Type)SettingsPerLayer[0].PowerOfTwoMode,
			SettingsPerLayer[0].ResizeDuringBuildX, SettingsPerLayer[0].ResizeDuringBuildY,
			AdjustedSizeX, AdjustedSizeY, AdjustedSizeZ);

		// For multi-block images, we may have scaled the max block size to be tile-sized, but individual blocks may still be smaller than 1 tile
		// These need to be scaled up as well (scaling up individual blocks has the effect of reducing the block's mip-bias)
		int32 LocalBlockSizeScaleX = DerivedInfo.BlockSizeScaleX;
		int32 LocalBlockSizeScaleY = DerivedInfo.BlockSizeScaleY;
		while (AdjustedSizeX * LocalBlockSizeScaleX < TileSize || AdjustedSizeY * LocalBlockSizeScaleY < TileSize)
		{
			check(BlockData.MipBias > 0u);
			--BlockData.MipBias;
			LocalBlockSizeScaleX *= 2;
			LocalBlockSizeScaleY *= 2;
		}

		// give each tile a unique DebugTexturePathName for DebugDump option :
		FString CurDebugTexturePathName = FString::Printf(TEXT("%s_L%d_B%d"), *DebugTexturePathName, LayerIndex, BlockIndex);

		// Use the texture compressor module to do all the hard work
		// this is the Build to Uncompressed to apply processing to create the source for the tiles
		TArray<FCompressedImage2D> CompressedMips;
		bool bBuildTextureResult = false;
		if (LocalBlockSizeScaleX == 1 && LocalBlockSizeScaleY == 1)
		{
			uint32 NumMipsInTail, ExtData;
			bBuildTextureResult = Compressor->BuildTexture(SourceMips, *CompositeSourceMips, TBSettings, CurDebugTexturePathName, CompressedMips, NumMipsInTail, ExtData, nullptr);

			// BuildTexture can free mips, they are no longer valid
			SourceMips.Empty();
			CompositeSourceMips->Empty();
		}
		else
		{
			// Need to generate scaled source images before building mips
			// Typically this is only needed to scale very small source images to be at least tile-sized, so performance shouldn't be a big concern here
			TArray<FImage> ScaledSourceMips;
			TArray<FImage> ScaledCompositeMips;
			ScaledSourceMips.Reserve(SourceMips.Num());
			ScaledCompositeMips.Reserve(CompositeSourceMips->Num());
			for (const FImage& SrcMip : SourceMips)
			{
				FImage* ScaledMip = new(ScaledSourceMips) FImage;
				// Pow22 cannot be used as a destination gamma, so change it to sRGB now :
				EGammaSpace GammaSpace = (SrcMip.GammaSpace == EGammaSpace::Pow22) ? EGammaSpace::sRGB : SrcMip.GammaSpace;
				SrcMip.ResizeTo(*ScaledMip, SrcMip.SizeX * LocalBlockSizeScaleX, SrcMip.SizeY * LocalBlockSizeScaleY, SrcMip.Format, GammaSpace);
			}

			for (const FImage& SrcMip : *CompositeSourceMips)
			{
				FImage* ScaledMip = new(ScaledCompositeMips) FImage;
				// Pow22 cannot be used as a destination gamma, so change it to sRGB now :
				EGammaSpace GammaSpace = (SrcMip.GammaSpace == EGammaSpace::Pow22) ? EGammaSpace::sRGB : SrcMip.GammaSpace;
				SrcMip.ResizeTo(*ScaledMip, SrcMip.SizeX * LocalBlockSizeScaleX, SrcMip.SizeY * LocalBlockSizeScaleY, SrcMip.Format, GammaSpace);
			}

			// Pow22 was converted to sRGB by Resize :
			TBSettings.bUseLegacyGamma = false;

			uint32 NumMipsInTail, ExtData;
			bBuildTextureResult = Compressor->BuildTexture(ScaledSourceMips, ScaledCompositeMips, TBSettings, CurDebugTexturePathName, CompressedMips, NumMipsInTail, ExtData, nullptr);
		}

		if (UE::Tasks::FCancellationTokenScope::IsCurrentWorkCanceled())
		{
			return;
		}

		check(bBuildTextureResult);

		// Get size of block from Compressor output, since it may have been padded/adjusted
		{
			BlockData.SizeX = CompressedMips[0].SizeX;
			BlockData.SizeY = CompressedMips[0].SizeY;

			// re-compute mip bias to account for any resizing of this block (typically due to clamped max size)
			const int32 MipBiasX = FMath::CeilLogTwo(DerivedInfo.BlockSizeX / BlockData.SizeX);
			const int32 MipBiasY = FMath::CeilLogTwo(DerivedInfo.BlockSizeY / BlockData.SizeY);
			checkf(MipBiasX == MipBiasY, TEXT("Mismatched aspect ratio (%d x %d), (%d x %d)"), DerivedInfo.BlockSizeX, DerivedInfo.BlockSizeY, BlockData.SizeX, BlockData.SizeY);
			BlockData.MipBias = MipBiasX;
		}

		check(BlockData.SizeX << BlockData.MipBias == DerivedInfo.BlockSizeX);
		check(BlockData.SizeY << BlockData.MipBias == DerivedInfo.BlockSizeY);

		// Use actual block size (not the the one UDIM's passe here) to determine how many
		// mips you'll have. As different blocks can be smaller than full UDIM block size
		const uint32 BlockSizeXY = FMath::Min(BlockData.SizeX, BlockData.SizeY);
		if (NumBlocks == 1u)
		{
			const uint32 MaxMipInBlockXY = FMath::CeilLogTwo(BlockSizeXY);
			BlockData.NumMips = FMath::Min<int32>(CompressedMips.Num(), MaxMipInBlockXY + 1);
		}
		else
		{
			const uint32 BlockSizeInTilesXY = FMath::DivideAndRoundUp<uint32>(BlockSizeXY, TileSize);
			const uint32 MaxMipInBlockXY = FMath::CeilLogTwo(BlockSizeInTilesXY);
			BlockData.NumMips = FMath::Min<int32>(CompressedMips.Num(), MaxMipInBlockXY + 1);
		}

		BlockData.Mips.Reserve(BlockData.NumMips);
		for (int32 MipIndex = 0; MipIndex < BlockData.NumMips; ++MipIndex)
		{
			FCompressedImage2D& CompressedMip = CompressedMips[MipIndex];
			check(CompressedMip.PixelFormat == LayerData.PixelFormat);
			FImage* Image = new(BlockData.Mips) FImage();
			Image->SizeX = CompressedMip.SizeX;
			Image->SizeY = CompressedMip.SizeY;
			Image->Format = LayerData.ImageFormat;
			Image->GammaSpace = LayerData.GammaSpace;
			Image->NumSlices = 1;
			check(Image->IsImageInfoValid());
			Image->RawData = MoveTemp(CompressedMip.RawData);
		}

		if (bNeedsMiptailBlock)
		{
			const FImage& SrcMipImage = BlockData.Mips[MaxMipInBlock - BlockData.MipBias];
			check(SrcMipImage.SizeX == MipWidthInBlock);
			check(SrcMipImage.SizeY == MipHeightInBlock);

			FPixelDataRectangle SrcPixelData(LayerData.SourceFormat, SrcMipImage.SizeX, SrcMipImage.SizeY, const_cast<uint8*>(SrcMipImage.RawData.GetData()));
			MiptailPixelData.CopyRectangle(BlockData.BlockX * MipWidthInBlock, BlockData.BlockY * MipHeightInBlock, SrcPixelData, 0, 0, MipWidthInBlock, MipHeightInBlock);
		}
		else
		{
			// Extract fallback color from last mip.
			{
				// this actually just samples one pixel ; it comes from last mip so it's often small already
				// @todo Oodle : just use a "get average color" function
				FImage OnePixelImage(1, 1, 1, ERawImageFormat::RGBA32F);
				BlockData.Mips.Last().ResizeTo(OnePixelImage, 1, 1, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
				OutData.LayerFallbackColors[LayerIndex] = OnePixelImage.AsRGBA32F()[0];
			}
		}

		BuildBlockTiles(LayerIndex, BlockIndex, BlockData, LayerData, bAllowAsync);

		// Mips not needed anymore
		BlockData.Mips.Empty();

		// SourceData for this Block & Layer not needed anymore, can free SourceData (and CompositeSourceData) now
		SourceData.Blocks[BlockIndex].MipsPerLayer[LayerIndex].Empty();
		if (CompositeSourceData.Blocks.Num() > 0)
		{
			CompositeSourceData.Blocks[BlockIndex].MipsPerLayer[LayerIndex].Empty();
		}
	}

	if (bNeedsMiptailBlock)
	{
#if SAVE_TILES
		{
			FString DebugName = FPaths::MakeValidFileName(*DebugTexturePathName, TEXT('_'));
			const FString BasePath = FPaths::ProjectUserDir();
			const FString MipFileName = BasePath / FString::Format(TEXT("{0}_{1}.png"), TArray<FStringFormatArg>({ *DebugName, LayerIndex }));
			MiptailPixelData.Save(MipFileName, ImageWrapper);
		}
#endif // SAVE_TILES

		FVTBlockPayload& BlockData = LayerPayload[LayerIndex].Blocks.Last();
		BlockData.BlockX = 0;
		BlockData.BlockY = 0;
		BlockData.SizeInBlocksX = DerivedInfo.SizeInBlocksX; // miptail block covers the entire logical source texture
		BlockData.SizeInBlocksY = DerivedInfo.SizeInBlocksY;
		BlockData.SizeX = FMath::Max(MipInputSizeX >> 1, 1u);
		BlockData.SizeY = FMath::Max(MipInputSizeY >> 1, 1u);
		BlockData.NumMips = OutData.NumMips - MaxMipInBlock - 1;
		BlockData.NumSlices = 1; // TODO?
		BlockData.MipBias = MaxMipInBlock + 1;

		const FTextureBuildSettings& BuildSettingsForLayer = SettingsPerLayer[LayerIndex];

		// Adjust the build settings to generate an uncompressed texture with mips but leave other settings
		// like color correction, ... in place
		FTextureBuildSettings TBSettings = SettingsPerLayer[0];
		TBSettings.MaxTextureResolution = FTextureBuildSettings::MaxTextureResolutionDefault;; // don't limit the size of the mip-tail, this limit only applies to each source block
		TBSettings.TextureFormatName = LayerData.FormatName;
		TBSettings.BaseTextureFormatName = LayerData.FormatName; // VTs never have platform prefix
		TBSettings.bSRGB = BuildSettingsForLayer.bSRGB;
		TBSettings.bUseLegacyGamma = BuildSettingsForLayer.bUseLegacyGamma;
		TBSettings.PowerOfTwoMode = ETexturePowerOfTwoSetting::None; // no resizing - that's for source blocks only, miptail block size is already set up to be a power of two

		// Make sure the output of the texture builder is in the same gamma space as we expect it.
		check(TBSettings.GetDestGammaSpace() == BuildSettingsForLayer.GetDestGammaSpace());

		// Leave original mip settings alone unless it's none at which point we will just generate them using a simple average
		if (TBSettings.MipGenSettings == TMGS_NoMipmaps || TBSettings.MipGenSettings == TMGS_LeaveExistingMips)
		{
			TBSettings.MipGenSettings = TMGS_SimpleAverage;
		}

		// give each tile a unique DebugTexturePathName for DebugDump option :
		FString CurDebugTexturePathName = FString::Printf(TEXT("%s_L%d_MT"), *DebugTexturePathName, LayerIndex);

		// Use the texture compressor module to do all the hard work
		// TODO - composite images?
		TArray<FCompressedImage2D> CompressedMips;
		uint32 NumMipsInTail, ExtData;
		// this is a Build to uncompressed, to apply processing
		TArray<FImage> EmptyImageArray;
		if (!Compressor->BuildTexture(MiptailInputImages, EmptyImageArray, TBSettings, CurDebugTexturePathName, CompressedMips, NumMipsInTail, ExtData, nullptr))
		{
			check(false);
		}

		MiptailInputImages.Empty();

		// We skip the first compressed mip output, since that will just be a copy of the input
		check(CompressedMips.Num() >= BlockData.NumMips + 1);
		// not true with padding options :
		//check(BlockData.SizeX == CompressedMips[1].SizeX);
		//check(BlockData.SizeY == CompressedMips[1].SizeY);

		BlockData.Mips.Reserve(CompressedMips.Num() - 1);
		for (int32 MipIndex = 1; MipIndex < BlockData.NumMips + 1; ++MipIndex)
		{
			FCompressedImage2D& CompressedMip = CompressedMips[MipIndex];
			check(CompressedMip.PixelFormat == LayerData.PixelFormat);
			FImage* Image = new(BlockData.Mips) FImage();
			Image->SizeX = CompressedMip.SizeX;
			Image->SizeY = CompressedMip.SizeY;
			Image->Format = LayerData.ImageFormat;
			Image->GammaSpace = LayerData.GammaSpace;
			Image->NumSlices = 1;
			check(Image->IsImageInfoValid());
			Image->RawData = MoveTemp(CompressedMip.RawData);
		}

		BuildBlockTiles(LayerIndex, NumBlocks, BlockData, LayerData, bAllowAsync);

		// Extract fallback color from last mip.
		{
			// this actually just samples one pixel ; it comes from last mip so it's often small already
			// @todo Oodle : just use a "get average color" function
			FImage OnePixelImage(1, 1, 1, ERawImageFormat::RGBA32F);
			BlockData.Mips.Last().ResizeTo(OnePixelImage, 1, 1, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
			OutData.LayerFallbackColors[LayerIndex] = OnePixelImage.AsRGBA32F()[0];
		}
	}
}

bool FVirtualTextureDataBuilder::BuildPagesForChunk(const TArray<FVTSourceTileEntry>& ActiveTileList)
{
	TArray<FLayerData> LayerData;
	LayerData.AddDefaulted(LayerPayload.Num());

	for (int32 LayerIndex = 0; LayerIndex < LayerData.Num(); LayerIndex++)
	{
		BuildTiles(ActiveTileList, LayerIndex, LayerData[LayerIndex]);
	}

	// Fill out tile offsets per layer if we haven't yet and if all layers are raw uncompressed data.
	if (OutData.TileDataOffsetPerLayer.Num() == 0)
	{
		bool bIsRawGPUData = true;
		for (int32 LayerIndex = 0; LayerIndex < LayerData.Num(); LayerIndex++)
		{
			if (LayerData[LayerIndex].Codec != EVirtualTextureCodec::RawGPU)
			{
				bIsRawGPUData = false;
				break;
			}
		}
		if (bIsRawGPUData)
		{
			int64 TileDataOffset = 0;
			OutData.TileDataOffsetPerLayer.Reserve(LayerData.Num());
			for (int32 LayerIndex = 0; LayerIndex < LayerData.Num(); LayerIndex++)
			{
				TileDataOffset += LayerData[LayerIndex].TilePayload[0].Num();
				OutData.TileDataOffsetPerLayer.Add( IntCastChecked<uint32>(TileDataOffset) );
			}
		}
	}

	// Write tiles out to chunk.
	return PushDataToChunk(ActiveTileList, LayerData);
}

void FVirtualTextureDataBuilder::BuildTiles(const TArray<FVTSourceTileEntry>& TileList, uint32 LayerIndex, FLayerData& GeneratedData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Texture.VT.BuildTiles);

	FThreadSafeBool bCompressionError = false;
	EPixelFormat CompressedFormat = PF_Unknown;
	const int32 TileSize = SettingsPerLayer[0].VirtualTextureTileSize;
	const int32 BlockSizeInTilesX = FMath::DivideAndRoundUp(DerivedInfo.BlockSizeX, TileSize);
	const int32 BlockSizeInTilesY = FMath::DivideAndRoundUp(DerivedInfo.BlockSizeY, TileSize);

	{
		GeneratedData.TilePayload.AddDefaulted(TileList.Num());
		
		for (int32 TileIndex = 0; TileIndex < TileList.Num(); TileIndex++)
		{
			const FVTSourceTileEntry& Tile = TileList[TileIndex];
			FVTBlockPayload& Block = LayerPayload[LayerIndex].Blocks[Tile.BlockIndex];

			int32 MipIndex = Tile.MipIndex - Block.MipBias;
			int32 BlockWidth = FMath::Max(Block.SizeX >> MipIndex, 1);
			int32 BlockWidthInTiles = FMath::DivideAndRoundUp(BlockWidth, TileSize);

			int32 MipBlockSizeInTilesX = FMath::Max(BlockSizeInTilesX >> Tile.MipIndex, 1);
			int32 MipBlockSizeInTilesY = FMath::Max(BlockSizeInTilesY >> Tile.MipIndex, 1);
			
			int32 TileInBlockX = Tile.TileX - Block.BlockX * MipBlockSizeInTilesX;
			int32 TileInBlockY = Tile.TileY - Block.BlockY * MipBlockSizeInTilesY;
			int32 TileIndexInBlock = TileInBlockY * BlockWidthInTiles + TileInBlockX;

			FVTTileMipPayload& MipPayload = Block.Tiles[TileIndexInBlock].Mips[Tile.MipIndexInBlock];
			GeneratedData.TilePayload[TileIndex] = MoveTemp(MipPayload.Payload);
			CompressedFormat = MipPayload.CompressedFormat;
		}

		GeneratedData.Codec = EVirtualTextureCodec::RawGPU;
	}

	if (OutData.LayerTypes[LayerIndex] == EPixelFormat::PF_Unknown)
	{
		OutData.LayerTypes[LayerIndex] = CompressedFormat;
	}
	else
	{
		checkf(OutData.LayerTypes[LayerIndex] == CompressedFormat, TEXT("The texture compressor used a different pixel format for some tiles."));
	}

	if (bCompressionError)
	{
		GeneratedData.TilePayload.Empty();
		GeneratedData.CodecPayload.Empty();
		GeneratedData.Codec = EVirtualTextureCodec::Max;
		UE_LOGF(LogVirtualTexturing, Fatal, "Failed build tile");
	}
	else
	{
		int CodecPayloadSize = GeneratedData.CodecPayload.Num();
		int64 TilePayloadTotalSize = 0;
		for(int i=0;i<GeneratedData.TilePayload.Num();i++)
		{
			TilePayloadTotalSize += GeneratedData.TilePayload[i].Num();
		}

		UE_LOGF(LogVirtualTexturing, Verbose, "VT CodecPayloadSize = %d TilePayloadTotalSize = %lld", CodecPayloadSize, TilePayloadTotalSize );
	}
}

bool FVirtualTextureDataBuilder::PushDataToChunk(const TArray<FVTSourceTileEntry>& Tiles, const TArray<FLayerData>& LayerData)
{
	const int32 NumLayers = LayerPayload.Num();

	int64 TotalSize = sizeof(FVirtualTextureChunkHeader);
	for (int32 Layer = 0; Layer < NumLayers; ++Layer)
	{
		TotalSize += LayerData[Layer].CodecPayload.Num();
		for (const TArray<uint8>& TilePayload : LayerData[Layer].TilePayload)
		{
			TotalSize += TilePayload.Num();
		}
	}
	
	// Built VT data structures use uint32 :
	if ( TotalSize >= MAX_uint32 )
	{
		UE_LOGF(LogVirtualTexturing,Error,"Cannot build VT; data bigger than 4 GB : %lld",TotalSize);
		return false;
	}

	FVirtualTextureDataChunk& Chunk = OutData.Chunks.AddDefaulted_GetRef();
	Chunk.SizeInBytes = TotalSize;
	FByteBulkData& BulkData = Chunk.BulkData;
	BulkData.Lock(LOCK_READ_WRITE);
	uint8* NewChunkData = (uint8*)BulkData.Realloc(TotalSize);
	int64 ChunkOffset = 0;

	// Header for the chunk
	FVirtualTextureChunkHeader* Header = (FVirtualTextureChunkHeader*)NewChunkData;
	FMemory::Memzero(*Header);

	ChunkOffset += sizeof(FVirtualTextureChunkHeader);

	// codec payloads
	for (int32 Layer = 0; Layer < NumLayers; ++Layer)
	{
		Chunk.CodecPayloadOffset[Layer] = IntCastChecked<uint32>( ChunkOffset );
		Chunk.CodecType[Layer] = LayerData[Layer].Codec;
		if (LayerData[Layer].CodecPayload.Num() > 0)
		{
			FMemory::Memcpy(NewChunkData + ChunkOffset, LayerData[Layer].CodecPayload.GetData(), LayerData[Layer].CodecPayload.Num());
			ChunkOffset += LayerData[Layer].CodecPayload.Num();
		}
	}
	Chunk.CodecPayloadSize = ChunkOffset;

	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); ++TileIdx)
	{
		const FVTSourceTileEntry& Tile = Tiles[TileIdx];
		const int32 MipIndex = Tile.MipIndex;
		// Set BaseOffsetPerMip from the first tile we find for the MipIndex.
		if (OutData.BaseOffsetPerMip[MipIndex] == ~0u)
		{
			OutData.BaseOffsetPerMip[MipIndex] = IntCastChecked<uint32>( ChunkOffset );
		}
		int32 TileIndex = Tile.TileIndex;
		for (int32 Layer = 0; Layer < NumLayers; ++Layer)
		{
			check(OutData.TileOffsetInChunk[TileIndex] == ~0u);
			OutData.TileOffsetInChunk[TileIndex] = IntCastChecked<uint32>( ChunkOffset );
			++TileIndex;

			const TArray<uint8>& TilePayload = LayerData[Layer].TilePayload[TileIdx];
			const uint32 Size = TilePayload.Num();
			check(Size > 0u);

			FMemory::Memcpy(NewChunkData + ChunkOffset, TilePayload.GetData(), Size);
			ChunkOffset += Size;
		}
	}

	check(ChunkOffset == TotalSize);

	FSHA1::HashBuffer(NewChunkData, TotalSize, Chunk.BulkDataHash.Hash);

#if SAVE_CHUNKS
	{
		FString DebugName = FPaths::MakeValidFileName(*DebugTexturePathName, TEXT('_'));
		FString BasePath = FPaths::ProjectUserDir();
		FString Name = BasePath / FString::Format(TEXT("chunk_{0}_{1}.bin"), TArray<FStringFormatArg>({ ChunkDumpIndex++, Chunk.BulkDataHash.ToString() }));
		FFileHelper::SaveArrayToFile(TArrayView<uint8> { NewChunkData, (int32)TotalSize }, * Name);
	}
#endif

	BulkData.Unlock();
	BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);

	return true;
}

int32 FVirtualTextureDataBuilder::FindSourceBlockIndex(int32 MipIndex, int32 BlockX, int32 BlockY)
{
	// VT assumes that layer 0 is largest layer when assigning block to tiles
	TArray<FVTBlockPayload>& Blocks = LayerPayload[0].Blocks;
	for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); ++BlockIndex)
	{
		FVTBlockPayload& Block = Blocks[BlockIndex];
		if (BlockX >= Block.BlockX && BlockX < Block.BlockX + Block.SizeInBlocksX &&
			BlockY >= Block.BlockY && BlockY < Block.BlockY + Block.SizeInBlocksY &&
			MipIndex >= Block.MipBias &&
			(MipIndex - Block.MipBias) < Block.NumMips)
		{
			return BlockIndex;
		}
	}
	return INDEX_NONE;
}

// Leaving this code here for now, in case we want to build a new/better system for creating/storing miptails
#if 0
void FVirtualTextureDataBuilder::BuildMipTails()
{
	OutData.MipTails.SetNum(Settings.Layers.Num());

	for (int32 Layer = 0; Layer < Settings.Layers.Num(); Layer++)
	{
		TArray<FImage> SourceList;
		TArray<FImage> EmptyList;

		int32 NumTailMips = SourcePixels[Layer].Num() - NumMips;

		// Make a list of mips to pass to the compressor
		for (int TailMip = 0; TailMip < NumTailMips; TailMip++)
		{
			FImage *TailMipImage = new (SourceList) FImage(*SourcePixels[Layer][TailMip + NumMips]);
		}

		// Adjust the build settings
		// The pixels we have already include things like color correction, mip blurring, ... so we just start
		// from pristine build settings here
		FTextureBuildSettings TBSettings;
		TBSettings.MaxTextureResolution = FTextureBuildSettings::MaxTextureResolutionDefault;;
		TBSettings.TextureFormatName = "BGRA8";
		TBSettings.bSRGB = Settings.Layers[Layer].SourceBuildSettings.bSRGB;
		TBSettings.bUseLegacyGamma = Settings.Layers[Layer].SourceBuildSettings.bUseLegacyGamma;
		TBSettings.MipGenSettings = TMGS_LeaveExistingMips;

		check(TBSettings.GetGammaSpace() == Settings.Layers[Layer].GammaSpace);

		TArray<FCompressedImage2D> CompressedMips;
		uint32 NumMipsInTail, ExtData;
		if (!Compressor->BuildTexture(SourceList, EmptyList, TBSettings, CompressedMips, NumMipsInTail, ExtData))
		{
			check(false);
		}

		OutData.MipTails[Layer].Empty();

		for (int Mip = 0; Mip < CompressedMips.Num(); Mip++)
		{
			check(CompressedMips[Mip].PixelFormat == EPixelFormat::PF_B8G8R8A8);
			OutData.MipTails[Layer].AddDefaulted();
			OutData.MipTails[Layer].Last().SizeX = CompressedMips[Mip].SizeX;
			OutData.MipTails[Layer].Last().SizeY = CompressedMips[Mip].SizeY;
			OutData.MipTails[Layer].Last().SizeZ = 1;
			OutData.MipTails[Layer].Last().Data = CompressedMips[Mip].RawData;
		}
	}
}
#endif // 0

#endif // WITH_EDITOR
