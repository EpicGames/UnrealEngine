// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "ImageCore.h"
#include "VT/VirtualTextureBuiltData.h"

struct FImage;
class ITextureCompressorModule;
class IImageWrapperModule;
struct FTextureSourceData;
struct FTextureSourceBlockData;
struct FTextureBuildSettings;
struct FSlowTask;

struct FVTTileMipPayload
{
	EPixelFormat CompressedFormat = EPixelFormat::PF_Unknown;
	TArray<uint8> Payload;
};

struct FVTTilePayload
{
	TArray<FVTTileMipPayload> Mips;
};

struct FVTBlockPayload
{
	// Block size in pixels (each block can be different size)
	int32 SizeX = 0;
	int32 SizeY = 0;

	// Block coordinate (in block units) in texture where block is located
	int32 BlockX = 0;
	int32 BlockY = 0;

	// Normally each blocks covers a 1x1 block area in output, but it can cover more.
	// For example, miptail covers multiple input blocks. Thease are in block units.
	int32 SizeInBlocksX = 1;
	int32 SizeInBlocksY = 1;

	int32 MipBias = 0; // First mip level in block that contains actual data (because blocks can be smaller than others)
	int32 NumMips = 0; // Count of mips (may be less than source texture reqests because of MipBias)
	int32 NumSlices = 0;
	TArray<FVTTilePayload> Tiles;
	TArray<FImage> Mips;
};

struct FVTLayerPayload
{
	TArray<FVTBlockPayload> Blocks;
};

struct FVTSourceTileEntry
{
	int32 BlockIndex = 0;
	int32 TileIndex = 0;
	int32 MipIndex = 0;
	int32 MipIndexInBlock = 0;
	int32 TileX = 0;
	int32 TileY = 0;
};

struct FLayerData
{
	TArray<TArray<uint8>> TilePayload;
	TArray<uint8> CodecPayload;
	EVirtualTextureCodec Codec = EVirtualTextureCodec::Max;
};

struct FVirtualTextureSourceLayerData
{
	// All of these should refer to the same format
	ERawImageFormat::Type ImageFormat;
	ETextureSourceFormat SourceFormat;
	EPixelFormat PixelFormat;
	FName FormatName;
	FName TextureFormatName;

	EGammaSpace GammaSpace;
	bool bHasAlpha;
};

// Holds a bunch of stuff we derive from the input data that we use during the build.
struct FVirtualTextureBuilderDerivedInfo
{
	int32 SizeInBlocksX = 0;
	int32 SizeInBlocksY = 0;
	int32 BlockSizeX = 0;
	int32 BlockSizeY = 0;
	int32 BlockSizeScaleX = 1;
	int32 BlockSizeScaleY = 1;
	int32 SizeX = 0;
	int32 SizeY = 0;
	int32 NumMips = 0;

	bool InitializeFromBuildSettings(const FTextureSourceData& InSourceData, const FTextureBuildSettings* InSettingsPerLayer);
};

/**
 * Helper class for building virtual texture data. This works on a set of FTextureSource objects. The idea is that if needed we can create
 * FTextureSource without creating actual UTextures. This is why the builder should stay independent of UTexture. Things it does:
 * - Splits texture into tiles
 * - Preprocesses the tiles
 * - Bakes mips
 * - Does compression
 * Note: Most of the heavy pixel processing itself is internally deferred to the TextureCompressorModule.
 *
 * Data is cached in the BuilderObject so the BuildLayer call is not thread safe between calls. Create separate FVirtualTextureDataBuilder
 * instances for each thread instead!
 *
 * Current assumptions:
 * - We can keep "at least" all the source data in memory. We do not do "streaming" conversions of source data.
 * - Output can be "streaming" we don't have to keep all the data output in memory
 */
class FVirtualTextureDataBuilder
{
public:
	FVirtualTextureDataBuilder(FVirtualTextureBuiltData &SetOutData, const FString& DebugTexturePathName, ITextureCompressorModule *InCompressor = nullptr, IImageWrapperModule* InImageWrapper = nullptr);
	~FVirtualTextureDataBuilder();

	// note: InSourceData is freed by this function
	bool Build(FTextureSourceData& InSourceData, FTextureSourceData& InCompositeSourceData, const FTextureBuildSettings* InSettingsPerLayer, bool bAllowAsync);

private:
	friend struct FAsyncMacroBlockTask;

	bool BuildPagesForChunk(const TArray<FVTSourceTileEntry>& ActiveTileList);
	void BuildTiles(const TArray<FVTSourceTileEntry>& TileList, uint32 layer, FLayerData& GeneratedData);
	bool PushDataToChunk(const TArray<FVTSourceTileEntry>& Tiles, const TArray<FLayerData>& LayerData);

	int32 FindSourceBlockIndex(int32 MipIndex, int32 BlockX, int32 BlockY);

	void BuildLayerBlocks(FSlowTask& BuildTask, uint32 LayerIndex, const FVirtualTextureSourceLayerData& LayerData, FTextureSourceData& SourceData, FTextureSourceData& CompositeSourceData, bool bAllowAsync);
	void BuildBlockTiles(uint32 LayerIndex, uint32 BlockIndex, FVTBlockPayload& Block, const FVirtualTextureSourceLayerData& LayerData, bool bAllowAsync);
	bool BuildChunks();

	TArray<FVTLayerPayload> LayerPayload;

	// Cached inside this object
	TArray<FTextureBuildSettings> SettingsPerLayer;
	FVirtualTextureBuiltData &OutData;

	// Some convenience variables (mostly derived from the passed in build settings)
	FVirtualTextureBuilderDerivedInfo DerivedInfo;

	ITextureCompressorModule *Compressor;
	IImageWrapperModule *ImageWrapper;

	const FString& DebugTexturePathName;
	int32 ChunkDumpIndex = 0;
};