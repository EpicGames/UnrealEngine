// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureLevelRedirector.h"

FVirtualTextureLevelRedirector::FVirtualTextureLevelRedirector(IVirtualTexture* InVirtualTexture0, IVirtualTexture* InVirtualTexture1, int32 InTransitionLevel)
	: VirtualTextures{ InVirtualTexture0, InVirtualTexture1 }
	, TransitionLevel(InTransitionLevel)
{
}

FVirtualTextureLevelRedirector::~FVirtualTextureLevelRedirector()
{
	delete VirtualTextures[0];
	delete VirtualTextures[1];
}

bool FVirtualTextureLevelRedirector::IsPageStreamed(uint8 vLevel, uint32 vAddress) const
{
	const int32 VirtualTextureIndex = vLevel < TransitionLevel ? 0 : 1;
	const int32 vLevelOffset = vLevel < TransitionLevel ? 0 : TransitionLevel;
	return VirtualTextures[VirtualTextureIndex]->IsPageStreamed(vLevel - vLevelOffset, vAddress);
}

FVTRequestPageResult FVirtualTextureLevelRedirector::RequestPageData(
	FRHICommandListBase& RHICmdList,
	const FVirtualTextureProducerHandle& ProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	EVTRequestPagePriority Priority)
{
	int32 VirtualTextureIndex = vLevel < TransitionLevel ? 0 : 1;
	int32 vLevelOffset = vLevel < TransitionLevel ? 0 : TransitionLevel;
	return VirtualTextures[VirtualTextureIndex]->RequestPageData(RHICmdList, ProducerHandle, LayerMask, vLevel - vLevelOffset, vAddress, Priority);
}

IVirtualTextureFinalizer* FVirtualTextureLevelRedirector::ProducePageData(
	FRHICommandListBase& RHICmdList,
	ERHIFeatureLevel::Type FeatureLevel,
	EVTProducePageFlags Flags,
	const FVirtualTextureProducerHandle& ProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	uint64 RequestHandle,
	const FVTProduceTargetLayer* TargetLayers)
{
	int32 VirtualTextureIndex = vLevel < TransitionLevel ? 0 : 1;
	int32 vLevelOffset = vLevel < TransitionLevel ? 0 : TransitionLevel;
	return VirtualTextures[VirtualTextureIndex]->ProducePageData(RHICmdList, FeatureLevel, Flags, ProducerHandle, LayerMask, vLevel - vLevelOffset, vAddress, RequestHandle, TargetLayers);
}

void FVirtualTextureLevelRedirector::GatherProducePageDataTasks(
	FVirtualTextureProducerHandle const& ProducerHandle, 
	FGraphEventArray& InOutTasks) const
{
	VirtualTextures[0]->GatherProducePageDataTasks(ProducerHandle, InOutTasks);
	VirtualTextures[1]->GatherProducePageDataTasks(ProducerHandle, InOutTasks);
}

void FVirtualTextureLevelRedirector::GatherProducePageDataTasks(
	uint64 RequestHandle,
	FGraphEventArray& InOutTasks) const
{
	VirtualTextures[0]->GatherProducePageDataTasks(RequestHandle, InOutTasks);
	VirtualTextures[1]->GatherProducePageDataTasks(RequestHandle, InOutTasks);
}
