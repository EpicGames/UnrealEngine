// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTexturing.h"

/** 
 * IVirtualTexture implementation that redirects requests to one of two children depending on vLevel.
 * We take ownership of the referenced IVirtualTexture objects meaning that they are destroyed when this object is destroyed.
 */
class FVirtualTextureLevelRedirector : public IVirtualTexture
{
public:
	FVirtualTextureLevelRedirector(IVirtualTexture* InVirtualTexture0, IVirtualTexture* InVirtualTexture1, int32 InTransitionLevel);
	virtual ~FVirtualTextureLevelRedirector();

	//~ Begin IVirtualTexture Interface.
	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override;

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override;

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override;

	virtual void GatherProducePageDataTasks(
		FVirtualTextureProducerHandle const& ProducerHandle, 
		FGraphEventArray& InOutTasks
	) const override;

	virtual void GatherProducePageDataTasks(
		uint64 RequestHandle, 
		FGraphEventArray& InOutTasks
	) const override;
	//~ End IVirtualTexture Interface.

private:
	IVirtualTexture* VirtualTextures[2];
	int32 TransitionLevel;
};
