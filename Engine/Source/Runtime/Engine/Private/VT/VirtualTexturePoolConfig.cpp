// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTexturePoolConfig.h"

#include "Containers/Ticker.h"
#include "HAL/IConsoleManager.h"
#include "RenderingThread.h"
#include "VT/VirtualTextureRecreate.h"

#if WITH_EDITOR
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTexturePoolConfig)

#define LOCTEXT_NAMESPACE "VirtualTexturePool"

static TAutoConsoleVariable<float> CVarVTPoolSizeScale(
	TEXT("r.VT.PoolSizeScale"),
	1.0f,
	TEXT("Scale factor for virtual texture physical pool size.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability | ECVF_ExcludeFromPreview
);

static TAutoConsoleVariable<bool> CVarVTPoolAutoGrow(
	TEXT("r.VT.PoolAutoGrow"),
	false,
	TEXT("Enable physical pool growing on oversubscription."),
	ECVF_RenderThreadSafe
);

static int32 GPageTableMode = 1;
static FAutoConsoleVariableRef CVarVTPageTableMode(
	TEXT("r.VT.PageTableMode"),
	GPageTableMode,
	TEXT("Sets whether to use 16bit or 32bit page table entries. Defaults to mode 1.\n")
	TEXT("0: Always use 32bit bit page table entries.\n")
	TEXT("1: Use 16bit bit page table entries for physical pools with fewer than 4K pages which can be 16bit addressed. (Saves page table memory but can increase virtual texture space count).\n")
	TEXT("2: Try to force 16bit bit page table entries by splitting the physical pools to 4K page chunks. (Saves page table memory but with less efficient physical pool usage).\n"),
	ECVF_RenderThreadSafe);

static FAutoConsoleVariableDeprecated CVarVTSupport16BitPageTable_Deprecated(TEXT("r.VT.Support16BitPageTable"), TEXT("r.VT.PageTableMode"), TEXT("5.7"));

/** Track changes and apply to relevant systems. This allows us to dynamically change the scalability settings. */
void OnVirtualTexturePoolConfigUpdate()
{
	// CVar updates can come early in start up before UVirtualTexturePoolConfig is serialized.
	// That leads to UVirtualTexturePoolConfig not loading correctly.
	// We early out here to avoid that case (we will update all later on serialization anyway).
	if (!IsClassLoaded<UVirtualTexturePoolConfig>())
	{
		return;
	}

	const uint32 ConfigHash = VirtualTexturePool::GetConfigHash();
	static float LastConfigHash = ConfigHash;

	if (LastConfigHash != ConfigHash)
	{
		LastConfigHash = ConfigHash;
		VirtualTexture::Recreate();
	}
}

FAutoConsoleVariableSink GVirtualTexturePoolConfigCVarSink(FConsoleCommandDelegate::CreateStatic(&OnVirtualTexturePoolConfigUpdate));

/** Version number used to help track configuration changes. */
struct FVirtualTexturePoolVersion
{
	uint32 Version_GameThread = 0;
	uint32 Version_RenderThread = 0;

	uint32 Get() const
	{
		return IsInRenderingThread() ? Version_RenderThread : Version_GameThread;
	}

	void Increment_GameThread()
	{
		uint32 Version = ++Version_GameThread;
		ENQUEUE_RENDER_COMMAND(IncrementVersion)([this, Version](FRHICommandListImmediate& RHICmdList)
		{
			Version_RenderThread = Version;
		});
	}
};
FVirtualTexturePoolVersion GVirtualTexturePoolVersion;

static FAutoConsoleCommand CVarVTForceRecreatePoolCommand(
	TEXT("r.VT.ForceRecreatePool"),
	TEXT("Forcefully recreates the backing virtual pool"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>&)
	{
		GVirtualTexturePoolVersion.Increment_GameThread();
		OnVirtualTexturePoolConfigUpdate();
	})
);

UVirtualTexturePoolConfig::UVirtualTexturePoolConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UVirtualTexturePoolConfig::FindPoolConfig(TEnumAsByte<EPixelFormat> const* InFormats, int32 InNumLayers, int32 InTileSize, FVirtualTextureSpacePoolConfig& OutConfig) const
{
	// Create a default config that will be used if no other default config is found.
	FVirtualTextureSpacePoolConfig DefaultConfig;
	DefaultConfig.SizeInMegabyte = DefaultSizeInMegabyte;
	DefaultConfig.bEnableResidencyMipMapBias = true;
	DefaultConfig.ResidencyMipMapBiasGroup = 0;
	DefaultConfig.bAllowSizeScale = true;

	bool bFoundDefaultConfig = false;

	// First look in transient pool descriptions.
	for (int32 Id = TransientPools.Num() - 1; Id >= 0; Id--)
	{
		const FVirtualTextureSpacePoolConfig& Config = TransientPools[Id];
		if (Config.MinTileSize <= InTileSize && (Config.MaxTileSize == 0 || Config.MaxTileSize >= InTileSize) && InNumLayers == Config.Formats.Num())
		{
			bool bAllFormatsMatch = true;
			for (int Layer = 0; Layer < InNumLayers && bAllFormatsMatch; ++Layer)
			{
				if (InFormats[Layer] != Config.Formats[Layer])
				{
					bAllFormatsMatch = false;
				}
			}

			if (bAllFormatsMatch)
			{
				OutConfig = Config;
				return;
			}
		}
	}

	// Now look in serialized pool descriptions.
	// Note that we reverse iterate so that project config can override base config.
	for (int32 Id = Pools.Num() - 1; Id >= 0; Id--)
	{
		const FVirtualTextureSpacePoolConfig& Config = Pools[Id];
		if (Config.MinTileSize <= InTileSize && (Config.MaxTileSize == 0 || Config.MaxTileSize >= InTileSize) && InNumLayers == Config.Formats.Num())
		{
			bool bAllFormatsMatch = true;
			for (int Layer = 0; Layer < InNumLayers && bAllFormatsMatch; ++Layer)
			{
				if (InFormats[Layer] != Config.Formats[Layer])
				{
					bAllFormatsMatch = false;
				}
			}

			if (bAllFormatsMatch)
			{
				OutConfig = Config;
				return;
			}
		}

		if (!bFoundDefaultConfig && Config.IsDefault())
		{
			DefaultConfig = Config;
			bFoundDefaultConfig = true;
		}
	}

	// Didn't find an exact match so return whatever default config that we first found.
	OutConfig = DefaultConfig;
}

bool UVirtualTexturePoolConfig::AddOrModifyTransientPoolConfig(FVirtualTextureSpacePoolConfig const& InConfig)
{
	for (int32 Index = TransientPools.Num() - 1; Index >= 0; Index--)
	{
		FVirtualTextureSpacePoolConfig& Config = TransientPools[Index];
		if (InConfig.Formats == Config.Formats && InConfig.MaxTileSize == Config.MaxTileSize && InConfig.MinTileSize == Config.MinTileSize)
		{
			if (Config.SizeInMegabyte != InConfig.SizeInMegabyte)
			{
				Config.SizeInMegabyte = InConfig.SizeInMegabyte;
				GVirtualTexturePoolVersion.Increment_GameThread();
				return true;
			}

			// No change.
			return false;
		}
	}

	GVirtualTexturePoolVersion.Increment_GameThread();
	TransientPools.Add(InConfig);
	return true;
}

#if WITH_EDITOR

void UVirtualTexturePoolConfig::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GVirtualTexturePoolVersion.Increment_GameThread();
	VirtualTexture::Recreate();
}

#endif


/**
 * Modify the virtual texture physical pool configs.
 * This needs queuing up on the game thread so that the virtual texture UObjects can be reinitialized correctly.
 */
static void AddOrModifyTransientPoolConfigs_GameThread(TArray<FVirtualTextureSpacePoolConfig> const& InConfigs)
{
	check(IsInGameThread());

	TArray<int32, TInlineAllocator<8>> ChangedConfigs;

	UVirtualTexturePoolConfig* PoolConfig = GetMutableDefault<UVirtualTexturePoolConfig>();
	for (int32 ConfigIndex = 0; ConfigIndex < InConfigs.Num(); ++ConfigIndex)
	{
		if (PoolConfig->AddOrModifyTransientPoolConfig(InConfigs[ConfigIndex]))
		{
			ChangedConfigs.Add(ConfigIndex);
		}
	}

	// Defer virtual texture recreation until all changes to config are done.
	for (int32 ConfigIndex : ChangedConfigs)
	{
		VirtualTexture::Recreate(InConfigs[ConfigIndex].Formats);
	}

#if WITH_EDITOR
	// Notify user about changes.
	if (ChangedConfigs.Num())
	{
		static TWeakPtr<class SNotificationItem> NotificationHandle;
		if (!NotificationHandle.IsValid())
		{
			FNotificationInfo Info(LOCTEXT("PoolResizeNotify", "Resizing Virtual Texture Pools."));
			Info.SubText = LOCTEXT("PoolResizeNotifySubtext", "Size changes are not saved by default. To keep changes, copy from 'Transient Pools' to 'Fixed Pools' in the Virtual Texture Pool settings.");
			Info.ExpireDuration = 8.0f;
			Info.HyperlinkText = LOCTEXT("OpenSettings", "Open Project Settings");
			Info.Hyperlink = FSimpleDelegate::CreateStatic([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("ProjectSettings")); });

			NotificationHandle = FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
#endif
}

void VirtualTexturePool::FindPoolConfig(TEnumAsByte<EPixelFormat> const* InFormats, int32 InNumLayers, int32 InTileSize, FVirtualTextureSpacePoolConfig& OutConfig)
{
	UVirtualTexturePoolConfig const* PoolConfig = GetDefault<UVirtualTexturePoolConfig>();
	PoolConfig->FindPoolConfig(InFormats, InNumLayers, InTileSize, OutConfig);
}

void VirtualTexturePool::AddOrModifyTransientPoolConfigs_RenderThread(TArray<FVirtualTextureSpacePoolConfig>& InConfigs)
{
	check(IsInRenderingThread());

	ExecuteOnGameThread(UE_SOURCE_LOCATION, [Configs = MoveTemp(InConfigs)]()
	{
		AddOrModifyTransientPoolConfigs_GameThread(Configs);
	});
}

float VirtualTexturePool::GetPoolSizeScale()
{
	return CVarVTPoolSizeScale.GetValueOnAnyThread();
}

bool VirtualTexturePool::GetPoolAutoGrow()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		UVirtualTexturePoolConfig const* PoolConfig = GetDefault<UVirtualTexturePoolConfig>();
		if (PoolConfig->bPoolAutoGrowInEditor)
		{
			return true;
		}
	}
#endif
	return CVarVTPoolAutoGrow.GetValueOnAnyThread();
}

bool VirtualTexturePool::Support16BitPageTables()
{
	return GPageTableMode != 0;
}

int32 VirtualTexturePool::GetSplitPhysicalPoolSize()
{
	return GPageTableMode == 2 ? 64 : 0;
}

uint32 VirtualTexturePool::GetConfigHash()
{
	uint32 Hash = GetTypeHash(GetPoolSizeScale());
	Hash = HashCombine(Hash, GetTypeHash(GetPoolAutoGrow()));
	Hash = HashCombine(Hash, GetTypeHash(GPageTableMode));
	Hash = HashCombine(Hash, GetTypeHash(GVirtualTexturePoolVersion.Get()));

	return Hash;
}

#undef LOCTEXT_NAMESPACE
