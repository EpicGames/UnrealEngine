// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureRecreate.h"

#include "EngineModule.h"
#include "RendererInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Engine/Texture.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "TextureResource.h"
#include "UObject/UObjectIterator.h"
#include "VirtualTexturing.h"
#include "Engine/VirtualTextureCollection.h"
#include "MaterialCache/IMaterialCacheTagProvider.h"
#include "VT/RuntimeVirtualTexture.h"

namespace VirtualTexture
{
	// Release all VT render resources here.
	// Assuming all virtual textures are released, then virtual texture pools will reach a zero ref count and release, which is needed for any pool size scale to be effective.
	// Note that for pool size scale changes, there will be a transition period (with high memory watermark) where new pools are created before old pools are released.
	void Recreate()
	{
		UE_LOGF(LogVirtualTexturing, Display, "Recreating virtual texture pools.");

		// Reinit streaming virtual textures.
		for (TObjectIterator<UTexture> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->IsCurrentlyVirtualTextured())
			{
				It->UpdateResource();
			}
		}

		// Reinit runtime virtual textures.
		for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			It->MarkRenderStateDirty();
		}

		// Reinit material cache virtual textures.
		for (TObjectIterator<UMaterialCacheVirtualTexture> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			It->RecreateAllocation();
		}
		
		for (TObjectIterator<UVirtualTextureCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			It->UpdateResource();
		}

		if (IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider())
		{
			TagProvider->NotifyAllTagSceneInvalidation();
		}
	}

	void Recreate(TConstArrayView< TEnumAsByte<EPixelFormat> > InFormat)
	{
		UE_LOGF(LogVirtualTexturing, Display, "Recreating virtual texture pools for formats.");

		// Reinit streaming virtual textures that match one of our passed in format arrays.
		for (TObjectIterator<UTexture> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->IsCurrentlyVirtualTextured())
			{
				if (FVirtualTexture2DResource* Resource = static_cast<FVirtualTexture2DResource*>(It->GetResource()))
				{
					bool bIsMatchingFormat = Resource->GetNumLayers() == InFormat.Num();
					for (int32 LayerIndex = 0; bIsMatchingFormat && LayerIndex < InFormat.Num(); LayerIndex++)
					{
						if (Resource->GetFormat(LayerIndex) != InFormat[LayerIndex])
						{
							bIsMatchingFormat = false;
							break;
						}
					}

					if (bIsMatchingFormat)
					{
						It->UpdateResource();
					}
				}
			}
		}

		// Reinit runtime virtual textures that match one of our passed in format arrays.
		for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			if (URuntimeVirtualTexture* VirtualTexture = It->GetVirtualTexture())
			{
				bool bIsMatchingFormat = VirtualTexture->GetLayerCount() == InFormat.Num();
				for (int32 LayerIndex = 0; bIsMatchingFormat && LayerIndex < InFormat.Num(); LayerIndex++)
				{
					if (VirtualTexture->GetLayerFormat(LayerIndex) != InFormat[LayerIndex])
					{
						bIsMatchingFormat = false;
						break;
					}
				}

				if (bIsMatchingFormat)
				{
					It->MarkRenderStateDirty();
				}
			}
		}

		// Reinit material cache virtual textures that match the formats
		for (TObjectIterator<UMaterialCacheVirtualTexture> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			FMaterialCacheTagLayout Layout = It->GetRuntimeLayout();

			if (Layout.Layers.Num() != InFormat.Num())
			{
				continue;
			}

			bool bFormatsMatch = true;
			for (int32 i = 0; i < Layout.Layers.Num(); i++)
			{
				if (Layout.Layers[i].StorageFormat != InFormat[i])
				{
					bFormatsMatch = false;
					break;
				}
			}

			if (bFormatsMatch)
			{
				It->RecreateAllocation();
			}
		}
		
		for (TObjectIterator<UVirtualTextureCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			if (InFormat.Num() == 1 && It->RuntimePixelFormat == InFormat[0])
			{
				It->UpdateResource();
			}
		}

		if (IMaterialCacheTagProvider* TagProvider = GetRendererModule().GetMaterialCacheTagProvider())
		{
			TagProvider->NotifyAllTagSceneInvalidation();
		}
	}
}
