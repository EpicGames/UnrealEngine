// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"

namespace VirtualTexture
{
	/** Reinitialize all virtual texture producers. */
	void Recreate();

	/** Reinitialize all virtual texture that match the passed in format. */
	void Recreate(TConstArrayView < TEnumAsByte<EPixelFormat> > InFormat);
}
