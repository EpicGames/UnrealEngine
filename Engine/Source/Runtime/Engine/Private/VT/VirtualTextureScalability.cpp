// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureScalability.h"

#include "HAL/IConsoleManager.h"
#include "VT/VirtualTexturePoolConfig.h"
#include "VT/VirtualTextureRecreate.h"

namespace VirtualTextureScalability
{
	static TAutoConsoleVariable<int32> CVarVTMaxUploadsPerFrame(
		TEXT("r.VT.MaxUploadsPerFrame"),
		2,
		TEXT("Max number of page uploads per frame in game"),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTMaxUploadsPerFrameStreaming(
		TEXT("r.VT.MaxUploadsPerFrame.Streaming"),
		32,
		TEXT("If positive, max number of page uploads per frame in game for streaming VT. Negative means no limit.\n")
		TEXT("If zero, SVTs won't be budgeted separately. They will be limited by r.VT.MaxUploadsPerFrame along with other types of VTs. This is the legacy behavior.\n")
		TEXT("This limit should be high if streaming pages is slow so that I/O requests are not throttled which can cause long delays to acquire page data."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarMaxPagesProducedPerFrame(
		TEXT("r.VT.MaxTilesProducedPerFrame"),
		32,
		TEXT("Max number of pages that can be produced per frame"),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTMaxContinuousUpdatesPerFrame(
		TEXT("r.VT.MaxContinuousUpdatesPerFrame"),
		1,
		TEXT("Max number of page uploads for pages that are already mapped."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

#if WITH_EDITOR
	static TAutoConsoleVariable<int32> CVarVTMaxUploadsPerFrameInEditor(
		TEXT("r.VT.MaxUploadsPerFrameInEditor"),
		32,
		TEXT("Max number of page uploads per frame when in editor"),
		ECVF_RenderThreadSafe
	);

	static TAutoConsoleVariable<int32> CVarVTMaxUploadsPerFrameStreamingInEditor(
		TEXT("r.VT.MaxUploadsPerFrameInEditor.Streaming"),
		500,
		TEXT("Max number of page uploads per frame in editor for streaming VT.")
		TEXT("Zero means no separate budget. Negative means no limit.\n"),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarMaxPagesProducedPerFrameInEditor(
		TEXT("r.VT.MaxTilesProducedPerFrameInEditor"),
		500,
		TEXT("Max number of pages that can be produced per frame when in editor"),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTMaxContinuousUpdatesPerFrameInEditor(
		TEXT("r.VT.MaxContinuousUpdatesPerFrameInEditor"),
		8,
		TEXT("Max number of page uploads for pages that are already mapped when in editor."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);
#endif

	static TAutoConsoleVariable<float> CVarVTMaxGatherTimePerFrame(
		TEXT("r.VT.MaxGatherTimePerFrame"),
		0,
		TEXT("Time slice limit in milliseconds used to limit time spent gathering virtual texture requests per frame.\n")
		TEXT("0 (default) disables."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<float> CVarVTMaxSubmitTimePerFrame(
		TEXT("r.VT.MaxSubmitTimePerFrame"),
		0,
		TEXT("Time slice limit in milliseconds used to limit time spent submitting virtual texture requests per frame.\n")
		TEXT("0 (default) disables."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTMaxReleasedPerFrame(
		TEXT("r.VT.MaxReleasedPerFrame"),
		5,
		TEXT("Max number of allocated virtual textures to release per frame in non editor builds."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVirtualTextureFeedbackFactor(
		TEXT("r.vt.FeedbackFactor"),
		16,
		TEXT("The size of the VT feedback buffer is calculated by dividing the render resolution by this factor.")
		TEXT("The value set here is rounded up to the nearest power of two before use."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTPageFreeThreshold(
		TEXT("r.VT.PageFreeThreshold"),
		15,
		TEXT("Number of frames since the last time a VT page was used, before it's considered free.\n")
		TEXT("VT pages are not necesarily marked as used on the CPU every time they're accessed by the GPU.\n")
		TEXT("Increasing this threshold reduces the chances that an in-use frame is considered free."),
		ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarVTKeepDirtyPageMappedFrameThreshold(
		TEXT("r.VT.RVT.DirtyPagesKeptMappedFrames"), 
		8,
		TEXT("Number of frames since the last time a VT page was used, before it will be unmapped instead of simply updated during RVT page invalidation."),
		ECVF_RenderThreadSafe);

	static const int NumScalabilityGroups = 3;

	static float GTileCountBiases[NumScalabilityGroups] = { 0 };
	static FAutoConsoleVariableRef CVarVTTileCountBias_ForBackwardsCompat(
		TEXT("r.VT.RVT.TileCountBias"),
		GTileCountBiases[0],
		TEXT("Bias to apply to Runtime Virtual Texture size.\n")
		TEXT(" Group 0"),
		ECVF_Scalability
	);
	static FAutoConsoleVariableRef CVarVTTileCountBias0(
		TEXT("r.VT.RVT.TileCountBias.Group0"),
		GTileCountBiases[0],
		TEXT("Bias to apply to Runtime Virtual Texture size.\n")
		TEXT(" Group 0"),
		ECVF_Scalability
	);
	static FAutoConsoleVariableRef CVarVTTileCountBias1(
		TEXT("r.VT.RVT.TileCountBias.Group1"),
		GTileCountBiases[1],
		TEXT("Bias to apply to Runtime Virtual Texture size.\n")
		TEXT(" Group 1"),
		ECVF_Scalability
	);
	static FAutoConsoleVariableRef CVarVTTileCountBias2(
		TEXT("r.VT.RVT.TileCountBias.Group2"),
		GTileCountBiases[2],
		TEXT("Bias to apply to Runtime Virtual Texture size.\n")
		TEXT(" Group 2"),
		ECVF_Scalability
	);

	static TAutoConsoleVariable<int32> CVarVTMobileEnableManualTrilinearFiltering(
		TEXT("r.VT.Mobile.ManualTrilinearFiltering"),
		1,
		TEXT("Whether to use a manual trilinear filtering for VTs on mobile platforms.\n")
		TEXT("This more expensive filtering is used on mobile platforms that do not support Temporal Anti-Aliasing.\n"),
		ECVF_RenderThreadSafe | ECVF_ReadOnly);

	static TAutoConsoleVariable<int32> CVarVTEnableAnisotropy(
		TEXT("r.VT.AnisotropicFiltering"),
		0,
		TEXT("Is anisotropic filtering for VTs enabled?"),
		ECVF_RenderThreadSafe | ECVF_ReadOnly);

	static TAutoConsoleVariable<int32> CVarVTMaxAnisotropy(
		TEXT("r.VT.MaxAnisotropy"),
		8,
		TEXT("MaxAnisotropy setting for Virtual Texture sampling."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);


	/** Track changes and apply to relevant systems. This allows us to dynamically change the scalability settings. */
	static void OnUpdate()
	{
		const float MaxAnisotropy = CVarVTMaxAnisotropy.GetValueOnGameThread();

		static float LastMaxAnisotropy = MaxAnisotropy;
		static float LastTileCountBiases[3] = { GTileCountBiases[0], GTileCountBiases[1], GTileCountBiases[2] };

		bool bUpdate = false;
		if (LastMaxAnisotropy != MaxAnisotropy)
		{
			LastMaxAnisotropy = MaxAnisotropy;
			bUpdate = true;
		}
		if (LastTileCountBiases[0] != GTileCountBiases[0] || LastTileCountBiases[1] != GTileCountBiases[1] || LastTileCountBiases[2] != GTileCountBiases[2])
		{
			LastTileCountBiases[0] = GTileCountBiases[0];
			LastTileCountBiases[1] = GTileCountBiases[1];
			LastTileCountBiases[2] = GTileCountBiases[2];
			bUpdate = true;
		}

		if (bUpdate)
		{
			VirtualTexture::Recreate();
		}
	}

	FAutoConsoleVariableSink GConsoleVariableSink(FConsoleCommandDelegate::CreateStatic(&OnUpdate));


	int32 GetMaxUploadsPerFrame()
	{
#if WITH_EDITOR
		// Vary setting in editor because we rely on reactive updates while editing.
		return GIsEditor ? CVarVTMaxUploadsPerFrameInEditor.GetValueOnAnyThread() : CVarVTMaxUploadsPerFrame.GetValueOnAnyThread();
#else
		return CVarVTMaxUploadsPerFrame.GetValueOnAnyThread();
#endif
	}

	int32 GetMaxUploadsPerFrameForStreamingVT()
	{
		int32 Budget = CVarVTMaxUploadsPerFrameStreaming.GetValueOnAnyThread();
#if WITH_EDITOR
		// Vary setting in editor because we rely on reactive updates while editing.
		Budget = GIsEditor ? CVarVTMaxUploadsPerFrameStreamingInEditor.GetValueOnAnyThread() : Budget;
#endif
		return (Budget < 0) ? MAX_int32 : Budget;
	}

	int32 GetMaxPagesProducedPerFrame()
	{
#if WITH_EDITOR
		// Vary setting in editor because we rely on reactive updates while editing.
		return GIsEditor ? CVarMaxPagesProducedPerFrameInEditor.GetValueOnAnyThread() : CVarMaxPagesProducedPerFrame.GetValueOnAnyThread();
#else
		return CVarMaxPagesProducedPerFrame.GetValueOnAnyThread();
#endif
	}

	int32 GetMaxContinuousUpdatesPerFrame()
	{
#if WITH_EDITOR
		// Vary setting in editor because we rely on reactive updates while editing. eg GPULightmass.
		return GIsEditor ? CVarVTMaxContinuousUpdatesPerFrameInEditor.GetValueOnAnyThread() : CVarVTMaxContinuousUpdatesPerFrame.GetValueOnAnyThread();
#else
		return CVarVTMaxContinuousUpdatesPerFrame.GetValueOnAnyThread();
#endif
	}

	float GetMaxGatherTimePerFrame()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			return 0.0f;
		}
#endif
		return CVarVTMaxGatherTimePerFrame.GetValueOnAnyThread() * 0.001f;
	}

	float GetMaxSubmitTimePerFrame()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			return 0.0f;
		}
#endif
		return CVarVTMaxSubmitTimePerFrame.GetValueOnAnyThread() * 0.001f;
	}

	int32 GetMaxAllocatedVTReleasedPerFrame()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			return 0;
		}
#endif
		return CVarVTMaxReleasedPerFrame.GetValueOnAnyThread();
	}

	uint32 GetVirtualTextureFeedbackFactor()
	{
		return CVarVirtualTextureFeedbackFactor.GetValueOnAnyThread();
	}

	uint32 GetPageFreeThreshold()
	{
		return FMath::Max(CVarVTPageFreeThreshold.GetValueOnRenderThread(), 0);
	}

	uint32 GetKeepDirtyPageMappedFrameThreshold()
	{
		return FMath::Max(CVarVTKeepDirtyPageMappedFrameThreshold.GetValueOnRenderThread(), 0);
	}

	int32 GetRuntimeVirtualTextureSizeBias(uint32 GroupIndex)
	{
		return GroupIndex < NumScalabilityGroups ? GTileCountBiases[GroupIndex] : 0;
	}

	bool IsAnisotropicFilteringEnabled()
	{
		return CVarVTEnableAnisotropy.GetValueOnAnyThread() != 0;
	}

	int32 GetMaxAnisotropy()
	{
		return CVarVTMaxAnisotropy.GetValueOnAnyThread();
	}
}
