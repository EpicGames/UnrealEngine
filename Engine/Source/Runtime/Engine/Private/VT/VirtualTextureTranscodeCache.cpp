// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureTranscodeCache.h"

#include "BlockCodingHelpers.h"
#include "EngineModule.h"
#include "Misc/Compression.h"
#include "RendererInterface.h"
#include "UploadingVirtualTexture.h"
#include "VT/VirtualTextureBuiltData.h"
#include "VT/VirtualTextureUploadCache.h"
#include "VirtualTextureChunkManager.h"

static int32 TranscodeRetireAge = 60; //1 second @ 60 fps
static FAutoConsoleVariableRef CVarVTTranscodeRetireAge(
	TEXT("r.VT.TranscodeRetireAge"),
	TranscodeRetireAge,
	TEXT("If a VT transcode request is not picked up after this number of frames, drop it and put request in cache as free. default 60\n")
	, ECVF_Default
);

/** Highlight decoding errors in hot pink. Remove this after we fix the bug where the DDC for virtual textures is getting poisoned with bad data. */
static bool GShowDecodeErrors = true;
static FAutoConsoleCommand CVarVTShowDecodeErrors(
	TEXT("r.VT.ShowDecodeErrors"),
	TEXT("Highlight virtual textures with decode errors in hot pink."),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray< FString >& Args)
		{
			if (Args.Num() != 1)
			{
				GShowDecodeErrors = !GShowDecodeErrors;
			}
			else
			{
				GShowDecodeErrors = TCString<TCHAR>::Atoi(*Args[0]) != 0;
			}

			GetRendererModule().FlushVirtualTextureCache();
		})
);

/** Fill virtual texture borders (or full tiles) with mip colors. */
static TAutoConsoleVariable<int32> CVarVirtualTextureBorders(
	TEXT("r.VT.Borders"),
	0,
	TEXT("Sets the virtual texture tile border size, in texels, that is filled with debug mip colors.\n")
	TEXT("Size may be rounded up for block compressed textures."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { GetRendererModule().FlushVirtualTextureCache(); }),
	ECVF_RenderThreadSafe
);

/** Select a single mip level to apply mip colors to. This can be useful for highlighting mip 0 sampling issues. */
static TAutoConsoleVariable<int32> CVarVirtualTextureBordersMip(
	TEXT("r.VT.Borders.Mip"),
	-1,
	TEXT("Set a single mip level that r.VT.Borders will apply to.\n")
	TEXT("-1 (default) applies to all mip levels."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { GetRendererModule().FlushVirtualTextureCache(); }),
	ECVF_RenderThreadSafe
);

struct FTranscodeTask
{
	FVTUploadTileBuffer StagingBuffer[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	FVTTranscodeParams Params;

	FTranscodeTask(const FVTUploadTileBuffer* InStagingBuffers, const FVTTranscodeParams& InParams)
		: Params(InParams)
	{
		if (Params.Codec)
		{
			Params.Codec->BeginTranscodeTask();
		}

		FMemory::Memcpy(StagingBuffer, InStagingBuffers, sizeof(FVTUploadTileBuffer) * VIRTUALTEXTURE_SPACE_MAXLAYERS);
	}

	~FTranscodeTask()
	{
		if (Params.Codec)
		{
			Params.Codec->EndTranscodeTask();
		}
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		DoTask(true);
	}

	void DoTask(bool bAsync)
	{
		SCOPED_NAMED_EVENT(VirtualTextureTranscodeTask_DoTask, FColor::Cyan);

		static uint8 Black[4] = { 0,0,0,0 };
		static uint8 OpaqueBlack[4] = { 0,0,0,255 };
		static uint8 White[4] = { 0xFF, 0xFF, 0xFF, 0xFF };
		static uint8 Flat[4] = { 127,127,255,255 };
		static uint8 ErrorColor[4] = { 255, 0, 255, 255 };

		const uint32 ChunkIndex = Params.ChunkIndex;
		const FVirtualTextureDataChunk& Chunk = Params.VTData->Chunks[ChunkIndex];
		const uint32 TilePixelSize = Params.VTData->GetPhysicalTileSize();
		const uint32 TileBorderPixelSize = Params.VTData->TileBorderSize;
		const uint32 NumLayers = Params.VTData->GetNumLayers();
		const uint8 vLevel = Params.vLevel;
		const uint32 vAddress = Params.vAddress;

		// code must be fully loaded by the time we start transcoding
		check(!Params.Codec || Params.Codec->IsCreationComplete());

		// Used to allocate any temp memory needed to decode tile
		// Inline allocator hopefully avoids heap allocation in most cases
		// 136x136 DXT5 tile is 18k uncompressed
		// TaskGraph threads currently look to be created with 384k of stack space
		TArray<uint8, TInlineAllocator<16u * 1024>> TempBuffer;

		uint32 TileBaseOffset = ~0u;
		for (uint32 LayerIndex = 0u; LayerIndex < NumLayers; ++LayerIndex)
		{
			if (!(Params.LayerMask & (1u << LayerIndex)))
			{
				continue;
			}

			const uint32 TileLayerOffset = Params.VTData->GetTileOffset(vLevel, vAddress,  LayerIndex);
			const uint32 NextTileLayerOffset = Params.VTData->GetTileOffset(vLevel, vAddress, LayerIndex + 1u);
			if (TileBaseOffset == ~0u)
			{
				TileBaseOffset = TileLayerOffset;
			}

			// We make a single IO request that covers all the required layers
			// this means if there's an unused layer in between two required layers, the unused layer will still be loaded
			// So we compute offset using offset to this layer vs offset to the first requested layer
			const uint32 DataOffset = TileLayerOffset - TileBaseOffset;
			const uint32 TileLayerSize = NextTileLayerOffset - TileLayerOffset;

			const EPixelFormat LayerFormat = Params.VTData->LayerTypes[LayerIndex];
			const uint32 TileWidthInBlocks = FMath::DivideAndRoundUp(TilePixelSize, (uint32)GPixelFormats[LayerFormat].BlockSizeX);
			const uint32 TileHeightInBlocks = FMath::DivideAndRoundUp(TilePixelSize, (uint32)GPixelFormats[LayerFormat].BlockSizeY);
			const uint32 PackedStride = TileWidthInBlocks * GPixelFormats[LayerFormat].BlockBytes;
			const size_t PackedOutputSize = PackedStride * TileHeightInBlocks;
			const FVTUploadTileBuffer& StagingBufferForLayer = StagingBuffer[LayerIndex];

			const EVirtualTextureCodec VTCodec = Chunk.CodecType[LayerIndex];
			bool bDecodeResult = true;
			switch (VTCodec)
			{
			case EVirtualTextureCodec::Black:
				UniformColorPixels(StagingBufferForLayer, TilePixelSize, TilePixelSize, LayerFormat, Black);
				break;
			case EVirtualTextureCodec::OpaqueBlack:
				UniformColorPixels(StagingBufferForLayer, TilePixelSize, TilePixelSize, LayerFormat, OpaqueBlack);
				break;
			case EVirtualTextureCodec::White:
				UniformColorPixels(StagingBufferForLayer, TilePixelSize, TilePixelSize, LayerFormat, White);
				break;
			case EVirtualTextureCodec::Flat:
				UniformColorPixels(StagingBufferForLayer, TilePixelSize, TilePixelSize, LayerFormat, Flat);
				break;
			case EVirtualTextureCodec::RawGPU:
				if (StagingBufferForLayer.Stride == PackedStride)
				{
					bDecodeResult = bDecodeResult && ensure(TileLayerSize <= StagingBufferForLayer.MemorySize);
					if (bDecodeResult)
					{
						Params.Data->CopyTo(StagingBufferForLayer.Memory, DataOffset, TileLayerSize);
					}
				}
				else
				{
					bDecodeResult = bDecodeResult && ensure(PackedStride <= StagingBufferForLayer.Stride);
					bDecodeResult = bDecodeResult && ensure(TileLayerSize <= PackedOutputSize);
					bDecodeResult = bDecodeResult && ensure(TileHeightInBlocks * StagingBufferForLayer.Stride <= StagingBufferForLayer.MemorySize);
					if (bDecodeResult)
					{
						TempBuffer.SetNumUninitialized(PackedOutputSize, EAllowShrinking::No);
						Params.Data->CopyTo(TempBuffer.GetData(), DataOffset, TileLayerSize);
						for (uint32 y = 0; y < TileHeightInBlocks; ++y)
						{
							FMemory::Memcpy((uint8*)StagingBufferForLayer.Memory + y * StagingBufferForLayer.Stride, TempBuffer.GetData() + y * PackedStride, PackedStride);
						}
					}
				}
				break;
			case EVirtualTextureCodec::Crunch_DEPRECATED:
			{
				bDecodeResult = false;
				break;
			}
			case EVirtualTextureCodec::ZippedGPU_DEPRECATED:
				if (StagingBufferForLayer.Stride == PackedStride)
				{
					// output buffer is tightly packed, can decompress directly
					bDecodeResult = bDecodeResult && ensure(PackedOutputSize <= StagingBufferForLayer.MemorySize);
					bDecodeResult = bDecodeResult && FCompression::UncompressMemoryStream(NAME_Zlib, StagingBufferForLayer.Memory, PackedOutputSize, Params.Data, DataOffset, TileLayerSize);
				}
				else
				{
					// output buffer has per-scanline padding, need to decompress to temp buffer, then copy line-by-line
					bDecodeResult = bDecodeResult && ensure(PackedStride <= StagingBufferForLayer.Stride);
					if (bDecodeResult)
					{
						TempBuffer.SetNumUninitialized(PackedOutputSize, EAllowShrinking::No);
						bDecodeResult = FCompression::UncompressMemoryStream(NAME_Zlib, TempBuffer.GetData(), PackedOutputSize, Params.Data, DataOffset, TileLayerSize);
						if (bDecodeResult)
						{
							check(TileHeightInBlocks * StagingBufferForLayer.Stride <= StagingBufferForLayer.MemorySize);
							for (uint32 y = 0; y < TileHeightInBlocks; ++y)
							{
								FMemory::Memcpy((uint8*)StagingBufferForLayer.Memory + y * StagingBufferForLayer.Stride, TempBuffer.GetData() + y * PackedStride, PackedStride);
							}
						}
					}
				}
				break;

			default:
				checkNoEntry();
				bDecodeResult = false;
				break;
			}

			if (!bDecodeResult && GShowDecodeErrors)
			{
				UE_LOGF(LogVirtualTexturing, Error, "Failed to decode VT tile (vAddress %08X, vLevel %d) for '%ls'", Params.vAddress, Params.vLevel, *Params.Name.ToString());
				UniformColorPixels(StagingBufferForLayer, TilePixelSize, TilePixelSize, LayerFormat, ErrorColor);
			}

			// Bake debug borders directly into the tile pixels
			const int32 DebugBorderSize = CVarVirtualTextureBorders.GetValueOnAnyThread();
			if (DebugBorderSize > 0)
			{
				const int32 DebugBorderMip = CVarVirtualTextureBordersMip.GetValueOnAnyThread();
				if (DebugBorderMip < 0 || DebugBorderMip == vLevel)
				{
					BakeDebugInfo(StagingBufferForLayer, TilePixelSize, TilePixelSize, TileBorderPixelSize + DebugBorderSize, LayerFormat, vLevel);
				}
			}
		}

		// We're done with the compressed data
		// The uncompressed data will be freed once it's uploaded to the GPU
		Params.Data.SafeRelease();
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyNormalThreadNormalTask; }

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(TranscodeJob, STATGROUP_VTP);
	}
};

FVirtualTextureTranscodeCache::FVirtualTextureTranscodeCache()
{
	Tasks.AddDefaulted(LIST_COUNT);
	for (int i = 0; i < LIST_COUNT; ++i)
	{
		FTaskEntry& Task = Tasks[i];
		Task.Magic = 0u;
		Task.NextIndex = Task.PrevIndex = i;
	}
}

FVTTranscodeKey FVirtualTextureTranscodeCache::GetKey(const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint32 vAddress)
{
	FVTTranscodeKey Result;
	Result.ProducerID = ProducerHandle.PackedValue;
	Result.vAddress = vAddress;
	Result.vLevel = vLevel;
	Result.LayerMask = LayerMask;
	Result.Hash = (uint16)MurmurFinalize64(Result.Key); //-V614
	return Result;
}

FVTTranscodeTileHandleAndStatus FVirtualTextureTranscodeCache::FindTask(const FVTTranscodeKey& InKey) const
{
	for (uint32 Index = TileIDToTaskIndex.First(InKey.Hash); TileIDToTaskIndex.IsValid(Index); Index = TileIDToTaskIndex.Next(Index))
	{
		const FTaskEntry& Task = Tasks[Index];
		if (Task.Key == InKey.Key)
		{
			const bool IsComplete = !Task.GraphEvent || Task.GraphEvent->IsComplete();
			return FVTTranscodeTileHandleAndStatus(FVTTranscodeTileHandle(Index, Task.Magic), IsComplete);
		}
	}

	return FVTTranscodeTileHandleAndStatus();
}

bool FVirtualTextureTranscodeCache::IsTaskFinished(FVTTranscodeTileHandle InHandle) const
{
	const uint32 TaskIndex = InHandle.Index;
	check(TaskIndex >= LIST_COUNT);
	const FTaskEntry& TaskEntry = Tasks[TaskIndex];
	check(TaskEntry.Magic == InHandle.Magic);
	return !TaskEntry.GraphEvent || TaskEntry.GraphEvent->IsComplete();
}

void FVirtualTextureTranscodeCache::WaitTaskFinished(FVTTranscodeTileHandle InHandle) const
{
	const uint32 TaskIndex = InHandle.Index;
	check(TaskIndex >= LIST_COUNT);
	const FTaskEntry& TaskEntry = Tasks[TaskIndex];
	check(TaskEntry.Magic == InHandle.Magic);
	if (TaskEntry.GraphEvent)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(TaskEntry.GraphEvent, ENamedThreads::GetRenderThread_Local());
	}
}

FGraphEventRef FVirtualTextureTranscodeCache::GetTaskEvent(FVTTranscodeTileHandle InHandle) const
{
	const uint32 TaskIndex = InHandle.Index;
	check(TaskIndex >= LIST_COUNT);
	const FTaskEntry& TaskEntry = Tasks[TaskIndex];
	check(TaskEntry.Magic == InHandle.Magic);
	return TaskEntry.GraphEvent;
}

void FVirtualTextureTranscodeCache::GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const
{
	const uint32 Hash = MurmurFinalize32(ProducerHandle.PackedValue);
	for (uint32 TaskIndex = ProducerToTaskIndex.First(Hash); ProducerToTaskIndex.IsValid(TaskIndex); TaskIndex = ProducerToTaskIndex.Next(TaskIndex))
	{
		FTaskEntry const& TaskEntry = Tasks[TaskIndex];
		if (TaskEntry.PackedProducerHandle == ProducerHandle.PackedValue && TaskEntry.GraphEvent && !TaskEntry.GraphEvent->IsComplete())
		{
			InOutTasks.Add(TaskEntry.GraphEvent);
		}
	}
}

const FVTUploadTileHandle* FVirtualTextureTranscodeCache::AcquireTaskResult(FVTTranscodeTileHandle InHandle)
{
	const uint32 TaskIndex = InHandle.Index;
	check(TaskIndex >= LIST_COUNT);
	FTaskEntry& TaskEntry = Tasks[TaskIndex];
	check(TaskEntry.Magic == InHandle.Magic);

	if (TaskEntry.GraphEvent && !TaskEntry.GraphEvent->IsComplete())
	{
		// GetRenderThread_Local() will allow render thread to continue to process other tasks while waiting for transcode task to finish
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(TaskEntry.GraphEvent, ENamedThreads::GetRenderThread_Local());
	}
	TaskEntry.GraphEvent.SafeRelease();

	RemoveFromList(TaskIndex);
	AddToList(LIST_FREE, TaskIndex);

	++TaskEntry.Magic;
	TileIDToTaskIndex.Remove(TaskEntry.Hash, TaskIndex);
	ProducerToTaskIndex.Remove(MurmurFinalize32(TaskEntry.PackedProducerHandle), TaskIndex);

	return TaskEntry.StageTileHandle;
}

FVTTranscodeTileHandle FVirtualTextureTranscodeCache::SubmitTask(
	FRHICommandListBase& RHICmdList,
	FVirtualTextureUploadCache& InUploadCache,
	const FVTTranscodeKey& InKey,
	const FVirtualTextureProducerHandle& InProducerHandle,
	const FVTTranscodeParams& InParams,
	const FGraphEventArray* InPrerequisites)
{
	// make sure we don't already have a task for this key
	checkSlow(!FindTask(InKey).Handle.IsValid());

	int32 TaskIndex = Tasks[LIST_FREE].NextIndex;
	if (TaskIndex == LIST_FREE)
	{
		TaskIndex = Tasks.AddDefaulted();
		FTaskEntry& NewTask = Tasks[TaskIndex];
		NewTask.NextIndex = NewTask.PrevIndex = TaskIndex;
	}
	else
	{
		RemoveFromList(TaskIndex);
	}

	AddToList(LIST_PENDING, TaskIndex);
	TileIDToTaskIndex.Add(InKey.Hash, TaskIndex);
	ProducerToTaskIndex.Add(MurmurFinalize32(InProducerHandle.PackedValue), TaskIndex);

	FTaskEntry& TaskEntry = Tasks[TaskIndex];
	TaskEntry.Key = InKey.Key;
	TaskEntry.PackedProducerHandle = InProducerHandle.PackedValue;
	TaskEntry.Hash = InKey.Hash;
	TaskEntry.FrameSubmitted = GFrameNumberRenderThread;
	FMemory::Memzero(TaskEntry.StageTileHandle);

	check(!TaskEntry.GraphEvent || TaskEntry.GraphEvent->IsComplete());
	
	if (TaskEntry.GraphEvent)
	{
		TaskEntry.GraphEvent.SafeRelease();
	}

	const FGraphEventArray* Prerequisites = InPrerequisites;
	if (Prerequisites)
	{
		bool bAnyEventPending = false;
		for (const FGraphEventRef& Event : *Prerequisites)
		{
			if (!Event->IsComplete())
			{
				bAnyEventPending = true;
				break;
			}
		}
		if (!bAnyEventPending)
		{
			Prerequisites = nullptr;
		}
	}

	const uint32 ChunkIndex = InParams.ChunkIndex;
	const FVirtualTextureDataChunk& Chunk = InParams.VTData->Chunks[ChunkIndex];
	const uint32 TilePixelSize = InParams.VTData->GetPhysicalTileSize();
	FVTUploadTileBuffer StagingBuffer[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	bool bNeedToLaunchTask = false;

	uint32 TileBaseOffset = ~0u;
	for (uint32 LayerIndex = 0u; LayerIndex < InParams.VTData->GetNumLayers(); ++LayerIndex)
	{
		if (InParams.LayerMask & (1u << LayerIndex))
		{
			const EPixelFormat LayerFormat = InParams.VTData->LayerTypes[LayerIndex];
			FVTUploadTileBuffer& StagingBufferForLayer = StagingBuffer[LayerIndex];

			const uint32 TileLayerOffset = InParams.VTData->GetTileOffset(InParams.vLevel, InParams.vAddress, LayerIndex);
			const uint32 NextTileLayerOffset = InParams.VTData->GetTileOffset(InParams.vLevel, InParams.vAddress, LayerIndex + 1u);
			if (TileBaseOffset == ~0u)
			{
				TileBaseOffset = TileLayerOffset;
			}

			TaskEntry.StageTileHandle[LayerIndex] = InUploadCache.PrepareTileForUpload(RHICmdList, StagingBufferForLayer, LayerFormat, TilePixelSize);

			// If there aren't any prerequisites, it's not worth launching a task for a raw memcpy, just do the work now
			if (Prerequisites || Chunk.CodecType[LayerIndex] != EVirtualTextureCodec::RawGPU)
			{
				// TODO - if this logic gets more complex, should mask off layers that are processed inline when kicking off jobs
				// for now in practice, should either process all layers inline, or all layers in job
				bNeedToLaunchTask = true;
			}
		}
	}

	if (bNeedToLaunchTask)
	{
		TaskEntry.GraphEvent = TGraphTask<FTranscodeTask>::CreateTask(Prerequisites).ConstructAndDispatchWhenReady(StagingBuffer, InParams);
	}
	else
	{
		FTranscodeTask LocalTask(StagingBuffer, InParams);
		LocalTask.DoTask(false);
	}
	return FVTTranscodeTileHandle(TaskIndex, TaskEntry.Magic);
}

void FVirtualTextureTranscodeCache::RetireOldTasks(FRHICommandListBase& RHICmdList, FVirtualTextureUploadCache& InUploadCache)
{
	const uint32 CurrentFrame = GFrameNumberRenderThread;

	int32 TaskIndex = Tasks[LIST_PENDING].NextIndex;
	while (TaskIndex != LIST_PENDING)
	{
		FTaskEntry& TaskEntry = Tasks[TaskIndex];
		const int32 NextIndex = TaskEntry.NextIndex;
		check(CurrentFrame >= TaskEntry.FrameSubmitted);
		const uint32 Age = CurrentFrame - TaskEntry.FrameSubmitted;
		if (Age < (uint32)TranscodeRetireAge)
		{
			break;
		}

		// can't retire until the task is complete
		// this should generally not be an issue, as the task should be complete by the time we consider retiring it
		if (TaskEntry.GraphEvent && !TaskEntry.GraphEvent->IsComplete())
		{
			break;
		}

		INC_DWORD_STAT(STAT_VTP_NumTranscodeRetired);

		++TaskEntry.Magic;
		TaskEntry.GraphEvent.SafeRelease();

		// release the staging buffers back to the upload cache
		FVTTranscodeKey Key;
		Key.Key = TaskEntry.Key;
		for (uint32 LayerIndex = 0u; LayerIndex < VIRTUALTEXTURE_SPACE_MAXLAYERS; ++LayerIndex)
		{
			const FVTUploadTileHandle StageTileHandle = TaskEntry.StageTileHandle[LayerIndex];
			if (Key.LayerMask & (1u << LayerIndex))
			{
				InUploadCache.CancelTile(RHICmdList, StageTileHandle);
			}
		}
	
		// release task entry back to free list
		RemoveFromList(TaskIndex);
		AddToList(LIST_FREE, TaskIndex);
		TileIDToTaskIndex.Remove(TaskEntry.Hash, TaskIndex);
		ProducerToTaskIndex.Remove(MurmurFinalize32(TaskEntry.PackedProducerHandle), TaskIndex);

		TaskIndex = NextIndex;
	}
}
