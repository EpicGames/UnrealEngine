// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VirtualTextureUploadCache.h"
#include "PixelFormat.h"
#include "Misc/MemoryReadStream.h"
#include "Containers/HashTable.h"

class FUploadingVirtualTexture;
class FVirtualTextureCodec;
struct FVirtualTextureBuiltData;

struct FVTTranscodeParams
{
	IMemoryReadStreamRef Data;
	const FVirtualTextureCodec* Codec;
	const FVirtualTextureBuiltData* VTData;
	FName Name;
	uint32 ChunkIndex;
	uint32 vAddress;
	uint8 vLevel;
	uint8 LayerMask;
};

union FVTTranscodeTileHandle
{
	explicit FVTTranscodeTileHandle(uint64 InPacked = 0u) : PackedData(InPacked) {}
	FVTTranscodeTileHandle(uint32 InIndex, uint32 InMagic) : Index(InIndex), Magic(InMagic) {}

	inline bool IsValid() const { return PackedData != 0u; }

	uint64 PackedData;
	struct
	{
		uint32 Index;
		uint32 Magic;
	};
};
static_assert(sizeof(FVTTranscodeTileHandle) == sizeof(uint64), "");

struct FVTTranscodeKey
{
	union
	{
		uint64 Key;
		struct
		{
			uint32 ProducerID;
			uint32 vAddress : 24;
			uint32 vLevel : 4;
			uint32 LayerMask : 4;
		};
	};
	uint16 Hash;
};

struct FVTTranscodeTileHandleAndStatus
{
	FVTTranscodeTileHandleAndStatus() : IsComplete(false) {}
	FVTTranscodeTileHandleAndStatus(const FVTTranscodeTileHandle& InHandle, bool InComplete) : Handle(InHandle), IsComplete(InComplete) {}

	FVTTranscodeTileHandle Handle;
	bool IsComplete;
};

class FVirtualTextureTranscodeCache
{
public:
	FVirtualTextureTranscodeCache();

	static FVTTranscodeKey GetKey(const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint32 vAddress);

	FVTTranscodeTileHandleAndStatus FindTask(const FVTTranscodeKey& InKey) const;

	FVTTranscodeTileHandle SubmitTask(
		FRHICommandListBase& RHICmdList,
		FVirtualTextureUploadCache& InUploadCache,
		const FVTTranscodeKey& InKey,
		const FVirtualTextureProducerHandle& InProducerHandle,
		const FVTTranscodeParams& InParams,
		const FGraphEventArray* InPrerequisites = NULL);

	bool IsTaskFinished(FVTTranscodeTileHandle InHandle) const;
	void WaitTaskFinished(FVTTranscodeTileHandle InHandle) const;
	const FVTUploadTileHandle* AcquireTaskResult(FVTTranscodeTileHandle InHandle);
	
	FGraphEventRef GetTaskEvent(FVTTranscodeTileHandle InHandle) const;
	void GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const;

	void RetireOldTasks(FRHICommandListBase& RHICmdList, FVirtualTextureUploadCache& InUploadCache);

private:
	enum ListType
	{
		LIST_FREE,
		LIST_PENDING,

		LIST_COUNT,
	};

	struct FTaskEntry
	{
		uint64 Key;
		FGraphEventRef GraphEvent;
		FVTUploadTileHandle StageTileHandle[VIRTUALTEXTURE_SPACE_MAXLAYERS];
		uint32 PackedProducerHandle;
		uint32 FrameSubmitted;
		uint16 Magic;
		uint16 Hash;
		uint16 NextIndex;
		uint16 PrevIndex;
	};

	void RemoveFromList(int32 Index)
	{
		FTaskEntry& Entry = Tasks[Index];
		check(Index >= LIST_COUNT); // if we're trying to remove a list head, something is corrupt
		Tasks[Entry.PrevIndex].NextIndex = Entry.NextIndex;
		Tasks[Entry.NextIndex].PrevIndex = Entry.PrevIndex;
		Entry.NextIndex = Entry.PrevIndex = Index;
	}

	void AddToList(int32 HeadIndex, int32 Index)
	{
		FTaskEntry& Head = Tasks[HeadIndex];
		FTaskEntry& Entry = Tasks[Index];
		check(Index >= LIST_COUNT); // if we're trying to add a list head, something is corrupt
		check(Index <= 0xffff);

		// make sure we're not currently in any list
		check(Entry.NextIndex == Index);
		check(Entry.PrevIndex == Index);

		Entry.NextIndex = HeadIndex;
		Entry.PrevIndex = Head.PrevIndex;
		Tasks[Head.PrevIndex].NextIndex = Index;
		Head.PrevIndex = Index;
	}

	TArray<FTaskEntry> Tasks;
	FHashTable TileIDToTaskIndex;
	FHashTable ProducerToTaskIndex;
};
