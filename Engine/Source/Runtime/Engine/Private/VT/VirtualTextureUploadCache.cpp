// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureUploadCache.h"

#include "RenderGraphBuilder.h"
#include "RenderUtils.h"
#include "Stats/StatsTrace.h"
#include "VirtualTextureChunkManager.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_MEMORY_STAT_POOL(TEXT("Total GPU Upload Memory"), STAT_TotalGPUUploadSize, STATGROUP_VirtualTextureMemory, FPlatformMemory::MCR_GPU);
DECLARE_MEMORY_STAT(TEXT("Total CPU Upload Memory"), STAT_TotalCPUUploadSize, STATGROUP_VirtualTextureMemory);

static TAutoConsoleVariable<int32> CVarVTUploadMemoryPageSize(
	TEXT("r.VT.UploadMemoryPageSize"),
	4,
	TEXT("Size in MB for a single page of virtual texture upload memory."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVTMaxUploadMemory(
	TEXT("r.VT.MaxUploadMemory"),
	64,
	TEXT("Maximum amount of upload memory to allocate in MB before throttling virtual texture streaming requests.\n")
	TEXT("We never throttle high priority requests so allocation can peak above this value.\n")
	TEXT("0 or negative values disable the restriction."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMaxUploadRequests(
	TEXT("r.VT.MaxUploadRequests"),
	0,
	TEXT("Maximum number of virtual texture tile upload requests that can be in flight.\n")
	TEXT("0 (default) or negative values disable the restriction."),
	ECVF_RenderThreadSafe);


uint32 FVTUploadTileAllocator::Allocate(FRHICommandListBase& RHICmdList, EVTUploadType InUploadType, EPixelFormat InFormat, uint32 InTileSize)
{
	UE::TScopeLock Lock(Mutex);

	// Find matching FormatBuffer.
	const FPixelFormatInfo& FormatInfo = GPixelFormats[InFormat];
	const uint32 TileWidthInBlocks = FMath::DivideAndRoundUp(InTileSize, (uint32)FormatInfo.BlockSizeX);
	const uint32 TileHeightInBlocks = FMath::DivideAndRoundUp(InTileSize, (uint32)FormatInfo.BlockSizeY);

	FSharedFormatDesc Desc;
	Desc.BlockBytes = FormatInfo.BlockBytes;
	Desc.Stride = TileWidthInBlocks * FormatInfo.BlockBytes;
	Desc.MemorySize = TileWidthInBlocks * TileHeightInBlocks * FormatInfo.BlockBytes;

	int32 FormatIndex = 0;
	for (; FormatIndex < FormatDescs.Num(); ++FormatIndex)
	{
		if (Desc.BlockBytes == FormatDescs[FormatIndex].BlockBytes && Desc.Stride == FormatDescs[FormatIndex].Stride && Desc.MemorySize == FormatDescs[FormatIndex].MemorySize)
		{
			break;
		}
	}

	if (FormatIndex == FormatDescs.Num())
	{
		// Add newly found format.
		check(FormatDescs.Num() < FHandle::MaxFormats);
		FormatDescs.Add(Desc);
		FormatBuffers.AddDefaulted();
	}

	FSharedFormatBuffers& FormatBuffer = FormatBuffers[FormatIndex];

	// Find available staging buffer.
	int32 StagingBufferIndex = 0;
	for (; StagingBufferIndex < FormatBuffer.StagingBuffers.Num(); ++StagingBufferIndex)
	{
		if (FormatBuffer.StagingBuffers[StagingBufferIndex].Memory == nullptr)
		{
			// Staging buffer was released so we can re-init and use it.
			break;
		}

		if (FormatBuffer.StagingBuffers[StagingBufferIndex].TileFreeList.Num())
		{
			// Staging buffer has free tiles available.
			break;
		}
	}

	if (StagingBufferIndex == FormatBuffer.StagingBuffers.Num())
	{
		// Current staging buffers are full. Need to allocate a new staging buffer.
		check(FormatBuffer.StagingBuffers.Num() < FHandle::MaxStagingBuffers);
		FormatBuffer.StagingBuffers.AddDefaulted();
	}

	FStagingBuffer& StagingBuffer = FormatBuffer.StagingBuffers[StagingBufferIndex];
	if (StagingBuffer.Memory == nullptr)
	{
		// Staging buffer needs underlying buffer allocating.
		StagingBuffer.Init(RHICmdList, InUploadType, Desc.BlockBytes, Desc.MemorySize);
		NumAllocatedBytes += StagingBuffer.TileSizeAligned * StagingBuffer.NumTiles;
	}

	// Pop a free tile and return handle.
	int32 TileIndex = StagingBuffer.TileFreeList.Pop(EAllowShrinking::No);

	FHandle Handle;
	Handle.FormatIndex = FormatIndex;
	Handle.StagingBufferIndex = StagingBufferIndex;
	Handle.TileIndex = TileIndex;
	return Handle.PackedValue;
}

void FVTUploadTileAllocator::Free(FRHICommandListBase& RHICmdList, uint32 InHandle)
{
	UE::TScopeLock Lock(Mutex);

	FHandle Handle;
	Handle.PackedValue = InHandle;

	// Push tile back onto free list.
	FStagingBuffer& StagingBuffer = FormatBuffers[Handle.FormatIndex].StagingBuffers[Handle.StagingBufferIndex];
	StagingBuffer.TileFreeList.Push(Handle.TileIndex);

	if (StagingBuffer.NumTiles == StagingBuffer.TileFreeList.Num())
	{
		// All tiles are free, so release the underlying memory.
		check(NumAllocatedBytes >= StagingBuffer.TileSizeAligned * StagingBuffer.NumTiles);
		NumAllocatedBytes -= StagingBuffer.TileSizeAligned * StagingBuffer.NumTiles;

		StagingBuffer.Release(&RHICmdList);
	}
}

FVTUploadTileBuffer FVTUploadTileAllocator::GetBufferFromHandle(uint32 InHandle) const
{
	FHandle Handle;
	Handle.PackedValue = InHandle;

	FSharedFormatDesc const& FormatDesc = FormatDescs[Handle.FormatIndex];
	FStagingBuffer const& StagingBuffer = FormatBuffers[Handle.FormatIndex].StagingBuffers[Handle.StagingBufferIndex];

	FVTUploadTileBuffer Buffer;
	Buffer.Memory = (uint8*)StagingBuffer.Memory + StagingBuffer.TileSizeAligned * Handle.TileIndex;
	Buffer.MemorySize = StagingBuffer.TileSize;
	Buffer.Stride = FormatDesc.Stride;
	return Buffer;
}

FVTUploadTileBufferExt FVTUploadTileAllocator::GetBufferFromHandleExt(uint32 InHandle) const
{
	FHandle Handle;
	Handle.PackedValue = InHandle;

	FSharedFormatDesc const& FormatDesc = FormatDescs[Handle.FormatIndex];
	FStagingBuffer const& StagingBuffer = FormatBuffers[Handle.FormatIndex].StagingBuffers[Handle.StagingBufferIndex];

	FVTUploadTileBufferExt Buffer;
	Buffer.RHIBuffer = StagingBuffer.RHIBuffer;
	Buffer.BufferMemory = StagingBuffer.Memory;
	Buffer.BufferOffset = StagingBuffer.TileSizeAligned * Handle.TileIndex;
	Buffer.Stride = FormatDesc.Stride;
	return Buffer;
}

FVTUploadTileAllocator::FStagingBuffer::~FStagingBuffer()
{
	Release(nullptr);
}

void FVTUploadTileAllocator::FStagingBuffer::Init(FRHICommandListBase& RHICmdList, EVTUploadType InUploadType, uint32 InBufferStrideBytes, uint32 InTileSizeBytes)
{
	TileSize = InTileSizeBytes;
	TileSizeAligned = Align(InTileSizeBytes, 128u);

	const uint32 RequestedBufferSize = CVarVTUploadMemoryPageSize.GetValueOnRenderThread() * 1024 * 1024;
	NumTiles = FMath::DivideAndRoundUp(RequestedBufferSize, TileSizeAligned);
	NumTiles = FMath::Min(NumTiles, FVTUploadTileAllocator::FHandle::MaxTiles);
	const uint32 BufferSize = TileSizeAligned * NumTiles;

	check(TileFreeList.Num() == 0);
	TileFreeList.AddUninitialized(NumTiles);
	for (uint32 Index = 0; Index < NumTiles; ++Index)
	{
		TileFreeList[Index] = NumTiles - Index - 1;
	}

	if (InUploadType == EVTUploadType::PersistentBuffer)
	{
		// Allocate staging buffer directly in GPU memory.
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("StagingBuffer"), BufferSize, InBufferStrideBytes)
			.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static | EBufferUsageFlags::KeepCPUAccessible)
			.SetInitialState(ERHIAccess::SRVMask);

		RHIBuffer = RHICmdList.CreateBuffer(CreateDesc);

		// Here we bypass 'normal' RHI operations in order to get a persistent pointer to GPU memory, on supported platforms.
		// This should be encapsulated into a proper RHI method at some point.
		Memory = RHICmdList.LockBuffer(RHIBuffer, 0u, BufferSize, RLM_WriteOnly_NoOverwrite);

		INC_MEMORY_STAT_BY(STAT_TotalGPUUploadSize, BufferSize);
	}
	else
	{
		// Allocate staging buffer in CPU memory.
		Memory = FMemory::Malloc(BufferSize, 128u);

		INC_MEMORY_STAT_BY(STAT_TotalCPUUploadSize, BufferSize);
	}
}

void FVTUploadTileAllocator::FStagingBuffer::Release(FRHICommandListBase* RHICmdList)
{
	const uint32 BufferSize = TileSizeAligned * NumTiles;

	if (RHIBuffer.IsValid())
	{
		check(RHICmdList);

		// Unmap and release the GPU buffer if present.
		RHICmdList->UnlockBuffer(RHIBuffer);
		RHIBuffer.SafeRelease();
		// In this case 'Memory' was the mapped pointer, so release it.
		Memory = nullptr;

		DEC_MEMORY_STAT_BY(STAT_TotalGPUUploadSize, BufferSize);
	}

	if (Memory != nullptr)
	{
		// If we still have 'Memory' pointer, it is CPU, release it now.
		FMemory::Free(Memory);
		Memory = nullptr;

		DEC_MEMORY_STAT_BY(STAT_TotalCPUUploadSize, BufferSize);
	}

	TileSize = 0u;
	NumTiles = 0u;
	TileFreeList.Reset();
}

FVirtualTextureUploadCache::FVirtualTextureUploadCache()
{
	const bool bSupportPersistentBuffer = (GRHISupportsDirectGPUMemoryLock && GRHISupportsUpdateFromBufferTexture);
	UploadType = bSupportPersistentBuffer ? EVTUploadType::PersistentBuffer : UploadType = EVTUploadType::StagingCopy;
}

int32 FVirtualTextureUploadCache::GetOrCreatePoolIndex(EPixelFormat InFormat, uint32 InTileSize)
{
	for (int32 i = 0; i < Pools.Num(); ++i)
	{
		const FPoolEntry& Entry = Pools[i];
		if (Entry.Format == InFormat && Entry.TileSize == InTileSize)
		{
			return i;
		}
	}

	const int32 PoolIndex = Pools.AddDefaulted();
	FPoolEntry& Entry = Pools[PoolIndex];
	Entry.Format = InFormat;
	Entry.TileSize = InTileSize;

	return PoolIndex;
}

/** Parameter struct for tracking target texture resource states. */
BEGIN_SHADER_PARAMETER_STRUCT(FVirtualTextureUploadCacheParameters, )
	RDG_TEXTURE_ACCESS_ARRAY(TextureArray)
END_SHADER_PARAMETER_STRUCT()

void FVirtualTextureUploadCache::Finalize(FRDGBuilder& GraphBuilder)
{
	for (int PoolIndex = 0; PoolIndex < Pools.Num(); ++PoolIndex)
	{
		FPoolEntry& PoolEntry = Pools[PoolIndex];
		const uint32 BatchCount = PoolEntry.PendingSubmit.Num();
		if (BatchCount == 0)
		{
			continue;
		}

		// Make a copy of required data for RDG pass.
		const int32 PoolTileSize = PoolEntry.TileSize;
		FTileEntry* TileEntries = GraphBuilder.AllocPODArray<FTileEntry>(BatchCount);
		FVTUploadTileBufferExt* UploadBuffers = GraphBuilder.AllocPODArray<FVTUploadTileBufferExt>(BatchCount);
		
		FVirtualTextureUploadCacheParameters* UploadParameters = GraphBuilder.AllocParameters<FVirtualTextureUploadCacheParameters>();

		for (uint32 Index = 0u; Index < BatchCount; ++Index)
		{
			FTileEntry& TileEntry = PoolEntry.PendingSubmit[Index];

			// Common case is that all target textures are the same.
			// So compare with the previous entry as a quick test for a faster path.
			if (Index > 1 && TileEntry.PooledRenderTarget == TileEntries[Index - 1].PooledRenderTarget)
			{
				TileEntry.Texture = TileEntries[Index - 1].Texture;
			}
			else
			{
				TileEntry.Texture = GraphBuilder.RegisterExternalTexture(TileEntry.PooledRenderTarget, ERDGTextureFlags::None);
				GraphBuilder.UseInternalAccessMode(TileEntry.Texture);
				UploadParameters->TextureArray.AddUnique(FRDGTextureAccess(TileEntry.Texture, ERHIAccess::CopyDest));
			}
			
			TileEntries[Index] = TileEntry;
			UploadBuffers[Index] = TileAllocator.GetBufferFromHandleExt(TileEntry.TileHandle);
		}

		// Can clear the pending queue now.
		PoolEntry.PendingSubmit.Reset();

		// Submit to RDG.
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VirtualTextureUpload"),
			UploadParameters,
			ERDGPassFlags::Copy,
			[PoolTileSize, BatchCount, TileEntries, UploadBuffers](FRDGAsyncTask,  FRHICommandList& RHICmdList)
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());
			SCOPED_DRAW_EVENT(RHICmdList, FVirtualTextureUploadCache_Finalize);
			SCOPE_CYCLE_COUNTER(STAT_VTP_FlushUpload)

			// These are already in the correct state from RDG, so disable the automatic RHI transitions.
			FRHICommandListScopedAllowExtraTransitions ScopedExtraTransitions(RHICmdList, false);

			for (uint32 Index = 0u; Index < BatchCount; ++Index)
			{
				const FTileEntry& Entry = TileEntries[Index];
				const uint32 SubmitTileSize = PoolTileSize - Entry.SubmitSkipBorderSize * 2;
				const FUpdateTextureRegion2D UpdateRegion(
					Entry.SubmitDestX * SubmitTileSize, Entry.SubmitDestY * SubmitTileSize, 
					Entry.SubmitSkipBorderSize, Entry.SubmitSkipBorderSize, 
					SubmitTileSize, SubmitTileSize);
				const FVTUploadTileBufferExt& UploadBuffer = UploadBuffers[Index];

				if (UploadBuffer.RHIBuffer != nullptr)
				{
					// This is the EVTUploadType::PersistentBuffer upload path.
					RHICmdList.UpdateFromBufferTexture2D(Entry.Texture->GetRHI(), 0u, UpdateRegion, UploadBuffer.Stride, UploadBuffer.RHIBuffer, UploadBuffer.BufferOffset);
				}
				else
				{
					// This is the EVTUploadType::StagingCopy upload path.
					RHICmdList.UpdateTexture2D(Entry.Texture->GetRHI(), 0u, UpdateRegion, UploadBuffer.Stride, (uint8*)UploadBuffer.BufferMemory + UploadBuffer.BufferOffset);
				}
			}
		});
	}
}

void FVirtualTextureUploadCache::ReleaseRHI()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// Complete/Cancel all work will release allocated staging buffers.
	UpdateFreeList(RHICmdList, true);
	for (TSparseArray<FTileEntry>::TIterator It(PendingUpload); It; ++It)
	{
		CancelTile(RHICmdList, FVTUploadTileHandle(It.GetIndex()));
	}
	
	// Release staging textures.
	Pools.Empty();
}

FVTUploadTileHandle FVirtualTextureUploadCache::PrepareTileForUpload(FRHICommandListBase& RHICmdList, FVTUploadTileBuffer& OutBuffer, EPixelFormat InFormat, uint32 InTileSize)
{
	SCOPE_CYCLE_COUNTER(STAT_VTP_StageTile)

	uint32 TileHandle = TileAllocator.Allocate(RHICmdList, UploadType, InFormat, InTileSize);
	OutBuffer = TileAllocator.GetBufferFromHandle(TileHandle);

	const int32 PoolIndex = GetOrCreatePoolIndex(InFormat, InTileSize);
	const FPoolEntry& PoolEntry = Pools[PoolIndex];
	
	FTileEntry Tile;
	Tile.PoolIndex = PoolIndex;
	Tile.TileHandle = TileHandle;

	uint32 Index = PendingUpload.Emplace(Tile);
	return FVTUploadTileHandle(Index);
}

void FVirtualTextureUploadCache::SubmitTile(FRHICommandListBase& RHICmdList, const FVTUploadTileHandle& InHandle, const FVTProduceTargetLayer& Target, int InSkipBorderSize)
{
	// Get entry and remove from pending uploads.
	check(PendingUpload.IsValidIndex(InHandle.Index));
	FTileEntry Entry = PendingUpload[InHandle.Index];
	PendingUpload.RemoveAt(InHandle.Index);

	// Place on deferred release queue.
	Entry.FrameSubmitted = GFrameNumberRenderThread;
	PendingRelease.Emplace(Entry);

	// Move to list of batched updates for the current pool
	Entry.PooledRenderTarget = Target.PooledRenderTarget;
	Entry.SubmitDestX = Target.pPageLocation.X;
	Entry.SubmitDestY = Target.pPageLocation.Y;
	Entry.SubmitSkipBorderSize = InSkipBorderSize;

	FPoolEntry& PoolEntry = Pools[Entry.PoolIndex];
	PoolEntry.PendingSubmit.Emplace(Entry);
}

void FVirtualTextureUploadCache::CancelTile(FRHICommandListBase& RHICmdList, const FVTUploadTileHandle& InHandle)
{
	check(PendingUpload.IsValidIndex(InHandle.Index));
	FTileEntry& Entry = PendingUpload[InHandle.Index];
	TileAllocator.Free(RHICmdList, Entry.TileHandle);
	PendingUpload.RemoveAt(InHandle.Index);
}

void FVirtualTextureUploadCache::UpdateFreeList(FRHICommandListBase& RHICmdList, bool bForceFreeAll)
{
	// Iterate tiles pending release and free them once we can be sure they have been used.
	const uint32 CurrentFrame = GFrameNumberRenderThread;
	static uint32 ReleaseFrameDelay = 2u;

	for (TSparseArray<FTileEntry>::TIterator It(PendingRelease); It; ++It)
	{
		if (CurrentFrame < It->FrameSubmitted + ReleaseFrameDelay)
		{
			continue;
		}

		TileAllocator.Free(RHICmdList, It->TileHandle);
		It.RemoveCurrent();
	}
}

uint32 FVirtualTextureUploadCache::IsInMemoryBudget() const
{
	const int32 MaxUploadRequests = CVarMaxUploadRequests.GetValueOnRenderThread();
	const uint64 MaxUploadMemoryBytes = (uint64)FMath::Max(CVarVTMaxUploadMemory.GetValueOnRenderThread(), 0) * 1024u * 1024u;

	if (MaxUploadRequests > 0 && PendingUpload.Num() + PendingRelease.Num() > MaxUploadRequests)
	{
		return false;
	}
	if (MaxUploadMemoryBytes > 0 && TileAllocator.TotalAllocatedBytes() > MaxUploadMemoryBytes)
	{
		return false;
	}
	return true;
}
