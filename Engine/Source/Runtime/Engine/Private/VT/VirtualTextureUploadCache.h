// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "RendererInterface.h"
#include "RenderResource.h"
#include "VirtualTexturing.h"

class FRDGTexture;
class FRHITexture;
class FRHIBuffer;
struct FVTProduceTargetLayer;

/** Opaque handle for referencing an upload tile returned by FVirtualTextureUploadCache::PrepareTileForUpload(). */
struct FVTUploadTileHandle
{
	explicit FVTUploadTileHandle(uint32 InIndex = ~0u) 
		: Index(InIndex) 
	{}

	inline bool IsValid() const { return Index != ~0u; }

	uint32 Index;
};

/** 
 * Memory buffer for uploading virtual texture data. 
 * This is a simple view of the buffer memory for a single tile intended for use by the streaming systems.
 */
struct FVTUploadTileBuffer
{
	void* Memory = nullptr;
	uint32 MemorySize = 0u;
	uint32 Stride = 0u;
};

/** 
 * Extended definition of the memory buffer for uploading virtual texture 
 * This extended view of the buffer is used internally by FVirtualTextureUploadCache.
 */
struct FVTUploadTileBufferExt
{
	FRHIBuffer* RHIBuffer;
	void* BufferMemory = nullptr;
	uint32 BufferOffset = 0u;
	uint32 Stride = 0u;
};

/** Enumeration of staging buffer types used for upload to the VT physical texture. */
enum class EVTUploadType
{
	/** "Directly" upload image data. Internally the RHI will copy to staging memory. */
	StagingCopy,
	/** Stream image data directly into a persistent staging buffer for upload. This is the optimal path but is only available on some platforms. */
	PersistentBuffer
};

/** 
 * Handles allocation of staging buffer memory. 
 */
class FVTUploadTileAllocator
{
public:
	/** Allocate a tile. Sometimes does an allocation of the backing CPU/GPU block of memory. */
	uint32 Allocate(FRHICommandListBase& RHICmdList, EVTUploadType InUploadType, EPixelFormat InFormat, uint32 InTileSize);
	/** Free a tile. Sometimes does a free of the backing CPU/GPU block of memory. */
	void Free(FRHICommandListBase& RHICmdList, uint32 InHandle);

	/** Get upload buffer description from handle. */
	FVTUploadTileBuffer GetBufferFromHandle(uint32 InHandle) const;
	/** Get upload buffer extended description from handle. */
	FVTUploadTileBufferExt GetBufferFromHandleExt(uint32 InHandle) const;
	
	/** Get allocated memory in bytes. */
	uint64 TotalAllocatedBytes() const { return NumAllocatedBytes; }

private:
	/** Mutex for Allocate/Free. */
	UE::FMutex Mutex;

	/** Handle for an allocated tile used by the allocation system. */
	union FHandle
	{
		uint32 PackedValue = 0u;
		struct
		{
			uint32 FormatIndex : 5;
			uint32 StagingBufferIndex : 14;
			uint32 TileIndex : 13;
		};

		static const uint32 MaxFormats = 1 << 5;
		static const uint32 MaxStagingBuffers = 1 << 14;
		static const uint32 MaxTiles = 1 << 13;
	};

	/**
	 * Backing memory for buffers used by the streaming/transcoding to write texture data.
	 * The memory is split into equal sized tiles for multiple upload tasks.
	 * Backing memory can be either CPU heap memory or a locked GPU memory buffer depending on the platform.
	 */
	struct FStagingBuffer
	{
		~FStagingBuffer();

		void Init(FRHICommandListBase& RHICmdList, EVTUploadType InUploadType, uint32 InBufferStrideBytes, uint32 InTileSizeBytes);
		void Release(FRHICommandListBase* RHICmdList);

		/** GPU buffer if used on platform. */
		TRefCountPtr<FRHIBuffer> RHIBuffer;

		/** Memory pointer to locked GPU buffer if used on platform, or to allocated CPU heap memory if not. */
		void* Memory = nullptr;
		uint32 TileSize = 0u;
		uint32 TileSizeAligned = 0u;
		uint32 NumTiles = 0u;
		/** List of tile indices that haven't been allocated. */
		TArray<uint16> TileFreeList;
	};

	/** Container for multiple staging buffers. */
	struct FSharedFormatBuffers
	{
		TArray<FStagingBuffer> StagingBuffers;
	};

	/**
	 * Description of values that affect staging buffer creation.
	 * Multiple virtual texture pools may map onto the same description and so can share staging buffer memory.
	 */
	struct FSharedFormatDesc
	{
		uint32 BlockBytes = 0u;
		uint32 Stride = 0u;
		uint32 MemorySize = 0u;
	};

	/** Array of all discovered format descriptions. */
	TArray<FSharedFormatDesc> FormatDescs;
	/** Array of staging buffers. Kept in sync with associated formats from FormatDescs. */
	TArray<FSharedFormatBuffers> FormatBuffers;
	/** Allocated memory counter. */
	uint64 NumAllocatedBytes = 0;
};

/** 
 * Finalizer implementation for uploading virtual textures.
 * Handles management of upload buffers and copying streamed data to the GPU physical texture.
 */
class FVirtualTextureUploadCache : public IVirtualTextureFinalizer, public FRenderResource
{
public:
	FVirtualTextureUploadCache();

	//~ Begin FRenderResource Interface.
	virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface.

	/** Get a staging upload buffer for streaming texture data into. */
	FVTUploadTileHandle PrepareTileForUpload(FRHICommandListBase& RHICmdList, FVTUploadTileBuffer& OutBuffer, EPixelFormat InFormat, uint32 InTileSize);
	/** 
	 * Mark streamed upload data ready for upload to to the physical virtual texture.
	 * Depending on the platform the upload might happen here, or be deferred to the Finalize() call.
	 */
	void SubmitTile(FRHICommandListBase& RHICmdList, const FVTUploadTileHandle& InHandle, const FVTProduceTargetLayer& Target, int InSkipBorderSize);
	/** Cancel a tile that was already in flight. */
	void CancelTile(FRHICommandListBase& RHICmdList, const FVTUploadTileHandle& InHandle);

	//~ Begin IVirtualTextureFinalizer Interface.
	virtual void Finalize(FRDGBuilder& GraphBuilder) override;
	//~ End IVirtualTextureFinalizer Interface.

	/** Call on a tick to recycle submitted staging buffers. */
	void UpdateFreeList(FRHICommandListBase& RHICmdList, bool bForceFreeAll = false);

	/** Returns true if underlying allocator within the budget set by r.VT.MaxUploadMemory.  */
	uint32 IsInMemoryBudget() const;

private:
	/** Get a pool for a format and tile size. */
	int32 GetOrCreatePoolIndex(EPixelFormat InFormat, uint32 InTileSize);

	/** Description of a single allocated tile. Carries mutable state as tile moves from uploading to submitting to pending delete. */
	struct FTileEntry
	{
		uint32 PoolIndex = 0;
		uint32 TileHandle = 0;
		IPooledRenderTarget* PooledRenderTarget = nullptr;
		FRDGTexture* Texture = nullptr;
		int32 SubmitDestX = 0;
		int32 SubmitDestY = 0;
		int32 SubmitSkipBorderSize = 0;
		uint32 FrameSubmitted = 0;
	};

	/** Staging texture used for tile upload. Only used on platforms that don't have faster upload methods. */
	struct FStagingTexture
	{
		TRefCountPtr<FRHITexture> RHITexture;
		uint32 WidthInTiles = 0u;
		uint32 BatchCapacity = 0u;
		bool bIsCPUWritable;
	};

	/** State for a single pool. A pool covers all virtual textures of the same format and tile size. */
	struct FPoolEntry
	{
		EPixelFormat Format = PF_Unknown;
		uint32 TileSize = 0u;

		static const uint32 NUM_STAGING_TEXTURES = 3u;
		FStagingTexture StagingTexture[NUM_STAGING_TEXTURES];
		uint32 BatchTextureIndex = 0u;

		TArray<FTileEntry> PendingSubmit;
	};

	EVTUploadType UploadType;
	TArray<FPoolEntry> Pools;
	FVTUploadTileAllocator TileAllocator;
	TSparseArray<FTileEntry> PendingUpload;
	TSparseArray<FTileEntry> PendingRelease;
};
