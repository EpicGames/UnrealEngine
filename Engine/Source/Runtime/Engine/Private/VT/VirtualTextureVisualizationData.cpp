// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureVisualizationData.h"

#include "SceneManagement.h"
#include "SceneView.h"

#define LOCTEXT_NAMESPACE "FVirtualTextureVisualizationData"

static FVirtualTextureVisualizationData GVirtualTextureVisualizationData;

const TCHAR* FVirtualTextureVisualizationData::GetVisualizeConsoleCommandName()
{
	return TEXT("r.VT.Visualize");
}

void FVirtualTextureVisualizationData::ConfigureConsoleCommand()
{
	FString AvailableVisualizationModes;
	for (TModeArray::TConstIterator It = ModeArray.CreateConstIterator(); It; ++It)
	{
		const FModeRecord& Record = *It;
		AvailableVisualizationModes += FString("\n  ");
		AvailableVisualizationModes += Record.ModeString;
	}

	ConsoleDocumentationVisualizationMode = TEXT("When the viewport view-mode is set to 'Virtual Texture Visualization', this command specifies which of the various channels to display. Values entered other than the allowed values shown below will be ignored.");
	ConsoleDocumentationVisualizationMode += AvailableVisualizationModes;

	IConsoleManager::Get().RegisterConsoleVariable(
		GetVisualizeConsoleCommandName(),
		TEXT(""),
		*ConsoleDocumentationVisualizationMode,
		ECVF_Cheat);
}

void FVirtualTextureVisualizationData::AddVisualizationMode(
	const TCHAR* ModeString,
	const FText& ModeText,
	const FText& ModeDesc,
	EVirtualTextureVisualizationMode ModeID)
{
	const FName ModeName = FName(ModeString);

	FModeRecord& Record = ModeArray.AddDefaulted_GetRef();
	Record.ModeString = FString(ModeString);
	Record.ModeName = ModeName;
	Record.ModeText = ModeText;
	Record.ModeDesc = ModeDesc;
	Record.ModeID = ModeID;
}

void FVirtualTextureVisualizationData::Initialize()
{
	if (!bIsInitialized)
	{
		AddVisualizationMode(
			TEXT("pending"),
			LOCTEXT("PendingMips", "Pending Mips"),
			LOCTEXT("PendingMipsDesc", "The number of pending virtual texture mips to reach the resolution wanted by the GPU at a pixel"),
			EVirtualTextureVisualizationMode::PendingMips);

		AddVisualizationMode(
			TEXT("count"),
			LOCTEXT("StackCount", "Stack Count"),
			LOCTEXT("StackCountDesc", "The number of virtual texture stack (page table) samples at a pixel"),
			EVirtualTextureVisualizationMode::StackCount);

		ConfigureConsoleCommand();

		bIsInitialized = true;
	}
}

FName FVirtualTextureVisualizationData::GetActiveMode(FSceneView const& InView) const
{
	if (IsInitialized())
	{
		static TConsoleVariableData<FString>* CVarViewMode = nullptr;
		if (CVarViewMode == nullptr)
		{
			if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(GetVisualizeConsoleCommandName()))
			{
				CVarViewMode = ConsoleVariable->AsVariableString();
			}
		}

		if (CVarViewMode)
		{
			const FString& ViewMode = CVarViewMode->GetValueOnAnyThread();
			if (const FModeRecord* Record = ModeArray.FindByPredicate([ViewMode](const FModeRecord& Record) { return Record.ModeString == ViewMode; }))
			{
				return Record->ModeName;
			}
		}

		if (InView.Family && InView.Family->EngineShowFlags.VisualizeVirtualTexture)
		{
			return InView.CurrentVirtualTextureVisualizationMode;
		}
	}

	return NAME_None;
}

EVirtualTextureVisualizationMode FVirtualTextureVisualizationData::GetModeID(FName const& InModeName) const
{
	if (const FModeRecord* Record = ModeArray.FindByPredicate([InModeName](const FModeRecord& Record) { return Record.ModeName == InModeName; }))
	{
		return Record->ModeID;
	}
	else
	{
		return EVirtualTextureVisualizationMode::None;
	}
}

FText FVirtualTextureVisualizationData::GetModeDisplayName(FName const& InModeName) const
{
	if (const FModeRecord* Record = ModeArray.FindByPredicate([InModeName](const FModeRecord& Record) { return Record.ModeName == InModeName; }))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText FVirtualTextureVisualizationData::GetModeDisplayDesc(FName const& InModeName) const
{
	if (const FModeRecord* Record = ModeArray.FindByPredicate([InModeName](const FModeRecord& Record) { return Record.ModeName == InModeName; }))
	{
		return Record->ModeDesc;
	}
	else
	{
		return FText::GetEmpty();
	}
}

EVirtualTextureVisualizationMode FVirtualTextureVisualizationData::SetModeUsingConsoleCommand(EVirtualTextureVisualizationMode InMode)
{
	EVirtualTextureVisualizationMode PreviousMode = EVirtualTextureVisualizationMode::None;
	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(GetVisualizeConsoleCommandName()))
	{
		FString PreviousModeString = ConsoleVariable->GetString();
		if (const FModeRecord* Record = ModeArray.FindByPredicate([PreviousModeString](const FModeRecord& Record) { return Record.ModeString == PreviousModeString; }))
		{
			PreviousMode = Record->ModeID;
		}
		
		FString ModeString; // Empty string will implement EVirtualTextureVisualizationMode::None
		if (const FModeRecord* Record = ModeArray.FindByPredicate([InMode](const FModeRecord& Record) { return Record.ModeID == InMode; }))
		{
			ModeString = Record->ModeString;
		}
		ConsoleVariable->AsVariable()->SetWithCurrentPriority(*ModeString);
	}
	return PreviousMode;
}

FVirtualTextureVisualizationData& GetVirtualTextureVisualizationData()
{
	if (!GVirtualTextureVisualizationData.IsInitialized())
	{
		GVirtualTextureVisualizationData.Initialize();
	}

	return GVirtualTextureVisualizationData;
}

#undef LOCTEXT_NAMESPACE
