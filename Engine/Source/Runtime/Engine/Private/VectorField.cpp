// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VectorField.cpp: Implementation of vector fields.
==============================================================================*/

#include "VectorField.h"
#include "Components/PrimitiveComponent.h"
#include "PrimitiveViewRelevance.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIStaticStates.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "ComponentReregisterContext.h"
#include "SceneView.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"
#include "VectorFieldVisualization.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderParameterUtils.h"
#include "GlobalShader.h"
#include "FXSystem.h"
#include "VectorField/VectorField.h"
#include "VectorField/VectorFieldAnimated.h"
#include "VectorField/VectorFieldStatic.h"
#include "Components/VectorFieldComponent.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/Material.h"
#include "Engine/Engine.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VectorField)

#if WITH_EDITORONLY_DATA
	#include "EditorFramework/AssetImportData.h"
#endif
	
#define MAX_GLOBAL_VECTOR_FIELDS (16)
DEFINE_LOG_CATEGORY(LogVectorField)

/*------------------------------------------------------------------------------
	FVectorFieldResource implementation.
------------------------------------------------------------------------------*/

FVectorFieldResource::FVectorFieldResource() {}
FVectorFieldResource::~FVectorFieldResource(){}

/**
 * Release RHI resources.
 */
void FVectorFieldResource::ReleaseRHI()
{
	VolumeTextureRHI.SafeRelease();
}

/*------------------------------------------------------------------------------
	FVectorFieldInstance implementation.
------------------------------------------------------------------------------*/

/** Destructor. */
FVectorFieldInstance::~FVectorFieldInstance()
{
	if (Resource && bInstancedResource)
	{
		FVectorFieldResource* InResource = Resource.GetReference();
		InResource->AddRef();
		ENQUEUE_RENDER_COMMAND(FDestroyVectorFieldResourceCommand)(
			[InResource](FRHICommandList& RHICmdList)
			{
				InResource->ReleaseResource();
				InResource->Release();
			});
	}
	Resource = nullptr;
}

/**
 * Initializes the instance for the given resource.
 * @param InResource - The resource to be used by this instance.
 * @param bInstanced - true if the resource is instanced and ownership is being transferred.
 */
void FVectorFieldInstance::Init(FVectorFieldResource* InResource, bool bInstanced)
{
	check(!Resource);
	Resource = InResource;
	bInstancedResource = bInstanced;
}

/**
 * Update the transforms for this vector field instance.
 * @param LocalToWorld - Transform from local space to world space.
 */
void FVectorFieldInstance::UpdateTransforms(const FMatrix& LocalToWorld)
{
	check(Resource);
	const FVector VolumeOffset = Resource->LocalBounds.Min;
	const FVector VolumeScale = Resource->LocalBounds.Max - Resource->LocalBounds.Min;
	VolumeToWorldNoScale = LocalToWorld.GetMatrixWithoutScale().RemoveTranslation();
	VolumeToWorld = FScaleMatrix(VolumeScale) * FTranslationMatrix(VolumeOffset)
		* LocalToWorld;
	WorldToVolume = VolumeToWorld.Inverse();
}

/*------------------------------------------------------------------------------
	UVectorField implementation.
------------------------------------------------------------------------------*/


UVectorField::UVectorField(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Intensity = 1.0f;
}

/**
 * Initializes an instance for use with this vector field.
 */
void UVectorField::InitInstance(class FVectorFieldInstance* Instance, bool bPreviewInstance)
{
	UE_LOGF(LogVectorField, Fatal,"%ls must override InitInstance.", *(GetClass()->GetName()));
}

/*------------------------------------------------------------------------------
	UVectorFieldStatic implementation.
------------------------------------------------------------------------------*/

/**
 * Resource for static vector fields.
 */
class FVectorFieldStaticResource : public FVectorFieldResource
{
public:

	/** Initialization constructor. */
	explicit FVectorFieldStaticResource( UVectorFieldStatic* InVectorField )
		: VolumeData(NULL)
	{
		// Copy vector field properties.
		SizeX = InVectorField->SizeX;
		SizeY = InVectorField->SizeY;
		SizeZ = InVectorField->SizeZ;
		Intensity = InVectorField->Intensity;
		LocalBounds = InVectorField->Bounds;

		// Grab a copy of the static volume data.
		InVectorField->SourceData.GetCopy(&VolumeData, /*bDiscardInternalCopy=*/ true);
	}

protected:
	/** Destructor. */
	virtual ~FVectorFieldStaticResource()
	{
		FMemory::Free(VolumeData);
		VolumeData = NULL;
	}

public:

	/**
	 * Initialize RHI resources.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (VolumeData && GSupportsTexture3D)
		{
			const uint32 DataSize = SizeX * SizeY * SizeZ * sizeof(FFloat16Color);
			FResourceBulkDataArrayView BulkDataInterface(VolumeData, DataSize);

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("FVectorFieldStaticResource"), SizeX, SizeY, SizeZ, PF_FloatRGBA)
				.SetFlags(ETextureCreateFlags::ShaderResource)
				.SetInitActionBulkData(&BulkDataInterface);

			VolumeTextureRHI = RHICmdList.CreateTexture(Desc);

			FMemory::Free(VolumeData);
			VolumeData = NULL;
		}
	}

	/**
	 * Update this resource based on changes to the asset.
	 */
	void UpdateResource(UVectorFieldStatic* InVectorField)
	{
		struct FUpdateParams
		{
			int32 SizeX;
			int32 SizeY;
			int32 SizeZ;
			float Intensity;
			FBox Bounds;
			void* VolumeData;
		};

		FUpdateParams UpdateParams;
		UpdateParams.SizeX = InVectorField->SizeX;
		UpdateParams.SizeY = InVectorField->SizeY;
		UpdateParams.SizeZ = InVectorField->SizeZ;
		UpdateParams.Intensity = InVectorField->Intensity;
		UpdateParams.Bounds = InVectorField->Bounds;
		UpdateParams.VolumeData = NULL;
		InVectorField->SourceData.GetCopy(&UpdateParams.VolumeData, /*bDiscardInternalCopy=*/ true);

		FVectorFieldStaticResource* Resource = this;
		ENQUEUE_RENDER_COMMAND(FUpdateStaticVectorFieldCommand)(
			[Resource, UpdateParams](FRHICommandListImmediate& RHICmdList)
			{
				// Free any existing volume data on the resource.
				FMemory::Free(Resource->VolumeData);

				// Update settings on this resource.
				Resource->SizeX = UpdateParams.SizeX;
				Resource->SizeY = UpdateParams.SizeY;
				Resource->SizeZ = UpdateParams.SizeZ;
				Resource->Intensity = UpdateParams.Intensity;
				Resource->LocalBounds = UpdateParams.Bounds;
				Resource->VolumeData = UpdateParams.VolumeData;

				// Update RHI resources.
				Resource->UpdateRHI(RHICmdList);
			});
	}

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
	{
		if (VolumeData)
		{
			CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SizeX * SizeY * SizeZ * sizeof(FFloat16Color));
		}
		if (VolumeTextureRHI)
		{
			const FPixelFormatInfo& FormatInfo = GPixelFormats[PF_FloatRGBA];
			const SIZE_T NumBlocksX = (SIZE_T)FMath::DivideAndRoundUp(SizeX, FormatInfo.BlockSizeX);
			const SIZE_T NumBlocksY = (SIZE_T)FMath::DivideAndRoundUp(SizeY, FormatInfo.BlockSizeY);
			const SIZE_T NumBlocksZ = (SIZE_T)FMath::DivideAndRoundUp(SizeZ, FormatInfo.BlockSizeZ);
			CumulativeResourceSize.AddDedicatedVideoMemoryBytes(NumBlocksX * NumBlocksY * NumBlocksZ * (SIZE_T)FormatInfo.BlockBytes);
		}
	}

private:

	/** Static volume texture data. */
	void* VolumeData;
};

UVectorFieldStatic::UVectorFieldStatic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAllowCPUAccess(false)
{
}

void UVectorFieldStatic::InitInstance(FVectorFieldInstance* Instance, bool bPreviewInstance)
{
	Instance->Init(Resource, /*bInstanced=*/ false);
}


void UVectorFieldStatic::InitResource()
{
	check(!Resource);

	// Loads and copies the bulk data into CPUData if bAllowCPUAccess is set, otherwise clear CPUData. 
	UpdateCPUData(false);

	Resource = new FVectorFieldStaticResource(this); // Will discard the contents of SourceData
	Resource->AddRef(); // Increment refcount because of UVectorFieldStatic::Resource is not a TRefCountPtr.

	BeginInitResource(Resource);
}


void UVectorFieldStatic::UpdateResource()
{
	check(Resource);

	// Loads and copies the bulk data into CPUData if bAllowCPUAccess is set, otherwise clears CPUData. 
	UpdateCPUData(false);

	FVectorFieldStaticResource* StaticResource = (FVectorFieldStaticResource*)Resource;
	StaticResource->UpdateResource(this); // Will discard the contents of SourceData
}

// Simple implementation of an accessor struct for grabbing the, now, inited resource on the render thread
// Impl structure is required because of the private nature of the FVectorFieldResource, which could be
// resolved...
struct FVectorFieldTextureAccessorImpl
{
	FVectorFieldTextureAccessorImpl(FVectorFieldResource* InResource)
		: Resource(InResource)
	{
	}

	FVectorFieldTextureAccessorImpl(const FVectorFieldTextureAccessorImpl& rhs)
		: Resource(rhs.Resource)
	{
	}

	TRefCountPtr<FVectorFieldResource> Resource;
};

FVectorFieldTextureAccessor::FVectorFieldTextureAccessor(UVectorField* InVectorField)
	: Impl(nullptr)
{
	if (UVectorFieldStatic* VectorFieldStatic = Cast<UVectorFieldStatic>(InVectorField))
	{
		Impl = MakeUnique<FVectorFieldTextureAccessorImpl>(VectorFieldStatic->Resource);
	}
}

FVectorFieldTextureAccessor::FVectorFieldTextureAccessor(const FVectorFieldTextureAccessor& rhs)
	: Impl(nullptr)
{
	if (rhs.Impl)
	{
		Impl = MakeUnique<FVectorFieldTextureAccessorImpl>(rhs.Impl->Resource);
	}
}

FVectorFieldTextureAccessor::~FVectorFieldTextureAccessor()
{

}

FRHITexture* FVectorFieldTextureAccessor::GetTexture() const
{
	if (Impl && Impl->Resource && Impl->Resource->VolumeTextureRHI)
	{
		return Impl->Resource->VolumeTextureRHI;
	}

	return GBlackVolumeTexture->TextureRHI;
}

#if WITH_EDITOR
ENGINE_API void UVectorFieldStatic::SetCPUAccessEnabled()
{
	bAllowCPUAccess = true;
	UpdateCPUData(true);
}
#endif // WITH_EDITOR

void UVectorFieldStatic::UpdateCPUData(bool bDiscardData)
{
	if (bAllowCPUAccess)
	{
		// Grab a copy of the bulk vector data. 
		// If the data is already loaded it makes a copy and discards the old content,
		// otherwise it simply loads the data directly from file into the pointer after allocating.
		FFloat16Color *Ptr = nullptr;
		SourceData.GetCopy((void**)&Ptr, bDiscardData);

		// Make sure the data is actually valid. 
		if (!ensure(Ptr))
		{
			UE_LOGF(LogVectorField, Error, "Vector field data is not loaded.");
			return;
		}

		// Make sure the size actually match what we expect
		if (!ensure(SourceData.GetBulkDataSize() == (SizeX*SizeY*SizeZ) * sizeof(FFloat16Color)))
		{
			UE_LOG(LogVectorField, Error, TEXT("Vector field bulk data size is different than expected. Expected %d bytes, got %" INT64_FMT "."), SizeX*SizeY*SizeZ, SourceData.GetBulkDataSize());
			FMemory::Free(Ptr);
			return;
		}

		// GetCopy should free/unload the data.
		if (bDiscardData && SourceData.IsBulkDataLoaded())
		{
			// NOTE(mv): This assertion will fail in the case where the bulk data is still available even though the bDiscardInternalCopy
			//           flag is toggled when FBulkData::CanLoadFromDisk() also fail. This happens when the user tries to allow 
			//           CPU access to a newly imported file that isn't reloaded. We still have our valid data, so we just issue a 
			//           warning and move on. See FBulkData::GetCopy() for more details. 
			UE_LOGF(LogVectorField, Warning, "SourceData.GetCopy() is supposed to unload the data after copying, but it is still loaded.");
		}

		const size_t SampleCount = SizeX * SizeY * SizeZ;

		FMemoryWriter Ar(CPUData);

#if VECTOR_FIELD_DATA_AS_HALF
		// Ensure we have enough space in the buffer to read the last element
		constexpr int32 SampleTypeSize = sizeof(FFloat16) * 3;
		constexpr int32 SampleReadSize = sizeof(FFloat16) * 4;
		CPUData.Reset((SampleCount * SampleTypeSize) - SampleTypeSize + SampleReadSize);

		for (size_t SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			Ar << Ptr[SampleIt].R;
			Ar << Ptr[SampleIt].G;
			Ar << Ptr[SampleIt].B;
		}
#else
		// Ensure we have enough space in the buffer to read the last element
		constexpr int32 SampleTypeSize = sizeof(FVector3f);
		constexpr int32 SampleReadSize = sizeof(FVector4f);
		CPUData.Reset((SampleCount * SampleTypeSize) - SampleTypeSize + SampleReadSize);

		for (size_t SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			FVector3f Value(Ptr[SampleIt].R.GetFloat(), Ptr[SampleIt].G.GetFloat(), Ptr[SampleIt].B.GetFloat());
			Ar << Value;
		}
#endif

		FMemory::Free(Ptr);
	}
	else
	{
		// If there's no need to access the CPU data just empty the array.
		CPUData.Empty();
	}
}

FVector UVectorFieldStatic::FilteredSample(const FVector& SamplePosition, const FVector& TilingAxes) const
{
	check(bAllowCPUAccess);

	static auto FVectorClamp = [](const FVector& v, const FVector& a, const FVector& b) {
		return FVector(FMath::Clamp(v.X, a.X, b.X),
			FMath::Clamp(v.Y, a.Y, b.Y),
			FMath::Clamp(v.Z, a.Z, b.Z));
	};

	static auto FVectorFloor = [](const FVector& v) {
		return FVector(FGenericPlatformMath::FloorToDouble(v.X),
			FGenericPlatformMath::FloorToDouble(v.Y),
			FGenericPlatformMath::FloorToDouble(v.Z));
	};

	const FVector Size(SizeX, SizeY, SizeZ);

	// 
	FVector Index0 = FVectorFloor(SamplePosition);
	FVector Index1 = Index0 + FVector::OneVector;

	// 
	FVector Fraction = SamplePosition - Index0;

	Index0 = Index0 - TilingAxes * FVectorFloor(Index0 / Size) * Size;
	Index1 = Index1 - TilingAxes * FVectorFloor(Index1 / Size) * Size;

	Index0 = FVectorClamp(Index0, FVector::ZeroVector, Size - FVector::OneVector);
	Index1 = FVectorClamp(Index1, FVector::ZeroVector, Size - FVector::OneVector);

	// Sample by regular trilinear interpolation:

	// Fetch corners
	FVector V000 = SampleInternal(int32(Index0.X + SizeX * Index0.Y + SizeX * SizeY * Index0.Z));
	FVector V100 = SampleInternal(int32(Index1.X + SizeX * Index0.Y + SizeX * SizeY * Index0.Z));
	FVector V010 = SampleInternal(int32(Index0.X + SizeX * Index1.Y + SizeX * SizeY * Index0.Z));
	FVector V110 = SampleInternal(int32(Index1.X + SizeX * Index1.Y + SizeX * SizeY * Index0.Z));
	FVector V001 = SampleInternal(int32(Index0.X + SizeX * Index0.Y + SizeX * SizeY * Index1.Z));
	FVector V101 = SampleInternal(int32(Index1.X + SizeX * Index0.Y + SizeX * SizeY * Index1.Z));
	FVector V011 = SampleInternal(int32(Index0.X + SizeX * Index1.Y + SizeX * SizeY * Index1.Z));
	FVector V111 = SampleInternal(int32(Index1.X + SizeX * Index1.Y + SizeX * SizeY * Index1.Z));

	// Blend x-axis
	FVector V00 = FMath::Lerp(V000, V100, Fraction.X);
	FVector V01 = FMath::Lerp(V001, V101, Fraction.X);
	FVector V10 = FMath::Lerp(V010, V110, Fraction.X);
	FVector V11 = FMath::Lerp(V011, V111, Fraction.X);

	// Blend y-axis
	FVector V0 = FMath::Lerp(V00, V10, Fraction.Y);
	FVector V1 = FMath::Lerp(V01, V11, Fraction.Y);

	// Blend z-axis
	return FMath::Lerp(V0, V1, Fraction.Z);
}

FVector UVectorFieldStatic::Sample(const FIntVector& SamplePosition) const
{
	check(bAllowCPUAccess);

	const int32 SampleX = FMath::Clamp(SamplePosition.X, 0, SizeX - 1);
	const int32 SampleY = FMath::Clamp(SamplePosition.Y, 0, SizeY - 1);
	const int32 SampleZ = FMath::Clamp(SamplePosition.Z, 0, SizeZ - 1);
	const int32 SampleIndex = SampleX + SizeX * (SampleY + SampleZ * SizeY);

	return SampleInternal(SampleIndex);
}

FORCEINLINE static FVector SampleInternalData(TConstArrayView<FFloat16> Samples, int32 SampleIndex)
{
	constexpr int32 ComponentCount = 3;
	const int32 SampleOffset = SampleIndex * ComponentCount;
	const FFloat16* HalfData = Samples.GetData();
	if (SampleOffset >= 0 && SampleOffset + ComponentCount <= Samples.Num())
	{
		HalfData += SampleOffset;
	}

	FVector4f Result;
	FPlatformMath::VectorLoadHalf(reinterpret_cast<float*>(&Result), reinterpret_cast<const uint16*>(HalfData));
	return FVector(Result.X, Result.Y, Result.Z);
}

FORCEINLINE static FVector SampleInternalData(TConstArrayView<float> Samples, int32 SampleIndex)
{
	constexpr int32 ComponentCount = 3;
	const int32 SampleOffset = SampleIndex * ComponentCount;
	const float* FloatData = Samples.GetData();
	if (SampleOffset >= 0 && SampleOffset + ComponentCount <= Samples.Num())
	{
		FloatData += SampleOffset;
	}
	return FVector(FloatData[0], FloatData[1], FloatData[2]);
}

FORCEINLINE FVector UVectorFieldStatic::SampleInternal(int32 SampleIndex) const
{
	check(bAllowCPUAccess);
	return SampleInternalData(ReadCPUData(), SampleIndex);
}

FRHITexture* UVectorFieldStatic::GetVolumeTextureRef()
{
	if (Resource)
	{
		return Resource->VolumeTextureRHI;
	}
	else
	{	
		// Fallback to a global 1x1x1 black texture when no vector field is loaded or unavailable
		return GBlackVolumeTexture->TextureRHI;
	}
}

void UVectorFieldStatic::ReleaseResource()
{
	if (Resource)
	{
		FVectorFieldResource* InResource = Resource;
		ENQUEUE_RENDER_COMMAND(ReleaseVectorFieldCommand)(
			[InResource](FRHICommandList& RHICmdList)
			{
				InResource->ReleaseResource();
				// Decrement the refcount and possibly delete (see refs from FVectorFieldInstance).
				InResource->Release(); 
			});
		Resource = nullptr;
	}
}

void UVectorFieldStatic::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Store bulk data inline for streaming (to prevent PostLoad spikes)
	if (Ar.IsCooking())
	{
		SourceData.SetBulkDataFlags(BULKDATA_ForceInlinePayload | BULKDATA_SingleUse);
	}

	SourceData.Serialize(Ar, this, INDEX_NONE, false);
}

void UVectorFieldStatic::PostLoad()
{
	Super::PostLoad();

	if ( !IsTemplate() )
	{
		InitResource();
	}

#if WITH_EDITORONLY_DATA
	if (!SourceFilePath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(SourceFilePath_DEPRECATED));
		AssetImportData->SourceData = MoveTemp(Info);
	}
#endif
}

void UVectorFieldStatic::BeginDestroy()
{
	ReleaseResource();
	Super::BeginDestroy();
}

void UVectorFieldStatic::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CPUData.GetAllocatedSize());
	if (SourceData.IsBulkDataLoaded())
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SourceData.GetBulkDataSize());
	}
	if (Resource)
	{
		((const FVectorFieldStaticResource*)Resource)->GetResourceSizeEx(CumulativeResourceSize);
	}
}

#if WITH_EDITOR
void UVectorFieldStatic::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateResource();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UVectorFieldStatic::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UVectorFieldStatic::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	Super::GetAssetRegistryTags(Context);
}

void UVectorFieldStatic::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}
#endif

class FVectorFieldCollectorResources : public FOneFrameResource
{
public:
	FVectorFieldCollectorResources(ERHIFeatureLevel::Type InFeatureLevel)
		: VisualizationVertexFactory(InFeatureLevel){}

	FVectorFieldVisualizationVertexFactory VisualizationVertexFactory;

	virtual ~FVectorFieldCollectorResources()
	{
		VisualizationVertexFactory.ReleaseResource();
	}
};

/*------------------------------------------------------------------------------
	Scene proxy for visualizing vector fields.
------------------------------------------------------------------------------*/

class FVectorFieldSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	explicit FVectorFieldSceneProxy( UVectorFieldComponent* VectorFieldComponent )
		: FPrimitiveSceneProxy(VectorFieldComponent)
		, VisualizationVertexFactory(GetScene().GetFeatureLevel())
	{
		bWillEverBeLit = false;
		VectorFieldInstance = VectorFieldComponent->VectorFieldInstance;
		check(VectorFieldInstance->Resource);
		check(VectorFieldInstance);
	}

	/** Destructor. */
	~FVectorFieldSceneProxy()
	{
		VisualizationVertexFactory.ReleaseResource();
	}

	/**
	 *	Called when the rendering thread adds the proxy to the scene.
	 *	This function allows for generating renderer-side resources.
	 *	Called in the rendering thread.
	 */
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override
	{
		VisualizationVertexFactory.InitResource(RHICmdList);
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{	
		QUICK_SCOPE_CYCLE_COUNTER( STAT_VectorFieldSceneProxy_GetDynamicMeshElements );

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				DrawVectorFieldBounds(PDI, View, VectorFieldInstance);

				// Draw a visualization of the vectors contained in the field when selected.
				if (IsSelected() || View->Family->EngineShowFlags.VectorFields)
				{
					FVectorFieldCollectorResources& CollectorResources = Collector.AllocateOneFrameResource<FVectorFieldCollectorResources>(View->GetFeatureLevel());
					CollectorResources.VisualizationVertexFactory.InitResource(Collector.GetRHICommandList());

					GetVectorFieldMesh(&CollectorResources.VisualizationVertexFactory, VectorFieldInstance, ViewIndex, Collector);
				}
			}
		}
	}

	/**
	 * Computes view relevance for this scene proxy.
	 */
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View); 
		Result.bDynamicRelevance = true;
		Result.bOpaque = true;
		return Result;
	}

	/**
	 * Computes the memory footprint of this scene proxy.
	 */
	virtual uint32 GetMemoryFootprint() const override
	{
		return sizeof(*this) + GetAllocatedSize();
	}

private:

	/** The vector field instance which this proxy is visualizing. */
	FVectorFieldInstance* VectorFieldInstance;
	/** Vertex factory for visualization. */
	FVectorFieldVisualizationVertexFactory VisualizationVertexFactory;
};

/*------------------------------------------------------------------------------
	UVectorFieldComponent implementation.
------------------------------------------------------------------------------*/
UVectorFieldComponent::UVectorFieldComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bHiddenInGame = true;
	Intensity = 1.0f;
}


FPrimitiveSceneProxy* UVectorFieldComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Proxy = NULL;
	if (VectorFieldInstance)
	{
		Proxy = new FVectorFieldSceneProxy(this);
	}
	return Proxy;
}

FBoxSphereBounds UVectorFieldComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds NewBounds;
	NewBounds.Origin = FVector(0.0f);
	NewBounds.BoxExtent = FVector(0.0f);
	NewBounds.SphereRadius = 0.0f;

	if ( VectorField )
	{
		VectorField->Bounds.GetCenterAndExtents( NewBounds.Origin, NewBounds.BoxExtent );
		NewBounds.SphereRadius = NewBounds.BoxExtent.Size();
	}

	return NewBounds.TransformBy( LocalToWorld );
}

void UVectorFieldComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (GEngine->LevelColorationUnlitMaterial != nullptr)
	{
		OutMaterials.Add(GEngine->LevelColorationUnlitMaterial);
	}
}

void UVectorFieldComponent::OnRegister()
{
	Super::OnRegister();

	if (VectorField)
	{
		if (bPreviewVectorField)
		{
			FVectorFieldInstance* Instance = new FVectorFieldInstance();
			VectorField->InitInstance(Instance, /*bPreviewInstance=*/ true);
			Instance->UpdateTransforms(GetComponentTransform().ToMatrixWithScale());
			VectorFieldInstance = Instance;
		}
		else
		{
			UWorld* World = GetWorld();
			if (World && World->Scene && World->Scene->GetFXSystem())
			{
				// Store the FX system for the world in which this component is registered.
				check(FXSystem == NULL);
				FXSystem = World->Scene->GetFXSystem();
				check(FXSystem != NULL);

				// Add this component to the FX system.
				FXSystem->AddVectorField(this);
			}
		}
	}
}


void UVectorFieldComponent::OnUnregister()
{
	if (bPreviewVectorField)
	{
		if (VectorFieldInstance)
		{
			FVectorFieldInstance* InVectorFieldInstance = VectorFieldInstance;
			ENQUEUE_RENDER_COMMAND(FDestroyVectorFieldInstanceCommand)(
				[InVectorFieldInstance](FRHICommandList& RHICmdList)
				{
					delete InVectorFieldInstance;
				});
			VectorFieldInstance = nullptr;
		}
	}
	else if (VectorFieldInstance)
	{
		check(FXSystem != NULL);
		// Remove the component from the FX system.
		FXSystem->RemoveVectorField(this);
	}
	FXSystem = NULL;
	Super::OnUnregister();
}


void UVectorFieldComponent::SendRenderTransform_Concurrent()
{
	Super::SendRenderTransform_Concurrent();
	if (FXSystem)
	{
		FXSystem->UpdateVectorField(this);
	}
}


void UVectorFieldComponent::SetIntensity(float NewIntensity)
{
	Intensity = NewIntensity;
	if (FXSystem)
	{
		FXSystem->UpdateVectorField(this);
	}
}


#if WITH_EDITOR
void UVectorFieldComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("VectorField"))
	{
		if (VectorField && !VectorField->IsA(UVectorFieldStatic::StaticClass()))
		{
			VectorField = NULL;
		}
	}
}
#endif // WITH_EDITOR

/*------------------------------------------------------------------------------
	AVectorFieldVolume implementation.
------------------------------------------------------------------------------*/


/*------------------------------------------------------------------------------
	Shader for constructing animated vector fields.
------------------------------------------------------------------------------*/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FCompositeAnimatedVectorFieldUniformParameters, )
	SHADER_PARAMETER( FVector4f, FrameA )
	SHADER_PARAMETER( FVector4f, FrameB )
	SHADER_PARAMETER( FVector3f, VoxelSize )
	SHADER_PARAMETER( float, FrameLerp )
	SHADER_PARAMETER( float, NoiseScale )
	SHADER_PARAMETER( float, NoiseMax )
	SHADER_PARAMETER( uint32, Op )
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FCompositeAnimatedVectorFieldUniformParameters,"CVF");

typedef TUniformBufferRef<FCompositeAnimatedVectorFieldUniformParameters> FCompositeAnimatedVectorFieldUniformBufferRef;

/** The number of threads per axis launched to construct the animated vector field. */
#define THREADS_PER_AXIS 8

/**
 * Compute shader used to generate an animated vector field.
 */
class FCompositeAnimatedVectorFieldCS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FCompositeAnimatedVectorFieldCS,Global);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine( TEXT("THREADS_X"), THREADS_PER_AXIS );
		OutEnvironment.SetDefine( TEXT("THREADS_Y"), THREADS_PER_AXIS );
		OutEnvironment.SetDefine( TEXT("THREADS_Z"), THREADS_PER_AXIS );
		OutEnvironment.SetDefine( TEXT("COMPOSITE_ANIMATED"), 1 );
	}

	/** Default constructor. */
	FCompositeAnimatedVectorFieldCS()
	{
	}

	/** Initialization constructor. */
	explicit FCompositeAnimatedVectorFieldCS( const ShaderMetaType::CompiledShaderInitializerType& Initializer )
		: FGlobalShader(Initializer)
	{
		AtlasTexture.Bind( Initializer.ParameterMap, TEXT("AtlasTexture") );
		AtlasTextureSampler.Bind( Initializer.ParameterMap, TEXT("AtlasTextureSampler") );
		NoiseVolumeTexture.Bind( Initializer.ParameterMap, TEXT("NoiseVolumeTexture") );
		NoiseVolumeTextureSampler.Bind( Initializer.ParameterMap, TEXT("NoiseVolumeTextureSampler") );
		OutVolumeTexture.Bind( Initializer.ParameterMap, TEXT("OutVolumeTexture") );
		OutVolumeTextureSampler.Bind( Initializer.ParameterMap, TEXT("OutVolumeTextureSampler") );
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FCompositeAnimatedVectorFieldUniformBufferRef& UniformBuffer,
		FRHIUnorderedAccessView* VolumeTextureUAV,
		FRHITexture* AtlasTextureRHI,
		FRHITexture* NoiseVolumeTextureRHI)
	{
		SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FCompositeAnimatedVectorFieldUniformParameters>(), UniformBuffer);

		FRHISamplerState* SamplerStateLinear = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		SetUAVParameter(BatchedParameters, OutVolumeTexture, VolumeTextureUAV);
		SetTextureParameter(BatchedParameters, AtlasTexture, AtlasTextureSampler, SamplerStateLinear, AtlasTextureRHI);
		SetTextureParameter(BatchedParameters, NoiseVolumeTexture, NoiseVolumeTextureSampler, SamplerStateLinear, NoiseVolumeTextureRHI);
	}

	void UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
	{
		UnsetUAVParameter(BatchedUnbinds, OutVolumeTexture);
	}

private:
	/** Vector field volume textures to composite. */
	LAYOUT_FIELD(FShaderResourceParameter, AtlasTexture);
	LAYOUT_FIELD(FShaderResourceParameter, AtlasTextureSampler);
	/** Volume texture to sample as noise. */
	LAYOUT_FIELD(FShaderResourceParameter, NoiseVolumeTexture);
	LAYOUT_FIELD(FShaderResourceParameter, NoiseVolumeTextureSampler);
	/** The global vector field volume texture to write to. */
	LAYOUT_FIELD(FShaderResourceParameter, OutVolumeTexture);
	LAYOUT_FIELD(FShaderResourceParameter, OutVolumeTextureSampler);
};
IMPLEMENT_SHADER_TYPE(,FCompositeAnimatedVectorFieldCS,TEXT("/Engine/Private/VectorFieldCompositeShaders.usf"),TEXT("CompositeAnimatedVectorField"),SF_Compute);

/*------------------------------------------------------------------------------
	Animated vector field asset.
------------------------------------------------------------------------------*/


enum
{
	/** Minimum volume size used for animated vector fields. */
	MIN_ANIMATED_VECTOR_FIELD_SIZE = 16,
	/** Maximum volume size used for animated vector fields. */
	MAX_ANIMATED_VECTOR_FIELD_SIZE = 64
};

class FVectorFieldAnimatedResource : public FVectorFieldResource
{
public:

	/** Unordered access view in to the volume texture. */
	FUnorderedAccessViewRHIRef VolumeTextureUAV;
	/** The animated vector field asset. */
	UVectorFieldAnimated* AnimatedVectorField;
	/** The accumulated frame time of the animation. */
	float FrameTime;

	/** Initialization constructor. */
	explicit FVectorFieldAnimatedResource(UVectorFieldAnimated* InVectorField)
		: AnimatedVectorField(InVectorField)
		, FrameTime(0.0f)
	{
		SizeX = InVectorField->VolumeSizeX;
		SizeY = InVectorField->VolumeSizeY;
		SizeZ = InVectorField->VolumeSizeZ;
		Intensity = InVectorField->Intensity;
		LocalBounds = InVectorField->Bounds;
	}

	/**
	 * Initialize RHI resources.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (GSupportsTexture3D)
		{
			check(SizeX > 0);
			check(SizeY > 0);
			check(SizeZ > 0);
			UE_LOG(LogVectorField,Verbose,TEXT("InitRHI for 0x%016" PTRINT_x_FMT " %dx%dx%d"),(PTRINT)this,SizeX,SizeY,SizeZ);

			ETextureCreateFlags TexCreateFlags = TexCreate_None;
			if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
			{
				TexCreateFlags = TexCreate_ShaderResource | TexCreate_UAV;
			}

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("FVectorFieldAnimatedResource"), SizeX, SizeY, SizeZ, PF_FloatRGBA)
				.SetFlags(TexCreateFlags);

			VolumeTextureRHI = RHICmdList.CreateTexture(Desc);

			if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
			{
				VolumeTextureUAV = RHICmdList.CreateUnorderedAccessView(VolumeTextureRHI, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(VolumeTextureRHI));
			}
		}
	}

	/**
	 * Release RHI resources.
	 */
	virtual void ReleaseRHI() override
	{
		VolumeTextureUAV.SafeRelease();
		FVectorFieldResource::ReleaseRHI();
	}

	/**
	 * Updates the vector field.
	 * @param DeltaSeconds - Elapsed time since the last update.
	 */
	virtual void Update(FRHICommandListImmediate& RHICmdList, float DeltaSeconds) override
	{
		check(IsInRenderingThread());

		if (GetFeatureLevel() >= ERHIFeatureLevel::SM5 && AnimatedVectorField && AnimatedVectorField->Texture && AnimatedVectorField->Texture->GetResource())
		{
			SCOPED_DRAW_EVENT(RHICmdList, AnimateVectorField);

			// Move frame time forward.
			FrameTime += AnimatedVectorField->FramesPerSecond * DeltaSeconds;

			// Compute the two frames to lerp.
			const int32 FrameA_Unclamped = FMath::TruncToInt(FrameTime);
			const int32 FrameA = AnimatedVectorField->bLoop ?
				(FrameA_Unclamped % AnimatedVectorField->FrameCount) :
				FMath::Min<int32>(FrameA_Unclamped, AnimatedVectorField->FrameCount - 1);
			const int32 FrameA_X = FrameA % AnimatedVectorField->SubImagesX;
			const int32 FrameA_Y = FrameA / AnimatedVectorField->SubImagesX;

			const int32 FrameB_Unclamped = FrameA + 1;
			const int32 FrameB = AnimatedVectorField->bLoop ?
				(FrameB_Unclamped % AnimatedVectorField->FrameCount) :
				FMath::Min<int32>(FrameB_Unclamped, AnimatedVectorField->FrameCount - 1);
			const int32 FrameB_X = FrameB % AnimatedVectorField->SubImagesX;
			const int32 FrameB_Y = FrameB / AnimatedVectorField->SubImagesX;

			FCompositeAnimatedVectorFieldUniformParameters Parameters;
			const FVector2D AtlasScale(
				1.0f / AnimatedVectorField->SubImagesX,
				1.0f / AnimatedVectorField->SubImagesY);
			Parameters.FrameA = FVector4f(
				AtlasScale.X,
				AtlasScale.Y,
				FrameA_X * AtlasScale.X,
				FrameA_Y * AtlasScale.Y );
			Parameters.FrameB = FVector4f(
				AtlasScale.X,
				AtlasScale.Y,
				FrameB_X * AtlasScale.X,
				FrameB_Y * AtlasScale.Y );
			Parameters.VoxelSize = FVector3f(1.0f / SizeX, 1.0f / SizeY, 1.0f / SizeZ);
			Parameters.FrameLerp = FMath::Fractional(FrameTime);
			Parameters.NoiseScale = AnimatedVectorField->NoiseScale;
			Parameters.NoiseMax = AnimatedVectorField->NoiseMax;
			Parameters.Op = (uint32)AnimatedVectorField->ConstructionOp;

			FCompositeAnimatedVectorFieldUniformBufferRef UniformBuffer = 
				FCompositeAnimatedVectorFieldUniformBufferRef::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleDraw);

			TShaderMapRef<FCompositeAnimatedVectorFieldCS> CompositeCS(GetGlobalShaderMap(GetFeatureLevel()));
			FRHITexture* NoiseVolumeTextureRHI = GBlackVolumeTexture->TextureRHI;
			if (AnimatedVectorField->NoiseField && AnimatedVectorField->NoiseField->Resource)
			{
				NoiseVolumeTextureRHI = AnimatedVectorField->NoiseField->Resource->VolumeTextureRHI;
			}

			RHICmdList.Transition(FRHITransitionInfo(VolumeTextureUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
			SetComputePipelineState(RHICmdList, CompositeCS.GetComputeShader());

			SetShaderParametersLegacyCS(
				RHICmdList,
				CompositeCS,
				UniformBuffer,
				VolumeTextureUAV,
				AnimatedVectorField->Texture->GetResource()->TextureRHI,
				NoiseVolumeTextureRHI);

			DispatchComputeShader(
				RHICmdList,
				CompositeCS.GetShader(),
				SizeX / THREADS_PER_AXIS,
				SizeY / THREADS_PER_AXIS,
				SizeZ / THREADS_PER_AXIS );

			UnsetShaderParametersLegacyCS(RHICmdList, CompositeCS);

			RHICmdList.Transition(FRHITransitionInfo(VolumeTextureUAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));
		}
	}

	/**
	 * Resets the vector field simulation.
	 */
	virtual void ResetVectorField() override
	{
		FrameTime = 0.0f;
	}
};

UVectorFieldAnimated::UVectorFieldAnimated(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VolumeSizeX = 16;
	VolumeSizeY = 16;
	VolumeSizeZ = 16;
	Bounds.Min = FVector(-0.5f, -0.5f, -0.5f);
	Bounds.Max = FVector(0.5, 0.5, 0.5);

}


void UVectorFieldAnimated::InitInstance(FVectorFieldInstance* Instance, bool bPreviewInstance)
{
	if (FApp::CanEverRender())
	{
		FVectorFieldAnimatedResource* Resource = new FVectorFieldAnimatedResource(this);
		if (!bPreviewInstance)
		{
			BeginInitResource(Resource);
		}
		Instance->Init(Resource, /*bInstanced=*/ true);
	}
}

static int32 ClampVolumeSize(int32 InVolumeSize)
{
	const int32 ClampedVolumeSize = FMath::Clamp<int32>(1 << FMath::CeilLogTwo(InVolumeSize),
		MIN_ANIMATED_VECTOR_FIELD_SIZE, MAX_ANIMATED_VECTOR_FIELD_SIZE);
	return ClampedVolumeSize;
}

#if WITH_EDITOR
void UVectorFieldAnimated::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	VolumeSizeX = ClampVolumeSize(VolumeSizeX);
	VolumeSizeY = ClampVolumeSize(VolumeSizeY);
	VolumeSizeZ = ClampVolumeSize(VolumeSizeZ);
	SubImagesX = FMath::Max<int32>(SubImagesX, 1);
	SubImagesY = FMath::Max<int32>(SubImagesY, 1);
	FrameCount = FMath::Max<int32>(FrameCount, 0);

	// If the volume size changes, all components must be reattached to ensure
	// that all volumes are resized.
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("VolumeSize"))
	{
		FGlobalComponentReregisterContext ReregisterComponents;
	}
}
#endif // WITH_EDITOR

