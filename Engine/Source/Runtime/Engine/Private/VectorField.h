// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VectorField.h: Interface to vector fields.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"

class Error;

/*------------------------------------------------------------------------------
	Forward declarations.
------------------------------------------------------------------------------*/

/** Interface with which primitives can be drawn. */
class FPrimitiveDrawInterface;
/** View from which a scene is rendered. */
class FSceneView;
/** A vertex factory that can be used to visualize vector fields. */
class FVectorFieldVisualizationVertexFactory;

class FRHICommandListImmediate;

/*------------------------------------------------------------------------------
	Vector field delcarations.
------------------------------------------------------------------------------*/

#define VECTOR_FIELD_LOG_VERBOSITY Error
DECLARE_LOG_CATEGORY_EXTERN(LogVectorField,VECTOR_FIELD_LOG_VERBOSITY,VECTOR_FIELD_LOG_VERBOSITY);

/**
 * Vector field resource.
 */
class FVectorFieldResource : public FRenderResource
{
public:

	/** The volume texture containing the vector field. */
	FTextureRHIRef VolumeTextureRHI;
	/** Size of the vector field (X). */
	int32 SizeX;
	/** Size of the vector field (Y). */
	int32 SizeY;
	/** Size of the vector field (Z). */
	int32 SizeZ;
	/** The amount by which to scale vectors in the field. */
	float Intensity;
	/** Local space bounds of the vector field. */
	FBox LocalBounds;

	/** Default constructor. */
	FVectorFieldResource();

	/**
	 * Release RHI resources.
	 */
	virtual void ReleaseRHI() override;

	/**
	 * Updates the vector field.
	 * @param DeltaSeconds - Elapsed time since the last update.
	 */
	virtual void Update(FRHICommandListImmediate& RHICmdList, float DeltaSeconds) {}

	/**
	 * Resets the vector field simulation.
	 */
	virtual void ResetVectorField() {}


	/** Ref couting **/
	uint32 AddRef() const
	{ 
		return (uint32)NumRefs.Increment(); 
	}

	uint32 Release() const
	{
		uint32 NewValue = (uint32)NumRefs.Decrement();
		if (NewValue == 0)
		{
			delete this;
		}
		return NewValue;
	}

protected:

	virtual ~FVectorFieldResource();

	/** Ref couting. Required becauce of how resources are shared between FVectorFieldInstance and UVectorField... **/
	mutable FThreadSafeCounter NumRefs;
};

/**
 * An instance of a vector field.
 */
class FVectorFieldInstance
{
public:

	/** The vector field resource. Kept an active ref count to manage release with UVectorFieldStatic. */
	TRefCountPtr<FVectorFieldResource> Resource;
	/** Bounds of the vector field in world space. */
	FBox WorldBounds;
	/** Transform from the vector field's local space to world space, no scaling is applied. */
	FMatrix VolumeToWorldNoScale;
	/** Tra32nsform from world space to the vector field's local space. */
	FMatrix WorldToVolume;
	/** Transform from the vector field's local space to world space. */
	FMatrix VolumeToWorld;
	/** How tightly particles adhere to the vector field. 0: Vectors act like forces, 1: Vectors act like velocities. */
	float Tightness;
	/** The amount by which to scale vectors for this instance of the field. */
	float Intensity;
	/** Index of the vector field in the world. */
	int32 Index;
	/** Tile vector field in x axis? */
	uint32 bTileX : 1;
	/** Tile vector field in y axis? */
	uint32 bTileY : 1;
	/** Tile vector field in z axis? */
	uint32 bTileZ : 1;
	/** Use fix delta time in the simulation? */
	uint32 bUseFixDT : 1;

	/** Default constructor. */
	FVectorFieldInstance()
		: Resource(NULL)
		, VolumeToWorldNoScale(FMatrix::Identity)
		, WorldToVolume(FMatrix::Identity)
		, VolumeToWorld(FMatrix::Identity)
		, Tightness(0.0f)
		, Intensity(0.0f)
		, Index(INDEX_NONE)
		, bTileX(false)
		, bTileY(false)
		, bTileZ(false)
		, bUseFixDT(false)
		, bInstancedResource(false)
	{
	}

	/** Destructor. */
	~FVectorFieldInstance();

	/**
	 * Initializes the instance for the given resource.
	 * @param InResource - The resource to be used by this instance.
	 * @param bInstanced - true if the resource is instanced and ownership is being transferred.
	 */
	void Init(FVectorFieldResource* InResource, bool bInstanced);

	/**
	 * Update the transforms for this vector field instance.
	 * @param LocalToWorld - Transform from local space to world space.
	 */
	void UpdateTransforms(const FMatrix& LocalToWorld);

private:

	/** true if the resource is instanced and owned by this instance. */
	bool bInstancedResource;
};

/** A list of vector field instances. */
typedef TSparseArray<FVectorFieldInstance*> FVectorFieldInstanceList;
