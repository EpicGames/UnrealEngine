// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VectorFieldVisualization.cpp: Visualization of vector fields.
==============================================================================*/

#include "VectorFieldVisualization.h"
#include "MeshDrawShaderBindings.h"
#include "RHIStaticStates.h"
#include "Misc/DelayedAutoRegister.h"
#include "SceneManagement.h"
#include "VectorField.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "FXSystem.h"
#include "MeshMaterialShader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIResourceUtils.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVectorFieldVisualizationParameters,"VectorFieldVis");

/*------------------------------------------------------------------------------
	Vertex factory for visualizing vector fields.
------------------------------------------------------------------------------*/

/**
 * Shader parameters for the vector field visualization vertex factory.
 */
class FVectorFieldVisualizationVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FVectorFieldVisualizationVertexFactoryShaderParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		VectorFieldTexture.Bind(ParameterMap, TEXT("VectorFieldTexture"));
		VectorFieldTextureSampler.Bind(ParameterMap, TEXT("VectorFieldTextureSampler"));
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;

private:

	/** The vector field texture parameter. */
	LAYOUT_FIELD(FShaderResourceParameter, VectorFieldTexture);
	LAYOUT_FIELD(FShaderResourceParameter, VectorFieldTextureSampler);
};

IMPLEMENT_TYPE_LAYOUT(FVectorFieldVisualizationVertexFactoryShaderParameters);

/**
 * Vertex declaration for visualizing vector fields.
 */
class FVectorFieldVisualizationVertexDeclaration : public FRenderResource
{
public:

	FVertexDeclarationRHIRef VertexDeclarationRHI;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float4, 0, sizeof(FVector4f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};
TGlobalResource<FVectorFieldVisualizationVertexDeclaration> GVectorFieldVisualizationVertexDeclaration;

const FVector4f GDummyVertexBufferData[2] =
{
	FVector4f(0.0f, 0.0f, 0.0f, 0.0f),
	FVector4f(1.0f, 1.0f, 1.0f, 1.0f),
};

/**
 * A dummy vertex buffer to bind when visualizing vector fields. This prevents
 * some D3D debug warnings about zero-element input layouts but is not strictly
 * required.
 */
class FDummyVertexBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FDummyVertexBuffer"), BUF_Static, MakeConstArrayView(GDummyVertexBufferData));
	}
};
TGlobalResource<FDummyVertexBuffer> GDummyVertexBuffer;

/**
 * Constructs render resources for this vertex factory.
 */
void FVectorFieldVisualizationVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexStream Stream;

	// No streams should currently exist.
	check( Streams.Num() == 0 );

	// Stream 0: Global particle texture coordinate buffer.
	Stream.VertexBuffer = &GDummyVertexBuffer;
	Stream.Stride = sizeof(FVector4f);
	Stream.Offset = 0;
	Streams.Add(Stream);

	// Set the declaration.
	check( IsValidRef(GVectorFieldVisualizationVertexDeclaration.VertexDeclarationRHI) );
	SetDeclaration( GVectorFieldVisualizationVertexDeclaration.VertexDeclarationRHI );
}

/**
 * Should we cache the material's shadertype on this platform with this vertex factory? 
 */
bool FVectorFieldVisualizationVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	// We support vector fields on mobile, but not the visualization, so we limit this VF to Feature Level Preview shader platforms
	return Parameters.MaterialParameters.bIsSpecialEngineMaterial &&
		SupportsGPUParticles(Parameters.Platform) &&
		(IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) || IsPCPlatform(Parameters.Platform));
}

/**
 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
 */
void FVectorFieldVisualizationVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

struct FVectorFieldVisualizationUserData : public FOneFrameResource
{
	/** Uniform buffer. */
	FVectorFieldVisualizationBufferRef UniformBuffer;

	/** Texture containing the vector field. */
	FTextureRHIRef VectorFieldTextureRHI;
};

void FVectorFieldVisualizationVertexFactoryShaderParameters::GetElementShaderBindings(
	const FSceneInterface* Scene,
	const FSceneView* View,
	const FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* InVertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	FVectorFieldVisualizationVertexFactory* VertexFactory = (FVectorFieldVisualizationVertexFactory*)InVertexFactory;
	const FVectorFieldVisualizationUserData* UserData = reinterpret_cast<const FVectorFieldVisualizationUserData*>(BatchElement.UserData);
	if (UserData)
	{
		FRHISamplerState* SamplerStatePoint = TStaticSamplerState<SF_Point>::GetRHI();
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FVectorFieldVisualizationParameters>(), UserData->UniformBuffer);
		ShaderBindings.AddTexture(VectorFieldTexture, VectorFieldTextureSampler, SamplerStatePoint, UserData->VectorFieldTextureRHI);
	}
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FVectorFieldVisualizationVertexFactory, SF_Vertex, FVectorFieldVisualizationVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FVectorFieldVisualizationVertexFactory,"/Engine/Private/VectorFieldVisualizationVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
);

/*------------------------------------------------------------------------------
	Drawing interface.
------------------------------------------------------------------------------*/

/**
 * Draw the bounds for a vector field instance.
 * @param PDI - The primitive drawing interface with which to draw.
 * @param View - The view in which to draw.
 * @param VectorFieldInstance - The vector field instance to draw.
 */
void DrawVectorFieldBounds(
	FPrimitiveDrawInterface* PDI,
	const FSceneView* View,
	FVectorFieldInstance* VectorFieldInstance )
{
	FVector Corners[8];
	FVectorFieldResource* Resource = VectorFieldInstance->Resource;
	const FVector HalfVoxelOffset(
		0.5f / Resource->SizeX, 0.5f / Resource->SizeY, 0.5f / Resource->SizeZ);
	const FVector LocalMin = -HalfVoxelOffset;
	const FVector LocalMax = FVector(1.0f) + HalfVoxelOffset;
	const FMatrix& VolumeToWorld = VectorFieldInstance->VolumeToWorld;
	const FLinearColor LineColor(1.0f,0.5f,0.0f,1.0f);
	const ESceneDepthPriorityGroup LineDPG = SDPG_World;

	// Compute all eight corners of the volume.
	Corners[0] = VolumeToWorld.TransformPosition(FVector(LocalMin.X, LocalMin.Y, LocalMin.Z));
	Corners[1] = VolumeToWorld.TransformPosition(FVector(LocalMax.X, LocalMin.Y, LocalMin.Z));
	Corners[2] = VolumeToWorld.TransformPosition(FVector(LocalMax.X, LocalMax.Y, LocalMin.Z));
	Corners[3] = VolumeToWorld.TransformPosition(FVector(LocalMin.X, LocalMax.Y, LocalMin.Z));
	Corners[4] = VolumeToWorld.TransformPosition(FVector(LocalMin.X, LocalMin.Y, LocalMax.Z));
	Corners[5] = VolumeToWorld.TransformPosition(FVector(LocalMax.X, LocalMin.Y, LocalMax.Z));
	Corners[6] = VolumeToWorld.TransformPosition(FVector(LocalMax.X, LocalMax.Y, LocalMax.Z));
	Corners[7] = VolumeToWorld.TransformPosition(FVector(LocalMin.X, LocalMax.Y, LocalMax.Z));

	// Draw the lines that form the box.
	for ( int32 Index = 0; Index < 4; ++Index )
	{
		const int32 NextIndex = (Index + 1) & 0x3;
		// Bottom face.
		PDI->DrawLine(Corners[Index], Corners[NextIndex], LineColor, LineDPG);
		// Top face.
		PDI->DrawLine(Corners[Index+4], Corners[NextIndex+4], LineColor, LineDPG);
		// Side faces.
		PDI->DrawLine(Corners[Index], Corners[Index+4], LineColor, LineDPG);
	}
}

void GetVectorFieldMesh(
	FVectorFieldVisualizationVertexFactory* VertexFactory,
	FVectorFieldInstance* VectorFieldInstance,
	int32 ViewIndex,
	FMeshElementCollector& Collector)
{
	FVectorFieldResource* Resource = VectorFieldInstance->Resource.GetReference();

	if (Resource && IsValidRef(Resource->VolumeTextureRHI))
	{
		FColoredMaterialRenderProxy* VisualizationMaterial = new FColoredMaterialRenderProxy(
			GEngine->LevelColorationUnlitMaterial->GetRenderProxy(),
			FLinearColor::Red
		);

		Collector.RegisterOneFrameMaterialProxy(VisualizationMaterial);

		// Set up parameters.
		const FDFVector3 VolumeToWorldOriginDF(VectorFieldInstance->VolumeToWorld.GetOrigin());
		const FDFMatrix VolumeToWorldDF = FDFMatrix::MakeToRelativeWorldMatrix(VolumeToWorldOriginDF.High, VectorFieldInstance->VolumeToWorld);

		FVectorFieldVisualizationParameters UniformParameters;
		UniformParameters.VolumeToWorldHigh = VolumeToWorldDF.PostTranslation;
		UniformParameters.RelativeVolumeToWorld = VolumeToWorldDF.M;
		UniformParameters.VolumeToWorldNoScale = FMatrix44f(VectorFieldInstance->VolumeToWorldNoScale);
		UniformParameters.VoxelSize = FVector3f( 1.0f / Resource->SizeX, 1.0f / Resource->SizeY, 1.0f / Resource->SizeZ );
		UniformParameters.Scale = VectorFieldInstance->Intensity * Resource->Intensity;

		FVectorFieldVisualizationUserData* UserData = &Collector.AllocateOneFrameResource<FVectorFieldVisualizationUserData>();
		UserData->UniformBuffer = FVectorFieldVisualizationBufferRef::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_SingleFrame);
		UserData->VectorFieldTextureRHI = Resource->VolumeTextureRHI;

		// Create a mesh batch for the visualization.
		FMeshBatch& MeshBatch = Collector.AllocateMesh();
		MeshBatch.CastShadow = false;
		MeshBatch.bUseAsOccluder = false;
		MeshBatch.VertexFactory = VertexFactory;
		MeshBatch.MaterialRenderProxy = VisualizationMaterial;
		MeshBatch.Type = PT_LineList;

		// A single mesh element.
		FMeshBatchElement& MeshElement = MeshBatch.Elements[0];
		MeshElement.NumPrimitives = 1;
		MeshElement.NumInstances = Resource->SizeX * Resource->SizeY * Resource->SizeZ;
		MeshElement.FirstIndex = 0;
		MeshElement.MinVertexIndex = 0;
		MeshElement.MaxVertexIndex = 1;
		MeshElement.PrimitiveUniformBuffer = GIdentityPrimitiveUniformBuffer.GetUniformBufferRHI();
		MeshElement.UserData = UserData;

		MeshBatch.bCanApplyViewModeOverrides = false;
		Collector.AddMesh(ViewIndex, MeshBatch);
	}
}
