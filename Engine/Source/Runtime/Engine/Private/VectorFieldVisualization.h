// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	VectorFieldVisualization.h: Visualization of vector fields.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"

class FMaterial;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class FSceneView;
class FVectorFieldInstance;

/*------------------------------------------------------------------------------
	Forward declarations.
------------------------------------------------------------------------------*/

/** An instance of a vector field. */
class FVectorFieldInstance;
/** Vertex factory shader parameters for visualizing vector fields. */
class FVectorFieldVisualizationVertexFactoryShaderParameters;
/** Interface with which primitives can be drawn. */
class FPrimitiveDrawInterface;
/** View from which a scene is rendered. */
class FSceneView;

/*------------------------------------------------------------------------------
	Vertex factory for visualizing vector fields.
------------------------------------------------------------------------------*/

/**
 * Uniform buffer to hold parameters for vector field visualization.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FVectorFieldVisualizationParameters, )
	SHADER_PARAMETER( FVector3f, VolumeToWorldHigh )
	SHADER_PARAMETER( FMatrix44f, RelativeVolumeToWorld )
	SHADER_PARAMETER( FMatrix44f, VolumeToWorldNoScale )
	SHADER_PARAMETER( FVector3f, VoxelSize )
	SHADER_PARAMETER( float, Scale )
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FVectorFieldVisualizationParameters> FVectorFieldVisualizationBufferRef;

/**
 * Vertex factory for visualizing vector field volumes.
 */
class FVectorFieldVisualizationVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FVectorFieldVisualizationVertexFactory);

public:
	FVectorFieldVisualizationVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FVertexFactory(InFeatureLevel)
	{
	}

	/**
	 * Constructs render resources for this vertex factory.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	 */
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/*------------------------------------------------------------------------------
	Drawing interface.
------------------------------------------------------------------------------*/

/**
 * Draw the bounds for a vector field instance.
 * @param PDI - The primitive drawing interface with which to draw.
 * @param View - The view in which to draw.
 * @param VectorFieldInstance - The vector field instance to draw.
 */
void DrawVectorFieldBounds(
	FPrimitiveDrawInterface* PDI,
	const FSceneView* View,
	FVectorFieldInstance* VectorFieldInstance );

void GetVectorFieldMesh(
	FVectorFieldVisualizationVertexFactory* VertexFactory,
	FVectorFieldInstance* VectorFieldInstance,
	int32 ViewIndex,
	FMeshElementCollector& Collector);
