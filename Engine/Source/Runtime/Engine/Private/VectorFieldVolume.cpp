// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorField/VectorFieldVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Components/VectorFieldComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VectorFieldVolume)

AVectorFieldVolume::AVectorFieldVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	VectorFieldComponent = CreateDefaultSubobject<UVectorFieldComponent>(TEXT("VectorFieldComponent0"));
	RootComponent = VectorFieldComponent;

#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> EffectsTextureObject;
			FName ID_Effects;
			FText NAME_Effects;
			FConstructorStatics()
				: EffectsTextureObject(TEXT("/Engine/EditorResources/S_VectorFieldVol"))
				, ID_Effects(TEXT("Effects"))
				, NAME_Effects(NSLOCTEXT("SpriteCategory", "Effects", "Effects"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.EffectsTextureObject.Get();
		SpriteComponent->bIsScreenSizeScaled = true;
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Effects;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Effects;
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->SetupAttachment(VectorFieldComponent);
		SpriteComponent->bReceivesDecals = false;
	}
#endif // WITH_EDITORONLY_DATA
}


