// Copyright Epic Games, Inc. All Rights Reserved.

#include "Vehicles/TireType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TireType)

UTireType::UTireType(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Property initialization
	FrictionScale = 1.0f;
}


