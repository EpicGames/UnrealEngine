// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/AppStyle.h"
#include "Engine/EngineBaseTypes.h"
#include "Styling/SlateBrush.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "UViewModeUtils"

TArray<FText> FillViewModeDisplayNames()
{
	// Allocate size
	TArray<FText> ViewModeDisplayNames;
	ViewModeDisplayNames.Reserve(VMI_Unknown);
	// Fill ViewModeIndex, VMI_Unknown+1 to include VMI_Unknown too
	for (int32 Index = 0; Index < VMI_Unknown + 1; ++Index)
	{
		const EViewModeIndex ViewModeIndex = (EViewModeIndex)Index;

		// Wireframe w/ brushes
		if (ViewModeIndex == VMI_BrushWireframe)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_BrushWireframe", "Wireframe only"));
		}
		// Wireframe w/ BSP
		else if (ViewModeIndex == VMI_Wireframe)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Wireframe", "CSG Wireframe"));
		}
		// Unlit
		else if (ViewModeIndex == VMI_Unlit)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Unlit", "Unlit"));
		}
		// Lit
		else if (ViewModeIndex == VMI_Lit)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Lit", "Lit"));
		}
		else if (ViewModeIndex == VMI_Lit_DetailLighting)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Lit_DetailLighting", "Detail Lighting"));
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		else if (ViewModeIndex == VMI_Lit_Wireframe)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Lit_Wireframe", "Lit Wireframe"));
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// Lit wo/ materials
		else if (ViewModeIndex == VMI_LightingOnly)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_LightingOnly", "Lighting Only"));
		}
		// Colored according to light count
		else if (ViewModeIndex == VMI_LightComplexity)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_LightComplexity", "Light Complexity"));
		}
		// Colored according to shader complexity
		else if (ViewModeIndex == VMI_ShaderComplexity)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_ShaderComplexity", "Shader Complexity"));
		}
		// Colored according to world-space LightMap texture density
		else if (ViewModeIndex == VMI_LightmapDensity)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_LightmapDensity", "Lightmap Density"));
		}
		// Colored according to light count - showing lightmap texel density on texture mapped objects
		else if (ViewModeIndex == VMI_LitLightmapDensity)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_LitLightmapDensity", "Lit Lightmap Density"));
		}
		else if (ViewModeIndex == VMI_ReflectionOverride)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_ReflectionOverride", "Reflections"));
		}
		else if (ViewModeIndex == VMI_VisualizeBuffer)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeBuffer", "Buffer Visualization"));
		}
		//	VMI_VoxelLighting = 13,
	
		// Colored according to stationary light overlap
		else if (ViewModeIndex == VMI_StationaryLightOverlap)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_StationaryLightOverlap", "Stationary Light Overlap"));
		}
	
		else if (ViewModeIndex == VMI_CollisionPawn)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_CollisionPawn", "Player Collision"));
		}
		else if (ViewModeIndex == VMI_CollisionVisibility)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_CollisionVisibility", "Visibility Collision"));
		}
		//VMI_UNUSED = 17,
		// Colored according to the current LOD index
		else if (ViewModeIndex == VMI_LODColoration)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_LODColoration", "Mesh LOD Coloration"));
		}
		// Colored according to the quad coverage
		else if (ViewModeIndex == VMI_QuadOverdraw)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_QuadOverdraw", "Quad Overdraw"));
		}
		// Visualize the accuracy of the primitive distance computed for texture streaming
		else if (ViewModeIndex == VMI_PrimitiveDistanceAccuracy)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_PrimitiveDistanceAccuracy", "Primitive Distance"));
		}
		// Visualize the accuracy of the mesh UV densities computed for texture streaming
		else if (ViewModeIndex == VMI_MeshUVDensityAccuracy)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_MeshUVDensityAccuracy", "Mesh UV Density"));
		}
		// Colored according to shader complexity, including quad overdraw
		else if (ViewModeIndex == VMI_ShaderComplexityWithQuadOverdraw)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_ShaderComplexityWithQuadOverdraw", "Shader Complexity & Quads"));
		}
		// Colored according to the current HLOD index
		else if (ViewModeIndex == VMI_HLODColoration)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_HLODColoration", "Hierarchical LOD Coloration"));
		}
		// Group item for LOD and HLOD coloration*/
		else if (ViewModeIndex == VMI_GroupLODColoration)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_GroupLODColoration", "Group LOD Coloration"));
		}
		// Visualize the accuracy of the material texture scales used for texture streaming
		else if (ViewModeIndex == VMI_MaterialTextureScaleAccuracy)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_MaterialTextureScaleAccuracy", "Material Texture Scales"));
		}
		// Compare the required texture resolution to the actual resolution
		else if (ViewModeIndex == VMI_RequiredTextureResolution)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_RequiredTextureResolution", "Required Texture Resolution"));
		}
		// Visualize Skin Cache
		else if (ViewModeIndex == VMI_VisualizeGPUSkinCache)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeGPUSkinCache", "GPU Skin Cache"));
		}
		else if (ViewModeIndex == VMI_LWCComplexity)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeLWCComplexity", "Material LWC Function Usage"));
		}

	
		// Ray tracing modes
		// Run path tracing pipeline
		else if (ViewModeIndex == VMI_PathTracing)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_PathTracing", "Path Tracing"));
		}
		// Run ray tracing debug pipeline 
		else if (ViewModeIndex == VMI_RayTracingDebug)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_RayTracingDebug", "Ray Tracing Debug"));
		}

		else if (ViewModeIndex == VMI_VisualizeNanite)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeNanite", "Nanite Visualization"));
		}

		else if (ViewModeIndex == VMI_VisualizeVirtualTexture)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeVirtualTexture", "Virtual Texture Visualization"));
		}

		else if (ViewModeIndex == VMI_VisualizeLumen)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeLumen", "Lumen Visualization"));
		}

		else if (ViewModeIndex == VMI_VisualizeMegaLights)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeMegaLights", "MegaLights Visualization"));
		}

		else if (ViewModeIndex == VMI_VisualizeVirtualShadowMap)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeVirtualShadowMap", "Virtual Shadow Map Visualization"));
		}
		else if (ViewModeIndex == VMI_ShadowCasters)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_ShadowCasters", "VSM Shadow Casters"));
		}

		else if (ViewModeIndex == VMI_VisualizeActorColoration)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_VisualizeActorColoration", "Actor Coloration Visualization"));
		}

		// Geometry Inspection Modes
		else if (ViewModeIndex == VMI_Clay)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI__Clay", "Clay"));
		}
		else if (ViewModeIndex == VMI_Zebra)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI__Zebra", "Zebra"));
		}
		else if (ViewModeIndex == VMI_FrontBackFace)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI__FrontBackFace", "Front/Back Face"));
		}
		else if (ViewModeIndex == VMI_RandomColor)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI__RandomColor", "Random Color"));
		}

		// Streaming Deficit modes
		else if (ViewModeIndex == VMI_StreamingTextureDeficit)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_StreamingTextureDeficit", "Streaming Texture Deficit"));
		}
		else if (ViewModeIndex == VMI_StreamingTextureResidency)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_StreamingTextureResidency", "Streaming Texture Residency"));
		}
		else if (ViewModeIndex == VMI_StreamingMeshLODDeficit)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_StreamingMeshLODDeficit", "Streaming Mesh LOD Deficit"));
		}
		else if (ViewModeIndex == VMI_StreamingMeshLODResidency)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_StreamingMeshLODResidency", "Streaming Mesh LOD Residency"));
		}

		// VMI_Max
		else if (ViewModeIndex == VMI_Max)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Max", "Max EViewModeIndex value"));
		}
		// VMI_Unknown
		else if (ViewModeIndex == VMI_Unknown)
		{
			ViewModeDisplayNames.Emplace(LOCTEXT("UViewModeUtils_VMI_Unknown", "Unknown EViewModeIndex value"));
		}
		// Not considered case
		else
		{
			ViewModeDisplayNames.Emplace(FText::GetEmpty());
		}
	}
	// Return ViewModeIndex
	return ViewModeDisplayNames;
}

const static TArray<FText> GViewModeDisplayNames = FillViewModeDisplayNames();

FText UViewModeUtils::GetViewModeDisplayName(const EViewModeIndex ViewModeIndex)
{
	const FText ViewModeName = GViewModeDisplayNames[ViewModeIndex];
	ensureMsgf(!ViewModeName.IsEmpty(), TEXT("Used an unknown value of EViewModeIndex (with value %d). Consider adding this new value in FillViewModeDisplayNames()"), ViewModeIndex);
	return ViewModeName;
}

// Icons
FName GetViewModeIconName(const EViewModeIndex InViewMode)
{
	static FName WireframeIcon("EditorViewport.WireframeMode");
	static FName UnlitIcon("EditorViewport.UnlitMode");
	static FName LitIcon("EditorViewport.LitMode");
	static FName DetailLightingIcon("EditorViewport.DetailLightingMode");
	static FName LitWireframeIcon("EditorViewport.LitWireframeMode");
	static FName LightingOnlyIcon("EditorViewport.LightingOnlyMode");
	static FName LightComplexityIcon("EditorViewport.LightComplexityMode");
	static FName ShaderComplexityIcon("EditorViewport.ShaderComplexityMode");
	static FName LightmapDensityIcon("EditorViewport.LightmapDensityMode");
	static FName ReflectionOverrideIcon("EditorViewport.ReflectionOverrideMode");
	static FName VisualizeBufferIcon("EditorViewport.VisualizeBufferMode");
	static FName VisualizeNaniteIcon("EditorViewport.VisualizeNaniteMode");
	static FName VisualizeLumenIcon("EditorViewport.VisualizeLumenMode");
	static FName VisualizeMegaLightsIcon("EditorViewport.VisualizeMegaLightsMode");
	static FName VisualizeSubstrateIcon("EditorViewport.VisualizeSubstrateMode");
	static FName VisualizeGroomIcon("EditorViewport.VisualizeGroomMode");
	static FName VisualizeVirtualShadowMapIcon("EditorViewport.VisualizeVirtualShadowMapMode");
	static FName StationaryLightOverlapIcon("EditorViewport.StationaryLightOverlapMode");
	static FName CollisionPawnIcon("EditorViewport.CollisionPawn");
	static FName CollisionVisibilityIcon("EditorViewport.CollisionVisibility");
	static FName LODIcon("EditorViewport.LOD");
	static FName QuadOverdrawIcon("EditorViewport.QuadOverdrawMode");
	static FName TexStreamAccPrimitiveDistanceIcon("EditorViewport.TexStreamAccPrimitiveDistanceMode");
	static FName TexStreamAccMeshUVDensityIcon("EditorViewport.TexStreamAccMeshUVDensityMode");
	static FName ShaderComplexityWithQuadOverdrawIcon("EditorViewport.ShaderComplexityWithQuadOverdrawMode");
	static FName TexStreamAccMaterialTextureScaleIcon("EditorViewport.TexStreamAccMaterialTextureScaleMode");
	static FName RequiredTextureResolutionIcon("EditorViewport.RequiredTextureResolutionMode");
	static FName VisualizeGPUSkinCacheIcon("EditorViewport.VisualizeGPUSkinCacheMode");
	static FName LWCComplexityIcon("EditorViewport.LWCComplexityMode");
	static FName PathTracingIcon("EditorViewport.PathTracingMode");
	static FName RayTracingDebugIcon("EditorViewport.RayTracingDebugMode");
	static FName VisualizeVirtualTextureIcon("EditorViewport.VisualizeVirtualTextureMode");
	static FName VisualizeActorColorationIcon("EditorViewport.VisualizeActorColorationMode");
	static FName GeometryInspectionIcon("EditorViewport.GeometryInspection");

	FName Icon = NAME_None;

	switch (InViewMode)
	{
	case VMI_BrushWireframe:
	case VMI_Wireframe:
		Icon = WireframeIcon;
		break;
	case VMI_Unlit:
		Icon = UnlitIcon;
		break;
	case VMI_Lit:
		Icon = LitIcon;
		break;
	case VMI_Lit_DetailLighting:
		Icon = DetailLightingIcon;
		break;
	case VMI_LightingOnly:
		Icon = LightingOnlyIcon;
		break;
	case VMI_LightComplexity:
		Icon = LightComplexityIcon;
		break;
	case VMI_ShaderComplexity:
		Icon = ShaderComplexityIcon;
		break;
	case VMI_LightmapDensity:
	case VMI_LitLightmapDensity:
		Icon = LightmapDensityIcon;
		break;
	case VMI_ReflectionOverride:
		Icon = ReflectionOverrideIcon;
		break;
	case VMI_VisualizeBuffer:
		Icon = VisualizeBufferIcon;
		break;
	case VMI_StationaryLightOverlap:
		Icon = StationaryLightOverlapIcon;
		break;
	case VMI_CollisionPawn:
		Icon = CollisionPawnIcon;
		break;
	case VMI_CollisionVisibility:
		Icon = CollisionVisibilityIcon;
		break;
	case VMI_LODColoration:
	case VMI_HLODColoration:
	case VMI_GroupLODColoration:
	case VMI_StreamingTextureDeficit:
	case VMI_StreamingTextureResidency:
	case VMI_StreamingMeshLODDeficit:
	case VMI_StreamingMeshLODResidency:
		Icon = LODIcon;
		break;
	case VMI_QuadOverdraw:
		Icon = QuadOverdrawIcon;
		break;
	case VMI_PrimitiveDistanceAccuracy:
		Icon = TexStreamAccPrimitiveDistanceIcon;
		break;
	case VMI_MeshUVDensityAccuracy:
		Icon = TexStreamAccMeshUVDensityIcon;
		break;
	case VMI_ShaderComplexityWithQuadOverdraw:
		Icon = ShaderComplexityWithQuadOverdrawIcon;
		break;
	case VMI_MaterialTextureScaleAccuracy:
		Icon = TexStreamAccMaterialTextureScaleIcon;
		break;
	case VMI_RequiredTextureResolution:
		Icon = RequiredTextureResolutionIcon;
		break;
	case VMI_PathTracing:
		Icon = PathTracingIcon;
		break;
	case VMI_RayTracingDebug:
		Icon = RayTracingDebugIcon;
		break;
	case VMI_VisualizeNanite:
		Icon = VisualizeNaniteIcon;
		break;
	case VMI_VisualizeVirtualTexture:
		Icon = VisualizeVirtualTextureIcon;
		break;
	case VMI_VisualizeLumen:
		Icon = VisualizeLumenIcon;
		break;
	case VMI_VisualizeMegaLights:
		Icon = VisualizeMegaLightsIcon;
		break;
	case VMI_VisualizeVirtualShadowMap:
	case VMI_ShadowCasters:
		Icon = VisualizeVirtualShadowMapIcon;
		break;
	case VMI_VisualizeGPUSkinCache:
		Icon = VisualizeGPUSkinCacheIcon;
		break;
	case VMI_VisualizeSubstrate:
		Icon = VisualizeSubstrateIcon;
		break;
	case VMI_VisualizeGroom:
		Icon = VisualizeGroomIcon;
		break;
	case VMI_LWCComplexity:
		Icon = LWCComplexityIcon;
		break;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case VMI_Lit_Wireframe:
		Icon = LitWireframeIcon;
		break;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	case VMI_VisualizeActorColoration:
		Icon = VisualizeActorColorationIcon;
		break;
	case VMI_Max:
		break;
	case VMI_Clay:
	case VMI_Zebra:
	case VMI_FrontBackFace:
	case VMI_RandomColor:
		Icon = GeometryInspectionIcon;
		break;
	case VMI_Unknown:
		break;
	}

	return Icon;
}

TArray<const FSlateBrush*> FillViewModeDisplayIcons()
{
	// Allocate size
	TArray<const FSlateBrush*> ViewModeDisplayIcons;
	ViewModeDisplayIcons.Reserve(VMI_Unknown);

	// Fill ViewModeIndex, VMI_Unknown+1 to include VMI_Unknown too
	for (int32 Index = 0; Index < VMI_Unknown + 1; ++Index)
	{
		const EViewModeIndex ViewModeIndex = (EViewModeIndex)Index;

		ViewModeDisplayIcons.Emplace(FAppStyle::Get().GetBrush(GetViewModeIconName(ViewModeIndex)));
	}

	return ViewModeDisplayIcons;
}

static TArray<const FSlateBrush*> GViewModeDisplayIcons;

const FSlateBrush* UViewModeUtils::GetViewModeDisplayIcon(const EViewModeIndex ViewModeIndex)
{
	static bool IconsInitialized = false;
	if (!IconsInitialized)
	{
		GViewModeDisplayIcons = FillViewModeDisplayIcons();
		IconsInitialized = true;
	}

	return GViewModeDisplayIcons[ViewModeIndex];
}

FSlateIcon UViewModeUtils::GetViewModeDisplaySlateIcon(const EViewModeIndex InViewModeIndex)
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), GetViewModeIconName(InViewModeIndex));
}

#undef LOCTEXT_NAMESPACE
