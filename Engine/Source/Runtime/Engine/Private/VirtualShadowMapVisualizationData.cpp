// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualShadowMapVisualizationData.h"
#include "VirtualShadowMapDefinitions.h"
#include "SceneManagement.h"

#define LOCTEXT_NAMESPACE "FVirtualShadowMapVisualizationData"

static FVirtualShadowMapVisualizationData GVirtualShadowMapVisualizationData;

void FVirtualShadowMapVisualizationData::Initialize()
{
	if (!bIsInitialized)
	{
		// NOTE: The first parameter determines the console command parameter. "none", "off" and "list" are reserved
		AddVisualizationMode(
			TEXT("mask"),
			LOCTEXT("ShadowMask", "Shadow Mask"),
			LOCTEXT("ShadowMaskDesc", "The final shadow mask that is used by shading"),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_FACTOR);

		AddVisualizationMode(
			TEXT("mip"),
			LOCTEXT("ClipmapOrMip", "Clipmap/Mip Level"),
			LOCTEXT("ClipmapOrMipDesc", "The chosen clipmap (for directional lights) or mip (for local lights) level"),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_OR_MIP);

		AddVisualizationMode(
			TEXT("vpage"),
			LOCTEXT("VirtualPage", "Virtual Page"),
			LOCTEXT("VirtualPageDesc", "Visualization of the virtual page address"),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_VIRTUAL_PAGE);

		AddVisualizationMode(
			TEXT("cache"),
			LOCTEXT("CachedPage", "Cached Page"),
			LOCTEXT("CachedPageDesc", "Cached pages are tinted green, uncached are red. Pages where only the static page is cached (dynamic uncached) are blue."),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_CACHED_PAGE);

		AddVisualizationMode(
			TEXT("naniteoverdraw"),
			LOCTEXT("NaniteOverdraw", "Nanite Overdraw"),
			LOCTEXT("NaniteOverdrawDesc", "Nanite overdraw into mapped pages"),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW);

		AddVisualizationMode(
			TEXT("casters"),
			LOCTEXT("ShadowCasters", "Shadow Casters"),
			LOCTEXT("ShadowCastersDesc", "Visualize objects that cast shadows, and which type of shadow map they invalidate."),
			FModeType::Standard,
			VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_CASTERS,
			VMI_ShadowCasters);

		AddVisualizationMode(
			TEXT("raycount"),
			LOCTEXT("SMRTRayCount", "SMRT Ray Count"),
			LOCTEXT("SMRTRayCountDesc", "Rays evaluated per pixel: red is more, green is fewer. Penumbra regions require more rays and are more expensive."),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_SMRT_RAY_COUNT);

		AddVisualizationMode(
			TEXT("dirty"),
			LOCTEXT("DirtyPage", "Dirty Page"),
			LOCTEXT("DirtyPageDebugDesc", "Show the pages marked as dirty."),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_DIRTY_PAGE);

		AddVisualizationMode(
			TEXT("invalid"),
			LOCTEXT("InvalidPage", "GPU Invalidated Page"),
			LOCTEXT("InvalidPageDebugDesc", "Show the pages marked for GPU-driven invalidation (World Position Offset)."),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_GPU_INVALIDATED_PAGE);

		AddVisualizationMode(
			TEXT("merged"),
			LOCTEXT("MergedPage", "Merged Page"),
			LOCTEXT("MergedPageDebugDesc", "Show the pages that were merged."),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_MERGED_PAGE);

		AddVisualizationMode(
			TEXT("debug"),
			LOCTEXT("GeneralDebug", "General Debug"),
			LOCTEXT("GeneralDebugDesc", "General-purpose debug for use during shader development"),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_GENERAL_DEBUG);

		AddVisualizationMode(
			TEXT("clipmapvirtual"),
			LOCTEXT("ClipmapVirtualSpace", "Clipmap Virtual Address Space"),
			LOCTEXT("ClipmapVirtualSpaceDesc", "Visualization of the clipmap virtual address space and mapped pages"),
			FModeType::Advanced,
			VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_VIRTUAL_SPACE);

		ConfigureConsoleCommand();

		bIsInitialized = true;
	}
}

void FVirtualShadowMapVisualizationData::ConfigureConsoleCommand()
{
	FString AvailableVisualizationModes;
	for (TModeMap::TConstIterator It = ModeMap.CreateConstIterator(); It; ++It)
	{
		const FModeRecord& Record = It.Value();
		AvailableVisualizationModes += FString("\n  ");
		AvailableVisualizationModes += Record.ModeString;
	}

	ConsoleDocumentationVisualizationMode = TEXT("When the viewport view-mode is set to 'Virtual Shadow Map Visualization', this command specifies which of the various channels to display. Values entered other than the allowed values shown below will be ignored.");
	ConsoleDocumentationVisualizationMode += AvailableVisualizationModes;

	IConsoleManager::Get().RegisterConsoleVariable(
		GetVisualizeConsoleCommandName(),
		TEXT(""),
		*ConsoleDocumentationVisualizationMode,
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("r.Shadow.Virtual.Visualize.NextLight"),
		TEXT("Set `r.Shadow.Virtual.Visualize.LightName` to the next light."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda(
			[this](const TArray<FString>& Arguments, FOutputDevice& Output)
			{
				SelectAdjacentVisualizeLight = 1;
			}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("r.Shadow.Virtual.Visualize.PrevLight"),
		TEXT("Set `r.Shadow.Virtual.Visualize.LightName` to the previous light."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda(
			[this](const TArray<FString>& Arguments, FOutputDevice& Output)
			{
				SelectAdjacentVisualizeLight = -1;
			}),
		ECVF_Cheat);
}

void FVirtualShadowMapVisualizationData::SetActiveMode(int32 ModeID, const FName& ModeName)
{
	ActiveVisualizationModeID = ModeID;
	ActiveVisualizationModeName = ModeName;
}

bool FVirtualShadowMapVisualizationData::IsActive() const
{
	if (!IsInitialized())
	{
		return false;
	}

	if (GetActiveModeID() == INDEX_NONE)
	{
		return false;
	}

	return true;
}

bool FVirtualShadowMapVisualizationData::Update(const FName& InViewMode)
{
	bool bForceShowFlag = false;

	if (!IsInitialized())
	{
		return false;
	}

	SetActiveMode(INDEX_NONE, NAME_None);
			
	// Check if the console command is set (overrides the editor)
	{
		static IConsoleVariable* ICVarVisualize = IConsoleManager::Get().FindConsoleVariable(GetVisualizeConsoleCommandName());
		if (ICVarVisualize)
		{
			const FString ConsoleVisualizationMode = ICVarVisualize->GetString();
			if (!ConsoleVisualizationMode.IsEmpty())
			{
				 if (ConsoleVisualizationMode == TEXT("off") || ConsoleVisualizationMode == TEXT("none"))
				{
					// Disable visualization
				}
				else
				{
					const FName  ModeName = FName(*ConsoleVisualizationMode);
					const int32  ModeID   = GetModeID(ModeName);
					if (ModeID == INDEX_NONE)
					{
						UE_LOGF(LogVirtualShadowMapVisualization, Warning, "Unknown virtual shadow map visualization mode '%ls'", *ConsoleVisualizationMode);
					}
					else
					{
						SetActiveMode(ModeID, ModeName);
						bForceShowFlag = true;
					}
				}
			}
		}
	}

	// Check the view mode state (set by editor).
	if (ActiveVisualizationModeID == INDEX_NONE && InViewMode != NAME_None)
	{
		const int32 ModeID = GetModeID(InViewMode);
		if (ModeID != INDEX_NONE)
		{
			SetActiveMode(ModeID, InViewMode);
		}
	}

	return bForceShowFlag;
}

void FVirtualShadowMapVisualizationData::AddVisualizationMode(
	const TCHAR* ModeString,
	const FText& ModeText,
	const FText& ModeDesc,
	const FModeType ModeType,
	int32 ModeID,
	EViewModeIndex ViewModeIndex
)
{
	const FName ModeName = FName(ModeString);

	FModeRecord& Record	= ModeMap.Emplace(ModeName);
	Record.ModeString			= FString(ModeString);
	Record.ModeName				= ModeName;
	Record.ModeText				= ModeText;
	Record.ModeDesc				= ModeDesc;
	Record.ModeType				= ModeType;
	Record.ModeID				= ModeID;
	Record.ViewModeIndex        = ViewModeIndex;
}

FText FVirtualShadowMapVisualizationData::GetModeDisplayName(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeText;
	}
	else
	{
		return FText::GetEmpty();
	}
}

EViewModeIndex FVirtualShadowMapVisualizationData::GetViewModeIndexForVisualization(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ViewModeIndex;
	}
	else
	{
		return VMI_VisualizeVirtualShadowMap;
	}
}

bool FVirtualShadowMapVisualizationData::IsVSMViewMode(EViewModeIndex ViewModeIndex) const
{
	return ViewModeIndex == VMI_VisualizeVirtualShadowMap
		|| ViewModeIndex == VMI_ShadowCasters;
}

int32 FVirtualShadowMapVisualizationData::GetModeID(const FName& InModeName) const
{
	if (const FModeRecord* Record = ModeMap.Find(InModeName))
	{
		return Record->ModeID;
	}
	else
	{
		return INDEX_NONE;
	}
}

FVirtualShadowMapVisualizationData& GetVirtualShadowMapVisualizationData()
{
	if (!GVirtualShadowMapVisualizationData.IsInitialized())
	{
		GVirtualShadowMapVisualizationData.Initialize();
	}

	return GVirtualShadowMapVisualizationData;
}

#undef LOCTEXT_NAMESPACE
