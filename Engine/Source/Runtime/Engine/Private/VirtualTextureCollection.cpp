// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/VirtualTextureCollection.h"
#include "EngineModule.h"
#include "Engine/Texture.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderTargetPool.h"
#include "VirtualTextureEnum.h"
#include "RHIStaticStates.h"
#include "Rendering/Texture2DResource.h"
#include "VT/CopyCompressShader.h"
#include "ImageUtils.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "TextureCompiler.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualTextureCollection)

#define LOCTEXT_NAMESPACE "UVirtualTextureCollection"

DEFINE_LOG_CATEGORY_STATIC(LogVirtualTextureCollection, Log, All);

struct FTextureCollectionProducerData
{
	/** All textures being produced (virtual or physical) */
	TArray<FVirtualTextureCollectionResource::FTextureEntry> Textures;

	/** The host-side page table for block to texture indices */
	TArray<int32> BlockVirtualPageTable;

	/** All starting block coordinates of the textures */
	TArray<FUintVector2> BlockCoordinates;

	/** Square allocation size, always power of two */
	TArray<uint32> AllocationSizes;

	/** Total block count of the shared VT */
	FUintVector2 BlockCount;
};

struct FTextureCollectionPackEntry
{
	/** Texture index, container is sorted */
	int32 TextureIndex;
	
	/** Number of blocks for square allocation */
	uint32 SquareSize;
};

struct FTextureCollectionPackResult
{
	/** All allocation coordinates */
	TArray<FUintVector2> Coordinates;
	
	/** All square allocation sizes */
	TArray<uint32> AllocationSizes;
	
	/** Final block count */
	FUintVector2 BlockCount;
};

struct FTextureCollectionPendingAdapterTile
{
	/** Texture entry to apply adapter to */
	FVirtualTextureCollectionResource::FTextureEntry Entry;
	
	/** Physical destination for the adapter */
	FVTProduceTargetLayer Target;

	/** Optional, finalizer for virtual textures, to be invoked before the adapter */
	IVirtualTextureFinalizer* VirtualFinalizer = nullptr;

	/** The intermediate physical target for the virtual texture, not physical */
	TRefCountPtr<IPooledRenderTarget> PooledVirtualRenderTarget;
	
	/** Intra-texture block address */
	uint64 Address = 0;
	
	/** Mip level */
	uint8 Level = 0;
};

static FUintVector2 DecodeBlockAddress(uint32 vAddress)
{
	return FUintVector2(FMath::ReverseMortonCode2_64(vAddress), FMath::ReverseMortonCode2_64(vAddress >> 1));
}

static uint32 EncodeBlockAddress(const FUintVector2& Addr)
{
	return FMath::MortonCode2_64(Addr.X) | (FMath::MortonCode2_64(Addr.Y) << 1);
}

static FTexture2DResource* GetTextureObjectResource2D(UTexture* Texture)
{
	FTextureResource* Resource = Texture ? Texture->GetResource() : nullptr;
	return Resource ? Resource->GetTexture2DResource() : nullptr;
}

static bool TryPackTextureCollectionAtlas(TArrayView<const FTextureCollectionPackEntry> Entries, uint32 SquareSize, FTextureCollectionPackResult& Result)
{
	// Number of full 64-masks per row
	const uint32 MasksPerRow = (SquareSize + 63) / 64;
	
	TArray<uint64> Occupancy;
	Occupancy.SetNumZeroed(SquareSize * MasksPerRow);

	for (const FTextureCollectionPackEntry& Entry : Entries)
	{
		bool bAllocated = false;

		// Can we use a single mask per row?
		if (Entry.SquareSize <= 64)
		{
			// Safe alignment positions, just because a certain position was free, doesn't mean it was aligned
			uint64 AlignMask = 0;
			for (uint32 BitOffset = 0; BitOffset + Entry.SquareSize <= 64; BitOffset += Entry.SquareSize)
			{
				AlignMask |= (1ull << BitOffset);
			}

			const uint64 OccupancyMask = (Entry.SquareSize < 64) ? ((1ull << Entry.SquareSize) - 1) : ~0ull;

			// Scanline for all rows, per 64-mask for columns
			for (uint32 MaskRow = 0; MaskRow <= SquareSize - Entry.SquareSize && !bAllocated; MaskRow += Entry.SquareSize)
			{
				for (uint32 MaskColumn = 0; MaskColumn < MasksPerRow && !bAllocated; MaskColumn++)
				{
					uint64 CombinedMask = 0;
					
					// Combine rows on the texture
					for (uint32 EntryRow = 0; EntryRow < Entry.SquareSize; EntryRow++)
					{
						CombinedMask |= Occupancy[(MaskRow + EntryRow) * MasksPerRow + MaskColumn];
					}

					// Reduce the free positions down, shift since it's aligned by 2
					uint64 FreeMask = ~CombinedMask;
					for (uint32 Shift = 1; Shift < Entry.SquareSize; Shift <<= 1)
					{
						FreeMask &= (FreeMask >> Shift);
					}
					
					// And, only accept aligned positions
					FreeMask &= AlignMask;

					// Any to allocate?
					if (FreeMask != 0)
					{
						const uint32 MaskPosition = FMath::CountTrailingZeros64(FreeMask);
						const uint32 BlockX       = MaskColumn * 64 + MaskPosition;

						// Out of bounds?
						if (BlockX + Entry.SquareSize <= SquareSize)
						{
							// Success, mark all rows as occupied
							for (uint32 EntryRow = 0; EntryRow < Entry.SquareSize; EntryRow++)
							{
								Occupancy[(MaskRow + EntryRow) * MasksPerRow + MaskColumn] |= (OccupancyMask << MaskPosition);
							}

							Result.Coordinates[Entry.TextureIndex]     = FUintVector2(BlockX, MaskRow);
							Result.AllocationSizes[Entry.TextureIndex] = Entry.SquareSize;
							bAllocated = true;
						}
					}
				}
			}
		}
		else
		{
			// Otherwise, number of masks needed
			const uint32 EntryRowMaskCount = Entry.SquareSize / 64;

			for (uint32 MaskRow = 0; MaskRow <= SquareSize - Entry.SquareSize && !bAllocated; MaskRow += Entry.SquareSize)
			{
				for (uint32 MaskColumn = 0; MaskColumn + EntryRowMaskCount <= MasksPerRow && !bAllocated; MaskColumn += EntryRowMaskCount)
				{
					bool bFree = true;
					
					// Check if we can allocate the entry at the current offset
					for (uint32 EntryRow = 0; EntryRow < Entry.SquareSize && bFree; EntryRow++)
					{
						const uint32 RowBase = (MaskRow + EntryRow) * MasksPerRow;
						for (uint32 MaskIndex = 0; MaskIndex < EntryRowMaskCount && bFree; MaskIndex++)
						{
							// Since we're >64, and always squared, just != 0
							if (Occupancy[RowBase + MaskColumn + MaskIndex] != 0)
							{
								bFree = false;
							}
						}
					}

					// Can we allocate?
					if (bFree)
					{
						const uint32 BlockX = MaskColumn * 64;

						// Mark all rows as occupied
						for (uint32 EntryRow = 0; EntryRow < Entry.SquareSize; EntryRow++)
						{
							const uint32 RowBase = (MaskRow + EntryRow) * MasksPerRow;
							for (uint32 MaskIndex = 0; MaskIndex < EntryRowMaskCount; MaskIndex++)
							{
								Occupancy[RowBase + MaskColumn + MaskIndex] = ~0ull;
							}
						}

						Result.Coordinates[Entry.TextureIndex] = FUintVector2(BlockX, MaskRow);
						Result.AllocationSizes[Entry.TextureIndex] = Entry.SquareSize;
						bAllocated = true;
					}
				}
			}
		}

		// if any failed to allocate, the entire thing fails
		if (!bAllocated)
		{
			return false;
		}
	}

	return true;
}

static FTextureCollectionPackResult PackTextureCollectionAtlas(TArrayView<const FTextureCollectionPackEntry> Entries, int32 NumTextures)
{
	FTextureCollectionPackResult Result;
	Result.Coordinates.SetNumZeroed(NumTextures);
	Result.AllocationSizes.SetNumZeroed(NumTextures);
	Result.BlockCount = FUintVector2(1, 1);

	if (Entries.IsEmpty())
	{
		return Result;
	}

	// Estimate initial atlas size from total area
	uint32 TotalArea = 0;
	for (const FTextureCollectionPackEntry& Entry : Entries)
	{
		TotalArea += Entry.SquareSize * Entry.SquareSize;
	}

	uint32 SquareAtlasSize = FMath::Max(1u, FMath::RoundUpToPowerOfTwo(FMath::CeilToInt32(FMath::Sqrt(static_cast<float>(TotalArea)))));

	// Maximum atlas size matches the 12-bit block coordinate limit
	constexpr uint32 MaxAtlasSquareSize = 0xFFF;

	// Just keep trying until we find a good fit
	while (!TryPackTextureCollectionAtlas(Entries, SquareAtlasSize, Result))
	{
		SquareAtlasSize *= 2;
		
		if (SquareAtlasSize > MaxAtlasSquareSize)
		{
			UE_LOGF(LogVirtualTextureCollection, Error, "Atlas packing failed: %d entries exceed maximum atlas size (%u)", Entries.Num(), MaxAtlasSquareSize);
			return Result;
		}
	}

	// Compute tight bounding box from actual placements
	for (const FTextureCollectionPackEntry& Entry : Entries)
	{
		Result.BlockCount.X = FMath::Max(Result.BlockCount.X, Result.Coordinates[Entry.TextureIndex].X + Entry.SquareSize);
		Result.BlockCount.Y = FMath::Max(Result.BlockCount.Y, Result.Coordinates[Entry.TextureIndex].Y + Entry.SquareSize);
	}

	return Result;
}

/** Adapter finalizer, handles (optional) format conversions and block compression */
class FTextureCollectionVirtualAdapterFinalizer : public IVirtualTextureFinalizer
{
public:
	FTextureCollectionVirtualAdapterFinalizer(const FVTProducerDescription& ProducerDesc) : ProducerDesc(ProducerDesc)
	{
		
	}

	virtual ~FTextureCollectionVirtualAdapterFinalizer() = default;

	virtual void RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderingContext) override
	{
		for (IVirtualTextureFinalizer* PendingFinalizer : PendingFinalizers)
		{
			PendingFinalizer->RenderFinalize(GraphBuilder, SceneRenderingContext);
		}
	}

	virtual void Finalize(FRDGBuilder& GraphBuilder) override
	{
		// First, finalize all intermediate data onto the temporary targets
		for (IVirtualTextureFinalizer* PendingFinalizer : PendingFinalizers)
		{
			PendingFinalizer->Finalize(GraphBuilder);
		}

		// Second, run the adapter on each pending tile
		for (const FTextureCollectionPendingAdapterTile& PendingTile : PendingTiles)
		{
			const FPooledRenderTargetDesc TargetDesc = PendingTile.Target.PooledRenderTarget->GetDesc();
			const bool bTargetIsCompressed = IsBlockCompressedFormat(TargetDesc.Format);
			
			// Physical tile location
			const int32     TileSize = ProducerDesc.TileSize + 2 * ProducerDesc.TileBorderSize;
			const FIntPoint DestinationPos(PendingTile.Target.pPageLocation.X * TileSize, PendingTile.Target.pPageLocation.Y * TileSize);
			const FIntRect  DestRect(DestinationPos, DestinationPos + FIntPoint(TileSize, TileSize));

			// Select the UAV aliasing format, 32 or 64 wide
			const bool         bAliasTo64bit = ProducerDesc.LayerFormat[0] == PF_DXT1 || ProducerDesc.LayerFormat[0] == PF_BC4;
			const EPixelFormat AliasFormat = bAliasTo64bit ? PF_R32G32_UINT : PF_R32G32B32A32_UINT;

			// If not compressed or the target supports aliasing, just write directly into the physical resource
			const bool bWriteToPhysical = !bTargetIsCompressed || GRHISupportsUAVFormatAliasing;
			
			FRDGTextureRef CurrentOutput;
			if (bWriteToPhysical)
			{
				CurrentOutput = GraphBuilder.RegisterExternalTexture(PendingTile.Target.PooledRenderTarget, ERDGTextureFlags::None);
			}
			else
			{
				CurrentOutput = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(
					FIntPoint(TileSize >> 2, TileSize >> 2),
					AliasFormat, 
					FClearValueBinding::None,
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV
				), TEXT("IntermediateTarget"));
			}

			// Select the compression permutation
			int32 CompressionDim = 0;
			if (IsBlockCompressedFormat(CurrentOutput->Desc.Format))
			{
				CompressionDim = FCopyCompressCS::GetCompressionPermutation(CurrentOutput->Desc.Format);
			}

			FCopyCompressCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyCompressCS::FSourceTextureSelector>(true);
			PermutationVector.Set<FCopyCompressCS::FDestSrgb>(ProducerDesc.bIsLayerSRGB[0]);
			PermutationVector.Set<FCopyCompressCS::FCompressionFormatDim>(CompressionDim);
			TShaderMapRef<FCopyCompressCS> Shader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

			FCopyCompressCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyCompressCS::FParameters>();
			Parameters->TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			Parameters->DestTexture = GraphBuilder.CreateUAV(CurrentOutput);
			Parameters->DestCompressTexture_64bit = bAliasTo64bit ? GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CurrentOutput, 0, PF_R32G32_UINT)) : nullptr;
			Parameters->DestCompressTexture_128bit = bAliasTo64bit ? nullptr : GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CurrentOutput, 0, PF_R32G32B32A32_UINT));
			Parameters->TexelOffsets = FVector2f(1.0f, 0.5f);
			Parameters->DestRect = FIntVector4(DestRect.Min.X, DestRect.Min.Y, DestRect.Max.X, DestRect.Max.Y);

			// Coming from a virtual texture?
			if (PendingTile.PooledVirtualRenderTarget)
			{
				// No mipped views, always a single tile
				Parameters->SourceTextureA = GraphBuilder.RHICmdList.CreateShaderResourceView(
					PendingTile.PooledVirtualRenderTarget->GetRHI(),
					FRHIViewDesc::CreateTextureSRV()
						.SetDimensionFromTexture(PendingTile.PooledVirtualRenderTarget->GetRHI())
				);

				// Set UV ranges
				Parameters->SourceUV  = FVector2f(0, 0);
				Parameters->TexelSize = FVector2f(1.0f / PendingTile.PooledVirtualRenderTarget->GetDesc().Extent.X, 1.0f / PendingTile.PooledVirtualRenderTarget->GetDesc().Extent.Y);
			}
			else
			{
				// Resource may have been released
				FTexture2DResource* Resource = GetTextureObjectResource2D(PendingTile.Entry.PhysicalTexture);
				if (!Resource)
				{
					continue;
				}
				
				FTextureRHIRef TextureRHI = Resource->GetTextureRHI();
				if (!ensure(TextureRHI.IsValid()))
				{
					continue;
				}

				// Create view into the repsective mip
				Parameters->SourceTextureA = GraphBuilder.RHICmdList.CreateShaderResourceView(
					TextureRHI,
					FRHIViewDesc::CreateTextureSRV()
						.SetDimensionFromTexture(TextureRHI)
						.SetMipRange(FMath::Clamp(static_cast<int32>(PendingTile.Level) - static_cast<int32>(Resource->GetCurrentFirstMip()), 0, Resource->GetCurrentMipCount() - 1), 1)
				);

				// Physical coordinates on the texture entry
				const float X        = static_cast<float>(FMath::ReverseMortonCode2_64(PendingTile.Address));
				const float Y        = static_cast<float>(FMath::ReverseMortonCode2_64(PendingTile.Address >> 1));
				const float DivisorX = static_cast<float>(PendingTile.Entry.BlockCount.X) / static_cast<float>(1 << PendingTile.Level);
				const float DivisorY = static_cast<float>(PendingTile.Entry.BlockCount.Y) / static_cast<float>(1 << PendingTile.Level);

				// To UV coordinates
				const FVector2f UV(X / DivisorX, Y / DivisorY);
				const FVector2f UVSize(1.0f / DivisorX, 1.0f / DivisorY);
				const FVector2f UVBorder = UVSize * (static_cast<float>(ProducerDesc.TileBorderSize) / static_cast<float>(ProducerDesc.TileSize));
				const FBox2f    UVRect(UV - UVBorder, UV + UVSize + UVBorder);

				// Set UV ranges
				Parameters->SourceUV  = UVRect.Min;
				Parameters->TexelSize = (UVRect.Max - UVRect.Min) / FVector2f(DestRect.Width(), DestRect.Height());
			}

			FIntPoint ThreadCount = FIntPoint(DestRect.Width(), DestRect.Height());

			// If compressed, the kernel's dealing with blocks, not texels
			if (bTargetIsCompressed)
			{
				ThreadCount /= 4;
				Parameters->DestRect /= 4;
				Parameters->TexelOffsets = FVector2f(4.0f, 0.5f);
			}

			ClearUnusedGraphResources(Shader, Parameters);
			
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("VirtualTextureCollectionAdapter"),
				Parameters,
				ERDGPassFlags::Compute,
				[Shader, Parameters, ThreadCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
				{
					FComputeShaderUtils::Dispatch(RHICmdList, Shader, *Parameters, FComputeShaderUtils::GetGroupCount(ThreadCount, 8));
				}
			);

			if (!bWriteToPhysical)
			{
				FRHICopyTextureInfo CopyInfo;
				CopyInfo.DestPosition = FIntVector(DestRect.Min.X, DestRect.Min.Y, 0);
				CopyInfo.Size = FIntVector(DestRect.Width(), DestRect.Height(), 0);

				if (bTargetIsCompressed)
				{
					CopyInfo.Size /= 4;
				}
				
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("VirtualTextureCollectionPhysicalBlit"),
					Parameters,
					ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
					[
						CurrentOutput,
						DestTexture = PendingTile.Target.PooledRenderTarget->GetRHI(),
						CopyInfo
					](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.CopyTexture(CurrentOutput->GetRHI(), DestTexture, CopyInfo);
				});
			}
		}

		PendingFinalizers.Empty();
		PendingTiles.Empty();
	}

	void Add(const FTextureCollectionPendingAdapterTile& Tile)
	{
		if (Tile.VirtualFinalizer)
		{
			PendingFinalizers.AddUnique(Tile.VirtualFinalizer);
		}
		
		PendingTiles.Add(Tile);
	}
	
private:
	TArray<IVirtualTextureFinalizer*> PendingFinalizers;
	
	TArray<FTextureCollectionPendingAdapterTile> PendingTiles;

	FVTProducerDescription ProducerDesc;
};

class FTextureCollectionVirtualRedirector : public IVirtualTexture
{
public:
	FTextureCollectionVirtualRedirector(FTextureCollectionProducerData&& Data, const FVTProducerDescription& ProducerDesc) : Data(MoveTemp(Data)), ProducerDesc(ProducerDesc), Finalizer(ProducerDesc)
	{
		
	}
	
	virtual ~FTextureCollectionVirtualRedirector() override = default;

	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
	{
		const FUintVector2 Address = DecodeBlockAddress(vAddress);

		// Get the owning texture, index table is always base mip
		const int32 TextureIndex = Data.BlockVirtualPageTable[(Address.Y << vLevel) * Data.BlockCount.X + (Address.X << vLevel)];
		if (TextureIndex == INDEX_NONE)
		{
			return false;
		}

		// Always allow for coarser mips to stream in to "unblock" finer ones, the physical level may be smaller than the logical due to packing
		const FVirtualTextureCollectionResource::FTextureEntry& Texture = Data.Textures[TextureIndex];
		if (vLevel > Texture.MaxLevel)
		{
			return true;
		}

		// Non-adapter entries are cheap to produce
		return !Texture.bRequiresAdapter;
	}

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority) override
	{
		const FUintVector2 Address = DecodeBlockAddress(vAddress);

		// Get the owning texture, index table is always base mip
		const int32 Index = Data.BlockVirtualPageTable[(Address.Y << vLevel) * Data.BlockCount.X + (Address.X << vLevel)];
		if (Index == INDEX_NONE)
		{
			return EVTRequestPageStatus::Invalid;
		}

		// Always allow for coarser mips to produce to "unblock" finer ones, the physical level may be smaller than the logical due to packing
		if (vLevel > Data.Textures[Index].MaxLevel)
		{
			FVTRequestPageResult Result = EVTRequestPageStatus::Available;
			return Result;
		}

		// Shift the coordinates to the mip
		FUintVector2 BlockAddress = Data.BlockCoordinates[Index];
		BlockAddress.X >>= vLevel;
		BlockAddress.Y >>= vLevel;

		FVTRequestPageResult Result = EVTRequestPageStatus::Saturated;

		const FVirtualTextureCollectionResource::FTextureEntry& Texture = Data.Textures[Index];
		if (Texture.VirtualProducerHandle.IsValid())
		{
			// Virtual, query the owning texture
			if (IVirtualTexture* VirtualTexture = GetRendererModule().FindProducer(Texture.VirtualProducerHandle))
			{
				Result = VirtualTexture->RequestPageData(
					RHICmdList,
					GetLocalProducerHandle(ProducerHandle, Index),
					LayerMask,
					vLevel,
					EncodeBlockAddress(Address - BlockAddress),
					Priority
				);
			}
		}
		else if (FTexture2DResource* Resource = GetTextureObjectResource2D(Texture.PhysicalTexture))
		{
			// Check if the mip is streamed in
			FStreamableRenderResourceState State = Resource->GetState();
			if (State.IsValid())
			{
				if (vLevel >= State.MaxNumLODs)
				{
					// Will never be produced
					// Due to VT-streaming logic, just assume it's available to mark the upper mips as available overall
					Result.Status = EVTRequestPageStatus::Available;
				}
				else
				{
					// Check if the mip is streamed in yet
					bool bIsStreamedIn = vLevel >= State.ResidentFirstLODIdx();
					Result.Status = bIsStreamedIn ? EVTRequestPageStatus::Available : EVTRequestPageStatus::Saturated;
				}
			}
			else
			{
				// The resource isn't streamed, always available
				Result.Status = EVTRequestPageStatus::Available;
			}
		}
		else
		{
			// Invalid resource
			Result.Status = EVTRequestPageStatus::Invalid;
		}

		return Result;
	}

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers) override
	{
		const FUintVector2 Address = DecodeBlockAddress(vAddress);

		// Get the owning texture, index table is always base mip
		const int32 Index = Data.BlockVirtualPageTable[(Address.Y << vLevel) * Data.BlockCount.X + (Address.X << vLevel)];
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}

		// Safe mip clamp
		const uint8 vLevelClamped = FMath::Min(vLevel, Data.Textures[Index].MaxLevel);

		// Compute clamped address
		FUintVector2 BlockAddress = Data.BlockCoordinates[Index];
		BlockAddress.X >>= vLevelClamped;
		BlockAddress.Y >>= vLevelClamped;

		// Back to physical
		FUintVector2 AddressClamped;
		AddressClamped.X = Address.X << (vLevel - vLevelClamped);
		AddressClamped.Y = Address.Y << (vLevel - vLevelClamped);

		const FVirtualTextureCollectionResource::FTextureEntry& Texture = Data.Textures[Index];

		// Optional, adapter tile for format conversion
		FTextureCollectionPendingAdapterTile AdapterTile;
		AdapterTile.Entry = Texture;
		AdapterTile.Target = *TargetLayers;
		AdapterTile.Level = vLevelClamped;
		AdapterTile.Address = EncodeBlockAddress(AddressClamped - BlockAddress);

		// Optional, target tile redirection for format conversion
		FVTProduceTargetLayer RedirectedTarget = AdapterTile.Target;

		if (Texture.VirtualProducerHandle.IsValid())
		{
			// Virtual, pass to the owning texture
			if (IVirtualTexture* VirtualTexture = GetRendererModule().FindProducer(Texture.VirtualProducerHandle))
			{
				// If this producer requires an adapter, we render to an intermediate target instead of the current tile
				if (Texture.bRequiresAdapter)
				{
					const int32 TileExtent = ProducerDesc.TileSize + ProducerDesc.TileBorderSize * 2;

					// Find a free target
					GRenderTargetPool.FindFreeElement(RHICmdList, FPooledRenderTargetDesc::Create2DDesc(
						FIntPoint(TileExtent, TileExtent),
						Texture.Format,
						FClearValueBinding(FLinearColor::Black),
						TexCreate_None, TexCreate_ShaderResource,
						false
					), AdapterTile.PooledVirtualRenderTarget, TEXT("VirtualRenderTarget"));

					// External access in FVirtualTexturePhysicalSpace::FinalizeTextures assumes SRVMask
					RHICmdList.TransitionInternal(FRHITransitionInfo(AdapterTile.PooledVirtualRenderTarget->GetRHI(), ERHIAccess::Unknown, ERHIAccess::SRVMask));
					
					RedirectedTarget.PooledRenderTarget = AdapterTile.PooledVirtualRenderTarget;
					RedirectedTarget.pPageLocation       = FIntVector::ZeroValue;
				}
				
				AdapterTile.VirtualFinalizer = VirtualTexture->ProducePageData(
					RHICmdList,
					FeatureLevel,
					Flags,
					GetLocalProducerHandle(ProducerHandle, Index),
					LayerMask,
					vLevelClamped,
					AdapterTile.Address,
					RequestHandle,
					&RedirectedTarget
				);
			}
		}

		// If this doesn't require an adapter, just use the producers finalizer
		if (!Texture.bRequiresAdapter)
		{
			return AdapterTile.VirtualFinalizer;
		}

		// Otherwise, use the adapters finalizer
		// This also invokes the above finalizer
		Finalizer.Add(AdapterTile);
		return &Finalizer;
	}

private:
	static FVirtualTextureProducerHandle GetLocalProducerHandle(const FVirtualTextureProducerHandle& RedirectorHandle, uint32 Index)
	{
		// Due to the virtual transcoder cache, we need to offset the local handle
		// to avoid hashing on physically different, but virtually/logically same requests
		FVirtualTextureProducerHandle LocalHandle = RedirectorHandle;
		LocalHandle.Magic += Index;
		return LocalHandle;
	}

private:
	FTextureCollectionProducerData Data;
	FVTProducerDescription         ProducerDesc;

	/** Shared finalizer for this redirector*/
	FTextureCollectionVirtualAdapterFinalizer Finalizer;
};

static EPixelFormat GetResourcePixelFormat(FTextureResource* Resource)
{
	if (FVirtualTexture2DResource* VirtualResource = Resource->GetVirtualTexture2DResource())
	{
		return VirtualResource->GetFormat(0);
	}
	
	if (FTexture2DResource* PhysicalResource = Resource->GetTexture2DResource())
	{
		return PhysicalResource->GetPixelFormat();
	}

	return PF_Unknown;
}

FVirtualTextureCollectionResource::FVirtualTextureCollectionResource(UVirtualTextureCollection* InParent) : FTextureCollectionResource(InParent)
{
	bIsBindless            = false;
	Textures               = InParent->Textures;
	bIsSRGB                = InParent->bIsSRGB;
	bAllowFormatConversion = InParent->bAllowFormatConversion;

	// Share the same build settings as general VT resources
	// This avoids costly handling of mismatched tile sizes
	BuildSettings.Init();

	// If implicit, find the best matching format for the collection
	// TODO[MP]: We could likely handle this at edit-time for most collections, then serialize?
	if (Format == PF_Unknown)
	{
		if (bAllowFormatConversion)
		{
			FindConservativeFormat();
		}
		else
		{
			FindFirstFormat();
		}
	}

	// TODO[MP]: We currently do not have runtime BC7 SRGB compression
	if (Format == PF_BC7)
	{
		bIsSRGB = false;
	}

	// Report back the chosen format
	InParent->RuntimePixelFormat = Format;
}

void FVirtualTextureCollectionResource::ComputeLayout(FTextureCollectionProducerData& Data)
{
	TArray<FTextureCollectionPackEntry> Entries;

	for (int32 i = 0; i < Textures.Num(); i++)
	{
		FTextureResource* Resource = Textures[i] ? Textures[i]->GetResource() : nullptr;
		if (!Resource)
		{
			continue;
		}

		uint32 WidthInBlocks = FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(Resource->GetSizeX(), static_cast<uint32>(BuildSettings.TileSize)));
		uint32 HeightInBlocks = FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(Resource->GetSizeY(), static_cast<uint32>(BuildSettings.TileSize)));
		
		uint32 SquareSize = FMath::Max(WidthInBlocks, HeightInBlocks);
		Entries.Add({ i, SquareSize });
	}

	// Descending allocation sizes, better packing
	Entries.Sort([](const FTextureCollectionPackEntry& A, const FTextureCollectionPackEntry& B)
	{
		return A.SquareSize > B.SquareSize;
	});

	// Actually pack
	FTextureCollectionPackResult PackResult = PackTextureCollectionAtlas(Entries, Textures.Num());
	Data.BlockCoordinates = MoveTemp(PackResult.Coordinates);
	Data.AllocationSizes = MoveTemp(PackResult.AllocationSizes);
	Data.BlockCount = PackResult.BlockCount;
}

static void OnTextureCollectionVirtualTextureDestroyed(const FVirtualTextureProducerHandle& InHandle, void* Baton)
{
	FTextureCollectionResource* Self = static_cast<FTextureCollectionResource*>(Baton);

	// Reinitialize contents from the game thread objects
	ENQUEUE_RENDER_COMMAND(Update)([Self](FRHICommandListImmediate& RHICmdList)
	{
		Self->InitRHI(RHICmdList);
	});
}

void FVirtualTextureCollectionResource::CreateIndexTable(FTextureCollectionProducerData& Data)
{
	Data.BlockVirtualPageTable.Init(INDEX_NONE, Data.BlockCount.Y * Data.BlockCount.X);

	// Create a block to texture index table
	for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); TextureIndex++)
	{
		const uint32 AllocationSize = Data.AllocationSizes[TextureIndex];
		if (AllocationSize == 0)
		{
			continue;
		}

		const FUintVector2 Offset = Data.BlockCoordinates[TextureIndex];

		// Write all block indices
		for (uint32 Row = 0; Row < AllocationSize; Row++)
		{
			const uint32 RowOffset = (Offset.Y + Row) * Data.BlockCount.X + Offset.X;
			
			for (uint32 Column = 0; Column < AllocationSize; Column++)
			{
				checkf(Data.BlockVirtualPageTable[RowOffset + Column] == INDEX_NONE, TEXT("Overlapping host index page table"));
				Data.BlockVirtualPageTable[RowOffset + Column] = TextureIndex;
			}
		}
	}
}

void FVirtualTextureCollectionResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	PageTable          = nullptr;
	PhysicalTextureSRV = nullptr;

	FTextureCollectionProducerData ProducerData;
	ProducerData.Textures.SetNumZeroed(Textures.Num());

	// Compute the general texture atlas layout
	ComputeLayout(ProducerData);

	// Create the host side index table
	CreateIndexTable(ProducerData);

	// The number of mips required
	uint32 MaxVirtualMipCount = 0;

	VirtualUniforms.SetNumZeroed(Textures.Num());
	
	for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); TextureIndex++)
	{
		FTextureResource* Resource = Textures[TextureIndex] ? Textures[TextureIndex]->GetResource() : nullptr;
		if (!Resource)
		{
			continue;
		}
		
		FTextureEntry TextureEntry;
		if (FVirtualTexture2DResource* VirtualResource = Resource->GetVirtualTexture2DResource())
		{
			if (VirtualResource->GetNumLayers() != 1)
			{
				FormatCollectionError(TEXT("Multiple physical layers are not supported"), TextureIndex);
				continue;
			}

			TextureEntry.Format = VirtualResource->GetFormat(0);
			if (TextureEntry.Format != Format)
			{
				if (!bAllowFormatConversion)
				{
					FormatCollectionError(TEXT("Mismatched format to collection"), TextureIndex);
					continue;
				}
				
				TextureEntry.bRequiresAdapter = true;
			}

			// Keep the producer handle for later queries
			TextureEntry.VirtualProducerHandle = VirtualResource->GetProducerHandle();

			// To block count
			TextureEntry.BlockCount.X = FMath::DivideAndRoundUp(VirtualResource->GetSizeX(), static_cast<uint32>(BuildSettings.TileSize)); 
			TextureEntry.BlockCount.Y = FMath::DivideAndRoundUp(VirtualResource->GetSizeY(), static_cast<uint32>(BuildSettings.TileSize));

			// Register destruction events, recreates from the new resources
			GetRendererModule().AddVirtualTextureProducerDestroyedCallback(
				VirtualResource->GetProducerHandle(),
				&OnTextureCollectionVirtualTextureDestroyed,
				this
			);
		}
		else if (FTexture2DResource* PhysicalResource = Resource->GetTexture2DResource())
		{
			if (!bAllowFormatConversion)
			{
				FormatCollectionError(TEXT("Non-virtual entries requires format conversion"), TextureIndex);
				continue;
			}

			// Physical textures always require adapters
			TextureEntry.bRequiresAdapter = true;
			TextureEntry.PhysicalTexture  = Textures[TextureIndex];
			TextureEntry.Format           = PhysicalResource->GetPixelFormat();
			
			// To block count
			TextureEntry.BlockCount.X = FMath::DivideAndRoundUp(PhysicalResource->GetSizeX(), static_cast<uint32>(BuildSettings.TileSize));
			TextureEntry.BlockCount.Y = FMath::DivideAndRoundUp(PhysicalResource->GetSizeY(), static_cast<uint32>(BuildSettings.TileSize));
		}
		else
		{
			FormatCollectionError(TEXT("Invalid resource"), TextureIndex);
			continue;
		}

		// Set per-entry MaxLevel from the square allocation size
		const uint32 AllocSize = ProducerData.AllocationSizes[TextureIndex];
		TextureEntry.MaxLevel = static_cast<uint8>(AllocSize > 0 ? FMath::CeilLogTwo(AllocSize) : 0);
		MaxVirtualMipCount = FMath::Max<uint32>(MaxVirtualMipCount, TextureEntry.MaxLevel);

		// Keep only the producer handle around
		ProducerData.Textures[TextureIndex] = TextureEntry;

		const uint32 BlockX = ProducerData.BlockCoordinates[TextureIndex].X;
		const uint32 BlockY = ProducerData.BlockCoordinates[TextureIndex].Y;
		checkf(BlockX <= 0xFFFF, TEXT("BlockX (%u) exceeds 16-bit limit for texture [%d]"), BlockX, TextureIndex);
		checkf(BlockY <= 0xFFFF, TEXT("BlockY (%u) exceeds 16-bit limit for texture [%d]"), BlockY, TextureIndex);
		checkf(TextureEntry.BlockCount.X <= 0xFFF, TEXT("BlockWidth (%u) exceeds 12-bit limit for texture [%d]"), TextureEntry.BlockCount.X, TextureIndex);
		checkf(TextureEntry.BlockCount.Y <= 0xFFF, TEXT("BlockHeight (%u) exceeds 12-bit limit for texture [%d]"), TextureEntry.BlockCount.Y, TextureIndex);
		checkf(TextureEntry.MaxLevel <= 0xFF, TEXT("MaxLevel (%u) exceeds 8-bit limit for texture [%d]"), TextureEntry.MaxLevel, TextureIndex);
		
		// Initialize uniforms
		// .x = BlockX (16) | BlockY (16)
		// .y = BlockWidth (12) | BlockHeight (12) | MaxLevel (8)
		// Real block dimensions for correct UV/SizeInPages, MaxLevel from square allocation
		UE::HLSL::FIndirectVirtualTextureEntry& VirtualUniform = VirtualUniforms[TextureIndex];
		VirtualUniform.PackedCoordinateAndSize.X = ProducerData.BlockCoordinates[TextureIndex].X;
		VirtualUniform.PackedCoordinateAndSize.X |= ProducerData.BlockCoordinates[TextureIndex].Y << 16;
		VirtualUniform.PackedCoordinateAndSize.Y = (TextureEntry.BlockCount.X & 0xFFF);
		VirtualUniform.PackedCoordinateAndSize.Y |= (TextureEntry.BlockCount.Y & 0xFFF) << 12;
		VirtualUniform.PackedCoordinateAndSize.Y |= static_cast<uint32>(TextureEntry.MaxLevel) << 24;
	}

	// Producer description, standard virtual texture with atlassed blocks
	FVTProducerDescription ProducerDesc;
	ProducerDesc.Name               = TEXT("TextureCollectionVirtualRedirector");
	ProducerDesc.FullNameHash       = GetTypeHash(ProducerDesc.Name);
	ProducerDesc.bContinuousUpdate  = false;
	ProducerDesc.Dimensions         = 2;
	ProducerDesc.TileSize           = BuildSettings.TileSize;
	ProducerDesc.TileBorderSize     = BuildSettings.TileBorderSize;
	ProducerDesc.BlockWidthInTiles  = ProducerData.BlockCount.X;
	ProducerDesc.BlockHeightInTiles = ProducerData.BlockCount.Y;
	ProducerDesc.DepthInTiles       = 1u;
	ProducerDesc.MaxLevel           = MaxVirtualMipCount;
	ProducerDesc.NumTextureLayers   = 1;
	ProducerDesc.NumPhysicalGroups  = 1;
	ProducerDesc.Priority           = EVTProducerPriority::Normal;
	ProducerDesc.LayerFormat[0]     = Format;
	ProducerDesc.bIsLayerSRGB[0]    = bIsSRGB;

	// Register producer on page feedback
	FTextureCollectionVirtualRedirector* Producer = new FTextureCollectionVirtualRedirector(MoveTemp(ProducerData), ProducerDesc);
	ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(RHICmdList, ProducerDesc, Producer);

	// Underlying VT, standard
	FAllocatedVTDescription VTDesc;
	VTDesc.Dimensions            = 2;
	VTDesc.NumTextureLayers      = 1;
	VTDesc.TileSize              = BuildSettings.TileSize;
	VTDesc.TileBorderSize        = BuildSettings.TileBorderSize;
	VTDesc.bShareDuplicateLayers = false;
	VTDesc.ProducerHandle[0]     = ProducerHandle;
	VTDesc.ProducerLayerIndex[0] = 0;
	
	AllocatedVT        = GetRendererModule().AllocateVirtualTexture(VTDesc);
	PhysicalTextureSRV = AllocatedVT->GetPhysicalTextureSRV(0, bIsSRGB);
	PageTable          = AllocatedVT->GetPageTableTexture(0);

	// Register destruction events
	GetRendererModule().AddVirtualTextureProducerDestroyedCallback(
		ProducerHandle,
		&OnTextureCollectionVirtualTextureDestroyed,
		this
	);

	// Safe fallback
	if (!PageTable || !PhysicalTextureSRV)
	{
		PageTable          = GBlackUintTexture->TextureRHI;
		PhysicalTextureSRV = GBlackTextureWithSRV->ShaderResourceViewRHI;
	}

	// Allow empty collections, zero'd page table uniforms will early out
	if (VirtualUniforms.IsEmpty())
	{
		VirtualUniforms.AddZeroed();
	}

	// Create as texel buffer
	VirtualCollectionRHI = RHICmdList.CreateBuffer(
		FRHIBufferCreateDesc::Create(
			TEXT("TextureCollectionVirtualUniforms"),
			VirtualUniforms.NumBytes(),
			sizeof(FUintVector2),
			EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource
		)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetInitActionResourceArray(&VirtualUniforms)
	);

	// Buffer SRV
	VirtualCollectionRHISRV = RHICmdList.CreateShaderResourceView(VirtualCollectionRHI, FRHIViewDesc::CreateBufferSRV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32G32_UINT)
	);
}

void FVirtualTextureCollectionResource::ReleaseRHI()
{
	if (AllocatedVT)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		GetRendererModule().DestroyVirtualTexture(AllocatedVT);
		GetRendererModule().ReleaseVirtualTextureProducer(ProducerHandle);
		AllocatedVT = nullptr;
	}

	FTextureCollectionResource::ReleaseRHI();
}

static int GetBlockCompressionFormatPriority(EPixelFormat Format)
{
	// Ordered first by component counts, second to source bit-width
	static EPixelFormat Ascending[] = {
		PF_BC4,  // R8
		PF_BC5,  // R8G8
		PF_BC6H, // R16G16B16
		PF_DXT1, // R5G6B5A1
		PF_DXT3, // R5G6B5A4
		PF_DXT5, // R5G6B5A8
		PF_BC7   // R8G8B8A8 (* variable)
	};

	for (int32 i = 0; i < UE_ARRAY_COUNT(Ascending); i++)
	{
		if (Ascending[i] == Format)
		{
			return i;
		}
	}
	
	checkf(false, TEXT("Unexpected block compressed format"));
	return 0;
}

static EPixelFormat GetBlockCompressionSourceFormat(EPixelFormat Format)
{
	// TODO[MP]: This isn't entirely accurate, as some formats are variable bit-width
	switch (Format) {
		default:
			checkf(false, TEXT("Unexpected block compressed format"));
			return PF_R8G8B8A8;
		case PF_BC4:
			return PF_R8;
		case PF_BC5:
			return PF_R8G8;
		case PF_BC6H:
			return PF_FloatRGB;
		case PF_DXT1:
			return PF_B5G5R5A1_UNORM;
		case PF_DXT3:
			return PF_R8G8B8A8;
		case PF_DXT5:
			return PF_R8G8B8A8;
		case PF_BC7: 
			return PF_R8G8B8A8;
	};
}

static EPixelFormat FindConservativeSourceFormat(int32 SourceComponentCount, int32 SourceByteWidth)
{
	EPixelFormat     CandidateFormat = PF_A32B32G32R32F;
	FPixelFormatInfo CandidateInfo   = GPixelFormats[CandidateFormat];

	for (const FPixelFormatInfo& PixelFormat : GPixelFormats)
	{
		if (!PixelFormat.Supported || IsBlockCompressedFormat(PixelFormat.UnrealFormat))
		{
			continue;
		}

		// Check if this format can accomodate at all
		if (PixelFormat.NumComponents < SourceComponentCount || PixelFormat.BlockBytes < SourceByteWidth)
		{
			continue;
		}
		
		// Check if this format has either a redued component count or byte-width
		if (PixelFormat.NumComponents < CandidateInfo.NumComponents || PixelFormat.BlockBytes < CandidateInfo.BlockBytes)
		{
			CandidateFormat = PixelFormat.UnrealFormat;
			CandidateInfo   = GPixelFormats[CandidateFormat];
		}
	}

	return CandidateFormat;
}

void FVirtualTextureCollectionResource::FindFirstFormat()
{
	// Find the first valid resource
	for (UTexture* Texture : Textures)
	{
		FTextureResource* Resource = Texture ? Texture->GetResource() : nullptr;
		if (!Resource)
		{
			continue;
		}

		EPixelFormat TextureFormat = GetResourcePixelFormat(Resource);
		if (TextureFormat != PF_Unknown)
		{
			Format  = TextureFormat;
			bIsSRGB = Resource->bSRGB;
			return;
		}
	}

	// No valid resource, dummy format
	Format  = PF_R8G8B8A8;
	bIsSRGB = true;
}

void FVirtualTextureCollectionResource::FindConservativeFormat()
{
	EPixelFormat CandidateFormat = PF_Unknown;
	bool         bAnyFormatSRGB  = false;

	int32 SourceComponentCount  = 0;
	int32 SourceByteWidth       = 0;

	for (UTexture* Texture : Textures)
	{
		FTextureResource* Resource = Texture ? Texture->GetResource() : nullptr;
		if (!Resource)
		{
			continue;
		}

		// If any format is SRGB, so is this
		bAnyFormatSRGB |= Resource->bSRGB;

		EPixelFormat TextureFormat = GetResourcePixelFormat(Resource);
		if (TextureFormat == PF_Unknown || TextureFormat == CandidateFormat)
		{
			continue;
		}

		// If first format, just accept it as is
		if (CandidateFormat == PF_Unknown)
		{
			CandidateFormat = TextureFormat;
			continue;
		}

		const bool bIsCandidateBC = IsBlockCompressedFormat(CandidateFormat);
		const bool bIsTextureBC   = IsBlockCompressedFormat(TextureFormat);

		// We keep the bit-width as is, so opt for decompressing the formats
		if (!bIsCandidateBC || !bIsTextureBC)
		{
			// "Decompress" if needed
			if (bIsCandidateBC)
			{
				CandidateFormat = GetBlockCompressionSourceFormat(CandidateFormat);
			}
			
			// "Decompress" if needed
			if (bIsTextureBC)
			{
				TextureFormat = GetBlockCompressionSourceFormat(TextureFormat);
			}

			const FPixelFormatInfo& CandidateInfo = GPixelFormats[TextureFormat];
			const FPixelFormatInfo& TextureInfo   = GPixelFormats[TextureFormat];

			// Search later on the channel count and byte-width
			SourceComponentCount  = FMath::Max3(SourceComponentCount, TextureInfo.NumComponents, CandidateInfo.NumComponents);
			SourceByteWidth       = FMath::Max3(SourceByteWidth,      TextureInfo.BlockBytes,    CandidateInfo.BlockBytes);
		}
		else
		{
			// Both are block compressed, find the best fit
			int Priority = GetBlockCompressionFormatPriority(TextureFormat);
			if (Priority > GetBlockCompressionFormatPriority(CandidateFormat))
			{
				CandidateFormat = TextureFormat;
			}
		}
	}

	// If this is a source format, find a conservative pixel format
	if (SourceComponentCount)
	{
		Format  = FindConservativeSourceFormat(SourceComponentCount, SourceByteWidth);
		bIsSRGB = bAnyFormatSRGB;
		return;
	}

	// No relevant resources, assign a dummy format
	if (CandidateFormat == PF_Unknown)
	{
		Format  = PF_R8G8B8A8;
		bIsSRGB = true;
		return;
	}

	Format  = CandidateFormat;
	bIsSRGB = bAnyFormatSRGB;
}

UE::HLSL::FIndirectVirtualTextureUniform FVirtualTextureCollectionResource::GetVirtualPackedUniform() const
{
	UE::HLSL::FIndirectVirtualTextureUniform Out{};
		
	if (!Textures.IsEmpty())
	{
		Out.UniformCountSub1 = Textures.Num() - 1;
	}

	AllocatedVT->GetPackedPageTableUniform(Out.PackedPageTableUniform);
	AllocatedVT->GetPackedUniform(&Out.PackedUniform, 0);
	
	return Out;
}

void FVirtualTextureCollectionResource::FormatCollectionError(const TCHAR* Reason, uint32 TextureIndex)
{
	UE_LOGF(
		LogVirtualTextureCollection, Error,
		"Texture collection '%ls' received texture [%i] '%ls' - %ls",
		*CollectionName.ToString(), TextureIndex, *Textures[TextureIndex]->GetFName().ToString(), Reason
	);
}

#if WITH_EDITOR
void UVirtualTextureCollection::ValidateVirtualCollection()
{
	ETextureSourceFormat   Format         = TSF_Invalid;
	FTextureFormatSettings FormatSettings = {};
	
	for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); TextureIndex++)
	{
		UTexture* Texture = Textures[TextureIndex];
		if (!Texture)
		{
			// Null/default textures are allowed
			continue;
		}

		UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
		if (!Texture2D)
		{
			FormatCollectionError(TEXT("Virtual collections only support 2d textures"), TextureIndex);
			continue;
		}

		// Format conversion can handle any kind of format differences
		if (bAllowFormatConversion)
		{
			continue;
		}

		// Actual formats are only known after building the textures
		// In case it's not built, let's just validate the source format settings
		FTextureFormatSettings TextureFormat;
		Texture->GetLayerFormatSettings(0, TextureFormat);
			
		if (Format == TSF_Invalid)
		{
			Format         = Texture->Source.GetFormat(0);
			FormatSettings = TextureFormat;
		}
		else if (Format != Texture->Source.GetFormat(0) || FormatSettings != TextureFormat)
		{
			FormatCollectionError(TEXT("Mismatched format settings, all textures in a (virtual) collection must share the same format settings"), TextureIndex);
		}
	}
}

void UVirtualTextureCollection::FormatCollectionError(const TCHAR* Reason, uint32 TextureIndex)
{
	UE_LOGF(
		LogVirtualTextureCollection, Error,
		"Texture collection '%ls' received texture [%i] '%ls' - %ls",
		*GetFName().ToString(), TextureIndex, *Textures[TextureIndex]->GetFName().ToString(), Reason
	);
}
#endif //  WITH_EDITOR

#if WITH_EDITOR
void UVirtualTextureCollection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UTextureCollection::PostEditChangeProperty(PropertyChangedEvent);
	ValidateVirtualCollection();
}
#endif // WITH_EDITOR

void UVirtualTextureCollection::BeginDestroy()
{
	ReleaseResource();
	Super::BeginDestroy();
}

FTextureCollectionResource* UVirtualTextureCollection::CreateResource()
{
	// Wait for the virtual resources to finish compilation
#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation(Textures);
#endif // WITH_EDITOR

	return new FVirtualTextureCollectionResource(this);
}

static FAutoConsoleCommand GRecreateVirtualTextureCollections(
	TEXT("r.VTC.Recreate"),
	TEXT("Recreate all virtual texture collections, refreshing resources and notifying materials"),
	FConsoleCommandDelegate::CreateLambda([]
	{
		UE_LOGF(LogVirtualTextureCollection, Display, "Recreating all virtual texture collections");
		
		for (TObjectIterator<UVirtualTextureCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			It->UpdateResource();
		}
	})
);

static FAutoConsoleCommand GDumpVirtualTextureCollections(
	TEXT("r.VTC.DumpToDisk"),
	TEXT("Dump all virtual texture collection atlas layouts to PNG files under Saved/VTC/"),
	FConsoleCommandDelegate::CreateLambda([] 
	{
		const int32 PixelsPerBlock = 16;
		const FColor EmptyColor(32, 32, 32);
		const FColor GridColor(64, 64, 64);
			
		for (TObjectIterator<UVirtualTextureCollection> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UVirtualTextureCollection* Collection = *It;
			if (!Collection->GetResource())
			{
				continue;
			}

			// Use default VT settings
			FVirtualTextureBuildSettings BuildSettings;
			BuildSettings.Init();

			// Collect entries
			TArray<FTextureCollectionPackEntry> Entries;
			for (int32 TextureIndex = 0; TextureIndex < Collection->Textures.Num(); TextureIndex++)
			{
				FTextureResource* TexResource = Collection->Textures[TextureIndex] ? Collection->Textures[TextureIndex]->GetResource() : nullptr;
				if (!TexResource)
				{
					continue;
				}

				uint32 WidthInBlocks  = FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(TexResource->GetSizeX(), static_cast<uint32>(BuildSettings.TileSize)));
				uint32 HeightInBlocks = FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(TexResource->GetSizeY(), static_cast<uint32>(BuildSettings.TileSize)));
				
				uint32 SquareSize     = FMath::Max(WidthInBlocks, HeightInBlocks);
				Entries.Add({ TextureIndex, SquareSize });
			}

			// Same sorting metric
			Entries.Sort([](const FTextureCollectionPackEntry& Lhs, const FTextureCollectionPackEntry& Rhs)
			{
				return Lhs.SquareSize > Rhs.SquareSize;
			});

			// Re-pack it
			FTextureCollectionPackResult PackResult = PackTextureCollectionAtlas(Entries, Collection->Textures.Num());

			const int32 NumEntries = Entries.Num();
			
			// Destination path
			const FString BaseDir = FPaths::ProjectSavedDir() / TEXT("VTC") / Collection->GetFName().ToString();
			const uint32 MaxLevel = FMath::CeilLogTwo(FMath::Max(PackResult.BlockCount.X, PackResult.BlockCount.Y));

			// Write out a separate image per level
			for (uint32 Level = 0; Level <= MaxLevel; Level++)
			{
				// Level block dimensions
				const uint32 LevelWidth = FMath::Max(1u, PackResult.BlockCount.X >> Level);
				const uint32 LevelHeight = FMath::Max(1u, PackResult.BlockCount.Y >> Level);

				// Level image dimensions
				const int32 ImageWidth = LevelWidth * PixelsPerBlock;
				const int32 ImageHeight = LevelHeight * PixelsPerBlock;

				FImage Image(ImageWidth, ImageHeight, ERawImageFormat::BGRA8, EGammaSpace::sRGB);
				TArrayView64<FColor> Pixels = Image.AsBGRA8();

				// Default to empty
				for (int64 PixelIndex = 0; PixelIndex < Pixels.Num(); PixelIndex++)
				{
					Pixels[PixelIndex] = EmptyColor;
				}

				// Write all entries
				for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
				{
					const FTextureCollectionPackEntry& Entry = Entries[EntryIndex];
					
					// Allocated coordinate
					const FUintVector2& EntryCoord = PackResult.Coordinates[Entry.TextureIndex];

					// Coordinates for this level
					const uint32 BlockX = EntryCoord.X >> Level;
					const uint32 BlockY = EntryCoord.Y >> Level;
					const uint32 BlockSpan = FMath::Max(1u, Entry.SquareSize >> Level);

					// Unique color
					const uint8 Hue = NumEntries > 1 ? static_cast<uint8>(EntryIndex * 255 / (NumEntries - 1)) : 0;
					FColor Color = FLinearColor::MakeFromHSV8(Hue, 200, 220).ToFColor(true);
					
					// If beyond the level, it's a useless block, just dim it
					if (Level > FMath::CeilLogTwo(Entry.SquareSize))
					{
						Color.R /= 3;
						Color.G /= 3;
						Color.B /= 3;
					}

					// Rasterize pixels
					for (uint32 BlockRow = 0; BlockRow < BlockSpan; BlockRow++)
					{
						for (uint32 BlockColumn = 0; BlockColumn < BlockSpan; BlockColumn++)
						{
							const int32 PixelStartX = (BlockX + BlockColumn) * PixelsPerBlock;
							const int32 PixelStartY = (BlockY + BlockRow) * PixelsPerBlock;

							for (int32 Y = 0; Y < PixelsPerBlock; Y++)
							{
								for (int32 X = 0; X < PixelsPerBlock; X++)
								{
									const int32 PixelX = PixelStartX + X;
									const int32 PixelY = PixelStartY + Y;

									if (PixelX < ImageWidth && PixelY < ImageHeight)
									{
										const bool bBorder = (X == 0 || Y == 0);
										Pixels[PixelY * ImageWidth + PixelX] = bBorder ? GridColor : Color;
									}
								}
							}
						}
					}
				}

				// Write it out
				const FString Filename = BaseDir / FString::Printf(TEXT("Mip%u_%ux%u.png"), Level, LevelWidth, LevelHeight);
				FImageUtils::SaveImageByExtension(*Filename, Image);
				UE_LOGF(LogVirtualTextureCollection, Display, "Dumped: %ls", *Filename);
			}

			// Info
			UE_LOGF(
				LogVirtualTextureCollection, Display, "Dumped '%ls': %u textures, atlas %ux%u, %u mip levels to %ls",
				*Collection->GetFName().ToString(), Collection->Textures.Num(), 
				PackResult.BlockCount.X, PackResult.BlockCount.Y, MaxLevel + 1, 
				*BaseDir
			);
		}
	})
);

#undef LOCTEXT_NAMESPACE
