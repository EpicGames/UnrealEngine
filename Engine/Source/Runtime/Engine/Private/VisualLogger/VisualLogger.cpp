// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLogger.h"
#include "AI/NavDataGenerator.h" // IWYU pragma: keep
#include "AI/NavigationSystemBase.h"
#include "Containers/StringFwd.h"
#include "EngineStats.h"
#include "Framework/Docking/TabManager.h"
#include "Logging/LogCategory.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeRWLock.h"
#include "Modules/ModuleManager.h"
#include "UnrealEngine.h"
#include "VisualLogger/VisualLoggerBinaryFileDevice.h"
#include "VisualLogger/VisualLoggerCustomVersion.h"
#include "VisualLogger/VisualLoggerDebugSnapshotInterface.h"
#include "VisualLogger/VisualLoggerSettings.h"
#include "VisualLogger/VisualLoggerTraceDevice.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Misc/App.h"
#else
#include "Engine/Engine.h"
#include "Serialization/CustomVersion.h"
#endif

DEFINE_LOG_CATEGORY(LogVisual);

#if UE_DEBUG_RECORDING_ENABLED

bool FVisualLogger::bUseVerbosityFilterWhenRecording = false;

namespace UE::VisualLogger
{
FAutoConsoleCommandWithOutputDevice CmdActivateVerbosityFilterWhenRecording(
	TEXT("vislog.ActivateVerbosityFilterWhenRecording"),
	TEXT("Active log category verbosity will be used to filter out entries to record.\n"),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& OutputDevice)
		{
			GetMutableDefault<UVisualLoggerSettings>()->SetUseVerbosityFilterWhenRecording(true);
			check(FVisualLogger::bUseVerbosityFilterWhenRecording);

			OutputDevice.Log(ELogVerbosity::Log, TEXT("LogCategory verbosity based filtering is now 'ACTIVE'"));
		}),
	ECVF_Default);

FAutoConsoleCommandWithOutputDevice CmdDeactivateVerbosityFilterWhenRecording(
	TEXT("vislog.DeactivateVerbosityFilterWhenRecording"),
	TEXT("Log category verbosity is deactivated and everything is recorded.\n"),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& OutputDevice)
		{
			GetMutableDefault<UVisualLoggerSettings>()->SetUseVerbosityFilterWhenRecording(false);
			check(!FVisualLogger::bUseVerbosityFilterWhenRecording);

			OutputDevice.Log(ELogVerbosity::Log, TEXT("LogCategory verbosity based filtering is now 'INACTIVE'"));
		}),
	ECVF_Default);
}

namespace UE::VisLog::Private
{
	/** Denotes the consecutive run number, giving us the ability to differentiate UObjectName_0 in Run 1 from UObjectName_0 in Run 2 */
	static int32 UniqueRunNumber = 0;

	/** Cached prefix string when using bForceUniqueLogNames (denotes UniqueRunNumber) */
	static FString UniqueLogPrefix;

#if WITH_EDITOR
	namespace EditorOnly
	{
		/** When in Editor, use this TimeStamp as a Base time so our values don't grow too large (and are canonically around 0.0). */
		static std::atomic<double> EditorBaseTimeStamp = 0.0;
	}
#endif // WITH_EDITOR
}

DEFINE_STAT(STAT_VisualLog);

#if UE_DEBUG_RECORDING_USING_VLOG
template <typename SerializeFuncType>
static void CollapsedLogfV(const SerializeFuncType& SerializeFunc, const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, va_list Args)
{
	SCOPE_CYCLE_COUNTER(STAT_VisualLog);

	FVisualLogEntry* CurrentEntry = nullptr;
	if (FVisualLogger::CheckVisualLogInputInternal(Object, CategoryName, Verbosity, /*OutWorld=*/ nullptr, &CurrentEntry))
	{
		SerializeFunc(CurrentEntry, TStringBuilder<512>().AppendV(Fmt, Args).ToString());
	}
}
#else
template <typename SerializeFuncType>
static void CollapsedLogfV(const SerializeFuncType& SerializeFunc, const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, va_list Args)
{
	SCOPE_CYCLE_COUNTER(STAT_VisualLog);

	FVisualLogEntry NewEntry;
	FVisualLogger::FObjectNames Names;
	if (FVisualLogger::CreateEntryInternal(Object, NewEntry, Names))
	{
		SerializeFunc(&NewEntry, TStringBuilder<512>().AppendV(Fmt, Args).ToString());

		FVisualLoggerTraceDevice::Get().Serialize(Object, Names.Name, Names.DisplayName, Names.ClassName, NewEntry);
	}
}
#endif // UE_DEBUG_RECORDING_USING_VLOG

#define COLLAPSED_LOGF(SerializeSnippet) \
	va_list Args; \
	va_start(Args, Fmt); \
	auto SerializeFunc = [&](FVisualLogEntry* CurrentEntry, const TCHAR* Buffer) { SerializeSnippet; }; \
	CollapsedLogfV(SerializeFunc, Object, CategoryName, Verbosity, Fmt, Args); \
	va_end(Args);

void FVisualLogger::CategorizedLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddText(Buffer, CategoryName, Verbosity);
	);
}

void FVisualLogger::SegmentLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Thickness, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddSegment(Start, End, CategoryName, Verbosity, Color, Buffer, Thickness);
	);
}

void FVisualLogger::ArrowLineLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const FColor& Color, const uint16 Mag, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddArrow(Start, End, CategoryName, Verbosity, Color, Buffer, Mag);
	);
}

void FVisualLogger::DiscLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, const FVector& UpAxis, const float Radius, const FColor& Color, const uint16 Thickness, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddCircle(Center, UpAxis, Radius, CategoryName, Verbosity, Color, Buffer, Thickness, bWireframe);
	);
}

void FVisualLogger::LocationLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Location, uint16 Thickness, const FColor& Color, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddLocation(Location, CategoryName, Verbosity, Color, Buffer, Thickness);
	);
}

void FVisualLogger::SphereLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Center, float Radius, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddSphere(Center, Radius, CategoryName, Verbosity, Color, Buffer, bWireframe);
	);
}

void FVisualLogger::BoxLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FBox& Box, const FMatrix& Matrix, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddBox(Box, Matrix, CategoryName, Verbosity, Color, Buffer, /*Thickness = */0, bWireframe);
	);
}

void FVisualLogger::ConeLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Origin, const FVector& Direction, const float Length, const float Angle, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddCone(Origin, Direction, Length, Angle, Angle, CategoryName, Verbosity, Color, Buffer, /*Thickness = */0, bWireframe);
	);
}

void FVisualLogger::CylinderLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Start, const FVector& End, const float Radius, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddCylinder(Start, End, Radius, CategoryName, Verbosity, Color, Buffer, /*Thickness = */0, bWireframe);
	);
}

void FVisualLogger::CapsuleLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Base, float HalfHeight, float Radius, const FQuat& Rotation, const FColor& Color, bool bWireframe, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddCapsule(Base, HalfHeight, Radius, Rotation, CategoryName, Verbosity, Color, Buffer, bWireframe);
	);
}

void FVisualLogger::PulledConvexLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& ConvexPoints, float MinZ, float MaxZ, const FColor& Color, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddPulledConvex(ConvexPoints, MinZ, MaxZ, CategoryName, Verbosity, Color, Buffer);
	);
}

void FVisualLogger::MeshLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Vertices, const TArray<int32>& Indices, const FColor& Color, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddMesh(Vertices, Indices, CategoryName, Verbosity, Color, Buffer);
	);
}

void FVisualLogger::ConvexLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const TArray<FVector>& Points, const FColor& Color, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddConvexElement(Points, CategoryName, Verbosity, Color, Buffer);
	);
}

void FVisualLogger::HistogramDataLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, FName GraphName, FName DataName, const FVector2D& Data, const FColor& Color, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddHistogramData(Data, CategoryName, Verbosity, GraphName, DataName);
	);
}

void FVisualLogger::CoordinateSystemLogfImpl(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type Verbosity, const FVector& AxisLoc, const FRotator& AxisRot, const float Scale, const FColor& Color, const uint16 Thickness, const TCHAR* Fmt, ...)
{
	COLLAPSED_LOGF(
		CurrentEntry->AddCoordinateSystem(AxisLoc, AxisRot, Scale, CategoryName, Verbosity, Color, Buffer, Thickness);
	);
}

#undef COLLAPSED_LOGF


namespace
{
	UWorld* GetWorldForVisualLogger(const UObject* Object)
	{
		UWorld* World = GEngine ? GEngine->GetWorldFromContextObject(Object, EGetWorldErrorMode::ReturnNull) : nullptr;
#if WITH_EDITOR
		UEditorEngine *EEngine = Cast<UEditorEngine>(GEngine);
		if (GIsEditor && EEngine != nullptr && World == nullptr)
		{
			// lets use PlayWorld during PIE/Simulate and regular world from editor otherwise, to draw debug information
			World = EEngine->PlayWorld != nullptr 
				? ToRawPtr(EEngine->PlayWorld) 
				: (EEngine->GetWorldContexts().Num() > 0 ? EEngine->GetEditorWorldContext().World() : nullptr);
		}

#endif
		if (!GIsEditor && World == nullptr && GEngine)
		{
			World = GEngine->GetWorld();
		}

		// In last resort we fall back on the global world (i.e., logs produced while world gets GCed)
		if (World == nullptr)
		{
			World = GWorld;
		}

		return World;
	}
}

int32 FVisualLogger::bIsRecording = false;
FVisualLogger::FNavigationDataDump FVisualLogger::NavigationDataDumpDelegate;

#if UE_DEBUG_RECORDING_USING_VLOG
TMap<const UWorld*, FVisualLogger::FOwnerToChildrenRedirectionMap> FVisualLogger::WorldToRedirectionMap;

bool FVisualLogger::CheckVisualLogInputInternal(const UObject* Object, const FName& CategoryName, ELogVerbosity::Type, UWorld** OutWorld, FVisualLogEntry** OutCurrentEntry)
{
	if (IsRecording() == false || !Object || !GEngine || GEngine->bDisableAILogging || Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	FVisualLogger& VisualLogger = Get();
	if (VisualLogger.IsBlockedForAllCategories() && VisualLogger.IsCategoryAllowed(CategoryName) == false)
	{
		return false;
	}

	if (OutWorld)
	{
		*OutWorld = GetWorldForVisualLogger(Object);
		if (!ensure(*OutWorld != nullptr))
		{
			return false;
		}
	}

	if (OutCurrentEntry)
	{
		*OutCurrentEntry = VisualLogger.GetEntryToWrite(Object, VisualLogger.GetTimeStampForObject(Object));
		if (*OutCurrentEntry == nullptr)
		{
			return false;
		}
	}

	return true;
}

#else // ^^^ UE_DEBUG_RECORDING_USING_VLOG ^^^ / vvv TRACE ONLY vvv

bool FVisualLogger::CreateEntryInternal(const UObject* Object, FVisualLogEntry& OutEntry, FVisualLogger::FObjectNames& OutNames)
{
	if (IsRecording() == false || !Object || !GEngine || GEngine->bDisableAILogging || Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	FVisualLogger& VisualLogger = Get();

	using namespace UE::VisLog::Private;
	FVisualLogEntry NewEntry;
	NewEntry.InitializeEntry(VisualLogger.GetTimeStampForObject(Object));

	FObjectNames* Names;
	{
		UE::TReadScopeLock Lock(VisualLogger.RedirectRWLock);
		Names = VisualLogger.ObjectToNamesMap.Find(Object);
	}

	if (Names == nullptr)
	{
		UE::TWriteScopeLock Lock(VisualLogger.RedirectRWLock);
		Names = VisualLogger.ObjectToNamesMap.Find(Object);
		if (Names == nullptr)
		{
			Names = &VisualLogger.ObjectToNamesMap.Add(Object, VisualLogger.CreateObjectNames(Object));
		}
	}

	OutEntry = MoveTemp(NewEntry);
	OutNames = *Names;

	return true;
}

#endif // UE_DEBUG_RECORDING_USING_VLOG

double FVisualLogger::GetTimeStampForObject(const UObject* Object) const
{
	// Licensees can write their own synchronized clock that can work across the network
	// such an implementation could use the exchanged ServerTime and the relative ClientTime offset.
	if (GetTimeStampFunc)
	{
		return GetTimeStampFunc(Object);
	}

#if WITH_EDITOR
	// When we're in the Editor, use a Global Engine TimeStamp so that we can synchronize between Client and Server instances.
	if (GIsEditor)
	{
		using namespace UE::VisLog::Private;

		double Expected = 0.0;
		EditorOnly::EditorBaseTimeStamp.compare_exchange_strong(Expected, FApp::GetCurrentTime());

		if (EditorOnly::EditorBaseTimeStamp > 0.0)
		{
			return FApp::GetCurrentTime() - EditorOnly::EditorBaseTimeStamp;
		}
	}
#endif // WITH_EDITOR

	// This will be the fallback mode in standalone.  We do not have a synchronized clock.
	const UWorld* WorldForTimeStamp = GetWorldForVisualLogger(Object);

	return WorldForTimeStamp ? WorldForTimeStamp->TimeSeconds : 0.0;
}

void FVisualLogger::SetGetTimeStampFunc(const TFunction<double(const UObject*)> Function)
{
	GetTimeStampFunc = Function;
}

#if UE_DEBUG_RECORDING_USING_VLOG
FVisualLogEntry* FVisualLogger::GetEntryToWrite(const UObject* LogOwner, const FLogCategoryBase& LogCategory)
{
	FVisualLogEntry* LogEntry;
	if (CheckVisualLogInputInternal(LogOwner, LogCategory.GetCategoryName(), ELogVerbosity::Log, /*OutWorld*/ nullptr, &LogEntry))
	{
		return LogEntry;
	}

	return nullptr;
}

void FVisualLogger::AddClassToAllowList(UClass& InClass)
{
	ClassAllowList.AddUnique(&InClass);
}

bool FVisualLogger::IsClassAllowed(const UClass& InClass) const
{
	for (const UClass* AllowedRoot : ClassAllowList)
	{
		if (InClass.IsChildOf(AllowedRoot))
		{
			return true;
		}
	}

	return false;
}

void FVisualLogger::AddObjectToAllowList(const UObject& InObject)
{
	UE::TWriteScopeLock Lock(EntryRWLock);
	bool bIsAlreadyInSet = false;
	ObjectAllowList.Add(&InObject, &bIsAlreadyInSet);

	if (!bIsAlreadyInSet)
	{
		{
			UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
			if (FVisualLogEntry* CurrentEntry = CurrentEntryPerObject.Find(&InObject))
			{
				CurrentEntry->SetPassedObjectAllowList(true);
			}
		}

		// We are not locking the thread entry map, we do not expect other threads adding new entries while editing allow list
		UE_MT_SCOPED_READ_ACCESS(ThreadLocalMapListAccessDetector);
		for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
		{
			checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
			UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
			if (FVisualLogEntry* ThreadCurrentEntry = ThreadCurrentEntryMap->Find(&InObject))
			{
				ThreadCurrentEntry->SetPassedObjectAllowList(true);
			}
		}
	}
}

void FVisualLogger::ClearObjectAllowList()
{
	UE::TWriteScopeLock Lock(EntryRWLock);
	for (FObjectKey It : ObjectAllowList)
	{
		{
			UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
			if (FVisualLogEntry* CurrentEntry = CurrentEntryPerObject.Find(It))
			{
				CurrentEntry->SetPassedObjectAllowList(false);
			}
		}

		// We are not locking the thread entry map, we do not expect other threads adding new entries while editing allow list
		UE_MT_SCOPED_READ_ACCESS(ThreadLocalMapListAccessDetector);
		for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
		{
			checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
			UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
			if (FVisualLogEntry* ThreadCurrentEntry = ThreadCurrentEntryMap->Find(It))
			{
				ThreadCurrentEntry->SetPassedObjectAllowList(false);
			}
		}
	}

	ObjectAllowList.Empty();
}

bool FVisualLogger::IsObjectAllowed(const UObject* InObject) const
{
	return ObjectAllowList.Contains(InObject);
}

FVisualLogEntry* FVisualLogger::GetLastEntryForObject(const UObject* Object)
{
	FVisualLogEntry* FoundEntry = nullptr;
	ExecuteOnLastEntryForObject(Object, [&FoundEntry](FVisualLogEntry& LogEntry)
	{
		FoundEntry = &LogEntry;
	});

	return FoundEntry;
}

bool FVisualLogger::ExecuteOnLastEntryForObject(const UObject* Object, TFunctionRef<void(FVisualLogEntry&)> Function)
{
	const UObject* LogOwner = nullptr;
	{
		UE::TReadScopeLock Lock(RedirectRWLock);
		LogOwner = FindRedirectionInternal(Object);
	}

	if (LogOwner != nullptr)
	{
		FVisualLoggerObjectEntryMap& ThreadCurrentEntryMap = GetThreadCurrentEntryMap();
		UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap.EntriesAccessDetector);
		if (FVisualLogEntry* LogEntry = ThreadCurrentEntryMap.Find(LogOwner))
		{
			Function(*LogEntry);
			return true;
		}
	}

	return false;
}

FVisualLogEntry* FVisualLogger::GetEntryToWrite(const UObject* Object, const double TimeStamp, ECreateIfNeeded ShouldCreate)
{
	const UObject* LogOwner = nullptr;
	{
		UE::TReadScopeLock Lock(RedirectRWLock);
		LogOwner = FindRedirectionInternal(Object);
	}
	if (LogOwner == nullptr)
	{
		return nullptr;
	}

	FVisualLoggerObjectEntryMap& ThreadCurrentEntryPerObject = GetThreadCurrentEntryMap();
	// Using the Write access since it might flush and reset an entry and/or initialize one.
	UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryPerObject.EntriesAccessDetector);
	FVisualLogEntry* ThreadCurrentEntry = ThreadCurrentEntryPerObject.Find(LogOwner);
	bool bInitializeEntry = false;
	if (ThreadCurrentEntry)
	{
		if (ThreadCurrentEntry->ShouldLog(ShouldCreate))
		{
			if (ThreadCurrentEntry->ShouldFlush(TimeStamp))
			{
				UE::TWriteScopeLock Lock(EntryRWLock);
				UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
				if (FVisualLogEntry* GlobalCurrentEntry = GetEntryToWriteInternal(LogOwner, ThreadCurrentEntry->TimeStamp, ShouldCreate))
				{
					ThreadCurrentEntry->MoveTo(*GlobalCurrentEntry);
				}
			}

			bInitializeEntry = !ThreadCurrentEntry->bIsInitialized;
		}
	}
	else
	{
		// Check to see if it exists in the global map, 
		// if so then force creation for the current thread
		if (ShouldCreate != ECreateIfNeeded::Create)
		{
			UE::TReadScopeLock Lock(EntryRWLock);
			UE_MT_SCOPED_READ_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
			if (CurrentEntryPerObject.Find(LogOwner))
			{
				ShouldCreate = ECreateIfNeeded::Create;
			}
		}

		if (ShouldCreate == ECreateIfNeeded::Create)
		{
			ThreadCurrentEntry = &ThreadCurrentEntryPerObject.Add(LogOwner);

			CalculateEntryAllowLogging(ThreadCurrentEntry, LogOwner, Object);

			bInitializeEntry = ThreadCurrentEntry->bIsAllowedToLog;
		}
	}

	if (bInitializeEntry)
	{
		checkf(ThreadCurrentEntry != nullptr, TEXT("bInitializeEntry can only be true when CurrentEntry is valid."));
		ThreadCurrentEntry->InitializeEntry(TimeStamp);
	}

	if (ThreadCurrentEntry != nullptr && ThreadCurrentEntry->bIsAllowedToLog)
	{
		bIsFlushRequired = true;
	}
	else
	{
		ThreadCurrentEntry = nullptr;
	}

	return ThreadCurrentEntry;
}

FVisualLogEntry* FVisualLogger::GetEntryToWriteInternal(const UObject* Object, const double TimeStamp, const ECreateIfNeeded ShouldCreate)
{
	using namespace UE::VisLog::Private;

	// No redirection needed, it should have been done at the time of the thread entry was computed
	const UObject* LogOwner = Object;
	if (LogOwner == nullptr)
	{
		return nullptr;
	}

	// Entry can be created or reused (after being flushed) and will need to be initialized
	bool bInitializeEntry = false;

	// We were previously using UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector) here
	// but Transactionally safe detectors are not reentrant so it is now the responsibility of the caller.
	FVisualLogEntry* CurrentEntry = CurrentEntryPerObject.Find(LogOwner);
	if (CurrentEntry)
	{
		if (CurrentEntry->ShouldLog(ShouldCreate))
		{
			if (CurrentEntry->ShouldFlush(TimeStamp))
			{
				FlushEntry(*CurrentEntry, LogOwner);
			}
	
			bInitializeEntry = !CurrentEntry->bIsInitialized;
		}
	}
	else if (ShouldCreate == ECreateIfNeeded::Create)
	{
		// It's first and only one usage of LogOwner as regular object to get names. We assume once that LogOwner is correct here and only here.
		CurrentEntry = &CurrentEntryPerObject.Add(LogOwner);

		ObjectToNamesMap.Add(LogOwner, CreateObjectNames(LogOwner));
		ObjectToWorldMap.Add(LogOwner, GetWorldForVisualLogger(LogOwner));

		CalculateEntryAllowLogging(CurrentEntry, LogOwner, Object);

		bInitializeEntry = CurrentEntry->bIsAllowedToLog;
	}

	if (bInitializeEntry)
	{
		checkf(CurrentEntry != nullptr, TEXT("bInitializeEntry can only be true when CurrentEntry is valid."));
		CurrentEntry->InitializeEntry(TimeStamp);

		// Let's record the World Time as the local instance sees it.
		if (const TWeakObjectPtr<const UWorld>* WorldWeakPtr = ObjectToWorldMap.Find(LogOwner))
		{
			if (const UWorld* World = WorldWeakPtr->Get())
			{
				CurrentEntry->WorldTimeStamp = World->TimeSeconds;
			}
		}

		if (const AActor* ObjectAsActor = Cast<AActor>(LogOwner))
		{
			CurrentEntry->Location = ObjectAsActor->GetActorLocation();
			CurrentEntry->bIsLocationValid = true;
		}
		else if (const UActorComponent* ObjectAsActorComponent = Cast<UActorComponent>(LogOwner))
		{
			if (const USceneComponent* ObjectAsSceneComponent = Cast<USceneComponent>(ObjectAsActorComponent))
			{
				CurrentEntry->Location = ObjectAsSceneComponent->GetComponentLocation();
				CurrentEntry->bIsLocationValid = true;
			}
			else if (const AActor* ComponentOwner = ObjectAsActorComponent->GetOwner())
			{
				CurrentEntry->Location = ComponentOwner->GetActorLocation();
				CurrentEntry->bIsLocationValid = true;
			}
		}

		UE::TReadScopeLock RedirectScopeLock(RedirectRWLock);
		FOwnerToChildrenRedirectionMap& RedirectionMap = GetRedirectionMap(LogOwner);
		if (const IVisualLoggerDebugSnapshotInterface* DebugSnapshotInterface = Cast<const IVisualLoggerDebugSnapshotInterface>(LogOwner))
		{
			DebugSnapshotInterface->GrabDebugSnapshot(CurrentEntry);
		}
		
		TArray<TWeakObjectPtr<const UObject>>* Children = RedirectionMap.Find(LogOwner);
		if (Children != nullptr)
		{
			for (TWeakObjectPtr<const UObject>& Child : *Children)
			{
				if (const IVisualLoggerDebugSnapshotInterface* DebugSnapshotInterface = Cast<const IVisualLoggerDebugSnapshotInterface>(Child.Get()))
				{
					DebugSnapshotInterface->GrabDebugSnapshot(CurrentEntry);
				}
			}
		}
	}

	if (CurrentEntry != nullptr && CurrentEntry->bIsAllowedToLog)
	{
		bIsFlushRequired = true;
	}
	else
	{
		CurrentEntry = nullptr;
	}

	return CurrentEntry;
}

void FVisualLogger::CalculateEntryAllowLogging(FVisualLogEntry* CurrentEntry, const UObject* LogOwner, const UObject* Object) const
{
	// IsClassAllowed isn't super fast, but this gets calculated only once for every 
	// object trying to log something
	CurrentEntry->bPassedClassAllowList = (ClassAllowList.Num() == 0) || IsClassAllowed(*LogOwner->GetClass()) || IsClassAllowed(*Object->GetClass());
	CurrentEntry->bPassedObjectAllowList = IsObjectAllowed(LogOwner);
	CurrentEntry->UpdateAllowedToLog();
}

void FVisualLogger::FlushThreadsEntries()
{
	UE_MT_SCOPED_READ_ACCESS(ThreadLocalMapListAccessDetector);
	UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
	for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
	{
		checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
		UE_MT_SCOPED_READ_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
		for (auto& ThreadCurrentEntry : *ThreadCurrentEntryMap)
		{
			if (ThreadCurrentEntry.Value.bIsInitialized)
			{
				const UObject* OwnerObject = ThreadCurrentEntry.Key.ResolveObjectPtrEvenIfGarbage();
				if (FVisualLogEntry* GlobalCurrentEntry = GetEntryToWriteInternal(OwnerObject, ThreadCurrentEntry.Value.TimeStamp, ECreateIfNeeded::Create))
				{
					ThreadCurrentEntry.Value.MoveTo(*GlobalCurrentEntry);
				}
			}
		}
	}
}

void FVisualLogger::Flush()
{
	UE::TWriteScopeLock Lock(EntryRWLock);

	FlushThreadsEntries();

	// Using the Write access since it might flush and reset entries.
	UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
	for (auto &CurrentEntry : CurrentEntryPerObject)
	{
		if (CurrentEntry.Value.bIsInitialized)
		{
			FlushEntry(CurrentEntry.Value, CurrentEntry.Key);
		}
	}
}

void FVisualLogger::FlushEntry(FVisualLogEntry& Entry, const FObjectKey& ObjectKey)
{
	ensureMsgf(Entry.bIsInitialized, TEXT("FlushEntry should only be called with an initialized entry."));

	const UObject* OwnerObject = ObjectKey.ResolveObjectPtrEvenIfGarbage();
	for (FVisualLogDevice* Device : OutputDevices)
	{
		const FObjectNames& Names = ObjectToNamesMap[ObjectKey];
		Device->Serialize(OwnerObject, Names.Name, Names.DisplayName, Names.ClassName, Entry);
	}
	Entry.Reset();
}

thread_local FVisualLogger::FVisualLoggerObjectEntryMap* GThreadCurrentEntryPerObject = nullptr;
FVisualLogger::FVisualLoggerObjectEntryMap& FVisualLogger::GetThreadCurrentEntryMap()
{
	if (GThreadCurrentEntryPerObject == nullptr)
	{
		GThreadCurrentEntryPerObject = new FVisualLoggerObjectEntryMap();
		UE::TWriteScopeLock Lock(EntryRWLock);
		UE_MT_SCOPED_WRITE_ACCESS(ThreadLocalMapListAccessDetector);
		ThreadCurrentEntryMaps.Add(GThreadCurrentEntryPerObject);
	}
	checkf(GThreadCurrentEntryPerObject, TEXT("Unexpected null map entry pointer"));
	return *GThreadCurrentEntryPerObject;
}

void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4, const FVisualLogEventBase& Event5, const FVisualLogEventBase& Event6)
{
	EventLog(Object, EventTag1, Event1, Event2, Event3, Event4, Event5);
	EventLog(Object, EventTag1, Event6);
}


void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4, const FVisualLogEventBase& Event5)
{
	EventLog(Object, EventTag1, Event1, Event2, Event3, Event4);
	EventLog(Object, EventTag1, Event5);
}


void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3, const FVisualLogEventBase& Event4)
{
	EventLog(Object, EventTag1, Event1, Event2, Event3);
	EventLog(Object, EventTag1, Event4);
}


void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2, const FVisualLogEventBase& Event3)
{
	EventLog(Object, EventTag1, Event1, Event2);
	EventLog(Object, EventTag1, Event3);
}


void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event1, const FVisualLogEventBase& Event2)
{
	EventLog(Object, EventTag1, Event1);
	EventLog(Object, EventTag1, Event2);
}


void FVisualLogger::EventLog(const UObject* LogOwner, const FVisualLogEventBase& Event1, const FName EventTag1, const FName EventTag2, const FName EventTag3, const FName EventTag4, const FName EventTag5, const FName EventTag6)
{
	EventLog(LogOwner, EventTag1, Event1, EventTag2, EventTag3, EventTag4, EventTag5, EventTag6);
}


void FVisualLogger::EventLog(const UObject* Object, const FName EventTag1, const FVisualLogEventBase& Event, const FName EventTag2, const FName EventTag3, const FName EventTag4, const FName EventTag5, const FName EventTag6)
{
	SCOPE_CYCLE_COUNTER(STAT_VisualLog);
	UWorld *World = nullptr;
	FVisualLogEntry *CurrentEntry = nullptr;
	const FName CategoryName(*Event.Name);
	if (CheckVisualLogInputInternal(Object, CategoryName, ELogVerbosity::Log, &World, &CurrentEntry) == false)
	{
		return;
	}

	int32 Index = CurrentEntry->Events.Find(FVisualLogEvent(Event));
	if (Index != INDEX_NONE)
	{
		CurrentEntry->Events[Index].Counter++;
	}
	else
	{
		Index = CurrentEntry->AddEvent(Event);
	}

	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag1)++;
	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag2)++;
	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag3)++;
	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag4)++;
	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag5)++;
	CurrentEntry->Events[Index].EventTags.FindOrAdd(EventTag6)++;
	CurrentEntry->Events[Index].EventTags.Remove(NAME_None);
}

void FVisualLogger::NavigationDataDump(const UObject* Object, const FLogCategoryBase& Category, const ELogVerbosity::Type Verbosity, const FBox& Box)
{
	NavigationDataDump(Object, Category.GetCategoryName(), Verbosity, Box);
}

void FVisualLogger::NavigationDataDump(const UObject* Object, const FName& CategoryName, const ELogVerbosity::Type Verbosity, const FBox& Box)
{
	SCOPE_CYCLE_COUNTER(STAT_VisualLog);

	UWorld* World = nullptr;
	FVisualLogEntry* CurrentEntry = nullptr;
	if (CheckVisualLogInputInternal(Object, CategoryName, Verbosity, &World, &CurrentEntry) == false
		|| CurrentEntry == nullptr)
	{
		return;
	}

	NavigationDataDumpDelegate.Broadcast(Object, CategoryName, Verbosity, Box, *World, *CurrentEntry);
}
#endif // UE_DEBUG_RECORDING_USING_VLOG

FVisualLogger& FVisualLogger::Get()
{
	static FVisualLogger GVisLog;
	return GVisLog;
}

FVisualLogger::FVisualLogger()
{
	bForceUniqueLogNames = true;

	if (FParse::Param(FCommandLine::Get(), TEXT("EnableVisualLoggerTrace")))
	{
		SetIsRecordingToTrace(true);
	}
	else
	{
		bIsRecordingToTrace = false;
	}

#if UE_DEBUG_RECORDING_USING_VLOG
	bIsRecordingToFile = false;
	bIsFlushRequired = false;

	BlockAllCategories(false);
	AddDevice(&FVisualLoggerBinaryFileDevice::Get());
	SetIsRecording(GEngine ? !!GEngine->bEnableVisualLogRecordingOnStart : false);
	SetIsRecordingOnServer(false);

	if (FParse::Param(FCommandLine::Get(), TEXT("EnableAILogging")))
	{
		SetIsRecording(true);
		SetIsRecordingToFile(true);
	}

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("VisualLogger"), 0.0f, [this](const float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FVisualLogger_Tick);

		Tick(DeltaTime);

		return true;
	});

#if WITH_EDITOR
	PIEStartedHandle = FEditorDelegates::StartPIE.AddLambda([this](bool bSimulating)
		{
			// Don't reset if we're not going to PIE in the same process
			if (TOptional<FRequestPlaySessionParams> PlaySessionRequest = GEditor->GetPlaySessionRequest())
			{
				if (PlaySessionRequest->SessionDestination != EPlaySessionDestinationType::InProcess)
				{
					return;
				}
			}

			// We only want to do this if we are using unique log names, otherwise it makes more sense
			// to just continue logging at later timestamps on the same timeline as the previous one.
			if (bForceUniqueLogNames)
			{
				int32 OldUniqueNumber = UE::VisLog::Private::UniqueRunNumber;
				OnDataReset();
				UE::VisLog::Private::UniqueRunNumber = OldUniqueNumber + 1;
			}
		});
#endif // WITH_EDITOR
#endif // UE_DEBUG_RECORDING_USING_VLOG
}

void FVisualLogger::SetIsRecording(const bool InIsRecording)
{
#if UE_DEBUG_RECORDING_USING_VLOG
	if (InIsRecording == false && InIsRecording != !!bIsRecording && FParse::Param(FCommandLine::Get(), TEXT("LogNavOctree")))
	{
		NavigationDataDump(GetWorldForVisualLogger(nullptr), LogNavigation, ELogVerbosity::Log, FBox());
	}

	if (IsRecordingToFile())
	{
		SetIsRecordingToFile(false);
	}
#endif // UE_DEBUG_RECORDING_USING_VLOG

	bIsRecording = InIsRecording;
};

void FVisualLogger::SetIsRecordingToTrace(const bool InIsRecording)
{
	if (!bIsRecording && InIsRecording)
	{
		SetIsRecording(true);
	}

	FVisualLoggerTraceDevice& Device = FVisualLoggerTraceDevice::Get();
	if (bIsRecordingToTrace && !InIsRecording)
	{
		Device.StopRecordingToFile(0.0);
#if UE_DEBUG_RECORDING_USING_VLOG
		RemoveDevice(&Device);
#endif
	}
	else if (!bIsRecordingToTrace && InIsRecording)
	{
		Device.StartRecordingToFile(0.0);
#if UE_DEBUG_RECORDING_USING_VLOG
		AddDevice(&Device);
#endif
	}

	bIsRecordingToTrace = InIsRecording;
}

void FVisualLogger::SetUseUniqueNames(const bool bEnable)
{
	bForceUniqueLogNames = bEnable;
	UE::VisLog::Private::UniqueLogPrefix.Empty();
}

void FVisualLogger::Redirect(const UObject* FromObject, const UObject* ToObject)
{
	FVisualLogger& Logger = Get();
	UObject* NewRedirection = nullptr;
	{
		UE::TWriteScopeLock Lock(Logger.RedirectRWLock);
		NewRedirection = Logger.RedirectInternal(FromObject, ToObject);
	}
	UE_CVLOG(FromObject != nullptr && NewRedirection != nullptr, FromObject, LogVisual, Log, TEXT("Redirected '%s' to '%s'"), *FromObject->GetName(), *NewRedirection->GetName());
}

UObject* FVisualLogger::RedirectInternal(const UObject* FromObject, const UObject* ToObject)
{
	if (FromObject == ToObject || FromObject == nullptr || ToObject == nullptr)
	{
		return nullptr;
	}

	const TWeakObjectPtr<const UObject> FromWeakPtr(FromObject);

	TWeakObjectPtr<const UObject>& WeakOwnerEntry = ChildToOwnerMap.FindOrAdd(FromObject);

#if UE_DEBUG_RECORDING_USING_VLOG
	const UObject* DirectOwner = WeakOwnerEntry.Get();
	const UObject* NewDirectOwner = ToObject;
	if (DirectOwner != ToObject)
	{
		FOwnerToChildrenRedirectionMap& OwnerToChildrenMap = GetRedirectionMap(FromObject);
		const TArray<TWeakObjectPtr<const UObject>>* ObjectChildren = OwnerToChildrenMap.Find(FromObject);

		// A valid direct owner indicates that FromObject was in one or many list(s)
		// of children so remove it and its children from the owner hierarchy
		if (DirectOwner != nullptr)
		{
			const UObject* Owner = DirectOwner;
			while (Owner != nullptr)
			{
				if (TArray<TWeakObjectPtr<const UObject>>* DirectOwnerChildren = OwnerToChildrenMap.Find(Owner))
				{
					DirectOwnerChildren->RemoveSingleSwap(FromWeakPtr);
					if (ObjectChildren != nullptr)
					{
						for (const TWeakObjectPtr<const UObject>& Child : *ObjectChildren)
						{
							DirectOwnerChildren->RemoveSingleSwap(Child);
						}
					}
				}

				const TWeakObjectPtr<const UObject>* WeakOwner = ChildToOwnerMap.Find(Owner);
				Owner = WeakOwner ? WeakOwner->Get() : nullptr;
			}
		}

		// Add this child with its children (if any) to the owner hierarchy
		const UObject* Owner = NewDirectOwner;
		while (Owner != nullptr)
		{
			TArray<TWeakObjectPtr<const UObject>>* OwnerChildren = OwnerToChildrenMap.Find(Owner);
			if (OwnerChildren == nullptr)
			{
				// Entry not found, add it and refresh current object children since it might have been reallocated
				OwnerChildren = &OwnerToChildrenMap.Emplace(Owner);
				ObjectChildren = ObjectChildren ? OwnerToChildrenMap.Find(FromObject) : nullptr;
			}

			check(OwnerChildren);
			OwnerChildren->Add(FromWeakPtr);
			if (ObjectChildren != nullptr)
			{
				OwnerChildren->Append(*ObjectChildren);
			}

			const TWeakObjectPtr<const UObject>* WeakOwner = ChildToOwnerMap.Find(Owner);
			Owner = WeakOwner ? WeakOwner->Get() : nullptr;
		}
	}
#endif // UE_DEBUG_RECORDING_USING_VLOG

	WeakOwnerEntry = ToObject;

	UObject* NewRedirection = FindRedirectionInternal(ToObject);
	return NewRedirection;
}

UObject* FVisualLogger::FindRedirectionInternal(const UObject* Object) const
{
	TWeakObjectPtr TargetWeakPtr(Object);
	const TWeakObjectPtr<const UObject>* Parent = &TargetWeakPtr;

	while (Parent)
	{
		Parent = ChildToOwnerMap.Find(TargetWeakPtr.Get(/*bEvenIfPendingKill*/true));
		if (Parent)
		{
			if (Parent->IsValid())
			{
				TargetWeakPtr = *Parent;
			}
			else
			{
				Parent = nullptr;
				bContainsInvalidRedirects = true;
			}
		}
	}

	const UObject* const Target = TargetWeakPtr.Get(/*bEvenIfPendingKill*/true);
	return const_cast<UObject*>(Target);
}

FVisualLogger::FObjectNames FVisualLogger::CreateObjectNames(TNotNull<const UObject*> LogOwner)
{
	if (bForceUniqueLogNames && UE::VisLog::Private::UniqueLogPrefix.IsEmpty())
	{
		UE::VisLog::Private::UniqueLogPrefix = FString::Printf(TEXT("[%d] "), UE::VisLog::Private::UniqueRunNumber);
	}

	const UWorld* World = GetWorldForVisualLogger(LogOwner);
	const bool bIsStandalone = (World == nullptr || World->GetNetMode() == NM_Standalone);

	// We use the fully qualified name in order to minimize the chance of name conflict. Otherwise, we end up losing or merging several log entries
	//  because they share the same object name (e.g. if the LogOwner is a given primitive component, then its name will likely be "PrimitiveComponent", which 
	//  means that if 2 log entries are added at the same frame for 2 different primitive components, only one will remain, because the OwnerName is the stored key, not the object...)
	const FName LogName(*FString::Printf(TEXT("%s%s%s"),
		*UE::VisLog::Private::UniqueLogPrefix,
		bIsStandalone ? TEXT("") : *FString::Printf(TEXT("(%s) "), *GetDebugStringForWorld(World)),
		*LogOwner->GetFullName()));

	const FName LogDisplayName(*FString::Printf(TEXT("%s%s%s%s"),
		*UE::VisLog::Private::UniqueLogPrefix,
		bIsStandalone ? TEXT("") : *FString::Printf(TEXT("(%s) "), *GetDebugStringForWorld(World)),
		*LogOwner->GetName(),
		bForceUniqueLogNames ? *FString::Printf(TEXT(" [%d]"), LogOwner->GetUniqueID()) : TEXT("")));

	return {.Name = LogName, .DisplayName = LogDisplayName, .ClassName = LogOwner->GetClass()->GetFName()};
}

void FVisualLogger::Tick(float DeltaTime)
{
#if UE_DEBUG_RECORDING_USING_VLOG
	if (bIsFlushRequired)
	{
		Flush();
		bIsFlushRequired = false;
	}
#endif //UE_DEBUG_RECORDING_USING_VLOG

	if (bContainsInvalidRedirects)
	{
		CleanupRedirects();
		bContainsInvalidRedirects = false;
	}
}

void FVisualLogger::CleanupRedirects()
{
	UE::TWriteScopeLock Lock(RedirectRWLock);
	for (auto It = ChildToOwnerMap.CreateIterator(); It; ++It)
	{
		if (!It.Value().IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

#if UE_DEBUG_RECORDING_USING_VLOG

void FVisualLogger::TearDown()
{
#if WITH_EDITOR
	FEditorDelegates::StartPIE.Remove(PIEStartedHandle);
#endif

	SetIsRecording(false);
	SetIsRecordingToFile(false);
	RemoveDevice(&FVisualLoggerBinaryFileDevice::Get());
}

void FVisualLogger::OnDataReset()
{
	using namespace UE::VisLog::Private;

	UniqueRunNumber = 0;
	UniqueLogPrefix.Empty();

#if WITH_EDITOR
	UEditorEngine* EditorEngine = GIsEditor ? Cast<UEditorEngine>(GEngine) : nullptr;
	if (EditorEngine)
	{
		const UWorld* EditorWorld = EditorEngine->GetEditorWorldContext().World();
		if (EditorWorld)
		{
			// Reset the base timestamp to zero so the next (aka first) log entry will set this variable to determine the global offset in GetTimeStampForObject
			// This ensures that when you start recording, the first entry appears to be at 0.0 (computed as TimeStamp - EditorBaseTimeStamp)
			EditorOnly::EditorBaseTimeStamp = 0.0;
		}
	}
#endif
}

UObject* FVisualLogger::FindRedirection(const UObject* Object)
{
	FVisualLogger& Logger = Get();
	UE::TReadScopeLock Lock(Logger.RedirectRWLock);
	return Logger.FindRedirectionInternal(Object);
}

void FVisualLogger::Cleanup(UWorld* OldWorld, const bool bReleaseMemory)
{
	const bool WasRecordingToFile = IsRecordingToFile();
	if (WasRecordingToFile)
	{
		SetIsRecordingToFile(false);
	}

	Flush();
	for (FVisualLogDevice* Device : Get().OutputDevices)
	{
		Device->Cleanup(bReleaseMemory);
	}

	if (OldWorld != nullptr)
	{
		// perform cleanup only if provided world is valid and was registered
		if (WorldToRedirectionMap.Remove(OldWorld))
		{
			if (WorldToRedirectionMap.Num() == 0)
			{
				WorldToRedirectionMap.Reset();
				ObjectToWorldMap.Reset();
				ChildToOwnerMap.Reset();

				UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
				CurrentEntryPerObject.Reset();

				for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
				{
					checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
					UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
					ThreadCurrentEntryMap->Reset();
				}
				ObjectToNamesMap.Reset();
			}
			else
			{
				UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
				for (auto It = ObjectToWorldMap.CreateIterator(); It; ++It)
				{
					if (It.Value() == OldWorld)
					{
						FObjectKey Obj = It.Key();
						It.RemoveCurrent();
						CurrentEntryPerObject.Remove(Obj);

						for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
						{
							checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
							UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
							ThreadCurrentEntryMap->Remove(Obj);
						}
						ObjectToNamesMap.Remove(Obj);
					}
				}

				for (FChildToOwnerRedirectionMap::TIterator It = ChildToOwnerMap.CreateIterator(); It; ++It)
				{
					UObject* Object = It->Key.ResolveObjectPtrEvenIfGarbage();
					if (Object == nullptr || Object->GetWorld() == OldWorld)
					{
						It.RemoveCurrent();
					}
				}
			}
		}
	}
	else
	{
		WorldToRedirectionMap.Reset();
		ObjectToWorldMap.Reset();
		ChildToOwnerMap.Reset();

		UE_MT_SCOPED_WRITE_ACCESS(CurrentEntryPerObject.EntriesAccessDetector);
		CurrentEntryPerObject.Reset();

		for (FVisualLoggerObjectEntryMap* ThreadCurrentEntryMap : ThreadCurrentEntryMaps)
		{
			checkf(ThreadCurrentEntryMap, TEXT("Unexpected null map entry pointer"));
			UE_MT_SCOPED_WRITE_ACCESS(ThreadCurrentEntryMap->EntriesAccessDetector);
			ThreadCurrentEntryMap->Reset();
		}
		ObjectToNamesMap.Reset();
	}

	LastUniqueIds.Reset();

	if (WasRecordingToFile)
	{
		SetIsRecordingToFile(true);
	}
}

int32 FVisualLogger::GetUniqueId(const double Timestamp)
{
	return LastUniqueIds.FindOrAdd(Timestamp)++;
}

FVisualLogger::FOwnerToChildrenRedirectionMap& FVisualLogger::GetRedirectionMap(const UObject* InObject)
{
	const UWorld* World = nullptr;
	if (const TWeakObjectPtr<const UWorld>* Entry = Get().ObjectToWorldMap.Find(InObject))
	{
		World = Entry->Get();
	}

	if (World == nullptr)
	{
		World = GetWorldForVisualLogger(InObject);
	}

	return WorldToRedirectionMap.FindOrAdd(World);
}

void FVisualLogger::SetIsRecordingToFile(const bool InIsRecording)
{
	if (!bIsRecording && InIsRecording)
	{
		SetIsRecording(true);
	}

	UWorld* World = GEngine ? GEngine->GetWorld() : nullptr;

	const FString BaseFileName = LogFileNameGetter.IsBound() ? LogFileNameGetter.Execute() : TEXT("VisualLog");
	const FString MapName = World ? World->GetMapName() : TEXT("");

	const FString OutputFileName = FString::Printf(TEXT("%s_%s"), *BaseFileName, *MapName);

	if (bIsRecordingToFile && !InIsRecording)
	{
		for (FVisualLogDevice* Device : OutputDevices)
		{
			if (Device->HasFlags(EVisualLoggerDeviceFlags::CanSaveToFile))
			{
				Device->SetFileName(OutputFileName);
				Device->StopRecordingToFile(World ? World->TimeSeconds : StartRecordingToFileTime);
			}
		}
	}
	else if (!bIsRecordingToFile && InIsRecording)
	{
		StartRecordingToFileTime = World ? World->TimeSeconds : 0;
		for (FVisualLogDevice* Device : OutputDevices)
		{
			if (Device->HasFlags(EVisualLoggerDeviceFlags::CanSaveToFile))
			{
				Device->StartRecordingToFile(StartRecordingToFileTime);
			}
		}
	}

	bIsRecordingToFile = InIsRecording;
}

void FVisualLogger::DiscardRecordingToFile()
{
	if (bIsRecordingToFile)
	{
		for (FVisualLogDevice* Device : OutputDevices)
		{
			if (Device->HasFlags(EVisualLoggerDeviceFlags::CanSaveToFile))
			{
				Device->DiscardRecordingToFile();
			}
		}

		bIsRecordingToFile = false;
	}
}

bool FVisualLogger::IsCategoryLogged(const FLogCategoryBase& Category) const
{
	if ((GEngine && GEngine->bDisableAILogging) || IsRecording() == false)
	{
		return false;
	}

	const FName CategoryName = Category.GetCategoryName();
	if (IsBlockedForAllCategories() && IsCategoryAllowed(CategoryName) == false)
	{
		return false;
	}

	return true;
}

#endif // UE_DEBUG_RECORDING_USING_VLOG

const FGuid EVisualLoggerVersion::GUID = FGuid(0xA4237A36, 0xCAEA41C9, 0x8FA218F8, 0x58681BF3);
FCustomVersionRegistration GVisualLoggerVersion(EVisualLoggerVersion::GUID, EVisualLoggerVersion::LatestVersion, TEXT("VisualLogger"));

#if UE_DEBUG_RECORDING_USING_VLOG

class FLogVisualizerExec : private FSelfRegisteringExec
{
protected:
	/** Console commands, see embedded usage statement **/
	virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("VISLOG")))
		{
			const FString Command = FParse::Token(Cmd, /*UseEscape*/ false);
			if (Command == TEXT("record"))
			{
				if (GIsEditor)
				{
					FVisualLogger::Get().SetIsRecording(true);
				}
				else
				{
					FVisualLogger::Get().SetIsRecordingToFile(true);
				}
				return true;
			}
			else if (Command == TEXT("stop"))
			{
				if (GIsEditor)
				{
					FVisualLogger::Get().SetIsRecording(false);
				}
				else
				{
					FVisualLogger::Get().SetIsRecordingToFile(false);
				}
				return true;
			}
			else if (Command == TEXT("disableallbut"))
			{
				const FString Category = FParse::Token(Cmd, /*UseEscape*/ true);
				FVisualLogger::Get().BlockAllCategories(true);
				FVisualLogger::Get().AddCategoryToAllowList(*Category);
				return true;
			}
			else if (Command.IsEmpty() && FModuleManager::Get().LoadModulePtr<IModuleInterface>("LogVisualizer"))
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("VisualLogger"));
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("LogNavOctree")))
		{
			FVisualLogger::NavigationDataDump(GetWorldForVisualLogger(InWorld), LogNavigation, ELogVerbosity::Log, FBox());
		}

		return false;
	}
} LogVisualizerExec;

#endif // UE_DEBUG_RECORDING_USING_VLOG

#endif // UE_DEBUG_RECORDING_ENABLED
