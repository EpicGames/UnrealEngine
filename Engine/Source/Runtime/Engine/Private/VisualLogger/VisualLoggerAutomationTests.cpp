// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLoggerAutomationTests.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/AutomationTest.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerAutomationTests)

namespace
{
	UWorld* GetSimpleEngineAutomationTestWorld(const EAutomationTestFlags TestFlags)
	{
		// Accessing the game world is only valid for game-only
		if (!(TestFlags & EAutomationTestFlags::EditorContext) && !(TestFlags & EAutomationTestFlags::ClientContext))
		{
			return nullptr;
		}
		if (GEngine->GetWorldContexts().Num() == 0)
		{
			return nullptr;
		}
		if (GEngine->GetWorldContexts()[0].WorldType != EWorldType::Game && GEngine->GetWorldContexts()[0].WorldType != EWorldType::Editor)
		{
			return nullptr;
		}

		return GEngine->GetWorldContexts()[0].World();
	}
}

UVisualLoggerAutomationTests::UVisualLoggerAutomationTests(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{

}

#if UE_DEBUG_RECORDING_USING_VLOG

class FVisualLoggerTestDevice : public FVisualLogDevice
{
public:
	FVisualLoggerTestDevice();
	virtual void Cleanup(bool bReleaseMemory = false) override;
	virtual void Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry) override;

	class UObject* LastObject;
	FVisualLogEntry LastEntry;
};

FVisualLoggerTestDevice::FVisualLoggerTestDevice()
{
	Cleanup();
}

void FVisualLoggerTestDevice::Cleanup(bool bReleaseMemory)
{
	LastObject = nullptr;
	LastEntry.Reset();
}

void FVisualLoggerTestDevice::Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry)
{
	LastObject = const_cast<class UObject*>(InLogOwner);
	LastEntry = InLogEntry;
}

#define CHECK_SUCCESS(__Test__) UTEST_TRUE(FString::Printf( TEXT("%s (%s:%d)"), TEXT(#__Test__), TEXT(__FILE__), __LINE__ ), __Test__)
#define CHECK_FAIL(__Test__) UTEST_FALSE(FString::Printf( TEXT("%s (%s:%d)"), TEXT(#__Test__), TEXT(__FILE__), __LINE__ ), __Test__)
#define CHECK_NOT_NULL(Pointer) \
	if (!TestNotNull(TEXT(#Pointer), Pointer))\
	{\
		return false;\
	}

template<typename TYPE = FVisualLoggerTestDevice>
struct FTestDeviceContext
{
	FTestDeviceContext()
	{
		Device.Cleanup();
		FVisualLogger::Get().SetIsRecording(false);
		FVisualLogger::Get().Cleanup(nullptr);
		FVisualLogger::Get().AddDevice(&Device);

		EngineDisableAILoggingFlag = GEngine->bDisableAILogging;
		GEngine->bDisableAILogging = false;

	}

	~FTestDeviceContext()
	{
		FVisualLogger::Get().SetIsRecording(false);
		FVisualLogger::Get().RemoveDevice(&Device);
		FVisualLogger::Get().Cleanup(nullptr);
		Device.Cleanup();

		GEngine->bDisableAILogging = EngineDisableAILoggingFlag;
	}

	TYPE Device;
	uint32 EngineDisableAILoggingFlag : 1;
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogTest, "System.Engine.VisualLogger.Logging simple text", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
/**
*
*
* @param Parameters - Unused for this test
* @return	TRUE if the test was successful, FALSE otherwise
*/

bool FVisualLogTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FTestDeviceContext<FVisualLoggerTestDevice> Context;

	FVisualLogger::Get().SetIsRecording(false);
	CHECK_FAIL(FVisualLogger::Get().IsRecording());

	// Make sure our log is recorded by setting the active verbosity.
	// This is required if verbosity filtering when recording is enabled.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogVisual, ELogVerbosity::Log);

	UE_VLOG(World, LogVisual, Log, TEXT("Simple text line to test vlog"));
	CHECK_SUCCESS(Context.Device.LastObject == nullptr);
	CHECK_SUCCESS(Context.Device.LastEntry.TimeStamp == -1);

	FVisualLogger::Get().SetIsRecording(true);
	CHECK_SUCCESS(FVisualLogger::Get().IsRecording());

	{
		const FString TextToLog = "Simple text line to test if UE_VLOG_UELOG works fine";
		double CurrentTimestamp = FVisualLogger::Get().GetTimeStampForObject(World);
		UE_VLOG_UELOG(World, LogVisual, Log, TEXT("%s"), *TextToLog);
		CHECK_SUCCESS(Context.Device.LastObject != World);
		CHECK_SUCCESS(Context.Device.LastEntry.TimeStamp == -1);
		FVisualLogEntry* CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, World->TimeSeconds, ECreateIfNeeded::DontCreate);
		
		{
			CHECK_NOT_NULL(CurrentEntry);
			CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
			CHECK_SUCCESS(CurrentEntry->LogLines.Num() == 1);
			CHECK_SUCCESS(CurrentEntry->LogLines[0].Category == LogVisual.GetCategoryName());
			CHECK_SUCCESS(CurrentEntry->LogLines[0].Line == TextToLog);

			const double NewTimestamp = CurrentTimestamp + 0.1;
			FVisualLogEntry* NewEntry = FVisualLogger::Get().GetEntryToWrite(World, NewTimestamp); //generate new entry and serialize old one
			FVisualLogger::Get().FlushThreadsEntries();
			CurrentEntry = &Context.Device.LastEntry;
			CHECK_NOT_NULL(CurrentEntry);
			CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
			CHECK_SUCCESS(CurrentEntry->LogLines.Num() == 1);
			CHECK_SUCCESS(CurrentEntry->LogLines[0].Category == LogVisual.GetCategoryName());
			CHECK_SUCCESS(CurrentEntry->LogLines[0].Line == TextToLog);

			CHECK_NOT_NULL(NewEntry);
			CHECK_SUCCESS(NewEntry->TimeStamp - NewTimestamp <= UE_SMALL_NUMBER);
			CHECK_SUCCESS(NewEntry->LogLines.Num() == 0);
		}
	}

	FVisualLogger::Get().SetIsRecording(false);
	CHECK_FAIL(FVisualLogger::Get().IsRecording());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogSegmentsTest, "System.Engine.VisualLogger.Logging segment shape", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogSegmentsTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FTestDeviceContext<FVisualLoggerTestDevice> Context;

	FVisualLogger::Get().SetIsRecording(false);
	CHECK_FAIL(FVisualLogger::Get().IsRecording());

	// Make sure our log is recorded by setting the active verbosity.
	// This is required if verbosity filtering when recording is enabled.
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogVisual, ELogVerbosity::Log);

	UE_VLOG_SEGMENT(World, LogVisual, Log, FVector(0, 0, 0), FVector(1, 0, 0), FColor::Red, TEXT("Simple segment log to test vlog"));
	CHECK_SUCCESS(Context.Device.LastObject == nullptr);
	CHECK_SUCCESS(Context.Device.LastEntry.TimeStamp == -1);

	FVisualLogger::Get().SetIsRecording(true);
	CHECK_SUCCESS(FVisualLogger::Get().IsRecording());
	{
		const FVector StartPoint(0, 0, 0);
		const FVector EndPoint(1, 0, 0);
		UE_VLOG_SEGMENT(World, LogVisual, Log, StartPoint, EndPoint, FColor::Red, TEXT("Simple segment log to test vlog"));
		CHECK_SUCCESS(Context.Device.LastObject == nullptr);
		CHECK_SUCCESS(Context.Device.LastEntry.TimeStamp == -1);
		FVisualLogEntry* CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, World->TimeSeconds, ECreateIfNeeded::DontCreate);

		double CurrentTimestamp = FVisualLogger::Get().GetTimeStampForObject(World);
		{
			CHECK_NOT_NULL(CurrentEntry);
			CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw.Num() == 1);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].GetType() == EVisualLoggerShapeElement::Segment);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points.Num() == 2);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points[0] == StartPoint);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points[1] == EndPoint);

			const double NewTimestamp = CurrentTimestamp + 0.1;
			FVisualLogEntry* NewEntry = FVisualLogger::Get().GetEntryToWrite(World, NewTimestamp); //generate new entry and serialize old one
			FVisualLogger::Get().FlushThreadsEntries();
			CurrentEntry = &Context.Device.LastEntry;
			CHECK_NOT_NULL(CurrentEntry);
			CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw.Num() == 1);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].GetType() == EVisualLoggerShapeElement::Segment);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points.Num() == 2);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points[0] == StartPoint);
			CHECK_SUCCESS(CurrentEntry->ElementsToDraw[0].Points[1] == EndPoint);

			CHECK_NOT_NULL(NewEntry);
			CHECK_SUCCESS(NewEntry->TimeStamp - NewTimestamp <= UE_SMALL_NUMBER);
			CHECK_SUCCESS(NewEntry->ElementsToDraw.Num() == 0);
		}
	}
	return true;
}

DEFINE_VLOG_EVENT(EventTest, All, "Simple event for vlog tests")

DEFINE_VLOG_EVENT(EventTest2, All, "Second simple event for vlog tests")

DEFINE_VLOG_EVENT(EventTest3, All, "Third simple event for vlog tests")

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogEventsTest, "System.Engine.VisualLogger.Logging events", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogEventsTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FTestDeviceContext<FVisualLoggerTestDevice> Context;
	FVisualLogger::Get().SetIsRecording(true);

	CHECK_SUCCESS(EventTest.Name == TEXT("EventTest"));
	CHECK_SUCCESS(EventTest.FriendlyDesc == TEXT("Simple event for vlog tests"));

	CHECK_SUCCESS(EventTest2.Name == TEXT("EventTest2"));
	CHECK_SUCCESS(EventTest2.FriendlyDesc == TEXT("Second simple event for vlog tests"));

	CHECK_SUCCESS(EventTest3.Name == TEXT("EventTest3"));
	CHECK_SUCCESS(EventTest3.FriendlyDesc == TEXT("Third simple event for vlog tests"));

	double CurrentTimestamp = FVisualLogger::Get().GetTimeStampForObject(World);
	FVisualLogEntry* CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, CurrentTimestamp, ECreateIfNeeded::DontCreate);
	CHECK_SUCCESS(CurrentEntry == nullptr);

	UE_VLOG_EVENTS(World, NAME_None, EventTest);
	CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, CurrentTimestamp, ECreateIfNeeded::DontCreate);
	CHECK_NOT_NULL(CurrentEntry);
	CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
	CHECK_SUCCESS(CurrentEntry->Events.Num() == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));

	UE_VLOG_EVENTS(World, NAME_None, EventTest, EventTest2);
	CHECK_NOT_NULL(CurrentEntry);
	CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
	CHECK_SUCCESS(CurrentEntry->Events.Num() == 2);
	CHECK_SUCCESS(CurrentEntry->Events[0].Counter == 2);
	CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));
	CHECK_SUCCESS(CurrentEntry->Events[1].Counter == 1);
	CHECK_SUCCESS(CurrentEntry->Events[1].Name == TEXT("EventTest2"));

	CurrentTimestamp = FVisualLogger::Get().GetTimeStampForObject(World);
	UE_VLOG_EVENTS(World, NAME_None, EventTest, EventTest2, EventTest3);

	{
		CHECK_NOT_NULL(CurrentEntry);
		CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
		CHECK_SUCCESS(CurrentEntry->Events.Num() == 3);
		CHECK_SUCCESS(CurrentEntry->Events[0].Counter == 3);
		CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));
		CHECK_SUCCESS(CurrentEntry->Events[1].Counter == 2);
		CHECK_SUCCESS(CurrentEntry->Events[1].Name == TEXT("EventTest2"));
		CHECK_SUCCESS(CurrentEntry->Events[2].Counter == 1);
		CHECK_SUCCESS(CurrentEntry->Events[2].Name == TEXT("EventTest3"));

		CHECK_SUCCESS(CurrentEntry->Events[0].UserFriendlyDesc == TEXT("Simple event for vlog tests"));
		CHECK_SUCCESS(CurrentEntry->Events[1].UserFriendlyDesc == TEXT("Second simple event for vlog tests"));
		CHECK_SUCCESS(CurrentEntry->Events[2].UserFriendlyDesc == TEXT("Third simple event for vlog tests"));

		// Create a NewEntry that has no data in it.  In this case, our entry won't coalesce with the previous data and we should end up with NewTimestamp.
		// This functionality (requesting an explicit TimeStamp) is deprecated and will be removed, since we should always be using GetTimeStampForObject()
		// as that functionality is what's used when actually logging.
		const double NewTimestamp = CurrentTimestamp + 0.1;
		FVisualLogEntry* NewEntry = FVisualLogger::Get().GetEntryToWrite(World, NewTimestamp); //generate new entry and serialize old one
		FVisualLogger::Get().FlushThreadsEntries();
		CurrentEntry = &Context.Device.LastEntry;
		CHECK_NOT_NULL(CurrentEntry);
		CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
		CHECK_SUCCESS(CurrentEntry->Events.Num() == 3);
		CHECK_SUCCESS(CurrentEntry->Events[0].Counter == 3);
		CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));
		CHECK_SUCCESS(CurrentEntry->Events[1].Counter == 2);
		CHECK_SUCCESS(CurrentEntry->Events[1].Name == TEXT("EventTest2"));
		CHECK_SUCCESS(CurrentEntry->Events[2].Counter == 1);
		CHECK_SUCCESS(CurrentEntry->Events[2].Name == TEXT("EventTest3"));

		CHECK_SUCCESS(CurrentEntry->Events[0].UserFriendlyDesc == TEXT("Simple event for vlog tests"));
		CHECK_SUCCESS(CurrentEntry->Events[1].UserFriendlyDesc == TEXT("Second simple event for vlog tests"));
		CHECK_SUCCESS(CurrentEntry->Events[2].UserFriendlyDesc == TEXT("Third simple event for vlog tests"));

		CHECK_NOT_NULL(NewEntry);
		CHECK_SUCCESS(NewEntry->TimeStamp - NewTimestamp <= UE_SMALL_NUMBER);
		CHECK_SUCCESS(NewEntry->Events.Num() == 0);
	}

	const FName EventTag1 = "ATLAS_C_0";
	const FName EventTag2 = "ATLAS_C_1";
	const FName EventTag3 = "ATLAS_C_2";

	CurrentTimestamp = World->TimeSeconds + 0.2;
	CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, CurrentTimestamp); //generate new entry and serialize old one
	UE_VLOG_EVENT_WITH_DATA(World, EventTest, EventTag1);
	CHECK_NOT_NULL(CurrentEntry);
	CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
	CHECK_SUCCESS(CurrentEntry->Events.Num() == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags.Num() == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags[EventTag1] == 1);

	CurrentTimestamp = World->TimeSeconds + 0.3;
	CurrentEntry = FVisualLogger::Get().GetEntryToWrite(World, CurrentTimestamp); //generate new entry and serialize old one
	UE_VLOG_EVENT_WITH_DATA(World, EventTest, EventTag1, EventTag2, EventTag3);
	UE_VLOG_EVENT_WITH_DATA(World, EventTest, EventTag3);
	CHECK_NOT_NULL(CurrentEntry);
	CHECK_SUCCESS(CurrentEntry->TimeStamp == CurrentTimestamp);
	CHECK_SUCCESS(CurrentEntry->Events.Num() == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].Name == TEXT("EventTest"));
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags.Num() == 3);
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags[EventTag1] == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags[EventTag2] == 1);
	CHECK_SUCCESS(CurrentEntry->Events[0].EventTags[EventTag3] == 2);


	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogRedirectionsCleanupTest, "System.Engine.VisualLogger.Redirections.Cleanup", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogRedirectionsCleanupTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FVisualLogger& Logger = FVisualLogger::Get();
	Logger.Cleanup(World);
	FVisualLogger::FOwnerToChildrenRedirectionMap& RedirectionMap = Logger.GetRedirectionMap(World);
	CHECK_SUCCESS(RedirectionMap.Num() == 0);

	const UObject* ObjA = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjA"), RF_Transient);
	const UObject* ObjB = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjB"), RF_Transient);
	const UObject* ObjC = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjC"), RF_Transient);

	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjB);
	// C -> B -> A
	CHECK_SUCCESS(RedirectionMap.Num() == 2);
	CHECK_NOT_NULL(RedirectionMap.Find(ObjA));
	CHECK_NOT_NULL(RedirectionMap.Find(ObjB));
	CHECK_SUCCESS(RedirectionMap.Find(ObjC) == nullptr);
	
	Logger.Cleanup(World);
	CHECK_SUCCESS(RedirectionMap.Num() == 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogRedirectionsMultipleChildrenTest, "System.Engine.VisualLogger.Redirections.MultipleChildren", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogRedirectionsMultipleChildrenTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FVisualLogger& Logger = FVisualLogger::Get();
	Logger.Cleanup(World);
	FVisualLogger::FOwnerToChildrenRedirectionMap& RedirectionMap = Logger.GetRedirectionMap(World);
	CHECK_SUCCESS(RedirectionMap.Num() == 0);

	const UObject* ObjA = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjA"), RF_Transient);
	const UObject* ObjB = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjB"), RF_Transient);
	const UObject* ObjC = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjC"), RF_Transient);
	const UObject* ObjD = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjD"), RF_Transient);

	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	// B -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_SUCCESS(ChildrenOfA->Num() == 1);
		CHECK_SUCCESS(Logger.FindRedirection(ObjB) == ObjA);
	}

	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjA);
	// B -> A
	// C -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_SUCCESS(ChildrenOfA->Num() == 2);
		CHECK_SUCCESS(Logger.FindRedirection(ObjC) == ObjA);
	}

	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjA);
	// B -> A
	// C -> A
	// D -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_SUCCESS(ChildrenOfA->Num() == 3);
		CHECK_SUCCESS(Logger.FindRedirection(ObjD) == ObjA);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogRedirectionsCreationOrderTest, "System.Engine.VisualLogger.Redirections.CreationOrder", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogRedirectionsCreationOrderTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FVisualLogger& Logger = FVisualLogger::Get();
	Logger.Cleanup(World);
	FVisualLogger::FOwnerToChildrenRedirectionMap& RedirectionMap = Logger.GetRedirectionMap(World);
	CHECK_SUCCESS(RedirectionMap.Num() == 0);

	const UObject* ObjA = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjA"), RF_Transient);
	const UObject* ObjB = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjB"), RF_Transient);
	const UObject* ObjC = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjC"), RF_Transient);
	const UObject* ObjD = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjD"), RF_Transient);

	// Validate that redirection creation order doesn't affect the final list of children
	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	// B -> A
	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjB);
	// C -> B -> A
	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjC);
	// D -> C -> B -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfC = RedirectionMap.Find(ObjC);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfC);
		CHECK_SUCCESS(ChildrenOfA->Num() == 3);
		CHECK_SUCCESS(ChildrenOfB->Num() == 2);
		CHECK_SUCCESS(ChildrenOfC->Num() == 1);
		CHECK_SUCCESS(Logger.FindRedirection(ObjB) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjC) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjD) == ObjA);
	}

	Logger.Cleanup(World);

	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjC);
	// D -> C
	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjB);
	// D -> C -> B
	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	// D -> C -> B -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfC = RedirectionMap.Find(ObjC);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfC);
		CHECK_SUCCESS(ChildrenOfA->Num() == 3);
		CHECK_SUCCESS(ChildrenOfB->Num() == 2);
		CHECK_SUCCESS(ChildrenOfC->Num() == 1);
		CHECK_SUCCESS(Logger.FindRedirection(ObjB) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjC) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjD) == ObjA);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogRedirectionsWithinHierarchyTest, "System.Engine.VisualLogger.Redirections.WithinHierarchy", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogRedirectionsWithinHierarchyTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FVisualLogger& Logger = FVisualLogger::Get();
	Logger.Cleanup(World);
	FVisualLogger::FOwnerToChildrenRedirectionMap& RedirectionMap = Logger.GetRedirectionMap(World);
	CHECK_SUCCESS(RedirectionMap.Num() == 0);

	const UObject* ObjA = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjA"), RF_Transient);
	const UObject* ObjB = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjB"), RF_Transient);
	const UObject* ObjC = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjC"), RF_Transient);
	const UObject* ObjD = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjD"), RF_Transient);
	const UObject* ObjE = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjE"), RF_Transient);

	REDIRECT_OBJECT_TO_VLOG(ObjE, ObjD);
	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjB);
	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjA);
	// E -> D -> B -> A
	//           C -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfD = RedirectionMap.Find(ObjD);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfD);
		CHECK_SUCCESS(ChildrenOfA->Num() == 4);
		CHECK_SUCCESS(ChildrenOfB->Num() == 2);
		CHECK_SUCCESS(ChildrenOfD->Num() == 1);
		CHECK_SUCCESS(Logger.FindRedirection(ObjB) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjC) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjD) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjE) == ObjA);
	}

	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjC);
	//           B -> A
	// E -> D -> C -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfC = RedirectionMap.Find(ObjC);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfD = RedirectionMap.Find(ObjD);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfC);
		CHECK_NOT_NULL(ChildrenOfD);
		CHECK_SUCCESS(ChildrenOfA->Num() == 4);
		CHECK_SUCCESS(ChildrenOfB->Num() == 0);
		CHECK_SUCCESS(ChildrenOfC->Num() == 2);
		CHECK_SUCCESS(ChildrenOfD->Num() == 1);
		CHECK_SUCCESS(Logger.FindRedirection(ObjB) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjC) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjD) == ObjA);
		CHECK_SUCCESS(Logger.FindRedirection(ObjE) == ObjA);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVisualLogRedirectionsDeepHierarchyToNewParentTest, "System.Engine.VisualLogger.Redirections.DeepHierarchyToNewParent", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FVisualLogRedirectionsDeepHierarchyToNewParentTest::RunTest(const FString& Parameters)
{
	UWorld* World = GetSimpleEngineAutomationTestWorld(GetTestFlags());
	CHECK_NOT_NULL(World);

	FVisualLogger& Logger = FVisualLogger::Get();
	Logger.Cleanup(World);
	FVisualLogger::FOwnerToChildrenRedirectionMap& RedirectionMap = Logger.GetRedirectionMap(World);

	const UObject* ObjA = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjA"), RF_Transient);
	const UObject* ObjB = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjB"), RF_Transient);
	const UObject* ObjC = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjC"), RF_Transient);
	const UObject* ObjD = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjD"), RF_Transient);
	const UObject* ObjE = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjE"), RF_Transient);
	const UObject* ObjF = NewObject<AActor>(World->GetCurrentLevel(), TEXT("VLogTestObjF"), RF_Transient);

	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjA);
	REDIRECT_OBJECT_TO_VLOG(ObjC, ObjB);
	REDIRECT_OBJECT_TO_VLOG(ObjD, ObjC);
	REDIRECT_OBJECT_TO_VLOG(ObjE, ObjD);
	// E -> D -> C -> B -> A
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfC = RedirectionMap.Find(ObjC);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfD = RedirectionMap.Find(ObjD);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfC);
		CHECK_NOT_NULL(ChildrenOfD);
		CHECK_SUCCESS(ChildrenOfA->Num() == 4);
		CHECK_SUCCESS(ChildrenOfB->Num() == 3);
		CHECK_SUCCESS(ChildrenOfC->Num() == 2);
		CHECK_SUCCESS(ChildrenOfD->Num() == 1);
	}
	
	REDIRECT_OBJECT_TO_VLOG(ObjB, ObjF);
	// E -> D -> C -> B -> F
	{
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfA = RedirectionMap.Find(ObjA);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfB = RedirectionMap.Find(ObjB);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfC = RedirectionMap.Find(ObjC);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfD = RedirectionMap.Find(ObjD);
		TArray<TWeakObjectPtr<const UObject>>* ChildrenOfF = RedirectionMap.Find(ObjF);
		CHECK_NOT_NULL(ChildrenOfA);
		CHECK_NOT_NULL(ChildrenOfB);
		CHECK_NOT_NULL(ChildrenOfC);
		CHECK_NOT_NULL(ChildrenOfD);
		CHECK_NOT_NULL(ChildrenOfF);
		CHECK_SUCCESS(ChildrenOfA->Num() == 0);
		CHECK_SUCCESS(ChildrenOfF->Num() == 4);
		CHECK_SUCCESS(ChildrenOfB->Num() == 3);
		CHECK_SUCCESS(ChildrenOfC->Num() == 2);
		CHECK_SUCCESS(ChildrenOfD->Num() == 1);
	}

	return true;
}

#undef CHECK_SUCCESS
#undef CHECK_FAIL

#endif // UE_DEBUG_RECORDING_USING_VLOG
