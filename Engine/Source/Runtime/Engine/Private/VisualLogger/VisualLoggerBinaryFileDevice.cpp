// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLoggerBinaryFileDevice.h"

#if UE_DEBUG_RECORDING_USING_VLOG
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "VisualLogger/VisualLogger.h"

FVisualLoggerBinaryFileDevice::FVisualLoggerBinaryFileDevice()
	: FileArchive(nullptr)
{
	Cleanup();

	bool DefaultFrameCacheLenght = 0;
	GConfig->GetBool(TEXT("VisualLogger"), TEXT("FrameCacheLenght"), DefaultFrameCacheLenght, GEngineIni);
	FrameCacheLenght = DefaultFrameCacheLenght;

	bool UseCompression = false;
	GConfig->GetBool(TEXT("VisualLogger"), TEXT("UseCompression"), UseCompression, GEngineIni);
	bUseCompression = UseCompression;
}

void FVisualLoggerBinaryFileDevice::Cleanup(bool bReleaseMemory)
{

}

void FVisualLoggerBinaryFileDevice::StartRecordingToFile(double TimeStamp)
{
	if (FileArchive != nullptr)
	{
		return;
	}

	// start new session
	SessionGUID = FGuid::NewGuid();

	StartRecordingTime = TimeStamp;
	LastLogTimeStamp = StartRecordingTime;
	TempFileName = FVisualLoggerHelpers::GenerateTemporaryFilename(VISLOG_FILENAME_EXT);
	
	const FString FullFilename = FPaths::Combine(*FPaths::ProjectLogDir(), *TempFileName);
	FileArchive = IFileManager::Get().CreateFileWriter(*FullFilename);
}

void FVisualLoggerBinaryFileDevice::StopRecordingToFile(double TimeStamp)
{
	if (FileArchive == nullptr)
	{
		return;
	}

	const int32 NumEntries = FrameCache.Num();
	if (NumEntries> 0)
	{
		FVisualLoggerHelpers::Serialize(*FileArchive, FrameCache);
		FrameCache.Reset();
	}

	const int64 TotalSize = FileArchive->TotalSize();
	FileArchive->Close();
	delete FileArchive;
	FileArchive = nullptr;

	const FString TempFullFilename = FPaths::Combine(*FPaths::ProjectLogDir(), *TempFileName);
	const FString NewFileName = FString::Printf(TEXT("%u_%s"), GetShortSessionID(), *FVisualLoggerHelpers::GenerateFilename(TempFileName, FileName, StartRecordingTime, LastLogTimeStamp));
	const FString NewFullFileName = FPaths::Combine(*FPaths::ProjectLogDir(), *NewFileName);

	if (TotalSize > 0)
	{
		// rename file when we serialized some data
		IFileManager::Get().Move(*NewFullFileName, *TempFullFilename, true);

		UE_LOGF(LogVisual, Display, "Vislog file saved: %ls", *NewFullFileName);
	}
	else
	{
		// or remove file if nothing serialized
		IFileManager::Get().Delete(*TempFullFilename, false, true, true);
	}
}

void FVisualLoggerBinaryFileDevice::DiscardRecordingToFile()
{
	if (FileArchive)
	{
		FileArchive->Close();
		delete FileArchive;
		FileArchive = nullptr;

		const FString TempFullFilename = FPaths::Combine(*FPaths::ProjectLogDir(), *TempFileName);
		IFileManager::Get().Delete(*TempFullFilename, false, true, true);
	}
}

void FVisualLoggerBinaryFileDevice::SetFileName(const FString& InFileName)
{
	FileName = InFileName;
}

void FVisualLoggerBinaryFileDevice::Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry)
{
	if (FileArchive)
	{
		const int32 NumEntries = FrameCache.Num();
		if (NumEntries> 0 && LastLogTimeStamp + FrameCacheLenght <= InLogEntry.TimeStamp)
		{
			FVisualLoggerHelpers::Serialize(*FileArchive, FrameCache);
			FrameCache.Reset();
			LastLogTimeStamp = InLogEntry.TimeStamp;
		}

		FrameCache.Add(FVisualLogEntryItem(InOwnerName, InOwnerDisplayName, InOwnerClassName, InLogEntry));
	}
}
#endif // UE_DEBUG_RECORDING_USING_VLOG
