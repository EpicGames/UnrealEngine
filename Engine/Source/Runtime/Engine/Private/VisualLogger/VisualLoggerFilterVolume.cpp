// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLoggerFilterVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerFilterVolume)

AVisualLoggerFilterVolume::AVisualLoggerFilterVolume(const FObjectInitializer& ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = false;
	bIsEditorOnlyActor = true;
}
