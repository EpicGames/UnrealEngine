// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLoggerKismetLibrary.h"
#include "VisualLogger/VisualLogger.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerKismetLibrary)

#define VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, Format, ...) \
	if (IsInGameThread())\
	{\
		FMessageLog(CategoryName).Info(FText::FromString(FString::Printf((Format), __VA_ARGS__)));\
	}\
	else\
	{\
		FMsg::Logf(__FILE__, __LINE__, UE_FNAME_TO_LOG_CATEGORY_NAME(CategoryName), DefaultVerbosity, (Format), __VA_ARGS__);\
	}

void UVisualLoggerKismetLibrary::RedirectVislog(UObject* SourceOwner, UObject* DestinationOwner)
{
	if (SourceOwner && DestinationOwner)
	{
		REDIRECT_OBJECT_TO_VLOG(SourceOwner, DestinationOwner);
	}
}

void UVisualLoggerKismetLibrary::EnableRecording(bool bEnabled)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::Get().SetIsRecording(bEnabled);
#endif // UE_DEBUG_RECORDING_ENABLED
}

void UVisualLoggerKismetLibrary::LogText(UObject* WorldContextObject, FString Text, FName CategoryName, bool bAddToMessageLog)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::CategorizedLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogText: '%s'"), *Text);
	}
}

void UVisualLoggerKismetLibrary::LogLocation(UObject* WorldContextObject, FVector Location, FString Text, FLinearColor ObjectColor, float Radius, FName CategoryName, bool bAddToMessageLog)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::LocationLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Location, static_cast<uint16>(Radius), ObjectColor.ToFColor(true), TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogLocation: '%s' - Location: (%s)"), *Text, *Location.ToString());
	}
}

void UVisualLoggerKismetLibrary::LogSphere(UObject* WorldContextObject, FVector Center, float Radius, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::SphereLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Center, Radius, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogSphere: '%s' - Center: (%s) | Radius: %f"), *Text, *Center.ToString(), Radius);
	}
}

void UVisualLoggerKismetLibrary::LogCone(UObject* WorldContextObject, FVector Origin, FVector Direction, float Length, float Angle, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::ConeLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Origin, Direction, Length, Angle, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogCone: '%s' - Origin: (%s) | Direction: (%s) | Length: %f | Angle: %f"), *Text, *Origin.ToString(), *Direction.ToString(), Length, Angle);
	}
}

void UVisualLoggerKismetLibrary::LogCylinder(UObject* WorldContextObject, FVector Start, FVector End, float Radius, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::CylinderLogf(WorldContextObject, CategoryName, DefaultVerbosity, Start, End, Radius, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogCylinder: '%s' - Start: (%s) | End: (%s) | Radius: %f"), *Text, *Start.ToString(), *End.ToString(), Radius);
	}
}

void UVisualLoggerKismetLibrary::LogCapsule(UObject* WorldContextObject, FVector Base, float HalfHeight, float Radius, FQuat Rotation, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::CapsuleLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Base, HalfHeight, Radius, Rotation, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogCapsule: '%s' - Base: (%s) | HalfHeight: %f | Radius: %f | Rotation: (%s)"), *Text, *Base.ToString(), HalfHeight, Radius, *Rotation.ToString());
	}
}

void UVisualLoggerKismetLibrary::LogBox(UObject* WorldContextObject, FBox Box, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::BoxLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Box, FMatrix::Identity, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogBox: '%s' - BoxMin: (%s) | BoxMax: (%s)"), *Text, *Box.Min.ToString(), *Box.Max.ToString());
	}
}

void UVisualLoggerKismetLibrary::LogOrientedBox(UObject* WorldContextObject, FBox Box, FTransform Transform, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::BoxLogf(WorldContextObject, CategoryName, DefaultVerbosity
		, Box, Transform.ToMatrixWithScale(), ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogOrientedBox: '%s' - BoxMin: (%s) | BoxMax: (%s) | Transform: (%s)"), *Text, *Box.Min.ToString(), *Box.Max.ToString(), *Transform.ToString());
	}
}

void UVisualLoggerKismetLibrary::LogArrow(UObject* WorldContextObject, const FVector SegmentStart, const FVector SegmentEnd, FString Text, FLinearColor ObjectColor, FName CategoryName, bool bAddToMessageLog, float ArrowHeadSize)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::ArrowLineLogf(WorldContextObject, CategoryName, DefaultVerbosity, SegmentStart, SegmentEnd, ObjectColor.ToFColor(true), ArrowHeadSize, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogArrow: '%s' - SegmentStart: (%s) | SegmentEnd:(%s)"), *Text, *SegmentStart.ToString(), *SegmentStart.ToString());
	}
}

void UVisualLoggerKismetLibrary::LogCircle(UObject* WorldContextObject, FVector Center, FVector UpAxis, float Radius, FString Text, FLinearColor ObjectColor, const float Thickness, FName CategoryName, bool bAddToMessageLog, bool bWireframe)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::DiscLogf(WorldContextObject, CategoryName, DefaultVerbosity, Center, UpAxis, Radius, ObjectColor.ToFColor(true), static_cast<uint16>(Thickness), bWireframe, TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogCircle: '%s' - Center: (%s) | UpAxis: (%s) | Radius: %f")	, *Text, *Center.ToString(), *UpAxis.ToString(), Radius);
	}
}

void UVisualLoggerKismetLibrary::LogSegment(UObject* WorldContextObject, const FVector SegmentStart, const FVector SegmentEnd, FString Text, FLinearColor ObjectColor, const float Thickness, FName CategoryName, bool bAddToMessageLog)
{
#if UE_DEBUG_RECORDING_ENABLED
	FVisualLogger::SegmentLogf(WorldContextObject, CategoryName, DefaultVerbosity, SegmentStart, SegmentEnd, ObjectColor.ToFColor(true), static_cast<uint16>(Thickness), TEXT("%s"), *Text);
#endif
	if (bAddToMessageLog)
	{
		VLOG_BP_LIBRARY_ADD_TO_LOG(CategoryName, TEXT("LogSegment: '%s' - SegmentStart: (%s) | SegmentEnd: (%s)")
			, *Text, *SegmentStart.ToString(), *SegmentStart.ToString());
	}
}

#undef VLOG_BP_LIBRARY_ADD_TO_LOG
