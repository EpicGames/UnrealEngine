// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLogger/VisualLoggerSettings.h"

#if UE_DEBUG_RECORDING_ENABLED
#include "VisualLogger/VisualLogger.h"

void UVisualLoggerSettings::SetUseVerbosityFilterWhenRecording(const bool bEnable)
{
	bUseVerbosityFilterWhenRecording = bEnable;
	FVisualLogger::bUseVerbosityFilterWhenRecording = bUseVerbosityFilterWhenRecording;
}

void UVisualLoggerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	FVisualLogger::bUseVerbosityFilterWhenRecording = bUseVerbosityFilterWhenRecording;
}

#if WITH_EDITOR
void UVisualLoggerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_UseVerbosityFilterWhenRecording = GET_MEMBER_NAME_CHECKED(UVisualLoggerSettings, bUseVerbosityFilterWhenRecording);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == NAME_UseVerbosityFilterWhenRecording)
	{
		SetUseVerbosityFilterWhenRecording(bUseVerbosityFilterWhenRecording);
	}
}
#endif // WITH_EDITOR
#endif // UE_DEBUG_RECORDING_ENABLED
