// Copyright Epic Games, Inc. All Rights Reserved.
#include "VisualLogger/VisualLoggerTraceDevice.h"

#if UE_DEBUG_RECORDING_ENABLED

#include "Serialization/BufferArchive.h"
#include "ObjectTrace.h"
#include "Trace/Trace.inl"
#include "VisualLogger/VisualLoggerCustomVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"

UE_TRACE_MINIMAL_CHANNEL_DEFINE(VisualLoggerChannel, "Visual Logger entries (owner object, timing, serialized log data) recorded into the trace stream. Allows reviewing visual log data in Unreal Insights instead of a standalone .vlog file.");

UE_TRACE_MINIMAL_EVENT_BEGIN(VisualLogger, VisualLogEntryHeader)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, OwnerID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint16, TotalChunkNum)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Size)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(VisualLogger, VisualLogEntryChunk)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint16, ChunkNum)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint16, Size)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8[], Data)
UE_TRACE_MINIMAL_EVENT_END()

FVisualLoggerTraceDevice& FVisualLoggerTraceDevice::Get()
{
	static FVisualLoggerTraceDevice GDevice;
	return GDevice;
}

FVisualLoggerTraceDevice::FVisualLoggerTraceDevice()
{

}

void FVisualLoggerTraceDevice::Cleanup(bool bReleaseMemory)
{

}

void FVisualLoggerTraceDevice::StartRecordingToFile(double TimeStamp)
{
#if UE_TRACE_MINIMAL_ENABLED
	UE::Trace::ToggleChannel(TEXT("VisualLogger"), true);
#endif
}

void FVisualLoggerTraceDevice::StopRecordingToFile(double TimeStamp)
{
#if UE_TRACE_MINIMAL_ENABLED
	UE::Trace::ToggleChannel(TEXT("VisualLogger"), false);
#endif
}

void FVisualLoggerTraceDevice::DiscardRecordingToFile()
{
}

void FVisualLoggerTraceDevice::SetFileName(const FString& InFileName)
{
}

void FVisualLoggerTraceDevice::Serialize(const UObject* InLogOwner, const FName& InOwnerName, const FName& InOwnerDisplayName, const FName& InOwnerClassName, const FVisualLogEntry& InLogEntry)
{
#if OBJECT_TRACE_ENABLED
	if (UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(VisualLoggerChannel))
	{
		TRACE_OBJECT(InLogOwner);

		FBufferArchive Archive;
		Archive.Reserve(1024);
		Archive.UsingCustomVersion(EVisualLoggerVersion::GUID);
		Archive.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		Archive.SetCustomVersion(FUE5MainStreamObjectVersion::GUID, FUE5MainStreamObjectVersion::LatestVersion, "UE5MainStreamObjectVersion");

		Archive << const_cast<FVisualLogEntry&>(InLogEntry);

		static std::atomic<uint32> EntryID = 0;
		const uint32 ID = AutoRTFM::Open([&] { return EntryID.fetch_add(1); });

		const uint32 DataSize = static_cast<uint32>(Archive.Num());
		constexpr uint32 MaxChunkSize = TNumericLimits<uint16>::Max();
		const uint16 ChunkNum = (DataSize + MaxChunkSize - 1) / MaxChunkSize;

		UE_TRACE_MINIMAL_LOG(VisualLogger, VisualLogEntryHeader, VisualLoggerChannel)
			<< VisualLogEntryHeader.ID(ID)
			<< VisualLogEntryHeader.Cycle(FPlatformTime::Cycles64())
			<< VisualLogEntryHeader.RecordingTime(FObjectTrace::GetWorldElapsedTime(InLogOwner->GetWorld()))
			<< VisualLogEntryHeader.OwnerID(FObjectTrace::GetObjectId(InLogOwner))
			<< VisualLogEntryHeader.TotalChunkNum(ChunkNum)
			<< VisualLogEntryHeader.Size(DataSize);

		uint32 RemainingSize = DataSize;
		for (uint32 Index = 0; Index < ChunkNum; ++Index)
		{
			const uint16 Size = static_cast<uint16>(FMath::Min(RemainingSize, MaxChunkSize));
			uint8* ChunkData = Archive.GetData() + MaxChunkSize * Index;

			UE_TRACE_MINIMAL_LOG(VisualLogger, VisualLogEntryChunk, VisualLoggerChannel)
				<< VisualLogEntryChunk.ID(ID)
				<< VisualLogEntryChunk.ChunkNum(Index)
				<< VisualLogEntryChunk.Size(Size)
				<< VisualLogEntryChunk.Data(ChunkData, Size);

			RemainingSize -= Size;
		}

		check(RemainingSize == 0);
	}
#endif

	ImmediateRenderDelegate.ExecuteIfBound(InLogOwner, InLogEntry);
}

bool FVisualLoggerTraceDevice::HasFlags(int32 InFlags) const
{
	return !!(InFlags & (EVisualLoggerDeviceFlags::CanSaveToFile | EVisualLoggerDeviceFlags::StoreLogsLocally));
}

#endif // UE_DEBUG_RECORDING_ENABLED
