// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VoiceChannel.cpp: Unreal voice traffic implementation.
=============================================================================*/

#include "Engine/VoiceChannel.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "Engine/NetConnection.h"
#include "Net/OnlineEngineInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoiceChannel)

/** Cleans up any voice data remaining in the queue */
bool UVoiceChannel::CleanUp( const bool bForDestroy, EChannelCloseReason CloseReason )
{
	// Clear out refs to any voice packets so we don't leak
	VoicePackets.Empty();
	// Route to the parent class for their cleanup
	return Super::CleanUp( bForDestroy, CloseReason );
}

/**
 * Processes the in bound bunch to extract the voice data
 *
 * @param Bunch the voice data to process
 */
void UVoiceChannel::ReceivedBunch(FInBunch& Bunch)
{
	if (Connection->Driver && Connection->Driver->World)
	{
		while (!Bunch.AtEnd())
		{
			// Give the data to the local voice processing
			TSharedPtr<FVoicePacket> VoicePacket = UOnlineEngineInterface::Get()->SerializeRemotePacket(Connection->Driver->World, Connection, Bunch);
			if (VoicePacket.IsValid())
			{
				if (Connection->Driver->ServerConnection == NULL)
				{
					// Possibly replicate the data to other clients
					Connection->Driver->ReplicateVoicePacket(VoicePacket, Connection);
				}
#if STATS
				// Increment the number of voice packets we've received
				Connection->Driver->VoicePacketsRecv++;
				Connection->Driver->VoiceBytesRecv += VoicePacket->GetBufferSize();
#endif
			}
			else
			{
				// Unable to serialize the data because the serializer doesn't exist or there was a problem with this packet
				Bunch.SetError();
				break;
			}
		}
	}
}

/**
 * Performs any per tick update of the VoIP state
 */
void UVoiceChannel::Tick()
{
	const bool bHandshakeCompleted = Connection->PlayerController && Connection->PlayerController->MuteList.bHasVoiceHandshakeCompleted;

	// If the handshaking hasn't completed throw away all unreliable voice data
	if (bHandshakeCompleted)
	{
		// Try to append each packet in turn
		int32 Index;
		for (Index = 0; Index < VoicePackets.Num(); Index++)
		{
			if (!Connection->IsNetReady())
			{
				// If the network is saturated bail early
				UE_LOGF(LogNet, Warning, "Network saturated");
				break;
			}

			FOutBunch Bunch(this, 0);

			TSharedPtr<FVoicePacket> Packet = VoicePackets[Index];

			// First send must be reliable as must any packet marked reliable
			Bunch.bReliable = OpenAcked == false || Packet->IsReliable();

			// Append the packet data (copies into the bunch)
			Packet->Serialize(Bunch);

#if STATS
			// Increment the number of voice packets we've sent
			Connection->Driver->VoicePacketsSent++;
			Connection->Driver->VoiceBytesSent += Packet->GetBufferSize();
#endif
			// Don't submit the bunch if something went wrong
			if (Bunch.IsError() == false)
			{
				// Submit the bunching with merging on
				SendBunch(&Bunch, 1);
			}
			else
			{
				//bail and try again next frame
				UE_LOGF(LogNet, Warning, "Bunch error");
				break;
			}
		}

		if (Index >= VoicePackets.Num())
		{
			// all sent, can throw everything away
			VoicePackets.Empty();
		}
		else if (Index > 0)
		{
			// didn't send everything, just remove all packets actually sent
			VoicePackets.RemoveAt(0,Index);
		}
	}

	// make sure we keep any reliable messages to try again next time
	// but ditch any unreliable messages we've not managed to send
	int32 PacketLoss = 0;
	for (int32 i = VoicePackets.Num() - 1; i >= 0; i--)
	{
		if (!VoicePackets[i]->IsReliable())
		{
			VoicePackets.RemoveAt(i,EAllowShrinking::No);
			PacketLoss++;
		}
	}
	if (bHandshakeCompleted && PacketLoss > 0)
	{
		UE_LOGF(LogNet, Log, "Dropped %d packets due to congestion in the voicechannel", PacketLoss);
	}
}

/**
 * Adds the voice packet to the list to send for this channel
 *
 * @param VoicePacket the voice packet to send
 */
void UVoiceChannel::AddVoicePacket(TSharedPtr<FVoicePacket> VoicePacket)
{
	if (VoicePacket.IsValid())
	{
		VoicePackets.Add(VoicePacket);

		UE_LOGF(LogNet, VeryVerbose, "AddVoicePacket: %ls [%ls] to=%ls from=%ls",
			*Connection->PlayerId.ToDebugString(),
			*Connection->Driver->GetDescription(),
			*Connection->LowLevelDescribe(),
			*VoicePacket->GetSender()->ToDebugString());
	}
}

