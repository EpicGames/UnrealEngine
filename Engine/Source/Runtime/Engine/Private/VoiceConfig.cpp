// Copyright Epic Games, Inc. All Rights Reserved.

#include "Net/VoiceConfig.h"
#include "Engine/World.h"
#include "GameFramework/GameStateBase.h"
#include "GameFramework/PlayerState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoiceConfig)

TMap<FUniqueNetIdWrapper, UVOIPTalker*> UVOIPStatics::VoiceTalkerMap;

static TAutoConsoleVariable<float> CVarVoiceSilenceDetectionAttackTime(TEXT("voice.SilenceDetectionAttackTime"),
	2.0f,
	TEXT("Attack time to be set for the VOIP microphone's silence detection algorithm in milliseconds.\n"),	
	ECVF_Default);

static TAutoConsoleVariable<float> CVarVoiceSilenceDetectionReleaseTime(TEXT("voice.SilenceDetectionReleaseTime"),
	1100.0f,
	TEXT("Release time to be set for the VOIP microphone's silence detection algorithm in milliseconds.\n"),	
	ECVF_Default);

static TAutoConsoleVariable<float> CVarVoiceSilenceDetectionThreshold(TEXT("voice.SilenceDetectionThreshold"),
	0.08f,
	TEXT("Threshold to be set for the VOIP microphone's silence detection algorithm.\n"),	
	ECVF_Default);

static float MicInputGainCvar = 1.0f;
FAutoConsoleVariableRef CVarMicInputGain(
	TEXT("voice.MicInputGain"),
	MicInputGainCvar,
	TEXT("The default gain amount in linear amplitude.\n")
	TEXT("Value: Gain multiplier."),
	ECVF_Default);

static float MicStereoBiasCvar = 0.0f;
FAutoConsoleVariableRef CVarMicStereoBias(
	TEXT("voice.MicStereoBias"),
	MicStereoBiasCvar,
	TEXT("This will attenuate the left or right channel.\n")
	TEXT("0.0: Centered. 1.0: right channel only. -1.0: Left channel only."),
	ECVF_Default);

static float JitterBufferDelayCvar = 0.3f;
FAutoConsoleVariableRef CVarJitterBufferDelay(
	TEXT("voice.JitterBufferDelay"),
	JitterBufferDelayCvar,
	TEXT("The default amount of audio we buffer, in seconds, before we play back audio. Decreasing this value will decrease latency but increase the potential for underruns.\n")
	TEXT("Value: Number of seconds of audio we buffer."),
	ECVF_Default);

static float MicNoiseGateThresholdCvar = 0.08f;
FAutoConsoleVariableRef CVarMicNoiseGateThreshold(
	TEXT("voice.MicNoiseGateThreshold"),
	MicNoiseGateThresholdCvar,
	TEXT("Our threshold, in linear amplitude, for  our noise gate on input. Similar to voice.SilenceDetectionThreshold, except that audio quieter than our noise gate threshold will still output silence.\n")
	TEXT("Value: Number of seconds of audio we buffer."),
	ECVF_Default);

static float MicNoiseGateAttackTimeCvar = 0.05f;
FAutoConsoleVariableRef CVarMicNoiseGateAttackTime(
	TEXT("voice.MicNoiseAttackTime"),
	MicNoiseGateAttackTimeCvar,
	TEXT("Sets the fade-in time for our noise gate.\n")
	TEXT("Value: Number of seconds we fade in over."),
	ECVF_Default);

static float MicNoiseGateReleaseTimeCvar = 0.30f;
FAutoConsoleVariableRef CVarMicNoiseGateReleaseTime(
	TEXT("voice.MicNoiseReleaseTime"),
	MicNoiseGateReleaseTimeCvar,
	TEXT("Sets the fade out time for our noise gate.\n")
	TEXT("Value: Number of seconds we fade out over."),
	ECVF_Default);

static int32 NumVoiceChannelsCvar = 1;
FAutoConsoleVariableRef CVarNumVoiceChannels(
	TEXT("voice.NumChannels"),
	NumVoiceChannelsCvar,
	TEXT("Default number of channels to capture from mic input, encode to Opus, and output. Can be set to 1 or 2.\n")
	TEXT("Value: Number of channels to use for VOIP input and output."),
	ECVF_Default);

int32 UVOIPStatics::GetVoiceSampleRate()
{
#if PLATFORM_UNIX
	return PLATFORM_LINUX ? 16000 : 48000;
#elif USE_DEFAULT_VOICE_SAMPLE_RATE
	return (int32) 16000;
#else
	static bool bRetrievedSampleRate = false;
	static int32 SampleRate = 0;

	if (bRetrievedSampleRate)
	{
		return SampleRate;
	}

	static FString DesiredSampleRateStr;
	
	if (GConfig->GetString(TEXT("/Script/Engine.AudioSettings"), TEXT("VoiPSampleRate"), DesiredSampleRateStr, GEngineIni))
	{
		if (DesiredSampleRateStr.Equals(TEXT("Low16000Hz")))
		{
			SampleRate = (int32)EVoiceSampleRate::Low16000Hz;
		}
		else if (DesiredSampleRateStr.Equals(TEXT("Normal24000Hz")))
		{
			SampleRate = (int32)EVoiceSampleRate::Normal24000Hz;
		}

		if (SampleRate > 0)
		{		
			bRetrievedSampleRate = true;
			return SampleRate;
		}
	}

	// If we've made it here, we couldn't find a specified sample rate from properties.
	return SampleRate = (int32) EVoiceSampleRate::Low16000Hz;
#endif
}

int32 UVOIPStatics::GetVoiceNumChannels()
{
	return FMath::Clamp<int32>(NumVoiceChannelsCvar, 1, 2);
}

uint32 UVOIPStatics::GetMaxVoiceDataSize()
{
	int32 SampleRate = GetVoiceSampleRate();
	// This max voice data size is based on approximations of how large the
	// encoded opus audio buffer will be.
	switch (SampleRate)
	{
		case 24000:
		{
			return 14 * 1024 * GetVoiceNumChannels();
		}
		case 48000:
		{
			return 32 * 1024 * GetVoiceNumChannels();
		}
		default:
		case 16000:
		{
			return 8 * 1024 * GetVoiceNumChannels();
		}
	}
}

uint32 UVOIPStatics::GetMaxUncompressedVoiceDataSizePerChannel()
{
	// Maximum amount of input samples we allocate for every time we get input audio from the mic on the game thread.
	// This amounts to approximates a second of audio for the minimum voice engine tick frequency.
	// At 48 kHz, DirectSound will occasionally have to load up to 256 samples into the overflow buffer.
	// This is the reason for the added 1024 bytes.
	return GetVoiceSampleRate() * sizeof(uint16) * GetVoiceNumChannels() + 1024 * GetVoiceNumChannels();
}

uint32 UVOIPStatics::GetMaxCompressedVoiceDataSize()
{
	int32 SampleRate = GetVoiceSampleRate();
	// These values are carried over from MAX_COMPRESSED_VOICE_DATA_SIZE in previous revisions
	// of the Voice Engine code.
	switch (SampleRate)
	{
		case 24000:
		{
			return 2 * 1024 * GetVoiceNumChannels();
		}
		case 48000:
		{
			return 4 * 1024 * GetVoiceNumChannels();
		}
		default:
		case 16000:
		{
			return 1 * 1024 * GetVoiceNumChannels();
		}
	}
}

float UVOIPStatics::GetRemoteTalkerTimeoutDuration()
{
	return 1.0f;
}

EAudioEncodeHint UVOIPStatics::GetAudioEncodingHint()
{
	// This may be exposed as a project settings in the future.
	return EAudioEncodeHint::VoiceEncode_Voice;
}

float UVOIPStatics::GetBufferingDelay()
{
	return JitterBufferDelayCvar;
}

float UVOIPStatics::GetVoiceNoiseGateLevel()
{
	return MicNoiseGateThresholdCvar;
}

int32 UVOIPStatics::GetNumBufferedPackets()
{
	// Evaluate number of packets as the total number of samples we'll need to buffer divided by
	// the number of samples per buffer, plus an arbitrary amount of packets to compensate for jitter.
	static int32 NumBufferedPackets = (GetBufferingDelay() * GetVoiceSampleRate()) / (GetMaxVoiceDataSize() / sizeof(float)) + MicSilenceDetectionConfig::PacketBufferSlack;
	return NumBufferedPackets;
}

APlayerState* UVOIPStatics::GetPlayerStateFromUniqueNetId(UWorld* InWorld, const FUniqueNetIdWrapper& InPlayerId)
{
	AGameStateBase* InBase = InWorld->GetGameState();
	if (InBase)
	{
		return InBase->GetPlayerStateFromUniqueNetId(InPlayerId);
	}

	return nullptr;
}

void UVOIPStatics::SetMicThreshold(float InThreshold)
{
	static IConsoleVariable* SilenceDetectionCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.SilenceDetectionThreshold"));
	check(SilenceDetectionCVar);
	SilenceDetectionCVar->Set(InThreshold, ECVF_SetByGameSetting);
}

void UVOIPStatics::SetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InPlayerId, UVOIPTalker* InTalker)
{
	VoiceTalkerMap.FindOrAdd(InPlayerId);
	VoiceTalkerMap[InPlayerId] = InTalker;
}

UVOIPTalker* UVOIPStatics::GetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InUniqueId,  FVoiceSettings& OutSettings, UWorld* InWorld, APlayerState** OutPlayerState /*= nullptr*/)
{
	UVOIPTalker** FoundTalker = nullptr;

	if (InWorld != nullptr && OutPlayerState != nullptr)
	{
		*OutPlayerState = GetPlayerStateFromUniqueNetId(InWorld, InUniqueId);
	}

	FoundTalker = VoiceTalkerMap.Find(InUniqueId);
	
	if (FoundTalker != nullptr)
	{
		OutSettings = (*FoundTalker)->Settings;
		return *FoundTalker;
	}
	else
	{
		return nullptr;
	}
}

UVOIPTalker* UVOIPStatics::GetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InPlayerId)
{
	UVOIPTalker** FoundTalker = VoiceTalkerMap.Find(InPlayerId);

	if (FoundTalker)
	{
		return *FoundTalker;
	}
	else
	{
		return nullptr;
	}
}

bool UVOIPStatics::IsVOIPTalkerStillAlive(UVOIPTalker* InTalker)
{
	for (auto& TalkerElement : VoiceTalkerMap)
	{
		if(TalkerElement.Value == InTalker)
		{
			return true;
		}
	}

	return false;
}

void UVOIPStatics::ResetPlayerVoiceTalker(APlayerState* InPlayerState)
{
	if (InPlayerState && InPlayerState->GetUniqueId().IsValid())
	{
		VoiceTalkerMap.Remove(InPlayerState->GetUniqueId());
	}
}

void UVOIPStatics::ResetPlayerVoiceTalker(const FUniqueNetIdWrapper& InPlayerId)
{
	VoiceTalkerMap.Remove(InPlayerId);
}

void UVOIPStatics::ClearAllSettings()
{
	VoiceTalkerMap.Empty();
}

UVOIPTalker::UVOIPTalker(const FObjectInitializer& ObjectInitializer)
	: PlayerId()
	, CachedVolumeLevel(0.0f)
	, bIsRegistered(false)
{
}

UVOIPTalker::~UVOIPTalker()
{
	if (bIsRegistered)
	{
		UnregisterFromVoiceTalkerMap();
	}
}

UVOIPTalker* UVOIPTalker::CreateTalkerForPlayer(APlayerState* OwningState)
{
	UVOIPTalker* NewTalker = NewObject<UVOIPTalker>();
	if (NewTalker != nullptr)
	{
		NewTalker->RegisterWithPlayerState(OwningState);
	}

	return NewTalker;
}

void UVOIPTalker::RegisterWithPlayerState(APlayerState* OwningState)
{
	checkf(OwningState != nullptr, TEXT("RegisterWithPlayerState called with null pointer to OwningState"));
	if (bIsRegistered)
	{
		UnregisterFromVoiceTalkerMap();
	}

	if (OwningState->GetUniqueId().IsValid())
	{
		UVOIPStatics::SetVOIPTalkerForPlayer(OwningState->GetUniqueId(), this);
		PlayerId = OwningState->GetUniqueId();
		bIsRegistered = true;
	}
}

void UVOIPTalker::OnAudioComponentEnvelopeValue(const UAudioComponent* InAudioComponent, const float EnvelopeValue)
{
	CachedVolumeLevel = EnvelopeValue;
}

void UVOIPTalker::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (bIsRegistered)
	{
		UnregisterFromVoiceTalkerMap();
	}
}

float UVOIPTalker::GetVoiceLevel()
{
	return CachedVolumeLevel;
}

void UVOIPTalker::BPOnTalkingBegin_Implementation(UAudioComponent* AudioComponent)
{

}

void UVOIPTalker::BPOnTalkingEnd_Implementation()
{

}

void UVOIPTalker::UnregisterFromVoiceTalkerMap()
{
	UVOIPStatics::ResetPlayerVoiceTalker(PlayerId);
	bIsRegistered = false;
}

