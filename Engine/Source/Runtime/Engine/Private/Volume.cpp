// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Volume.cpp: AVolume and subclasses
=============================================================================*/

#include "GameFramework/Volume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Components/BrushComponent.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Volume)

#if WITH_EDITOR
/** Define static delegate */
AVolume::FOnVolumeShapeChanged AVolume::OnVolumeShapeChanged;
#endif

DEFINE_LOG_CATEGORY(LogVolume);

AVolume::AVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->AlwaysLoadOnClient = true;
	GetBrushComponent()->AlwaysLoadOnServer = true;
	static FName CollisionProfileName("OverlapAll");
	GetBrushComponent()->SetCollisionProfileName(CollisionProfileName);
	GetBrushComponent()->SetGenerateOverlapEvents(true);
	SetReplicatingMovement(false);
#if WITH_EDITORONLY_DATA
	bActorLabelEditable = true;
#endif // WITH_EDITORONLY_DATA

	SetCanBeDamaged(false);
}

#if WITH_EDITOR

void AVolume::PostEditImport()
{
	Super::PostEditImport();

	OnVolumeShapeChanged.Broadcast(*this);
}

void AVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NAME_BrushBuilder("BrushBuilder");

	// The brush builder that created this volume has changed. Notify listeners
	// Also notify on null property change events submitted during undo/redo
	if( PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive &&
		((GIsTransacting && !PropertyChangedEvent.MemberProperty) || (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == NAME_BrushBuilder)) )
	{
		OnVolumeShapeChanged.Broadcast(*this);
	}
}

FName AVolume::GetCustomIconName() const
{
	// Our parent class ABrush provides a default for all brushes (which includes volumes), but we don't want to use those since all volume types
	// have custom icons
	return NAME_None;
}

#endif // WITH_EDITOR

/** @returns the coarse bounds of this volume */
FBoxSphereBounds AVolume::GetBounds() const
{
	if (GetBrushComponent())
	{
		return GetBrushComponent()->CalcBounds(GetBrushComponent()->GetComponentTransform());
	}
	else
	{
		UE_LOGF(LogVolume, Error, "AVolume::GetBounds : No BrushComponent");
		return FBoxSphereBounds(ForceInitToZero);
	}
}

bool AVolume::EncompassesPoint(FVector Point, float SphereRadius/*=0.f*/, float* OutDistanceToPoint) const
{
	if (GetBrushComponent())
	{
#if 1
		FVector ClosestPoint;
		float DistanceSqr;

		if (GetBrushComponent()->GetSquaredDistanceToCollision(Point, DistanceSqr, ClosestPoint) == false)
		{
			if (OutDistanceToPoint)
			{
				*OutDistanceToPoint = -1.f;
			}
			return false;
		}
#else
		FBoxSphereBounds Bounds = GetBrushComponent()->CalcBounds(GetBrushComponent()->GetComponentTransform());
		const float DistanceSqr = Bounds.GetBox().ComputeSquaredDistanceToPoint(Point);
#endif

		if (OutDistanceToPoint)
		{
			*OutDistanceToPoint = FMath::Sqrt(DistanceSqr);
		}

		return DistanceSqr >= 0.f && DistanceSqr <= FMath::Square(SphereRadius);
	}
	else
	{
		UE_LOGF(LogVolume, Log, "AVolume::EncompassesPoint : No BrushComponent on %ls", *GetActorNameOrLabel());
		return false;
	}
}

bool AVolume::IsLevelBoundsRelevant() const
{
	return false;
}
	
bool AVolume::IsStaticBrush() const
{
	return false;
}

bool AVolume::IsVolumeBrush() const
{
	return true;
}





