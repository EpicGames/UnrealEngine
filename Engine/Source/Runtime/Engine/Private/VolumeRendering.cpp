// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumeRendering.cpp: Volume rendering implementation.
=============================================================================*/

#include "VolumeRendering.h"
#include "ScreenRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"

FWriteToSliceVS::FWriteToSliceVS() = default;

FWriteToSliceVS::FWriteToSliceVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	UVScaleBias.Bind(Initializer.ParameterMap, TEXT("UVScaleBias"));
	MinZ.Bind(Initializer.ParameterMap, TEXT("MinZ"));
}

bool FWriteToSliceVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

void FWriteToSliceVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	if (RHISupportsGeometryShaders(Parameters.Platform))
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_VertexToGeometryShader);
	}
}

IMPLEMENT_SHADER_TYPE(,FWriteToSliceVS,TEXT("/Engine/Private/TranslucentLightingShaders.usf"),TEXT("WriteToSliceMainVS"),SF_Vertex);


FWriteToSliceGS::FWriteToSliceGS() = default;

FWriteToSliceGS::FWriteToSliceGS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	MinZ.Bind(Initializer.ParameterMap, TEXT("MinZ"));
}

bool FWriteToSliceGS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && RHISupportsGeometryShaders(Parameters.Platform);
}

IMPLEMENT_SHADER_TYPE(, FWriteToSliceGS, TEXT("/Engine/Private/TranslucentLightingShaders.usf"), TEXT("WriteToSliceMainGS"), SF_Geometry);

void FVolumeRasterizeVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Used as a non-indexed triangle strip, so 4 vertices per quad
	const uint32 Size = 4 * sizeof(FScreenVertex);

	const FRHIBufferCreateDesc Desc =
		FRHIBufferCreateDesc::CreateVertex<FScreenVertex>(TEXT("FVolumeRasterizeVertexBuffer"), 4)
		.AddUsage(BUF_Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<FScreenVertex> Initializer = RHICmdList.CreateBufferInitializer(Desc);

	// Setup a full - render target quad
	// A viewport and UVScaleBias will be used to implement rendering to a sub region
	Initializer[0] = FScreenVertex { FVector2f( 1, -GProjectionSignY), FVector2f(1, 1) };
	Initializer[1] = FScreenVertex { FVector2f( 1,  GProjectionSignY), FVector2f(1, 0) };
	Initializer[2] = FScreenVertex { FVector2f(-1, -GProjectionSignY), FVector2f(0, 1) };
	Initializer[3] = FScreenVertex { FVector2f(-1,  GProjectionSignY), FVector2f(0, 0) };

	VertexBufferRHI = Initializer.Finalize();
}

TGlobalResource<FVolumeRasterizeVertexBuffer> GVolumeRasterizeVertexBuffer;

/** Draws a quad per volume texture slice to the subregion of the volume texture specified. */
ENGINE_API void RasterizeToVolumeTexture(FRHICommandList& RHICmdList, FVolumeBounds VolumeBounds)
{
	// Setup the viewport to only render to the given bounds
	RHICmdList.SetViewport(VolumeBounds.MinX, VolumeBounds.MinY, 0, VolumeBounds.MaxX, VolumeBounds.MaxY, 0);
	RHICmdList.SetStreamSource(0, GVolumeRasterizeVertexBuffer.VertexBufferRHI, 0);
	const int32 NumInstances = VolumeBounds.MaxZ - VolumeBounds.MinZ;
	// Render a quad per slice affected by the given bounds
	RHICmdList.DrawPrimitive(0, 2, NumInstances);
}
