// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumeTexture.cpp: UvolumeTexture implementation.
=============================================================================*/

#include "Engine/VolumeTexture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "EngineLogs.h"
#include "EngineUtils.h"
#include "ImageUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderUtils.h"
#include "Streaming/Texture2DMipDataProvider_DDC.h"
#include "Streaming/TextureStreamIn.h"
#include "Streaming/Texture2DMipDataProvider_IO.h"
#include "Streaming/TextureStreamOut.h"
#include "Streaming/VolumeTextureStreaming.h"
#include "TextureCompiler.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "ImageCoreUtils.h"
#include "Math/GuardedInt.h"

#if WITH_EDITOR
#include "AsyncCompilationHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeTexture)

#define LOCTEXT_NAMESPACE "UVolumeTexture"

//*****************************************************************************

// Externed global switch to control whether streaming is enabled for volume texture. 
bool GSupportsVolumeTextureStreaming = true;
static FAutoConsoleVariableRef CVarSupportsVolumeTextureStreaming(
	TEXT("r.SupportsVolumeTextureStreaming"),
	GSupportsVolumeTextureStreaming,
	TEXT("Enable Support of VolumeTexture Streaming\n")
);

// Limit the possible depth of volume texture otherwise when the user converts 2D textures, they can crash the engine.
const int32 MAX_VOLUME_TEXTURE_DEPTH = 512;

// Returns 0 if the product of A and B would overflow
static int32 CheckedNonNegativeProduct(int32 A, int32 B)
{
	FGuardedInt32 Product = FGuardedInt32(A) * FGuardedInt32(B);
	return Product.Get(0);
}

//*****************************************************************************
//***************************** UVolumeTexture ********************************
//*****************************************************************************

UVolumeTexture* UVolumeTexture::CreateTransient(int32 InSizeX, int32 InSizeY, int32 InSizeZ, EPixelFormat InFormat, const FName InName)
{
	UVolumeTexture* NewTexture = nullptr;
	if (InSizeX > 0 && InSizeY > 0 && InSizeZ > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0)
	{
		NewTexture = NewObject<UVolumeTexture>(
			GetTransientPackage(),
			InName,
			RF_Transient
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->SetNumSlices(InSizeZ);
		NewTexture->GetPlatformData()->PixelFormat = InFormat;
		NewTexture->bNotOfflineProcessed = true;

		// Allocate first mipmap.
		int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		int32 NumBlocksZ = InSizeZ / GPixelFormats[InFormat].BlockSizeZ;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY, InSizeZ);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc((int64)GPixelFormats[InFormat].BlockBytes * NumBlocksX * NumBlocksY * NumBlocksZ);
		Mip->BulkData.Unlock();
	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UVolumeTexture::CreateTransient()");
	}
	return NewTexture;
}

/**
 * Get the optimal placeholder to use during texture compilation
 */
static UVolumeTexture* GetDefaultVolumeTexture(const UVolumeTexture* Texture)
{
	static TStrongObjectPtr<UVolumeTexture> CheckerboardTexture;
	if (!CheckerboardTexture.IsValid())
	{
		CheckerboardTexture.Reset(FImageUtils::CreateCheckerboardVolumeTexture(FColor(200, 200, 200, 128), FColor(128, 128, 128, 128)));
	}
	return CheckerboardTexture.Get();
}

UVolumeTexture::UVolumeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PrivatePlatformData(nullptr)
{
	SRGB = true;
}

FTexturePlatformData** UVolumeTexture::GetRunningPlatformData()
{
	// @todo DC GetRunningPlatformData is fundamentally unsafe but almost unused... should we replace it with Get/SetRunningPlatformData directly in the base class
	return &PrivatePlatformData;
}

void UVolumeTexture::SetPlatformData(FTexturePlatformData* InPlatformData)
{
	if (PrivatePlatformData)
	{
		ReleaseResource();
		delete PrivatePlatformData;
	}

	PrivatePlatformData = InPlatformData;
}

// Any direct access to GetPlatformData() will stall until the structure
// is safe to use. It is advisable to replace those use case with
// async aware code to avoid stalls where possible.
const FTexturePlatformData* UVolumeTexture::GetPlatformData() const
{
#if WITH_EDITOR
	if (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UVolumeTexture::GetPlatformDataStall);
		UE_LOGF(LogTexture, Log, "Call to GetPlatformData() is forcing a wait on data that is not yet ready.");

		FText Msg = FText::Format(LOCTEXT("WaitOnTextureCompilation", "Waiting on texture compilation {0} ..."), FText::FromString(GetName()));
		FScopedSlowTask Progress(1.f, Msg, true);
		Progress.MakeDialog(true);
		uint64 StartTime = FPlatformTime::Cycles64();
		PrivatePlatformData->FinishCache();
		AsyncCompilationHelpers::SaveStallStack(FPlatformTime::Cycles64() - StartTime);
	}
#endif // #if WITH_EDITOR
	
	return PrivatePlatformData;
}

FTexturePlatformData* UVolumeTexture::GetPlatformData()
{
	// For now, this is the same implementation as the const version.
	const UVolumeTexture* ConstThis = this;
	return const_cast<FTexturePlatformData*>(ConstThis->GetPlatformData());
}

bool UVolumeTexture::UpdateSourceFromSourceTexture()
{
	bool bSourceValid = false;

#if WITH_EDITOR

	Modify(true);

	if (Source2DTexture && Source2DTileSizeX > 0 && Source2DTileSizeY > 0 && Source2DTexture->Source.IsValid())
	{
		FTextureSource& InitialSource = Source2DTexture->Source;
		const int32 Num2DTileX = InitialSource.GetSizeX() / Source2DTileSizeX;
		const int32 Num2DTileY = InitialSource.GetSizeY() / Source2DTileSizeY;
		const int32 TileSizeZ = FMath::Min<int32>(Num2DTileX * Num2DTileY, MAX_VOLUME_TEXTURE_DEPTH);
		if (TileSizeZ > 0)
		{
			const int32 FormatDataSize = InitialSource.GetBytesPerPixel();

			// Do an overflow checked product to avoid AllocationSize being smaller than the actual required memory which would cause
			// the following loops to write past the allocation. We also force allocations to be <2GB with this by using int32.
			const int32 AllocationSize = CheckedNonNegativeProduct(
				CheckedNonNegativeProduct(Source2DTileSizeX, Source2DTileSizeY), 
				CheckedNonNegativeProduct(TileSizeZ, FormatDataSize));

			if (AllocationSize > 0)
			{
				TArray64<uint8> Ref2DData;
				if (InitialSource.GetMipData(Ref2DData, 0))
				{
					uint8* NewData = (uint8*)FMemory::Malloc(AllocationSize);
					uint8* CurPos = NewData;
					
					for (int32 PosZ = 0; PosZ < TileSizeZ; ++PosZ)
					{
						const int32 RefTile2DPosX = (PosZ % Num2DTileX) * Source2DTileSizeX;
						const int32 RefTile2DPosY = ((PosZ / Num2DTileX) % Num2DTileY) * Source2DTileSizeY;

						for (int32 PosY = 0; PosY < Source2DTileSizeY; ++PosY)
						{
							const int32 Ref2DPosY = RefTile2DPosY + PosY; 

							for (int32 PosX = 0; PosX < Source2DTileSizeX; ++PosX)
							{
								const int32 Ref2DPosX = RefTile2DPosX + PosX; 
								const int32 RefPos = Ref2DPosX + Ref2DPosY * InitialSource.GetSizeX();
								FMemory::Memcpy(CurPos, Ref2DData.GetData() + RefPos * FormatDataSize, FormatDataSize);
								CurPos += FormatDataSize;
							}
						}
					}

					Source.Init(Source2DTileSizeX, Source2DTileSizeY, TileSizeZ, 1, InitialSource.GetFormat(), NewData);
					bSourceValid = true;

					FMemory::Free(NewData);
				}
			}
		}
	}

	if (bSourceValid)
	{
		SetLightingGuid(); // Because the content has changed, use a new GUID.
	}
	else
	{
		Source = FTextureSource();
		Source.SetOwner(this);
		// Source2DTexture = nullptr; // ??

		if (GetPlatformData())
		{
			SetPlatformData(new FTexturePlatformData());
		}
	}
	
	ValidateSettingsAfterImportOrEdit();
#endif // WITH_EDITOR

	return bSourceValid;
}

ENGINE_API bool UVolumeTexture::UpdateSourceFromFunction(TFunction<void(int32, int32, int32, void*)> Func, int32 SizeX, int32 SizeY, int32 SizeZ, ETextureSourceFormat Format)
{
	bool bSourceValid = false;

#if WITH_EDITOR
	if (SizeX <= 0 || SizeY <= 0 || SizeZ <= 0)
	{
		UE_LOGF(LogTexture, Warning, "%ls UpdateSourceFromFunction size in x,y, and z must be greater than zero", *GetFullName());
		return false;
	}
	
	PreEditChange(nullptr);

	Source2DTexture = nullptr;
	const int64 FormatDataSize = ERawImageFormat::GetBytesPerPixel(FImageCoreUtils::ConvertToRawImageFormat(Format));

	// Allocate temp buffer used to fill texture
	uint8* const NewData = (uint8*)FMemory::Malloc(SizeX * SizeY * SizeZ * FormatDataSize);
	uint8* DataPtr = NewData;

	// use PixelDataScratch just in case the function does something like write RGBA32F
	//	 when our format is actually BGRA8 , to try to avoid crashing
	uint64 PixelDataScratch[4];

	// Loop over all voxels and fill from our TFunction
	for (int32 PosZ = 0; PosZ < SizeZ; ++PosZ)
	{
		for (int32 PosY = 0; PosY < SizeY; ++PosY)
		{
			for (int32 PosX = 0; PosX < SizeX; ++PosX)
			{
				// ?? Func knows our pixel format somehow?
				//	should have defined this so that Func always writes FLinearColor and we convert
				Func(PosX, PosY, PosZ, PixelDataScratch);

				memcpy(DataPtr,PixelDataScratch,FormatDataSize);
				DataPtr += FormatDataSize;
			}
		}
	}

	// Init the final source data from the temp buffer
	Source.Init(SizeX, SizeY, SizeZ, 1, Format, NewData);
	
	// Free temp buffers
	FMemory::Free(NewData);

	SetLightingGuid(); // Because the content has changed, use a new GUID.

	PostEditChange();

	bSourceValid = true;
#endif

	return bSourceValid;
}

void UVolumeTexture::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UVolumeTexture::Serialize"), STAT_VolumeTexture_Serialize, STATGROUP_LoadTime);

	Super::Serialize(Ar);

	FStripDataFlags StripFlags(Ar);
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (bCooked || Ar.IsCooking())
	{
		SerializeCookedPlatformData(Ar);
	}
}

void UVolumeTexture::PostLoad()
{
#if WITH_EDITOR

	if (FApp::CanEverRender())
	{
		if (FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
		{
			BeginCachePlatformData();
		}
		else
		{
			FinishCachePlatformData();
		}
	}
#endif // #if WITH_EDITOR

	Super::PostLoad();
}

void UVolumeTexture::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UVolumeTexture::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITOR
	int32 SizeX = Source.GetSizeX();
	int32 SizeY = Source.GetSizeY();
	int32 SizeZ = Source.GetNumSlices();
#else
	int32 SizeX = 0;
	int32 SizeY = 0;
	int32 SizeZ = 0;
#endif

	const FString Dimensions = FString::Printf(TEXT("%dx%dx%d"), SizeX, SizeY, SizeZ);
	Context.AddTag( FAssetRegistryTag("Dimensions", Dimensions, FAssetRegistryTag::TT_Dimensional) );
	Context.AddTag( FAssetRegistryTag("Format", GPixelFormats[GetPixelFormat()].Name, FAssetRegistryTag::TT_Alphabetical) );

	Super::GetAssetRegistryTags(Context);
}

void UVolumeTexture::UpdateResourceWithParams(EUpdateResourceFlags InFlags)
{
#if WITH_EDITOR
	bool bAsyncFlags = false;
	// Recache platform data if the source has changed.
	if (!EnumHasAnyFlags(InFlags, EUpdateResourceFlags::Synchronous) &&
		FTextureCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		bAsyncFlags = true;
	}

	CachePlatformData(bAsyncFlags, bAsyncFlags, bAsyncFlags, nullptr, EnumHasAnyFlags(InFlags, EUpdateResourceFlags::ForceRebuild));
#endif // #if WITH_EDITOR

	// Route to super.
	Super::UpdateResourceWithParams(InFlags);
}

FString UVolumeTexture::GetDesc()
{
	return FString::Printf(TEXT("Volume: %dx%dx%d [%s]"),
		GetSizeX(),
		GetSizeY(),
		GetSizeZ(),
		GPixelFormats[GetPixelFormat()].Name
		);
}

uint32 UVolumeTexture::CalcTextureMemorySize(int32 MipCount) const
{
#if WITH_EDITOR
	if (IsDefaultTexture())
	{
		return GetDefaultVolumeTexture(this)->CalcTextureMemorySize(MipCount);
	}
#endif

	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat()];

	uint32 Size = 0;
	if (FormatInfo.Supported && GetPlatformData())
	{
		const EPixelFormat Format = GetPixelFormat();
		if (Format != PF_Unknown)
		{
			const ETextureCreateFlags Flags = (SRGB ? TexCreate_SRGB : TexCreate_None)  | (bNotOfflineProcessed ? TexCreate_None : TexCreate_OfflineProcessed) | (bNoTiling ? TexCreate_NoTiling : TexCreate_None);

			uint32 SizeX = 0;
			uint32 SizeY = 0;
			uint32 SizeZ = 0;
			CalcMipMapExtent3D(GetSizeX(), GetSizeY(), GetSizeZ(), Format, FMath::Max<int32>(0, GetNumMips() - MipCount), SizeX, SizeY, SizeZ);

			const FRHITextureDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("Temp"), SizeX, SizeY, SizeZ, Format)
				.SetNumMips(FMath::Max(1, MipCount))
				.SetFlags(Flags)
				.SetExtData(GetPlatformData()->GetExtData());

			Size = RHICalcTexturePlatformSize(Desc).Size;
		}
	}
	return Size;
}

uint32 UVolumeTexture::CalcTextureMemorySizeEnum( ETextureMipCount Enum ) const
{
	if ( Enum == TMC_ResidentMips || Enum == TMC_AllMipsBiased )
	{
		return CalcTextureMemorySize(GetNumMips() - GetCachedLODBias());
	}
	else
	{
		return CalcTextureMemorySize(GetNumMips());
	}
}

// While compiling the platform data in editor, we will return the 
// placeholders value to ensure rendering works as expected and that
// there are no thread-unsafe access to the platform data being built.
// Any process requiring a fully up-to-date platform data is expected to
// call FTextureCompiler:Get().FinishCompilation on UTexture first.
int32 UVolumeTexture::GetSizeX() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultVolumeTexture(this)->GetSizeX();
		}
#endif
		return PrivatePlatformData->SizeX;
	}
	return 0;
}

int32 UVolumeTexture::GetSizeY() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultVolumeTexture(this)->GetSizeY();
		}
#endif
		return PrivatePlatformData->SizeY;
	}
	return 0;
}

int32 UVolumeTexture::GetSizeZ() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultVolumeTexture(this)->GetSizeZ();
		}
#endif
		return PrivatePlatformData->GetNumSlices();
	}
	return 0;
}

int32 UVolumeTexture::GetNumMips() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultVolumeTexture(this)->GetNumMips();
		}
#endif
		return PrivatePlatformData->Mips.Num();
	}
	return 0;
}

EPixelFormat UVolumeTexture::GetPixelFormat() const
{
	if (PrivatePlatformData)
	{
#if WITH_EDITOR
		if (IsDefaultTexture())
		{
			return GetDefaultVolumeTexture(this)->GetPixelFormat();
		}
#endif
		return PrivatePlatformData->PixelFormat;
	}
	return PF_Unknown;
}

FTextureResource* UVolumeTexture::CreateResource()
{
#if WITH_EDITOR
	if (PrivatePlatformData)
	{
		if (PrivatePlatformData->IsAsyncWorkComplete())
		{
			// Make sure AsyncData has been destroyed in case it still exists to avoid
			// IsDefaultTexture thinking platform data is still being computed.
			PrivatePlatformData->FinishCache();
		}
		else
		{
			FTextureCompilingManager::Get().AddTextures({ this });
			UnlinkStreaming();
			return new FTexture3DResource(this, (const FTexture3DResource*)GetDefaultVolumeTexture(this)->GetResource());
		}
	}
#endif

	const FPixelFormatInfo& FormatInfo = GPixelFormats[GetPixelFormat()];
	const bool bFormatIsSupported = FormatInfo.Supported;

	if (GetNumMips() > 0 && GSupportsTexture3D && bFormatIsSupported)
	{
		return new FTexture3DResource(this, GetResourcePostInitState(GetPlatformData(), GSupportsVolumeTextureStreaming));
	}
	else if (GetNumMips() == 0)
	{
		UE_LOGF(LogTexture, Warning, "%ls contains no miplevels! Please delete.", *GetFullName());
	}
	else if (!GSupportsTexture3D)
	{
		UE_LOGF(LogTexture, Warning, "%ls cannot be created, rhi does not support 3d textures.", *GetFullName());
	}
	else if (!bFormatIsSupported)
	{
		UE_LOGF(LogTexture, Warning, "%ls cannot be created, rhi does not support format %ls.", *GetFullName(), FormatInfo.Name);
	}
	return nullptr;
}

void UVolumeTexture::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	CumulativeResourceSize.AddUnknownMemoryBytes(CalcTextureMemorySizeEnum(TMC_ResidentMips));
}

#if WITH_EDITOR

bool UVolumeTexture::IsDefaultTexture() const
{
	return (PrivatePlatformData && !PrivatePlatformData->IsAsyncWorkComplete()) || (GetResource() && GetResource()->IsProxy());
}

void UVolumeTexture::SetDefaultSource2DTileSize()
{
	Source2DTileSizeX = 0;
	Source2DTileSizeY = 0;

	if (Source2DTexture && Source2DTexture->Source.IsValid())
	{
		const int32 SourceSizeX = Source2DTexture->Source.GetSizeX();
		const int32 SourceSizeY = Source2DTexture->Source.GetSizeY();
		if (SourceSizeX > 0 && SourceSizeY > 0)
		{
			const int32 NumPixels = SourceSizeX * SourceSizeY;
			const int32 TileSize = FMath::Max(1, FMath::RoundToInt(FMath::Pow((float)NumPixels, 1.f / 3.f)));
			const int32 NumTilesBySide = FMath::Max(1, FMath::RoundToInt(FMath::Sqrt((float)(SourceSizeX / TileSize) * (SourceSizeY / TileSize))));
			Source2DTileSizeX = SourceSizeX / NumTilesBySide;
			Source2DTileSizeY = SourceSizeY / NumTilesBySide;
		}
	}
}

void UVolumeTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
 	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyChangedEvent.Property)
	{
		static const FName SourceTextureName("Source2DTexture");
		static const FName TileSizeXName("Source2DTileSizeX");
		static const FName TileSizeYName("Source2DTileSizeY");

		const FName PropertyName = PropertyChangedEvent.Property->GetFName();

		// Set default tile size if none is currently specified.
		if (PropertyName == SourceTextureName)
		{
			SetDefaultSource2DTileSize();
		}
		// Update the content of the volume texture
		if (PropertyName == SourceTextureName || PropertyName == TileSizeXName || PropertyName == TileSizeYName)
		{
			UpdateSourceFromSourceTexture();
		}
		// Potentially update sampler state in the materials
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UVolumeTexture, AddressMode))
		{
			NotifyMaterials();
		}
	}
		
	Super::PostEditChangeProperty(PropertyChangedEvent);

}


uint32 UVolumeTexture::GetMaximumDimension() const
{
	return GMaxVolumeTextureDimensions;
}

#endif // #if WITH_EDITOR

bool UVolumeTexture::StreamOut(int32 NewMipCount)
{
	FTexture3DResource* Texture3DResource = GetResource() ? GetResource()->GetTexture3DResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamOut(NewMipCount) && ensure(Texture3DResource))
	{
		FTextureMipAllocator* MipAllocator = nullptr;

		// FVolumeTextureMipAllocator_Virtual?
		MipAllocator = new FVolumeTextureMipAllocator_Reallocate(this);

		PendingUpdate = new FTextureStreamOut(this, MipAllocator);
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

bool UVolumeTexture::StreamIn(int32 NewMipCount, bool bHighPrio)
{
	FTexture3DResource* Texture3DResource = GetResource() ? GetResource()->GetTexture3DResource() : nullptr;
	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamIn(NewMipCount) && ensure(Texture3DResource))
	{
		FTextureMipDataProvider* CustomMipDataProvider = nullptr;
		for (UAssetUserData* UserData : AssetUserData)
		{
			UTextureMipDataProviderFactory* CustomMipDataProviderFactory = Cast<UTextureMipDataProviderFactory>(UserData);
			if (CustomMipDataProviderFactory)
			{
				CustomMipDataProvider = CustomMipDataProviderFactory->AllocateMipDataProvider(this);
				if (CustomMipDataProvider)
				{
					break;
				}
			}
		}

		FTextureMipAllocator* MipAllocator = nullptr;
		FTextureMipDataProvider* DefaultMipDataProvider = nullptr;

#if WITH_EDITORONLY_DATA
		if (FPlatformProperties::HasEditorOnlyData() && !GetOutermost()->bIsCookedForEditor)
		{
			DefaultMipDataProvider = new FVolumeTextureMipDataProvider_DDC(this);
		}
		else 
#endif
		{
			DefaultMipDataProvider = new FVolumeTextureMipDataProvider_IO(this, bHighPrio);
		}

		// FVolumeTextureMipAllocator_Virtual?
		MipAllocator = new FVolumeTextureMipAllocator_Reallocate(this);

		PendingUpdate = new FTextureStreamIn(this, MipAllocator, CustomMipDataProvider, DefaultMipDataProvider);
		return !PendingUpdate->IsCancelled();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
