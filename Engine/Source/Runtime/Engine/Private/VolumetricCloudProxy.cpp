// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FVolumetricCloudSceneProxy implementation.
=============================================================================*/

#include "VolumetricCloudProxy.h"
#include "Components/VolumetricCloudComponent.h"



FVolumetricCloudSceneProxy::FVolumetricCloudSceneProxy(const UVolumetricCloudComponent* InComponent)
	: LayerBottomAltitudeKm(InComponent->LayerBottomAltitude)
	, LayerHeightKm(InComponent->LayerHeight)
	, TracingStartMaxDistance(InComponent->TracingStartMaxDistance)
	, TracingStartDistanceFromCamera(InComponent->TracingStartDistanceFromCamera)
	, TracingMaxDistanceMode(uint8(InComponent->TracingMaxDistanceMode))
	, TracingMaxDistance(InComponent->TracingMaxDistance)
	, PlanetRadiusKm(InComponent->PlanetRadius)
	, GroundAlbedo(InComponent->GroundAlbedo)
	, bUsePerSampleAtmosphericLightTransmittance(InComponent->bUsePerSampleAtmosphericLightTransmittance)
	, bHoldout(InComponent->bHoldout)
	, bRenderInMainPass(InComponent->bRenderInMainPass)
	, bVisibleInRealTimeSkyCaptures(InComponent->bVisibleInRealTimeSkyCaptures)
	, SkyLightCloudBottomOcclusion(InComponent->SkyLightCloudBottomOcclusion)
	, ViewSampleCountScale(InComponent->ViewSampleCountScale)
	, ReflectionViewSampleCountScale(InComponent->ReflectionViewSampleCountScaleValue)
	, ShadowViewSampleCountScale(InComponent->ShadowViewSampleCountScale)
	, ShadowReflectionViewSampleCountScale(InComponent->ShadowReflectionViewSampleCountScaleValue)
	, ShadowTracingDistance(InComponent->ShadowTracingDistance)
	, StopTracingTransmittanceThreshold(InComponent->StopTracingTransmittanceThreshold)
	, AerialPespectiveMieScatteringStartDistance(InComponent->AerialPespectiveMieScatteringStartDistance) 
	, AerialPespectiveMieScatteringFadeDistance(InComponent->AerialPespectiveMieScatteringFadeDistance) 
	, AerialPespectiveRayleighScatteringStartDistance(InComponent->AerialPespectiveRayleighScatteringStartDistance)
	, AerialPespectiveRayleighScatteringFadeDistance(InComponent->AerialPespectiveRayleighScatteringFadeDistance)
	, CloudVolumeMaterial(InComponent->GetMaterial())
{
}

FVolumetricCloudSceneProxy::~FVolumetricCloudSceneProxy()
{
}


