// Copyright Epic Games, Inc. All Rights Reserved.

#include "WeightedRandomSampler.h"
#include "Misc/DelayedAutoRegister.h"

IMPLEMENT_ABSTRACT_TYPE_LAYOUT(FWeightedRandomSampler);

FWeightedRandomSampler::FWeightedRandomSampler()
	: TotalWeight(0.0f)
{

}

FWeightedRandomSampler::~FWeightedRandomSampler() = default;

void FWeightedRandomSampler::Initialize()
{
	//Gather all weights.
	TArray<float> P;
	TotalWeight = GetWeights(P);

	int32 NumElements = P.Num();
	TArray<int32> Small;
	TArray<int32> Large;
	Small.Reserve(NumElements);
	Large.Reserve(NumElements);

	Prob.SetNumUninitialized(NumElements);
	Alias.SetNumUninitialized(NumElements);

	//Normalize weights and rescale to 0...NumElements.
	float Scale = TotalWeight ? (NumElements / TotalWeight) : 0.0f;
	for (int32 i = NumElements - 1; i >= 0; --i)
	{
		P[i] *= Scale;
		check(P[i] >= 0.0f && P[i] <= (float)NumElements);
		if (P[i] < 1)
		{
			Small.Add(i);
		}
		else
		{
			Large.Add(i);
		}
	}

	while (Small.Num() && Large.Num())
	{
		int32 SmallIdx = Small.Pop(EAllowShrinking::No);
		int32 LargeIdx = Large.Pop(EAllowShrinking::No);

		Prob[SmallIdx] = P[SmallIdx];
		Alias[SmallIdx] = LargeIdx;

		P[LargeIdx] = (P[LargeIdx] + P[SmallIdx]) - 1;
		if (P[LargeIdx] < 1.0f)
		{
			Small.Add(LargeIdx);
		}
		else
		{
			Large.Add(LargeIdx);
		}
	}

	//Any remaining entries in Large should now be 1.
	for (int32& LargeIdx : Large)
	{
		Prob[LargeIdx] = 1;
		Alias[LargeIdx] = LargeIdx;
	}

	//FP inaccuracies can lead S to still have entries on occasion, these should also be now 1.
	for (int32& SmallIdx : Small)
	{
		Prob[SmallIdx] = 1;
		Alias[SmallIdx] = SmallIdx;
	}
}

void FWeightedRandomSampler::Serialize(FArchive& Ar)
{
	Ar << Prob;
	Ar << Alias;
	Ar << TotalWeight;
}
