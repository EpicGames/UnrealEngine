// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/WindDirectionalSource.h"
#include "SceneInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/ArrowComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneProxies/WindSourceSceneProxy.h"
#include "Components/WindDirectionalSourceComponent.h"
#include "Components/BillboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WindDirectionalSource)

AWindDirectionalSource::AWindDirectionalSource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Component = CreateDefaultSubobject<UWindDirectionalSourceComponent>(TEXT("WindDirectionalSourceComponent0"));
	RootComponent = Component;

#if WITH_EDITORONLY_DATA
	ArrowComponent = CreateEditorOnlyDefaultSubobject<UArrowComponent>(TEXT("ArrowComponent0"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTexture;
			FName ID_Wind;
			FText NAME_Wind;
			FConstructorStatics()
				: SpriteTexture(TEXT("/Engine/EditorResources/S_WindDirectional"))
				, ID_Wind(TEXT("Wind"))
				, NAME_Wind(NSLOCTEXT("SpriteCategory", "Wind", "Wind"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (ArrowComponent)
		{
			ArrowComponent->ArrowColor = FColor(150, 200, 255);
			ArrowComponent->bTreatAsASprite = true;
			ArrowComponent->SpriteInfo.Category = ConstructorStatics.ID_Wind;
			ArrowComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Wind;
			ArrowComponent->SetupAttachment(Component);
			ArrowComponent->bIsScreenSizeScaled = true;
			ArrowComponent->bUseInEditorScaling = true;
		}

		if (UBillboardComponent* SpriteComp = GetSpriteComponent())
		{
			SpriteComp->Sprite = ConstructorStatics.SpriteTexture.Get();
			SpriteComp->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			SpriteComp->SpriteInfo.Category = ConstructorStatics.ID_Wind;
			SpriteComp->SpriteInfo.DisplayName = ConstructorStatics.NAME_Wind;
			SpriteComp->SetupAttachment(Component);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FWindData::PrepareForAccumulate()
{
	FMemory::Memset(this, 0, sizeof(FWindData));
}

void FWindData::AddWeighted(const FWindData& InWindData, float Weight)
{	
	Speed += InWindData.Speed * Weight;
	MinGustAmt += InWindData.MinGustAmt * Weight;
	MaxGustAmt += InWindData.MaxGustAmt * Weight;
	Direction += InWindData.Direction * Weight;
}

void FWindData::NormalizeByTotalWeight(float TotalWeight)
{
	if (TotalWeight > 0.0f)
	{
		Speed /= TotalWeight;
		MinGustAmt /= TotalWeight;
		MaxGustAmt /= TotalWeight;
		Direction /= TotalWeight;
		Direction.Normalize();
	}
}

bool FWindSourceSceneProxy::GetWindParameters(const FVector& EvaluatePosition, FWindData& WindData, float& Weight) const
{ 
	if (bIsPointSource)
	{
		const float Distance = (Position - EvaluatePosition).Size();
		if (Distance <= Radius)
		{
			// Mimic Engine point light attenuation with a FalloffExponent of 1
			const float RadialFalloff = FMath::Max<FVector::FReal>(1.0f - ((EvaluatePosition - Position) / Radius).SizeSquared(), 0.0f);
			//WindDirectionAndSpeed = FVector4((EvaluatePosition - Position) / Distance * Strength * RadialFalloff, Speed); 

			WindData.Direction = (EvaluatePosition - Position) / Distance;
			WindData.Speed = Speed * RadialFalloff;
			WindData.MinGustAmt = MinGustAmt * RadialFalloff;
			WindData.MaxGustAmt = MaxGustAmt * RadialFalloff;

			Weight = (Radius - Distance) / Radius * Strength;
			return true;
		}
		Weight = 0.f;
		WindData = FWindData();
		return false;
	}

	Weight				= Strength;
	WindData.Direction	= Direction;
	WindData.Speed		= Speed;
	WindData.MinGustAmt = MinGustAmt;
	WindData.MaxGustAmt = MaxGustAmt;	
	return true;
}

bool FWindSourceSceneProxy::GetDirectionalWindParameters(FWindData& WindData, float& Weight) const
{ 
	if (bIsPointSource)
	{
		Weight = 0.f;
		WindData = FWindData();
		return false;
	}

	Weight				= Strength;
	WindData.Direction	= Direction;
	WindData.Speed		= Speed;
	WindData.MinGustAmt = MinGustAmt;
	WindData.MaxGustAmt = MaxGustAmt;
	return true;
}

void FWindSourceSceneProxy::ApplyWorldOffset(FVector InOffset)
{
	if (bIsPointSource)
	{
		Position+= InOffset;
	}
}

UWindDirectionalSourceComponent::UWindDirectionalSourceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Strength = 0.1f;
	Speed = 0.1f;
	MinGustAmount = 0.1f;
	MaxGustAmount = 0.2f;

	// wind will be activated automatically by default
	bAutoActivate = true;
}

void UWindDirectionalSourceComponent::Activate(bool bReset)
{
	Super::Activate(bReset);
	if (bRenderStateCreated)
	{
		GetWorld()->Scene->AddWindSource(this);
	}
}

void UWindDirectionalSourceComponent::Deactivate()
{
	Super::Deactivate();
	if (SceneProxy)
	{
		GetWorld()->Scene->RemoveWindSource(this);
	}
}

void UWindDirectionalSourceComponent::SetStrength(float InNewStrength)
{
	Strength = InNewStrength;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::SetSpeed(float InNewSpeed)
{
	Speed = InNewSpeed;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::SetMinimumGustAmount(float InNewMinGust)
{
	MinGustAmount = InNewMinGust;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::SetMaximumGustAmount(float InNewMaxGust)
{
	MaxGustAmount = InNewMaxGust;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::SetRadius(float InNewRadius)
{
	Radius = InNewRadius;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::SetWindType(EWindSourceType InNewType)
{
	bPointWind = InNewType == EWindSourceType::Point;

	// Need to update render thread proxy with new data
	MarkRenderDynamicDataDirty();
}

void UWindDirectionalSourceComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	GetWorld()->Scene->AddWindSource(this);  // AddWindSource accesses the WindComponents_GameThread array, so it might be concurrent friendly, but is not thread safe!
}

void UWindDirectionalSourceComponent::SendRenderTransform_Concurrent()
{
	Super::SendRenderTransform_Concurrent();
	GetWorld()->Scene->UpdateWindSource(this);  // UpdateWindSource does not access the WindComponents_GameThread array, and is therefore thread safe when iterating in parallel over components.
}

void UWindDirectionalSourceComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();
	GetWorld()->Scene->UpdateWindSource(this);  // UpdateWindSource does not access the WindComponents_GameThread array, and is therefore thread safe when iterating in parallel over components.
}

void UWindDirectionalSourceComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();
	GetWorld()->Scene->RemoveWindSource(this);  // RemoveWindSource accesses the WindComponents_GameThread array, so it might be concurrent friendly, but is not thread safe!
}

FWindSourceSceneProxy* UWindDirectionalSourceComponent::CreateSceneProxy() const
{
	if (bPointWind)
	{
		return new FWindSourceSceneProxy(
			GetComponentTransform().GetLocation(),
			Strength,
			Speed,
			MinGustAmount,
			MaxGustAmount,
			Radius
			);
	}
	else
	{
		return new FWindSourceSceneProxy(
			GetComponentTransform().GetUnitAxis(EAxis::X),
			Strength,
			Speed,
			MinGustAmount,
			MaxGustAmount
			);
	}
}

bool UWindDirectionalSourceComponent::GetWindParameters(const FVector& EvaluatePosition, FWindData& OutData, float& Weight) const
{
	// Whether we found any wind at the requested position (perhaps there's only point wind in the scene, or no wind)
	bool bFoundWind = false;

	if(bPointWind)
	{
		FWindSourceSceneProxy LocalProxy = FWindSourceSceneProxy(GetComponentTransform().GetLocation(), Strength, Speed, MinGustAmount, MaxGustAmount, Radius);
		bFoundWind = LocalProxy.GetWindParameters(EvaluatePosition, OutData, Weight);
	}
	else
	{
		FWindSourceSceneProxy LocalProxy = FWindSourceSceneProxy(GetComponentTransform().GetUnitAxis(EAxis::X), Strength, Speed, MinGustAmount, MaxGustAmount);
		bFoundWind = LocalProxy.GetWindParameters(EvaluatePosition, OutData, Weight);
	}

	return bFoundWind;
}


