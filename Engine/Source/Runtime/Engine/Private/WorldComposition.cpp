// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldComposition.cpp: UWorldComposition implementation
=============================================================================*/
#include "Engine/WorldComposition.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Engine/LevelStreaming.h"
#include "HAL/PlatformFileManager.h"
#include "Engine/World.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/AssetManager.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldComposition)

DEFINE_LOG_CATEGORY_STATIC(LogWorldComposition, Log, All);

#if WITH_EDITOR
UWorldComposition::FEnableWorldCompositionEvent UWorldComposition::EnableWorldCompositionEvent;
UWorldComposition::FWorldCompositionChangedEvent UWorldComposition::WorldCompositionChangedEvent;
#endif // WITH_EDITOR

UWorldComposition::UWorldComposition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bTemporallyDisableOriginTracking(false)
	, bTemporarilyDisableOriginTracking(false)
#endif
	, TilesStreamingTimeThreshold(1.0)
	, bLoadAllTilesDuringCinematic(false)
	, bRebaseOriginIn3DSpace(false)
	, RebaseOriginDistance(UE_OLD_HALF_WORLD_MAX1*0.5f)
{
}

void UWorldComposition::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);
	
	// We serialize this data only for PIE and cooked data
	// In normal editor game this data is regenerated on object construction
	const bool bIsPIE = Ar.GetPortFlags() & PPF_DuplicateForPIE;
	const bool bIsCooked = Ar.IsCooking() || (Ar.IsLoading() && FPlatformProperties::RequiresCookedData());
	if (bIsPIE || bIsCooked)
	{
		Ar << WorldRoot;
		Ar << Tiles;
		Ar << TilesStreaming;
	}
}

void UWorldComposition::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

#if WITH_EDITOR
	// Tiles have been deserialized from original, so we should be considered up-to-date
	bRescanned = true;
#endif // WITH_EDITOR

	if (bDuplicateForPIE)
	{
		FixupForPIE(GetOutermost()->GetPIEInstanceID());	
	}
}

void UWorldComposition::PostLoad()
{
	Super::PostLoad();

	if (FPlatformProperties::RequiresCookedData())
	{
		// Create streaming levels for each Tile
		PopulateStreamingLevels();
		// Calculate absolute positions since they are not serialized to disk
		CaclulateTilesAbsolutePositions();
	}

	UWorld* World = GetWorld();

	if (World->IsGameWorld())
	{
		// Remove streaming levels created by World Browser, to avoid duplication with streaming levels from world composition
		World->SetStreamingLevels(TilesStreaming);
	}
}

void UWorldComposition::FixupForPIE(int32 PIEInstanceID)
{
	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); ++TileIdx)
	{
		FWorldCompositionTile& Tile = Tiles[TileIdx];
		
		FString PIEPackageName = UWorld::ConvertToPIEPackageName(Tile.PackageName.ToString(), PIEInstanceID);
		Tile.PackageName = FName(*PIEPackageName);
		FSoftObjectPath::AddPIEPackageName(Tile.PackageName);
		for (FName& LODPackageName : Tile.LODPackageNames)
		{
			FString PIELODPackageName = UWorld::ConvertToPIEPackageName(LODPackageName.ToString(), PIEInstanceID);
			LODPackageName = FName(*PIELODPackageName);
			FSoftObjectPath::AddPIEPackageName(LODPackageName);
		}
	}
}

FString UWorldComposition::GetWorldRoot() const
{
	return WorldRoot;
}

UWorld* UWorldComposition::GetWorld() const
{
	return Cast<UWorld>(GetOuter());
}

struct FTileLODCollection
{
	FString PackageNames[WORLDTILE_LOD_MAX_INDEX + 1];
};

struct FPackageNameAndLODIndex
{
	// Package name without LOD suffix
	FString PackageName;
	// LOD index this package represents
	int32	LODIndex;
};

struct FWorldTilesGatherer
	: public IPlatformFile::FDirectoryVisitor
{
	TArray<FString> MapFilesToConsider;

	// List of tile long package names (non LOD)
	TArray<FString> TilesCollection;
	
	// Tile short pkg name -> Tiles LOD
	TMultiMap<FString, FPackageNameAndLODIndex> TilesLODCollection;
	
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		// for all packages
		if (!bIsDirectory)
		{
			FString FullPath = FilenameOrDirectory;

			if (FPaths::GetExtension(FullPath, true) == FPackageName::GetMapPackageExtension() 
#if WITH_EDITOR
				// Make sure we don't gather partitioned levels in World Composition
				&& !ULevel::GetIsLevelPartitionedFromPackage(FName(FPackageName::FilenameToLongPackageName(FullPath)))
#endif
				)
			{
				MapFilesToConsider.Emplace(MoveTemp(FullPath));
			}
		}

		return true;
	}

	void BuildTileCollection(const FString& WorldRootFilename)
	{
		FPlatformFileManager::Get().GetPlatformFile().IterateDirectoryRecursively(*WorldRootFilename, *this);

		IAssetRegistry& AssetRegistry = UAssetManager::Get().GetAssetRegistry();
		AssetRegistry.ScanFilesSynchronous(MapFilesToConsider);

		for (const FString& FullPath : MapFilesToConsider)
		{
			FString TilePackageName = FPackageName::FilenameToLongPackageName(FullPath);
			const FSoftObjectPath TilePackagePath(FTopLevelAssetPath(FName(*TilePackageName), FName(*FPackageName::GetLongPackageAssetName(TilePackageName))));

			const FAssetData MapAssetData = AssetRegistry.GetAssetByObjectPath(TilePackagePath);

			if (MapAssetData.IsValid() && !MapAssetData.IsRedirector())
			{
				FPackageNameAndLODIndex PackageNameLOD = BreakToNameAndLODIndex(TilePackageName);

				if (PackageNameLOD.LODIndex != INDEX_NONE)
				{
					if (PackageNameLOD.LODIndex == 0)
					{
						// non-LOD tile
						TilesCollection.Emplace(MoveTemp(TilePackageName));
					}
					else
					{
						// LOD tile
						FString TileShortName = FPackageName::GetShortName(PackageNameLOD.PackageName);
						TilesLODCollection.Add(MoveTemp(TileShortName), MoveTemp(PackageNameLOD));
					}
				}
			}
		}
	}

	FPackageNameAndLODIndex BreakToNameAndLODIndex(const FString& PackageName) const
	{
		// LOD0 packages do not have LOD suffixes
		FPackageNameAndLODIndex Result = {PackageName, 0};
		
		int32 SuffixPos = PackageName.Find(WORLDTILE_LOD_PACKAGE_SUFFIX, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (SuffixPos != INDEX_NONE)
		{
			// Extract package name without LOD suffix
			FString PackageNameWithoutSuffix = PackageName.LeftChop(PackageName.Len() - SuffixPos);
			// Extract number from LOD suffix which represents LOD index
			FString LODIndexStr = PackageName.RightChop(SuffixPos + FString(WORLDTILE_LOD_PACKAGE_SUFFIX).Len());
			// Convert number to a LOD index
			int32 LODIndex = FCString::Atoi(*LODIndexStr);
			// Validate LOD index
			if (LODIndex > 0 && LODIndex <= WORLDTILE_LOD_MAX_INDEX)
			{
				Result.PackageName = PackageNameWithoutSuffix;
				Result.LODIndex = LODIndex;
			}
			else
			{
				// Invalid LOD index
				Result.LODIndex = INDEX_NONE;
			}
		}
		
		return Result;
	}
};

void UWorldComposition::Rescan(bool bForce)
{
	checkfSlow(!FPlatformProperties::RequiresCookedData(), TEXT("Rescan should not be called in a cooked game with serialized tile info."));

#if WITH_EDITOR
	if (bRescanned && !bForce)
	{
		return;
	}
#endif // WITH_EDITOR

	// Save tiles state, so we can restore it for dirty tiles after rescan is done
	FTilesList SavedTileList = Tiles;
		
	Reset();	

	UWorld* OwningWorld = GetWorld();
	
	FString RootPackageName = GetOutermost()->GetName();
	RootPackageName = UWorld::StripPIEPrefixFromPackageName(RootPackageName, OwningWorld->StreamingLevelsPrefix);
	if (!FPackageName::DoesPackageExist(RootPackageName))
	{
		return;	
	}
	
	WorldRoot = FPaths::GetPath(RootPackageName) + TEXT("/");
			
	// Gather tiles packages from a specified folder
	FWorldTilesGatherer Gatherer;
	FString WorldRootFilename = FPackageName::LongPackageNameToFilename(WorldRoot);
	Gatherer.BuildTileCollection(WorldRootFilename);

	// Make sure we have persistent level name without PIE prefix
	FString PersistentLevelPackageName = UWorld::StripPIEPrefixFromPackageName(OwningWorld->GetOutermost()->GetName(), OwningWorld->StreamingLevelsPrefix);
		
	// Add found tiles to a world composition, except persistent level
	for (const FString& TilePackageName : Gatherer.TilesCollection)
	{
		// Discard persistent level entry
		if (TilePackageName == PersistentLevelPackageName)
		{
			continue;
		}

		FWorldTileInfo Info;
		FString TileFilename = FPackageName::LongPackageNameToFilename(TilePackageName, FPackageName::GetMapPackageExtension());
		if (!FWorldTileInfo::Read(TileFilename, Info))
		{
			continue;
		}

		FWorldCompositionTile Tile;
		Tile.PackageName = FName(*TilePackageName);
		Tile.Info = Info;
		
		// Assign LOD tiles
		FString TileShortName = FPackageName::GetShortName(TilePackageName);
		TArray<FPackageNameAndLODIndex> TileLODList;
		Gatherer.TilesLODCollection.MultiFind(TileShortName, TileLODList);
		if (TileLODList.Num())
		{
			Tile.LODPackageNames.SetNum(WORLDTILE_LOD_MAX_INDEX);
			FString TilePath = FPackageName::GetLongPackagePath(TilePackageName) + TEXT("/");
			for (const FPackageNameAndLODIndex& TileLOD : TileLODList)
			{
				// LOD tiles should be in the same directory or in nested directory
				// Basically tile path should be a prefix of a LOD tile path
				if (TileLOD.PackageName.StartsWith(TilePath))
				{
					Tile.LODPackageNames[TileLOD.LODIndex-1] = FName(*FString::Printf(TEXT("%s_LOD%d"), *TileLOD.PackageName, TileLOD.LODIndex));
				}
			}

			// Remove null entries in LOD list
			int32 NullEntryIdx;
			if (Tile.LODPackageNames.Find(FName(), NullEntryIdx))
			{
				Tile.LODPackageNames.SetNum(NullEntryIdx);
			}
		}
		
		Tiles.Add(Tile);
	}

#if WITH_EDITOR
	RestoreDirtyTilesInfo(SavedTileList);
#endif// WITH_EDITOR
	
	// Create streaming levels for each Tile
	PopulateStreamingLevels();

	// Calculate absolute positions since they are not serialized to disk
	CaclulateTilesAbsolutePositions();

#if WITH_EDITOR
	bRescanned = true;
#endif // WITH_EDITOR
}

void UWorldComposition::ReinitializeForPIE()
{
	FixupForPIE(GetOutermost()->GetPIEInstanceID());
	GetWorld()->SetStreamingLevels(TilesStreaming);
}

bool UWorldComposition::DoesTileExists(const FName& InTilePackageName) const
{
	for (const FWorldCompositionTile& Tile : Tiles)
	{
		if (Tile.PackageName == InTilePackageName)
		{
			return true;
		}
	}
	return false;
}

ULevelStreaming* UWorldComposition::CreateStreamingLevel(const FWorldCompositionTile& InTile) const
{
	UWorld* OwningWorld = GetWorld();
	ULevelStreamingDynamic* StreamingLevel = NewObject<ULevelStreamingDynamic>(OwningWorld, NAME_None, RF_Transient);
		
	// Associate a package name.
	StreamingLevel->SetWorldAssetByPackageName(InTile.PackageName);
	StreamingLevel->PackageNameToLoad	= InTile.PackageName;

	//Associate LOD packages if any
	StreamingLevel->LODPackageNames		= InTile.LODPackageNames;
		
	return StreamingLevel;
}

void UWorldComposition::CaclulateTilesAbsolutePositions()
{
	for (FWorldCompositionTile& Tile : Tiles)
	{
		TSet<FName> VisitedParents;
		
		Tile.Info.AbsolutePosition = FIntVector::ZeroValue;
		FWorldCompositionTile* ParentTile = &Tile;
		
		do 
		{
			// Sum relative offsets
			Tile.Info.AbsolutePosition+= ParentTile->Info.Position;
			VisitedParents.Add(ParentTile->PackageName);

			FName NextParentTileName = FName(*ParentTile->Info.ParentTilePackageName);
			// Detect loops in parent->child hierarchy
			FWorldCompositionTile* NextParentTile = FindTileByName(NextParentTileName);
			if (NextParentTile && VisitedParents.Find(NextParentTileName) != nullptr)
			{
				UE_LOGF(LogWorldComposition, Warning, "World composition tile (%ls) has a cycled parent (%ls)", *Tile.PackageName.ToString(), *NextParentTileName.ToString());
				NextParentTile = nullptr;
				ParentTile->Info.ParentTilePackageName = FName(NAME_None).ToString();
			}
			
			ParentTile = NextParentTile;
		} 
		while (ParentTile);
	}
}

void UWorldComposition::Reset()
{
	WorldRoot.Empty();
	Tiles.Empty();
	TilesStreaming.Empty();
}

int32 UWorldComposition::FindTileIndexByName(const FName& InPackageName) const
{
	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); ++TileIdx)
	{
		const FWorldCompositionTile& Tile = Tiles[TileIdx];

		if (Tile.PackageName == InPackageName)
		{
			return TileIdx;
		}
		
		// Check LOD names
		for (const FName& LODPackageName : Tile.LODPackageNames)
		{
			if (LODPackageName == InPackageName)
			{
				return TileIdx;
			}
		}
	}

	return INDEX_NONE;
}

FWorldCompositionTile* UWorldComposition::FindTileByName(const FName& InPackageName) const
{
	int32 TileIdx = FindTileIndexByName(InPackageName);
	if (TileIdx != INDEX_NONE)
	{
		return const_cast<FWorldCompositionTile*>(&Tiles[TileIdx]);
	}
	else
	{
		return nullptr;
	}
}

UWorldComposition::FTilesList& UWorldComposition::GetTilesList()
{
	return Tiles;
}

#if WITH_EDITOR

FWorldTileInfo UWorldComposition::GetTileInfo(const FName& InPackageName) const
{
	FWorldCompositionTile* Tile = FindTileByName(InPackageName);
	if (Tile)
	{
		return Tile->Info;
	}
	
	return FWorldTileInfo();
}

void UWorldComposition::OnTileInfoUpdated(const FName& InPackageName, const FWorldTileInfo& InInfo)
{
	FWorldCompositionTile* Tile = FindTileByName(InPackageName);
	bool PackageDirty = false;
	
	if (Tile)
	{
		PackageDirty = !(Tile->Info == InInfo);
		Tile->Info = InInfo;
	}
	else
	{
		PackageDirty = true;
		UWorld* OwningWorld = GetWorld();
		
		FWorldCompositionTile NewTile;
		NewTile.PackageName = InPackageName;
		NewTile.Info = InInfo;
		
		Tiles.Add(NewTile);
		TilesStreaming.Add(CreateStreamingLevel(NewTile));
		Tile = &Tiles.Last();
	}
	
	// Assign info to level package in case package is loaded
	UPackage* LevelPackage = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), NULL, Tile->PackageName));
	if (LevelPackage)
	{
		if (!LevelPackage->GetWorldTileInfo())
		{
			LevelPackage->SetWorldTileInfo(MakeUnique<FWorldTileInfo>(Tile->Info));
			PackageDirty = true;
		}
		else
		{
			*(LevelPackage->GetWorldTileInfo()) = Tile->Info;
		}

		if (PackageDirty)
		{
			LevelPackage->MarkPackageDirty();
		}
	}
}

void UWorldComposition::RestoreDirtyTilesInfo(const FTilesList& TilesPrevState)
{
	if (!TilesPrevState.Num())
	{
		return;
	}
	
	for (FWorldCompositionTile& Tile : Tiles)
	{
		UPackage* LevelPackage = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), NULL, Tile.PackageName));
		if (LevelPackage && LevelPackage->IsDirty())
		{
			const FWorldCompositionTile* FoundTile = TilesPrevState.FindByPredicate([=](const FWorldCompositionTile& TilePrev)
			{
				return TilePrev.PackageName == Tile.PackageName;
			});
			
			if (FoundTile)
			{
				Tile.Info = FoundTile->Info;
			}
		}
	}
}

void UWorldComposition::CollectTilesToCook(TArray<FString>& PackageNames)
{
	// UWorld::InitWorld has not been invoked yet, but we need the tiles so they get packaged. Scan now.
	Rescan();

	for (const FWorldCompositionTile& Tile : Tiles)
	{
		PackageNames.AddUnique(Tile.PackageName.ToString());
		
		for (const FName& TileLODName : Tile.LODPackageNames)
		{
			PackageNames.AddUnique(TileLODName.ToString());
		}
	}
}

#endif //WITH_EDITOR

void UWorldComposition::PopulateStreamingLevels()
{
	TilesStreaming.Empty(Tiles.Num());
	
	for (const FWorldCompositionTile& Tile : Tiles)
	{
		TilesStreaming.Add(CreateStreamingLevel(Tile));
	}
}

void UWorldComposition::GetDistanceVisibleLevels(
	const FVector& InLocation, 
	TArray<FDistanceVisibleLevel>& OutVisibleLevels,
	TArray<FDistanceVisibleLevel>& OutHiddenLevels) const
{
	GetDistanceVisibleLevels(&InLocation, 1, OutVisibleLevels, OutHiddenLevels);
}

void UWorldComposition::GetDistanceVisibleLevels(
	const FVector* InLocations,
	int32 NumLocations,
	TArray<FDistanceVisibleLevel>& OutVisibleLevels,
	TArray<FDistanceVisibleLevel>& OutHiddenLevels) const
{
	const UWorld* OwningWorld = GetWorld();

	FIntPoint WorldOriginLocationXY = FIntPoint(OwningWorld->OriginLocation.X, OwningWorld->OriginLocation.Y);
			
	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); TileIdx++)
	{
		const FWorldCompositionTile& Tile = Tiles[TileIdx];
		
		// Skip non distance based levels
		if (!IsDistanceDependentLevel(TileIdx))
		{
			continue;
		}

		FDistanceVisibleLevel VisibleLevel = 
		{
			TileIdx,
			TilesStreaming[TileIdx],
			INDEX_NONE
		};

		bool bIsVisible = false;

		if (OwningWorld->IsNetMode(NM_DedicatedServer))
		{
			// Dedicated server always loads all distance dependent tiles
			bIsVisible = true;
		}
		else if (IsRunningCommandlet())
		{
			// Commandlets have no concept of viewer location, so always load all distance-dependent tiles.
			bIsVisible = true;
		}
		else
		{
			//
			// Check if tile bounding box intersects with a sphere with origin at provided location and with radius equal to tile layer distance settings
			//
			FIntPoint LevelPositionXY = FIntPoint(Tile.Info.AbsolutePosition.X, Tile.Info.AbsolutePosition.Y);
			FIntPoint LevelOffsetXY = LevelPositionXY - WorldOriginLocationXY;
			FBox LevelBounds = Tile.Info.Bounds.ShiftBy(FVector(LevelOffsetXY));
			// We don't care about third dimension yet
			LevelBounds.Min.Z = -WORLD_MAX;
			LevelBounds.Max.Z = +WORLD_MAX;
				
			int32 NumAvailableLOD = FMath::Min(Tile.Info.LODList.Num(), Tile.LODPackageNames.Num());
			// Find LOD
			// INDEX_NONE for original non-LOD level
			for (int32 LODIdx = INDEX_NONE; LODIdx < NumAvailableLOD; ++LODIdx)
			{
				if (bIsVisible && LODIdx > VisibleLevel.LODIndex)
				{
					// no point to loop more, we have visible tile with best possible LOD
					break;
				}
				
				int32 TileStreamingDistance = Tile.Info.GetStreamingDistance(LODIdx);
				for (int32 LocationIdx = 0; LocationIdx < NumLocations; ++LocationIdx)
				{
					FSphere QuerySphere(InLocations[LocationIdx], TileStreamingDistance);
					if (FMath::SphereAABBIntersection(QuerySphere, LevelBounds))
					{
						VisibleLevel.LODIndex = LODIdx;
						bIsVisible = true;
						break;
					}
				}
			}
		}

		if (bIsVisible)
		{
			OutVisibleLevels.Add(VisibleLevel);
		}
		else
		{
			OutHiddenLevels.Add(VisibleLevel);
		}
	}
}

void UWorldComposition::UpdateStreamingState(const FVector& InLocation)
{
	UpdateStreamingState(&InLocation, 1);
}

void UWorldComposition::UpdateStreamingState(const FVector* InLocations, int32 Num)
{
	UWorld* OwningWorld = GetWorld();

	// Get the list of visible and hidden levels from current view point
	TArray<FDistanceVisibleLevel> DistanceVisibleLevels;
	TArray<FDistanceVisibleLevel> DistanceHiddenLevels;
	GetDistanceVisibleLevels(InLocations, Num, DistanceVisibleLevels, DistanceHiddenLevels);
	
	// Dedicated server always blocks on load
	bool bShouldBlock = (OwningWorld->GetNetMode() == NM_DedicatedServer);
	
	// Set distance hidden levels to unload
	for (const FDistanceVisibleLevel& Level : DistanceHiddenLevels)
	{
		CommitTileStreamingState(OwningWorld, Level.TileIdx, false, false, bShouldBlock, Level.LODIndex);
	}

	// Set distance visible levels to load
	for (const FDistanceVisibleLevel& Level : DistanceVisibleLevels)
	{
		CommitTileStreamingState(OwningWorld, Level.TileIdx, true, true, bShouldBlock, Level.LODIndex);
	}
}

void UWorldComposition::UpdateStreamingStateCinematic(const FVector* InLocations, int32 Num)
{
	if (!bLoadAllTilesDuringCinematic)
	{
		UpdateStreamingState(InLocations, Num);
		return;
	}

	// Cinematic always blocks on load
	bool bShouldBlock = true;
	bool bStreamingStateChanged = false;
		
	// All tiles should be loaded and visible regardless of distance
	for (int32 TileIdx = 0; TileIdx < Tiles.Num(); TileIdx++)
	{
		FWorldCompositionTile& Tile = Tiles[TileIdx];
		// Skip non distance based levels
		if (!IsDistanceDependentLevel(TileIdx))
		{
			continue;
		}
		// Reset streaming state cooldown to ensure that new state will be committed
		Tile.StreamingLevelStateChangeTime = 0.0;
		//
		bStreamingStateChanged|= CommitTileStreamingState(GetWorld(), TileIdx, true, true, bShouldBlock, INDEX_NONE);
	}

	if (bStreamingStateChanged)
	{
		GetWorld()->FlushLevelStreaming(EFlushLevelStreamingType::Full);
	}
}

void UWorldComposition::UpdateStreamingState()
{
	UWorld* PlayWorld = GetWorld();
	check(PlayWorld);

	// Commandlets and Dedicated server does not use distance based streaming and just loads everything
	if (IsRunningCommandlet() || PlayWorld->GetNetMode() == NM_DedicatedServer)
	{
		UpdateStreamingState(FVector::ZeroVector);
		return;
	}
	
	int32 NumPlayers = GEngine->GetNumGamePlayers(PlayWorld);
	if (NumPlayers == 0)
	{
		return;
	}

	// calculate centroid location using local players views
	bool bCinematic = false;
	FVector CentroidLocation = FVector::ZeroVector;
	TArray<FVector, TInlineAllocator<16>> Locations;
		
	for (int32 PlayerIndex = 0; PlayerIndex < NumPlayers; ++PlayerIndex)
	{
		ULocalPlayer* Player = GEngine->GetGamePlayer(PlayWorld, PlayerIndex);
		if (Player && Player->PlayerController)
		{
			FVector ViewLocation;
			FRotator ViewRotation;
			Player->PlayerController->GetPlayerViewPoint(ViewLocation, ViewRotation);
			Locations.Add(ViewLocation);
			CentroidLocation+= ViewLocation;
			bCinematic|= Player->PlayerController->bCinematicMode;
		}
	}

	// In case there is no valid views don't bother updating level streaming state
	if (Locations.Num())
	{
		CentroidLocation/= Locations.Num();
		if (PlayWorld->GetWorldSettings()->bEnableWorldOriginRebasing && PlayWorld->OriginOffsetThisFrame.IsZero())
		{
			EvaluateWorldOriginLocation(CentroidLocation);
		}
		
		if (bCinematic)
		{
			UpdateStreamingStateCinematic(Locations.GetData(), Locations.Num());
		}
		else
		{
			UpdateStreamingState(Locations.GetData(), Locations.Num());
		}
	}
}

#if WITH_EDITOR
TArray<FWorldTileLayer> UWorldComposition::GetDistanceDependentLayers() const
{
	TArray<FWorldTileLayer> Layers;
	for (const FWorldCompositionTile& Tile : Tiles)
	{
		if (IsDistanceDependentLevel(Tile.PackageName))
		{
			Layers.AddUnique(Tile.Info.Layer);
		}
	}
	return Layers;
}

bool UWorldComposition::UpdateEditorStreamingState(const FVector& InLocation)
{
	UWorld* OwningWorld = GetWorld();
	bool bStateChanged = false;
	
	// Handle only editor worlds
	if (!OwningWorld->IsGameWorld() && 
		!OwningWorld->IsVisibilityRequestPending())
	{
		// Get the list of visible and hidden levels from current view point
		TArray<FDistanceVisibleLevel> DistanceVisibleLevels;
		TArray<FDistanceVisibleLevel> DistanceHiddenLevels;
		GetDistanceVisibleLevels(InLocation, DistanceVisibleLevels, DistanceHiddenLevels);

		// Hidden levels
		for (const FDistanceVisibleLevel& Level : DistanceHiddenLevels)
		{
			ULevelStreaming* EditorStreamingLevel = OwningWorld->GetLevelStreamingForPackageName(TilesStreaming[Level.TileIdx]->GetWorldAssetPackageFName());
			if (EditorStreamingLevel && 
				EditorStreamingLevel->IsLevelLoaded() &&
				EditorStreamingLevel->GetShouldBeVisibleInEditor())
			{
				EditorStreamingLevel->SetShouldBeVisibleInEditor(false);
				bStateChanged = true;
			}
		}

		// Visible levels
		for (const FDistanceVisibleLevel& Level : DistanceVisibleLevels)
		{
			ULevelStreaming* EditorStreamingLevel = OwningWorld->GetLevelStreamingForPackageName(TilesStreaming[Level.TileIdx]->GetWorldAssetPackageFName());
			if (EditorStreamingLevel &&
				EditorStreamingLevel->IsLevelLoaded() &&
				!EditorStreamingLevel->GetShouldBeVisibleInEditor())
			{
				EditorStreamingLevel->SetShouldBeVisibleInEditor(true);
				bStateChanged = true;
			}
		}
	}

	return bStateChanged;
}
#endif// WITH_EDITOR

void UWorldComposition::EvaluateWorldOriginLocation(const FVector& ViewLocation)
{
	UWorld* OwningWorld = GetWorld();
	
	FVector Location = ViewLocation;

	if (!bRebaseOriginIn3DSpace)
	{
		// Consider only XY plane
		Location.Z = 0.f;
	}
		
	// Request to shift world in case current view is quite far from current origin
	if (Location.SizeSquared() > FMath::Square(RebaseOriginDistance))
	{
		OwningWorld->RequestNewWorldOrigin(FIntVector(Location.X, Location.Y, Location.Z) + OwningWorld->OriginLocation);
	}
}

bool UWorldComposition::IsDistanceDependentLevel(int32 TileIdx) const
{
	if (TileIdx != INDEX_NONE)
	{
		return (Tiles[TileIdx].Info.Layer.DistanceStreamingEnabled && !TilesStreaming[TileIdx]->bDisableDistanceStreaming);
	}
	else
	{
		return false;
	}
}

bool UWorldComposition::IsDistanceDependentLevel(FName PackageName) const
{
	int32 TileIdx = FindTileIndexByName(PackageName);
	return IsDistanceDependentLevel(TileIdx);
}

bool UWorldComposition::CommitTileStreamingState(UWorld* PersistentWorld, int32 TileIdx, bool bShouldBeLoaded, bool bShouldBeVisible, bool bShouldBlock, int32 LODIdx)
{
	if (!Tiles.IsValidIndex(TileIdx))
	{
		return false;
	}

	FWorldCompositionTile& Tile = Tiles[TileIdx];
	ULevelStreaming* StreamingLevel = TilesStreaming[TileIdx];

	// Quit early in case state is not going to be changed
	if (StreamingLevel->bShouldBlockOnLoad == bShouldBlock &&
		StreamingLevel->GetLevelLODIndex() == LODIdx &&
		StreamingLevel->GetShouldBeVisibleFlag() == bShouldBeVisible &&
		StreamingLevel->ShouldBeLoaded() == bShouldBeLoaded
		)
	{
		return false;
	}

	// Quit early in case we have cooldown on streaming state changes
	const bool bUseStreamingStateCooldown = (PersistentWorld->IsGameWorld() && PersistentWorld->FlushLevelStreamingType == EFlushLevelStreamingType::None);
	if (bUseStreamingStateCooldown && TilesStreamingTimeThreshold > 0.0)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double TimePassed = CurrentTime - Tile.StreamingLevelStateChangeTime;
		if (TimePassed < TilesStreamingTimeThreshold)
		{
			return false;
		}

		// Save current time as state change time for this tile
		Tile.StreamingLevelStateChangeTime = CurrentTime;
	}

	// Commit new state
	StreamingLevel->bShouldBlockOnLoad	= bShouldBlock;
	StreamingLevel->SetShouldBeLoaded(bShouldBeLoaded);
	StreamingLevel->SetShouldBeVisible(bShouldBeVisible);
	StreamingLevel->SetLevelLODIndex(LODIdx);
	return true;
}

void UWorldComposition::OnLevelAddedToWorld(ULevel* InLevel)
{
#if WITH_EDITOR
	if (bTemporarilyDisableOriginTracking)
	{
		return;
	}
#endif
		
	// Move level according to current global origin
	FIntVector LevelOffset = GetLevelOffset(InLevel);
	InLevel->ApplyWorldOffset(FVector(LevelOffset), false);
}

void UWorldComposition::OnLevelRemovedFromWorld(ULevel* InLevel)
{
#if WITH_EDITOR
	if (bTemporarilyDisableOriginTracking)
	{
		return;
	}
#endif
	
	// Move level to its local origin
	FIntVector LevelOffset = GetLevelOffset(InLevel);
	InLevel->ApplyWorldOffset(-FVector(LevelOffset), false);
}

void UWorldComposition::OnLevelPostLoad(ULevel* InLevel)
{
	UPackage* LevelPackage = InLevel->GetOutermost();	
	if (LevelPackage && InLevel->OwningWorld)
	{
		FWorldTileInfo Info;
		UWorld* World = InLevel->OwningWorld;
				
		if (World->WorldComposition)
		{
			// Assign WorldLevelInfo previously loaded by world composition
			FWorldCompositionTile* Tile = World->WorldComposition->FindTileByName(LevelPackage->GetFName());
			if (Tile)
			{
				Info = Tile->Info;
			}
		}
		else if (!World->IsPartitionedWorld())
		{
#if WITH_EDITOR
			// Try to preserve FWorldTileInfo in case sub-level was loaded in the editor outside of world composition
			FString PackageFilename;
			if (FPackageName::TryConvertLongPackageNameToFilename(LevelPackage->GetName(), PackageFilename, FPackageName::GetMapPackageExtension()))
			{
				FWorldTileInfo::Read(PackageFilename, Info);
			}
#endif //WITH_EDITOR
		}
		
		const bool bIsDefault = (Info == FWorldTileInfo());
		if (!bIsDefault)
		{
			LevelPackage->SetWorldTileInfo(MakeUnique<FWorldTileInfo>(Info));
		}
	}
}

void UWorldComposition::OnLevelPreSave(ULevel* InLevel)
{
	if (InLevel->bIsVisible == true)
	{
		OnLevelRemovedFromWorld(InLevel);
	}
}

void UWorldComposition::OnLevelPostSave(ULevel* InLevel)
{
	if (InLevel->bIsVisible == true)
	{
		OnLevelAddedToWorld(InLevel);
	}
}

FIntVector UWorldComposition::GetLevelOffset(ULevel* InLevel) const
{
	UWorld* OwningWorld = GetWorld();
	UPackage* LevelPackage = InLevel->GetOutermost();
	
	FIntVector LevelPosition = FIntVector::ZeroValue;
	if (LevelPackage->GetWorldTileInfo())
	{
		LevelPosition = LevelPackage->GetWorldTileInfo()->AbsolutePosition;
	}
	
	return LevelPosition - OwningWorld->OriginLocation;
}

FBox UWorldComposition::GetLevelBounds(ULevel* InLevel) const
{
	UWorld* OwningWorld = GetWorld();
	UPackage* LevelPackage = InLevel->GetOutermost();
	
	FBox LevelBBox(ForceInit);
	if (LevelPackage->GetWorldTileInfo())
	{
		LevelBBox = LevelPackage->GetWorldTileInfo()->Bounds.ShiftBy(FVector(GetLevelOffset(InLevel)));
	}
	
	return LevelBBox;
}

