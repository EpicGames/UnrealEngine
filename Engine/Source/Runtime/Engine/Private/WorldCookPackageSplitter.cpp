// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldCookPackageSplitter.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "Editor.h"
#include "WorldPartition/WorldPartitionLog.h"

REGISTER_COOKPACKAGE_SPLITTER(FWorldCookPackageSplitter, UWorld);

TMap<UClass*, FWorldCookPackageSplitter::FSubSplitterFactory> FWorldCookPackageSplitter::RegisteredCookPackageSubSplitterFactories;

FWorldCookPackageSplitter::~FWorldCookPackageSplitter()
{
	check(!ReferencedWorld);
}

void FWorldCookPackageSplitter::RegisterCookPackageSubSplitterFactory(UClass* Class, FWorldCookPackageSplitter::FSubSplitterFactory Factory)
{
	check(!RegisteredCookPackageSubSplitterFactories.Contains(Class));
	RegisteredCookPackageSubSplitterFactories.Emplace(Class, MoveTemp(Factory));
}

void FWorldCookPackageSplitter::UnregisterCookPackageSubSplitterFactory(UClass* Class)
{
	check(RegisteredCookPackageSubSplitterFactories.Contains(Class));
	RegisteredCookPackageSubSplitterFactories.Remove(Class);
}

void FWorldCookPackageSplitter::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	checkf(InWorld != ReferencedWorld, TEXT("[Cook] %s is being cleaned up while still referenced by a package splitter."), *GetFullNameSafe(InWorld));
}

void FWorldCookPackageSplitter::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ReferencedWorld);
}

FString FWorldCookPackageSplitter::GetReferencerName() const
{
	return TEXT("FWorldCookPackageSplitter");
}

bool FWorldCookPackageSplitter::ShouldSplit(UObject* SplitData)
{
	bool bResult = false;
	if (UWorld* World = Cast<UWorld>(SplitData))
	{
		ForEachRegisteredCookPackageSubSplitterFactories(
			[&bResult, World](const FWorldCookPackageSplitter::FSubSplitterFactory& WorldCookPackageSplitterFactory)
			{
				bResult = WorldCookPackageSplitterFactory.ShouldSplit(World);
				return !bResult;
			});
	}
	return bResult;
}

bool FWorldCookPackageSplitter::UseInternalReferenceToAvoidGarbageCollect()
{
	return true;
}

ICookPackageSplitter::EGeneratedRequiresGenerator FWorldCookPackageSplitter::DoesGeneratedRequireGenerator()
{
	return EGeneratedRequiresGenerator::Populate;
}

bool FWorldCookPackageSplitter::RequiresGeneratorPackageDestructBeforeResplit()
{
	return true;
}

bool FWorldCookPackageSplitter::RequiresEmptyPackageBeforePopulate(const UPackage* Package)
{
	TOptional<bool> Result;

	ForEachCookPackageSubSplitters(
		[Package, &Result](FWorldCookPackageSplitter::ISubSplitter* WorldCookPackageSplitter)
		{
			TOptional<bool> SubResult = WorldCookPackageSplitter->RequiresEmptyPackageBeforePopulate(Package);
			check(!SubResult.IsSet() || !Result.IsSet());
			Result = SubResult;
		});

	if (Result.IsSet())
	{
		return *Result;
	}

	return ICookPackageSplitter::RequiresEmptyPackageBeforePopulate(Package);
}

ICookPackageSplitter::FGenerationManifest FWorldCookPackageSplitter::ReportGenerationManifest(
	const UPackage* OwnerPackage, const UObject* OwnerObject)
{
	TNotNull<const UWorld*> OwnerWorld = CastChecked<const UWorld>(OwnerObject);

	ReferencedWorld = const_cast<UWorld*>(NotNullGet(OwnerWorld));

	check(!bInitializedPhysicsSceneForSave && !bForceInitializedWorld);
	bInitializedPhysicsSceneForSave = GEditor->InitializePhysicsSceneForSaveIfNecessary(ReferencedWorld, bForceInitializedWorld);

	check(!CookPackageSubSplitters.Num());
	ForEachRegisteredCookPackageSubSplitterFactories(
		[this, OwnerWorld](const FWorldCookPackageSplitter::FSubSplitterFactory& WorldCookPackageSplitterFactory)
		{
			if (WorldCookPackageSplitterFactory.ShouldSplit(OwnerWorld))
			{
				CookPackageSubSplitters.Add({ &WorldCookPackageSplitterFactory, WorldCookPackageSplitterFactory.MakeInstance(OwnerWorld) });
			}
		});
	check(CookPackageSubSplitters.Num());

	ICookPackageSplitter::FGenerationManifest Manifest;
	TOptional<bool> SkipSaveGeneratorPackage;

	ForEachCookPackageSubSplitters(
		[this, &Manifest, &SkipSaveGeneratorPackage, OwnerPackage, OwnerWorld](FWorldCookPackageSplitter::ISubSplitter* WorldCookPackageSplitter)
		{
			const TArray<ICookPackageSplitter::FGeneratedPackage> GenerateList = WorldCookPackageSplitter->GetGenerateList(OwnerPackage);
			
			Manifest.GeneratedPackages.Reserve(Manifest.GeneratedPackages.Num() + GenerateList.Num());
			for (const ICookPackageSplitter::FGeneratedPackage& GeneratePackage : GenerateList)
			{
				ICookPackageSplitter::FGeneratedPackage LocalGeneratePackage(GeneratePackage);

				if (LocalGeneratePackage.GeneratedRootPath.IsEmpty())
				{
					LocalGeneratePackage.GeneratedRootPath = OwnerPackage->GetName();
				}

				Manifest.GeneratedPackages.Add(LocalGeneratePackage);

				const TPair<FName, FName> FullPackageName = { *LocalGeneratePackage.GeneratedRootPath, *LocalGeneratePackage.RelativePath };
				check(!SplittersGenerateListMap.Contains(FullPackageName));
				SplittersGenerateListMap.Add(FullPackageName, WorldCookPackageSplitter);
			}

			// Any sub-splitter can disable save of the map
			TOptional<bool> SkipSave = WorldCookPackageSplitter->SkipSaveGeneratorPackage(OwnerPackage);
			if (SkipSave.IsSet())
			{
				if (SkipSaveGeneratorPackage.IsSet() && SkipSave.GetValue() != SkipSaveGeneratorPackage.GetValue())
				{
					UE_LOGF(LogWorldPartition, Error, "[Cook][ReportGenerationManifest] At least 2 sub-splitters disagree on whether to skip the generator package save, which also means 2 sub-splitters are attempting to handle the same package. It will stay enabled as a consequence.");
					SkipSaveGeneratorPackage = false; // Do not skip, as this will avoid missing packages and be easier to debug for unconcerned users.
				}
				else
				{
					SkipSaveGeneratorPackage = SkipSave.GetValue();
				}
			}
		});

	// If any sub-splitter requested that we skip the generator package save, and none requested that we keep it, disable the save. Otherwise the default is to save.
	Manifest.bCookSaveGeneratorPackage = !SkipSaveGeneratorPackage.Get(false);

	FWorldDelegates::OnWorldCleanup.AddRaw(this, &FWorldCookPackageSplitter::OnWorldCleanup);

	return Manifest;
}

bool FWorldCookPackageSplitter::PopulateGeneratedPackage(FPopulateContext& PopulateContext)
{
	const ICookPackageSplitter::FGeneratedPackageForPopulate* TargetGeneratedPackage = PopulateContext.GetTargetGeneratedPackage();
	const TPair<FName, FName> FullPackageName = { *TargetGeneratedPackage->GeneratedRootPath, *TargetGeneratedPackage->RelativePath };
	ISubSplitter* SubSplitter = SplittersGenerateListMap.FindChecked(FullPackageName);
	return SubSplitter->PopulateGeneratedPackage(PopulateContext);
}

bool FWorldCookPackageSplitter::PopulateGeneratorPackage(FPopulateContext& PopulateContext)
{
	bool bResult = false;
	ForEachCookPackageSubSplitters(
		[&bResult, &PopulateContext](FWorldCookPackageSplitter::ISubSplitter* WorldCookPackageSplitter)
		{
			bResult = WorldCookPackageSplitter->PopulateGeneratorPackage(PopulateContext);
			return bResult;
		});
	return bResult;
}

void FWorldCookPackageSplitter::OnOwnerReloaded(UPackage* OwnerPackage, UObject* OwnerObject)
{
	check(!ReferencedWorld);
}

void FWorldCookPackageSplitter::Teardown(ICookPackageSplitter::ETeardown Status)
{
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

	ForEachCookPackageSubSplitters(
		[Status](FWorldCookPackageSplitter::ISubSplitter* WorldCookPackageSplitter)
		{
			WorldCookPackageSplitter->Teardown(Status);
		});

	SplittersGenerateListMap.Empty();

	for (const TPair<const FWorldCookPackageSplitter::FSubSplitterFactory*, ISubSplitter*>& CookPackageSubSplitterPair : CookPackageSubSplitters)
	{
		CookPackageSubSplitterPair.Key->ReleaseInstance(ReferencedWorld, CookPackageSubSplitterPair.Value);
	}
	CookPackageSubSplitters.Empty();

	if (bInitializedPhysicsSceneForSave)
	{
		GEditor->CleanupPhysicsSceneThatWasInitializedForSave(ReferencedWorld, bForceInitializedWorld);
		bInitializedPhysicsSceneForSave = false;
		bForceInitializedWorld = false;
	}

	ReferencedWorld = nullptr;
}
#endif // WITH_EDITOR