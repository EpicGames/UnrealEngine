// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldGridPreviewer.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#include "Engine/World.h"
#include "Engine/PostProcessVolume.h"
#include "Materials/MaterialInstanceConstant.h"

FWorldGridPreviewer::FWorldGridPreviewer()
	: CellSize(0)
	, LoadingRange(0)
	, GridColor(ForceInit)
	, GridOffset(ForceInit)
	, World(nullptr)
{}

FWorldGridPreviewer::FWorldGridPreviewer(UWorld* InWorld, bool bInIs2D)
	: FWorldGridPreviewer()
{
	World = InWorld;

	if (!World)
	{
		return;
	}

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.ObjectFlags |= RF_Transient;
	SpawnParameters.ObjectFlags &= ~RF_Transactional;

	PostProcessVolume = World->SpawnActor<APostProcessVolume>(SpawnParameters);
	Material = PostProcessVolume.IsValid() ? LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/WorldGridPreviewMaterial")) : nullptr;
	MaterialInstance = NewObject<UMaterialInstanceConstant>();
	MaterialInstance->Parent = Material;

	if (MaterialInstance)
	{
		FStaticParameterSet StaticParams;
		MaterialInstance->GetStaticParameterValues(StaticParams);

		static FName NAME_IsGrid2D("Is_Grid_2D");
		for (int ParamIdx = 0; ParamIdx < StaticParams.StaticSwitchParameters.Num(); ParamIdx++)
		{
			if (StaticParams.StaticSwitchParameters[ParamIdx].ParameterInfo.Name == NAME_IsGrid2D)
			{
				MaterialInstance->SetStaticSwitchParameterValueEditorOnly(StaticParams.StaticSwitchParameters[ParamIdx].ParameterInfo, bInIs2D);
				break;
			}
		}

		MaterialInstance->UpdateStaticPermutation();

		PostProcessVolume->Settings.WeightedBlendables.Array.Emplace(1.0f, MaterialInstance);
		PostProcessVolume->bUnbound = true;
		PostProcessVolume->bEnabled = true;
	}
}

FWorldGridPreviewer::~FWorldGridPreviewer()
{
	if (PostProcessVolume.IsValid())
	{
		World->DestroyActor(PostProcessVolume.Get());
	}
}

void FWorldGridPreviewer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(Material);
	InCollector.AddReferencedObject(MaterialInstance);
}

void FWorldGridPreviewer::Update()
{
	if (MaterialInstance)
	{
		MaterialInstance->SetScalarParameterValueEditorOnly(TEXT("Grid_CellSize"), (float)CellSize);
		MaterialInstance->SetScalarParameterValueEditorOnly(TEXT("Grid_LoadingRange"), (float)LoadingRange);
		MaterialInstance->SetVectorParameterValueEditorOnly(TEXT("Grid_Color"), GridColor);
		// Grid material has cell's origin at center, and our grid are using top-left
		MaterialInstance->SetVectorParameterValueEditorOnly(TEXT("Grid_Offset"), FLinearColor(GridOffset + CellSize * 0.5f));
	}
}
#endif