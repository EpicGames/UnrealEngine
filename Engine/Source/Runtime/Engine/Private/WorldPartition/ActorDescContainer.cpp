// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorDescContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorDescContainer)

#if WITH_EDITOR
#include "Editor.h"
#include "Algo/Transform.h"
#include "Engine/Level.h"
#include "ExternalPackageHelper.h"
#include "UObject/ObjectSaveContext.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "String/Split.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionClassDescRegistry.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "DeletedObjectPlaceholder.h"

UActorDescContainer::FActorDescContainerInitializeDelegate UActorDescContainer::OnActorDescContainerInitialized;

FUObjectAnnotationSparse<FDeletedObjectPlaceholderAnnotation, true> UActorDescContainer::DeletedObjectPlaceholdersAnnotation;

FDeletedObjectPlaceholderAnnotation::FDeletedObjectPlaceholderAnnotation(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder, const FString& InActorDescContainerName)
	: DeletedObjectPlaceholder(InDeletedObjectPlaceholder)
	, ActorDescContainerName(InActorDescContainerName)
{
}

UActorDescContainer* FDeletedObjectPlaceholderAnnotation::GetActorDescContainer() const
{
	UActorDescContainerSubsystem* ActorDescContainerSubsystem = UActorDescContainerSubsystem::Get();
	UActorDescContainer* ActorDescContainer = ActorDescContainerSubsystem ? ActorDescContainerSubsystem->GetActorDescContainer(ActorDescContainerName) : nullptr;
	return ActorDescContainer;
}

#endif

UActorDescContainer::UActorDescContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bContainerInitialized(false)
	, bRegisteredDelegates(false)
#endif
{}

#if WITH_EDITOR
void UActorDescContainer::Initialize(const FInitializeParams& InitParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainer::Initialize);
	check(!bContainerInitialized);
	if (InitParams.PreInitialize)
	{
		InitParams.PreInitialize(this);
	}

	ContainerPackageName = InitParams.PackageName;
	if (InitParams.ExternalDataLayerAsset)
	{
		ensure(!InitParams.ContentBundleGuid.IsValid());
		ExternalDataLayerAsset = InitParams.ExternalDataLayerAsset;
	}
	else if (InitParams.ContentBundleGuid.IsValid())
	{
		ContentBundleGuid = InitParams.ContentBundleGuid;
	}

	TArray<FAssetData> ExternalAssets;
	TArray<FString> InternalAssets;
	if (!ContainerPackageName.IsNone() && !FPackageName::IsTempPackage(ContainerPackageName.ToString()))
	{
		const FString ContainerExternalActorsPath = GetExternalActorPath();

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		// Do a synchronous scan of the level external actors path.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ScanSynchronous);
			AssetRegistry.ScanSynchronous({ ContainerExternalActorsPath }, TArray<FString>());
		}

		// Gather external actors
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetExternalAssets);

			FARFilter Filter;
			Filter.bRecursivePaths = true;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.PackagePaths.Add(*ContainerExternalActorsPath);

			FExternalPackageHelper::GetSortedAssets(Filter, ExternalAssets);
		}

		// Gather non-external actors
		if (InitParams.bShouldGatherInternalActors)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetInternalAssets);

			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.PackageNames.Add(ContainerPackageName);

			TArray<FAssetData> WorldAssetData;
			AssetRegistry.GetAssets(Filter, WorldAssetData);

			// Transform world assets
			static FName NAME_ActorsMetaData("ActorsMetaData");
			for (const FAssetData& AssetData : WorldAssetData)
			{
				FString ActorsMetaDataStr;
				if (AssetData.GetTagValue(NAME_ActorsMetaData, ActorsMetaDataStr))
				{
					TArray<FString> ActorsMetaData;
					if (ActorsMetaDataStr.ParseIntoArray(ActorsMetaData, TEXT(";")))
					{
						InternalAssets.Append(ActorsMetaData);
					}
				}
			}
		}
	}

	UE_LOGF(LogWorldPartition, Verbose, "Parsed actor descriptor container package '%ls': %d external actors, %d internal actors", *InitParams.PackageName.ToString(), ExternalAssets.Num(), InternalAssets.Num());

	TSet<FTopLevelAssetPath> ClassPathsToPrefetch;
	FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherDescriptorsClass);
		
		for (const FAssetData& Asset : ExternalAssets)
		{
			ClassPathsToPrefetch.Add(Asset.AssetClassPath);
		}

		for (const FString& InternalAsset : InternalAssets)
		{
			FWorldPartitionActorDescUtils::FActorDescInitParams ActorDescInitParams(InternalAsset);

			if (!ActorDescInitParams.BaseClassName.IsNone())
			{
				ClassPathsToPrefetch.Emplace(ActorDescInitParams.BaseClassName.ToString());
			}
			else
			{
				ClassPathsToPrefetch.Emplace(ActorDescInitParams.NativeClassName.ToString());
			}
		}

		ClassDescRegistry.PrefetchClassDescs(ClassPathsToPrefetch.Array());
	}

	TMap<FGuid, TUniquePtr<FWorldPartitionActorDesc>> ValidActorDescs;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateDescriptors);

		TMap<FName, FWorldPartitionActorDesc*> ActorDescsByPackage;
		for (const FAssetData& Asset : ExternalAssets)
		{
			TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(Asset);
								
			if (!ActorDesc.IsValid())
			{
				UE_LOGF(LogWorldPartition, Warning, "Invalid actor descriptor for actor '%ls' from package '%ls'", *Asset.GetObjectPathString(), *Asset.PackageName.ToString());
				InvalidActors.Emplace(Asset);
			} 
			else if (!ActorDesc->GetNativeClass().IsValid())
			{
				UE_LOGF(LogWorldPartition, Warning, "Invalid actor native class: Actor: '%ls' (guid '%ls') from package '%ls'",
					*ActorDesc->GetActorNameString(),
					*ActorDesc->GetGuid().ToString(),
					*ActorDesc->GetActorPackage().ToString());
				InvalidActors.Emplace(Asset);
			}
			else if (ActorDesc->GetBaseClass().IsValid() && !ClassDescRegistry.IsRegisteredClass(ActorDesc->GetBaseClass()))
			{
				UE_LOGF(LogWorldPartition, Warning, "Unknown actor base class `%ls`: Actor: '%ls' (guid '%ls') from package '%ls'",
					*ActorDesc->GetBaseClass().ToString(),
					*ActorDesc->GetActorNameString(),
					*ActorDesc->GetGuid().ToString(),
					*ActorDesc->GetActorPackage().ToString());
				InvalidActors.Emplace(Asset);
			}
			else if (InitParams.FilterActorDesc && !InitParams.FilterActorDesc(ActorDesc.Get()))
			{
				InvalidActors.Emplace(Asset);
			}
			// At this point, the actor descriptor is well formed and valid on its own. We now make validations based on the already registered
			// actor descriptors, such as duplicated actor GUIDs or multiple actors in the same package, etc.
			else if (FWorldPartitionActorDesc* ExistingDescPackage = ActorDescsByPackage.FindRef(ActorDesc->GetActorPackage()))
			{
				UE_LOGF(LogWorldPartition, Warning, "Duplicate actor descriptor in package `%ls`: Actor: '%ls' -> Existing actor '%ls'", 
					*ActorDesc->GetActorPackage().ToString(), 
					*ActorDesc->GetActorNameString(), 
					*ExistingDescPackage->GetActorNameString());

				// No need to add all actors in the same package several times as we only want to open the package for delete when repairing
				if (ValidActorDescs.Contains(ActorDesc->GetGuid()))
				{
					InvalidActors.Emplace(Asset);
					ValidActorDescs.Remove(ActorDesc->GetGuid());
				}
			}
			else if (TUniquePtr<FWorldPartitionActorDesc>* ExistingActorDescPtr = ValidActorDescs.Find(ActorDesc->GetGuid()))
			{
				const FWorldPartitionActorDesc* ExistingActorDesc = ExistingActorDescPtr->Get();
				check(ExistingActorDesc->GetGuid() == ActorDesc->GetGuid());
				UE_LOGF(LogWorldPartition, Warning, "Duplicate actor descriptor guid `%ls`: Actor: '%ls' from package '%ls' -> Existing actor '%ls' from package '%ls'", 
					*ActorDesc->GetGuid().ToString(), 
					*ActorDesc->GetActorNameString(), 
					*ActorDesc->GetActorPackage().ToString(),
					*ExistingActorDesc->GetActorNameString(),
					*ExistingActorDesc->GetActorPackage().ToString());
				InvalidActors.Emplace(Asset);
			}
			else
			{
				ActorDescsByPackage.Add(ActorDesc->GetActorPackage(), ActorDesc.Get());
				ValidActorDescs.Add(ActorDesc->GetGuid(), MoveTemp(ActorDesc));
			}
		}

		for (const FString& InternalAsset : InternalAssets)
		{
			FWorldPartitionActorDescUtils::FActorDescInitParams ActorDescInitParams(InternalAsset);

			TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromInitParams(ActorDescInitParams, ContainerPackageName);

			if (!ActorDesc.IsValid())
			{
				UE_LOGF(LogWorldPartition, Warning, "Invalid actor descriptor for actor '%ls' from package '%ls'", *ActorDescInitParams.PathName.ToString(), *InitParams.PackageName.ToString());
			} 
			else if (!ActorDesc->GetNativeClass().IsValid())
			{
				UE_LOGF(LogWorldPartition, Warning, "Invalid actor native class: Actor: '%ls' (guid '%ls') from package '%ls'",
					*ActorDesc->GetActorNameString(),
					*ActorDesc->GetGuid().ToString(),
					*ActorDesc->GetActorPackage().ToString());
			}
			else if (ActorDesc->GetBaseClass().IsValid() && !ClassDescRegistry.IsRegisteredClass(ActorDesc->GetBaseClass()) && ClassPathsToPrefetch.Contains(ActorDesc->GetBaseClass()))
			{
				UE_LOGF(LogWorldPartition, Warning, "Unknown actor base class `%ls`: Actor: '%ls' (guid '%ls') from package '%ls'",
					*ActorDesc->GetBaseClass().ToString(),
					*ActorDesc->GetActorNameString(),
					*ActorDesc->GetGuid().ToString(),
					*ActorDesc->GetActorPackage().ToString());
			}
			else if (!InitParams.FilterActorDesc || InitParams.FilterActorDesc(ActorDesc.Get()))
			{
				// At this point, the actor descriptor is well formed and valid on its own. We now make validations based on the already registered
				// actor descriptors, such as duplicated actor GUIDs or multiple actors in the same package, etc.
				if (TUniquePtr<FWorldPartitionActorDesc>* ExistingActorDescPtr = ValidActorDescs.Find(ActorDesc->GetGuid()))
				{
					const FWorldPartitionActorDesc* ExistingActorDesc = ExistingActorDescPtr->Get();
					check(ExistingActorDesc->GetGuid() == ActorDesc->GetGuid());
					UE_LOGF(LogWorldPartition, Warning, "Duplicate actor descriptor guid `%ls`: Actor: '%ls' from package '%ls' -> Existing actor '%ls' from package '%ls'", 
						*ActorDesc->GetGuid().ToString(), 
						*ActorDesc->GetActorNameString(), 
						*ActorDesc->GetActorPackage().ToString(),
						*ExistingActorDesc->GetActorNameString(),
						*ExistingActorDesc->GetActorPackage().ToString());
				}
				else
				{
					ActorDescsByPackage.Add(ActorDesc->GetActorPackage(), ActorDesc.Get());
					ValidActorDescs.Add(ActorDesc->GetGuid(), MoveTemp(ActorDesc));
				}
			}
		}

		if (UActorDescContainerSubsystem* ActorDescContainerSubsystem = UActorDescContainerSubsystem::Get())
		{
			InvalidActors.Append(ActorDescContainerSubsystem->InvalidMapAssets.FindRef(ContainerPackageName).Array());
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RegisterDescriptors);
		for (auto& [ActorGuid, ActorDesc] : ValidActorDescs)
		{
			AddChildActorToParentMap(ActorDesc.Get());
			RegisterActorDescriptor(ActorDesc.Release());
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateActorToWorld);
		for (FActorDescList::TIterator<> ActorDescIterator(this); ActorDescIterator; ++ActorDescIterator)
		{
			// Update ActorToWorld for all actors, starting from "root" actors
			if (ActorDescIterator->GetParentActor().IsValid() == false)
			{
				ActorDescIterator->UpdateActorToWorld();
				PropagateActorToWorldUpdate(*ActorDescIterator);
			}
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnActorDescContainerInitialized)
		OnActorDescContainerInitialized.Broadcast(this);
	}

	bRegisteredDelegates = InitParams.bShouldRegisterEditorDeletages && ShouldRegisterDelegates();
	
	if (bRegisteredDelegates)
	{
		RegisterEditorDelegates();
	}

	bContainerInitialized = true;
}

void UActorDescContainer::Uninitialize()
{
	if (bContainerInitialized)
	{
		if (bRegisteredDelegates)
		{
			UnregisterEditorDelegates();
			bRegisteredDelegates = false;
		}
		bContainerInitialized = false;
	}

	for (TUniquePtr<FWorldPartitionActorDesc>& ActorDescPtr : ActorDescList)
	{
		if (FWorldPartitionActorDesc* ActorDesc = ActorDescPtr.Get())
		{
			UnregisterActorDescriptor(ActorDesc);
		}
		ActorDescPtr.Reset();
	}
}

void UActorDescContainer::BeginDestroy()
{
	Super::BeginDestroy();

	Uninitialize();
}

FString UActorDescContainer::GetExternalActorPath() const
{
	return ULevel::GetExternalActorsPath(ContainerPackageName.ToString());
}

FString UActorDescContainer::GetExternalObjectPath() const
{
	return FExternalPackageHelper::GetExternalObjectsPath(ContainerPackageName.ToString());
}

bool UActorDescContainer::HasExternalContent() const
{
	check(!ExternalDataLayerAsset || ExternalDataLayerAsset->GetUID().IsValid());
	return ExternalDataLayerAsset ? true : GetContentBundleGuid().IsValid();
}

bool UActorDescContainer::IsActorDescHandled(const AActor* InActor, bool bInUseLoadedPath) const
{
	// Actor External Content Guid must match Container's External Content Guid to be considered
	// AWorldDataLayers actors are an exception as they don't have an External Content Guid
	const bool bIsCandidateActor = InActor->IsA<AWorldDataLayers>() ||
		(!HasExternalContent() && !InActor->HasExternalContent()) ||
		(ExternalDataLayerAsset && (ExternalDataLayerAsset == InActor->GetExternalDataLayerAsset())) ||
		(ContentBundleGuid.IsValid() && (ContentBundleGuid == InActor->GetContentBundleGuid()));

	if (bIsCandidateActor)
	{
		const FName LoadedPackageName = InActor->GetPackage()->GetLoadedPath().GetPackageFName();
		const FString ActorPackageName = bInUseLoadedPath && !LoadedPackageName.IsNone() ? LoadedPackageName.ToString() : InActor->GetPackage()->GetName();

		if (InActor->GetExternalPackage())
		{
			const FString ExternalActorPath = GetExternalActorPath() / TEXT("");
			return ActorPackageName.StartsWith(ExternalActorPath);
		}
		else if (InActor->HasAllFlags(RF_Transient))
		{
			return ActorPackageName == LoadedPackageName.ToString();
		}
	}
	return false;
}

void UActorDescContainer::ReassignActorDesc(TUniquePtr<FWorldPartitionActorDesc>& DescToUpdate, TFunctionRef<void(TUniquePtr<FWorldPartitionActorDesc>& ActorDesc)> UpdateAction)
{
	// 1 - Prepare users of the actor desc
	OnActorDescPreUpdate(DescToUpdate.Get());
	// 2 - Apply the transform, which is a function that depends on the context of the change, and which may have created a new ActorDesc and destroyed the old one, updating the passed-in unique ptr
	UpdateAction(DescToUpdate);
	// 3 - Update the pointer when needed, on all container instances, before proceeding with computing actor hashes, etc. because this may involve using other containers
	OnActorDescUpdating(DescToUpdate.Get());
	// 4 - Carry on processing the new actor desc
	OnActorDescUpdated(DescToUpdate.Get());
}

void UActorDescContainer::UpdateActorDesc(FWorldPartitionActorDesc* ActorDesc, TFunctionRef<void(FWorldPartitionActorDesc* ActorDesc)> UpdateAction)
{
	// 1 - Prepare users of the actor desc
	OnActorDescPreUpdate(ActorDesc);
	// 2 - Apply the transform, which is a function that depends on the context of the change, and which may have created a new ActorDesc and destroyed the old one, updating the passed-in unique ptr
	UpdateAction(ActorDesc);
	// 3 - Carry on processing the modified actor desc
	OnActorDescUpdated(ActorDesc);
}

bool UActorDescContainer::IsActorDescHandled(const AActor* InActor) const
{
	return IsActorDescHandled(InActor, false);
}

void UActorDescContainer::RegisterActorDescriptor(FWorldPartitionActorDesc* InActorDesc)
{
	FActorDescList::AddActorDescriptor(InActorDesc);

	InActorDesc->SetContainer(this);
		
	ActorsByName.Add(InActorDesc->GetActorName(), ActorsByGuid.FindChecked(InActorDesc->GetGuid()));

	UE_LOGF(LogWorldPartition, Verbose, "\tRegistered actor descriptor '%ls'", *InActorDesc->ToString(FWorldPartitionActorDesc::EToStringMode::Verbose));
}

void UActorDescContainer::UnregisterActorDescriptor(FWorldPartitionActorDesc* InActorDesc)
{
	FActorDescList::RemoveActorDescriptor(InActorDesc);
	InActorDesc->SetContainer(nullptr);

	if (!ActorsByName.Remove(InActorDesc->GetActorName()))
	{
		bool bRemoved = false;
		for (auto& [ActorName, ActorDesc] : ActorsByName)
		{
			if ((*ActorDesc)->GetGuid() == InActorDesc->GetGuid())
			{
				UE_LOGF(LogWorldPartition, Log, "Removed actor '%ls' from container '%ls' with unexpected name `%ls`", *InActorDesc->GetActorNameString(), *ContainerPackageName.ToString(), *ActorName.ToString());
				ActorsByName.Remove(ActorName);				
				bRemoved = true;
				break;
			}
		}

		UE_CLOGF(!bRemoved, LogWorldPartition, Log, "Missing actor '%ls' from container '%ls'", *InActorDesc->GetActorNameString(), *ContainerPackageName.ToString());
	}
}

bool UActorDescContainer::ShouldHandleActorEvent(const AActor* Actor, bool bInUseLoadedPath) const
{
	return Actor && IsActorDescHandled(Actor, bInUseLoadedPath) && Actor->IsMainPackageActor() && Actor->GetLevel();
}

bool UActorDescContainer::ShouldHandleActorEvent(const AActor* Actor)
{
	return ShouldHandleActorEvent(Actor, false);
}

const FWorldPartitionActorDesc* UActorDescContainer::GetActorDescByPath(const FString& ActorPath) const
{
	FStringView ActorName;
	FStringView ActorContext;
	if (!UE::String::SplitLastChar(FStringView(ActorPath), TEXT('.'), ActorContext, ActorName, ESearchCase::CaseSensitive))
	{
		ActorName = ActorPath;
	}

	return GetActorDescByName(FName(ActorName));
}

const FWorldPartitionActorDesc* UActorDescContainer::GetActorDescByPath(const FSoftObjectPath& ActorPath) const
{
	return GetActorDescByPath(ActorPath.ToString());
}

const FWorldPartitionActorDesc* UActorDescContainer::GetActorDescByName(FName ActorName) const
{
	if (const TUniquePtr<FWorldPartitionActorDesc>* const* ActorDesc = ActorsByName.Find(ActorName))
	{
		return (*ActorDesc)->Get();
	}

	return nullptr;
}

bool UActorDescContainer::ShouldHandleDeletedObjectPlaceholderEvent(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder) const
{
	const FExternalDataLayerUID ContainerExternalDataLayerUID = ExternalDataLayerAsset ? ExternalDataLayerAsset->GetUID() : FExternalDataLayerUID();
	if (ContainerExternalDataLayerUID == InDeletedObjectPlaceholder->GetExternalDataLayerUID())
	{
		const FString PackageName = InDeletedObjectPlaceholder->GetPackage()->GetName();
		const FString ContainerExternalActorPath = GetExternalActorPath() / TEXT("");
		return PackageName.StartsWith(ContainerExternalActorPath);
	}
	return false;
};

void UActorDescContainer::OnDeletedObjectPlaceholderCreated(const UDeletedObjectPlaceholder* InDeletedObjectPlaceholder)
{
	const UObject* OriginalObject = InDeletedObjectPlaceholder->GetOriginalObject();
	if (const AActor* Actor = Cast<AActor>(OriginalObject))
	{
		if (ShouldHandleDeletedObjectPlaceholderEvent(InDeletedObjectPlaceholder))
		{
			if (GetActorDescriptor(Actor->GetActorGuid()))
			{
				DeletedObjectPlaceholdersAnnotation.AddAnnotation(Actor, FDeletedObjectPlaceholderAnnotation(InDeletedObjectPlaceholder, GetContainerName()));
			}
		}
	}
}

void UActorDescContainer::OnObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	if (!SaveContext.IsProceduralSave() && !SaveContext.IsFromAutoSave())
	{
		if (const AActor* Actor = Cast<AActor>(Object))
		{
			if (ShouldHandleActorEvent(Actor))
			{
				check(IsValidChecked(Actor));

				// Handle the case where the actor changed package but the old/empty package has not been processed/deleted
				// One case where this can happen is if the user choses to save the new package but unchecks the deleted package
				// Remove(unhash) the corresponding original actor (guid) from its original container before adding the new one (unhash before hashing)
				if (FDeletedObjectPlaceholderAnnotation Annotation = DeletedObjectPlaceholdersAnnotation.GetAndRemoveAnnotation(Actor); Annotation.IsValid())
				{
					// In the case where the object changed to a new container and created a new package, then changed back to its original location,
					// OnDeletedObjectPlaceholderCreated will not be called for the newly created package
					// This is why we need to validate that the annotation's container is still relevant by the annotation's DeletedObjectPlaceholder using ShouldHandleDeletedObjectPlaceholderEvent
					UActorDescContainer* ActorDescContainer = Annotation.GetActorDescContainer();
					if (ActorDescContainer && ActorDescContainer->ShouldHandleDeletedObjectPlaceholderEvent(Annotation.GetDeletedObjectPlaceholder()))
					{
						check(Annotation.GetDeletedObjectPlaceholder()->GetOriginalObject() == Actor);
						verify(ActorDescContainer->RemoveActor(Actor->GetActorGuid()));
					}
				}
				
				if (TUniquePtr<FWorldPartitionActorDesc>* ExistingActorDesc = GetActorDescriptor(Actor->GetActorGuid()))
				{
					// Existing actor
					RemoveChildActorFromParentMap(ExistingActorDesc->Get());

					ReassignActorDesc(*ExistingActorDesc,
						[Actor](TUniquePtr<FWorldPartitionActorDesc>& ActorDesc)
						{
							// This changes the actor desc pointer
							FWorldPartitionActorDescUtils::UpdateActorDescriptorFromActor(Actor, ActorDesc);
						}
					);

					AddChildActorToParentMap(ExistingActorDesc->Get());

					PropagateActorToWorldUpdate(ExistingActorDesc->Get());
				}
				else
				{
					// New actor
					FWorldPartitionActorDesc* AddedActorDesc = Actor->CreateActorDesc().Release();
					RegisterActorDescriptor(AddedActorDesc);
					OnActorDescAdded(AddedActorDesc);
					AddChildActorToParentMap(AddedActorDesc);

					PropagateActorToWorldUpdate(AddedActorDesc);
				}
			}
		}
	}
}

void UActorDescContainer::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewObjectMap)
{
	OnObjectsReplacedEvent.Broadcast(InOldToNewObjectMap);
}

void UActorDescContainer::OnPackageDeleted(UPackage* Package)
{
	if (const AActor* Actor = AActor::FindActorInPackage(Package))
	{
		if (ShouldHandleActorEvent(Actor))
		{
			RemoveActor(Actor->GetActorGuid());
		}
	}
	else if (UDeletedObjectPlaceholder* DeletedObjectPlaceholder = UDeletedObjectPlaceholder::FindInPackage(Package))
	{
		if (ShouldHandleDeletedObjectPlaceholderEvent(DeletedObjectPlaceholder))
		{
			// Here we validate that we didn't already processed the DeletedObjectPlaceholder in OnObjectPreSave
			if (const AActor* OriginalActor = Cast<AActor>(DeletedObjectPlaceholder->GetOriginalObject()))
			{
				if (FDeletedObjectPlaceholderAnnotation Annotation = DeletedObjectPlaceholdersAnnotation.GetAndRemoveAnnotation(OriginalActor); Annotation.IsValid())
				{
					check(Annotation.GetDeletedObjectPlaceholder() == DeletedObjectPlaceholder);
					check(Annotation.GetActorDescContainer() == this);
					verify(RemoveActor(OriginalActor->GetActorGuid()));
				}
			}
		}
	}
}

void UActorDescContainer::OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor)
{
	OnEditorActorReplacedEvent.Broadcast(InOldActor, InNewActor);
}

void UActorDescContainer::OnClassDescriptorUpdated(const FWorldPartitionActorDesc* InClassDesc)
{
	FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();

	TArray<FString> ActorPackages;
	for (FActorDescList::TIterator<> ActorDescIterator(this); ActorDescIterator; ++ActorDescIterator)
	{
		if (ActorDescIterator->GetBaseClass().IsValid())
		{
			if (const FWorldPartitionActorDesc* ActorClassDesc = ClassDescRegistry.GetClassDescDefaultForActor(ActorDescIterator->GetBaseClass()))
			{
				if (ClassDescRegistry.IsDerivedFrom(ActorClassDesc, InClassDesc))
				{
					ActorPackages.Add(ActorDescIterator->GetActorPackage().ToString());
				}
			}
		}
	}

	if (ActorPackages.Num())
	{
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackageNames.Reserve(ActorPackages.Num());
		Algo::Transform(ActorPackages, Filter.PackageNames, [](const FString& ActorPath) { return *ActorPath; });

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.ScanSynchronous(TArray<FString>(), ActorPackages);

		TArray<FAssetData> Assets;
		AssetRegistry.GetAssets(Filter, Assets);

		for (const FAssetData& Asset : Assets)
		{
			TUniquePtr<FWorldPartitionActorDesc> NewActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(Asset);
								
			if (NewActorDesc.IsValid() && NewActorDesc->GetNativeClass().IsValid())
			{
				if (TUniquePtr<FWorldPartitionActorDesc>* ExistingActorDesc = GetActorDescriptor(NewActorDesc->GetGuid()))
				{
					ReassignActorDesc(*ExistingActorDesc,
						[&NewActorDesc](TUniquePtr<FWorldPartitionActorDesc>& ActorDesc)
						{
							// This changes the actor desc pointer
							FWorldPartitionActorDescUtils::UpdateActorDescriptorFromActorDescriptor(NewActorDesc, ActorDesc);
						}
					);
				}
			}
		}
	}
}

bool UActorDescContainer::RemoveActor(const FGuid& ActorGuid)
{
	if (TUniquePtr<FWorldPartitionActorDesc>* ExistingActorDesc = GetActorDescriptor(ActorGuid))
	{
		RemoveChildActorFromParentMap(ExistingActorDesc->Get());
		OnActorDescRemoved(ExistingActorDesc->Get());
		UnregisterActorDescriptor(ExistingActorDesc->Get());
		ExistingActorDesc->Reset();
		return true;
	}

	return false;
}

bool UActorDescContainer::ShouldRegisterDelegates() const
{
	return GEditor && !IsTemplate() && !IsRunningCookCommandlet();
}

void UActorDescContainer::RegisterEditorDelegates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainer::RegisterEditorDelegates);

	FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UActorDescContainer::OnObjectPreSave);
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UActorDescContainer::OnObjectsReplaced);
	FEditorDelegates::OnPackageDeleted.AddUObject(this, &UActorDescContainer::OnPackageDeleted);
	FEditorDelegates::OnEditorActorReplaced.AddUObject(this, &UActorDescContainer::OnEditorActorReplaced);

	FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();
	ClassDescRegistry.OnClassDescriptorUpdated().AddUObject(this, &UActorDescContainer::OnClassDescriptorUpdated);

	UDeletedObjectPlaceholder::OnObjectCreated.AddUObject(this, &UActorDescContainer::OnDeletedObjectPlaceholderCreated);
}

void UActorDescContainer::UnregisterEditorDelegates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainer::UnregisterEditorDelegates);

	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FEditorDelegates::OnPackageDeleted.RemoveAll(this);
	FEditorDelegates::OnEditorActorReplaced.RemoveAll(this);

	FEditorDelegates::OnPackageDeleted.RemoveAll(this);

	FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();
	ClassDescRegistry.OnClassDescriptorUpdated().RemoveAll(this);

	UDeletedObjectPlaceholder::OnObjectCreated.RemoveAll(this);
}

void UActorDescContainer::OnActorDescAdded(FWorldPartitionActorDesc* NewActorDesc)
{
	OnActorDescAddedEvent.Broadcast(NewActorDesc);
}

void UActorDescContainer::OnActorDescRemoved(FWorldPartitionActorDesc* ActorDesc)
{
	OnActorDescRemovedEvent.Broadcast(ActorDesc);
}

void UActorDescContainer::OnActorDescPreUpdate(FWorldPartitionActorDesc* ActorDesc)
{
	// Pass to prepare for change to the ActorDesc
	OnActorDescPreUpdateEvent.Broadcast(ActorDesc);
}

void UActorDescContainer::OnActorDescUpdating(FWorldPartitionActorDesc* ActorDesc)
{
	// Pass to update all pointers, because the previous pointer is now deleted
	OnActorDescUpdatingEvent.Broadcast(ActorDesc);
}

void UActorDescContainer::OnActorDescUpdated(FWorldPartitionActorDesc* ActorDesc)
{
	// Pass to process the new ActorDesc, which may include using other containers which must also point to the updated ActorDesc
	OnActorDescUpdatedEvent.Broadcast(ActorDesc);
}

void UActorDescContainer::AddChildActorToParentMap(FWorldPartitionActorDesc* ActorDesc)
{
	FGuid ParentActorGuid = ActorDesc->GetParentActor();
	if (ParentActorGuid.IsValid())
	{
		ParentActorToChildrenMap.FindOrAdd(ParentActorGuid).Add(ActorDesc->GetGuid());
	}
}

void UActorDescContainer::RemoveChildActorFromParentMap(FWorldPartitionActorDesc* ActorDesc)
{
	FGuid ParentActorGuid = ActorDesc->GetParentActor();
	if (ParentActorGuid.IsValid())
	{
		if (TSet<FGuid>* ChildActors = ParentActorToChildrenMap.Find(ParentActorGuid))
		{
			ChildActors->Remove(ActorDesc->GetGuid());
			if (ChildActors->IsEmpty())
			{
				ParentActorToChildrenMap.Remove(ParentActorGuid);
			}
		}
	}
}

void UActorDescContainer::ForEachChildActorDesc(const FGuid& InActorGuid, TFunctionRef<void(const FGuid&)> InFunc)
{
	TSet<FGuid> CycleDetector;

	TFunction<void(const FGuid&)> ForEachChildActorDescInternal = [&ForEachChildActorDescInternal, Func = InFunc, &CycleDetector, this](const FGuid& ActorGuid)
	{
		bool bAlreadyVisited = false;
		CycleDetector.Add(ActorGuid, &bAlreadyVisited);
		if (bAlreadyVisited)
		{
			// Each actor can have at most one parent, which means that it can appear on at most one Children list in ParentActorToChildrenMap.
			// Because of that, reaching the same actor twice, in one series of PropagateActorToWorldUpdate calls, means that we found a cycle.
			// That can happen when the actor currently being saved becomes a descendant of one of its ex-descendants
			// and that ex-descendant actor is not saved yet. In that case we can break the cycle here and stop propagating updates. 
			// They'll be properly propagated once the "ex-descendant" actor mentioned above gets saved.
			return;
		}

		if (const TSet<FGuid>* Children = ParentActorToChildrenMap.Find(ActorGuid))
		{
			for (const FGuid& ChildGuid : *Children)
			{
				if (ensure(ChildGuid.IsValid()))
				{
					Func(ChildGuid);
					ForEachChildActorDescInternal(ChildGuid);
				}
			}
		}
	};

	ForEachChildActorDescInternal(InActorGuid);
}

void UActorDescContainer::PropagateActorToWorldUpdate(FWorldPartitionActorDesc* ActorDesc)
{
	ForEachChildActorDesc(ActorDesc->GetGuid(), [this](const FGuid& ChildActorGuid)
	{
		if (FWorldPartitionActorDesc* ChildActor = GetActorDesc(ChildActorGuid))
		{
			UpdateActorDesc(ChildActor,
				[](FWorldPartitionActorDesc* ActorDesc)
				{
					ActorDesc->UpdateActorToWorld();
				}
			);
		}
	});
}
#endif

