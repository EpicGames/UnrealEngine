// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorDescContainerInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorDescContainerInstance)

#if WITH_EDITOR
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "Editor.h"
#endif

#if WITH_EDITOR
UActorDescContainerInstance::FActorDescContainerInstanceInitializeDelegate UActorDescContainerInstance::OnActorDescContainerInstanceInitialized;
UActorDescContainerInstance::FActorDescInstanceAddedToContainerEvent UActorDescContainerInstance::OnActorDescInstanceAddedToContainer;
UActorDescContainerInstance::FActorDescInstanceRemovedFromContainerEvent UActorDescContainerInstance::OnActorDescInstanceRemovedFromContainer;

FName UActorDescContainerInstance::GetContainerPackageNameFromWorld(UWorld* InWorld)
{
	check(InWorld);

	// return non instanced package name
	UPackage* ContainerPackage = InWorld->PersistentLevel->GetOutermost();

	// Duplicated worlds (ex: WorldPartitionRenameDuplicateBuilder) will not have a loaded path 
	return ContainerPackage->GetLoadedPath().GetPackageFName().IsNone() ? ContainerPackage->GetFName() : ContainerPackage->GetLoadedPath().GetPackageFName();
}

void UActorDescContainerInstance::RegisterContainer(const FInitializeParams& InParams)
{
	UActorDescContainer::FInitializeParams ContainerInitParams(InParams.ContainerPackageName);
	ContainerInitParams.FilterActorDesc = [&InParams](const FWorldPartitionActorDesc* InActorDesc) -> bool
	{
		return !InParams.FilterActorDescFunc || InParams.FilterActorDescFunc(InActorDesc);
	};
	ContainerInitParams.ContentBundleGuid = InParams.ContentBundleGuid;
	ContainerInitParams.ExternalDataLayerAsset = InParams.ExternalDataLayerAsset;
	ContainerInitParams.bShouldRegisterEditorDeletages = InParams.bShouldRegisterEditorDeletages;
	ContainerInitParams.bShouldGatherInternalActors = InParams.bShouldGatherInternalActors;

	SetContainer(UActorDescContainerSubsystem::GetChecked().RegisterContainer(ContainerInitParams));
}

void UActorDescContainerInstance::UnregisterContainer()
{
	if (!IsEngineExitRequested())
	{
		UActorDescContainerSubsystem::GetChecked().UnregisterContainer(Container);
	}
	Container = nullptr;
}

void UActorDescContainerInstance::OnContainerReplaced(UActorDescContainer* InOldContainer, UActorDescContainer* InNewContainer)
{
	if (ChildContainerInstances.Num())
	{
		TMap<FGuid, TObjectPtr<UActorDescContainerInstance>> CopyChildContainerInstances(ChildContainerInstances);
		for (auto& [ContainerGuid, ContainerInstance] : CopyChildContainerInstances)
		{
			if (ContainerInstance->GetContainer() == InOldContainer)
			{
				FWorldPartitionActorDescInstance* ContainerDescInstance = GetActorDescInstance(ContainerGuid);
				check(ContainerDescInstance);
				ContainerDescInstance->UpdateChildContainerInstance();
				check(ContainerDescInstance->GetChildContainerInstance()->GetContainer() == InNewContainer);
			}
		}
	}
}

void UActorDescContainerInstance::Initialize(const FInitializeParams& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainerInstance::Initialize);

	FName OuterWorldContainerPackageName;

	Transform = InParams.Transform;
	ContainerActorGuid = InParams.ContainerActorGuid;

	if (InParams.ContainerActorGuid.IsValid())
	{
		// It is possible to not have a ParentContainerInstance if we are in a non-WP main world
		// We want this ContainerInstance to still not have a Main ContainerID to properly handle IsMainWorldOnly() actors
		ContainerID = FActorContainerID(InParams.ParentContainerInstance ? InParams.ParentContainerInstance->GetContainerID() : FActorContainerID(), InParams.ContainerActorGuid);
		ParentContainerInstance = InParams.ParentContainerInstance;
	}
				
	// Only consider world if we are outered to a WorldPartition directly
	UWorld* OuterWorld = GetOuterWorldPartition() ? GetTypedOuter<UWorld>() : nullptr;
	bool bIsInstanced = false;
	FString SourceWorldPath, RemappedWorldPath;

	const FString OuterWorldPackageNameStr = OuterWorld ? OuterWorld->GetPackage()->GetFName().ToString() : FString();

	if (OuterWorld)
	{
		// World package name can differ from ContainerPackage name as the ContainerPackageName can be a ContentBundle
		OuterWorldContainerPackageName = GetContainerPackageNameFromWorld(OuterWorld);
		
		// Currently known Instancing use cases:
		//  - Level Instances
		//  - Runtime Streamed World Partition levels
		//	- World Partition map template (New Level)
		//	- PIE World Travel / -game
		bIsInstanced = OuterWorld->GetSoftObjectPathMapping(SourceWorldPath, RemappedWorldPath);

		if (bIsInstanced)
		{
			SourceWorldContainerPath = SourceWorldPath;
			WorldContainerPath = RemappedWorldPath;

			InstancingContext = FLinkerInstancingContext();
			InstancingContext->AddPackageMapping(OuterWorldContainerPackageName, OuterWorld->GetPackage()->GetFName());

			// SoftObjectPaths: Specific case for new maps (/Temp/Untitled) where we need to remap the AssetPath and not just the Package name because the World gets renamed (See UWorld::PostLoad)
			InstancingContext->AddPathMapping(
				FSoftObjectPath(*FString::Format(TEXT("{0}.{1}"), { OuterWorldContainerPackageName.ToString(), FPackageName::GetShortName(OuterWorldContainerPackageName) })),
				FSoftObjectPath(OuterWorld)
			);
		}
	}
		
	RegisterContainer(InParams);
	check(Container);

	// Instanced map Invalid Actors
	if (bIsInstanced)
	{
		// If a Valid Actor references an Invalid Actor:
		// Make sure Invalid Actors do not load their imports (ex: outer non instanced world).
		for (const FAssetData& InvalidActor : Container->InvalidActors)
		{
			InstancingContext->AddPackageMapping(InvalidActor.PackageName, NAME_None);
		}
	}

	// Create ActorDescInstances
	for (FActorDescList::TIterator<> It(Container); It; ++It)
	{
		FWorldPartitionActorDescInstance ActorDescInstance = CreateActorDescInstance(*It);
		if (bIsInstanced)
		{
			const FString LongActorPackageName = It->GetActorPackage().ToString();
			const FString InstancedName = ULevel::GetExternalActorPackageInstanceName(OuterWorldPackageNameStr, LongActorPackageName);

			if (LongActorPackageName != OuterWorldContainerPackageName)
			{
				InstancingContext->AddPackageMapping(*LongActorPackageName, *InstancedName);
			}
			ActorDescInstance.ActorPath = It->GetActorSoftPath().ToString().Replace(*SourceWorldPath, *RemappedWorldPath);
		}

		AddActorDescInstance(MoveTemp(ActorDescInstance));
	}

	if (InParams.OnInitializedFunc)
	{
		InParams.OnInitializedFunc(this);
	}

	OnActorDescContainerInstanceInitialized.Broadcast(this);
		
	// If Container Instance is required to create hierarchy go ahead
	if (InParams.bCreateContainerInstanceHierarchy)
	{
		bCreateChildContainerHierarchy = true;
		for (UActorDescContainerInstance::TIterator<> It(this); It; ++It)
		{
			if (It->IsChildContainerInstance())
			{
				It->RegisterChildContainerInstance();
			}
		}
	}

	// Register Delegates
	bRegisteredDelegates = InParams.bShouldRegisterEditorDeletages && ShouldRegisterDelegates();
	
	if (bRegisteredDelegates)
	{
		RegisterDelegates();
	}

	bIsInitialized = true;
}

void UActorDescContainerInstance::OnRegisterChildContainerInstance(const FGuid& InActorGuid, UActorDescContainerInstance* InChildContainerInstance)
{
	check(bCreateChildContainerHierarchy);
	check(!ChildContainerInstances.Contains(InActorGuid));
	ChildContainerInstances.Add(InActorGuid, InChildContainerInstance);
}

void UActorDescContainerInstance::OnUnregisterChildContainerInstance(const FGuid& InActorGuid)
{
	check(bCreateChildContainerHierarchy);
	check(ChildContainerInstances.Contains(InActorGuid));
	ChildContainerInstances.Remove(InActorGuid);
}

void UActorDescContainerInstance::Uninitialize()
{
	// Ensure that a non-deferred context is currently being used, because otherwise the actor desc instance and container deletion resulting from this call will risk the context
	// use deleted memory. If this check fails, a fix would be to flush the current FWorldPartitionLoadingContext::IContext through a new static method, after having forced unregistrations below.
	check(!FWorldPartitionLoadingContext::IsDeferringRawObjects());

	const UWorld* OuterWorld = GetTypedOuter<UWorld>();

	bIsInitialized = false;

	if (bRegisteredDelegates)
	{
		UnregisterDelegates();
		bRegisteredDelegates = false;
	}

	for (TUniquePtr<FWorldPartitionActorDescInstance>& ActorDescInstancePtr : ActorDescList)
	{
		if (ActorDescInstancePtr.IsValid())
		{
			if (ActorDescInstancePtr->GetHardRefCount())
			{
				if (OuterWorld && !OuterWorld->IsBeingCleanedUp())
				{
					// Force actor unregistration when destroying the container instance without the world being destroyed to avoid dangling actors
					FWorldPartitionReference LastReference(ActorDescInstancePtr.Get());
				}
			}

			RemoveActorDescInstance(&ActorDescInstancePtr);
		}
	}
	check(ChildContainerInstances.IsEmpty());

	UnregisterContainer();
	Container = nullptr;
}

bool UActorDescContainerInstance::ShouldRegisterDelegates() const
{
	// No World Partition means we are a ChildContainerInstance created for StreamingGeneration and need to listen to some events for updates
	const UWorldPartition* WorldPartition = GetOuterWorldPartition();
	const UWorld* OwningWorld = WorldPartition ? WorldPartition->GetWorld() : nullptr;
	return (!OwningWorld || !OwningWorld->IsGameWorld()) && !IsRunningCookCommandlet();
}

void UActorDescContainerInstance::RegisterDelegates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainerInstance::RegisterDelegates);

	check(Container);

	// Only listen to Object replaced events on ContainerInstance that have a direct World Partition outer (Loaded Container Instances: Main World or Loaded Level Instances)
	if (GetOuterWorldPartition())
	{
		Container->OnObjectsReplacedEvent.AddUObject(this, &UActorDescContainerInstance::OnObjectsReplaced);
		Container->OnEditorActorReplacedEvent.AddUObject(this, &UActorDescContainerInstance::OnEditorActorReplaced);
	}

	// Only listen to this event if we have registered Child Container Instances
	if (bCreateChildContainerHierarchy)
	{
		UActorDescContainerSubsystem::GetChecked().ContainerReplaced().AddUObject(this, &UActorDescContainerInstance::OnContainerReplaced);
	}

	// No need to register Added descs events for instanced worlds as they don't support it for now (Level Instances get reloaded after an edit)
	if (!GetInstancingContext())
	{
		Container->OnActorDescAddedEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescAdded);
	}

	// Important to hook the other events that will invalidate existing FWorldPartitionActorDescInstance's because those can be hashed and loaded
	// even in Instanced worlds (Level Instances)
	Container->OnActorDescRemovedEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescRemoved);
		
	Container->OnActorDescPreUpdateEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescPreUpdate);
	Container->OnActorDescUpdatingEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescUpdating);
	Container->OnActorDescUpdatedEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescUpdated);
}

void UActorDescContainerInstance::UnregisterDelegates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorDescContainerInstance::UnregisterDelegates);

	check(Container);
	Container->OnObjectsReplacedEvent.RemoveAll(this);
	Container->OnEditorActorReplacedEvent.RemoveAll(this);
	Container->OnActorDescAddedEvent.RemoveAll(this);
	Container->OnActorDescRemovedEvent.RemoveAll(this);
	Container->OnActorDescPreUpdateEvent.RemoveAll(this);
	Container->OnActorDescUpdatingEvent.RemoveAll(this);
	Container->OnActorDescUpdatedEvent.RemoveAll(this);

	if (UActorDescContainerSubsystem* ActorDescContainerSubsystem = UActorDescContainerSubsystem::Get())
	{
		ActorDescContainerSubsystem->ContainerReplaced().RemoveAll(this);
	}
}

TUniquePtr<FWorldPartitionActorDescInstance>* UActorDescContainerInstance::GetActorDescInstancePtr(const FGuid& InActorGuid) const
{
	if (TUniquePtr<FWorldPartitionActorDescInstance>* const* ActorDescInstancePtr = ActorsByGuid.Find(InActorGuid))
	{
		return *ActorDescInstancePtr;
	}

	return nullptr;
}

FWorldPartitionActorDescInstance* UActorDescContainerInstance::GetActorDescInstance(const FGuid& InActorGuid) const
{
	if (TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstancePtr = GetActorDescInstancePtr(InActorGuid))
	{
		return ActorDescInstancePtr->Get();
	}

	return nullptr;
}

FWorldPartitionActorDescInstance& UActorDescContainerInstance::GetActorDescInstanceChecked(const FGuid& InActorGuid) const
{
	FWorldPartitionActorDescInstance* ActorDescInstance = GetActorDescInstance(InActorGuid);
	check(ActorDescInstance);
	return *ActorDescInstance;
}

const FWorldPartitionActorDescInstance* UActorDescContainerInstance::GetActorDescInstanceByPath(const FString& ActorPath) const
{
	if (const FWorldPartitionActorDesc* ActorDesc = GetContainer()->GetActorDescByPath(ActorPath))
	{
		return GetActorDescInstance(ActorDesc->GetGuid());
	}

	return nullptr;
}

const FWorldPartitionActorDescInstance* UActorDescContainerInstance::GetActorDescInstanceByPath(const FSoftObjectPath& ActorPath) const
{
	if (const FWorldPartitionActorDesc* ActorDesc = GetContainer()->GetActorDescByPath(ActorPath))
	{
		return GetActorDescInstance(ActorDesc->GetGuid());
	}

	return nullptr;
}

const FWorldPartitionActorDescInstance* UActorDescContainerInstance::GetActorDescInstanceByName(FName ActorName) const
{
	if (const FWorldPartitionActorDesc* ActorDesc = GetContainer()->GetActorDescByName(ActorName))
	{
		return GetActorDescInstance(ActorDesc->GetGuid());
	}

	return nullptr;
}

bool UActorDescContainerInstance::IsActorDescHandled(const AActor* Actor) const
{
	if (Container->IsActorDescHandled(Actor))
	{
		return true;
	}

	// Special case of Newly created maps where the Container might point to a template map but our map path is different
	if (!Actor->GetContentBundleGuid().IsValid() && 
		!GetContentBundleGuid().IsValid() &&
		GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated) && 
		GetPackage()->GetName() != GetContainerPackage())
	{
		const FString ActorPackageName = Actor->GetPackage()->GetName();
		const FString ExternalActorPath = ULevel::GetExternalActorsPath(GetPackage()->GetName()) / TEXT("");
		return ActorPackageName.StartsWith(ExternalActorPath);
	}
			
	return false;
}

FWorldPartitionActorDesc* UActorDescContainerInstance::GetActorDesc(const FGuid& InActorGuid) const
{
	return Container->GetActorDesc(InActorGuid);
}

FWorldPartitionActorDesc* UActorDescContainerInstance::GetActorDescChecked(const FGuid& InActorGuid) const
{
	FWorldPartitionActorDesc* ActorDesc = Container->GetActorDesc(InActorGuid);
	check(ActorDesc);
	return ActorDesc;
}

void UActorDescContainerInstance::SetContainerPackage(FName InContainerPackageName)
{
	if (GetInstancingContext())
	{
		InstancingContext.Reset();

		// Now that the world is no longer instanced, we need to register the delegates we previously skipped in RegisterDelegates.
		if (bRegisteredDelegates)
		{
			Container->OnActorDescAddedEvent.AddUObject(this, &UActorDescContainerInstance::OnActorDescAdded);
		}
	}

	UActorDescContainerSubsystem::GetChecked().SetContainerPackage(Container, InContainerPackageName);
}

FName UActorDescContainerInstance::GetContainerPackage() const
{
	return Container->GetContainerPackage();
}

FGuid UActorDescContainerInstance::GetContentBundleGuid() const
{
	return Container->GetContentBundleGuid();
}

const UExternalDataLayerAsset* UActorDescContainerInstance::GetExternalDataLayerAsset() const
{
	return Container->GetExternalDataLayerAsset();
}

bool UActorDescContainerInstance::HasExternalContent() const
{
	return Container->HasExternalContent();
}

FString UActorDescContainerInstance::GetExternalActorPath() const
{
	return Container->GetExternalActorPath();
}

FString UActorDescContainerInstance::GetExternalObjectPath() const
{
	return Container->GetExternalObjectPath();
}

FWorldPartitionActorDescInstance* UActorDescContainerInstance::AddActor(FWorldPartitionActorDesc* InActorDesc)
{
	// We don't support adding actors when instanced
	check(!InstancingContext.IsSet());

	return AddActorDescInstance(CreateActorDescInstance(InActorDesc));
}

FWorldPartitionActorDescInstance* UActorDescContainerInstance::AddActorDescInstance(FWorldPartitionActorDescInstance&& InActorDescInstance)
{
	check(InActorDescInstance.GetActorDesc());
	
	FWorldPartitionActorDescInstance* NewActorDescInstance = new FWorldPartitionActorDescInstance(MoveTemp(InActorDescInstance));
	check(NewActorDescInstance->IsValid());

	AddActorDescriptor(NewActorDescInstance);
	
	OnActorDescInstanceAddedToContainer.Broadcast(this, NewActorDescInstance);
	
	return NewActorDescInstance;
}

void UActorDescContainerInstance::RemoveActor(const FGuid& InActorGuid)
{
	if (TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstance = GetActorDescriptor(InActorGuid))
	{
		if (UWorldPartition* WorldPartition = GetOuterWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemoved(ActorDescInstance->Get());
		}

		OnActorDescInstanceRemovedEvent.Broadcast(ActorDescInstance->Get());
		RemoveActorDescInstance(ActorDescInstance);
	}
}

void UActorDescContainerInstance::RemoveActorDescInstance(TUniquePtr<FWorldPartitionActorDescInstance>* InActorDescInstance)
{
	check(InActorDescInstance && InActorDescInstance->IsValid());
	
	FWorldPartitionActorDescInstance* ActorDescInstance = InActorDescInstance->Get();
	
	OnActorDescInstanceRemovedFromContainer.Broadcast(this, ActorDescInstance);
	
	if (bCreateChildContainerHierarchy && ActorDescInstance->IsChildContainerInstance())
	{
		ActorDescInstance->UnregisterChildContainerInstance();
		check(!ChildContainerInstances.Contains(ActorDescInstance->GetGuid()));
	}

	// This actor desc might have bIsRegisteringOrUnregistering set to true, meaning it is currently stored in the Registrations array in a FDeferred loading context.
	// RemoveActorDescriptor would delete it without removing it from the array, leading to dangling pointer access.
	FWorldPartitionLoadingContext::CancelOperationIfAny(ActorDescInstance);

	RemoveActorDescriptor(ActorDescInstance);

	InActorDescInstance->Get()->Invalidate();
	InActorDescInstance->Reset();
}

const FLinkerInstancingContext* UActorDescContainerInstance::GetInstancingContext() const
{
	return InstancingContext.GetPtrOrNull();
}

const FTransform& UActorDescContainerInstance::GetTransform() const
{
	// UActorDescContainerInstance outered to a UWorldPartition
	if (UWorldPartition* WorldPartition = GetOuterWorldPartition())
	{
		// GameWorld: Container instance is necessarly used for Streaming generation. In which case we want to return Identity and Transform of the World Partition will be applied on the LevelStreaming objects.
		if (GetWorld()->IsGameWorld())
		{
			check(GetContainerID().IsMainContainer());
			return FTransform::Identity;
		}

		return WorldPartition->GetInstanceTransform();
	}
	
	// Transform is set when this Container Instance is a Child Container Instance created for Streaming Generation
	return Transform.IsSet() ? Transform.GetValue() : FTransform::Identity;
}

UWorldPartition* UActorDescContainerInstance::GetTopWorldPartition() const
{
	return GetTypedOuter<UWorldPartition>();
}

UWorldPartition* UActorDescContainerInstance::GetOuterWorldPartition() const
{
	return Cast<UWorldPartition>(GetOuter());
}

void UActorDescContainerInstance::LoadAllActors(TArray<FWorldPartitionReference>& OutReferences)
{
	FWorldPartitionLoadingContext::FDeferred LoadingContext;
	OutReferences.Reserve(OutReferences.Num() + ActorsByGuid.Num());
	for (UActorDescContainerInstance::TIterator<> Iterator(this); Iterator; ++Iterator)
	{
		OutReferences.Emplace(this, Iterator->GetGuid());
	}
}

void UActorDescContainerInstance::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewObjectMap)
{
	// Patch up Actor pointers in ActorDescInstances
	for (auto [OldObject, NewObject] : InOldToNewObjectMap)
	{
		if (AActor* OldActor = Cast<AActor>(OldObject))
		{
			OnEditorActorReplaced(OldActor, Cast<AActor>(NewObject));
		}
	}
}

void UActorDescContainerInstance::OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor)
{
	// Patch up Actor pointers in ActorDescInstances
	if (Container->ShouldHandleActorEvent(InOldActor, InstancingContext.GetPtrOrNull() != nullptr))
	{
		// Only replace if OldActor matches ActorDescInstance ActorPtr.
		// No need to replace the ActorDescInstance ActorPtr if it isn't set, it will get refreshed on its next GetActor call. This avoids replacing pointers on the wrong ActorDescInstance for Instanced worlds
		if (FWorldPartitionActorDescInstance* ActorDescInstance = GetActorDescInstance(InOldActor->GetActorGuid()); ActorDescInstance && ActorDescInstance->ActorPtr == InOldActor)
		{
			FWorldPartitionActorDescUtils::ReplaceActorDescriptorPointerFromActor(InOldActor, InNewActor, ActorDescInstance);
			OnActorReplacedEvent.Broadcast(ActorDescInstance);
		}
	}
}

void UActorDescContainerInstance::OnActorDescAdded(FWorldPartitionActorDesc* InActorDesc)
{
	FWorldPartitionActorDescInstance* NewActorDescInstance = AddActor(InActorDesc);

	if (bCreateChildContainerHierarchy && NewActorDescInstance->IsChildContainerInstance())
	{
		NewActorDescInstance->RegisterChildContainerInstance();
	}

	if (UWorldPartition* WorldPartition = GetOuterWorldPartition())
	{
		WorldPartition->OnActorDescInstanceAdded(NewActorDescInstance);
	}
	else if (UWorldPartition* TopWorldPartition = GetTopWorldPartition())
	{
		if (const UDataLayerManager* ResolvingDataLayerManager = TopWorldPartition->GetResolvingDataLayerManager())
		{
			ResolvingDataLayerManager->ResolveActorDescInstanceDataLayers(NewActorDescInstance);
		}
	}

	OnActorDescInstanceAddedEvent.Broadcast(NewActorDescInstance);
}

void UActorDescContainerInstance::OnActorDescRemoved(FWorldPartitionActorDesc* InActorDesc)
{
	RemoveActor(InActorDesc->GetGuid());
}

void UActorDescContainerInstance::OnActorDescPreUpdate(FWorldPartitionActorDesc* InActorDesc)
{
	// It is possible for a container instance with an instancing context to listen to updates but not newly added actors
	// so it might not find the actor if it was newly added (through the edit of a different instance of the same Level Instance)
	// See UActorDescContainerInstance::RegisterDelegates
	if (TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstanceUniquePtr = GetActorDescInstancePtr(InActorDesc->GetGuid()))
	{
		check(ActorDescInstanceUniquePtr->IsValid());
		FWorldPartitionActorDescInstance* ActorDescInstance = ActorDescInstanceUniquePtr->Get();

		if (bCreateChildContainerHierarchy && ActorDescInstance->IsChildContainerInstance())
		{
			ActorDescInstance->UnregisterChildContainerInstance();
		}

		if (UWorldPartition* WorldPartition = GetOuterWorldPartition())
		{
			WorldPartition->OnActorDescInstanceUpdating(ActorDescInstance);
		}

		ActorDescInstance->OnPreUpdate();

		OnActorDescInstanceUpdatingEvent.Broadcast(ActorDescInstance);
	}
}

void UActorDescContainerInstance::OnActorDescUpdating(FWorldPartitionActorDesc* InActorDesc)
{
	// See comment in UActorDescContainerInstance::OnActorDescPreUpdate
	if (TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstance = GetActorDescInstancePtr(InActorDesc->GetGuid()))
	{
		check(ActorDescInstance->IsValid());

		// Update instance desc pointer
		ActorDescInstance->Get()->UpdateActorDesc(InActorDesc);
	}
}

void UActorDescContainerInstance::OnActorDescUpdated(FWorldPartitionActorDesc* InActorDesc)
{
	// See comment in UActorDescContainerInstance::OnActorDescPreUpdate
	if (TUniquePtr<FWorldPartitionActorDescInstance>* ActorDescInstance = GetActorDescInstancePtr(InActorDesc->GetGuid()))
	{
		check(ActorDescInstance->IsValid());

		// Re-register container
		if (bCreateChildContainerHierarchy && ActorDescInstance->Get()->IsChildContainerInstance())
		{
			ActorDescInstance->Get()->RegisterChildContainerInstance();
		}

		if (UWorldPartition* WorldPartition = GetOuterWorldPartition())
		{
			WorldPartition->OnActorDescInstanceUpdated(ActorDescInstance->Get());
		}
		else if (UWorldPartition* TopWorldPartition = GetTopWorldPartition())
		{
			if (const UDataLayerManager* ResolvingDataLayerManager = TopWorldPartition->GetResolvingDataLayerManager())
			{
				ResolvingDataLayerManager->ResolveActorDescInstanceDataLayers(ActorDescInstance->Get());
			}
		}

		OnActorDescInstanceUpdatedEvent.Broadcast(ActorDescInstance->Get());
	}
}

#endif