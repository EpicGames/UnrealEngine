// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorDescContainerSubsystem)

#if WITH_EDITOR
UActorDescContainerSubsystem* UActorDescContainerSubsystem::Get()
{
	if (!IsEngineExitRequested() && GEngine)
	{
		return GEngine->GetEngineSubsystem<UActorDescContainerSubsystem>();
	}

	return nullptr;
}

UActorDescContainerSubsystem& UActorDescContainerSubsystem::GetChecked()
{
	UActorDescContainerSubsystem* ContainerSubsystem = Get();
	checkf(ContainerSubsystem, TEXT("Failed to get ActorDescContainerSubsystem: IsEngineExitRequested=%d, GEngine=%s"), IsEngineExitRequested() ? 1 : 0, *GetNameSafe(GEngine));
	return *Get();
}

void UActorDescContainerSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(FName("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetCollision_Private().AddUObject(this, &UActorDescContainerSubsystem::OnAssetCollision);
		}		
	}
}

void UActorDescContainerSubsystem::Deinitialize()
{
	Super::Deinitialize();

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(FName("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetCollision_Private().RemoveAll(this);
		}
	}
}

void UActorDescContainerSubsystem::OnAssetCollision(FAssetData& A, FAssetData& B, FAssetData*& Keep)
{
	if (A.PackagePath.ToString().Contains(ULevel::GetExternalActorsFolderName()))
	{
		auto IsMatching = [](const FAssetData& Asset, EActorPackagingScheme PackagingScheme)
		{
			const FString AssetPath = ULevel::GetActorPackageName(TEXT(""), PackagingScheme, Asset.GetSoftObjectPath().ToString());
			return Asset.PackageName.ToString().EndsWith(AssetPath);
		};

		if (IsMatching(A, EActorPackagingScheme::Reduced) || IsMatching(A, EActorPackagingScheme::Original))
		{
			Keep = &A;
			FSoftObjectPath ObjectPathB(B.GetOptionalOuterPathName().ToString());
			InvalidMapAssets.FindOrAdd(ObjectPathB.GetAssetPath().GetPackageName()).Add(B);
			return;
		}

		if (IsMatching(B, EActorPackagingScheme::Reduced) || IsMatching(A, EActorPackagingScheme::Original))
		{
			Keep = &B;
			FSoftObjectPath ObjectPathA(A.GetOptionalOuterPathName().ToString());
			InvalidMapAssets.FindOrAdd(ObjectPathA.GetAssetPath().GetPackageName()).Add(A);
			return;
		}
	}

	Keep = nullptr;
}

void UActorDescContainerSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UActorDescContainerSubsystem* This = CastChecked<UActorDescContainerSubsystem>(InThis);

	This->ContainerManager.AddReferencedObjects(Collector);
}

void UActorDescContainerSubsystem::FContainerManager::FRegisteredContainer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Container);
}

void UActorDescContainerSubsystem::FContainerManager::FRegisteredContainer::UpdateBounds()
{
	EditorBounds.Init();
	Bounds.Init();
	for (FActorDescList::TIterator<> ActorDescIt(Container); ActorDescIt; ++ActorDescIt)
	{
		if (ActorDescIt->IsMainWorldOnly())
		{
			continue;
		}

		FWorldPartitionActorDesc* ActorDesc = Container->GetActorDesc(ActorDescIt->GetGuid());
		const FBox RuntimeBounds = ActorDesc->GetRuntimeBounds();
		if (RuntimeBounds.IsValid)
		{
			Bounds += RuntimeBounds;
		}
		const FBox ActorDescEditorBounds = ActorDesc->GetEditorBounds();
		if (ActorDescEditorBounds.IsValid)
		{
			EditorBounds += ActorDescEditorBounds;
		}
	}
}

void UActorDescContainerSubsystem::FContainerManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& [Name, ContainerInstance] : RegisteredContainers)
	{
		ContainerInstance.AddReferencedObjects(Collector);
	}
}

void UActorDescContainerSubsystem::FContainerManager::UnregisterContainer(UActorDescContainer* Container)
{
	FString ContainerName = Container->GetContainerName();
	FRegisteredContainer& RegisteredContainer = RegisteredContainers.FindChecked(ContainerName);

	if (--RegisteredContainer.RefCount == 0)
	{
		RegisteredContainer.Container->Uninitialize();
		RegisteredContainers.FindAndRemoveChecked(ContainerName);
	}
}

FBox UActorDescContainerSubsystem::FContainerManager::GetContainerBounds(const FString& ContainerName, bool bIsEditorBounds) const
{
	if (const FRegisteredContainer* RegisteredContainer = RegisteredContainers.Find(ContainerName))
	{
		return bIsEditorBounds ? RegisteredContainer->EditorBounds : RegisteredContainer->Bounds;
	}
	return FBox(ForceInit);
}

void UActorDescContainerSubsystem::FContainerManager::UpdateContainerBounds(const FString& ContainerName)
{
	if (FRegisteredContainer* RegisteredContainer = RegisteredContainers.Find(ContainerName))
	{
		RegisteredContainer->UpdateBounds();
	}
}

void UActorDescContainerSubsystem::FContainerManager::UpdateContainerBoundsFromPackage(FName ContainerPackage)
{
	for (auto& [ContainerName, RegisteredContainer] : RegisteredContainers)
	{
		if (RegisteredContainer.Container->GetContainerPackage() == ContainerPackage)
		{
			RegisteredContainer.UpdateBounds();
		}
	}
}

void UActorDescContainerSubsystem::FContainerManager::SetContainerPackage(UActorDescContainer* Container, FName PackageName)
{
	// Remove and Copy existing Registration (with previous ContainerName: ex: /Temp/Untitled)
	FRegisteredContainer RegisteredContainer;
	const bool bRegistered = RegisteredContainers.RemoveAndCopyValue(Container->GetContainerName(), RegisteredContainer);

	// Update Name
	Container->SetContainerPackage(PackageName);
		
	if(bRegistered)
	{
		// Check if we have an existing container registered with the new name, which means we are saving a map over another one
		FRegisteredContainer ReplacedContainer;
		if (RegisteredContainers.RemoveAndCopyValue(Container->GetContainerName(), ReplacedContainer))
		{
			// Move it out of the way with unique package name without losing the ref counts so that NotifyContainerReplaced listeners will be able to properly unregister
			FString ReplacePackageName = PackageName.ToString() + TEXT("_Replaced_") + FGuid::NewGuid().ToString();
			ReplacedContainer.Container->SetContainerPackage(*ReplacePackageName);
			RegisteredContainers.Add(ReplacedContainer.Container->GetContainerName(), ReplacedContainer);
		}
		
		check(RegisteredContainer.Container == Container);
		RegisteredContainers.Add(Container->GetContainerName(), RegisteredContainer);

		if (ReplacedContainer.Container)
		{
			Owner->NotifyContainerReplaced(ReplacedContainer.Container, RegisteredContainer.Container);
		}
	}
}
#endif
