// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorDescList.h"

#if WITH_EDITOR
FWorldPartitionActorDesc* FActorDescList::AddActor(const AActor* InActor)
{
	FWorldPartitionActorDesc* NewActorDesc = InActor->CreateActorDesc().Release();
	check(NewActorDesc);

	AddActorDescriptor(NewActorDesc);

	return NewActorDesc;
}

const FWorldPartitionActorDesc* FActorDescList::GetActorDesc(const FGuid& Guid) const
{
	const TUniquePtr<FWorldPartitionActorDesc>* ActorDesc = GetActorDescriptor(Guid);
	return ActorDesc ? ActorDesc->Get() : nullptr;
}

FWorldPartitionActorDesc* FActorDescList::GetActorDesc(const FGuid& Guid)
{
	TUniquePtr<FWorldPartitionActorDesc>* ActorDesc = GetActorDescriptor(Guid);
	return ActorDesc ? ActorDesc->Get() : nullptr;
}

const FWorldPartitionActorDesc& FActorDescList::GetActorDescChecked(const FGuid& Guid) const
{
	const TUniquePtr<FWorldPartitionActorDesc>* ActorDesc = GetActorDescriptorChecked(Guid);
	return *ActorDesc->Get();
}

FWorldPartitionActorDesc& FActorDescList::GetActorDescChecked(const FGuid& Guid)
{
	TUniquePtr<FWorldPartitionActorDesc>* ActorDesc = GetActorDescriptorChecked(Guid);
	return *ActorDesc->Get();
}
#endif