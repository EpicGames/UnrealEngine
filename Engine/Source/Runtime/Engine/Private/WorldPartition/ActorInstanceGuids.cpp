// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorInstanceGuids.h"
#include "UObject/UObjectAnnotation.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

struct FLevelInstanceGuid
{
	TWeakObjectPtr<ULevel> Level;
	TWeakObjectPtr<ULevel> OwnerLevel;
	FGuid LevelInstanceGuid;
	FGuid ResolvedLevelInstanceGuid;
	bool bIsDefault = true;

	bool IsDefault()
	{
		return bIsDefault;
	}
};

FUObjectAnnotationSparse<FActorInstanceGuid, true>	GActorGuids;
FUObjectAnnotationSparse<FLevelInstanceGuid, true>	GLevelInstanceGuids;

void FActorInstanceGuid::ReleaseLevelInstanceGuid(ULevel* Level)
{
	GLevelInstanceGuids.RemoveAnnotation(Level);
}

void FActorInstanceGuid::SetLevelInstanceGuid(ULevel* Level, ULevel* OwnerLevel, const FGuid& Guid, const FGuid& ResolvedGuid)
{
	auto NewOrIdenticalRegistration = [Level, OwnerLevel, &Guid] () -> bool
	{
		FLevelInstanceGuid LevelGuids = GLevelInstanceGuids.GetAnnotation(Level);
		
		if (!LevelGuids.IsDefault())
		{
			return  (LevelGuids.Level == Level) &&
					(LevelGuids.OwnerLevel == OwnerLevel) &&
					(LevelGuids.LevelInstanceGuid == Guid);
		}

		return true;
	};
	
	// Double registration is often an order issue (LevelInstance GUID tried to be accessed before FLevelInstanceActorImpl::OnLevelInstanceLoaded was called) 
	// But there are also some paths where we end-up reregistering the same Level* with the same GUID when Levels are reused so we allow it
	// if the same values are passed. 
	ensure(NewOrIdenticalRegistration());

	FLevelInstanceGuid	LIGuid;

	LIGuid.Level = Level;
	LIGuid.OwnerLevel = OwnerLevel;
	LIGuid.LevelInstanceGuid = Guid;
	LIGuid.ResolvedLevelInstanceGuid = ResolvedGuid;
	LIGuid.bIsDefault = false;
	
	GLevelInstanceGuids.AddAnnotation(Level, LIGuid);

	// try to force a resolve for the guid
	GetLevelInstanceGuid(Level);
}

FGuid FActorInstanceGuid::GetLevelInstanceGuid(ULevel* Level)
{
	if (!Level)
	{
		return FGuid();
	}

	FLevelInstanceGuid LevelGuids = GLevelInstanceGuids.GetAnnotation(Level);	
	
	if (LevelGuids.IsDefault())
	{
		// If we can find an OwningLevel this is most likely the WorldPartition streaming level of a WP LevelInstance inside a non-WP map
		if (ULevel* OwningLevel = ULevelInstanceSubsystem::GetOwningLevel(Level))
		{
			LevelGuids = GLevelInstanceGuids.GetAnnotation(OwningLevel);			
			if (!LevelGuids.IsDefault())
			{
				// Duplicate the annotation to this level to avoid having to look it up all the time
				check(LevelGuids.ResolvedLevelInstanceGuid.IsValid() || !LevelGuids.LevelInstanceGuid.IsValid());
				GLevelInstanceGuids.AddAnnotation(Level, LevelGuids);
			}
		}
	}
	
	// When we reach a non-instanced Level in the chain, register it so that we don't try to resolve it later-on
	if (LevelGuids.IsDefault())
	{
		SetLevelInstanceGuid(Level, nullptr, FGuid());
	}
	
	if (!LevelGuids.ResolvedLevelInstanceGuid.IsValid() && LevelGuids.LevelInstanceGuid.IsValid())
	{		
		FGuid OwnerLevelGuid;

		if (LevelGuids.OwnerLevel.IsValid())
		{
			OwnerLevelGuid = GetLevelInstanceGuid(LevelGuids.OwnerLevel.Get());
		}

		LevelGuids.ResolvedLevelInstanceGuid = FGuid::Combine(OwnerLevelGuid, LevelGuids.LevelInstanceGuid);

		// Update the Annotations after modifying it
		GLevelInstanceGuids.AddAnnotation(Level, LevelGuids);
	}

	return LevelGuids.ResolvedLevelInstanceGuid;
}

#if WITH_EDITOR
void FActorInstanceGuid::InitializeFrom(const AActor& InActor)
{
	ActorGuid = InActor.GetActorGuid();
	ActorInstanceGuid = InActor.GetActorInstanceGuid();	

	if (ActorInstanceGuid == ActorGuid)
	{
		// not an instance/owner level unknown delay resolve until we have all the info available
		ActorInstanceGuid.Invalidate();
	}
}
#endif

bool FActorInstanceGuid::IsDefault()
{
	return !ActorGuid.IsValid() && !ActorInstanceGuid.IsValid();
}

FActorInstanceGuid FActorInstanceGuid::GetActorGuids(const AActor& InActor)
{
	FActorInstanceGuid Guids;

#if WITH_EDITOR
	Guids.InitializeFrom(InActor);
#else
	Guids = GActorGuids.GetAnnotation(&InActor);
#endif
		
	//@todo_ow: ideally would be done during serialization in runtime, but in the async thread it's not safe to use GetLevelInstanceGuid
	// ActorInstanceGuid is invalid, either the Actor is in the main map or nobody assigned the ActorInstanceGuid upon loading a LevelInstance
	// So verify which case we're in
	if (!Guids.ActorInstanceGuid.IsValid())
	{
		// Resolve ActorInstanceGuid by combining it with the LevelGUID 
		FGuid LevelInstanceGuid = GetLevelInstanceGuid(InActor.GetLevel());

		if (LevelInstanceGuid.IsValid())
		{
			Guids.ActorInstanceGuid = FGuid::Combine(LevelInstanceGuid, Guids.ActorGuid);
		}
		else
		{
			Guids.ActorInstanceGuid = Guids.ActorGuid;
		}

#if !WITH_EDITOR
		GActorGuids.AddAnnotation(&InActor, Guids);
#endif
	}
	
	return Guids;
}

FGuid FActorInstanceGuid::GetActorInstanceGuid(const AActor& InActor)
{
	FActorInstanceGuid Guids = GetActorGuids(InActor);

	return Guids.ActorInstanceGuid;
}

void FActorInstanceGuid::ReleaseActorInstanceGuid(const AActor& InActor)
{
	// Only loaded objects will have those annotations
	if (InActor.HasAnyFlags(RF_WasLoaded))
	{
		GActorGuids.RemoveAnnotation(&InActor);
	}
}

FArchive &operator <<(FArchive& Ar, FActorInstanceGuid& InActorInstanceGuids)
{
	Ar << InActorInstanceGuids.ActorGuid;
	Ar << InActorInstanceGuids.ActorInstanceGuid;

	return Ar;
}

void FActorInstanceGuid::Serialize(FArchive& Ar, AActor& InActor)
{
	FActorInstanceGuid Guids;

#if WITH_EDITOR
	Guids.InitializeFrom(InActor);
#endif

	Ar << Guids;
	
	if (Ar.IsLoadingFromCookedPackage())
	{
		GActorGuids.AddAnnotation(&InActor, MoveTemp(Guids));
	}
}


