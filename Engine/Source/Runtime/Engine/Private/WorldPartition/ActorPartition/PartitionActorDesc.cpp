// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ActorPartition/PartitionActorDesc.h"
#include "ActorPartition/ActorPartitionSubsystem.h"

#if WITH_EDITOR
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

FPartitionActorDesc::FPartitionActorDesc()
	: GridSize(0)
	, GridIndexX(0)
	, GridIndexY(0)
	, GridIndexZ(0)
{
}

void FPartitionActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const APartitionActor* PartitionActor = CastChecked<APartitionActor>(InActor);

	GridSize = PartitionActor->GetGridSize();

	if (!bIsDefaultActorDesc)
	{
		GridGuid = PartitionActor->GetGridGuid();
	
		const FVector ActorLocation = InActor->GetActorLocation();
		SetGridIndices(ActorLocation.X, ActorLocation.Y, ActorLocation.Z);

		SetEditorBounds(UActorPartitionSubsystem::FCellCoord::GetCellBounds({ GridIndexX, GridIndexY, GridIndexZ, 0 }, GridSize));
	}
}

void FPartitionActorDesc::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	FWorldPartitionActorDesc::Serialize(Ar);
	
	if (!bIsDefaultActorDesc)
	{
		Ar << GridSize << GridIndexX << GridIndexY << GridIndexZ;

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeEditorBounds)
		{
			SetEditorBounds(UActorPartitionSubsystem::FCellCoord::GetCellBounds({ GridIndexX, GridIndexY, GridIndexZ, 0 }, GridSize));
		}

		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::PartitionActorDescSerializeGridGuid)
		{
			Ar << GridGuid;
		}
	}
}

bool FPartitionActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FPartitionActorDesc* PartitionActorDesc = (FPartitionActorDesc*)Other;

		return
			GridSize == PartitionActorDesc->GridSize &&
			GridIndexX == PartitionActorDesc->GridIndexX &&
			GridIndexY == PartitionActorDesc->GridIndexY &&
			GridIndexZ == PartitionActorDesc->GridIndexZ &&
			GridGuid == PartitionActorDesc->GridGuid;		
	}

	return false;
}

void FPartitionActorDesc::TransferWorldData(const FWorldPartitionActorDesc* From)
{
	FWorldPartitionActorDesc::TransferWorldData(From);

	// TransferWorldData is called for actors that are not added to the world (not registered)
	// Transfer properties that depend on actor being added to the world (components being registered)
	const FPartitionActorDesc* FromPartitionActorDesc = (FPartitionActorDesc*)From;
	GridIndexX = FromPartitionActorDesc->GridIndexX;
	GridIndexY = FromPartitionActorDesc->GridIndexY;
	GridIndexZ = FromPartitionActorDesc->GridIndexZ;
}

void FPartitionActorDesc::SetGridIndices(double X, double Y, double Z)
{
	if (GridSize > 0)
	{
		GridIndexX = FMath::FloorToInt(X / GridSize);
		GridIndexY = FMath::FloorToInt(Y / GridSize);
		GridIndexZ = FMath::FloorToInt(Z / GridSize);
	}
	else
	{
		GridIndexX = GridIndexY = GridIndexZ = 0;
	}
}

#endif
