// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundle.h"

#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "Engine/World.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"

#if WITH_EDITOR
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#endif

FContentBundle::FContentBundle(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld)
	: FContentBundleBase(InClient, InWorld)
	, ExternalStreamingObjectPackage(nullptr)
	, ExternalStreamingObject(nullptr)
{

}

void FContentBundle::DoInitialize()
{
#if WITH_EDITOR
	InitializeForPIE();
#else
	const FString ExternalStreamingObjectPackagePath = GetExternalStreamingObjectPackagePath();
	if (FPackageName::DoesPackageExist(ExternalStreamingObjectPackagePath))
	{
		ExternalStreamingObjectPackage = LoadPackage(nullptr, *ExternalStreamingObjectPackagePath, LOAD_None);
		if (ExternalStreamingObjectPackage != nullptr)
		{
			ExternalStreamingObject = Cast<URuntimeHashExternalStreamingObjectBase>((UObject*)FindObjectWithOuter(ExternalStreamingObjectPackage, URuntimeHashExternalStreamingObjectBase::StaticClass()));

			if (ExternalStreamingObject)
			{
				ExternalStreamingObject->OnStreamingObjectLoaded(GetInjectedWorld());
			}
			else
			{
				UE_LOGF(LogContentBundle, Error, "%ls No streaming object found in package %ls. No content will be injected.", *ContentBundle::Log::MakeDebugInfoString(*this), *GetExternalStreamingObjectPackagePath());
			}
		}
		else
		{
			UE_LOGF(LogContentBundle, Error, "%ls Streaming package %ls failed to load. No content will be injected.", *ContentBundle::Log::MakeDebugInfoString(*this), *GetExternalStreamingObjectPackagePath());
		}
	}
	else
	{
		UE_LOGF(LogContentBundle, Log, "%ls Has no content for the current World (this can happen if the ContentBundle has no Actors for the current World, or if the GameFeaturePlugin containing the CB is not yet mounted).", *ContentBundle::Log::MakeDebugInfoString(*this));
	}
#endif

	SetStatus(EContentBundleStatus::Registered);
}

void FContentBundle::DoUninitialize()
{
	SetStatus(EContentBundleStatus::Unknown);

	ExternalStreamingObject = nullptr;
	ExternalStreamingObjectPackage = nullptr;
}

void FContentBundle::DoInjectContent()
{
	if (ExternalStreamingObject != nullptr)
	{
		if (GetInjectedWorld()->GetWorldPartition()->InjectExternalStreamingObject(ExternalStreamingObject))
		{
			UE_LOGF(LogContentBundle, Log, "%ls Streaming Object Injected.", *ContentBundle::Log::MakeDebugInfoString(*this));
			SetStatus(EContentBundleStatus::ContentInjected);
		}
		else
		{
			UE_LOGF(LogContentBundle, Error, "%ls Failed to inject streaming object.", *ContentBundle::Log::MakeDebugInfoString(*this));
			SetStatus(EContentBundleStatus::FailedToInject);
		}
	}
	else
	{
		UE_LOGF(LogContentBundle, Log, "%ls No streaming object to inject.", *ContentBundle::Log::MakeDebugInfoString(*this));
		SetStatus(EContentBundleStatus::ContentInjected);
	}
}

void FContentBundle::DoRemoveContent()
{
	if (ExternalStreamingObject != nullptr)
	{
		if (!GetInjectedWorld()->GetWorldPartition()->RemoveExternalStreamingObject(ExternalStreamingObject))
		{
			UE_LOGF(LogContentBundle, Error, "%ls Error while removing streaming object.", *ContentBundle::Log::MakeDebugInfoString(*this));
		}
	}
	
	SetStatus(EContentBundleStatus::Registered);
}

void FContentBundle::AddReferencedObjects(FReferenceCollector& Collector)
{
	FContentBundleBase::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(ExternalStreamingObjectPackage);
	Collector.AddReferencedObject(ExternalStreamingObject);
}

bool FContentBundle::IsValid() const
{
	return GetDescriptor()->IsValid();
}

bool FContentBundle::HasContent() const
{
	return !!ExternalStreamingObject;
}

#if WITH_EDITOR
void FContentBundle::InitializeForPIE()
{
	UContentBundleManager* ContentBundleManager = GetInjectedWorld()->ContentBundleManager;
	if (UContentBundleDuplicateForPIEHelper* PIEHelper = ContentBundleManager->GetPIEDuplicateHelper())
	{
		ExternalStreamingObject = PIEHelper->RetrieveContentBundleStreamingObject(*this);
		if (ExternalStreamingObject == nullptr)
		{
			UE_LOGF(LogContentBundle, Log, "%ls No streaming object found. There are %u existing streaming objects.", *ContentBundle::Log::MakeDebugInfoString(*this), PIEHelper->GetStreamingObjectCount());
		}
	}
}
#endif
