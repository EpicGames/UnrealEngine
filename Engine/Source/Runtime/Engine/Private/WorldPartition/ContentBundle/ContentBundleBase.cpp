// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleBase.h"

#include "UObject/Class.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"

FContentBundleBase::FContentBundleBase(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld)
	: Client(InClient)
	, InjectedWorld(InWorld)
	, Descriptor(InClient->GetDescriptor())
	, Status(EContentBundleStatus::Unknown)
{}

FContentBundleBase::~FContentBundleBase()
{
	check(GetStatus() == EContentBundleStatus::Unknown);
}

void FContentBundleBase::Initialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleBase::Initialize);

	check(GetStatus() == EContentBundleStatus::Unknown);

	DoInitialize();

	check(GetStatus() == EContentBundleStatus::Registered);

	if (TSharedPtr<FContentBundleClient> PinnedClient = Client.Pin())
	{
		PinnedClient->OnContentRegisteredInWorld(GetStatus(), GetInjectedWorld());
	}
}

void FContentBundleBase::Uninitialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleBase::Uninitialize);

	check(GetStatus() != EContentBundleStatus::Unknown);

	if (GetStatus() == EContentBundleStatus::ReadyToInject || GetStatus() == EContentBundleStatus::ContentInjected)
	{
		RemoveContent();
	}

	DoUninitialize();

	check(GetStatus() == EContentBundleStatus::Unknown);
}

void FContentBundleBase::InjectContent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleBase::InjectContent);

	check(GetStatus() == EContentBundleStatus::Registered);

	DoInjectContent();

	check(GetStatus() == EContentBundleStatus::ReadyToInject || GetStatus() == EContentBundleStatus::ContentInjected || GetStatus() == EContentBundleStatus::FailedToInject);

	if (TSharedPtr<FContentBundleClient> PinnedClient = Client.Pin())
	{
		PinnedClient->OnContentInjectedInWorld(GetStatus(), GetInjectedWorld());
	}
}

void FContentBundleBase::RemoveContent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleBase::RemoveContent);

	check(GetStatus() == EContentBundleStatus::ReadyToInject || GetStatus() == EContentBundleStatus::ContentInjected || GetStatus() == EContentBundleStatus::FailedToInject);

	if (GetStatus() == EContentBundleStatus::ContentInjected || GetStatus() == EContentBundleStatus::ReadyToInject)
	{
		DoRemoveContent();
	}

	check(GetStatus() == EContentBundleStatus::Registered);

	if (TSharedPtr<FContentBundleClient> PinnedClient = Client.Pin())
	{
		PinnedClient->OnContentRemovedFromWorld(GetStatus(), GetInjectedWorld());
	}
}

const TWeakPtr<FContentBundleClient>& FContentBundleBase::GetClient() const
{
	return Client;
}

const UContentBundleDescriptor* FContentBundleBase::GetDescriptor() const
{
	return Descriptor;
}

FString FContentBundleBase::GetExternalStreamingObjectPackageName() const
{
	return TEXT("StreamingObject_") + GetDescriptor()->GetGuid().ToString(EGuidFormats::Base36Encoded);
}

FString FContentBundleBase::GetExternalStreamingObjectPackagePath() const
{
	return ContentBundlePaths::GetCookedContentBundleLevelFolder(*this) + ContentBundlePaths::GetGeneratedFolderName() + TEXT("/") + GetExternalStreamingObjectPackageName();
}

UWorld* FContentBundleBase::GetInjectedWorld() const
{
	return InjectedWorld;
}

const FString& FContentBundleBase::GetDisplayName() const
{
	return GetDescriptor()->GetDisplayName();
}

const FColor& FContentBundleBase::GetDebugColor() const
{
	return GetDescriptor()->GetDebugColor();
}

void FContentBundleBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Descriptor);
}

void FContentBundleBase::SetStatus(EContentBundleStatus NewStatus)
{
	check(NewStatus != Status);

	UE_LOGF(LogContentBundle, Verbose, "%ls State changing from %ls to %ls", 
		*ContentBundle::Log::MakeDebugInfoString(*this), *UEnum::GetDisplayValueAsText(Status).ToString(), *UEnum::GetDisplayValueAsText(NewStatus).ToString());

	Status = NewStatus;
}
