// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleClient.h"

#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "Engine/Engine.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleClient)

TSharedPtr<FContentBundleClient> FContentBundleClient::CreateClient(const UContentBundleDescriptor* InContentBundleDescriptor, FString const& InDisplayName)
{
	TSharedPtr<FContentBundleClient> ContentBundleClient = GEngine->GetEngineSubsystem<UContentBundleEngineSubsystem>()->RegisterContentBundle(InContentBundleDescriptor, InDisplayName);
	UE_CLOGF(ContentBundleClient == nullptr, LogContentBundle, Error, "FContentBundleClient::CreateClient failed to create a content bundle client for %ls using %ls", *InDisplayName, *InContentBundleDescriptor->GetDisplayName());
	return ContentBundleClient;
}

FContentBundleClient::FContentBundleClient(const UContentBundleDescriptor* InContentBundleDescriptor, FString const& InDisplayName)
	:ContentBundleDescriptor(InContentBundleDescriptor)
	, DisplayName(InDisplayName)
	, State(EContentBundleClientState::Unregistered)
{
	UE_LOGF(LogContentBundle, Verbose, "[CB: %ls] Client Created", *GetDescriptor()->GetDisplayName());
	SetState(EContentBundleClientState::Registered);
}

void FContentBundleClient::RequestContentInjection()
{ 
	if (State == EContentBundleClientState::Registered)
	{
		SetState(EContentBundleClientState::ContentInjectionRequested);
		GEngine->GetEngineSubsystem<UContentBundleEngineSubsystem>()->RequestContentInjection(*this);
	}
	else
	{
		UE_LOGF(LogContentBundle, Log, "[CB: %ls] Client failed to request content injection. Its state is %ls", *GetDescriptor()->GetDisplayName(), *UEnum::GetDisplayValueAsText(State).ToString());
	}
}

void FContentBundleClient::RequestRemoveContent()
{
	if (State == EContentBundleClientState::Unregistered)
	{
		check(!HasContentToRemove());
		UE_LOGF(LogContentBundle, Warning, "[CB: %ls] Requesting content removal on an unregistered client.", *GetDescriptor()->GetDisplayName());
		return;
	}

	SetState(EContentBundleClientState::ContentRemovalRequested);

	if (HasContentToRemove())
	{
		GEngine->GetEngineSubsystem<UContentBundleEngineSubsystem>()->RequestContentRemoval(*this);
	}
	else
	{
		// No ContentBundleBundles were injected (Non-WP Worlds). Reset the state to Register.
		// Since the client will never receive OnContentRemovedFromWorld.
		SetState(EContentBundleClientState::Registered);
	}
}

void FContentBundleClient::RequestUnregister()
{
	if (State != EContentBundleClientState::Unregistered)
	{
		DoOnClientToUnregister();
		GEngine->GetEngineSubsystem<UContentBundleEngineSubsystem>()->UnregisterContentBundle(*this);
		SetState(EContentBundleClientState::Unregistered);
	}
	else
	{
		UE_LOGF(LogContentBundle, Log, "[CB: %ls] Client failed to request unregister. Its state is %ls", *GetDescriptor()->GetDisplayName(), *UEnum::GetDisplayValueAsText(State).ToString());
	}
}

bool FContentBundleClient::ShouldInjectContent(UWorld* World) const
{
	return GetState() == EContentBundleClientState::ContentInjectionRequested && GetWorldContentState(World) == EWorldContentState::NoContent;
}

bool FContentBundleClient::ShouldRemoveContent(UWorld* World) const
{
	return GetState() != EContentBundleClientState::ContentInjectionRequested && GetWorldContentState(World) != EWorldContentState::NoContent;
}

void FContentBundleClient::OnContentRegisteredInWorld(EContentBundleStatus ContentBundleStatus, UWorld* World)
{
	check(ContentBundleStatus == EContentBundleStatus::Registered);

	DoOnContentRegisteredInWorld(World);
}

void FContentBundleClient::OnContentInjectedInWorld(EContentBundleStatus InjectionStatus, UWorld* InjectedWorld)
{
	check(ShouldInjectContent(InjectedWorld));
	check(InjectionStatus == EContentBundleStatus::ContentInjected
		|| InjectionStatus == EContentBundleStatus::ReadyToInject
		|| InjectionStatus == EContentBundleStatus::FailedToInject);

	SetWorldContentState(InjectedWorld, EWorldContentState::ContentBundleInjected);

	DoOnContentInjectedInWorld(InjectionStatus, InjectedWorld);
}

void FContentBundleClient::OnContentRemovedFromWorld(EContentBundleStatus RemovalStatus, UWorld* InjectedWorld)
{
	// Removal of content bundle can come from the client or WorldUnload
	check(ShouldRemoveContent(InjectedWorld) || !InjectedWorld->ContentBundleManager->CanInject());
	check(RemovalStatus == EContentBundleStatus::Registered);

	SetWorldContentState(InjectedWorld, EWorldContentState::NoContent);

	DoOnContentRemovedFromWorld(InjectedWorld);

	// If the client request content removal and removed everything set the client state as registered.
	// If the client did not request content removal but removed everything keep the client state. We just changed levels. The next level will have its content bundle injected.
	bool bHasProcessAllRemoveRequest = !HasContentToRemove() && GetState() == EContentBundleClientState::ContentRemovalRequested;
	if (bHasProcessAllRemoveRequest)
	{
		SetState(EContentBundleClientState::Registered);
	}
}

void FContentBundleClient::SetWorldContentState(UWorld* World, EWorldContentState NewState)
{
	if (EWorldContentState* OldState = WorldContentStates.Find(World))
	{
		UE_LOGF(LogContentBundle, Verbose, "%ls Client WorldState changing from %ls to %ls", *ContentBundle::Log::MakeDebugInfoString(*this, World), *UEnum::GetDisplayValueAsText(*OldState).ToString(), *UEnum::GetDisplayValueAsText(NewState).ToString());
		*OldState = NewState;
		return;
	}
	
	UE_LOGF(LogContentBundle, Verbose, "%ls Client WorldState changing to %ls", *ContentBundle::Log::MakeDebugInfoString(*this, World), *UEnum::GetDisplayValueAsText(NewState).ToString());
	WorldContentStates.Add(World, NewState);
}

EWorldContentState FContentBundleClient::GetWorldContentState(UWorld* World) const
{
	const EWorldContentState* WorldState = WorldContentStates.Find(World);
	if (WorldState != nullptr)
	{
		return *WorldState;
	}

	return EWorldContentState::NoContent;
}

bool FContentBundleClient::HasContentToRemove() const
{
	for (auto& WorldState : WorldContentStates)
	{
		if (ShouldRemoveContent(WorldState.Key.Get()))
		{
			return true;
		}
	}

	return false;
}

void FContentBundleClient::SetState(EContentBundleClientState NewState)
{
	check(NewState != State);

	UE_LOGF(LogContentBundle, Verbose, "%ls Client State changing from %ls to %ls", 
		*ContentBundle::Log::MakeDebugInfoString(*this), *UEnum::GetDisplayValueAsText(State).ToString(), *UEnum::GetDisplayValueAsText(NewState).ToString());

	State = NewState;
}
