// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleContainer.h"

#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"

FContentBundleContainer::FContentBundleContainer(UWorld* WorldToInjectIn)
	: InjectedWorld(WorldToInjectIn)
{}

FContentBundleContainer::~FContentBundleContainer()
{
#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		check(GetEditorContentBundles().IsEmpty());
		return;
	}
#endif

	check(GetGameContentBundles().IsEmpty());
}

UWorld* FContentBundleContainer::GetInjectedWorld() const
{
	return InjectedWorld;
}

void FContentBundleContainer::Initialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleContainer::Initialize);
	UE_LOGF(LogContentBundle, Verbose, "%ls Creating new container.", *ContentBundle::Log::MakeDebugInfoString(*this));

#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		ContentBundlesVariant.Emplace<ContentBundleEditorArray>(ContentBundleEditorArray());
	}
	else
	{
		ContentBundlesVariant.Emplace<ContentBundleGameArray>(ContentBundleGameArray());
	}	
#endif

	InitializeContentBundlesForegisteredClients();
	RegisterContentBundleClientEvents();

#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		UWorldPartition* WorldPartition = GetInjectedWorld()->GetWorldPartition();
		WorldPartition->OnPreGenerateStreaming.AddRaw(this, &FContentBundleContainer::OnPreGenerateStreaming);
		WorldPartition->OnBeginCook.AddRaw(this, &FContentBundleContainer::OnBeginCook);
		WorldPartition->OnEndCook.AddRaw(this, &FContentBundleContainer::OnEndCook);
	}
#endif
}

void FContentBundleContainer::Deinitialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleContainer::Deinitialize);

	if (GetInjectedWorld())
	{
		UE_LOGF(LogContentBundle, Verbose, "%ls Deleting container.", *ContentBundle::Log::MakeDebugInfoString(*this));

#if WITH_EDITOR
		if (UseEditorContentBundle())
		{
			UWorldPartition* WorldPartition = GetInjectedWorld()->GetWorldPartition();
			WorldPartition->OnPreGenerateStreaming.RemoveAll(this);
			WorldPartition->OnBeginCook.RemoveAll(this);
			WorldPartition->OnEndCook.RemoveAll(this);
		}
#endif
	}

	UnregisterContentBundleClientEvents();
	DeinitializeContentBundles();
}

void FContentBundleContainer::AddReferencedObjects(FReferenceCollector& Collector)
{
	ForEachContentBundle([&Collector](FContentBundleBase* ContentBundle) { ContentBundle->AddReferencedObjects(Collector); });
}

uint32 FContentBundleContainer::GetNumContentBundles() const
{
#if WITH_EDITOR
	if(UseEditorContentBundle())
	{
		return GetEditorContentBundles().Num();
	}
#endif

	return GetGameContentBundles().Num();
}

const TArray<TUniquePtr<FContentBundle>>& FContentBundleContainer::GetGameContentBundles() const
{
#if WITH_EDITOR
	check(!UseEditorContentBundle());
	return ContentBundlesVariant.Get<ContentBundleGameArray>();
#else
	return ContentBundles;
#endif
}

TArray<TUniquePtr<FContentBundle>>& FContentBundleContainer::GetGameContentBundles()
{
	return const_cast<TArray<TUniquePtr<FContentBundle>>&>(const_cast<const FContentBundleContainer*>(this)->GetGameContentBundles());
}

#if WITH_EDITOR

TSharedPtr<FContentBundleEditor> FContentBundleContainer::GetEditorContentBundle(const UContentBundleDescriptor* Descriptor) const
{
	for (const TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		if (ContentBundle->GetDescriptor() == Descriptor)
		{
			return ContentBundle;
		}
	}

	return nullptr;
}

TSharedPtr<FContentBundleEditor> FContentBundleContainer::GetEditorContentBundle(const FGuid& ContentBundleGuid) const
{
	for (const TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		if (ContentBundle->GetDescriptor()->GetGuid() == ContentBundleGuid)
		{
			return ContentBundle;
		}
	}

	return nullptr;
}

const TArray<TSharedPtr<FContentBundleEditor>>& FContentBundleContainer::GetEditorContentBundles() const
{
	check(UseEditorContentBundle());
	return ContentBundlesVariant.Get<ContentBundleEditorArray>();
}

TArray<TSharedPtr<FContentBundleEditor>>& FContentBundleContainer::GetEditorContentBundles()
{
	return const_cast<TArray<TSharedPtr<FContentBundleEditor>>&>(const_cast<const FContentBundleContainer*>(this)->GetEditorContentBundles());
}

bool FContentBundleContainer::UseEditorContentBundle() const
{
	return !GetInjectedWorld()->IsGameWorld() || IsRunningCookCommandlet();
}

#endif

void FContentBundleContainer::ForEachContentBundle(TFunctionRef<void(FContentBundleBase*)> Func) const
{
#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		for (const TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
		{
			Func(ContentBundle.Get());
		}
		return;
	}
#endif

	for (const TUniquePtr<FContentBundle>& ContentBundle : GetGameContentBundles())
	{
		Func(ContentBundle.Get());
	}
}

void FContentBundleContainer::ForEachContentBundleBreakable(TFunctionRef<bool(FContentBundleBase*)> Func) const
{
#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		for (const TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
		{
			if (!Func(ContentBundle.Get()))
			{
				return;
			}
		}
		return;
	}
#endif

	for (const TUniquePtr<FContentBundle>& ContentBundle : GetGameContentBundles())
	{
		if (!Func(ContentBundle.Get()))
		{
			return;
		}
	}
}

FContentBundleBase* FContentBundleContainer::GetContentBundle(const FContentBundleClient& ContentBundleClient)
{
	FContentBundleBase* ContentBundlePtr = nullptr;
	ForEachContentBundleBreakable([&ContentBundleClient, &ContentBundlePtr] (FContentBundleBase* ContentBundle) 
	{
		if (ContentBundle->GetDescriptor() == ContentBundleClient.GetDescriptor())
		{
			ContentBundlePtr = ContentBundle;
			return false;
		}
		return true;
	});
	
	if (ContentBundlePtr != nullptr)
	{
		return ContentBundlePtr;
	}

	return nullptr;
}

FContentBundleBase& FContentBundleContainer::InitializeContentBundle(TSharedPtr<FContentBundleClient>& ContentBundleClient)
{
	check(InjectedWorld->GetWorldPartition()->IsInitialized());
	check(GetContentBundle(*ContentBundleClient) == nullptr);

	FContentBundleBase* ContentBundle = nullptr;

	UE_LOGF(LogContentBundle, Verbose, "%ls Creating new content bundle from client %ls with client state %ls.", 
		*ContentBundle::Log::MakeDebugInfoString(*ContentBundleClient , GetInjectedWorld()), *ContentBundleClient->GetDisplayName(), *UEnum::GetDisplayValueAsText(ContentBundleClient->GetState()).ToString());

#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		if (TSharedPtr<FContentBundleEditor> CBE = GetEditorContentBundle(ContentBundleClient->GetDescriptor()->GetGuid()))
		{
			UE_LOGF(LogContentBundle, Error, "Found duplicate content bundle GUIDs: %ls for content bundles %ls AND %ls", *ContentBundleClient->GetDescriptor()->GetGuid().ToString(), *CBE->GetDescriptor()->GetPackage()->GetPathName(), *ContentBundleClient->GetDescriptor()->GetPackage()->GetPathName());
		}

		ContentBundle = GetEditorContentBundles().Emplace_GetRef(MakeShared<FContentBundleEditor>(ContentBundleClient, GetInjectedWorld())).Get();
	}
	else
	{
		ContentBundle = GetGameContentBundles().Emplace_GetRef(MakeUnique<FContentBundle>(ContentBundleClient, GetInjectedWorld())).Get();
	}
#else
	 ContentBundle = GetGameContentBundles().Emplace_GetRef(MakeUnique<FContentBundle>(ContentBundleClient, GetInjectedWorld())).Get();
#endif

	 ContentBundle->Initialize();
	 return *ContentBundle;
}

void FContentBundleContainer::DeinitializeContentBundle(FContentBundleBase& ContentBundle)
{
	ContentBundle.Uninitialize();

#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		uint32 Index = GetEditorContentBundles().IndexOfByPredicate([&ContentBundle](const TSharedPtr<FContentBundleEditor>& Other) { return Other.Get() == &ContentBundle; });
		check(Index != INDEX_NONE);
		GetEditorContentBundles().RemoveAtSwap(Index);
		return;
	}
#endif

	uint32 Index = GetGameContentBundles().IndexOfByPredicate([&ContentBundle](const TUniquePtr<FContentBundle>& Other) { return Other.Get() == &ContentBundle; });
	GetGameContentBundles().RemoveAtSwap(Index);
}

bool FContentBundleContainer::InjectContentBundle(FContentBundleClient& ContentBundleClient)
{
	if (FContentBundleBase* ContentBundle = GetContentBundle(ContentBundleClient))
	{
		return InjectContentBundle(*ContentBundle);
	}

	UE_LOGF(LogContentBundle, Log, "%ls Failed to inject content bundle from client %ls. It was not found in world.",
		*ContentBundle::Log::MakeDebugInfoString(ContentBundleClient, InjectedWorld), *ContentBundleClient.GetDisplayName());
	return false;
}

bool FContentBundleContainer::InjectContentBundle(FContentBundleBase& ContentBundle)
{
	if (ContentBundle.GetStatus() == EContentBundleStatus::Registered)
	{
		TSharedPtr<FContentBundleClient> ContentBundleClient = ContentBundle.GetClient().Pin();
		if (ContentBundleClient != nullptr && ContentBundleClient->ShouldInjectContent(InjectedWorld))
		{
			ContentBundle.InjectContent();
			return true;
		}
	}
	
	return false;
}

bool FContentBundleContainer::RemoveContentBundle(FContentBundleClient& ContentBundleClient)
{
	if (FContentBundleBase* ContentBundle = GetContentBundle(ContentBundleClient))
	{
		return RemoveContentBundle(*ContentBundle);
	}

	UE_LOGF(LogContentBundle, Log, "%ls Failed to remove content bundle from client %ls. It was not found in world.",
		*ContentBundle::Log::MakeDebugInfoString(ContentBundleClient, InjectedWorld), *ContentBundleClient.GetDisplayName());
	return false;
}

bool FContentBundleContainer::RemoveContentBundle(FContentBundleBase& ContentBundle)
{
	if ((ContentBundle.GetStatus() == EContentBundleStatus::ContentInjected || ContentBundle.GetStatus() == EContentBundleStatus::ReadyToInject))
	{
		TSharedPtr<FContentBundleClient> ContentBundleClient = ContentBundle.GetClient().Pin();
		if (ContentBundleClient == nullptr || ContentBundleClient->ShouldRemoveContent(InjectedWorld) || !InjectedWorld->ContentBundleManager->CanInject())
		{
			ContentBundle.RemoveContent();
			return true;
		}
		else
		{
			UE_LOGF(LogContentBundle, Log, "%ls Client %ls determined content bundle will not be removed from world.",
				*ContentBundle::Log::MakeDebugInfoString(ContentBundle), *ContentBundleClient->GetDisplayName());
		}
	}
	
	return false;
}

void FContentBundleContainer::OnContentBundleClientRegistered(TSharedPtr<FContentBundleClient>& ContentBundleClient)
{
	FContentBundleBase& ContentBundle = InitializeContentBundle(ContentBundleClient);
	InjectContentBundle(ContentBundle);
}

void FContentBundleContainer::OnContentBundleClientUnregistered(FContentBundleClient& ContentBundleClient)
{
	if (FContentBundleBase* ContentBundle = GetContentBundle(ContentBundleClient))
	{
		DeinitializeContentBundle(*ContentBundle);
	}
}

void FContentBundleContainer::OnContentBundleClientContentInjectionRequested(FContentBundleClient& ContentBundleClient)
{
	InjectContentBundle(ContentBundleClient);
}

void FContentBundleContainer::OnContentBundleClientContentRemovalRequested(FContentBundleClient& ContentBundleClient)
{
	RemoveContentBundle(ContentBundleClient);
}

void FContentBundleContainer::RegisterContentBundleClientEvents()
{
	UContentBundleEngineSubsystem* ContentBundleEngineSubsystem = UContentBundleEngineSubsystem::Get();
	ContentBundleEngineSubsystem->OnContentBundleClientRegistered.AddRaw(this, &FContentBundleContainer::OnContentBundleClientRegistered);
	ContentBundleEngineSubsystem->OnContentBundleClientUnregistered.AddRaw(this, &FContentBundleContainer::OnContentBundleClientUnregistered);
	ContentBundleEngineSubsystem->OnContentBundleClientRequestedContentInjection.AddRaw(this, &FContentBundleContainer::OnContentBundleClientContentInjectionRequested);
	ContentBundleEngineSubsystem->OnContentBundleClientRequestedContentRemoval.AddRaw(this, &FContentBundleContainer::OnContentBundleClientContentRemovalRequested);
}

void FContentBundleContainer::UnregisterContentBundleClientEvents()
{
	UContentBundleEngineSubsystem* ContentBundleEngineSubsystem = UContentBundleEngineSubsystem::Get();
	ContentBundleEngineSubsystem->OnContentBundleClientRegistered.RemoveAll(this);
	ContentBundleEngineSubsystem->OnContentBundleClientUnregistered.RemoveAll(this);
	ContentBundleEngineSubsystem->OnContentBundleClientRequestedContentInjection.RemoveAll(this);
	ContentBundleEngineSubsystem->OnContentBundleClientRequestedContentRemoval.RemoveAll(this);
}

void FContentBundleContainer::InitializeContentBundlesForegisteredClients()
{
	UContentBundleEngineSubsystem* ContentBundleEngineSubsystem = UContentBundleEngineSubsystem::Get();
	TArrayView<TSharedPtr<FContentBundleClient>> ContentBundleClients = ContentBundleEngineSubsystem->GetContentBundleClients();

	if (!ContentBundleClients.IsEmpty())
	{
		UE_LOGF(LogContentBundle, Verbose, "%ls Begin initializing ContentBundles from %u registered clients.", *ContentBundle::Log::MakeDebugInfoString(*this), ContentBundleClients.Num());

		for (TSharedPtr<FContentBundleClient>& ContentBundleClient : ContentBundleClients)
		{
			OnContentBundleClientRegistered(ContentBundleClient);
		}

		UE_LOGF(LogContentBundle, Verbose, "%ls End initializing ContentBundles.", *ContentBundle::Log::MakeDebugInfoString(*this));
	}
}

void FContentBundleContainer::DeinitializeContentBundles()
{
	ForEachContentBundle([](FContentBundleBase* ContentBundle) { ContentBundle->Uninitialize(); });

#if WITH_EDITOR
	if (UseEditorContentBundle())
	{
		GetEditorContentBundles().Empty();
		return;
	}
#endif

	GetGameContentBundles().Empty();
}

#if WITH_EDITOR

void FContentBundleContainer::OnPreGenerateStreaming(TArray<FString>* OutPackageToGenerate)
{
	UE_LOGF(LogContentBundle, Log, "%ls Generating Streaming for %u Content Bundles.", *ContentBundle::Log::MakeDebugInfoString(*this), GetEditorContentBundles().Num());

	if (IsRunningCookCommandlet())
	{
		// Cooking for plugin is not in yet.
		// Avoid calling OnPreGenerateStreaming from the cook package splitter.
		// It is not the correct flow & It outputs an error in the build log.
		return;
	}

	GetInjectedWorld()->ContentBundleManager->GetPIEDuplicateHelper()->Clear();

	const bool bIsPIE = true;
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		ContentBundle->GenerateStreaming(OutPackageToGenerate, bIsPIE);
	}
}

void FContentBundleContainer::OnBeginCook(IWorldPartitionCookPackageContext& CookContext)
{
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		ContentBundle->OnBeginCook(CookContext);
	}
}

void FContentBundleContainer::OnEndCook(IWorldPartitionCookPackageContext& CookContext)
{
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		ContentBundle->OnEndCook(CookContext);
	}
}
#endif
