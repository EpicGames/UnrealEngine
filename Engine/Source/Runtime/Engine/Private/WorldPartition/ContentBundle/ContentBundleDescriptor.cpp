// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"

#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleDescriptor)

UContentBundleDescriptor::UContentBundleDescriptor(const FObjectInitializer& ObjectInitializer)
	: DebugColor(FColor::Black)
{

}

FString UContentBundleDescriptor::GetPackageRoot() const
{
	return FPackageName::GetPackageMountPoint(GetPackage()->GetName()).ToString();
}

bool UContentBundleDescriptor::IsValid() const
{
	return Guid.IsValid() && !DisplayName.IsEmpty();
}

FString UContentBundleDescriptor::GetContentBundleCompactString(const FGuid& InContentBundleID)
{
	UE_CLOGF(!InContentBundleID.IsValid(), LogContentBundle, Log, "Called UContentBundleDescriptor::GetContentBundleCompactString with an invalid Content Bundle guid.")
	return FString::Printf(TEXT("%X"), GetTypeHash(InContentBundleID));
}

#if WITH_EDITOR
void UContentBundleDescriptor::InitializeObject(const FString& InContentBundleName)
{
	Guid = FGuid::NewGuid();
	DisplayName = InContentBundleName;
	InitDebugColor();
}

void UContentBundleDescriptor::PostLoad()
{
	InitDebugColor();

	Super::PostLoad();
}

#if WITH_EDITOR
void UContentBundleDescriptor::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		Guid = FGuid::NewGuid();
		InitDebugColor();
	}
}
#endif

void UContentBundleDescriptor::InitDebugColor()
{
	// If not set, generate a color based on guid
	if (DebugColor == FColor::Black)
	{
		DebugColor = FColor::MakeRandomSeededColor(GetTypeHash(Guid));
	}
}
#endif
