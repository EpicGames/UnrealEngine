// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleEditor)

#if WITH_EDITOR

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionSettings.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystemInterface.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/Cook/WorldPartitionCookPackage.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContextInterface.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"

FContentBundleEditor::FContentBundleEditor(TSharedPtr<FContentBundleClient>& InClient, UWorld* InWorld)
	: FContentBundleBase(InClient, InWorld)
	, UnsavedActorMonitor(nullptr)
	, ExternalStreamingObject(nullptr)
	, TreeItemID(FGuid::NewGuid())
	, bIsBeingEdited(false)
	, bIsInCook(false)
{}

FContentBundleEditor::~FContentBundleEditor()
{
	check(!bIsInCook);
}

void FContentBundleEditor::DoInitialize()
{
	SetStatus(EContentBundleStatus::Registered);

	if (IContentBundleEditorSubsystemInterface* EditorSubsystem = IContentBundleEditorSubsystemInterface::Get())
	{
		EditorSubsystem->NotifyContentBundleAdded(this);
	}
}

void FContentBundleEditor::DoUninitialize()
{
	if (IContentBundleEditorSubsystemInterface* EditorSubsystem = IContentBundleEditorSubsystemInterface::Get())
	{
		EditorSubsystem->NotifyContentBundleRemoved(this);
	}

	SetStatus(EContentBundleStatus::Unknown);
}

void FContentBundleEditor::DoInjectContent()
{
	FString ActorDescContainerPackage;
	bool bCreatedContainerPath = ContentBundlePaths::BuildActorDescContainerPackagePath(GetDescriptor()->GetPackageRoot(), GetDescriptor()->GetGuid(), GetInjectedWorld()->GetPackage()->GetName(), ActorDescContainerPackage);
	if (bCreatedContainerPath)
	{
		UWorldPartition* WorldPartition = GetInjectedWorld()->GetWorldPartition();
		UActorDescContainerInstance::FInitializeParams InitParams(*ActorDescContainerPackage);
		
		const FTopLevelAssetPath InjectedWorldAssetPath = FSoftObjectPath(GetInjectedWorld()).GetAssetPath();
		InitParams.ContentBundleGuid = GetDescriptor()->GetGuid();
		InitParams.FilterActorDescFunc = [&](const FWorldPartitionActorDesc* ActorDesc)
		{
			if (ActorDesc->GetActorSoftPath().GetAssetPath() != InjectedWorldAssetPath)
			{
				UE_LOGF(LogContentBundle, Display, "%ls Container '%ls' : Actor '%ls' object path doesn't match injected world path '%ls'",
					*ContentBundle::Log::MakeDebugInfoString(*this),
					*ActorDescContainerPackage,
					*ActorDesc->GetActorSoftPath().ToString(),
					*InjectedWorldAssetPath.ToString());
				return false;
			}

			return true;
		};

		InitParams.OnInitializedFunc = [this](UActorDescContainerInstance* InActorDescContainerInstance)
		{ 
			check(InActorDescContainerInstance->GetContentBundleGuid() == GetDescriptor()->GetGuid());
		};

		ActorDescContainerInstance = WorldPartition->RegisterActorDescContainerInstance(InitParams);
		if (ActorDescContainerInstance.IsValid() && ActorDescContainerInstance->IsInitialized())
		{
			UE_CLOGF(ActorDescContainerInstance->GetActorDescInstanceCount() > 0, LogContentBundle, Log, "%ls ExternalActors in %ls found. %u actors were injected", *ContentBundle::Log::MakeDebugInfoString(*this), *ActorDescContainerInstance->GetExternalActorPath(), ActorDescContainerInstance->GetActorDescInstanceCount());

			check(GetDescriptor()->GetGuid().IsValid());

			if (!ActorDescContainerInstance->GetContainer()->IsEmpty())
			{
				WorldDataLayersActorReference = FWorldDataLayersReference(ActorDescContainerInstance.Get(), BuildWorlDataLayersName());
				SetStatus(EContentBundleStatus::ContentInjected);
			}
			else
			{
				SetStatus(EContentBundleStatus::ReadyToInject);
			}

			UnsavedActorMonitor = NewObject<UContentBundleUnsavedActorMonitor>(GetTransientPackage(), NAME_None, RF_Transactional);
			UnsavedActorMonitor->Initialize(*this);

			RegisterDelegates();
		}
		else
		{
			UE_LOGF(LogContentBundle, Error, "%ls Failed to register actor desc container with %ls", *ContentBundle::Log::MakeDebugInfoString(*this), *ActorDescContainerPackage);
			SetStatus(EContentBundleStatus::FailedToInject);
		}
	}
	else
	{
		UE_LOGF(LogContentBundle, Error, "%ls Failed to build Container Package Path using %ls", *ContentBundle::Log::MakeDebugInfoString(*this), *GetInjectedWorld()->GetPackage()->GetName());
		SetStatus(EContentBundleStatus::FailedToInject);
	}

	if (IContentBundleEditorSubsystemInterface* EditorSubsystem = IContentBundleEditorSubsystemInterface::Get())
	{
		EditorSubsystem->NotifyContentBundleInjectedContent(this);
	}
}

void FContentBundleEditor::DoRemoveContent()
{
	UnreferenceAllActors();

	if (IContentBundleEditorSubsystemInterface* EditorSubsystem = IContentBundleEditorSubsystemInterface::Get())
	{
		EditorSubsystem->NotifyContentBundleRemovedContent(this);
	}

	// Make sure the content bundle is no longer flagged as edited before unloading actors
	check(!IsBeingEdited());

	WorldDataLayersActorReference.Reset();

	UnsavedActorMonitor->Uninitialize();

	// Might have been uninitialized by the WorldPartition 
	if (UActorDescContainerInstance* ContainerInstance = ActorDescContainerInstance.Get(); ContainerInstance && ContainerInstance->IsInitialized())
	{
		UnregisterDelegates();

		GetInjectedWorld()->GetWorldPartition()->UnregisterActorDescContainerInstance(ActorDescContainerInstance.Get());
	}
	ActorDescContainerInstance = nullptr;

	ExternalStreamingObject = nullptr;
	CookPackageIdsToCell.Empty();

	SetStatus(EContentBundleStatus::Registered);

	BroadcastChanged();
}

bool FContentBundleEditor::IsValid() const
{
	return true;
}

bool FContentBundleEditor::HasContent() const
{
	return !!ExternalStreamingObject;
}

bool FContentBundleEditor::AddActor(AActor* InActor)
{
	check(GetStatus() == EContentBundleStatus::ContentInjected || GetStatus() == EContentBundleStatus::ReadyToInject);

	if (InActor->GetWorld() != GetInjectedWorld() || InActor->HasAllFlags(RF_Transient) || !InActor->IsMainPackageActor())
	{
		return false;
	}

	if (InActor->IsA<AWorldDataLayers>())
	{
		return false;
	}

	if (GetStatus() == EContentBundleStatus::ReadyToInject)
	{
		UE_LOGF(LogContentBundle, Verbose, "%ls Adding first actor to content bundle.", *ContentBundle::Log::MakeDebugInfoString(*this));
		SetStatus(EContentBundleStatus::ContentInjected);
	}

	check(GetStatus() == EContentBundleStatus::ContentInjected);

	// Assign the container guid to the actor
	check(!InActor->GetContentBundleGuid().IsValid());
	check(GetDescriptor()->GetGuid().IsValid());
	FSetActorContentBundleGuid SetActorContentBundleGuid(InActor, GetDescriptor()->GetGuid());

	// Rename the actor so it is saved in the content bundle location
	FString ActorPackageNameInContentBundle = ContentBundlePaths::MakeExternalActorPackagePath(ActorDescContainerInstance->GetExternalActorPath(), InActor->GetName());
	verify(InActor->GetPackage()->Rename(*ActorPackageNameInContentBundle));

	UnsavedActorMonitor->MonitorActor(InActor);

	UE_LOGF(LogContentBundle, Verbose, "%ls Added new actor %ls, ActorCount:  %u. Package %ls.", *ContentBundle::Log::MakeDebugInfoString(*this), *InActor->GetActorNameOrLabel(), GetActorCount(), *InActor->GetPackage()->GetName());

	return true;
}

bool FContentBundleEditor::ContainsActor(const AActor* InActor) const
{
	if (InActor != nullptr)
	{
		return ActorDescContainerInstance->GetActorDescInstance(InActor->GetActorGuid()) != nullptr || UnsavedActorMonitor->IsMonitoring(InActor);
	}

	return false;
}

bool FContentBundleEditor::GetActors(TArray<AActor*>& Actors)
{
	Actors.Reserve(GetActorCount());

	if (ActorDescContainerInstance.IsValid())
	{
		for (UActorDescContainerInstance::TIterator<> It(ActorDescContainerInstance.Get()); It; ++It)
		{
			if (AActor* Actor = It->GetActor())
			{
				if (Actor != WorldDataLayersActorReference.Get())
				{
					Actors.Add(Actor);
				}
			}
		}
	}

	if (UnsavedActorMonitor)
	{
		for (auto UnsavedActor : UnsavedActorMonitor->GetUnsavedActors())
		{
			if (UnsavedActor.IsValid())
			{
				Actors.Add(UnsavedActor.Get());
			}
		}
	}

	return !Actors.IsEmpty();
}

bool FContentBundleEditor::HasUserPlacedActors() const
{
	// If there is only one actor in the container its the WorldDataLayer automatically created when injecting base content.
	bool bActorDescContHasUserPlacedActors = ActorDescContainerInstance.IsValid() && ActorDescContainerInstance->GetActorDescInstanceCount() > 1;
	return (bActorDescContHasUserPlacedActors || UnsavedActorMonitor->IsMonitoringActors());
}

uint32 FContentBundleEditor::GetActorCount() const
{
	if (GetStatus() == EContentBundleStatus::ContentInjected)
	{
		uint32 UnsavedWorldDataLayerCount = WorldDataLayersActorReference.IsValid() && ActorDescContainerInstance.IsValid() && ActorDescContainerInstance->IsEmpty() ? 1 : 0;
		return ActorDescContainerInstance->GetActorDescInstanceCount() + UnsavedActorMonitor->GetActorCount() + UnsavedWorldDataLayerCount;
	}

	return 0;
}

uint32 FContentBundleEditor::GetUnsavedActorAcount() const
{
	if (GetStatus() == EContentBundleStatus::ContentInjected)
	{
		return UnsavedActorMonitor->GetActorCount();
	}

	return 0;
}

void FContentBundleEditor::ReferenceAllActors()
{
	if (ActorDescContainerInstance.IsValid())
	{
		ActorDescContainerInstance->LoadAllActors(ForceLoadedActors);
	}
}

void FContentBundleEditor::UnreferenceAllActors()
{
	ForceLoadedActors.Empty();
}

void FContentBundleEditor::StartEditing()
{
	check(GetStatus() == EContentBundleStatus::ReadyToInject || GetStatus() == EContentBundleStatus::ContentInjected);

	UnsavedActorMonitor->StartListenOnActorEvents();

	bIsBeingEdited = true;
}

void FContentBundleEditor::StopEditing()
{
	check(GetStatus() == EContentBundleStatus::ReadyToInject || GetStatus() == EContentBundleStatus::ContentInjected);

	UnsavedActorMonitor->StopListeningOnActorEvents();

	bIsBeingEdited = false;
}

void FContentBundleEditor::InjectBaseContent()
{
	check(GetStatus() == EContentBundleStatus::ReadyToInject);
	UE_LOGF(LogContentBundle, Log, "%ls Injecting Base Content", *ContentBundle::Log::MakeDebugInfoString(*this));

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = BuildWorlDataLayersName();
	SpawnParameters.OverrideLevel = GetInjectedWorld()->PersistentLevel;
	SpawnParameters.OverridePackage = CreateActorPackage(SpawnParameters.Name);
	SpawnParameters.bCreateActorPackage = false;

	WorldDataLayersActorReference = FWorldDataLayersReference(SpawnParameters);

	WorldDataLayersActorReference->SetActorLabel(GetDisplayName());

	SetStatus(EContentBundleStatus::ContentInjected);

	BroadcastChanged();
}

void FContentBundleEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	FContentBundleBase::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(ExternalStreamingObject);
	Collector.AddReferencedObject(UnsavedActorMonitor);
}

void FContentBundleEditor::GenerateStreaming(TArray<FString>* OutPackageToGenerate, bool bIsPIE)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FContentBundleEditor::GenerateStreaming);

	// Skipping streaming generation for non-injected content
	if (GetStatus() != EContentBundleStatus::ContentInjected)
	{
		return;
	}

	FActorDescContainerInstanceCollection Collection({ TObjectPtr<UActorDescContainerInstance>(ActorDescContainerInstance.Get()) });
	UWorldPartition::FGenerateStreamingParams Params = UWorldPartition::FGenerateStreamingParams()
		.SetContainerInstanceCollection(Collection, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAsContentBundle);

	// Only dump streaming generation logs when generating for PIE or cook
	if (!bIsPIE && !IsRunningCookCommandlet())
	{
		Params.SetOutputLogType(TEXT(""));
	}

	UWorldPartition::FGenerateStreamingContext Context = UWorldPartition::FGenerateStreamingContext()
		.SetLevelPackagesToGenerate(OutPackageToGenerate);

	UWorldPartition* WorldPartition = GetInjectedWorld()->GetWorldPartition();
	WorldPartition->GenerateContainerStreaming(Params, Context);

	ExternalStreamingObject = WorldPartition->FlushStreamingToExternalStreamingObject();
	ExternalStreamingObject->SetPackagePathToCreate(GetExternalStreamingObjectPackagePath()); 

	uint32 CellCount = 0;
	ExternalStreamingObject->ForEachStreamingCells([&CellCount](const UWorldPartitionRuntimeCell& Cell)
	{
		CellCount++;
		return true;
	});
	UE_LOGF(LogContentBundle, Log, "%ls Generated streaming cells. %u cells were generated", *ContentBundle::Log::MakeDebugInfoString(*this), CellCount);

	if (bIsPIE)
	{
		UContentBundleManager* ContentBundleManager = GetInjectedWorld()->ContentBundleManager;
		UContentBundleDuplicateForPIEHelper* DuplicateForPIEHelper = ContentBundleManager->GetPIEDuplicateHelper();
		if (!DuplicateForPIEHelper->StoreContentBundleStreamingObect(*this, ExternalStreamingObject))
		{
			UE_LOGF(LogContentBundle, Error, "%ls Failed to store streaming object. PIE duplication will not work.", *ContentBundle::Log::MakeDebugInfoString(*this));
		}

		// Clear streaming object. It will be kept alive & duplicated by UContentBundleDuplicateForPIEHelper.
		ExternalStreamingObject = nullptr;
	}
}

void FContentBundleEditor::OnBeginCook(IWorldPartitionCookPackageContext& CookContext)
{
	check(!bIsInCook);
	CookContext.RegisterPackageCookPackageGenerator(this);
	bIsInCook = true;
}

void FContentBundleEditor::OnEndCook(IWorldPartitionCookPackageContext& CookContext)
{
	check(bIsInCook);
	CookContext.UnregisterPackageCookPackageGenerator(this);
	bIsInCook = false;
}

bool FContentBundleEditor::GatherPackagesToCook(class IWorldPartitionCookPackageContext& CookContext)
{
	TArray<FString> PackageToGenerate;
	const bool bIsPIE = false;
	GenerateStreaming(&PackageToGenerate, bIsPIE);

	CookPackageIdsToCell.Empty();

	bool bIsSuccess = true;

	// Do not cook ContentBundles when building with (exclusive) EDL filters
	if (HasCookedContent() && UWorldPartitionSettings::GetEDLCookFilters().IsEmpty())
	{
		ExternalStreamingObject->ForEachStreamingCells([this, &bIsSuccess, &CookContext](UWorldPartitionRuntimeCell& RuntimeCell)
		{
			if (const FWorldPartitionCookPackage* CookPackage = CookContext.AddPackageToGenerate(this, &RuntimeCell, ContentBundlePaths::GetCookedContentBundleLevelFolder(*this), RuntimeCell.GetPackageNameToCreate()))
			{
				CookPackageIdsToCell.Emplace(CookPackage->PackageId, &RuntimeCell);
			}
			else
			{
				UE_LOGF(LogContentBundle, Error, "%ls[Cook] Failed to add cell package %ls in cook context.", *ContentBundle::Log::MakeDebugInfoString(*this), *RuntimeCell.GetPackageNameToCreate());
				bIsSuccess = false;
			}
		});
		
		const FWorldPartitionCookPackage* CookPackage = CookContext.AddPackageToGenerate(this, ExternalStreamingObject, ContentBundlePaths::GetCookedContentBundleLevelFolder(*this), GetExternalStreamingObjectPackageName());
		if (CookPackage == nullptr)
		{
			UE_LOGF(LogContentBundle, Error, "%ls[Cook] Failed to add streaming object package in cook context.", *ContentBundle::Log::MakeDebugInfoString(*this));
			bIsSuccess = false;
		}
	}


	return bIsSuccess;
}

bool FContentBundleEditor::PopulateGeneratorPackageForCook(class IWorldPartitionCookPackageContext& CookContext, const TArray<FWorldPartitionCookPackage*>& PackagesToCook, TArray<UPackage*>& OutModifiedPackages)
{
	bool bIsSuccess = true;

	if (HasCookedContent())
	{
		UE_LOGF(LogContentBundle, Log, "%ls[Cook] Populating Generator Package. %u Packages", *ContentBundle::Log::MakeDebugInfoString(*this), PackagesToCook.Num());

		for (const FWorldPartitionCookPackage* CookPackage : PackagesToCook)
		{
			if (CookPackage->Type == FWorldPartitionCookPackage::EType::Level)
			{
				UWorldPartitionRuntimeCell** MatchingCell = const_cast<UWorldPartitionRuntimeCell**>(CookPackageIdsToCell.Find(CookPackage->PackageId));
				if (UWorldPartitionRuntimeCell* Cell = MatchingCell ? *MatchingCell : nullptr)
				{
					// Make sure the cell outer is set to the  ExternalStreamingObject so it will be saved in the right package at the end of the cook.
					check(Cell->GetOuter() == ExternalStreamingObject);

					if (!Cell->OnPopulateGeneratorPackageForCook(CookContext, CookPackage->GetPackage()))
					{
						UE_LOGF(LogContentBundle, Error, "%ls[Cook] Failed to prepare cell with package %ls for cook.", *ContentBundle::Log::MakeDebugInfoString(*this), *CookPackage->RelativePath);
						bIsSuccess = false;
					}
				}
				else
				{
					UE_LOGF(LogContentBundle, Error, "%ls[Cook] Could not find cell for package %ls while populating generator packages.", *ContentBundle::Log::MakeDebugInfoString(*this), *CookPackage->RelativePath);
					bIsSuccess = false;
				}
			}
		}

		ExternalStreamingObject->OnPopulateGeneratorPackageForCook(CookContext, nullptr);
	}
	

	return bIsSuccess;
}

bool FContentBundleEditor::PopulateGeneratedPackageForCook(class IWorldPartitionCookPackageContext& CookContext, const FWorldPartitionCookPackage& PackageToCook, TArray<UPackage*>& OutModifiedPackages)
{
	bool bIsSuccess = true;

	UE_LOGF(LogContentBundle, Log, "%ls[Cook] Populating Generated Package %ls", *ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);

	if (PackageToCook.Type == FWorldPartitionCookPackage::EType::Level)
	{
		if (UWorldPartitionRuntimeCell** MatchingCell = const_cast<UWorldPartitionRuntimeCell**>(CookPackageIdsToCell.Find(PackageToCook.PackageId)))
		{
			UWorldPartitionRuntimeCell* Cell = *MatchingCell;
			if (ensure(Cell))
			{
				if (!Cell->IsAlwaysLoaded())
				{
					TArray<UPackage*> ModifiedPackages;
					if (Cell->OnPopulateGeneratedPackageForCook(CookContext, PackageToCook.GetPackage(), OutModifiedPackages))
					{
						UWorld* CellWorld = FindObject<UWorld>(PackageToCook.GetPackage(), *GetInjectedWorld()->GetName());
						if (CellWorld != nullptr)
						{
							// @todo_ow : Order on PopulateGeneratedPackageForCook for each PackageToCook parameters is not deterministic.
							// PopulateGeneratedPackageForCook can be called on some cell(s) before it was called for the ExternalStreamingObject.
							// This lead to a wrong path in ULevel::WorldPartitionRuntimeCell because the ExternalStreamingObject could not yet have been renamed.
							// Compute & set ULevel::WorldPartitionRuntimeCell by hand in FWorldPartitionLevelHelper::RemapLevelCellPathInContentBundle
							// We already know that the cell will reside in the ExternalStreamingObject package at runtime.
							if (!FWorldPartitionLevelHelper::RemapLevelCellPathInContentBundle(CellWorld->PersistentLevel, this, Cell))
							{
								UE_LOGF(LogContentBundle, Warning, "%ls[Cook] Failed to Remap cell in level for package %ls.", *ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);
								bIsSuccess = false;
							}
						}
						else
						{
							UE_LOGF(LogContentBundle, Warning, "%ls[Cook] Failed to retrieve cell world from package %ls.", *ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);
							bIsSuccess = false;
						}
					}
					else
					{
						UE_LOGF(LogContentBundle, Error, "%ls[Cook] Failed to populate cell package %ls.", *ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);
						bIsSuccess = false;
					}
				}
				else
				{
					UE_LOGF(LogContentBundle, Warning, "%ls[Cook] Cell %ls is flagged always loaded. Content Bundles cells should never be always loaded. It will not be populated and be empty at runtime.",
						*ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);
					bIsSuccess = false;
				}
			}
		}
		else
		{
			UE_LOGF(LogContentBundle, Error, "%ls[Cook] Could not find cell for package %ls while populating generated pacakges.", *ContentBundle::Log::MakeDebugInfoString(*this), *PackageToCook.RelativePath);
			bIsSuccess = false;
		}
	}
	else
	{
		if (!ExternalStreamingObject->OnPopulateGeneratedPackageForCook(CookContext, PackageToCook.GetPackage(), OutModifiedPackages))
		{
			UE_LOGF(LogContentBundle, Error, "%ls[Cook] Failed to rename streaming object package.", *ContentBundle::Log::MakeDebugInfoString(*this));
			bIsSuccess = false;
		}
	}

	return bIsSuccess;
}

UWorldPartitionRuntimeCell* FContentBundleEditor::GetCellForPackage(const FWorldPartitionCookPackage& PackageToCook) const
{
	UWorldPartitionRuntimeCell** MatchingCell = const_cast<UWorldPartitionRuntimeCell**>(CookPackageIdsToCell.Find(PackageToCook.PackageId));
	return MatchingCell ? *MatchingCell : nullptr;
}

void FContentBundleEditor::BroadcastChanged()
{
	if (IContentBundleEditorSubsystemInterface* EditorSubsystem = IContentBundleEditorSubsystemInterface::Get())
	{
		EditorSubsystem->NotifyContentBundleChanged(this);
	}
}

UPackage* FContentBundleEditor::CreateActorPackage(const FName& ActorName) const
{
	FString ActorPackagePath = ULevel::GetActorPackageName(ActorDescContainerInstance->GetExternalActorPath(), EActorPackagingScheme::Reduced, ActorName.ToString());
	UPackage* ActorPackage = CreatePackage(*ActorPackagePath);

	ActorPackage->SetDirtyFlag(true);

	return ActorPackage;
}

FName FContentBundleEditor::BuildWorlDataLayersName() const
{
	return *GetDescriptor()->GetGuid().ToString();
}

void FContentBundleEditor::RegisterDelegates()
{
	ActorDescContainerInstance->OnActorDescInstanceAddedEvent.AddRaw(this, &FContentBundleEditor::OnActorDescInstanceAdded);
	ActorDescContainerInstance->OnActorDescInstanceRemovedEvent.AddRaw(this, &FContentBundleEditor::OnActorDescInstanceRemoved);
}

void FContentBundleEditor::UnregisterDelegates()
{
	ActorDescContainerInstance->OnActorDescInstanceAddedEvent.RemoveAll(this);
	ActorDescContainerInstance->OnActorDescInstanceRemovedEvent.RemoveAll(this);
}

void FContentBundleEditor::OnActorDescInstanceAdded(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	UE_LOGF(LogContentBundle, Verbose, "%ls Added actor %ls to container, ActorCount: %u. Package %ls.", 
		*ContentBundle::Log::MakeDebugInfoString(*this), *ActorDescInstance->GetActorLabelOrName().ToString(), GetActorCount(), *ActorDescInstance->GetActorPackage().ToString());

	AActor* Actor = ActorDescInstance->GetActor();
	UnsavedActorMonitor->StopMonitoringActor(Actor);
}

void FContentBundleEditor::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	UE_LOGF(LogContentBundle, Verbose, "%ls Removed actor %ls from container, ActorCount:  %u. Package %ls.", 
		*ContentBundle::Log::MakeDebugInfoString(*this), *ActorDescInstance->GetActorLabelOrName().ToString(), GetActorCount(), *ActorDescInstance->GetActorPackage().ToString());

	if (!HasUserPlacedActors())
	{
		if (GetStatus() == EContentBundleStatus::ContentInjected)
		{
			SetStatus(EContentBundleStatus::ReadyToInject);
		}
	}

	AActor* ActorInWorld = ActorDescInstance->GetActor(false, false);
	check(ActorInWorld == nullptr || !UnsavedActorMonitor->IsMonitoring(ActorInWorld)); // ActorDesc existed is being deleted. Make sure the actor is not present in the unsaved list as it should have been saved for the desc to exist.
}

void FContentBundleEditor::OnUnsavedActorDeleted(AActor* Actor)
{
	UE_LOGF(LogContentBundle, Verbose, "%ls Removed unsaved actor %ls, ActorCount: %u. Package %ls.",
		*ContentBundle::Log::MakeDebugInfoString(*this), *Actor->GetActorNameOrLabel(), GetActorCount(), *Actor->GetPackage()->GetName());

	if (!HasUserPlacedActors())
	{
		SetStatus(EContentBundleStatus::ReadyToInject);
	}
}

#endif

UContentBundleUnsavedActorMonitor::~UContentBundleUnsavedActorMonitor()
{
#if WITH_EDITOR
	check(UnsavedActors.IsEmpty());
	check(ContentBundle == nullptr);
#endif
}

#if WITH_EDITOR

void UContentBundleUnsavedActorMonitor::Initialize(FContentBundleEditor& InContentBundleEditor)
{
	ContentBundle = &InContentBundleEditor;
}

void UContentBundleUnsavedActorMonitor::StartListenOnActorEvents()
{
	GEngine->OnLevelActorDeleted().AddUObject(this, &UContentBundleUnsavedActorMonitor::OnActorDeleted);
}

void UContentBundleUnsavedActorMonitor::StopListeningOnActorEvents()
{
	GEngine->OnLevelActorDeleted().RemoveAll(this);
}

void UContentBundleUnsavedActorMonitor::Uninitialize()
{
	StopListeningOnActorEvents();

	for (TWeakObjectPtr<AActor>& ActorPtr : UnsavedActors)
	{
		// @todo_ow: figure out how can this happen
		if (AActor* Actor = ActorPtr.Get())
		{
			ContentBundle->GetInjectedWorld()->DestroyActor(Actor);
		}
	}
	UnsavedActors.Empty();

	ContentBundle = nullptr;
}

void UContentBundleUnsavedActorMonitor::MonitorActor(AActor* InActor)
{
	Modify();
	UnsavedActors.Add(InActor);
}

bool UContentBundleUnsavedActorMonitor::StopMonitoringActor(AActor* InActor)
{
	uint32 Idx = UnsavedActors.IndexOfByKey(InActor);
	if (Idx != INDEX_NONE)
	{
		Modify();
		UnsavedActors.Remove(InActor);
		check(UnsavedActors.IndexOfByKey(InActor) == INDEX_NONE);
		return true;
	}

	return false;
}

bool UContentBundleUnsavedActorMonitor::IsMonitoring(const AActor* Actor) const
{ 
	return UnsavedActors.Contains(Actor); 
}

void UContentBundleUnsavedActorMonitor::OnActorDeleted(AActor* InActor)
{
	if (StopMonitoringActor(InActor))
	{
		ContentBundle->OnUnsavedActorDeleted(InActor);
	}
}

#endif
