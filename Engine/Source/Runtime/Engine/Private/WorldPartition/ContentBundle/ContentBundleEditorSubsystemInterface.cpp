// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystemInterface.h"
#include "CoreGlobals.h"

#if WITH_EDITOR

IContentBundleEditorSubsystemInterface* IContentBundleEditorSubsystemInterface::Instance = nullptr;

void IContentBundleEditorSubsystemInterface::SetInstance(IContentBundleEditorSubsystemInterface* InInstance)
{ 
	check(InInstance == nullptr || Instance == nullptr); 
	Instance = InInstance; 
}

#endif
