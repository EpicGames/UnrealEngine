// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"

#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "WorldPartition/ContentBundle/ContentBundleTypeFactory.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleEngineSubsystem)

void UContentBundleEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UClass* ContentBundlTypeFactoryClassValue = ContentBundleTypeFactoryClass.Get();
	if (!ContentBundlTypeFactoryClassValue)
	{
		ContentBundlTypeFactoryClassValue = UContentBundleTypeFactory::StaticClass();
	}
	ContentBundleTypeFactory = NewObject<UContentBundleTypeFactory>(this, ContentBundlTypeFactoryClassValue);
	check(ContentBundleTypeFactory);

	FWorldDelegates::OnPreWorldInitialization.AddUObject(this, &UContentBundleEngineSubsystem::OnPreWorldInitialization);
	FWorldDelegates::OnPostWorldCleanup.AddUObject(this, &UContentBundleEngineSubsystem::OnWorldPostCleanup);
}

void UContentBundleEngineSubsystem::Deinitialize()
{
	ContentBundleClients.Empty();

	FWorldDelegates::OnPreWorldInitialization.RemoveAll(this);
	FWorldDelegates::OnPostWorldCleanup.RemoveAll(this);

	ContentBundleTypeFactory = nullptr;
}

TSharedPtr<FContentBundleClient> UContentBundleEngineSubsystem::RegisterContentBundle(const UContentBundleDescriptor* Descriptor, const FString& ClientDisplayName)
{
	if (Descriptor != nullptr)
	{
		TSharedPtr<FContentBundleClient>* RegisteredClient = FindRegisteredClient(Descriptor);
		if (RegisteredClient == nullptr)
		{
			check(FindRegisteredClient(Descriptor) == nullptr);;

			TSharedPtr<FContentBundleClient>& ContentBundleClient = ContentBundleClients.Add_GetRef(ContentBundleTypeFactory->CreateClient(Descriptor, ClientDisplayName));

			UE_LOGF(LogContentBundle, Log, "[Client] New client registered. Client: %ls, Descriptor: %ls", *ContentBundleClient->GetDisplayName(), *ContentBundleClient->GetDescriptor()->GetDisplayName());

			OnContentBundleClientRegistered.Broadcast(ContentBundleClient);

			return ContentBundleClient;
		}
		else
		{
			UE_LOGF(LogContentBundle, Error, "[Client] Content bundle descriptor %ls is already registered by client %ls", *Descriptor->GetDisplayName(), *(*RegisteredClient)->GetDisplayName());
		}
	}

	return nullptr;
}	

void UContentBundleEngineSubsystem::UnregisterContentBundle(FContentBundleClient& InClient)
{
	uint32 Index = ContentBundleClients.IndexOfByPredicate([&InClient](const TSharedPtr<FContentBundleClient>& Client) {return Client.Get() == &InClient;});
	if (Index != INDEX_NONE)
	{
		const UContentBundleDescriptor* DescriptorBackup = InClient.GetDescriptor();
		check(FindRegisteredClient(DescriptorBackup) != nullptr);

		UE_LOGF(LogContentBundle, Log, "[Client] Unregistered Client. Client: %ls, Descriptor: %ls", *InClient.GetDisplayName(), *InClient.GetDescriptor()->GetDisplayName());

		OnContentBundleClientUnregistered.Broadcast(*ContentBundleClients[Index]);
		ContentBundleClients.RemoveAtSwap(Index);

		check(FindRegisteredClient(DescriptorBackup) == nullptr);
	}
	else 
	{
		UE_LOGF(LogContentBundle, Error, "[Client] Cannot unregister client %ls. It was not registered", *InClient.GetDisplayName());
	}	
}

void UContentBundleEngineSubsystem::RequestContentInjection(FContentBundleClient& InClient)
{
	if (TSharedPtr<FContentBundleClient>* Client = FindRegisteredClient(InClient.GetDescriptor()))
	{
		UE_LOGF(LogContentBundle, Log, "[Client] Client requested content injection. Client: %ls, Descriptor: %ls", *InClient.GetDisplayName(), *InClient.GetDescriptor()->GetDisplayName());

		OnContentBundleClientRequestedContentInjection.Broadcast(**Client);
	}
	else
	{
		UE_LOGF(LogContentBundle, Error, "[Client] Cannot inject client %ls\'s content. It was not registered", *InClient.GetDisplayName());
	}
}

void UContentBundleEngineSubsystem::RequestContentRemoval(FContentBundleClient& InClient)
{
	if (TSharedPtr<FContentBundleClient>* Client = FindRegisteredClient(InClient.GetDescriptor()))
	{
		UE_LOGF(LogContentBundle, Log, "[Client] Client requested content removal. Client: %ls, Descriptor: %ls", *InClient.GetDisplayName(), *InClient.GetDescriptor()->GetDisplayName());

		OnContentBundleClientRequestedContentRemoval.Broadcast(**Client);
	}
	else
	{
		UE_LOGF(LogContentBundle, Error, "[Client] Cannot inject client %ls\'s content. It was not registered", *InClient.GetDisplayName());
	}
}

const UContentBundleDescriptor* UContentBundleEngineSubsystem::GetContentBundleDescriptor(const FGuid& ContentBundleGuid) const
{
	for (const TSharedPtr<FContentBundleClient>& Client : ContentBundleClients)
	{
		const UContentBundleDescriptor* Descriptor = Client->GetDescriptor();
		if (Descriptor->GetGuid() == ContentBundleGuid)
		{
			return Descriptor;
		}
	}

	return nullptr;
}

UContentBundleEngineSubsystem* UContentBundleEngineSubsystem::Get()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UContentBundleEngineSubsystem>();
	}
	check(0); // Retrieving the ContentBundleEngineSubsystem before it was created.
	return nullptr;
}

TSharedPtr<FContentBundleClient>* UContentBundleEngineSubsystem::FindRegisteredClient(const UContentBundleDescriptor* Descriptor)
{
	return ContentBundleClients.FindByPredicate([Descriptor](const TSharedPtr<FContentBundleClient>& Client) { return Client->GetDescriptor() == Descriptor; });
}

TSharedPtr<FContentBundleClient>* UContentBundleEngineSubsystem::FindRegisteredClient(FContentBundleClient& InClient)
{
	TSharedPtr<FContentBundleClient>* RegisteredClient = FindRegisteredClient(InClient.GetDescriptor());
	check(RegisteredClient == nullptr || RegisteredClient->Get() == &InClient);
	return RegisteredClient;
}

void UContentBundleEngineSubsystem::OnPreWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (World->GetWorldPartition() && World->ContentBundleManager == nullptr)
	{
		World->ContentBundleManager = NewObject<UContentBundleManager>(World);
	}
	else if (!World->GetWorldPartition() && World->ContentBundleManager != nullptr)
	{
		World->ContentBundleManager = nullptr;
	}

	if (World->ContentBundleManager != nullptr)
	{
		World->ContentBundleManager->Initialize();
	}	
}

void UContentBundleEngineSubsystem::OnWorldPostCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	if (World->ContentBundleManager != nullptr)
	{
		World->ContentBundleManager->Deinitialize();
	}
}
