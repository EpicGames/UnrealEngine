// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/ContentBundleLog.h"

#include "Engine/World.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "WorldPartition/ContentBundle/ContentBundleContainer.h"
#include "UnrealEngine.h"

DEFINE_LOG_CATEGORY(LogContentBundle);

namespace ContentBundle
{
#if !NO_LOGGING
	namespace LogPrivate
	{
		FString MakeDebugInfoString(const UContentBundleDescriptor& ContentBundleDescriptor)
		{
			return FString::Printf(TEXT("[CB: %s]"), *ContentBundleDescriptor.GetDisplayName());
		}

		FString MakeWorldDebugInfoString(UWorld* World)
		{
			return FString::Printf(TEXT("[%s(%s)]"), *World->GetName(), *GetDebugStringForWorld(World));
		}

		FString MakeDebugInfoString(const UContentBundleDescriptor& ContentBundleDescriptor, UWorld* World)
		{
			return FString::Printf(TEXT("%s%s"), *MakeDebugInfoString(ContentBundleDescriptor), *MakeWorldDebugInfoString(World));
		}	
	}
#endif

FString Log::MakeDebugInfoString(const FContentBundleBase& ContentBundle)
{
#if !NO_LOGGING
	return LogPrivate::MakeDebugInfoString(*ContentBundle.GetDescriptor(), ContentBundle.GetInjectedWorld());
#else
	return FString();
#endif
}

FString Log::MakeDebugInfoString(const FContentBundleClient& ContentBundleClient)
{
#if !NO_LOGGING
	return LogPrivate::MakeDebugInfoString(*ContentBundleClient.GetDescriptor());
#else
	return FString();
#endif
}

FString Log::MakeDebugInfoString(const FContentBundleContainer& ContentBundleContainer)
{
#if !NO_LOGGING
	return LogPrivate::MakeWorldDebugInfoString(ContentBundleContainer.GetInjectedWorld());
#else
	return FString();
#endif
}

FString Log::MakeDebugInfoString(const FContentBundleClient& ContentBundleClient, UWorld* World)
{
#if !NO_LOGGING
	return LogPrivate::MakeDebugInfoString(*ContentBundleClient.GetDescriptor(), World);
#else
	return FString();
#endif
}

}