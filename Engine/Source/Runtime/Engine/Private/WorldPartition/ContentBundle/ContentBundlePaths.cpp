// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "Misc/Paths.h"
#include "AssetRegistry/AssetData.h"
#include "ExternalPackageHelper.h"

#if WITH_EDITOR
#include "PackageTools.h"
#endif

FName ContentBundlePaths::GetContentBundleGuidsAssetRegistryTag()
{
	static FName ContentBundleAssetRegistryTag(ContentBundleAssetRegistryTagString);
 
	return ContentBundleAssetRegistryTag;
}

void ContentBundlePaths::AddRegistryTags(FAssetRegistryTagsContext Context, TArray<FGuid>& ContentBundleGuids)
{
	if (ContentBundleGuids.Num() > 0)
	{
		FString ContentBundleGuidsStr = FString::JoinBy(ContentBundleGuids, TEXT(","), [&](const FGuid& Guid) { return Guid.ToString(); });
		Context.AddTag(UObject::FAssetRegistryTag(ContentBundlePaths::GetContentBundleGuidsAssetRegistryTag(), ContentBundleGuidsStr, UObject::FAssetRegistryTag::TT_Hidden));
	}
}

TArray<FGuid> ContentBundlePaths::ParseContentBundleGuids(const FAssetData& AssetData)
{
	FString ContentBundleGuidsStr;
	if (AssetData.GetTagValue(ContentBundlePaths::GetContentBundleGuidsAssetRegistryTag(), ContentBundleGuidsStr))
	{
		return ParseContentBundleGuids(ContentBundleGuidsStr);
	}
	else
	{
		return TArray<FGuid>();
	}
}

TArray<FGuid> ContentBundlePaths::ParseContentBundleGuids(const FString& ContentBundleGuidsStr)
{
	TArray<FGuid> ContentBundleGuids;
	TArray<FString> ContentBundleGuidsStrArray;

	ContentBundleGuidsStr.ParseIntoArray(ContentBundleGuidsStrArray, TEXT(","));
	for (const FString& GuidStr : ContentBundleGuidsStrArray)
	{
		FGuid ContentBundleGuid;
		if (FGuid::Parse(GuidStr, ContentBundleGuid))
		{
			ContentBundleGuids.Add(ContentBundleGuid);
		}
	}

	return ContentBundleGuids;
}

FString ContentBundlePaths::GetCookedContentBundleLevelFolder(const FContentBundleBase& ContentBundle)
{
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> GeneratedContentBundleLevelFolder;
	GeneratedContentBundleLevelFolder += TEXT("/");
	GeneratedContentBundleLevelFolder += ContentBundle.GetDescriptor()->GetPackageRoot();
	GeneratedContentBundleLevelFolder += TEXT("/CB/");
	GeneratedContentBundleLevelFolder += GetRelativeLevelFolder(ContentBundle);
	GeneratedContentBundleLevelFolder += TEXT("/");

	FString Result = *GeneratedContentBundleLevelFolder;
	FPaths::RemoveDuplicateSlashes(Result);
	return Result;
}

FString ContentBundlePaths::GetRelativeLevelFolder(const FContentBundleBase& ContentBundle)
{
	const UWorld* InjectedWorld = ContentBundle.GetInjectedWorld();
	FString WorldPackageName = InjectedWorld->GetPackage()->GetName();

	UE_CLOGF(!FPackageName::IsValidPath(WorldPackageName), LogContentBundle, Warning, "Invalid world package name '%ls' (maybe not yet mounted), Content Bundle won't work as expected", *WorldPackageName);

	FString PackageRoot, PackagePath, PackageName;
	FPackageName::SplitPackageName(WorldPackageName, &PackageRoot, &PackagePath, &PackageName);

	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> RelativeLevelFolder;
	RelativeLevelFolder += PackagePath;
	RelativeLevelFolder += PackageName;
	RelativeLevelFolder += TEXT("/");

	return *RelativeLevelFolder;
}

#if WITH_EDITOR

FString ContentBundlePaths::MakeExternalActorPackagePath(const FString& ContentBundleExternalActorFolder, const FString& ActorName)
{
	const FString ContentBundleExternalActor = ULevel::GetActorPackageName(ContentBundleExternalActorFolder, EActorPackagingScheme::Reduced, *ActorName);
	check(IsAContentBundleExternalActorPackagePath(ContentBundleExternalActor));
	return ContentBundleExternalActor;
}

bool ContentBundlePaths::IsAContentBundleExternalActorPackagePath(FStringView InPackagePath)
{
	return GetContentBundleGuidFromExternalActorPackagePath(InPackagePath).IsValid();
}

FStringView ContentBundlePaths::GetRelativeExternalActorPackagePath(FStringView InContentBundleExternalActorPackagePath)
{
	FStringView RelativeContentBundlePath = GetActorPathRelativeToExternalActors(InContentBundleExternalActorPackagePath);
	if (!RelativeContentBundlePath.IsEmpty())
	{
		check(RelativeContentBundlePath.Left(GetContentBundleFolder().Len()).Equals(GetContentBundleFolder()));
		RelativeContentBundlePath = RelativeContentBundlePath.RightChop(GetContentBundleFolder().Len());
		if (!RelativeContentBundlePath.IsEmpty())
		{
			return RelativeContentBundlePath.RightChop(RelativeContentBundlePath.Find(TEXT("/")));
		}
	}
	
	return FStringView();
}

FGuid ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(FStringView InContentBundleExternalActorPackagePath)
{
	FGuid Result;

	FStringView RelativeContentBundlePath = GetActorPathRelativeToExternalActors(InContentBundleExternalActorPackagePath);
	if (!RelativeContentBundlePath.IsEmpty())
	{
		check(RelativeContentBundlePath.Left(GetContentBundleFolder().Len()).Equals(GetContentBundleFolder()));
		RelativeContentBundlePath = RelativeContentBundlePath.RightChop(GetContentBundleFolder().Len());
		if (!RelativeContentBundlePath.IsEmpty())
		{
			RelativeContentBundlePath= RelativeContentBundlePath.LeftChop(RelativeContentBundlePath.Len() - RelativeContentBundlePath.Find(TEXT("/")));
			verify(FGuid::Parse(FString(RelativeContentBundlePath), Result));
		}
	}

	return Result;
}

FStringView ContentBundlePaths::GetActorPathRelativeToExternalActors(FStringView InContentBundleExternalActorPackagePath)
{
	uint32 ExternalActorIdx = UE::String::FindFirst(InContentBundleExternalActorPackagePath, FPackagePath::GetExternalActorsFolderName(), ESearchCase::IgnoreCase);
	if (ExternalActorIdx != INDEX_NONE)
	{
		FStringView RelativeContentBundlePath = InContentBundleExternalActorPackagePath.RightChop(ExternalActorIdx + FCString::Strlen(FPackagePath::GetExternalActorsFolderName()));
		if (RelativeContentBundlePath.Left(GetContentBundleFolder().Len()).Equals(GetContentBundleFolder()))
		{
			return RelativeContentBundlePath;
		}
	}
	return FStringView();
}

bool ContentBundlePaths::BuildContentBundleExternalActorPath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid, FString& OutContentBundleRootPath)
{
	if (InContenBundleMountPoint.IsEmpty() || !InContentBundleGuid.IsValid())
	{
		return false;
	}

	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ContentBundleRootPathBuilder;
	ContentBundleRootPathBuilder += TEXT("/");
	ContentBundleRootPathBuilder += InContenBundleMountPoint;
	ContentBundleRootPathBuilder += GetContentBundleFolder();
	ContentBundleRootPathBuilder += InContentBundleGuid.ToString();
	ContentBundleRootPathBuilder += TEXT("/");
	
	OutContentBundleRootPath = *ContentBundleRootPathBuilder;

	return true;
}

bool ContentBundlePaths::BuildContentBundleActorsRootPath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid, FString& OutContentBundleRootPath)
{
	if (InContenBundleMountPoint.IsEmpty() || !InContentBundleGuid.IsValid())
	{
		return false;
	}

	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ContentBundleRootPathBuilder;
	ContentBundleRootPathBuilder += TEXT("/");
	ContentBundleRootPathBuilder += InContenBundleMountPoint;
	ContentBundleRootPathBuilder += TEXT("/");
	ContentBundleRootPathBuilder += FPackagePath::GetExternalActorsFolderName();
	ContentBundleRootPathBuilder += GetContentBundleFolder();
	ContentBundleRootPathBuilder += InContentBundleGuid.ToString();

	OutContentBundleRootPath = *ContentBundleRootPathBuilder;

	return true;
}

bool ContentBundlePaths::BuildActorDescContainerPackagePath(const FString& InContenBundleMountPoint, const FGuid& InContentBundleGuid, const FString& InLevelPackagePath, FString& OutContainerPackagePath)
{
	FString ContentBundleRootPath;
	if (!BuildContentBundleExternalActorPath(InContenBundleMountPoint, InContentBundleGuid, ContentBundleRootPath))
	{
		return false;
	}

	FString LevelRoot, LevelPackagePath, LevelName;
	if (FPackageName::SplitLongPackageName(InLevelPackagePath, LevelRoot, LevelPackagePath, LevelName))
	{
		TStringBuilderWithBuffer<TCHAR, NAME_SIZE> ContentBundleActorDescContainerPackage;
		ContentBundleActorDescContainerPackage += ContentBundleRootPath;
		ContentBundleActorDescContainerPackage += LevelPackagePath;
		ContentBundleActorDescContainerPackage += LevelName;

		OutContainerPackagePath = UPackageTools::SanitizePackageName(*ContentBundleActorDescContainerPackage);
		return true;
	}

	return false;
}

FStringView ContentBundlePaths::GetRelativePath(FStringView InContentBundlePath)
{
	uint32 ContentBundleFolderIdx = UE::String::FindFirst(InContentBundlePath, GetContentBundleFolder(), ESearchCase::IgnoreCase);
	if (ContentBundleFolderIdx != INDEX_NONE)
	{
		FStringView RelativeContentBundlePath = InContentBundlePath.RightChop(ContentBundleFolderIdx + GetContentBundleFolder().Len());
		uint32 ContentBundleGuidEndIdx = UE::String::FindFirst(RelativeContentBundlePath, TEXT("/"), ESearchCase::IgnoreCase);
		if (ContentBundleGuidEndIdx != INDEX_NONE)
		{
			return RelativeContentBundlePath.RightChop(ContentBundleGuidEndIdx + 1); // + 1 to remove the "/"
		}
	}
	return FStringView();
}

bool ContentBundlePaths::IsAContentBundlePath(FStringView InContentBundlePath)
{
	return GetRelativePath(InContentBundlePath).Len() > 0;
}

void ContentBundlePaths::ForEachContentBundleLevelPackagePath(const FString& InLevelPackageName, TFunctionRef<void(const FString&)> Func)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ContentBundlePaths::ForEachContentBundleLevelPackagePath);

	UClass* GameFeatureDataClass = FindObject<UClass>(nullptr, TEXT("/Script/GameFeatures.GameFeatureData"));
	if (GameFeatureDataClass)
	{
		TArray<FAssetData> AssetsData;

		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = false;
		Filter.ClassPaths = { GameFeatureDataClass->GetClassPathName() };
		Filter.bRecursiveClasses = true;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets- avoids indeterministic results
		FExternalPackageHelper::GetSortedAssets(Filter, AssetsData);
		
		for (const FAssetData& AssetData : AssetsData)
		{
			TArray<FGuid> ContentBundleGuids;
			ContentBundleGuids = ParseContentBundleGuids(AssetData);

			if (ContentBundleGuids.Num())
			{
				const FString MountPoint = FPackageName::GetPackageMountPoint(AssetData.PackagePath.ToString()).ToString();

				for (const FGuid& ContentBundleGuid : ContentBundleGuids)
				{
					FString ContentBundleExternalActorPath;
					if (ContentBundlePaths::BuildActorDescContainerPackagePath(MountPoint, ContentBundleGuid, InLevelPackageName, ContentBundleExternalActorPath))
					{
						Func(ContentBundleExternalActorPath);
					}
				}
			}
		}
	}
}


#endif
