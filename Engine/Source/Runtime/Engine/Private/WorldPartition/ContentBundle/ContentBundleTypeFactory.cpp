// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleTypeFactory.h"

#include "WorldPartition/ContentBundle/ContentBundleClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleTypeFactory)

TSharedPtr<FContentBundleClient> UContentBundleTypeFactory::CreateClient(const UContentBundleDescriptor* Descriptor, const FString& ClientDisplayName)
{
	return MakeShared<FContentBundleClient>(Descriptor, ClientDisplayName);
}
