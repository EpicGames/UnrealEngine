// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"

#include "WorldPartition/ContentBundle/ContentBundleContainer.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleStatus.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleWorldSubsystem)


#if WITH_EDITOR
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "Editor.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#else
#include "Engine/Engine.h"
#endif

UContentBundleManager::UContentBundleManager()
{

}

void UContentBundleManager::Initialize()
{
#if WITH_EDITOR
	if (!GetWorld()->IsGameWorld())
	{
		PIEDuplicateHelper = NewObject<UContentBundleDuplicateForPIEHelper>(this);
		PIEDuplicateHelper->Initialize();
	}
#endif

	GetWorld()->OnWorldPartitionInitialized().AddUObject(this, &UContentBundleManager::OnWorldPartitionInitialized);
	GetWorld()->OnWorldPartitionUninitialized().AddUObject(this, &UContentBundleManager::OnWorldPartitionUninitialized);
}

void UContentBundleManager::Deinitialize()
{
	GetWorld()->OnWorldPartitionInitialized().RemoveAll(this);
	GetWorld()->OnWorldPartitionUninitialized().RemoveAll(this);

	for (TUniquePtr<FContentBundleContainer>& ContentBundleContainer : ContentBundleContainers)
	{
		ContentBundleContainer->Deinitialize();
	}
	ContentBundleContainers.Empty();

#if WITH_EDITOR
	if(PIEDuplicateHelper != nullptr)
	{
		PIEDuplicateHelper->Deinitialize();
		PIEDuplicateHelper = nullptr;
	}
#endif
}

bool UContentBundleManager::CanInject() const
{
	if (UWorldPartition* WorldPartition = GetTypedOuter<UWorld>()->GetWorldPartition())
	{
		return WorldPartition->IsInitialized();
	}

	return false;
}

void UContentBundleManager::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UContentBundleManager* Subsystem = CastChecked<UContentBundleManager>(InThis);
	for (TUniquePtr<FContentBundleContainer>& ContentBundleContainer : Subsystem->ContentBundleContainers)
	{
		ContentBundleContainer->AddReferencedObjects(Collector);
	}
}

void UContentBundleManager::DrawContentBundlesStatus(const UWorld* InWorld, UCanvas* Canvas, FVector2D& Offset) const
{
	if (const TUniquePtr<FContentBundleContainer>* ContentBundleContainer = GetContentBundleContainer(InWorld))
	{
		if ((*ContentBundleContainer)->GetNumContentBundles())
		{
			FVector2D Pos = Offset;
			float MaxTextWidth = 0.f;

			bool bHasInjectedContentBundle = false;
			(*ContentBundleContainer)->ForEachContentBundleBreakable([&bHasInjectedContentBundle](FContentBundleBase* ContentBundle)
			{
				if (ContentBundle->GetStatus() == EContentBundleStatus::ContentInjected)
				{
					bHasInjectedContentBundle = true;
				}
				return !bHasInjectedContentBundle;
			});

			if (bHasInjectedContentBundle)
			{
				FWorldPartitionDebugHelper::DrawText(Canvas, TEXT("Injected Content Bundles"), GEngine->GetSmallFont(), FColor::Green, Pos, &MaxTextWidth);

				(*ContentBundleContainer)->ForEachContentBundle([&Offset, &Canvas, &Pos, &MaxTextWidth](FContentBundleBase* ContentBundle)
				{
					if (ContentBundle->HasContent())
					{
						FWorldPartitionDebugHelper::DrawLegendItem(Canvas, *FString::Printf(TEXT("%s [%s]"), *ContentBundle->GetDisplayName(), *UContentBundleDescriptor::GetContentBundleCompactString(ContentBundle->GetDescriptor()->GetGuid())), GEngine->GetSmallFont(), ContentBundle->GetDebugColor(), FColor::White, Pos, &MaxTextWidth);
					}
				});
			}

			Offset.X += MaxTextWidth + 10;
		}
	}
}

const FContentBundleBase* UContentBundleManager::GetContentBundle(const UWorld* InWorld, const FGuid& Guid) const
{
	const FContentBundleBase* FoundContentBundle = nullptr;
	if (const TUniquePtr<FContentBundleContainer>* ContentBundleContainer = GetContentBundleContainer(InWorld))
	{
		(*ContentBundleContainer)->ForEachContentBundleBreakable([&FoundContentBundle, &Guid](FContentBundleBase* ContentBundle)
		{
			if (ContentBundle->GetDescriptor()->GetGuid() == Guid)
			{
				FoundContentBundle = ContentBundle;
				return false;
			}
			return true;
		});
	}
	return FoundContentBundle;
}

#if WITH_EDITOR

bool UContentBundleManager::GetEditorContentBundle(TArray<TSharedPtr<FContentBundleEditor>>& OutContentBundles)
{
	for (TUniquePtr<FContentBundleContainer>& ContentBundleContainer : ContentBundleContainers)
	{
		OutContentBundles.Append(ContentBundleContainer->GetEditorContentBundles());
	}

	return !OutContentBundles.IsEmpty();
}

TSharedPtr<FContentBundleEditor> UContentBundleManager::GetEditorContentBundle(const UContentBundleDescriptor* Descriptor, const UWorld* ContentBundleWorld) const
{
	for (const TUniquePtr<FContentBundleContainer>& ContentBundleContainer : ContentBundleContainers)
	{
		if (ContentBundleContainer->GetInjectedWorld() == ContentBundleWorld)
		{
			return ContentBundleContainer->GetEditorContentBundle(Descriptor);
		}
	}

	return nullptr;
}

TSharedPtr<FContentBundleEditor> UContentBundleManager::GetEditorContentBundle(const FGuid& ContentBundleGuid) const
{
	for (const TUniquePtr<FContentBundleContainer>& ContentBundleContainer : ContentBundleContainers)
	{
		TSharedPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleContainer->GetEditorContentBundle(ContentBundleGuid);
		if (ContentBundleEditor)
		{
			return ContentBundleEditor;
		}
	}

	return nullptr;
}

void UContentBundleManager::CheckForErrors() const
{
	TMap<FGuid, const TSharedPtr<FContentBundleEditor>> ContentBundlesEditor;

	for (const TUniquePtr<FContentBundleContainer>& ContentBundleContainer : ContentBundleContainers)
	{
		for (const TSharedPtr<FContentBundleEditor>& ContentBundleEditor : ContentBundleContainer->GetEditorContentBundles())
		{
			const TSharedPtr<FContentBundleEditor>& ExistingValue = ContentBundlesEditor.FindOrAdd(ContentBundleEditor->GetDescriptor()->GetGuid(), ContentBundleEditor);
			if (ExistingValue.Get() != ContentBundleEditor.Get())
			{
				const FText DuplicateGuidText = FText::FromString(ContentBundleEditor->GetDescriptor()->GetGuid().ToString());

				FMessageLog("MapCheck").Error()
					->AddToken(FTextToken::Create(FText::Format(NSLOCTEXT("ContentBundle", "ContentBundleErrorDuplicateGUIDs", "Found content bundles sharing the same GUID ({0}):"), DuplicateGuidText)))
					->AddToken(FAssetNameToken::Create(ExistingValue->GetDescriptor()->GetPathName()))
					->AddToken(FAssetNameToken::Create(ContentBundleEditor->GetDescriptor()->GetPathName()));
			}
		}
	}
}

#endif

uint32 UContentBundleManager::GetContentBundleContainerIndex(const UWorld* InjectedWorld) const
{
	return ContentBundleContainers.IndexOfByPredicate([InjectedWorld](const TUniquePtr<FContentBundleContainer>& ContentBundleContainer)
		{
			return ContentBundleContainer->GetInjectedWorld() == InjectedWorld;
		});
}

const TUniquePtr<FContentBundleContainer>* UContentBundleManager::GetContentBundleContainer(const UWorld* InjectedWorld) const
{
	uint32 ContainerIndex = GetContentBundleContainerIndex(InjectedWorld);
	if (ContainerIndex != INDEX_NONE)
	{
		return &ContentBundleContainers[ContainerIndex];
	}

	return nullptr;
}

TUniquePtr<FContentBundleContainer>* UContentBundleManager::GetContentBundleContainer(const UWorld* InjectedWorld)
{
	return const_cast<TUniquePtr<FContentBundleContainer>*>(const_cast<const UContentBundleManager*>(this)->GetContentBundleContainer(InjectedWorld));
}

void UContentBundleManager::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	if (InWorldPartition->GetWorld() != InWorldPartition->GetTypedOuter<UWorld>())
	{
		return;
	}

	if (!InWorldPartition->IsContentBundleEnabled())
	{
		return;
	}

	check(GetContentBundleContainer(InWorldPartition->GetTypedOuter<UWorld>()) == nullptr);
	TUniquePtr<FContentBundleContainer>& ContentBundleContainer = ContentBundleContainers.Emplace_GetRef(MakeUnique<FContentBundleContainer>(InWorldPartition->GetTypedOuter<UWorld>()));
	ContentBundleContainer->Initialize();
}

void UContentBundleManager::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	if (InWorldPartition->GetWorld() != InWorldPartition->GetTypedOuter<UWorld>())
	{
		return;
	}

	uint32 ContainerIndex = GetContentBundleContainerIndex(InWorldPartition->GetTypedOuter<UWorld>());
	if (ContainerIndex == INDEX_NONE)
	{
		return;
	}

	check(ContentBundleContainers.IsValidIndex(ContainerIndex));
	ContentBundleContainers[ContainerIndex]->Deinitialize();
	ContentBundleContainers.RemoveAtSwap(ContainerIndex);
}

#if WITH_EDITOR

void UContentBundleDuplicateForPIEHelper::Initialize()
{
	FEditorDelegates::EndPIE.AddUObject(this, &UContentBundleDuplicateForPIEHelper::OnPIEEnded);
}

void UContentBundleDuplicateForPIEHelper::Deinitialize()
{
	FEditorDelegates::EndPIE.RemoveAll(this);

	Clear();
}

bool UContentBundleDuplicateForPIEHelper::StoreContentBundleStreamingObect(const FContentBundleEditor& ContentBundleEditor, URuntimeHashExternalStreamingObjectBase* StreamingObject)
{
	FGuid ContentBundleUid = ContentBundleEditor.GetDescriptor()->GetGuid();
	if (!StreamingObjects.Contains(ContentBundleUid))
	{
		StreamingObjects.Add(ContentBundleUid, StreamingObject);
	}
	
	return true;
}

URuntimeHashExternalStreamingObjectBase* UContentBundleDuplicateForPIEHelper::RetrieveContentBundleStreamingObject(const FContentBundle& ContentBundle) const
{
	const TObjectPtr<URuntimeHashExternalStreamingObjectBase>* ContentBundleStreamingObject = StreamingObjects.Find(ContentBundle.GetDescriptor()->GetGuid());
	return ContentBundleStreamingObject != nullptr ? *ContentBundleStreamingObject : nullptr;
}

void UContentBundleDuplicateForPIEHelper::OnPIEEnded(const bool bIsSimulating)
{
	Clear();
}

#endif
