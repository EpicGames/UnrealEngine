// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/WorldDataLayerReference.h"

#if WITH_EDITOR

#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

FWorldDataLayersReference::FWorldDataLayersReference()
{
	WorldDataLayersVariant.Set<AWorldDataLayers*>(nullptr);
}

FWorldDataLayersReference::~FWorldDataLayersReference()
{
	Reset();
}

FWorldDataLayersReference::FWorldDataLayersReference(UActorDescContainerInstance* ContainerInstance, FName WorldDataLayerName)
{
	TrySetReference(ContainerInstance, WorldDataLayerName);
}

FWorldDataLayersReference::FWorldDataLayersReference(const FActorSpawnParameters& SpawnParameters)
{
	AWorldDataLayers* WorldDataLayers = AWorldDataLayers::Create(SpawnParameters);
	WorldDataLayers->SetActorLabel(SpawnParameters.Name.ToString());
	WorldDataLayersVariant.Set<AWorldDataLayers*>(WorldDataLayers);
}

FWorldDataLayersReference::FWorldDataLayersReference(FWorldDataLayersReference&& Other)
	:WorldDataLayersVariant(MoveTemp(Other.WorldDataLayersVariant))
{
	if (Other.WorldDataLayersVariant.IsType<AWorldDataLayers*>())
	{
		Other.WorldDataLayersVariant.Set<AWorldDataLayers*>(nullptr);
	}
}

FWorldDataLayersReference& FWorldDataLayersReference::operator=(FWorldDataLayersReference&& Other)
{
	if (this != &Other)
	{
		WorldDataLayersVariant = MoveTemp(Other.WorldDataLayersVariant);

		if (Other.WorldDataLayersVariant.IsType<AWorldDataLayers*>())
		{
			Other.WorldDataLayersVariant.Set<AWorldDataLayers*>(nullptr);
		}
	}
	
	return *this;
}

const AWorldDataLayers* FWorldDataLayersReference::Get() const
{
	if (WorldDataLayersVariant.IsType<FWorldPartitionReference>())
	{
		const FWorldPartitionReference& WorldDataLayerReference = WorldDataLayersVariant.Get<FWorldPartitionReference>();
		if (WorldDataLayerReference.IsValid())
		{
			return Cast<AWorldDataLayers>(WorldDataLayerReference.GetActor());
		}
		return nullptr;
	}

	return WorldDataLayersVariant.Get<AWorldDataLayers*>();
}

bool FWorldDataLayersReference::TrySetReference(UActorDescContainerInstance* ContainerInstance, FName WorldDataLayerName)
{
	for (UActorDescContainerInstance::TIterator<> Iterator(ContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->GetActorNativeClass()->IsChildOf<AWorldDataLayers>())
		{
			if (Iterator->GetActorName() == WorldDataLayerName)
			{
				WorldDataLayersVariant.Emplace<FWorldPartitionReference>(ContainerInstance, Iterator->GetGuid());
				return true;
			}
		}
	}

	return false;
}

void FWorldDataLayersReference::Reset()
{
	// An Actor Descriptor might have been created since we created the reference.
	// Try resolving the reference. If it still does note resolve, delete the actor.
	if (WorldDataLayersVariant.IsType<AWorldDataLayers*>())
	{
		AWorldDataLayers* WorldDataLayers = WorldDataLayersVariant.Get<AWorldDataLayers*>();
		if (WorldDataLayers != nullptr)
		{
			if (UWorld* World = WorldDataLayers->GetWorld()) // May be null during Garbage collection
			{
				UWorldPartition* WorldPartition = World->GetWorldPartition();
				FWorldPartitionReference ResolvedReference(WorldPartition, WorldDataLayers->GetActorGuid());
				if (ResolvedReference.IsValid())
				{
					check(WorldDataLayers == ResolvedReference.GetActor());
					WorldDataLayersVariant.Emplace<FWorldPartitionReference>(ResolvedReference);
				}
				else
				{
					World->DestroyActor(WorldDataLayers);
					WorldDataLayersVariant.Set<AWorldDataLayers*>(nullptr);
				}
			}
			
		}
	}

	if (WorldDataLayersVariant.IsType<FWorldPartitionReference>())
	{
		WorldDataLayersVariant.Get<FWorldPartitionReference>().Reset();
	}
}

#endif // WITH_EDITOR
