// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionSettings.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/Cook/WorldPartitionCookPackage.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageInterface.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContextInterface.h"

void UWorldPartition::BeginCook(IWorldPartitionCookPackageContext& CookContext)
{
	OnBeginCook.Broadcast(CookContext);

	CookContext.RegisterPackageCookPackageGenerator(this);
}

void UWorldPartition::EndCook(IWorldPartitionCookPackageContext& CookContext)
{
	OnEndCook.Broadcast(CookContext);

	CookContext.UnregisterPackageCookPackageGenerator(this);

	FlushStreaming();
}

bool UWorldPartition::GatherPackagesToCook(IWorldPartitionCookPackageContext& CookContext)
{
	TArray<FString> LevelPackagesToCook;
	TArray<URuntimeHashExternalStreamingObjectBase*> ExternalStreamingObjectsToCook;
	FGenerateStreamingContext Context = FGenerateStreamingContext()
		.SetLevelPackagesToGenerate(&LevelPackagesToCook)
		.SetGeneratedExternalStreamingObjects(&ExternalStreamingObjectsToCook);

	// Generate streaming
	FGenerateStreamingParams Params = FGenerateStreamingParams()
		.SetContainerInstanceCollection(*this, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs)
		.SetFilteredClasses(CookContext.GetParams().FilteredClasses);

	// Only dump streaming generation logs when cooking
	if (!IsRunningCookCommandlet())
	{
		Params.SetOutputLogType(TEXT(""));
	}

	if (!GenerateContainerStreaming(Params, Context))
	{
		return false;
	}

	bool bIsSuccess = true;
	const FString WorldPackageName = GetPackage()->GetName();
	auto AddPackageToCookContext = [this, &CookContext, &WorldPackageName, &bIsSuccess](IWorldPartitionCookPackageObject* InCookPackageObject, const FString& InPackageName)
	{
		if (InCookPackageObject)
		{
			check(InCookPackageObject->GetPackageNameToCreate() == InPackageName);
			const UExternalDataLayerAsset* ExternalDataLayerAsset = InCookPackageObject->GetExternalDataLayerAsset();

			// The EDL filter list, when set, will exclude any package which is not part of the EDL list
			const TSet<FTopLevelAssetPath>& EDLCookFilters = UWorldPartitionSettings::GetEDLCookFilters();
			if (!EDLCookFilters.IsEmpty())
			{
				if (!ExternalDataLayerAsset || !EDLCookFilters.Contains(FTopLevelAssetPath(ExternalDataLayerAsset)))
				{
					return;
				}
			}

			const FString ExternalContentRoot = ExternalDataLayerAsset ? ExternalDataLayerManager->GetExternalDataLayerLevelRootPath(ExternalDataLayerAsset) : FString();
			const FString& Root = ExternalDataLayerAsset ? ExternalContentRoot : WorldPackageName;
			if (CookContext.AddPackageToGenerate(this, InCookPackageObject, Root, InPackageName))
			{
				return;
			}
		}
		bIsSuccess = false;
	};

	// Add level packages to the CookContext
	for (const FString& PackageName : LevelPackagesToCook)
	{
		IWorldPartitionCookPackageObject* CookPackageObject = GetCookPackageObject(FWorldPartitionCookPackage(TEXT("<Root>"), PackageName, FWorldPartitionCookPackage::EType::Level, FWorldPartitionPackageHash()));
		AddPackageToCookContext(CookPackageObject, PackageName);
	}

	// Add external streaming objects to the CookContext
	for (IWorldPartitionCookPackageObject* CookPackageObject : ExternalStreamingObjectsToCook)
	{
		AddPackageToCookContext(CookPackageObject, CookPackageObject->GetPackageNameToCreate());
	}

	return bIsSuccess;
}

bool UWorldPartition::PrepareGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, TArray<UPackage*>& OutModifiedPackages)
{
	bool bIsSuccess = true;
	check(RuntimeHash);
	for (IWorldPartitionCookPackageObject* CookPackageObject : RuntimeHash->GetAlwaysLoadedCells())
	{
		if (!CookPackageObject->OnPrepareGeneratorPackageForCook(OutModifiedPackages))
		{
			bIsSuccess = false;
		}
	}
	ExternalDataLayerManager->ForEachExternalStreamingObjects([&OutModifiedPackages, &bIsSuccess](URuntimeHashExternalStreamingObjectBase* StreamingObject)
	{
		if (!StreamingObject->OnPrepareGeneratorPackageForCook(OutModifiedPackages))
		{
			bIsSuccess = false;
		}
		return true;
	});
	return bIsSuccess;
}

bool UWorldPartition::PopulateGeneratorPackageForCook(IWorldPartitionCookPackageContext& CookContext, const TArray<FWorldPartitionCookPackage*>& InPackagesToCook, TArray<UPackage*>& OutModifiedPackages)
{
	auto OnPopulateGeneratorPackageForCook = [this, &CookContext](const FWorldPartitionCookPackage* InPackageToCook)
	{
		IWorldPartitionCookPackageObject* CookPackageObject = GetCookPackageObject(*InPackageToCook);
		return CookPackageObject && CookPackageObject->OnPopulateGeneratorPackageForCook(CookContext, InPackageToCook->GetPackage());
	};

	TArray<const FWorldPartitionCookPackage*> GenericPackagesToCook;
	for (const FWorldPartitionCookPackage* PackageToCook : InPackagesToCook)
	{
		if (PackageToCook->Type == FWorldPartitionCookPackage::EType::Level)
		{
			if (!OnPopulateGeneratorPackageForCook(PackageToCook))
			{
				return false;
			}
		}
		else // Delay generic packages as they depend on level packages
		{
			check(PackageToCook->Type == FWorldPartitionCookPackage::EType::Generic);
			GenericPackagesToCook.Add(PackageToCook);
		}
	}

	// Move on to delayed packages
	for (const FWorldPartitionCookPackage* GenericPackageToCook : GenericPackagesToCook)
	{
		if (!OnPopulateGeneratorPackageForCook(GenericPackageToCook))
		{
			return false;
		}
	}

	return true;
}

bool UWorldPartition::PopulateGeneratedPackageForCook(IWorldPartitionCookPackageContext& CookContext, const FWorldPartitionCookPackage& InPackageToCook, TArray<UPackage*>& OutModifiedPackages)
{
	auto GetCookPackageObjectPath = [this](IWorldPartitionCookPackageObject* InCookPackageObject)
	{
		const UExternalDataLayerAsset* ExternalDataLayerAsset = InCookPackageObject->GetExternalDataLayerAsset();
		const URuntimeHashExternalStreamingObjectBase* StreamingObject = ExternalDataLayerManager->ExternalStreamingObjects.FindChecked(ExternalDataLayerAsset);
		TStringBuilder<256> CookPackageObjectPath;
		CookPackageObjectPath += ExternalDataLayerManager->GetExternalStreamingObjectPackagePath(ExternalDataLayerAsset);
		CookPackageObjectPath += TEXT(".");
		CookPackageObjectPath += URuntimeHashExternalStreamingObjectBase::GetCookedExternalStreamingObjectName();
		CookPackageObjectPath += TEXT(".");
		CookPackageObjectPath += Cast<UObject>(InCookPackageObject)->GetName();
		return FSoftObjectPath(*CookPackageObjectPath);
	};

	OutModifiedPackages.Reset();
	if (IWorldPartitionCookPackageObject* CookPackageObject = GetCookPackageObject(InPackageToCook))
	{
		if (CookPackageObject->OnPopulateGeneratedPackageForCook(CookContext, InPackageToCook.GetPackage(), OutModifiedPackages))
		{
			// Since PopulateGeneratedPackageForCook on the external streaming object package changes its outer (which affects its cells object path) and can be
			// called after a call to PopulateGeneratedPackageForCook on a package of any cell external streaming object, we need to generate the cell path manually.
			if ((InPackageToCook.Type == FWorldPartitionCookPackage::EType::Level) && CookPackageObject->GetExternalDataLayerAsset())
			{
				if (UWorld* PackageWorld = UWorld::FindWorldInPackage(InPackageToCook.GetPackage()); ensure(PackageWorld))
				{
					FSetWorldPartitionRuntimeCell SetWorldPartitionRuntimeCell(PackageWorld->PersistentLevel, GetCookPackageObjectPath(CookPackageObject));
				}
			}
			return true;
		}
	}
	return false;
}

IWorldPartitionCookPackageObject* UWorldPartition::GetCookPackageObject(const FWorldPartitionCookPackage& InPackageToCook) const
{
	if (InPackageToCook.Type == FWorldPartitionCookPackage::EType::Level)
	{
		return GetCellForPackage(InPackageToCook);
	}
	else if (InPackageToCook.Type == FWorldPartitionCookPackage::EType::Generic)
	{
		return ExternalDataLayerManager->GetExternalStreamingObjectForCookPackage(InPackageToCook.RelativePath);
	}
	return nullptr;
}

UWorldPartitionRuntimeCell* UWorldPartition::GetCellForPackage(const FWorldPartitionCookPackage& InPackageToCook) const
{
	if (InPackageToCook.Type == FWorldPartitionCookPackage::EType::Level)
	{
		if (UWorldPartitionRuntimeCell* Cell = RuntimeHash->GetCellForCookPackage(InPackageToCook.RelativePath))
		{
			return Cell;
		}
		return ExternalDataLayerManager->GetCellForCookPackage(InPackageToCook.RelativePath);
	}
	return nullptr;
}

#endif