// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/Cook/WorldPartitionCookPackageContext.h"

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageGenerator.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "Cooker/CookDependency.h"

FWorldPartitionCookPackageContext::FWorldPartitionCookPackageContext()
{
}

void FWorldPartitionCookPackageContext::RegisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* CookPackageGenerator)
{
	check(!CookPackageGenerators.Contains(CookPackageGenerator));
	CookPackageGenerators.Add(CookPackageGenerator);
	check(!PackagesToCookByGenerator.Contains(CookPackageGenerator));
}

void FWorldPartitionCookPackageContext::UnregisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* CookPackageGenerator)
{
	verify(CookPackageGenerators.Remove(CookPackageGenerator));
}

const FWorldPartitionCookPackage* FWorldPartitionCookPackageContext::AddPackageToGenerate(IWorldPartitionCookPackageGenerator* Generator, IWorldPartitionCookPackageObject* InCookPackageObject, const FString& Root, const FString& RelativePath)
{
	const FWorldPartitionCookPackage* WorldPartitionCookPackage = nullptr;

	if (InCookPackageObject->IsLevelPackage())
	{
		WorldPartitionCookPackage = AddPackageToGenerateInternal(Generator, Root, RelativePath, FWorldPartitionCookPackage::EType::Level,  InCookPackageObject->GetGenerationHash());
	}
	else
	{
		WorldPartitionCookPackage = AddPackageToGenerateInternal(Generator, Root, RelativePath, FWorldPartitionCookPackage::EType::Generic, InCookPackageObject->GetGenerationHash());
	}

	if (WorldPartitionCookPackage)
	{
		PackageObjectToPackageId.Add(Cast<UObject>(InCookPackageObject), WorldPartitionCookPackage->PackageId);
	}

	return WorldPartitionCookPackage;
}

FString FWorldPartitionCookPackageContext::GetGeneratedPackagePath(IWorldPartitionCookPackageObject* InCookPackageObject) const
{
	if (const FWorldPartitionCookPackage::IDType* PackageId = PackageObjectToPackageId.Find(Cast<UObject>(InCookPackageObject)))
	{
		if (const TUniquePtr<FWorldPartitionCookPackage>* ExistingPackage = PackagesToCookById.Find(*PackageId))
		{
			return (*ExistingPackage)->GetFullGeneratedPath();
		}
	}
	return FString();
}

const FWorldPartitionCookPackage* FWorldPartitionCookPackageContext::AddLevelStreamingPackageToGenerate(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath)
{
	FWorldPartitionPackageHash GenerationHash;
	return AddPackageToGenerateInternal(CookPackageGenerator, Root, RelativePath, FWorldPartitionCookPackage::EType::Level, GenerationHash);
}

const FWorldPartitionCookPackage* FWorldPartitionCookPackageContext::AddGenericPackageToGenerate(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath)
{
	FWorldPartitionPackageHash GenerationHash;
	return AddPackageToGenerateInternal(CookPackageGenerator, Root, RelativePath, FWorldPartitionCookPackage::EType::Generic, GenerationHash);
}

bool FWorldPartitionCookPackageContext::GetCookPackageGeneratorAndPackage(const FString& PackageRoot, const FString& PackageRelativePath, IWorldPartitionCookPackageGenerator*& CookPackageGenerator, FWorldPartitionCookPackage*& CookPackage)
{
	FWorldPartitionCookPackage::IDType PackageId = FWorldPartitionCookPackage::MakeCookPackageID(PackageRoot, PackageRelativePath);
	if (IWorldPartitionCookPackageGenerator** GeneratorPtr = CookGeneratorByPackageId.Find(PackageId))
	{
		if (TUniquePtr<FWorldPartitionCookPackage>* PackagePtr = PackagesToCookById.Find(PackageId))
		{
			check((*PackagePtr)->Root.Equals(PackageRoot, ESearchCase::IgnoreCase) && (*PackagePtr)->RelativePath.Equals(PackageRelativePath, ESearchCase::IgnoreCase));

			CookPackageGenerator = *GeneratorPtr;
			CookPackage = (*PackagePtr).Get();
			return true;
		}
	}

	return false;
}

const FWorldPartitionCookPackage* FWorldPartitionCookPackageContext::AddPackageToGenerateInternal(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath, FWorldPartitionCookPackage::EType Type, const FWorldPartitionPackageHash& GenerationHash)
{
	if (CookPackageGenerators.Contains(CookPackageGenerator))
	{
		FWorldPartitionCookPackage::IDType PackageId = FWorldPartitionCookPackage::MakeCookPackageID(FWorldPartitionCookPackage::SanitizePathComponent(Root), FWorldPartitionCookPackage::SanitizePathComponent(RelativePath));
		TUniquePtr<FWorldPartitionCookPackage>* ExistingPackage = PackagesToCookById.Find(PackageId);
		if (ExistingPackage == nullptr)
		{
			TUniquePtr<FWorldPartitionCookPackage>& CookPackage = PackagesToCookById.Emplace(PackageId, MakeUnique<FWorldPartitionCookPackage>(FWorldPartitionCookPackage::SanitizePathComponent(Root), FWorldPartitionCookPackage::SanitizePathComponent(RelativePath), Type, GenerationHash));
			check(PackageId == CookPackage->PackageId);

			CookGeneratorByPackageId.Add(PackageId, CookPackageGenerator);

			TArray<FWorldPartitionCookPackage*>& PackagesToCookForHandler = PackagesToCookByGenerator.FindOrAdd(CookPackageGenerator);
			PackagesToCookForHandler.Add(CookPackage.Get());

			UE_LOGF(LogWorldPartition, Verbose, "[Cook] Added Package %ls with ID %llu in context", *CookPackage->GetFullGeneratedPath(), PackageId);

			return CookPackage.Get();
		}
		else
		{
			UE_LOGF(LogWorldPartition, Error, "[Cook] Trying to add package %ls in context but there is already a package to generate with the same ID (%llu). Other package: %ls Id %llu",
				*FWorldPartitionCookPackage::MakeGeneratedFullPath(Root, RelativePath), PackageId, *(*ExistingPackage)->GetFullGeneratedPath(), (*ExistingPackage)->PackageId);
		}
	}
	else
	{
		UE_LOGF(LogWorldPartition, Error, "[Cook] Trying to add package %ls in context, but its generator is not registered.", *FWorldPartitionCookPackage::MakeGeneratedFullPath(Root, RelativePath));
	}

	return nullptr;
}

bool FWorldPartitionCookPackageContext::GatherPackagesToCook(const FWorldPartitionCookPackageContextParams& InParams)
{
	bool bIsSuccess = true;
	Params = InParams;

	for (IWorldPartitionCookPackageGenerator* CookPackageGenerator : CookPackageGenerators)
	{
		if (CookPackageGenerator->GatherPackagesToCook(*this))
		{
			if (const TArray<FWorldPartitionCookPackage*>* CookPackages = GetCookPackages(CookPackageGenerator))
			{
				for (FWorldPartitionCookPackage* CookPackage : *CookPackages)
				{
					if (UWorldPartitionRuntimeCell* Cell = CookPackage ? CookPackageGenerator->GetCellForPackage(*CookPackage) : nullptr)
					{
						Cell->SetLevelPackageName(*CookPackage->GetFullGeneratedPath());
					}
				}
			}
		}
		else
		{
			bIsSuccess = false;
		}
	}

	return bIsSuccess;
}

const FWorldPartitionCookPackageContextParams& FWorldPartitionCookPackageContext::GetParams() const
{
	return Params;
}

void FWorldPartitionCookPackageContext::ReportSaveDependency(UE::Cook::FCookDependency CookDependency) const
{
	if (PopulateContext)
	{
		PopulateContext->ReportSaveDependency(CookDependency);
	}
}

#endif