// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/ObjectKey.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContextInterface.h"
#include "WorldPartition/Cook/WorldPartitionCookPackage.h"
#include "CookPackageSplitter.h"

class IWorldPartitionCookPackageGenerator;

class FWorldPartitionCookPackageContext : public IWorldPartitionCookPackageContext
{
public:
	FWorldPartitionCookPackageContext();

	//~ Begin IWorldPartitionCookPackageContext Interface 
	virtual void RegisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* CookPackageGenerator) override;
	virtual void UnregisterPackageCookPackageGenerator(IWorldPartitionCookPackageGenerator* CookPackageGenerator) override;
	virtual const FWorldPartitionCookPackage* AddPackageToGenerate(IWorldPartitionCookPackageGenerator* Generator, IWorldPartitionCookPackageObject* InCookPackageObject, const FString& Root, const FString& RelativePath) override;
	virtual FString GetGeneratedPackagePath(IWorldPartitionCookPackageObject* InCookPackageObject) const override;
	virtual bool GatherPackagesToCook(const FWorldPartitionCookPackageContextParams& Params = FWorldPartitionCookPackageContextParams()) override;
	virtual const FWorldPartitionCookPackageContextParams& GetParams() const override;
	//~ End IWorldPartitionCookPackageContext Interface 

	const TArray<FWorldPartitionCookPackage*>* GetCookPackages(const IWorldPartitionCookPackageGenerator* CookPackageGenerator) const { return PackagesToCookByGenerator.Find(CookPackageGenerator); }

	bool GetCookPackageGeneratorAndPackage(const FString& PackageRoot, const FString& PackageRelativePath, IWorldPartitionCookPackageGenerator*& CookPackageGenerator, FWorldPartitionCookPackage*& CookPackage);

	uint32 NumPackageToGenerate() const { return PackagesToCookById.Num(); }
	uint32 NumGenerators() const { return CookPackageGenerators.Num(); }

	TArray<IWorldPartitionCookPackageGenerator*>& GetCookPackageGenerators() { return CookPackageGenerators; }
	const TArray<IWorldPartitionCookPackageGenerator*>& GetCookPackageGenerators() const { return CookPackageGenerators; }

	virtual void ReportSaveDependency(UE::Cook::FCookDependency CookDependency) const override;
	ICookPackageSplitter::FPopulateContext* PopulateContext = nullptr;

	inline void RegisterGeneratedPackage(const FString& GeneratedPackageName) const
	{
		CookGeneratedPackages.Add(GeneratedPackageName);
	}

	inline bool IsGeneratingPackage(const UPackage* Package) const
	{
		return CookGeneratedPackages.Contains(Package->GetName());
	}

private:
	const FWorldPartitionCookPackage* AddLevelStreamingPackageToGenerate(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath);
	const FWorldPartitionCookPackage* AddGenericPackageToGenerate(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath);
	const FWorldPartitionCookPackage* AddPackageToGenerateInternal(IWorldPartitionCookPackageGenerator* CookPackageGenerator, const FString& Root, const FString& RelativePath, FWorldPartitionCookPackage::EType Type, const FWorldPartitionPackageHash& GenerationHash);

	TArray<IWorldPartitionCookPackageGenerator*> CookPackageGenerators;
	TMap<FWorldPartitionCookPackage::IDType, TUniquePtr<FWorldPartitionCookPackage>> PackagesToCookById;
	TMap<FWorldPartitionCookPackage::IDType, IWorldPartitionCookPackageGenerator*> CookGeneratorByPackageId;
	TMap<IWorldPartitionCookPackageGenerator*, TArray<FWorldPartitionCookPackage*>> PackagesToCookByGenerator;
	TMap<FObjectKey, FWorldPartitionCookPackage::IDType> PackageObjectToPackageId;
	mutable TSet<FString> CookGeneratedPackages;
	FWorldPartitionCookPackageContextParams Params;
};

#endif