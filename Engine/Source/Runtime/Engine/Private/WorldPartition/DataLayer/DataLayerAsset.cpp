// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerAsset.h"

#include "Math/Color.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerAsset)

UDataLayerAsset::UDataLayerAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DataLayerType(EDataLayerType::Editor)
	, bSupportsActorFilters(false)
	, DebugColor(FColor::Black)
{}

bool UDataLayerAsset::IsPrivate() const
{
	return !!GetTypedOuter<UDataLayerInstance>();
}

#if WITH_EDITOR
bool UDataLayerAsset::CanBeReferencedByActor(const TSoftObjectPtr<UDataLayerAsset>& InDataLayerAsset, AActor* InActor)
{
	if (UDataLayerAsset* DataLayerAsset = InDataLayerAsset.Get())
	{
		return DataLayerAsset->CanBeReferencedByActor(InActor);
	}

	// Only accept references to unloaded UDataLayerAsset if the path is indeed an Asset Path (no sub path string)
	return InDataLayerAsset.ToSoftObjectPath().IsAsset();
}

bool UDataLayerAsset::CanBeReferencedByActor(AActor* InActor) const
{
	return !IsPrivate() || GetTypedOuter<ULevel>() == InActor->GetLevel();
}

void UDataLayerAsset::PostLoad()
{
	if (DebugColor == FColor::Black)
	{
		DebugColor = FColor::MakeRandomSeededColor(GetTypeHash(GetName()));
	}

	Super::PostLoad();
}

bool UDataLayerAsset::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerAsset, DataLayerType))
		{
			return CanEditDataLayerType();
		}
		else if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerAsset, LoadFilter))
		{
			// Only runtime data layer assets can be set to client only
			return IsRuntime();
		}
	}

	return true;
}

FText UDataLayerAsset::GetDisplayNameText() const
{
	if (IsPrivate())
	{
		if (const UDataLayerInstance* DataLayerInstance = GetTypedOuter<UDataLayerInstance>())
		{
			return FText::FromString(DataLayerInstance->GetDataLayerShortName());
		}
	}
	return Super::GetDisplayNameText();
}

void UDataLayerAsset::OnCreated()
{
	SetDebugColor(FColor::MakeRandomSeededColor(GetTypeHash(GetFullName())));
	if (!IsPrivate())
	{
		SetType(EDataLayerType::Runtime);
	}
}

bool UDataLayerAsset::CanEditDataLayerType() const
{
    // UDataLayerAsset outered to a UDataLayerInstance do not support Runtime type
	return !IsPrivate();
}

void UDataLayerAsset::SetType(EDataLayerType InDataLayerType)
{
	check(DataLayerType == EDataLayerType::Editor || !IsPrivate());
	if (CanEditDataLayerType())
	{
		DataLayerType = InDataLayerType;
	}
}
void UDataLayerAsset::SetLoadFilter(EDataLayerLoadFilter InLoadFilter)
{
	check(DataLayerType == EDataLayerType::Runtime || !IsPrivate());
	if (CanEditDataLayerType())
	{
		LoadFilter = InLoadFilter;
	}
}

#endif
