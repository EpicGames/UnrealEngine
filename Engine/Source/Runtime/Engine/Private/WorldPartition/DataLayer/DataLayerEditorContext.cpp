// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"

#if WITH_EDITOR

#include "WorldPartition/DataLayer/DataLayerManager.h"

/*
 * FDataLayerEditorContext
 */
FDataLayerEditorContext::FDataLayerEditorContext(UWorld* InWorld, const TArray<FName>& InDataLayerInstances)
	: Hash(FDataLayerEditorContext::EmptyHash)
{
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld);
	if (!DataLayerManager)
	{
		return;
	}

	for (const FName& DataLayerInstanceName : InDataLayerInstances)
	{
		if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstance(DataLayerInstanceName))
		{
			DataLayerInstances.AddUnique(DataLayerInstance->GetFName());
		}
	}

	if (DataLayerInstances.Num())
	{
		DataLayerInstances.Sort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });
		for (FName InstanceName : DataLayerInstances)
		{
			Hash = FCrc::StrCrc32(*InstanceName.ToString(), Hash);
		}
		check(Hash != FDataLayerEditorContext::EmptyHash);
	}
}

#endif
