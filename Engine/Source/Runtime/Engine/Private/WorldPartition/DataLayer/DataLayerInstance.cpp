// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerInstance.h"

#include "Internationalization/Text.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UnrealType.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerInstance)

#define LOCTEXT_NAMESPACE "DataLayer"

UDataLayerInstance::UDataLayerInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bIsVisible(true)
	, bIsInitiallyVisible(true)
	, bIsInitiallyLoadedInEditor(true)
	, bIsLoadedInEditor(true)
	, bIsLocked(false)
	, OverrideBlockOnSlowStreaming(EOverrideBlockOnSlowStreaming::NoOverride)
	, StreamingPriority(0)
#endif
	, InitialRuntimeState(EDataLayerRuntimeState::Unloaded)
{}

void UDataLayerInstance::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Initialize bIsVisible with persistent flag bIsInitiallyVisible
	bIsVisible = bIsInitiallyVisible;
#endif

	if (Parent)
	{
		Parent->AddChild(this);
	}
}

UWorld* UDataLayerInstance::GetOuterWorld() const
{
	return UObject::GetTypedOuter<UWorld>();
}

AWorldDataLayers* UDataLayerInstance::GetOuterWorldDataLayers() const
{
	check(GetOuterWorld());
	return GetOuterWorld()->GetWorldDataLayers();
}

AWorldDataLayers* UDataLayerInstance::GetDirectOuterWorldDataLayers() const
{
	// To retrieve the direct outer WorldDataLayers use UObject interface since UDataLayerInstance::GetTypedOuter<AWorldDataLayers>() won't compile.
	AWorldDataLayers* DirectOuterWorldDataLayers = this->UObject::GetTypedOuter<AWorldDataLayers>();
	return DirectOuterWorldDataLayers;
}

#if WITH_EDITOR
bool UDataLayerInstance::IsAsset() const
{
	// When using external packaging, Data Layer Instances are considered assets to allow using the asset logic for save dialogs, etc.
	// Also, they return true even if pending kill, in order to show up as deleted in these dialogs.
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_Transient | RF_ClassDefaultObject) && !GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
}

namespace DataLayerInstance
{
	static const FName NAME_DataLayerInstanceName("DataLayerInstanceName");
	static const FName NAME_DataLayerInstanceParentName("DataLayerInstanceParentName");
	static const FName NAME_DataLayerInstanceAssetPath("DataLayerInstanceAssetPath");
	static const FName NAME_DataLayerInstanceIsPrivate("DataLayerInstanceIsPrivate");
	static const FName NAME_DataLayerInstanceIsIncludedInActorFilterDefault("DataLayerInstanceIsIncludedInActorFilterDefault");
	static const FName NAME_DataLayerInstancePrivateDataLayerSupportsActorFilter("DataLayerInstancePrivateDataLayerSupportsActorFilter");
	static const FName NAME_DataLayerInstancePrivateShortName("DataLayerInstancePrivateShortName");
	static const FName NAME_DataLayerInstanceInitialRuntimeState("DataLayerInstanceInitialRuntimeState");
};

void UDataLayerInstance::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (IsPackageExternal())
	{
		// Set generic FPrimaryAssetId::PrimaryAssetDisplayNameTag
		Context.AddTag(FAssetRegistryTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag, *GetDataLayerShortName(), FAssetRegistryTag::TT_Hidden));

		// Set DataLayerInstance specific tags
		Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceName, *GetFName().ToString(), FAssetRegistryTag::TT_Hidden));
		if (GetParent())
		{
			Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceParentName, *GetParent()->GetFName().ToString(), FAssetRegistryTag::TT_Hidden));
		}
		if (const UDataLayerAsset* DataLayerAsset = GetAsset())
		{
			Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceAssetPath, *DataLayerAsset->GetPathName(), FAssetRegistryTag::TT_Hidden));
			Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceIsPrivate, DataLayerAsset->IsPrivate() ? TEXT("1") : TEXT("0"), FAssetRegistryTag::TT_Hidden));
		}
		Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceIsIncludedInActorFilterDefault, IsIncludedInActorFilterDefault() ? TEXT("1") : TEXT("0"), FAssetRegistryTag::TT_Hidden));
		Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstancePrivateDataLayerSupportsActorFilter, SupportsActorFilters() ? TEXT("1") : TEXT("0"), FAssetRegistryTag::TT_Hidden));
		Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstancePrivateShortName, *GetDataLayerShortName(), FAssetRegistryTag::TT_Hidden));
		Context.AddTag(FAssetRegistryTag(DataLayerInstance::NAME_DataLayerInstanceInitialRuntimeState, *StaticEnum<EDataLayerRuntimeState>()->GetNameStringByValue(static_cast<int64>(InitialRuntimeState)), FAssetRegistryTag::TT_Hidden));
	}
}

bool UDataLayerInstance::GetAssetRegistryInfoFromPackage(FName InDataLayerInstancePackageName, FDataLayerInstanceDesc& OutDataLayerInstanceDesc)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	OutDataLayerInstanceDesc.bIsUsingAsset = true;
	TArray<FAssetData> Assets;
	AssetRegistry.GetAssetsByPackageName(InDataLayerInstancePackageName, Assets, true);
	check(Assets.Num() <= 1);
	if (Assets.Num() == 1)
	{
		return GetAssetRegistryInfoFromPackage(Assets[0], OutDataLayerInstanceDesc);
	}
	return false;
}

bool UDataLayerInstance::GetAssetRegistryInfoFromPackage(const FAssetData& InAsset, FDataLayerInstanceDesc& OutDataLayerInstanceDesc)
{
	FString Value;
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceName, Value))
	{
		OutDataLayerInstanceDesc.Name = *Value;
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceParentName, Value))
	{
		OutDataLayerInstanceDesc.ParentName = *Value;
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceAssetPath, Value))
	{
		FName AssetPath = *Value;
		UAssetRegistryHelpers::FixupRedirectedAssetPath(AssetPath);
		OutDataLayerInstanceDesc.AssetPath = AssetPath;
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceIsPrivate, Value))
	{
		OutDataLayerInstanceDesc.bIsPrivate = (Value == TEXT("1"));
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceIsIncludedInActorFilterDefault, Value))
	{
		OutDataLayerInstanceDesc.bIsIncludedInActorFilterDefault = (Value == TEXT("1"));
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstancePrivateDataLayerSupportsActorFilter, Value))
	{
		OutDataLayerInstanceDesc.bPrivateDataLayerSupportsActorFilter = (Value == TEXT("1"));
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstancePrivateShortName, Value))
	{
		OutDataLayerInstanceDesc.PrivateShortName = Value;
	}
	if (InAsset.GetTagValue(DataLayerInstance::NAME_DataLayerInstanceInitialRuntimeState, Value))
	{
		OutDataLayerInstanceDesc.InitialRuntimeState = static_cast<EDataLayerRuntimeState>(StaticEnum<EDataLayerRuntimeState>()->GetValueByNameString(Value));
	}
	return !OutDataLayerInstanceDesc.Name.IsNone();
}

void UDataLayerInstance::PreEditUndo()
{
	Super::PreEditUndo();
	bUndoIsLoadedInEditor = bIsLoadedInEditor;
}

void UDataLayerInstance::PostEditUndo()
{
	Super::PostEditUndo();
	if (bIsLoadedInEditor != bUndoIsLoadedInEditor)
	{
		IWorldPartitionActorLoaderInterface::RefreshLoadedState(true);
	}
}

void UDataLayerInstance::SetVisible(bool bInIsVisible)
{
	if (bIsVisible != bInIsVisible)
	{
		Modify(/*bAlwaysMarkDirty*/false);
		bIsVisible = bInIsVisible;
	}
}

void UDataLayerInstance::SetIsInitiallyVisible(bool bInIsInitiallyVisible)
{
	if (bIsInitiallyVisible != bInIsInitiallyVisible)
	{
		Modify();
		bIsInitiallyVisible = bInIsInitiallyVisible;
	}
}

void UDataLayerInstance::SetIsLoadedInEditor(bool bInIsLoadedInEditor, bool bInFromUserChange)
{
	if (bIsLoadedInEditor != bInIsLoadedInEditor)
	{
		Modify(false);
		bIsLoadedInEditor = bInIsLoadedInEditor;
		bIsLoadedInEditorChangedByUserOperation |= bInFromUserChange;
	}
}

bool UDataLayerInstance::IsEffectiveLoadedInEditor() const
{
	bool bResult = IsLoadedInEditor();

	const UDataLayerInstance* ParentDataLayer = GetParent();
	while (ParentDataLayer && bResult)
	{
		bResult = bResult && ParentDataLayer->IsLoadedInEditor();
		ParentDataLayer = ParentDataLayer->GetParent();
	}
	return bResult;
}

bool UDataLayerInstance::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (IsReadOnly())
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerInstance, InitialRuntimeState))
	{
		if (!IsRuntime())
		{
			return false;
		}
	}

	return true;
}

bool UDataLayerInstance::IsLocked(FText* OutReason) const
{
	if (bIsLocked)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("DataLayerInstanceLocked", "Data layer instance is locked.");
		}
		return true;
	}

	if (IsRuntime() && !GetOuterWorldDataLayers()->GetAllowRuntimeDataLayerEditing())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("DataLayerRuntimeDataLayerEditingNotAllowed", "Runtime data layer editing is not allowed.");
		}
		return true;
	}

	return false;
}

bool UDataLayerInstance::IsReadOnly(FText* OutReason) const
{
	const UWorld* World = GetWorld();
	if (!World || World->IsGameWorld())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("DataLayerInstanceReadOnlyInGameWorld", "Data layer instance is read-only in game world.");
		}
		return true;
	}

	// Check if root External Data Layer is read-only
	const UExternalDataLayerInstance* ExternalDataLayerInstance = GetRootExternalDataLayerInstance();
	if (ExternalDataLayerInstance && (ExternalDataLayerInstance != this))
	{
		if (ExternalDataLayerInstance->IsReadOnly(OutReason))
		{
			return true;
		}
	}

	return IsLocked(OutReason);
}

const TCHAR* UDataLayerInstance::GetDataLayerIconName() const
{
	return FDataLayerUtils::GetDataLayerIconName(GetType());
}

bool UDataLayerInstance::CanUserAddActors(FText* OutReason) const
{
	return !IsReadOnly(OutReason);
}

bool UDataLayerInstance::CanUserRemoveActors(FText* OutReason) const
{
	return !IsReadOnly(OutReason);
}

bool UDataLayerInstance::CanAddActor(AActor* InActor, FText* OutReason) const
{
	if (!InActor)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddActorInvalidActor", "Invalid actor.");
		}
		return false;
	}

	if (IsReadOnly(OutReason))
	{
		return false;
	}

	if (!InActor->CanAddDataLayer(this, OutReason))
	{
		return false;
	}

	const UExternalDataLayerInstance* ExternalDataLayerInstance = GetRootExternalDataLayerInstance();
	if (ExternalDataLayerInstance && (ExternalDataLayerInstance != this))
	{
		if (InActor->GetExternalDataLayerAsset() != ExternalDataLayerInstance->GetAsset())
		{
			if (OutReason)
			{
				*OutReason = LOCTEXT("CantAddActorMismatchRootExternalDataLayer", "Can't assign actors to a Data Layer child of an External Data Layer that is not already assigned to the actors");
			}
			return false;
		}
	}
	return true;
}

bool UDataLayerInstance::AddActor(AActor* InActor) const
{
	if (CanAddActor(InActor))
	{
		return PerformAddActor(InActor);
	}

	return false;
}

bool UDataLayerInstance::CanRemoveActor(AActor* InActor, FText* OutReason) const
{
	if (!InActor)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantRemoveActorInvalidActor", "Invalid actor.");
		}
		return false;
	}

	if (!InActor->GetDataLayerInstances().Contains(this) && !InActor->GetDataLayerInstancesForLevel().Contains(this))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantRemoveActorNotAssignedToDataLayer", "Actor is not assigned to data layer.");
		}
		return false;
	}

	return true;
}

bool UDataLayerInstance::RemoveActor(AActor* InActor) const
{
	if (CanRemoveActor(InActor))
	{
		return PerformRemoveActor(InActor);
	}

	return false;
}

bool UDataLayerInstance::CanBeChildOf(const UDataLayerInstance* InParent, FText* OutReason) const
{
	if (this == InParent)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("ParentIsThis", "Can't parent to itself");
		}
		return false;
	}

	if (Parent == InParent)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("SameParent", "Data Layer already has this parent");
		}
		return false;
	}

	// Check if root External Data Layer is the same for the parent
	if (const UExternalDataLayerInstance* ExternalDataLayerInstance = GetRootExternalDataLayerInstance())
	{
		if (!InParent || (InParent->GetRootExternalDataLayerInstance() != ExternalDataLayerInstance))
		{
			if (OutReason)
			{
				*OutReason = LOCTEXT("ParentHasDifferentRootExternalDataLayer", "Can't reparent Data Layer in a different root External Data Layer");
			}
			return false;
		}
	}

	if (InParent == nullptr)
	{
		// nullptr is considered a valid parent
		return true;
	}

	if (!CanHaveParentDataLayerInstance())
	{
		if (OutReason)
		{
			*OutReason = FText::Format(LOCTEXT("ParentDataLayerUnsupported", "Data Layer \"{0}\" does not support parent data layers"), FText::FromString(GetDataLayerShortName()));
		}
		return false;
	}

	if (!InParent->CanHaveChildDataLayerInstance(this))
	{
		if (OutReason)
		{
			*OutReason = FText::Format(LOCTEXT("ChildDataLayerUnsuported", "Data Layer \"{0}\" does not support child data layers"), FText::FromString(InParent->GetDataLayerShortName()));
		}
		return false;
	}

	if (!IsParentDataLayerTypeCompatible(InParent, OutReason))
	{
		return false;
	}

	if (InParent->GetDirectOuterWorldDataLayers() != GetDirectOuterWorldDataLayers())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("DifferentOuterWorldDataLayer", "Parent WorldDataLayers is a different from child WorldDataLayers");
		}
		return false;
	}

	return true;
}

bool UDataLayerInstance::IsParentDataLayerTypeCompatible(const UDataLayerInstance* InParent, FText* OutReasonText, IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason* OutReason) const
{
	if (InParent == nullptr)
	{
		return false;
	}

	if (IsClientOnly() || IsServerOnly())
	{
		if (OutReason)
		{
			*OutReason = IsClientOnly() ? IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild : IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason::ServerOnlyDataLayerCantBeChild;
		}
		if (OutReasonText)
		{
			*OutReasonText = FText::Format(LOCTEXT("ClientOrServerOnlyCantHaveParent", "{0} Data Layer cannot be a child Data Layer"), IsClientOnly() ? FText::FromString(TEXT("Client-Only")) : FText::FromString(TEXT("Server-Only")));
		}
		return false;
	}

	if (!FDataLayerUtils::AreDataLayerTypesCompatible(InParent->GetType(), GetType(), InParent->IsA<UExternalDataLayerInstance>(), OutReasonText))
	{
		if (OutReason)
		{
			*OutReason = IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason::IncompatibleDataLayerType;
		}
		return false;
	}
	return true;
}

void UDataLayerInstance::SetInitialRuntimeState(EDataLayerRuntimeState InInitialRuntimeState)
{
	if (IsRuntime())
	{
		InitialRuntimeState = InInitialRuntimeState;
	}
}

bool UDataLayerInstance::SetParent(UDataLayerInstance* InParent)
{
	if (!CanBeChildOf(InParent))
	{
		return false;
	}

	check(CanHaveParentDataLayerInstance());

	Modify();

	// If we find ourself in the parent chain of the provided parent
	UDataLayerInstance* CurrentInstance = InParent;
	while (CurrentInstance)
	{
		if (CurrentInstance == this)
		{
			// Detach the parent from its parent
			InParent->SetParent(nullptr);
			break;
		}
		CurrentInstance = CurrentInstance->GetParent();
	};

	if (Parent)
	{
		Parent->RemoveChild(this);
	}
	Parent = InParent;
	if (Parent)
	{
		Parent->AddChild(this);
	}

	return true;
}

void UDataLayerInstance::OnRemovedFromWorldDataLayers()
{
	while (Children.Num())
	{
		// If can't reparent, move to root
		UDataLayerInstance* ChildNewParent = Children[0]->CanBeChildOf(Parent) ? Parent.Get() : nullptr;
		verify(Children[0]->SetParent(ChildNewParent));
	};

	if (Parent)
	{
		Parent->RemoveChild(this);
	}
}

void UDataLayerInstance::RemoveChild(UDataLayerInstance* InDataLayer)
{
	Modify(false);
	check(Children.Contains(InDataLayer));
	Children.RemoveSingle(InDataLayer);
}

FText UDataLayerInstance::GetDataLayerText(const UDataLayerInstance* InDataLayer)
{
	return InDataLayer ? FText::FromString(InDataLayer->GetDataLayerShortName()) : LOCTEXT("InvalidDataLayerShortName", "<None>");
}

bool UDataLayerInstance::Validate(IStreamingGenerationErrorHandler* ErrorHandler) const
{
	// No need to run the validation for game worlds
	UWorld* World = GetOuterWorld();
	if (World->IsGameWorld())
	{
		return true;
	}

	if (GetParent() != nullptr)
	{
		FText ReasonText;
		IStreamingGenerationErrorHandler::EDataLayerHierarchyInvalidReason Reason;
		if (!IsParentDataLayerTypeCompatible(GetParent(), &ReasonText, &Reason))
		{
			ErrorHandler->OnDataLayerHierarchyTypeMismatch(this, GetParent(), Reason);
			return false;
		}
	}

	return true;
}

bool UDataLayerInstance::CanBeInActorEditorContext() const
{
	return !IsReadOnly() && GetOuterWorldDataLayers()->ContainsDataLayer(this);
}

bool UDataLayerInstance::IsActorEditorContextCurrentColorized() const
{
	return GetOuterWorldDataLayers()->IsActorEditorContextCurrentColorized(this);
}

bool UDataLayerInstance::IsInActorEditorContext() const
{
	return GetOuterWorldDataLayers()->IsInActorEditorContext(this);
}

bool UDataLayerInstance::AddToActorEditorContext()
{
	return GetOuterWorldDataLayers()->AddToActorEditorContext(this);
}

bool UDataLayerInstance::RemoveFromActorEditorContext()
{
	return GetOuterWorldDataLayers()->RemoveFromActorEditorContext(this);
}

#endif

bool UDataLayerInstance::IsInitiallyVisible() const
{
#if WITH_EDITOR
	return bIsInitiallyVisible;
#else
	return false;
#endif
}

bool UDataLayerInstance::IsVisible() const
{
#if WITH_EDITOR
	return bIsVisible;
#else
	return false;
#endif
}

bool UDataLayerInstance::IsEffectiveVisible() const
{
#if WITH_EDITOR
	bool bResult = IsVisible();
	const UDataLayerInstance* ParentDataLayer = GetParent();
	while (ParentDataLayer && bResult)
	{
		bResult = bResult && ParentDataLayer->IsVisible();
		ParentDataLayer = ParentDataLayer->GetParent();
	}
	return bResult && IsEffectiveLoadedInEditor();
#else
	return false;
#endif
}

void UDataLayerInstance::ForEachChild(TFunctionRef<bool(const UDataLayerInstance*)> Operation) const
{
	for (const UDataLayerInstance* Child : Children)
	{
		if (!Operation(Child))
		{
			break;
		}
	}
}

void UDataLayerInstance::AddChild(UDataLayerInstance* InDataLayer)
{
	check(InDataLayer->GetDirectOuterWorldDataLayers() == GetDirectOuterWorldDataLayers())
	check(CanHaveChildDataLayerInstance(InDataLayer));
	Modify(false);
	checkSlow(!Children.Contains(InDataLayer));
	Children.Add(InDataLayer);
}

EDataLayerRuntimeState UDataLayerInstance::GetRuntimeState() const
{
	return GetOuterWorldDataLayers()->GetDataLayerRuntimeStateByName(GetFName());
}

EDataLayerRuntimeState UDataLayerInstance::GetEffectiveRuntimeState() const
{
	return GetOuterWorldDataLayers()->GetDataLayerEffectiveRuntimeStateByName(GetFName());
}

#undef LOCTEXT_NAMESPACE
