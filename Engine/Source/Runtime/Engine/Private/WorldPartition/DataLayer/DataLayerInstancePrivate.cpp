// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerInstancePrivate.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "Engine/Level.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerInstancePrivate)

#define LOCTEXT_NAMESPACE "UDataLayerInstancePrivate"

UDataLayerInstancePrivate::UDataLayerInstancePrivate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bIsIncludedInActorFilterDefault(true)
#endif
{
	DataLayerAsset = CreateDefaultSubobject<UDataLayerAsset>(TEXT("DataLayerAsset"));
#if WITH_EDITOR
	DataLayerAsset->SetType(EDataLayerType::Editor);
#endif
}

#if WITH_EDITOR
FName UDataLayerInstancePrivate::MakeName()
{
	return FName(FString::Format(TEXT("DataLayer_{0}"), { FGuid::NewGuid().ToString() }));
}

void UDataLayerInstancePrivate::OnCreated()
{
	Modify(/*bAlwaysMarkDirty*/false);
	FDataLayerUtils::SetDataLayerShortName(this, TEXT("DataLayer"));
		
	SetVisible(true);
}

bool UDataLayerInstancePrivate::IsReadOnly(FText* OutReason) const
{
	if (Super::IsReadOnly(OutReason))
	{
		return true;
	}
	return GetDirectOuterWorldDataLayers()->IsReadOnly(OutReason);
}

bool UDataLayerInstancePrivate::PerformAddActor(AActor* InActor) const
{
	check(GetDirectOuterWorldDataLayers() == InActor->GetLevel()->GetWorldDataLayers()); // Make sure the instance is part of the same WorldDataLayers as the actor's level WorldDataLayer.
	check(UDataLayerManager::GetDataLayerManager(InActor)->GetDataLayerInstance(DataLayerAsset) != nullptr); // Make sure the DataLayerInstance exists for this level
	return FAssignActorDataLayer::AddDataLayerAsset(InActor, DataLayerAsset);
}

bool UDataLayerInstancePrivate::CanAddActor(AActor* InActor, FText* OutReason) const
{
	if (!Super::CanAddActor(InActor, OutReason))
	{
		return false;
	}

	// Make sure the instance is part of the same WorldDataLayers as the actor's level WorldDataLayer
	if (GetDirectOuterWorldDataLayers() != InActor->GetLevel()->GetWorldDataLayers())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddDataLayerPrivateDataLayerNotPartActorLevelWorldDataLayers", "Actor is not in the same level as the private data layer.");
		}
		return false;
	}
	// Make sure the DataLayerInstance exists for this level
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InActor);
	if (!DataLayerManager || !DataLayerManager->GetDataLayerInstance(DataLayerAsset))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddDataLayerPrivateDataLayerInstanceNotFoundInActorLevel", "Private data layer instance doesn't exist in the actor level.");
		}
		return false;
	}
	return true;
}

bool UDataLayerInstancePrivate::PerformRemoveActor(AActor* InActor) const
{
	return FAssignActorDataLayer::RemoveDataLayerAsset(InActor, DataLayerAsset);
}

bool UDataLayerInstancePrivate::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty && (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerInstancePrivate, bIsIncludedInActorFilterDefault)))
	{
		return SupportsActorFilters();
	}

	return true;
}

#endif

#undef LOCTEXT_NAMESPACE