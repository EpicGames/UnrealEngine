// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerInstanceProviderInterface.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerInstanceProviderInterface)

UDataLayerInstanceProvider::UDataLayerInstanceProvider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

const UExternalDataLayerAsset* IDataLayerInstanceProvider::GetRootExternalDataLayerAsset() const
{
	const UExternalDataLayerInstance* ExternalDataLayerInstance = GetRootExternalDataLayerInstance();
	return ExternalDataLayerInstance ? ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
}