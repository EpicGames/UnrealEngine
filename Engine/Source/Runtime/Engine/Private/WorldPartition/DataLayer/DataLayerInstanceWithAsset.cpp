// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "Engine/Level.h"
#include "Misc/StringFormatArg.h"
#include "UObject/UnrealType.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerInstanceWithAsset)

#define LOCTEXT_NAMESPACE "DataLayerInstanceWithAsset"

UDataLayerInstanceWithAsset::UDataLayerInstanceWithAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bIsIncludedInActorFilterDefault(true)
#endif
{
#if WITH_EDITOR
	bSkipCheckReadOnlyForSubLevels = false;
#endif
}

UWorld* UDataLayerInstanceWithAsset::GetOuterWorld() const
{
	if (URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = GetTypedOuter<URuntimeHashExternalStreamingObjectBase>())
	{
		return ExternalStreamingObject->GetOuterWorld();
	}
	check(GetDirectOuterWorldDataLayers()->GetTypedOuter<UWorld>() == Super::GetOuterWorld());
	return Super::GetOuterWorld();
}


#if WITH_EDITOR
const UExternalDataLayerInstance* UDataLayerInstanceWithAsset::GetRootExternalDataLayerInstance() const
{
	IDataLayerInstanceProvider* DataLayerInstanceProvider = GetImplementingOuter<IDataLayerInstanceProvider>();
	if (ensure(DataLayerInstanceProvider))
	{
		return DataLayerInstanceProvider->GetRootExternalDataLayerInstance();
	}
	return nullptr;
}

FName UDataLayerInstanceWithAsset::MakeName(const UDataLayerAsset* InDataLayerAsset)
{
	return FName(FString::Format(TEXT("DataLayer_{0}"), { FGuid::NewGuid().ToString() }));
}

TSubclassOf<UDataLayerInstanceWithAsset> UDataLayerInstanceWithAsset::GetDataLayerInstanceClass()
{
	return UDataLayerManager::GetDataLayerInstanceWithAssetClass();
}

void UDataLayerInstanceWithAsset::OnCreated(const UDataLayerAsset* Asset)
{
	checkf(Asset->IsA<UExternalDataLayerAsset>() == this->IsA<UExternalDataLayerInstance>(), TEXT("Only ExternalDataLayerInstance can reference ExternalDataLayerAsset"));

	Modify(/*bAlwaysMarkDirty*/false);

	check(DataLayerAsset == nullptr);
	DataLayerAsset = Asset;

	SetVisible(true);
}

bool UDataLayerInstanceWithAsset::IsReadOnly(FText* OutReason) const
{
	if (Super::IsReadOnly(OutReason))
	{
		return true;
	}
	return !bSkipCheckReadOnlyForSubLevels && GetDirectOuterWorldDataLayers()->IsReadOnly(OutReason);
}

bool UDataLayerInstanceWithAsset::CanBeInActorEditorContext() const
{
	if (!Super::CanBeInActorEditorContext())
	{
		return false;
	}

	return DataLayerAsset != nullptr;
}

bool UDataLayerInstanceWithAsset::CanAddActor(AActor* InActor, FText* OutReason) const
{
	if (!Super::CanAddActor(InActor, OutReason))
	{
		return false;
	}

	if (!DataLayerAsset)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddActorInvalidDataLayerAsset", "Invalid data layer asset.");
		}
		return false;
	}

	if (!DataLayerAsset->CanBeReferencedByActor(InActor))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddActorDataLayerAssetCantBeReferencedByActor", "Data layer asset can't be referenced by actor.");
		}
		return false;
	}

	return true;
}

bool UDataLayerInstanceWithAsset::PerformAddActor(AActor* InActor) const
{	
	check(GetOuterWorldDataLayers() == InActor->GetLevel()->GetWorldDataLayers()); // Make sure the instance is part of the same WorldDataLayers as the actor's level WorldDataLayer.
	check(UDataLayerManager::GetDataLayerManager(InActor)->GetDataLayerInstance(DataLayerAsset) != nullptr); // Make sure the DataLayerInstance exists for this level
	return FAssignActorDataLayer::AddDataLayerAsset(InActor, DataLayerAsset);
}

bool UDataLayerInstanceWithAsset::CanRemoveActor(AActor* InActor, FText* OutReason) const
{
	if (!Super::CanRemoveActor(InActor, OutReason))
	{
		return false;
	}

	if (!DataLayerAsset)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantRemoveActorInvalidDataLayerAsset", "Invalid data layer asset.");
		}
		return false;
	}

	return true;
}

bool UDataLayerInstanceWithAsset::PerformRemoveActor(AActor* InActor) const
{
	return FAssignActorDataLayer::RemoveDataLayerAsset(InActor, DataLayerAsset);
}

bool UDataLayerInstanceWithAsset::Validate(IStreamingGenerationErrorHandler* ErrorHandler) const
{
	bool bIsValid = true;

	if (GetAsset() == nullptr)
	{
		ErrorHandler->OnInvalidReferenceDataLayerAsset(this);
		return false;
	}

	if (GetAsset()->IsA<UExternalDataLayerAsset>() != this->IsA<UExternalDataLayerInstance>())
	{
		ErrorHandler->OnInvalidDataLayerAssetType(this, GetAsset());
		return false;
	}

	FText ReasonText;
	const AWorldDataLayers* WorldDataLayers = GetDirectOuterWorldDataLayers();
	if (WorldDataLayers && !WorldDataLayers->CanReferenceDataLayerAsset(GetAsset(), &ReasonText))
	{
		ErrorHandler->OnInvalidWorldDataLayersReference(WorldDataLayers, this, ReasonText);
		return false;
	}

	// Get the DataLayerManager for this DataLayerInstance which will be the one of its outer world
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(this);
	if (ensure(DataLayerManager))
	{
		DataLayerManager->ForEachDataLayerInstance([&bIsValid, this, ErrorHandler](UDataLayerInstance* DataLayerInstance)
		{
			if (DataLayerInstance != this)
			{
				if (UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
				{
					if (DataLayerInstanceWithAsset->GetAsset() == GetAsset())
					{
						ErrorHandler->OnDataLayerAssetConflict(this, DataLayerInstanceWithAsset);
						bIsValid = false;
						return false;
					}
				}
			}
	
			return true;
		});
	}

	bIsValid &= Super::Validate(ErrorHandler);

	return bIsValid;
}

void UDataLayerInstanceWithAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataLayerInstanceWithAsset, DataLayerAsset))
	{
		if (AWorldDataLayers* WorldDataLayers = GetOuterWorldDataLayers())
		{
			WorldDataLayers->ResolveActorDescContainers();
		}
	}
}

bool UDataLayerInstanceWithAsset::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty && (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerInstanceWithAsset, bIsIncludedInActorFilterDefault)))
	{
		return SupportsActorFilters();
	}

	if (InProperty && (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDataLayerInstanceWithAsset, DataLayerAsset)))
	{
		return CanEditDataLayerAsset();
	}

	return true;
}

bool UDataLayerInstanceWithAsset::SupportsActorFilters() const
{
	return DataLayerAsset && DataLayerAsset->SupportsActorFilters();
}

bool UDataLayerInstanceWithAsset::IsIncludedInActorFilterDefault() const
{
	return bIsIncludedInActorFilterDefault;
}

void UDataLayerInstanceWithAsset::PreEditUndo()
{
	Super::PreEditUndo();
	CachedDataLayerAsset = DataLayerAsset;
}

void UDataLayerInstanceWithAsset::PostEditUndo()
{
	Super::PostEditUndo();
	if (CachedDataLayerAsset != DataLayerAsset)
	{
		GetOuterWorldDataLayers()->ResolveActorDescContainers();
	}
	CachedDataLayerAsset = nullptr;
}
#endif

#undef LOCTEXT_NAMESPACE 