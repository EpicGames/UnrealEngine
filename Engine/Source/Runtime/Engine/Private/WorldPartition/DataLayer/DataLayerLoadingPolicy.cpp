// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerLoadingPolicy.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerLoadingPolicy)

#if WITH_EDITOR
EWorldPartitionDataLayersLogicOperator UDataLayerLoadingPolicy::GetDataLayersLogicOperator() const
{
	return GetOuterUDataLayerManager()->GetWorld()->GetWorldPartition()->GetDataLayersLogicOperator();
}

bool UDataLayerLoadingPolicy::ResolveIsLoadedInEditor(TArray<const UDataLayerInstance*>& InDataLayerInstances) const
{
	check(!InDataLayerInstances.IsEmpty());
	switch (GetDataLayersLogicOperator())
	{
		case EWorldPartitionDataLayersLogicOperator::Or:
			return Algo::AnyOf(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsEffectiveLoadedInEditor(); });
		case EWorldPartitionDataLayersLogicOperator::And:
			return Algo::AllOf(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsEffectiveLoadedInEditor(); });
	}
	return false;
}
#endif
