// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/DataLayerSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerLoadingPolicy.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "Engine/Canvas.h"
#include "Debug/DebugDrawService.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerManager)

#if WITH_EDITOR
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/IDataLayerEditorModule.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/ActorDescList.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/LevelStreaming.h"
#include "EngineUtils.h"
#include "Algo/Transform.h"
#include "Algo/AnyOf.h"
#include "LevelUtils.h"
#include "Editor.h"
#else
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#endif

extern int32 GDrawDataLayersLoadTime;

FAutoConsoleCommandWithOutputDevice UDataLayerManager::DumpDataLayersCommand(
	TEXT("wp.Runtime.DumpDataLayers"),
	TEXT("Dumps data layers to the log"),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& OutputDevice)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{
					WorldPartitionSubsystem->ForEachWorldPartition([&OutputDevice](UWorldPartition* WorldPartition)
					{
						if (UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
						{
							DataLayerManager->DumpDataLayers(OutputDevice);
						}
						return true;
					});
				}
			}
		}
	})
);

static const FString GToggleDataLayerActivationCommandName("wp.Runtime.ToggleDataLayerActivation");
FAutoConsoleCommand UDataLayerManager::ToggleDataLayerActivation(
	*GToggleDataLayerActivationCommandName,
	TEXT("Toggles DataLayers active state. Args [DataLayerNames]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				FWorldPartitionHelpers::ServerExecConsoleCommand(World, GToggleDataLayerActivationCommandName, InArgs);
				if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{
					WorldPartitionSubsystem->ForEachWorldPartition([&InArgs](UWorldPartition* WorldPartition)
					{
						if (UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
						{
							TArray<UDataLayerInstance*> DataLayerInstances = DataLayerManager->ConvertArgsToDataLayers(InArgs);
							for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
							{
								DataLayerManager->SetDataLayerInstanceRuntimeState(DataLayerInstance, DataLayerManager->GetDataLayerInstanceRuntimeState(DataLayerInstance) == EDataLayerRuntimeState::Activated ? EDataLayerRuntimeState::Unloaded : EDataLayerRuntimeState::Activated);
							}
						}
						return true;
					});
				}
			}
		}
	})
);

inline bool GetDataLayerRuntimeStateFromName(const FString& InStateName, EDataLayerRuntimeState& OutState)
{
	if (InStateName.Equals(GetDataLayerRuntimeStateName(EDataLayerRuntimeState::Unloaded), ESearchCase::IgnoreCase))
	{
		OutState = EDataLayerRuntimeState::Unloaded;
		return true;
	}
	else if (InStateName.Equals(GetDataLayerRuntimeStateName(EDataLayerRuntimeState::Loaded), ESearchCase::IgnoreCase))
	{
		OutState = EDataLayerRuntimeState::Loaded;
		return true;
	}
	else if (InStateName.Equals(GetDataLayerRuntimeStateName(EDataLayerRuntimeState::Activated), ESearchCase::IgnoreCase))
	{
		OutState = EDataLayerRuntimeState::Activated;
		return true;
	}
	return false;
}

static const FString GSetDataLayerRuntimeStateCommandCommandName("wp.Runtime.SetDataLayerRuntimeState");
FAutoConsoleCommand UDataLayerManager::SetDataLayerRuntimeStateCommand(
	*GSetDataLayerRuntimeStateCommandCommandName,
	TEXT("Sets Runtime DataLayers state. Args [State = Unloaded, Loaded, Activated] [DataLayerNames]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		if (InArgs.Num() < 2)
		{
			UE_LOGF(LogWorldPartition, Warning, "%ls : Requires at least 2 arguments. First argument should be the target state and the next ones should be the list of DataLayers.", *GSetDataLayerRuntimeStateCommandCommandName);
			return;
		}

		TArray<FString> Args(InArgs);
		FString StatetStr;
		Args.HeapPop(StatetStr);
		EDataLayerRuntimeState State;
		if (!GetDataLayerRuntimeStateFromName(StatetStr, State))
		{
			UE_LOGF(LogWorldPartition, Warning, "%ls : Invalid first argument, expected one of these values : Unloaded, Loaded, Activated.", *GSetDataLayerRuntimeStateCommandCommandName);
			return;
		}

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				FWorldPartitionHelpers::ServerExecConsoleCommand(World, GSetDataLayerRuntimeStateCommandCommandName, InArgs);

				if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{
					WorldPartitionSubsystem->ForEachWorldPartition([&Args, State](UWorldPartition* WorldPartition)
					{
						if (UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
						{
							TArray<UDataLayerInstance*> DataLayerInstances = DataLayerManager->ConvertArgsToDataLayers(Args);
							for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
							{
								DataLayerManager->SetDataLayerInstanceRuntimeState(DataLayerInstance, State);
							}
						}
						return true;
					});
				}
			}
		}
	})
);

UDataLayerManager::UDataLayerManager()
{
#if WITH_EDITOR
	DataLayerActorEditorContextID = 0;
	bCanResolveDataLayers = false;
#endif
}

void UDataLayerManager::Initialize()
{
#if WITH_EDITOR
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	UWorldPartition* OuterWorldPartition = GetOuterUWorldPartition();
	UActorDescContainerInstance* ActorDescContainerInstance = OuterWorldPartition->GetActorDescContainerInstance();
	
	// Partitioned Level Instance's DataLayerManager will never resolve DataLayers (it's up to its owning WorldPartition DataLayerManager to do the job.
	ULevelInstanceSubsystem* LevelInstanceSubsystem = !GetWorld()->IsGameWorld() ? UWorld::GetSubsystem<ULevelInstanceSubsystem>(GetWorld()) : nullptr;
	ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel) : nullptr;
	bCanResolveDataLayers = (LevelInstance == nullptr);

	// In PIE, main world partition doesn't have an ActorDescContainer and doesn't need it, but instanced world partitions do have one.
	check(!ActorDescContainerInstance || ActorDescContainerInstance->IsInitialized());

	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	if (!WorldDataLayers)
	{
		if (ActorDescContainerInstance)
		{
			// Try to find and load AWorldDataLayers actor
			WorldDataLayersActor = UDataLayerManager::LoadWorldDataLayersActor(ActorDescContainerInstance);
		}

		WorldDataLayers = GetWorldDataLayers();
		if (!WorldDataLayers)
		{
			// Create missing AWorldDataLayers actor
			AWorldDataLayers* NewWorldDataLayers = AWorldDataLayers::Create(OuterWorld);
			OuterWorld->SetWorldDataLayers(NewWorldDataLayers);
			WorldDataLayers = GetWorldDataLayers();
			check(WorldDataLayers);
		}
	}

	// Initialize WorldDataLayers
	WorldDataLayers->OnDataLayerManagerInitialized();

	// Some levels do not have the WorldDataLayer actor serialized as part of their Actors array. Make sure we add it here if it isn't.
	// Make sure WorldDataLayers is part of the Actors list so that it gets cooked properly as part of the Persistent Level
	// This auto-corrects itself when resaving the level.
	ULevel* WorldDataLayerLevel = WorldDataLayers->GetLevel();
	WorldDataLayerLevel->TryAddActorToList(WorldDataLayers, /*bAddUnique*/true);

	DataLayerLoadingPolicy = NewObject<UDataLayerLoadingPolicy>(this, GetDataLayerLoadingPolicyClass());

	if (GEditor)
	{
		FModuleManager::LoadModuleChecked<IDataLayerEditorModule>("DataLayerEditor");
	}

	if (CanResolveDataLayers())
	{
		UActorDescContainerInstance::OnActorDescContainerInstanceInitialized.AddUObject(this, &UDataLayerManager::OnActorDescContainerInstanceInitialized);

		// Manually call OnActorDescContainerInstanceInitialized on already initialized outer world partition container instances
		OuterWorldPartition->ForEachActorDescContainerInstance([this](UActorDescContainerInstance* ActorDescContainerInstance)
		{
			OnActorDescContainerInstanceInitialized(ActorDescContainerInstance);
		}, true);
	}

	// SaveAs of a partition world will duplicate actors which will trigger AActor::FixupDataLayer and DataLayerManager is not yet created.
	// Here we make sure to revisit loaded actors of the outer world of this DataLayerManager (editor only)
	for (TActorIterator<AActor> It(OuterWorld); It; ++It)
	{
		check(IsValid(*It));
		It->FixupDataLayers();
	}
#else
	if (AWorldDataLayers* WorldDataLayers = GetWorldDataLayers())
	{
		WorldDataLayers->OnDataLayerManagerInitialized();
	}
#endif
}

void UDataLayerManager::DeInitialize()
{
	if (AWorldDataLayers* WorldDataLayers = GetWorldDataLayers())
	{
		WorldDataLayers->OnDataLayerManagerDeinitialized();
	}

#if WITH_EDITOR
	UActorDescContainerInstance::OnActorDescContainerInstanceInitialized.RemoveAll(this);

	WorldDataLayersActor = FWorldPartitionReference();
#endif
}

AWorldDataLayers* UDataLayerManager::GetWorldDataLayers() const
{
	return GetTypedOuter<UWorld>()->GetWorldDataLayers();
}

TArray<UDataLayerInstance*> UDataLayerManager::GetDataLayerInstances() const
{
	TArray<UDataLayerInstance*> Result;
	ForEachDataLayerInstance([&Result](UDataLayerInstance* DataLayerInstance)
	{
		Result.Add(DataLayerInstance);
		return true;
	});
	return Result;
}

const UDataLayerInstance* UDataLayerManager::GetDataLayerInstanceFromAsset(const UDataLayerAsset* InDataLayerAsset) const
{
	return GetDataLayerInstance(InDataLayerAsset);
}

const UDataLayerInstance* UDataLayerManager::GetDataLayerInstanceFromName(const FName& InDataLayerInstanceName) const
{
	return GetDataLayerInstance(InDataLayerInstanceName);
}

const UDataLayerInstance* UDataLayerManager::GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const
{
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetDataLayerInstanceFromAssetName(InDataLayerAssetPathName) : nullptr;
}

bool UDataLayerManager::SetDataLayerInstanceRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (InDataLayerInstance)
	{
		return InDataLayerInstance->GetOuterWorldDataLayers()->SetDataLayerRuntimeState(InDataLayerInstance, InState, bInIsRecursive);
	}
	UE_LOGF(LogWorldPartition, Error, "Invalid Data Layer Instance.");
	return false;
}

bool UDataLayerManager::SetDataLayerRuntimeState(const UDataLayerAsset* InDataLayerAsset, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (const UDataLayerInstance* DataLayerInstance = InDataLayerAsset ? GetDataLayerInstanceFromAsset(InDataLayerAsset) : nullptr)
	{
		return SetDataLayerInstanceRuntimeState(DataLayerInstance, InState, bInIsRecursive);
	}
	return false;
}

void UDataLayerManager::BroadcastOnDataLayerInstanceRuntimeStateChanged(const UDataLayerInstance* InDataLayer, EDataLayerRuntimeState InState)
{
	// For backward compatibility
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (UDataLayerSubsystem* DataLayerSubsystem = UWorld::GetSubsystem<UDataLayerSubsystem>(GetWorld()))
	{
		DataLayerSubsystem->OnDataLayerRuntimeStateChanged.Broadcast(InDataLayer, InState);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	OnDataLayerInstanceRuntimeStateChanged.Broadcast(InDataLayer, InState);
}

EDataLayerRuntimeState UDataLayerManager::GetDataLayerInstanceRuntimeState(const UDataLayerInstance* InDataLayerInstance) const
{
	if (InDataLayerInstance)
	{
		return InDataLayerInstance->GetRuntimeState();
	}
	UE_LOGF(LogWorldPartition, Error, "Invalid Data Layer Instance.");
	return EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState UDataLayerManager::GetDataLayerInstanceEffectiveRuntimeState(const UDataLayerInstance* InDataLayerInstance) const
{
	if (InDataLayerInstance)
	{
		return InDataLayerInstance->GetEffectiveRuntimeState();
	}
	UE_LOGF(LogWorldPartition, Error, "Invalid Data Layer Instance.");
	return EDataLayerRuntimeState::Unloaded;
}

const TSet<FName>& UDataLayerManager::GetEffectiveActiveDataLayerNames() const
{
	static TSet<FName> EmptySet;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetEffectiveActiveDataLayerNames() : EmptySet;
}

const TSet<FName>& UDataLayerManager::GetEffectiveLoadedDataLayerNames() const
{
	static TSet<FName> EmptySet;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return WorldDataLayers ? WorldDataLayers->GetEffectiveLoadedDataLayerNames() : EmptySet;
}

bool UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState) const
{
	static FWorldDataLayersEffectiveStates EmptyStates;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState(InDataLayerNames, InState, WorldDataLayers ? FWorldDataLayersEffectiveStatesAccessor::Get(WorldDataLayers) : EmptyStates);
}

bool UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState, const FWorldDataLayersEffectiveStates& InEffectiveStates)
{
	if (InState == EDataLayerRuntimeState::Activated)
	{
		for (const FName& DataLayerName : InDataLayerNames)
		{
			if (InEffectiveStates.GetAllEffectiveActiveDataLayerNames().Contains(DataLayerName))
			{
				return true;
			}
		}
	}
	else if (InState == EDataLayerRuntimeState::Loaded)
	{
		for (const FName& DataLayerName : InDataLayerNames)
		{
			if (InEffectiveStates.GetAllEffectiveLoadedDataLayerNames().Contains(DataLayerName))
			{
				return true;
			}
		}
	}
	return false;
}

bool UDataLayerManager::IsAllDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState) const
{
	static FWorldDataLayersEffectiveStates EmptyStates;
	AWorldDataLayers* WorldDataLayers = GetWorldDataLayers();
	return UDataLayerManager::IsAllDataLayerInEffectiveRuntimeState(InDataLayerNames, InState, WorldDataLayers ? FWorldDataLayersEffectiveStatesAccessor::Get(WorldDataLayers) : EmptyStates);
}

bool UDataLayerManager::IsAllDataLayerInEffectiveRuntimeState(TArrayView<const FName> InDataLayerNames, EDataLayerRuntimeState InState, const FWorldDataLayersEffectiveStates& InEffectiveStates)
{
	if (InState == EDataLayerRuntimeState::Activated)
	{
		for (const FName& DataLayerName : InDataLayerNames)
		{
			if (!InEffectiveStates.GetAllEffectiveActiveDataLayerNames().Contains(DataLayerName))
			{
				return false;
			}
		}
	}
	else if (InState == EDataLayerRuntimeState::Loaded)
	{
		for (const FName& DataLayerName : InDataLayerNames)
		{
			if (!InEffectiveStates.GetAllEffectiveLoadedDataLayerNames().Contains(DataLayerName) && !InEffectiveStates.GetAllEffectiveActiveDataLayerNames().Contains(DataLayerName))
			{
				return false;
			}
		}
	}
	return true;
}

void UDataLayerManager::DumpDataLayers(FOutputDevice& OutputDevice) const
{
	if (AWorldDataLayers* WorldDataLayers = GetWorldDataLayers())
	{
		WorldDataLayers->DumpDataLayers(OutputDevice);
	}
}

void UDataLayerManager::DrawDataLayersStatus(UCanvas* Canvas, FVector2D& Offset) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDataLayerManager::DrawDataLayersStatus);

	if (!Canvas || !Canvas->SceneView)
	{
		return;
	}

	FVector2D Pos = Offset;
	float MaxTextWidth = 0.f;

	UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld());

	auto DrawLayerNames = [this, Canvas, &Pos, &MaxTextWidth](const FString& Title, FColor HeaderColor, FColor TextColor, const TSet<FName>& LayerNames)
	{
		if (LayerNames.Num() > 0)
		{
			FWorldPartitionDebugHelper::DrawText(Canvas, Title, GEngine->GetSmallFont(), HeaderColor, Pos, &MaxTextWidth);

			TArray<const UDataLayerInstance*> DataLayers;
			DataLayers.Reserve(LayerNames.Num());
			for (const FName& DataLayerName : LayerNames)
			{
				if (const UDataLayerInstance* DataLayerInstance = GetDataLayerInstanceFromName(DataLayerName))
				{
					DataLayers.Add(DataLayerInstance);
				}
			}

			DataLayers.Sort([](const UDataLayerInstance& A, const UDataLayerInstance& B) { return A.GetDataLayerFullName() < B.GetDataLayerFullName(); });

			UFont* DataLayerFont = GEngine->GetSmallFont();
			for (const UDataLayerInstance* DataLayerInstance : DataLayers)
			{
				FString DataLayerString = DataLayerInstance->GetDataLayerShortName();

				if (GDrawDataLayersLoadTime)
				{
					if (double* DataLayerLoadTime = ActiveDataLayersLoadTime.Find(DataLayerInstance))
					{
						if (*DataLayerLoadTime < 0)
						{
							DataLayerString += FString::Printf(TEXT(" (streaming %s)"), *FPlatformTime::PrettyTime(FPlatformTime::Seconds() + *DataLayerLoadTime));
						}
						else
						{
							DataLayerString += FString::Printf(TEXT(" (took %s)"), *FPlatformTime::PrettyTime(*DataLayerLoadTime));
						}
					}
				}

				FWorldPartitionDebugHelper::DrawLegendItem(Canvas, *DataLayerString, DataLayerFont, DataLayerInstance->GetDebugColor(), TextColor, Pos, &MaxTextWidth);
			}
		}
	};

	const TSet<FName> LoadedDataLayers = GetEffectiveLoadedDataLayerNames();
	const TSet<FName> ActiveDataLayers = GetEffectiveActiveDataLayerNames();

	const FString OuterWorld = UWorld::RemovePIEPrefix(FPaths::GetBaseFilename(GetOuterUWorldPartition()->GetPackage()->GetName()));
	FWorldPartitionDebugHelper::DrawText(Canvas, *OuterWorld, GEngine->GetSmallFont(), FColor::Yellow, Pos, &MaxTextWidth);
	DrawLayerNames(TEXT("Loaded Data Layers"), FColor::Cyan, FColor::White, LoadedDataLayers);
	DrawLayerNames(TEXT("Active Data Layers"), FColor::Green, FColor::White, ActiveDataLayers);

	TSet<FName> UnloadedDataLayers;
	ForEachDataLayerInstance([&LoadedDataLayers, &ActiveDataLayers, &UnloadedDataLayers](UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance->IsRuntime())
		{
			const FName DataLayerName = DataLayerInstance->GetFName();
			if (!LoadedDataLayers.Contains(DataLayerName) && !ActiveDataLayers.Contains(DataLayerName))
			{
				UnloadedDataLayers.Add(DataLayerName);
			}
		}
		return true;
	});
	DrawLayerNames(TEXT("Unloaded Data Layers"), FColor::Silver, FColor(192, 192, 192), UnloadedDataLayers);

	Offset.X += MaxTextWidth + 10;

	// Update data layers load times
	if (GDrawDataLayersLoadTime)
	{
		for (FName DataLayerName : UnloadedDataLayers)
		{
			if (const UDataLayerInstance* DataLayerInstance = GetDataLayerInstanceFromName(DataLayerName))
			{
				ActiveDataLayersLoadTime.Remove(DataLayerInstance);
			}
		}

		TArray<const UDataLayerInstance*> LoadingDataLayers;
		LoadingDataLayers.Reserve(LoadedDataLayers.Num() + ActiveDataLayers.Num());
		auto TransformAndAppendDataLayers = [this](const TSet<FName>& InDataLayerNames, TArray<const UDataLayerInstance*>& OutDataLayerInstances)
		{
			for (FName DataLayerName : InDataLayerNames)
			{
				if (const UDataLayerInstance* DataLayerInstance = GetDataLayerInstanceFromName(DataLayerName))
				{
					OutDataLayerInstances.Add(DataLayerInstance);
				}
			}
		};
		TransformAndAppendDataLayers(LoadedDataLayers, LoadingDataLayers);
		TransformAndAppendDataLayers(ActiveDataLayers, LoadingDataLayers);

		for (const UDataLayerInstance* DataLayerInstance : LoadingDataLayers)
		{
			double* DataLayerLoadTime = ActiveDataLayersLoadTime.Find(DataLayerInstance);

			auto IsDataLayerReady = [WorldPartitionSubsystem](const UDataLayerInstance* InDataLayerInstance, EWorldPartitionRuntimeCellState InTargetState)
			{
				FWorldPartitionStreamingQuerySource QuerySource;
				QuerySource.bDataLayersOnly = true;
				QuerySource.bSpatialQuery = false;
				QuerySource.DataLayers.Add(InDataLayerInstance->GetFName());
				return WorldPartitionSubsystem->IsStreamingCompleted(InTargetState, { QuerySource }, true);
			};

			const EWorldPartitionRuntimeCellState TargetState = ActiveDataLayers.Contains(DataLayerInstance->GetFName()) ? EWorldPartitionRuntimeCellState::Activated : EWorldPartitionRuntimeCellState::Loaded;

			if (!DataLayerLoadTime)
			{
				if (!IsDataLayerReady(DataLayerInstance, TargetState))
				{
					DataLayerLoadTime = &ActiveDataLayersLoadTime.Add(DataLayerInstance, -FPlatformTime::Seconds());
				}
			}

			if (DataLayerLoadTime && (*DataLayerLoadTime < 0))
			{
				if (IsDataLayerReady(DataLayerInstance, TargetState))
				{
					*DataLayerLoadTime = FPlatformTime::Seconds() + *DataLayerLoadTime;
				}
			}
		}
	}
	else
	{
		ActiveDataLayersLoadTime.Empty();
	}
}

TArray<UDataLayerInstance*> UDataLayerManager::ConvertArgsToDataLayers(const TArray<FString>& InArgs)
{
	TSet<UDataLayerInstance*> OutDataLayers;

	const TCHAR* QuoteChar = TEXT("\"");
	bool bQuoteStarted = false;
	TStringBuilder<512> Builder;
	TArray<FString> Args;
	for (const FString& Arg : InArgs)
	{
		if (!bQuoteStarted && Arg.StartsWith(QuoteChar))
		{
			Builder.Append(Arg.Replace(QuoteChar, TEXT("")));
			if (Arg.EndsWith(QuoteChar) && Arg.Len() > 1)
			{
				Args.Add(Builder.ToString());
				Builder.Reset();
			}
			else
			{
				bQuoteStarted = true;
			}
		}
		else if (bQuoteStarted)
		{
			Builder.Append(TEXT(" "));
			Builder.Append(Arg.Replace(QuoteChar, TEXT("")));
			if (Arg.EndsWith(QuoteChar))
			{
				bQuoteStarted = false;
				Args.Add(Builder.ToString());
				Builder.Reset();
			}
		}
		else
		{
			Args.Add(Arg);
		}
	}
	if (bQuoteStarted)
	{
		Args.Add(Builder.ToString());
	}

	for (const FString& Arg : Args)
	{
		FName DataLayerName = FName(Arg);
		bool bShortNameFound = false;
		FString SanitizedDataLayerName = DataLayerName.ToString().Replace(TEXT(" "), TEXT(""));
		ForEachDataLayerInstance([&OutDataLayers, &SanitizedDataLayerName, &bShortNameFound](UDataLayerInstance* DataLayerInstance)
		{
			if (DataLayerInstance->GetDataLayerShortName().Compare(SanitizedDataLayerName, ESearchCase::IgnoreCase) == 0)
			{
				if (bShortNameFound)
				{
					UE_LOGF(LogWorldPartition, Error, "Found 2 data layers with the ShortName %ls when converting arguments. Consider using the data layers FullName or renaming one of the two.", *SanitizedDataLayerName);
					return false;
				}

				OutDataLayers.Add(DataLayerInstance);
				bShortNameFound = true;
			}
			else if (DataLayerInstance->GetDataLayerFullName().Find(SanitizedDataLayerName, ESearchCase::IgnoreCase) == 0)
			{
				OutDataLayers.Add(DataLayerInstance);
				return false;
			}

			return true;
		});
	}

	return OutDataLayers.Array();
}

void UDataLayerManager::ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func)
{
	if (AWorldDataLayers* WorldDataLayers = GetWorldDataLayers())
	{
		bool bShouldContinue = false;
		auto CallAndSetContinueFunc = [Func, &bShouldContinue](UDataLayerInstance* DataLayerInstance)
		{
			bShouldContinue = Func(DataLayerInstance);
			return bShouldContinue;
		};

		WorldDataLayers->ForEachDataLayerInstance(CallAndSetContinueFunc);
	}
}

void UDataLayerManager::ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func) const
{
	const_cast<UDataLayerManager*>(this)->ForEachDataLayerInstance(Func);
}

/*
 * UDataLayerManager Editor-Only API
 */

#if WITH_EDITOR
TSubclassOf<UDataLayerLoadingPolicy> UDataLayerManager::GetDataLayerLoadingPolicyClass() const
{
	// Use UDataLayerManager::DataLayerLoadingPolicyClass
	UClass* DataLayerLoadingPolicyClassValue = DataLayerLoadingPolicyClass.Get();
	if (!DataLayerLoadingPolicyClassValue)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Fallback on UDataLayerSubsystem::DataLayerLoadingPolicyClass
		UDataLayerSubsystem* DataLayerSubsystem = UWorld::GetSubsystem<UDataLayerSubsystem>(GetWorld());
		DataLayerLoadingPolicyClassValue = DataLayerSubsystem ? DataLayerSubsystem->DataLayerLoadingPolicyClass.Get() : nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	if (!DataLayerLoadingPolicyClassValue)
	{
		// Default to UDataLayerLoadingPolicy class
		DataLayerLoadingPolicyClassValue = UDataLayerLoadingPolicy::StaticClass();
	}
	return DataLayerLoadingPolicyClassValue;
}

TSubclassOf<UDataLayerInstanceWithAsset> UDataLayerManager::GetDataLayerInstanceWithAssetClass()
{
	UClass* DataLayerLoadingPolicyClassValue = GetDefault<UDataLayerManager>()->DataLayerInstanceWithAssetClass.Get();
	if (!DataLayerLoadingPolicyClassValue)
	{
		// Default to UDataLayerInstanceWithAsset class
		DataLayerLoadingPolicyClassValue = UDataLayerInstanceWithAsset::StaticClass();
	}
	return DataLayerLoadingPolicyClassValue;
}

void UDataLayerManager::UpdateDataLayerEditorPerProjectUserSettings() const
{
	TArray<const UDataLayerInstance*> DataLayersNotLoadedInEditor;
	TArray<const UDataLayerInstance*> DataLayersLoadedInEditor;

	GetUserLoadedInEditorStates(DataLayersLoadedInEditor, DataLayersNotLoadedInEditor);

	GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->SetWorldDataLayersNonDefaultEditorLoadStates(GetWorld(), DataLayersLoadedInEditor, DataLayersNotLoadedInEditor);
}

void UDataLayerManager::GetUserLoadedInEditorStates(TArray<const UDataLayerInstance*>& OutDataLayersLoadedInEditor, TArray<const UDataLayerInstance*>& OutDataLayersNotLoadedInEditor) const
{
	OutDataLayersLoadedInEditor.Empty();
	OutDataLayersNotLoadedInEditor.Empty();

	const TArray<const UDataLayerAsset*>& SettingsDataLayerAssetsNotLoadedInEditor = GetDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetWorldDataLayerAssetsNotLoadedInEditor(GetWorld());
	const TArray<const UDataLayerAsset*>& SettingsDataLayerAssetsLoadedInEditor = GetDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetWorldDataLayerAssetsLoadedInEditor(GetWorld());

	ForEachDataLayerInstance([&OutDataLayersLoadedInEditor, &OutDataLayersNotLoadedInEditor, &SettingsDataLayerAssetsNotLoadedInEditor, &SettingsDataLayerAssetsLoadedInEditor](UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance->IsLoadedInEditorChangedByUserOperation())
		{
			if (!DataLayerInstance->IsLoadedInEditor() && DataLayerInstance->IsInitiallyLoadedInEditor())
			{
				OutDataLayersNotLoadedInEditor.Add(DataLayerInstance);
			}
			else if (DataLayerInstance->IsLoadedInEditor() && !DataLayerInstance->IsInitiallyLoadedInEditor())
			{
				OutDataLayersLoadedInEditor.Add(DataLayerInstance);
			}

			DataLayerInstance->ClearLoadedInEditorChangedByUserOperation();
		}
		else
		{
			const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset();
			if (SettingsDataLayerAssetsNotLoadedInEditor.Contains(DataLayerAsset))
			{
				OutDataLayersNotLoadedInEditor.Add(DataLayerInstance);
			}
			else if (SettingsDataLayerAssetsLoadedInEditor.Contains(DataLayerAsset))
			{
				OutDataLayersLoadedInEditor.Add(DataLayerInstance);
			}
		}

		return true;
	});
}

TArray<AWorldDataLayers*> UDataLayerManager::GetActorEditorContextWorldDataLayers() const
{
	TArray<AWorldDataLayers*> WorldDataLayersArray;
	if (AWorldDataLayers* WorldDataLayers = GetWorldDataLayers())
	{
		WorldDataLayersArray.Add(WorldDataLayers);
	}
	const UWorld* OuterWorld = GetTypedOuter<UWorld>();
	if (const ULevel* CurrentLevel = (OuterWorld->GetCurrentLevel() && (OuterWorld->GetCurrentLevel() != OuterWorld->PersistentLevel)) ? OuterWorld->GetCurrentLevel() : nullptr)
	{
		if (AWorldDataLayers* CurrentLevelWorldDataLayers = CurrentLevel->GetWorldDataLayers())
		{
			WorldDataLayersArray.Add(CurrentLevelWorldDataLayers);
		}
	}
	return WorldDataLayersArray;
}

void UDataLayerManager::PushActorEditorContext(bool bDuplicateContext) const
{
	++DataLayerActorEditorContextID;
	for (AWorldDataLayers* WorldDataLayers : GetActorEditorContextWorldDataLayers())
	{
		WorldDataLayers->PushActorEditorContext(DataLayerActorEditorContextID, bDuplicateContext);
	}
}

void UDataLayerManager::PopActorEditorContext() const
{
	check(DataLayerActorEditorContextID > 0);
	for (AWorldDataLayers* WorldDataLayers : GetActorEditorContextWorldDataLayers())
	{
		WorldDataLayers->PopActorEditorContext(DataLayerActorEditorContextID);
	}
	--DataLayerActorEditorContextID;
}

TArray<UDataLayerInstance*> UDataLayerManager::GetActorEditorContextDataLayers() const
{
	TArray<UDataLayerInstance*> ActorEditorContextDataLayers;
	for (AWorldDataLayers* WorldDataLayers : GetActorEditorContextWorldDataLayers())
	{
		ActorEditorContextDataLayers.Append(WorldDataLayers->GetActorEditorContextDataLayers());
	}
	return ActorEditorContextDataLayers;
}

uint32 UDataLayerManager::GetDataLayerEditorContextHash() const
{
	TArray<FName> DataLayerInstanceNames;
	for (UDataLayerInstance* DataLayerInstance : GetActorEditorContextDataLayers())
	{
		DataLayerInstanceNames.Add(DataLayerInstance->GetFName());
	}
	return FDataLayerEditorContext(GetWorld(), DataLayerInstanceNames).GetHash();
}

bool UDataLayerManager::CanResolveDataLayers() const
{
	return (GetWorldDataLayers() != nullptr) && bCanResolveDataLayers;
}

void UDataLayerManager::OnActorDescContainerInstanceInitialized(UActorDescContainerInstance* InActorDescContainerInstance)
{
	ResolveActorDescContainerInstanceDataLayers(InActorDescContainerInstance);
}

void UDataLayerManager::ResolveActorDescContainersDataLayers() const
{
	check(CanResolveDataLayers());
	for (TObjectIterator<UActorDescContainerInstance> ContainerIt; ContainerIt; ++ContainerIt)
	{
		if (UActorDescContainerInstance* ContainerInstance = *ContainerIt; ContainerInstance)
		{
			ResolveActorDescContainerInstanceDataLayers(ContainerInstance);
		}
	}
}

FWorldPartitionReference UDataLayerManager::LoadWorldDataLayersActor(UActorDescContainerInstance* InActorDescContainerInstance)
{
	FWorldPartitionReference WDLReference;
	for (UActorDescContainerInstance::TIterator<> Iterator(InActorDescContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->GetActorNativeClass()->IsChildOf<AWorldDataLayers>())
		{
			WDLReference = FWorldPartitionReference(InActorDescContainerInstance, Iterator->GetGuid());
			break;
		}
	}
	return WDLReference;
}

void UDataLayerManager::ResolveActorDescContainerInstanceDataLayers(UActorDescContainerInstance* InActorDescContainerInstance) const
{
	ResolveActorDescContainerInstanceDataLayersInternal(InActorDescContainerInstance, nullptr);
}

void UDataLayerManager::ResolveActorDescInstanceDataLayers(FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	ResolveActorDescContainerInstanceDataLayersInternal(InActorDescInstance->GetContainerInstance(), InActorDescInstance);
}

void UDataLayerManager::ResolveActorDescContainerInstanceDataLayersInternal(UActorDescContainerInstance* InActorDescContainerInstance, FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	check(InActorDescContainerInstance);
	check(!InActorDescInstance || (InActorDescInstance->GetContainerInstance() == InActorDescContainerInstance));
		
	const UWorldPartition* ContainerOuterWorldPartition = InActorDescContainerInstance->GetTopWorldPartition();
	// Skip resolving for template containers (will be done on ActorDescViews)
	if (!ContainerOuterWorldPartition)
	{
		return;
	}

	const ULevelStreaming* ContainerLevelStreaming = FLevelUtils::FindStreamingLevel(ContainerOuterWorldPartition->GetTypedOuter<UWorld>()->PersistentLevel);
	const UWorld* ContainerLevelStreamingWorld = ContainerLevelStreaming ? ContainerLevelStreaming->GetWorld() : nullptr;
	const UWorldPartition* ContainerOwningWorldPartition = ContainerLevelStreamingWorld && !ContainerLevelStreamingWorld->IsGameWorld() ? ContainerLevelStreamingWorld->GetWorldPartition() : ContainerOuterWorldPartition;
	const UWorldPartition* DataLayerManagerOwningWorldPartition = GetOuterUWorldPartition();

	// Skip resolving for containers part of another owning world partition
	if (DataLayerManagerOwningWorldPartition != ContainerOwningWorldPartition)
	{
		return;
	}

	// Resolve ActorDescs DataLayerInstanceNames
	check(CanResolveDataLayers());
	if (InActorDescInstance)
	{
		InActorDescInstance->SetDataLayerInstanceNames(FDataLayerUtils::ResolveDataLayerInstanceNames(this, InActorDescInstance->GetActorDesc()));
	}
	else
	{
		for (UActorDescContainerInstance::TIterator<> Iterator(InActorDescContainerInstance); Iterator; ++Iterator)
		{
			Iterator->SetDataLayerInstanceNames(FDataLayerUtils::ResolveDataLayerInstanceNames(this, Iterator->GetActorDesc()));
		}
	}
}

bool UDataLayerManager::ResolveIsLoadedInEditor(const TArray<FName>& InDataLayerInstanceNames) const
{
	TArray<const UDataLayerInstance*> AllDataLayerInstances = GetDataLayerInstances(InDataLayerInstanceNames);
	TArray<const UDataLayerInstance*> DataLayerInstances;
	TArray<const UExternalDataLayerInstance*> ExternalDataLayerInstances;
	for (const UDataLayerInstance* DataLayerInstance : AllDataLayerInstances)
	{
		if (const UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(DataLayerInstance))
		{
			ExternalDataLayerInstances.Add(ExternalDataLayerInstance);
		}
		else
		{
			DataLayerInstances.Add(DataLayerInstance);
		}
	}

	// If part of an External Data Layer, the External Data Layer must be loaded in editor
	const bool bAnyExternalDataLayerInstanceLoaded = ExternalDataLayerInstances.Num() ? Algo::AnyOf(ExternalDataLayerInstances, [](const UExternalDataLayerInstance* ExternalDataLayerInstance) { return ExternalDataLayerInstance->IsEffectiveLoadedInEditor(); }) : true;
	if (bAnyExternalDataLayerInstanceLoaded)
	{
		const bool bAnyDataLayerInstanceLoaded = DataLayerInstances.Num() ? DataLayerLoadingPolicy->ResolveIsLoadedInEditor(DataLayerInstances) : true;
		return bAnyDataLayerInstanceLoaded;
	}
	return false;
}

TArray<const UDataLayerInstance*> UDataLayerManager::GetRuntimeDataLayerInstances(const TArray<FName>& InDataLayerInstanceNames) const
{
	TArray<const UDataLayerInstance*> DataLayerInstances;

	for (const FName& DataLayerInstanceName : InDataLayerInstanceNames)
	{
		if (const UDataLayerInstance* DataLayerInstance = GetDataLayerInstanceFromName(DataLayerInstanceName))
		{
			if (ensure(DataLayerInstance->IsRuntime()))
			{
				DataLayerInstances.Add(DataLayerInstance);
			}
		}
	}

	return DataLayerInstances;
}

FDataLayersEditorBroadcast& FDataLayersEditorBroadcast::Get()
{
	static FDataLayersEditorBroadcast DataLayersEditorBroadcast;
	return DataLayersEditorBroadcast;
}

void FDataLayersEditorBroadcast::StaticOnActorDataLayersEditorLoadingStateChanged(bool bIsFromUserChange)
{
	Get().DataLayerEditorLoadingStateChanged.Broadcast(bIsFromUserChange);
	IWorldPartitionActorLoaderInterface::RefreshLoadedState(bIsFromUserChange);
}
#endif
