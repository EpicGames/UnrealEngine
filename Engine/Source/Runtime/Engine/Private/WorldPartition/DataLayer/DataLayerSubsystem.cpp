// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerSubsystem.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerEditorContext.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "Engine/Canvas.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataLayerSubsystem)

#if WITH_EDITOR
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/IDataLayerEditorModule.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#else
#include "Engine/Engine.h"
#endif

bool UDataLayerSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::Inactive || WorldType == EWorldType::EditorPreview;
}

#if WITH_EDITOR

TArray<const UDataLayerInstance*> UDataLayerSubsystem::GetRuntimeDataLayerInstances(UWorld* InWorld, const TArray<FName>& InDataLayerInstanceNames)
{
	static TArray<const UDataLayerInstance*> Empty;
	UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(InWorld);
	return DataLayerManager ? DataLayerManager->GetRuntimeDataLayerInstances(InDataLayerInstanceNames) : Empty;
}

#endif

const TSet<FName>& UDataLayerSubsystem::GetEffectiveActiveDataLayerNames() const
{
	static TSet<FName> Empty;
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetEffectiveActiveDataLayerNames() : Empty;
}

const TSet<FName>& UDataLayerSubsystem::GetEffectiveLoadedDataLayerNames() const
{
	static TSet<FName> Empty;
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetEffectiveLoadedDataLayerNames() : Empty;
}

void UDataLayerSubsystem::SetDataLayerInstanceRuntimeState(const UDataLayerAsset* InDataLayerAsset, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->SetDataLayerInstanceRuntimeState(DataLayerManager->GetDataLayerInstance(InDataLayerAsset), InState, bInIsRecursive);
	}
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerInstanceRuntimeState(const UDataLayerAsset* InDataLayerAsset) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceRuntimeState(DataLayerManager->GetDataLayerInstance(InDataLayerAsset)) : EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerInstanceEffectiveRuntimeState(const UDataLayerAsset* InDataLayerAsset) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceEffectiveRuntimeState(DataLayerManager->GetDataLayerInstance(InDataLayerAsset)) : EDataLayerRuntimeState::Unloaded;
}

void UDataLayerSubsystem::SetDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->SetDataLayerInstanceRuntimeState(InDataLayerInstance, InState, bInIsRecursive);
	}
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceRuntimeState(InDataLayerInstance) : EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerRuntimeStateByName(const FName& InDataLayerName) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceRuntimeState(DataLayerManager->GetDataLayerInstanceFromName(InDataLayerName)) : EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerEffectiveRuntimeState(const UDataLayerInstance* InDataLayerInstance) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceEffectiveRuntimeState(InDataLayerInstance) : EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerEffectiveRuntimeStateByName(const FName& InDataLayerName) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceEffectiveRuntimeState(DataLayerManager->GetDataLayerInstanceFromName(InDataLayerName)) : EDataLayerRuntimeState::Unloaded;
}

bool UDataLayerSubsystem::IsAnyDataLayerInEffectiveRuntimeState(const TArray<FName>& InDataLayerNames, EDataLayerRuntimeState InState) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->IsAnyDataLayerInEffectiveRuntimeState(InDataLayerNames, InState) : false;
}

void UDataLayerSubsystem::DrawDataLayersStatus(UCanvas* Canvas, FVector2D& Offset) const
{
	if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->DrawDataLayersStatus(Canvas, Offset);
	}
}

void UDataLayerSubsystem::DumpDataLayers(FOutputDevice& OutputDevice) const
{
	if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->DumpDataLayers(OutputDevice);
	}
}

UDataLayerInstance* UDataLayerSubsystem::GetDataLayerInstanceFromAsset(const UDataLayerAsset* InDataLayerAsset) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? const_cast<UDataLayerInstance*>(DataLayerManager->GetDataLayerInstance(InDataLayerAsset)) : nullptr;
}

const UDataLayerInstance* UDataLayerSubsystem::GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const
{
	const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld());
	return DataLayerManager ? DataLayerManager->GetDataLayerInstanceFromAssetName(InDataLayerAssetPathName) : nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UDataLayerInstance* UDataLayerSubsystem::GetDataLayer(const FActorDataLayer& InDataLayer) const
{
	return GetDataLayerFromName(InDataLayer.Name);
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerRuntimeStateByLabel(const FName& InDataLayerLabel) const
{
	return GetDataLayerRuntimeState(GetDataLayerFromLabel(InDataLayerLabel));
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerEffectiveRuntimeStateByLabel(const FName& InDataLayerLabel) const
{
	return GetDataLayerEffectiveRuntimeState(GetDataLayerFromLabel(InDataLayerLabel));
}

void UDataLayerSubsystem::SetDataLayerRuntimeState(const FActorDataLayer& InDataLayer, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromName(InDataLayer.Name))
		{
			SetDataLayerRuntimeState(DataLayerInstance, InState, bInIsRecursive);
		}
		else
		{
			UE_LOGF(LogWorldPartition, Warning, "UDataLayerSubsystem::SetDataLayerRuntimeState unknown Data Layer: '%ls'", *InDataLayer.Name.ToString());
		}
	}
}

void UDataLayerSubsystem::SetDataLayerRuntimeStateByLabel(const FName& InDataLayerLabel, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (UDataLayerInstance* DataLayerInstance = GetDataLayerFromLabel(InDataLayerLabel))
	{
		SetDataLayerRuntimeState(DataLayerInstance, InState, bInIsRecursive);
	}
	else
	{
		UE_LOGF(LogWorldPartition, Warning, "UDataLayerSubsystem::SetDataLayerRuntimeStateByLabel unknown Data Layer: '%ls'", *InDataLayerLabel.ToString());
	}
}

UDataLayerInstance* UDataLayerSubsystem::GetDataLayerFromName(FName InDataLayerName) const
{
	return GetDataLayerInstance(InDataLayerName);
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerRuntimeState(const FActorDataLayer& InDataLayer) const
{
	return GetDataLayerRuntimeStateByName(InDataLayer.Name);
}

EDataLayerRuntimeState UDataLayerSubsystem::GetDataLayerEffectiveRuntimeState(const FActorDataLayer& InDataLayer) const
{
	return GetDataLayerEffectiveRuntimeStateByName(InDataLayer.Name);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS