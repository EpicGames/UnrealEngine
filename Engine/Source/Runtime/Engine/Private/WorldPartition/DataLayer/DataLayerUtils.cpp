// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayerUtils.h"

#if WITH_EDITOR
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "Algo/AllOf.h"

#define LOCTEXT_NAMESPACE "DataLayer"

bool FDataLayerUtils::ForEachDataLayerInstance(const UDataLayerManager* InDataLayerManager,
	const FWorldPartitionActorDesc* InActorDesc,
	const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs,
	TFunctionRef<void(const UDataLayerInstance*, const FDataLayerInstanceDesc*, bool)> Callback)
{
	const FName ActorDescExternalDataLayer = InActorDesc->GetExternalDataLayer();
	const TArray<FName>& ActorDescNonExternalDataLayers = InActorDesc->GetNonExternalDataLayers();
	if (ActorDescNonExternalDataLayers.IsEmpty() && ActorDescExternalDataLayer.IsNone())
	{
		return true;
	}
	
	if (InDataLayerManager && InDataLayerManager->CanResolveDataLayers())
	{
		if (const UDataLayerInstance* ExternalDataLayerInstance = InDataLayerManager->GetDataLayerInstanceFromAssetName(ActorDescExternalDataLayer))
		{
			Callback(ExternalDataLayerInstance, nullptr, true);
		}
		for (const FName& DataLayerAssetPath : ActorDescNonExternalDataLayers)
		{
			if (const UDataLayerInstance* DataLayerInstance = InDataLayerManager->GetDataLayerInstanceFromAssetName(DataLayerAssetPath))
			{
				Callback(DataLayerInstance, nullptr, false);
			}
		}
		return true;
	}
	// Fallback on FWorldDataLayersActorDesc
	else if (!InWorldDataLayersActorDescs.IsEmpty() && AreWorldDataLayersActorDescsSane(InWorldDataLayersActorDescs))
	{
		if (const FDataLayerInstanceDesc* ExternalDataLayerInstanceDesc = GetDataLayerInstanceDescFromAssetPath(InWorldDataLayersActorDescs, ActorDescExternalDataLayer))
		{
			Callback(nullptr, ExternalDataLayerInstanceDesc, true);
		}
		for (const FName& DataLayerAssetPath : ActorDescNonExternalDataLayers)
		{
			if (const FDataLayerInstanceDesc* DataLayerInstanceDesc = GetDataLayerInstanceDescFromAssetPath(InWorldDataLayersActorDescs, DataLayerAssetPath))
			{
				Callback(nullptr, DataLayerInstanceDesc, false);
			}
		}
		return true;
	}

	return false;
}

FDataLayerInstanceNames FDataLayerUtils::ResolveDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const FWorldPartitionActorDesc* InActorDesc, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs)
{
	const FName ActorDescExternalDataLayer = InActorDesc->GetExternalDataLayer();
	const TArray<FName>& ActorDescNonExternalDataLayers = InActorDesc->GetNonExternalDataLayers();
	if (ActorDescNonExternalDataLayers.IsEmpty() && ActorDescExternalDataLayer.IsNone())
	{
		return FDataLayerInstanceNames();
	}

	TArray<FName> ResolvedDLINames;

	bool bIsFirstDataLayerExternal = false;
	const bool bSuccess = ForEachDataLayerInstance(InDataLayerManager, InActorDesc, InWorldDataLayersActorDescs,
		[&ResolvedDLINames, &bIsFirstDataLayerExternal, &ActorDescNonExternalDataLayers](
			const UDataLayerInstance* Instance, const FDataLayerInstanceDesc* InstanceDesc, const bool bExternal)
	{
		if (ResolvedDLINames.Num() == 0)
		{
			bIsFirstDataLayerExternal = bExternal;
			ResolvedDLINames.Reserve(ActorDescNonExternalDataLayers.Num() + 1);
		}
		else
		{
		    check(!bExternal);
		}

		if (Instance)
		{
			ResolvedDLINames.Add(Instance->GetFName());
		}
		else
		{
			check(InstanceDesc);
			ResolvedDLINames.Add(InstanceDesc->GetName());
		}
	});

	if (bSuccess)
	{
		return FDataLayerInstanceNames(ResolvedDLINames, bIsFirstDataLayerExternal);
	}
	return FDataLayerInstanceNames(ActorDescNonExternalDataLayers, ActorDescExternalDataLayer);
}

// For performance reasons, this function assumes that InActorDesc's DataLayerInstanceNames was already resolved.
bool FDataLayerUtils::ResolveRuntimeDataLayerInstanceNames(const UDataLayerManager* InDataLayerManager, const IWorldPartitionActorDescInstanceView& InActorDescView, const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, FDataLayerInstanceNames& OutRuntimeDataLayerInstanceNames)
{
	const FDataLayerInstanceNames& ActorDescViewDataLayerInstanceNames = InActorDescView.GetDataLayerInstanceNames();
	const FName EDLInstanceName = ActorDescViewDataLayerInstanceNames.GetExternalDataLayer();
	const TArrayView<const FName> NonEDLInstanceNames = ActorDescViewDataLayerInstanceNames.GetNonExternalDataLayers();

	if (InDataLayerManager && InDataLayerManager->CanResolveDataLayers())
	{
		TArray<FName> ResolvedRuntimeDLINames;
		ResolvedRuntimeDLINames.Reserve(NonEDLInstanceNames.Num() + 1);
		const UDataLayerInstance* ExternalDataLayerInstance = InDataLayerManager->GetDataLayerInstanceFromName(EDLInstanceName);
		const bool bIsFirstDataLayerExternal = (ExternalDataLayerInstance && ExternalDataLayerInstance->IsRuntime());
		if (bIsFirstDataLayerExternal)
		{
			ResolvedRuntimeDLINames.Add(EDLInstanceName);
		}
		for (FName DataLayerInstanceName : NonEDLInstanceNames)
		{
			const UDataLayerInstance* DataLayerInstance = InDataLayerManager->GetDataLayerInstanceFromName(DataLayerInstanceName);
			if (DataLayerInstance && DataLayerInstance->IsRuntime())
			{
				ResolvedRuntimeDLINames.Add(DataLayerInstanceName);
			}
		}
		OutRuntimeDataLayerInstanceNames = FDataLayerInstanceNames(ResolvedRuntimeDLINames, bIsFirstDataLayerExternal);
		return true;
	}

	// Fallback on FWorldDataLayersActorDesc
	const TArray<const FWorldDataLayersActorDesc*>& WorldDataLayersActorDescs = InWorldDataLayersActorDescs;
	if (WorldDataLayersActorDescs.Num())
	{
		check(AreWorldDataLayersActorDescsSane(WorldDataLayersActorDescs));

		TArray<FName> ResolvedRuntimeDLINames;
		ResolvedRuntimeDLINames.Reserve(NonEDLInstanceNames.Num() + 1);
		const FDataLayerInstanceDesc* ExternalDataLayerInstanceDesc = GetDataLayerInstanceDescFromInstanceName(WorldDataLayersActorDescs, EDLInstanceName);
		const bool bIsFirstDataLayerExternal = (ExternalDataLayerInstanceDesc && (ExternalDataLayerInstanceDesc->GetDataLayerType() == EDataLayerType::Runtime));
		if (bIsFirstDataLayerExternal)
		{
			ResolvedRuntimeDLINames.Add(EDLInstanceName);
		}

		for (FName DataLayerInstanceName : NonEDLInstanceNames)
		{
			const FDataLayerInstanceDesc* DataLayerInstanceDesc = GetDataLayerInstanceDescFromInstanceName(WorldDataLayersActorDescs, DataLayerInstanceName);
			if (DataLayerInstanceDesc && (DataLayerInstanceDesc->GetDataLayerType() == EDataLayerType::Runtime))
			{
				ResolvedRuntimeDLINames.Add(DataLayerInstanceName);
			}
		}
		OutRuntimeDataLayerInstanceNames = FDataLayerInstanceNames(ResolvedRuntimeDLINames, bIsFirstDataLayerExternal);
		return true;
	}

	return false;
}

const FDataLayerInstanceDesc* FDataLayerUtils::GetDataLayerInstanceDescFromInstanceName(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, const FName& InDataLayerInstanceName)
{
	for (const FWorldDataLayersActorDesc* WorldDataLayerActorDesc : InWorldDataLayersActorDescs)
	{
		if (const FDataLayerInstanceDesc* DataLayerInstanceDesc = WorldDataLayerActorDesc->GetDataLayerInstanceFromInstanceName(InDataLayerInstanceName))
		{
			return DataLayerInstanceDesc;
		}
	}

	return nullptr;
}

const FDataLayerInstanceDesc* FDataLayerUtils::GetDataLayerInstanceDescFromAssetPath(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs, const FName& InDataLayerAssetPath)
{
	for (const FWorldDataLayersActorDesc* WorldDataLayerActorDesc : InWorldDataLayersActorDescs)
	{
		if (const FDataLayerInstanceDesc* DataLayerInstanceDesc = WorldDataLayerActorDesc->GetDataLayerInstanceFromAssetPath(InDataLayerAssetPath))
		{
			return DataLayerInstanceDesc;
		}
	}

	return nullptr;
}

bool FDataLayerUtils::AreWorldDataLayersActorDescsSane(const TArray<const FWorldDataLayersActorDesc*>& InWorldDataLayersActorDescs)
{
	// Deprecation handling: Case where pre 5.1 map, without WorldDataLayersActorDescs, are resolving their data layer outside of their world (changelist validation). They need proper Actor descs to resolve.
	return Algo::AllOf(InWorldDataLayersActorDescs, [](const FWorldDataLayersActorDesc* WorldDataLayerActorDescs) { return WorldDataLayerActorDescs->IsValid(); });
}

bool FDataLayerUtils::SetDataLayerShortName(UDataLayerInstance* InDataLayerInstance, const FString& InNewShortName)
{
	check(InDataLayerInstance->CanEditDataLayerShortName());
	FString UniqueShortName = FDataLayerUtils::GenerateUniqueDataLayerShortName(UDataLayerManager::GetDataLayerManager(InDataLayerInstance), InNewShortName);
	if (InDataLayerInstance->GetDataLayerShortName() != UniqueShortName)
	{
		InDataLayerInstance->Modify();
		InDataLayerInstance->PerformSetDataLayerShortName(UniqueShortName);
		return true;
	}

	return false;
}

bool FDataLayerUtils::FindDataLayerByShortName(const UDataLayerManager* InDataLayerManager, const FString& InShortName, TSet<UDataLayerInstance*>&  OutDataLayersWithShortName)
{
	OutDataLayersWithShortName.Empty();
	InDataLayerManager->ForEachDataLayerInstance([&](UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance->GetDataLayerShortName() == InShortName)
		{
			OutDataLayersWithShortName.Add(DataLayerInstance);
		}
		return true;
	});
	return OutDataLayersWithShortName.Num() > 0;
}

FString FDataLayerUtils::GenerateUniqueDataLayerShortName(const UDataLayerManager* InDataLayerManager, const FString& InNewShortName)
{
	int32 DataLayerIndex = 0;
	const FString DataLayerShortNameSanitized = FDataLayerUtils::GetSanitizedDataLayerShortName(InNewShortName);
	FString UniqueNewDataLayerShortName = DataLayerShortNameSanitized;
		
	TSet<UDataLayerInstance*> OutDataLayersWithShortName;
	while (FindDataLayerByShortName(InDataLayerManager, UniqueNewDataLayerShortName, OutDataLayersWithShortName))
	{
		UniqueNewDataLayerShortName = FString::Printf(TEXT("%s%d"), *DataLayerShortNameSanitized, ++DataLayerIndex);
	}
		
	return UniqueNewDataLayerShortName;
}

bool FDataLayerUtils::AreDataLayerTypesCompatible(EDataLayerType ParentDataLayerType, EDataLayerType ChildDataLayerType, bool bIsParentExternalDataLayer, FText* OutReason)
{
	if ((ChildDataLayerType == EDataLayerType::Unknown) ||
		(ParentDataLayerType == EDataLayerType::Unknown) ||
		(!bIsParentExternalDataLayer && ParentDataLayerType != EDataLayerType::Editor && ChildDataLayerType != EDataLayerType::Runtime))
	{
		if (OutReason)
		{
			*OutReason = FText::Format(LOCTEXT("IncompatibleChildType", "{0} Data Layer cannot have {1} child Data Layers"), UEnum::GetDisplayValueAsText(ParentDataLayerType), UEnum::GetDisplayValueAsText(ChildDataLayerType));
		}
		return false;
	}
	return true;
}

bool FDataLayerUtils::IsClientOnlyFromDataLayers(const TArray<const UDataLayerInstance*>& InCellDataLayerInstances)
{
	if (!InCellDataLayerInstances.Num())
	{
		return false;
	}

	return Algo::AllOf(InCellDataLayerInstances, [bHasMultipleDataLayers = InCellDataLayerInstances.Num() > 1](const UDataLayerInstance* DataLayerInstance)
	{
		// When an EDL is mixed with other data layers, skip its IsClientOnly() check
		// as long as it is not server-only (i.e. its load filter is None or ClientOnly).
		if (bHasMultipleDataLayers && DataLayerInstance->IsA<UExternalDataLayerInstance>())
		{
			return !DataLayerInstance->IsServerOnly();
		}
		return DataLayerInstance->IsClientOnly();
	});
}

#undef LOCTEXT_NAMESPACE

#endif