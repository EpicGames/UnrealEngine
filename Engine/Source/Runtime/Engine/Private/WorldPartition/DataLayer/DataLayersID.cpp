// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/DataLayersID.h"

#include "WorldPartition/DataLayer/DataLayerInstance.h"

#include "Algo/Transform.h"

#if WITH_EDITOR
FDataLayersID::FDataLayersID()
	: Hash(0)
{}

FDataLayersID::FDataLayersID(const TArray<const UDataLayerInstance*>& InDataLayerInstances)
	: Hash(0)
{
	TArray<FName> DataLayers;
	Algo::TransformIf(InDataLayerInstances, DataLayers, [](const UDataLayerInstance* Item) { return Item->IsRuntime(); }, [](const UDataLayerInstance* Item) { return Item->GetFName(); });

	if (DataLayers.Num())
	{
		TArray<FName> SortedDataLayers = DataLayers;
		SortedDataLayers.Sort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });

		for (FName LayerName : SortedDataLayers)
		{
			Hash = FCrc::StrCrc32(*LayerName.ToString(), Hash);
		}

		check(Hash);
	}
}
#endif
