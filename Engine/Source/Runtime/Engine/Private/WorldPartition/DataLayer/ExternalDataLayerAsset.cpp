// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerAsset)

UExternalDataLayerAsset::UExternalDataLayerAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DataLayerType = EDataLayerType::Runtime;
	bSupportsActorFilters = false;
}

#if WITH_EDITOR

void UExternalDataLayerAsset::OnCreated()
{
	Super::OnCreated();
	UID.Value = FExternalDataLayerUID::NewUID();
}

void UExternalDataLayerAsset::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	UID.Value = FExternalDataLayerUID::NewUID();
}

namespace ExternalDataLayerAsset
{
	static const FName NAME_ExternalDataLayerUID("ExternalDataLayerUID");
};

void UExternalDataLayerAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	Context.AddTag(FAssetRegistryTag(ExternalDataLayerAsset::NAME_ExternalDataLayerUID, *UID.ToString(), FAssetRegistryTag::TT_Hidden));
}

bool UExternalDataLayerAsset::GetAssetRegistryInfoFromPackage(const FAssetData& InAsset, FExternalDataLayerUID& OutExternalDataLayerUID)
{
	FString Value;
	if (InAsset.GetTagValue(ExternalDataLayerAsset::NAME_ExternalDataLayerUID, Value))
	{
		return FExternalDataLayerUID::Parse(Value, OutExternalDataLayerUID);
	}
	return false;
}

#endif
