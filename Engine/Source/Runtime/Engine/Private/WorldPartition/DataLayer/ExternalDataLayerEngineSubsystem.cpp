// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInjectionPolicy.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "Engine/Engine.h"
#include "Algo/Find.h"
#if WITH_EDITOR
#include "Editor.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/DataLayer/DataLayerInstanceProviderInterface.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerEngineSubsystem)

#define LOCTEXT_NAMESPACE "ExternalDataLayerEngineSubsystem"

UExternalDataLayerEngineSubsystem& UExternalDataLayerEngineSubsystem::Get()
{
	check(GEngine); // Retrieving the UExternalDataLayerEngineSubsystem before it was created.
	UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = *GEngine->GetEngineSubsystem<UExternalDataLayerEngineSubsystem>();
#if WITH_EDITORONLY_DATA
	// Lazy create InjectionPolicy
	if (!ExternalDataLayerEngineSubsystem.InjectionPolicy)
	{
		const UClass* InjectionPolicyClass = !ExternalDataLayerEngineSubsystem.InjectionPolicyClass ? UExternalDataLayerInjectionPolicy::StaticClass() : *ExternalDataLayerEngineSubsystem.InjectionPolicyClass;
		ExternalDataLayerEngineSubsystem.InjectionPolicy = NewObject<UExternalDataLayerInjectionPolicy>(&ExternalDataLayerEngineSubsystem, InjectionPolicyClass);
	}
#endif
	return ExternalDataLayerEngineSubsystem;
}

void UExternalDataLayerEngineSubsystem::Tick(float DeltaTime)
{
#if WITH_EDITOR
	for (auto& [ExternalDataLayerAsset, RegisteredEDL] : PreDeletedExternalDataLayerAssets)
	{
		if (ExternalDataLayerAsset.IsValid())
		{
			for (const FObjectKey& RegisteredClient : RegisteredEDL.RegisteredClients)
			{
				RegisterExternalDataLayerAsset(ExternalDataLayerAsset.Get(), RegisteredClient.ResolveObjectPtr());
			}
			for (const FObjectKey& ActiveClient : RegisteredEDL.ActiveClients)
			{
				ActivateExternalDataLayerAsset(ExternalDataLayerAsset.Get(), ActiveClient.ResolveObjectPtr());
			}
		}
	}
	PreDeletedExternalDataLayerAssets.Reset();
#endif
}

bool UExternalDataLayerEngineSubsystem::IsTickable() const
{
#if WITH_EDITOR
	return PreDeletedExternalDataLayerAssets.Num() > 0;
#else
	return false;
#endif
}

TStatId UExternalDataLayerEngineSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UExternalDataLayerEngineSubsystem, STATGROUP_Tickables);
}

#if WITH_EDITOR
void UExternalDataLayerEngineSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	LevelExternalActorsPathsProviderDelegateHandle = ULevel::RegisterLevelExternalActorsPathsProvider(ULevel::FLevelExternalActorsPathsProviderDelegate::CreateUObject(this, &UExternalDataLayerEngineSubsystem::OnGetLevelExternalActorsPaths));
	LevelMountPointResolverDelegateHandle = ULevel::RegisterLevelMountPointResolver(ULevel::FLevelMountPointResolverDelegate::CreateUObject(this, &UExternalDataLayerEngineSubsystem::OnResolveLevelMountPoint));
	FEditorDelegates::OnAssetsPreDelete.AddUObject(this, &UExternalDataLayerEngineSubsystem::OnAssetsPreDelete);
}

void UExternalDataLayerEngineSubsystem::Deinitialize()
{
	Super::Deinitialize();
	ULevel::UnregisterLevelExternalActorsPathsProvider(LevelExternalActorsPathsProviderDelegateHandle);
	ULevel::UnregisterLevelMountPointResolver(LevelMountPointResolverDelegateHandle);
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
}

void UExternalDataLayerEngineSubsystem::OnAssetsPreDelete(const TArray<UObject*>& Objects)
{
	// Backup External Data Layer Assets in case the user cancels the deletion (will be processed in the Tick)
	for (UObject* Object : Objects)
	{
		if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(Object))
		{
			if (ExternalDataLayerAssets.Contains(ExternalDataLayerAsset))
			{
				FRegisteredExternalDataLayers RegisteredEDL = ExternalDataLayerAssets.FindChecked(ExternalDataLayerAsset);
				check(RegisteredEDL.State >= EExternalDataLayerRegistrationState::Registered);
				PreDeletedExternalDataLayerAssets.Add(ExternalDataLayerAsset, RegisteredEDL);
				verify(ExternalDataLayerAssets.Remove(ExternalDataLayerAsset));
				if (RegisteredEDL.State == EExternalDataLayerRegistrationState::Active)
				{
					OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(ExternalDataLayerAsset, EExternalDataLayerRegistrationState::Active, EExternalDataLayerRegistrationState::Registered);
				}
				OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(ExternalDataLayerAsset, EExternalDataLayerRegistrationState::Registered, EExternalDataLayerRegistrationState::Unregistered);
			}
		}
	}
}

void UExternalDataLayerEngineSubsystem::OnGetLevelExternalActorsPaths(const FString& InLevelPackageName, const FString& InPackageShortName, TArray<FString>& OutExternalActorsPaths)
{
	FExternalDataLayerHelper::ForEachExternalDataLayerLevelPackagePath(InLevelPackageName, [&OutExternalActorsPaths, &InPackageShortName](const FString& InEDLLevelPackagePath)
	{
		const FString EDLExternalActorsPath = ULevel::GetExternalActorsPath(InEDLLevelPackagePath, InPackageShortName);
		OutExternalActorsPaths.Add(EDLExternalActorsPath);
	});
}

bool UExternalDataLayerEngineSubsystem::OnResolveLevelMountPoint(const FString& InLevelPackageName, const UObject* InLevelMountPointContext, FString& OutResolvedLevelMountPoint)
{
	if (const UExternalDataLayerAsset* ExternalDataLayerAssetContext = FExternalDataLayerHelper::GetExternalDataLayerAssetFromObject(InLevelMountPointContext))
	{
		OutResolvedLevelMountPoint = FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(ExternalDataLayerAssetContext, InLevelPackageName);
		return true;
	}
	return false;
}

UWorld* UExternalDataLayerEngineSubsystem::GetTickableGameObjectWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}
#endif

bool UExternalDataLayerEngineSubsystem::CanWorldInjectExternalDataLayerAsset(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, FText* OutFailureReason) const
{
#if WITH_EDITOR
	if (!IsExternalDataLayerAssetRegistered(InExternalDataLayerAsset))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FText::Format(LOCTEXT("CantInjectNotRegisteredExternalDataLayerAsset", "External Data Layer Asset {0} is not registered"), FText::FromString(InExternalDataLayerAsset->GetName()));
		}
		return false;
	}

	const FRegisteredExternalDataLayers& RegisteredEDL = ExternalDataLayerAssets.FindChecked(InExternalDataLayerAsset);
	
	// ForcedAllowInjection is only used by UGameFeatureActionConvertContentBundleWorldPartitionBuilder
	if (ForcedAllowInjection.Contains(FForcedExternalDataLayerInjectionKey(InWorld, InExternalDataLayerAsset)))
	{
		return true;
	}

	// When unregistering through UnregisterExternalDataLayerAsset, RegisteredEDL.RegisteredClients is empty (in this case we don't inject)
	const UObject* Client = RegisteredEDL.RegisteredClients.Num() ? RegisteredEDL.RegisteredClients.Array()[0].ResolveObjectPtr() : nullptr;
	return Client ? InjectionPolicy->CanInject(InWorld, InExternalDataLayerAsset, Client, OutFailureReason) : false;
#else
	if (!IsExternalDataLayerAssetActive(InExternalDataLayerAsset))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FText::Format(LOCTEXT("CantInjectNotActiveExternalDataLayerAsset", "External Data Layer Asset {0} is not active"), FText::FromString(InExternalDataLayerAsset->GetName()));
		}
		return false;
	}
	return true;
#endif
}

bool UExternalDataLayerEngineSubsystem::CanInjectOverride(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, bool& bOutOverrideValue) const
{
	if (OnExternalDataLayerOverrideInjection.IsBound())
	{
		OnExternalDataLayerOverrideInjection.Broadcast(InWorld, InExternalDataLayerAsset, bOutOverrideValue);
		return true;
	}

	return false;
}

void UExternalDataLayerEngineSubsystem::RegisterExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient)
{
	check(IsValid(InExternalDataLayerAsset));
	check(InExternalDataLayerAsset->GetUID().IsValid());
	for (const auto& [ExternalDataLayerAsset, Clients] : ExternalDataLayerAssets)
	{
		if ((InExternalDataLayerAsset != ExternalDataLayerAsset) && (InExternalDataLayerAsset->GetUID() == ExternalDataLayerAsset->GetUID()))
		{
			UE_LOGF(LogWorldPartition, Error, "External Data Layer Asset %ls is already registered with UID %ls. Can't register External Data Layer Asset %ls under the same UID.", 
				*ExternalDataLayerAsset->GetPathName(), *ExternalDataLayerAsset->GetUID().ToString(), *InExternalDataLayerAsset->GetPathName());
			return;
		}
	}

	FRegisteredExternalDataLayers& RegisteredEDL = ExternalDataLayerAssets.FindOrAdd(InExternalDataLayerAsset);
	bool bIsAlreadyInSet = false;
	RegisteredEDL.RegisteredClients.Add(InClient, &bIsAlreadyInSet);
	if (!bIsAlreadyInSet && (RegisteredEDL.RegisteredClients.Num() == 1))
	{
		check(RegisteredEDL.ActiveClients.Num() == 0);
		check(RegisteredEDL.State == EExternalDataLayerRegistrationState::Registered);
		OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(InExternalDataLayerAsset, EExternalDataLayerRegistrationState::Unregistered, RegisteredEDL.State);
	}
}

void UExternalDataLayerEngineSubsystem::ActivateExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient)
{
	check(IsValid(InExternalDataLayerAsset));
	if (!IsExternalDataLayerAssetRegistered(InExternalDataLayerAsset, InClient))
	{
		UE_LOGF(LogWorldPartition, Error, "External Data Layer Asset %ls is not registered.", *InExternalDataLayerAsset->GetPathName());
		return;
	}

	FRegisteredExternalDataLayers& RegisteredEDL = ExternalDataLayerAssets.FindChecked(InExternalDataLayerAsset);
	bool bIsAlreadyInSet = false;
	RegisteredEDL.ActiveClients.Add(InClient, &bIsAlreadyInSet);
	if (!bIsAlreadyInSet && (RegisteredEDL.ActiveClients.Num() == 1))
	{
		check(RegisteredEDL.State == EExternalDataLayerRegistrationState::Registered);
		RegisteredEDL.State = EExternalDataLayerRegistrationState::Active;
		OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(InExternalDataLayerAsset, EExternalDataLayerRegistrationState::Registered, RegisteredEDL.State);
	}
}

void UExternalDataLayerEngineSubsystem::DeactivateExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient)
{
	if (!IsExternalDataLayerAssetActive(InExternalDataLayerAsset, InClient))
	{
		return;
	}

	FRegisteredExternalDataLayers& RegisteredEDL = ExternalDataLayerAssets.FindChecked(InExternalDataLayerAsset);
	if (RegisteredEDL.ActiveClients.Contains(InClient))
	{
		verify(RegisteredEDL.ActiveClients.Remove(InClient));
		if (RegisteredEDL.ActiveClients.Num() == 0)
		{
			check(RegisteredEDL.RegisteredClients.Num() > 0);
			check(RegisteredEDL.State == EExternalDataLayerRegistrationState::Active);
			RegisteredEDL.State = EExternalDataLayerRegistrationState::Registered;
			OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(InExternalDataLayerAsset, EExternalDataLayerRegistrationState::Active, RegisteredEDL.State);
		}
	}
}

void UExternalDataLayerEngineSubsystem::UnregisterExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient)
{
	if (FRegisteredExternalDataLayers* RegisteredEDL = InExternalDataLayerAsset ? ExternalDataLayerAssets.Find(InExternalDataLayerAsset) : nullptr)
	{
		// If Active, deactivate before unregistering
		DeactivateExternalDataLayerAsset(InExternalDataLayerAsset, InClient);

		if (RegisteredEDL->RegisteredClients.Contains(InClient))
		{
			verify(RegisteredEDL->RegisteredClients.Remove(InClient));
			if (RegisteredEDL->RegisteredClients.Num() == 0)
			{
				check(RegisteredEDL->ActiveClients.Num() == 0);
				check(RegisteredEDL->State == EExternalDataLayerRegistrationState::Registered)
				RegisteredEDL->State = EExternalDataLayerRegistrationState::Unregistered;
				OnExternalDataLayerAssetRegistrationStateChanged.Broadcast(InExternalDataLayerAsset, EExternalDataLayerRegistrationState::Registered, RegisteredEDL->State);

				verify(ExternalDataLayerAssets.Remove(InExternalDataLayerAsset));
			}
		}
	}
};

TArray<UObject*> UExternalDataLayerEngineSubsystem::GetClientsForExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	const FRegisteredExternalDataLayers* RegisteredEDL = ExternalDataLayerAssets.Find(InExternalDataLayerAsset);
	TSet<UObject*> Clients;
	for (const FObjectKey Client : RegisteredEDL->RegisteredClients)
	{
		Clients.Add(Client.ResolveObjectPtr());
	}
	for (const FObjectKey Client : RegisteredEDL->ActiveClients)
	{
		Clients.Add(Client.ResolveObjectPtr());
	}
	Clients.Remove(nullptr);
	return Clients.Array();
}

EExternalDataLayerRegistrationState UExternalDataLayerEngineSubsystem::GetExternalDataLayerAssetRegistrationState(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	const FRegisteredExternalDataLayers* RegisteredEDL = ExternalDataLayerAssets.Find(InExternalDataLayerAsset);
	return RegisteredEDL ? RegisteredEDL->State : EExternalDataLayerRegistrationState::Unregistered;
}

bool UExternalDataLayerEngineSubsystem::IsExternalDataLayerAssetRegistered(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient) const
{
	const FRegisteredExternalDataLayers* RegisteredEDL = InExternalDataLayerAsset ? ExternalDataLayerAssets.Find(InExternalDataLayerAsset) : nullptr;
	if (RegisteredEDL && (!InClient || RegisteredEDL->RegisteredClients.Contains(InClient)))
	{
		return RegisteredEDL->State >= EExternalDataLayerRegistrationState::Registered;
	}
	return false;
}

bool UExternalDataLayerEngineSubsystem::IsExternalDataLayerAssetActive(const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient) const
{
	const FRegisteredExternalDataLayers* RegisteredEDL = InExternalDataLayerAsset ? ExternalDataLayerAssets.Find(InExternalDataLayerAsset) : nullptr;
	if (RegisteredEDL && (!InClient || RegisteredEDL->ActiveClients.Contains(InClient)))
	{
		return RegisteredEDL->State == EExternalDataLayerRegistrationState::Active;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE 
