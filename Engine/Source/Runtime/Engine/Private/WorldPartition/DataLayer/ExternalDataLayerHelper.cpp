// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "AssetRegistry/ARFilter.h"
#include "ExternalPackageHelper.h"
#include "UObject/Object.h"
#include "UObject/MetaData.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "DeletedObjectPlaceholder.h"
#include "ReferencedAssetsUtils.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "ExternalDataLayerHelper"

FString FExternalDataLayerHelper::GetExternalStreamingObjectPackageName(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(InExternalDataLayerAsset);
	return FString::Printf(TEXT("StreamingObject_%X"), (uint32)InExternalDataLayerAsset->GetUID());
}

bool FExternalDataLayerHelper::BuildExternalDataLayerRootPath(const FString& InEDLMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, FString& OutExternalDataLayerRootPath)
{
	if (InEDLMountPoint.IsEmpty() || !InExternalDataLayerUID.IsValid())
	{
		return false;
	}

	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> Builder;
	Builder += TEXT("/");
	Builder += InEDLMountPoint;
	Builder += GetExternalDataLayerFolder();
	Builder += InExternalDataLayerUID.ToString();
	OutExternalDataLayerRootPath = *Builder;
	return true;
}

bool FExternalDataLayerHelper::BuildExternalDataLayerActorsRootPath(const FString& InEDLMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, FString& OutExternalDataLayerRootPath)
{
	if (InEDLMountPoint.IsEmpty() || !InExternalDataLayerUID.IsValid())
	{
		return false;
	}

	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> Builder;
	Builder += TEXT("/");
	Builder += InEDLMountPoint;
	Builder += TEXT("/");
	Builder += FPackagePath::GetExternalActorsFolderName();
	Builder += GetExternalDataLayerFolder();
	Builder += InExternalDataLayerUID.ToString();
	OutExternalDataLayerRootPath = *Builder;
	return true;
}

FString FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(const UExternalDataLayerAsset* InExternalDataLayerAsset, const FString& InLevelPackagePath)
{
	check(InExternalDataLayerAsset);
	check(InExternalDataLayerAsset->GetUID().IsValid());
	return FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(FPackageName::GetPackageMountPoint(InExternalDataLayerAsset->GetPackage()->GetName()).ToString(), InExternalDataLayerAsset->GetUID(), InLevelPackagePath);
}

FString FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(const FString& InExternalDataLayerMountPoint, const FExternalDataLayerUID& InExternalDataLayerUID, const FString& InLevelPackagePath)
{
	FString ExternalDataLayerRootPath;
	verify(BuildExternalDataLayerRootPath(InExternalDataLayerMountPoint, InExternalDataLayerUID, ExternalDataLayerRootPath));
	TStringBuilderWithBuffer<TCHAR, NAME_SIZE> Builder;
	Builder += ExternalDataLayerRootPath;
	Builder += TEXT("/");
	Builder += InLevelPackagePath;
	FString Result = *Builder;
	FPaths::RemoveDuplicateSlashes(Result);
	return Result;
}

#if WITH_EDITOR

static FName GetExternalDataLayerUIDsAssetRegistryTag()
{
	static const FName ExternalDataLayerUIDsTag("ExternalDataLayerUIDs");
	return ExternalDataLayerUIDsTag;
}

void FExternalDataLayerHelper::AddAssetRegistryTags(FAssetRegistryTagsContext OutContext, const TArray<FExternalDataLayerUID>& InExternalDataLayerUIDs)
{
	if (InExternalDataLayerUIDs.Num() > 0)
	{
		FString ExternalDataLayerUIDsStr = FString::JoinBy(InExternalDataLayerUIDs, TEXT(","), [&](const FExternalDataLayerUID& ExternalDataLayerUID) { return ExternalDataLayerUID.ToString(); });
		OutContext.AddTag(UObject::FAssetRegistryTag(GetExternalDataLayerUIDsAssetRegistryTag(), ExternalDataLayerUIDsStr, UObject::FAssetRegistryTag::TT_Hidden));
	}
}

void FExternalDataLayerHelper::GetExternalDataLayerUIDs(const FAssetData& Asset, TArray<FExternalDataLayerUID>& OutExternalDataLayerUIDs)
{
	FString ExternalDataLayerUIDsStr;
	if (Asset.GetTagValue(GetExternalDataLayerUIDsAssetRegistryTag(), ExternalDataLayerUIDsStr))
	{
		TArray<FString> ExternalDataLayerUIDStrArray;
		ExternalDataLayerUIDsStr.ParseIntoArray(ExternalDataLayerUIDStrArray, TEXT(","));
		for (const FString& ExternalDataLayerUIDStr : ExternalDataLayerUIDStrArray)
		{
			FExternalDataLayerUID ExternalDataLayerUID;
			if (FExternalDataLayerUID::Parse(ExternalDataLayerUIDStr, ExternalDataLayerUID))
			{
				OutExternalDataLayerUIDs.Add(ExternalDataLayerUID);
			}
		}
	}
}

namespace UE::Private::ExternalDataLayerHelper
{
	static bool ValidateAssetUsingAssetReferenceRestrictions(const UObject* InAsset, const TSet<UObject*>& InReferencedAssets, TMap<FString, TArray<FString>>& OutInvalidReferenceReasons)
	{
		if (!InAsset)
		{
			return false;
		}

		uint32 ErrorCount = 0;
		if (InReferencedAssets.Num() > 0)
		{
			FAssetReferenceFilterContext AssetReferenceFilterContext;
			AssetReferenceFilterContext.AddReferencingAsset(FAssetData(InAsset));
			TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
			if (ensure(AssetReferenceFilter.IsValid()))
			{
				for (UObject* ReferencedAsset : InReferencedAssets)
				{
					FText FailureReason;
					FAssetData ReferencedAssetData(ReferencedAsset);
					if (!AssetReferenceFilter->PassesFilter(ReferencedAssetData, &FailureReason))
					{
						const FString MountPoint = FPackageName::GetPackageMountPoint(ReferencedAssetData.PackagePath.ToString()).ToString();
						OutInvalidReferenceReasons.FindOrAdd(FailureReason.ToString()).Add(FString::Printf(TEXT("%s (Mount Point: %s)"), *ReferencedAssetData.GetObjectPathString(), *MountPoint));
						++ErrorCount;
					}
				}
			}
		}
		return !ErrorCount;
	}
}

bool FExternalDataLayerHelper::CanMoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const FMoveToExternalDataLayerParams& InParams, FText* OutFailureReason)
{
	auto CanMoveActorToExternalDataLayer = [](AActor* InActor, const FMoveToExternalDataLayerParams& InParams, FText& OutFailureReason)
	{
		const TArray<UClass*> IgnoreClasses;
		const TArray<UPackage*> IgnorePackages;

		check(!InActor->IsTemplate());
		check(InActor->GetLevel());

		if (!InActor->IsPackageExternal())
		{
			OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_NotPackageExternal", "Actor {0} is not using external package."), FText::FromString(InActor->GetName()));
			return false;
		}

		if (!InParams.bAllowNonUserManaged && !InActor->IsUserManaged() && !InActor->GetExternalPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
		{
			OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_NotUserManaged", "Actor {0} cannot be manually modified."), FText::FromString(InActor->GetName()));
			return false;
		}

		if (!InActor->IsMainPackageActor())
		{
			OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_ChildActorNotSupported", "Child Actor {0} cannot be moved to External Data Layer."), FText::FromString(InActor->GetName()));
			return false;
		}

		FExternalDataLayerUID OldExternalDataLayerUIDFromPackage;
		const UPackage* OldPackage = InActor->GetExternalPackage();
		FExternalDataLayerHelper::IsExternalDataLayerPath(OldPackage->GetPathName(), &OldExternalDataLayerUIDFromPackage);
		const UExternalDataLayerAsset* OldExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
		const FExternalDataLayerUID OldExternalDataLayerUIDFromAsset = OldExternalDataLayerAsset ? OldExternalDataLayerAsset->GetUID() : FExternalDataLayerUID();
		const UExternalDataLayerAsset* NewExternalDataLayerAsset = InParams.ExternalDataLayerInstance ? InParams.ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;

		// Detect if old actor package and EDL asset mismatch (this can happen during the replace actor process).
		// In this case, skip the optimization that tries to detect no change that are relying on OldExternalDataLayerAsset (as it's unreliable).
		const bool bDoesOldExternalDataLayerUIDMismatch = OldExternalDataLayerUIDFromAsset != OldExternalDataLayerUIDFromPackage;
		if (!bDoesOldExternalDataLayerUIDMismatch)
		{
			if (!OldExternalDataLayerAsset && !NewExternalDataLayerAsset)
			{
				OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_NoExternalDataLayer", "Actor {0} has already no External Data Layer."), FText::FromString(InActor->GetName()));
				return false;
			}

			if (OldExternalDataLayerAsset == NewExternalDataLayerAsset)
			{
				OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_SameExternalDataLayer", "Actor {0} is already assigned to this External Data Layer."), FText::FromString(InActor->GetName()));
				return false;
			}
		}

		if (NewExternalDataLayerAsset && !InActor->SupportsDataLayerType(UExternalDataLayerInstance::StaticClass()))
		{
			OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_EDLNotSupported", "Actor {0} doesn't support External Data Layers."), FText::FromString(InActor->GetName()));
			return false;
		}

		if (InParams.ExternalDataLayerInstance && InParams.ExternalDataLayerInstance->IsReadOnly())
		{
			OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDL_ReadOnlyExternalDataLayer", "External Data Layer is read-only."), FText::FromString(InParams.ExternalDataLayerInstance->GetDataLayerShortName()));
			return false;
		}

		UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();
		if (!EditorActorSubsystem)
		{
			OutFailureReason = LOCTEXT("CantMoveActorToEDL_MissingEditorActorSubsystem", "Missing EditorActorSubsystem.");
			return false;
		}

		// Gather actor asset references
		const bool bOnlyDirectReferences = true;
		TSet<UObject*> ActorReferencedAssets;
		{
			const bool bIncludeDefaultRefs = false;
			FFindReferencedAssets::BuildAssetList(InActor, IgnoreClasses, IgnorePackages, ActorReferencedAssets, bIncludeDefaultRefs, bOnlyDirectReferences);
		}
		TArray<UObject*> ReferencedContent;
		InActor->GetReferencedContentObjects(ReferencedContent);
		// Remove itself and its data layer assets from the list
		ActorReferencedAssets.Append(ReferencedContent);
		ActorReferencedAssets.Remove(InActor);
		// Gather and remove CDO assets
		{
			const bool bIncludeDefaultRefs = true;
			TSet<UObject*> CDOReferencedAssets;
			FFindReferencedAssets::BuildAssetList(InActor->GetClass()->GetDefaultObject(), IgnoreClasses, IgnorePackages, CDOReferencedAssets, bIncludeDefaultRefs, bOnlyDirectReferences);
			for (UObject* CDOAsset : CDOReferencedAssets)
			{
				ActorReferencedAssets.Remove(CDOAsset);
			}
		}
		
		for (auto It = ActorReferencedAssets.CreateIterator(); It; ++It)
		{
			if (!(*It)->IsAsset() || (*It)->IsA<UDataLayerAsset>())
			{
				It.RemoveCurrent();
			}
		}

		// Validate if there are restrictions between the world or the new data layer asset and the actor asset references
		TMap<FString, TArray<FString>> InvalidReferences;
		const UObject* Referencer = NewExternalDataLayerAsset ? (UObject*)NewExternalDataLayerAsset : (UObject*)InActor->GetLevel();
		if (!UE::Private::ExternalDataLayerHelper::ValidateAssetUsingAssetReferenceRestrictions(Referencer, ActorReferencedAssets, InvalidReferences))
		{
			TStringBuilder<0> StringBuilder;
			for (auto& [Reason, InvalidReferenceAssets] : InvalidReferences)
			{
				const FString JoinedInvalidReferences = FString::Join(InvalidReferenceAssets, TEXT(", "));
				StringBuilder.Appendf(TEXT(" - Reason: %s\n - Invalid References: \n"), *Reason);
				for (const FString& InvalidReference : InvalidReferenceAssets)
				{
					StringBuilder.Appendf(TEXT("   - %s\n"), *InvalidReference);
				}
			}
			const FString JoinedReasons = StringBuilder.ToString();
			
			if (NewExternalDataLayerAsset)
			{
				OutFailureReason = FText::Format(LOCTEXT("CantMoveActorToEDLReferenceRestrictions", "Can't move Actor {0} to External Data Layer {1}.\n{2}"), FText::FromString(InActor->GetName()), FText::FromString(NewExternalDataLayerAsset->GetName()), FText::FromString(JoinedReasons));
			}
			else
			{
				OutFailureReason = FText::Format(LOCTEXT("CantRemoveEDLFromActorReferenceRestrictions", "Can't remove External Data Layer {0} from Actor {1}.\n{2}"), FText::FromString(OldExternalDataLayerAsset ? OldExternalDataLayerAsset->GetName() : OldExternalDataLayerUIDFromPackage.ToString()), FText::FromString(InActor->GetName()), FText::FromString(JoinedReasons));
			}
			return false;
		}

		return true;
	};

	// Validate that all actors can change their External Data Layer asset
	for (AActor* Actor : InActors)
	{
		FText FailureReason;
		if (!CanMoveActorToExternalDataLayer(Actor, InParams, FailureReason))
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FailureReason;
			}
			return false;
		}
	}

	return true;
}

bool FExternalDataLayerHelper::MoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const UExternalDataLayerInstance* InExternalDataLayerInstance, FText* OutFailureReason)
{
	return MoveActorsToExternalDataLayer(InActors, FMoveToExternalDataLayerParams(InExternalDataLayerInstance), OutFailureReason);
}

bool FExternalDataLayerHelper::MoveActorsToExternalDataLayer(const TArray<AActor*>& InActors, const FMoveToExternalDataLayerParams& InParams, FText* OutFailureReason)
{
	auto MoveActorToExternalDataLayer = [](AActor* InActor, const FMoveToExternalDataLayerParams& InParams)
	{
		check(InActor->IsMainPackageActor());
		const UPackage* OldActorPackage = InActor->GetExternalPackage();
		const UExternalDataLayerAsset* NewExternalDataLayerAsset = InParams.ExternalDataLayerInstance ? InParams.ExternalDataLayerInstance->GetExternalDataLayerAsset() : nullptr;
		const bool bShouldDirty = true;
		const bool bLevelPackageWasDirty = InActor->GetLevel()->GetPackage()->IsDirty();
		InActor->SetPackageExternal(false, bShouldDirty);

		// Get all other dependant objects in the old actor package
		TArray<UObject*> DependantObjects;
		ForEachObjectWithPackage(OldActorPackage, [&DependantObjects](UObject* Object)
		{
			if (!Cast<UDeletedObjectPlaceholder>(Object) && !Cast<AActor>(Object))
			{
				DependantObjects.Add(Object);
			}
			return true;
		}, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage); // Skip garbage objects (like child actors destroyed when de-externalizing the actor)

		// Clear Content Bundle Guid
		FSetActorContentBundleGuid(InActor, FGuid());

		// If set, remove EDL from actor
		const UExternalDataLayerAsset* OldExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
		if (OldExternalDataLayerAsset)
		{
			FAssignActorDataLayer::RemoveDataLayerAsset(InActor, OldExternalDataLayerAsset);
			for (const UDataLayerInstance* DataLayerInstance : InActor->GetDataLayerInstances())
			{
				if (DataLayerInstance->GetRootExternalDataLayerInstance())
				{
					FAssignActorDataLayer::RemoveDataLayerAsset(InActor, DataLayerInstance->GetAsset());
				}
			}
		}

		// If set, add actor to new EDL
		if (NewExternalDataLayerAsset)
		{
			FAssignActorDataLayer::AddDataLayerAsset(InActor, NewExternalDataLayerAsset);
		}

		InActor->SetPackageExternal(true, bShouldDirty);

		// Move dependant objects into the new actor package
		UPackage* NewActorPackage = InActor->GetExternalPackage();
		for (UObject* DependantObject : DependantObjects) //-V1078
		{
			DependantObject->Rename(nullptr, NewActorPackage, REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
		}

		if (!bLevelPackageWasDirty)
		{
			InActor->GetLevel()->GetPackage()->SetDirtyFlag(false);
		}

		return (InActor->GetExternalDataLayerAsset() == NewExternalDataLayerAsset);
	};
	
	// First, validate that the whole operation can be done without any validation errors
	if (!CanMoveActorsToExternalDataLayer(InActors, InParams, OutFailureReason))
	{
		return false;
	}

	// Change all actors External Data Layer asset
	for (AActor* Actor : InActors)
	{
		if (ensure(MoveActorToExternalDataLayer(Actor, InParams)))
		{
			// Basic validation on the actor and its new External Data Layer asset
			UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(Actor);
			check(ExternalDataLayerManager->ValidateOnActorExternalDataLayerAssetChanged(Actor));

			// Notify actor's External Data Layer asset changed
			FProperty* ExternalDataLayerAssetChangeProperty = FindFProperty<FProperty>(Actor->GetClass(), "ExternalDataLayerAsset");
			FPropertyChangedEvent PropertyChangedEvent(ExternalDataLayerAssetChangeProperty);
			Actor->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	return true;
}

void FExternalDataLayerHelper::ForEachExternalDataLayerLevelPackagePath(const FString& InLevelPackageName, TFunctionRef<void(const FString&)> Func)
{
	UClass* GameFeatureDataClass = FindObject<UClass>(nullptr, TEXT("/Script/GameFeatures.GameFeatureData"));
	if (GameFeatureDataClass)
	{
		FARFilter Filter;
		Filter.ClassPaths = { GameFeatureDataClass->GetClassPathName() };
		Filter.bRecursiveClasses = true;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets- avoids indeterministic results
		TArray<FAssetData> AssetsData;
		FExternalPackageHelper::GetSortedAssets(Filter, AssetsData);

		for (const FAssetData& AssetData : AssetsData)
		{
			const FString MountPoint = FPackageName::GetPackageMountPoint(AssetData.PackagePath.ToString()).ToString();

			TArray<FExternalDataLayerUID> ExternalDataLayerUIDs;
			FExternalDataLayerHelper::GetExternalDataLayerUIDs(AssetData, ExternalDataLayerUIDs);
			for (const FExternalDataLayerUID& ExternalDataLayerUID : ExternalDataLayerUIDs)
			{
				if (ExternalDataLayerUID.IsValid())
				{
					FString LevelPackageEDLPath = FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(MountPoint, ExternalDataLayerUID, InLevelPackageName);
					Func(LevelPackageEDLPath);
				}
			}
		}
	}
}

FStringView FExternalDataLayerHelper::GetRelativeExternalActorPackagePath(FStringView InExternalDataLayerExternalActorPackagePath)
{
	uint32 ExternalActorIdx = UE::String::FindFirst(InExternalDataLayerExternalActorPackagePath, FPackagePath::GetExternalActorsFolderName(), ESearchCase::IgnoreCase);
	if (ExternalActorIdx != INDEX_NONE)
	{
		FStringView RelativePath = InExternalDataLayerExternalActorPackagePath.RightChop(ExternalActorIdx + FCString::Strlen(FPackagePath::GetExternalActorsFolderName()));
		if (RelativePath.Left(ExternalDataLayerFolder.Len()).Equals(GetExternalDataLayerFolder()))
		{
			if (!RelativePath.IsEmpty())
			{
				check(RelativePath.Left(GetExternalDataLayerFolder().Len()).Equals(GetExternalDataLayerFolder()));
				RelativePath = RelativePath.RightChop(GetExternalDataLayerFolder().Len());
				if (!RelativePath.IsEmpty())
				{
					return RelativePath.RightChop(RelativePath.Find(TEXT("/")));
				}
			}
		}
	}
	return FStringView();
}

bool FExternalDataLayerHelper::IsExternalDataLayerPath(FStringView InExternalDataLayerPath, FExternalDataLayerUID* OutExternalDataLayerUID)
{
	int32 ExternalDataLayerFolderIdx = UE::String::FindFirst(InExternalDataLayerPath, GetExternalDataLayerFolder(), ESearchCase::IgnoreCase);
	if (ExternalDataLayerFolderIdx != INDEX_NONE)
	{
		FStringView RelativeExternalDataLayerPath = InExternalDataLayerPath.RightChop(ExternalDataLayerFolderIdx + GetExternalDataLayerFolder().Len());
		int32 ExternalDataLayerUIDEndIdx = UE::String::FindFirst(RelativeExternalDataLayerPath, TEXT("/"), ESearchCase::IgnoreCase);
		if (ExternalDataLayerUIDEndIdx != INDEX_NONE)
		{
			if (RelativeExternalDataLayerPath.RightChop(ExternalDataLayerUIDEndIdx + 1).Len() > 0) // + 1 to remove the "/"
			{
				FExternalDataLayerUID UID;
				const FString ExternalDataLayerUIDStr = FString(RelativeExternalDataLayerPath.Mid(0, ExternalDataLayerUIDEndIdx));
				return FExternalDataLayerUID::Parse(ExternalDataLayerUIDStr, OutExternalDataLayerUID ? *OutExternalDataLayerUID : UID);
			}
		}
	}
	return false;
}

const UExternalDataLayerAsset* FExternalDataLayerHelper::GetExternalDataLayerAssetFromObject(const UObject* InContextObject)
{
	const UExternalDataLayerAsset* ExternalDataLayerAssetContext = nullptr;
	if (InContextObject)
	{
		ExternalDataLayerAssetContext = Cast<UExternalDataLayerAsset>(InContextObject);
		if (!ExternalDataLayerAssetContext && InContextObject->Implements<UDataLayerInstanceProvider>())
		{
			ExternalDataLayerAssetContext = CastChecked<IDataLayerInstanceProvider>(InContextObject)->GetRootExternalDataLayerAsset();
		}
		if (!ExternalDataLayerAssetContext && InContextObject->IsA<AActor>())
		{
			ExternalDataLayerAssetContext = CastChecked<AActor>(InContextObject)->GetExternalDataLayerAsset();
		}
	}
	return ExternalDataLayerAssetContext;
}

#endif

#undef LOCTEXT_NAMESPACE