// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/ExternalDataLayerInjectionPolicy.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerInjectionPolicy)

#define LOCTEXT_NAMESPACE "ExternalDataLayerInjectionPolicy"

#if WITH_EDITOR
bool UExternalDataLayerInjectionPolicy::CanInject(const UWorld* InWorld, const UExternalDataLayerAsset* InExternalDataLayerAsset, const UObject* InClient, FText* OutFailureReason) const
{
	bool bOverrideCanInject = false;
	const bool bOverrideInjectionPolicy = UExternalDataLayerEngineSubsystem::Get().CanInjectOverride(InWorld, InExternalDataLayerAsset, bOverrideCanInject);
	
	// If running cook commandlet or forced by custom injection logic
	if (IsRunningCookCommandlet() || (bOverrideInjectionPolicy && bOverrideCanInject))
	{
		if (!UExternalDataLayerEngineSubsystem::Get().IsExternalDataLayerAssetRegistered(InExternalDataLayerAsset, InClient))
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(LOCTEXT("CantInjectNotRegisteredExternalDataLayerAsset", "External Data Layer Asset {0} not registered"), FText::FromString(InExternalDataLayerAsset->GetName()));
			}
			return false;
		}
	}
	// If rejected by override
	else if ((bOverrideInjectionPolicy && !bOverrideCanInject))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FText::Format(LOCTEXT("CantInjectOverride", "External Data Layer Asset {0} rejected by custom injection logic"), FText::FromString(InExternalDataLayerAsset->GetName()));
		}
		return false;
	}
	// Only allow injection if active
	else if (!UExternalDataLayerEngineSubsystem::Get().IsExternalDataLayerAssetActive(InExternalDataLayerAsset, InClient))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FText::Format(LOCTEXT("CantInjectNotActiveExternalDataLayerAsset", "External Data Layer Asset {0} not active"), FText::FromString(InExternalDataLayerAsset->GetName()));
		}
		return false;
	}

	return true;
}
#endif

#undef LOCTEXT_NAMESPACE 
