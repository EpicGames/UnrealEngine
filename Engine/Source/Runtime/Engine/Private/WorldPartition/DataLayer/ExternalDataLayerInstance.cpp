// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"

#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerInstance)

#define LOCTEXT_NAMESPACE "ExternalDataLayerInstance"

UWorld* UExternalDataLayerInstance::GetOuterWorld() const
{
#if !WITH_EDITOR
	// Cooked ExternalDataLayerInstance should always be outered to a URuntimeHashExternalStreamingObjectBase
	check(GetTypedOuter<URuntimeHashExternalStreamingObjectBase>());
#endif
	return Super::GetOuterWorld();
}

const UExternalDataLayerAsset* UExternalDataLayerInstance::GetExternalDataLayerAsset() const
{
	return CastChecked<UExternalDataLayerAsset>(GetAsset());
}

#if WITH_EDITOR

FName UExternalDataLayerInstance::MakeName(const UDataLayerAsset* Asset)
{
	check(Asset->IsA<UExternalDataLayerAsset>());
	return FName(FString::Format(TEXT("ExternalDataLayer_{0}"), { FGuid::NewGuid().ToString() }));
}

TSubclassOf<UDataLayerInstanceWithAsset> UExternalDataLayerInstance::GetDataLayerInstanceClass()
{
	return UExternalDataLayerInstance::StaticClass();
}

const TCHAR* UExternalDataLayerInstance::GetDataLayerIconName() const
{
	static constexpr const TCHAR* DataLayerIconName = TEXT("DataLayer.External");
	return DataLayerIconName;
}

void UExternalDataLayerInstance::OnCreated(const UDataLayerAsset* Asset)
{
	check(Asset->IsA<UExternalDataLayerAsset>());
	Super::OnCreated(Asset);

	InitialRuntimeState = EDataLayerRuntimeState::Activated;
}

bool UExternalDataLayerInstance::CanUserAddActors(FText* OutReason) const
{
	if (!Super::CanUserAddActors(OutReason))
	{
		return false;
	}

	if (OutReason)
	{
		*OutReason = LOCTEXT("UserCantAddActorsToExternalDataLayer", "Can't manually assign actors to External Data Layer.");
	}
	return false;
}

bool UExternalDataLayerInstance::CanUserRemoveActors(FText* OutReason) const
{
	if (!Super::CanUserRemoveActors(OutReason))
	{
		return false;
	}

	if (OutReason)
	{
		*OutReason = LOCTEXT("UserCantRemoveActorsFromExternalDataLayer", "Can't manually remove actors from External Data Layer.");
	}
	return false;
}

bool UExternalDataLayerInstance::CanAddActor(AActor* InActor, FText* OutReason) const
{
	if (!Super::CanAddActor(InActor, OutReason))
	{
		return false;
	}

	if (InActor->GetWorld() != GetWorld())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddActorExternalDataLayerInstanceLevelMismatch", "Actor level must match external data layer instance's level.");
		}
		return false;
	}

	if (!InActor->IsMainPackageActor())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantAddActorChildActorNotSupported", "Can't assign child actor to external data layer.");
		}
		return false;
	}

	return true;
}

bool UExternalDataLayerInstance::CanRemoveActor(AActor* InActor, FText* OutReason) const
{
	if (!Super::CanRemoveActor(InActor, OutReason))
	{
		return false;
	}

	// For now, the only way to remove an Actor from an ExternalDataLayerInstance is to delete it.
	if (OutReason)
	{
		*OutReason = LOCTEXT("CantRemoveActorNotSupportedOnExternalDataLayer", "Can't remove external data layer asset.");
	}
	return false;
}

bool UExternalDataLayerInstance::CanHaveChildDataLayerInstance(const UDataLayerInstance* InChildDataLayerInstance) const
{
	if (!Super::CanHaveChildDataLayerInstance(InChildDataLayerInstance))
	{
		return false;
	}

	if (GetDirectOuterWorldDataLayers() != InChildDataLayerInstance->GetDirectOuterWorldDataLayers())
	{
		return false;
	}

	return true;
}

bool UExternalDataLayerInstance::IsReadOnly(FText* OutReason) const
{
	if (Super::IsReadOnly(OutReason))
	{
		return true;
	}

	AWorldDataLayers* PersistentWorldDataLayers = GetDirectOuterWorldDataLayers();
	UPackage* Package = PersistentWorldDataLayers ? PersistentWorldDataLayers->GetExternalPackage() : nullptr;
	if (!Package || Package->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("ExternalDataLayerInstanceReadOnlyNeedSave", "Unsaved external data layer instances are read-only.");
		}
		return true;
	}

	// When creating a Level Instance with actors part of an EDL, skip check of current level (flag will be removed once EDLs inside LevelInstance are fully supported)
	if (!bSkipCheckReadOnlyForSubLevels)
	{
		const UWorld* OwningWorld = PersistentWorldDataLayers ? PersistentWorldDataLayers->GetWorld() : nullptr;
		if (OwningWorld && !OwningWorld->PersistentLevel->IsCurrentLevel())
		{
			if (OutReason)
			{
				*OutReason = LOCTEXT("ExternalDataLayerInstanceReadOnlySubLevel", "External data layer instances are not yet supported for sub-levels.");
			}
			return true;
		}
	}

	return false;
}

bool UExternalDataLayerInstance::PerformAddActor(AActor* InActor) const
{
	if (const UExternalDataLayerAsset* ActorExternalDataLayer = InActor->GetExternalDataLayerAsset())
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Can't move actor %ls Package %ls from EDL %ls.", *GetDataLayerShortName(), *InActor->GetActorNameOrLabel(), *InActor->GetPackage()->GetName(), *ActorExternalDataLayer->GetName());
		return false;
	}

	const UExternalDataLayerAsset* ExternalDataLayer = GetExternalDataLayerAsset();
	check(ExternalDataLayer);
	if (FAssignActorDataLayer::AddDataLayerAsset(InActor, ExternalDataLayer))
	{
		if (UExternalDataLayerManager::GetExternalDataLayerManager(GetOuterWorld())->ValidateOnActorExternalDataLayerAssetChanged(InActor))
		{
			return true;
		}
		FAssignActorDataLayer::RemoveDataLayerAsset(InActor, ExternalDataLayer);
	}
	UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to add actor %ls to EDL %ls.", *GetDataLayerShortName(), *InActor->GetActorNameOrLabel(), *ExternalDataLayer->GetName());
	return false;
}

bool UExternalDataLayerInstance::PerformRemoveActor(AActor* InActor) const
{
	return false;
}

#endif

#undef LOCTEXT_NAMESPACE
