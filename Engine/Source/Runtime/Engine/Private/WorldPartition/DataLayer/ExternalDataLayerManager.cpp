// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"

#include "Algo/Transform.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "UObject/Package.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/Cook/WorldPartitionCookPackage.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EngineUtils.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerManager)

#define LOCTEXT_NAMESPACE "ExternalDataLayerManager"

UExternalDataLayerManager::UExternalDataLayerManager()
	: bIsInitialized(false)
	, bIsRunningGameOrInstancedWorldPartition(false)
{}

void UExternalDataLayerManager::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		Ar << ExternalStreamingObjects;
	}
#endif
}

void UExternalDataLayerManager::Initialize()
{
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	UWorld* OwningWorld = GetOuterUWorldPartition()->GetWorld();

	check(OuterWorld);
	check(OwningWorld);

	// EDL in LevelInstance is not currently supported
	// In this case, don't initialize to make sure it will do nothing.
	const ULevelInstanceSubsystem* LevelInstanceSubsystem = OwningWorld->GetSubsystem<ULevelInstanceSubsystem>();
	if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel) : nullptr)
	{
		return;
	}
	
	const bool bIsInstanced = (OuterWorld != OwningWorld);
	bIsRunningGameOrInstancedWorldPartition = IsRunningGame() || bIsInstanced;
	bIsInitialized = true;

	UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
	ExternalDataLayerEngineSubsystem.OnExternalDataLayerAssetRegistrationStateChanged.AddUObject(this, &UExternalDataLayerManager::OnExternalDataLayerAssetRegistrationStateChanged);
	for (const auto& [ExternalDataLayerAsset, Owners] : ExternalDataLayerEngineSubsystem.GetRegisteredExternalDataLayerAssets())
	{
		UpdateExternalDataLayerInjectionState(ExternalDataLayerAsset);
	}
}

void UExternalDataLayerManager::DeInitialize()
{
	if (!IsInitialized())
	{
		return;
	}

	UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
	ExternalDataLayerEngineSubsystem.OnExternalDataLayerAssetRegistrationStateChanged.RemoveAll(this);
	for (const auto& [ExternalDataLayerAsset, Owners] : ExternalDataLayerEngineSubsystem.GetRegisteredExternalDataLayerAssets())
	{
		if (IsExternalDataLayerInjected(ExternalDataLayerAsset))
		{
			RemoveExternalDataLayer(ExternalDataLayerAsset);
		}
	}

#if WITH_EDITOR
	check(InjectedExternalDataLayerAssets.IsEmpty());
	check(EDLContainerMap.IsEmpty());
	check(EDLWorldDataLayersMap.IsEmpty());
#endif

	bIsInitialized = false;
}

bool UExternalDataLayerManager::CanInjectExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset, FText* OutReason) const
{
	if (!InExternalDataLayerAsset)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantInjectExternalDataLayerAssetInvalidAsset", "Invalid External Data Layer.");
		}
		return false;
	}

	if (GetTypedOuter<UWorld>()->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantInjectExternalDataLayerAssetWorldMustBeSaved", "World must be saved first.");
		}
		return false;
	}

	if (!IsInitialized())
	{
		if (OutReason)
		{
			if (GetTypedOuter<UWorldPartition>()->GetWorld() != GetTypedOuter<UWorld>())
			{
				*OutReason = LOCTEXT("CantInjectExternalDataLayerAssetSubLevelsNotSupported", "External Data Layers are not yet supported for sub-levels.");
			}
			else
			{
				*OutReason = LOCTEXT("CantInjectExternalDataLayerAssetNotSupported", "External Data Layers are not supported.");
			}
		}
		return false;
	}

	if (GetExternalDataLayerInstance(InExternalDataLayerAsset))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CantInjectExternalDataLayerAssetAlreadyAdded", "External Data Layer is already added.");
		}
		return false;
	}

	if (!UExternalDataLayerEngineSubsystem::Get().CanWorldInjectExternalDataLayerAsset(GetWorld(), InExternalDataLayerAsset, OutReason))
	{
		return false;
	}

	return true;
}

bool UExternalDataLayerManager::RegisterExternalStreamingObjectForGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(GetTypedOuter<UWorld>()->IsGameWorld());
#if !WITH_EDITOR
	check(InExternalDataLayerAsset);
	if (ExternalStreamingObjects.Contains(InExternalDataLayerAsset))
	{
		UE_LOGF(LogWorldPartition, Warning, "[EDL: %ls] External streaming object already registered.", *InExternalDataLayerAsset->GetName());
		return false;
	}

	// @todo_ow: FAsyncPackage::CreateLinker doesn't respect load flag (LOAD_Quiet, LOAD_NoWarn).
	// Test first that the package exists to avoid a log error by LoadPackage.
	const FString ExternalStreamingObjectPackagePath = GetExternalStreamingObjectPackagePath(InExternalDataLayerAsset);
	if (!FPackageName::DoesPackageExist(ExternalStreamingObjectPackagePath))
	{
		return false;
	}

	// Find outer world's instancing suffix and use it to create the package for the ExternalStreamingObject
	UPackage* DestPackage = nullptr;
	FLinkerInstancingContext InstancingContext;
	FString SourceWorldPath, RemappedWorldPath;
	if (GetTypedOuter<UWorld>()->GetSoftObjectPathMapping(SourceWorldPath, RemappedWorldPath))
	{
		FString Source = FTopLevelAssetPath(SourceWorldPath).GetPackageName().ToString();
		FString Remapped = FTopLevelAssetPath(RemappedWorldPath).GetPackageName().ToString();
		InstancingContext.AddPackageMapping(FName(Source), FName(Remapped));
		int32 Index = UE::String::FindFirst(Remapped, Source, ESearchCase::IgnoreCase);
		if (Index != INDEX_NONE)
		{
			const FString Suffix = Remapped.RightChop(Index + Source.Len());
			const FString RemappedExternalStreamingObjectPackagePath = ExternalStreamingObjectPackagePath + Suffix;
			DestPackage = CreatePackage(*RemappedExternalStreamingObjectPackagePath);
		}
	}

	UPackage* ExternalStreamingObjectPackage = LoadPackage(DestPackage, *ExternalStreamingObjectPackagePath, LOAD_Quiet | LOAD_NoWarn, nullptr, &InstancingContext);
	if (!ExternalStreamingObjectPackage)
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] No external streaming object found.", *InExternalDataLayerAsset->GetName());
		return false;
	}

	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = Cast<URuntimeHashExternalStreamingObjectBase>((UObject*)FindObjectWithOuter(ExternalStreamingObjectPackage, URuntimeHashExternalStreamingObjectBase::StaticClass()));

	if (!ExternalStreamingObject)
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] No external streaming object found in package %ls.", *InExternalDataLayerAsset->GetName(), *ExternalStreamingObjectPackagePath);
		return false;
	}

	// Do some validation on ExternalStreamingObject's OuterWorld and OwningWorld
	check(ExternalStreamingObject->GetOuterWorld() == GetTypedOuter<UWorld>());
	check(ExternalStreamingObject->GetOwningWorld() == GetOuterUWorldPartition()->GetWorld());

	ExternalStreamingObjects.Emplace(InExternalDataLayerAsset, ExternalStreamingObject);
#endif
	return true;
}

bool UExternalDataLayerManager::UnregisterExternalStreamingObjectForGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(GetTypedOuter<UWorld>()->IsGameWorld());
#if !WITH_EDITOR
	check(InExternalDataLayerAsset);
	if (!ExternalStreamingObjects.Contains(InExternalDataLayerAsset))
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] External streaming object not registered.", *InExternalDataLayerAsset->GetName());
		return false;
	}
	ExternalStreamingObjects.Remove(InExternalDataLayerAsset);
#else
	UWorldPartition* OuterWorldPartition = GetOuterUWorldPartition();
	if ((OuterWorldPartition->InitState == EWorldPartitionInitState::Uninitializing) && IsRunningGameOrInstancedWorldPartition())
	{
		ExternalStreamingObjects.Remove(InExternalDataLayerAsset);
	}
#endif
	return true;
}

static bool IsUndoRedoInProgress()
{
#if WITH_EDITORONLY_DATA
	return GIsTransacting;
#else
	return false;
#endif
}

bool UExternalDataLayerManager::IsExternalDataLayerInjected(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
#if WITH_EDITOR
	if (!GetTypedOuter<UWorld>()->IsGameWorld() || IsRunningGameOrInstancedWorldPartition())
	{
		return InjectedExternalDataLayerAssets.Contains(InExternalDataLayerAsset);
	}
#endif

	if (GetTypedOuter<UWorld>()->IsGameWorld())
	{
		const TObjectPtr<URuntimeHashExternalStreamingObjectBase>* ExternalStreamingObjectPtr = ExternalStreamingObjects.Find(InExternalDataLayerAsset);
		URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = ExternalStreamingObjectPtr ? ExternalStreamingObjectPtr->Get() : nullptr;
		if (ExternalStreamingObject)
		{
			UWorld* InjectedWorld = GetTypedOuter<UWorld>();
			check(InjectedWorld->IsGameWorld());
			check(InjectedWorld == ExternalStreamingObject->GetOuterWorld());

			UWorldPartition* WorldPartition = InjectedWorld->GetWorldPartition();
			if (WorldPartition->IsExternalStreamingObjectInjected(ExternalStreamingObject))
			{
				const AWorldDataLayers* WorldDataLayers = InjectedWorld->GetWorldDataLayers();
				return WorldDataLayers && WorldDataLayers->GetExternalDataLayerInstance(InExternalDataLayerAsset);
			}
		}
	}

	return false;
}

bool UExternalDataLayerManager::InjectExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(IsInitialized());
	check(InExternalDataLayerAsset);

	// When transacting, the ExternalDataLayerInstance part of WorldDataLayers TransientDataLayerInstances is already present.
	UExternalDataLayerInstance* ExistingExternalDataLayerInstance = GetExternalDataLayerInstance(InExternalDataLayerAsset);
	if (ExistingExternalDataLayerInstance && !IsUndoRedoInProgress())
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL %ls] External Data Layer already added.", *InExternalDataLayerAsset->GetName());
		return false;
	}

#if WITH_EDITOR
	if (!GetTypedOuter<UWorld>()->IsGameWorld() || IsRunningGameOrInstancedWorldPartition())
	{
		if (!RegisterExternalDataLayerActorDescContainer(InExternalDataLayerAsset))
		{
			return false;
		}
		// This case is to support re-injecting in -game (in this case, ExternalStreamingObjects won't be empty when re-injecting)
		if (IsRunningGameOrInstancedWorldPartition() && ExternalStreamingObjects.Num() && !InjectIntoGameWorld(InExternalDataLayerAsset))
		{
			return false;
		}
		return true;
	}
#endif
	
	if (GetTypedOuter<UWorld>()->IsGameWorld())
	{
		if (!RegisterExternalStreamingObjectForGameWorld(InExternalDataLayerAsset))
		{
			return false;
		}
		return InjectIntoGameWorld(InExternalDataLayerAsset);
	}

	return true;
}

bool UExternalDataLayerManager::InjectIntoGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(GetTypedOuter<UWorld>()->IsGameWorld());

	TObjectPtr<URuntimeHashExternalStreamingObjectBase>* ExternalStreamingObjectPtr = ExternalStreamingObjects.Find(InExternalDataLayerAsset);
	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = ExternalStreamingObjectPtr ? ExternalStreamingObjectPtr->Get() : nullptr;
	if (!ExternalStreamingObject)
	{
		UE_LOGF(LogWorldPartition, Log, "[EDL: %ls] No external streaming object found. No content will be injected.", *InExternalDataLayerAsset->GetName());
		return false;
	}

	UExternalDataLayerInstance* ExternalDataLayerInstance = ExternalStreamingObject->GetRootExternalDataLayerInstance();
	check(ExternalDataLayerInstance);
	check(ExternalDataLayerInstance->GetOuter() == ExternalStreamingObject);
	check(ExternalDataLayerInstance->GetExternalDataLayerAsset() == InExternalDataLayerAsset);

	UWorld* InjectedWorld = GetTypedOuter<UWorld>();
	check(InjectedWorld->IsGameWorld());
	check(InjectedWorld == ExternalStreamingObject->GetOuterWorld());

	ExternalStreamingObject->OnStreamingObjectLoaded(InjectedWorld);

	UWorldPartition* WorldPartition = InjectedWorld->GetWorldPartition();
	AWorldDataLayers* WorldDataLayers = InjectedWorld->GetWorldDataLayers();
	if (!WorldDataLayers)
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Injection failed : No WorldDataLayers found for World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
		return false;
	}

#if WITH_EDITOR
	// In PIE/-game the registration of the EDL Container registered the ExternalDataLayerInstance of the EDL WorldDataLayer.
	// Before injecting the ExternalDataLayerInstance part of the ExternalStreamingObject, remove this existing ExternalDataLayerInstance.
	if (UExternalDataLayerInstance* ExistingExternalDataLayerInstance = WorldDataLayers->GetExternalDataLayerInstance(InExternalDataLayerAsset))
	{
		AWorldDataLayers* OuterWorldDataLayers = ExistingExternalDataLayerInstance->GetDirectOuterWorldDataLayers();
		check(OuterWorldDataLayers);
		verify(WorldDataLayers->RemoveExternalDataLayerInstance(ExistingExternalDataLayerInstance));
	}
#endif

	if (!WorldDataLayers->AddExternalDataLayerInstance(ExternalDataLayerInstance))
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Injection failed : Adding External Data Layer failed for World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
		return false;
	}

	if (!WorldPartition->InjectExternalStreamingObject(ExternalStreamingObject))
	{
		// Remove from the Data layer Manager so it is not partially injected.
		verify(WorldDataLayers->RemoveExternalDataLayerInstance(ExternalDataLayerInstance));
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Injection failed : Injecting external streaming object failed for World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
		return false;
	}

	UE_LOGF(LogWorldPartition, Log, "[EDL: %ls] Injection succeeded for World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
	return true;
}

bool UExternalDataLayerManager::RemoveFromGameWorld(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(GetTypedOuter<UWorld>()->IsGameWorld());
	UExternalDataLayerInstance* ExternalDataLayerInstance = GetExternalDataLayerInstance(InExternalDataLayerAsset);
	if (!ExternalDataLayerInstance)
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to find External Data Layer Instance.", *InExternalDataLayerAsset->GetName());
		return false;
	}

	bool bSuccess = true;
	TObjectPtr<URuntimeHashExternalStreamingObjectBase>* ExternalStreamingObjectPtr = ExternalStreamingObjects.Find(InExternalDataLayerAsset);
	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = ExternalStreamingObjectPtr ? ExternalStreamingObjectPtr->Get() : nullptr;
	if (!ExternalStreamingObject)
	{
#if WITH_EDITOR
		// When running game or instanced WP, don't generate an error as there's no guarantee that the plugin was registered prior to injection
		if (IsRunningGameOrInstancedWorldPartition())
		{
			return true;
		}
#endif
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to find external streaming object.", *InExternalDataLayerAsset->GetName());
		bSuccess = false;
	}
	else
	{
		UWorld* InjectedWorld = GetTypedOuter<UWorld>();
		check(InjectedWorld->IsGameWorld());
		check(InjectedWorld == ExternalStreamingObject->GetOuterWorld());

		UWorldPartition* WorldPartition = InjectedWorld->GetWorldPartition();
		if (!WorldPartition->RemoveExternalStreamingObject(ExternalStreamingObject))
		{
			UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to remove external streaming object from World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
			bSuccess = false;
		}

		UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
		AWorldDataLayers* WorldDataLayers = DataLayerManager->GetWorldDataLayers();
		if (!WorldDataLayers || !WorldDataLayers->RemoveExternalDataLayerInstance(ExternalDataLayerInstance))
		{
			UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to remove External Data Layer from World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
			bSuccess = false;
		}

		UE_CLOGF(bSuccess, LogWorldPartition, Log, "[EDL: %ls] Successfully removed from World %ls.", *InExternalDataLayerAsset->GetName(), *InjectedWorld->GetName());
	}
	return bSuccess;
}

FString UExternalDataLayerManager::GetExternalDataLayerLevelRootPath(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	check(InExternalDataLayerAsset);
	const UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const UPackage* OuterWorldPackage = OuterWorld->GetPackage();
	const FString OuterWorldPackageName = OuterWorldPackage->GetLoadedPath().GetPackageFName().IsNone() ? OuterWorldPackage->GetName() : OuterWorldPackage->GetLoadedPath().GetPackageName();
	return FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(InExternalDataLayerAsset, OuterWorldPackageName);
}

FString UExternalDataLayerManager::GetExternalStreamingObjectPackagePath(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	return GetExternalDataLayerLevelRootPath(InExternalDataLayerAsset) + TEXT("/") + FWorldPartitionCookPackage::GetGeneratedFolderName() + TEXT("/") + FExternalDataLayerHelper::GetExternalStreamingObjectPackageName(InExternalDataLayerAsset);
}

bool UExternalDataLayerManager::RemoveExternalDataLayer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	bool bSuccess = true;
	check(IsInitialized());
	check(InExternalDataLayerAsset);
	if (GetTypedOuter<UWorld>()->IsGameWorld())
	{
		bSuccess = RemoveFromGameWorld(InExternalDataLayerAsset);
		if (!UnregisterExternalStreamingObjectForGameWorld(InExternalDataLayerAsset))
		{
			bSuccess = false;
		}
		UE_CLOGF(!bSuccess, LogWorldPartition, Error, "[EDL: %ls] Failed to find External Data Layer in the world.", *InExternalDataLayerAsset->GetName());
	}
#if WITH_EDITOR
	if (!GetTypedOuter<UWorld>()->IsGameWorld() || IsRunningGameOrInstancedWorldPartition())
	{
		if (!UnregisterExternalDataLayerActorDescContainer(InExternalDataLayerAsset))
		{
			bSuccess = false;
		}
	}
#endif
	return bSuccess;
}

UDataLayerManager& UExternalDataLayerManager::GetDataLayerManager() const
{
	UWorldPartition* OuterWorldPartition = GetOuterUWorldPartition();
	UDataLayerManager* DataLayerManager = OuterWorldPartition->GetDataLayerManager();
	check(DataLayerManager);
	return *DataLayerManager;
}

UExternalDataLayerInstance* UExternalDataLayerManager::GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	AWorldDataLayers* WorldDataLayers = GetDataLayerManager().GetWorldDataLayers();
	UExternalDataLayerInstance* ExternalDataLayerInstance = WorldDataLayers ? WorldDataLayers->GetExternalDataLayerInstance(InExternalDataLayerAsset) : nullptr;
	return ExternalDataLayerInstance;
}

const UExternalDataLayerInstance* UExternalDataLayerManager::GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	return const_cast<UExternalDataLayerManager*>(this)->GetExternalDataLayerInstance(InExternalDataLayerAsset);
}

void UExternalDataLayerManager::OnExternalDataLayerAssetRegistrationStateChanged(const UExternalDataLayerAsset* InExternalDataLayerAsset, EExternalDataLayerRegistrationState InOldState, EExternalDataLayerRegistrationState InNewState)
{
	// Undo/redo is already handled by UExternalDataLayerManager::PreEditUndo/UExternalDataLayerManager::PostEditUndo
	if (!IsUndoRedoInProgress())
	{
		UpdateExternalDataLayerInjectionState(InExternalDataLayerAsset);
	}
}

void UExternalDataLayerManager::UpdateExternalDataLayerInjectionState(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(InExternalDataLayerAsset);
	check(IsInitialized());

	const bool bShouldInject = UExternalDataLayerEngineSubsystem::Get().CanWorldInjectExternalDataLayerAsset(GetWorld(), InExternalDataLayerAsset);
	const bool bIsInjected = IsExternalDataLayerInjected(InExternalDataLayerAsset);
	if (bShouldInject != bIsInjected)
	{
		if (bShouldInject)
		{
			InjectExternalDataLayer(InExternalDataLayerAsset);
		}
		else
		{
			RemoveExternalDataLayer(InExternalDataLayerAsset);
		}
	}
}

#if WITH_EDITOR

const UExternalDataLayerAsset* UExternalDataLayerManager::GetActorEditorContextCurrentExternalDataLayer() const
{
	for (UDataLayerInstance* DataLayerInstance : GetDataLayerManager().GetActorEditorContextDataLayers())
	{
		if (UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(DataLayerInstance))
		{
			return ExternalDataLayerInstance->GetExternalDataLayerAsset();
		}
	}
	return nullptr;
}

const UExternalDataLayerAsset* UExternalDataLayerManager::GetMatchingExternalDataLayerAssetForObjectPath(const FSoftObjectPath& InObjectPath) const
{
	const FName MountPoint = FPackageName::GetPackageMountPoint(InObjectPath.ToString());

	FExternalDataLayerUID ExternalDataLayerUIDFromPackage;
	FExternalDataLayerHelper::IsExternalDataLayerPath(InObjectPath.ToString(), &ExternalDataLayerUIDFromPackage);

	// Find injected EDL assets with the same mount point than the provided object path
	TArray<const UExternalDataLayerAsset*> Candidates;
	for (const UExternalDataLayerAsset* ExternalDataLayerAsset : InjectedExternalDataLayerAssets)
	{
		// If the provided object path contains a valid EDL UID and we find a matching external data layer asset, use this asset.
		if (ExternalDataLayerUIDFromPackage.IsValid() && (ExternalDataLayerUIDFromPackage == ExternalDataLayerAsset->GetUID()))
		{
			check(FPackageName::GetPackageMountPoint(ExternalDataLayerAsset->GetPackage()->GetName()) == MountPoint);
			return ExternalDataLayerAsset;
		}

		if (FPackageName::GetPackageMountPoint(ExternalDataLayerAsset->GetPackage()->GetName()) == MountPoint)
		{
			Candidates.Add(ExternalDataLayerAsset);
		}
	}

	if (Candidates.Num())
	{
		if (Candidates.Num() > 1)
		{
			// Prefer the one in the context
			const UExternalDataLayerAsset* Current = GetActorEditorContextCurrentExternalDataLayer();
			if (Candidates.Contains(Current))
			{
				return Current;
			}
		}
		// Use first found
		return Candidates[0];
	}

	return nullptr;
}

void UExternalDataLayerManager::PreEditUndo()
{
	Super::PreEditUndo();

	if (!IsInitialized())
	{
		return;
	}

	PreEditUndoExternalDataLayerAssets = InjectedExternalDataLayerAssets;
}

void UExternalDataLayerManager::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsInitialized())
	{
		return;
	}

	// Detect added/removed External Data Layer Assets
	TSet<TObjectPtr<const UExternalDataLayerAsset>> PostEditUndoExternalDataLayerAssets = InjectedExternalDataLayerAssets;
	TSet<TObjectPtr<const UExternalDataLayerAsset>> AddedExternalDataLayerAssets = PostEditUndoExternalDataLayerAssets.Difference(PreEditUndoExternalDataLayerAssets);
	TSet<TObjectPtr<const UExternalDataLayerAsset>> RemovedExternalDataLayerAssets = PreEditUndoExternalDataLayerAssets.Difference(PostEditUndoExternalDataLayerAssets);
	const UExternalDataLayerEngineSubsystem::FRegisteredExternalDataLayerAssetMap& RegisteredExternalDataLayerAssets = UExternalDataLayerEngineSubsystem::Get().GetRegisteredExternalDataLayerAssets();

	// Handle added ones
	for (const TObjectPtr<const UExternalDataLayerAsset>& AddedExternalDataLayerAsset : AddedExternalDataLayerAssets)
	{
		if (RegisteredExternalDataLayerAssets.Contains(AddedExternalDataLayerAsset))
		{
			InjectExternalDataLayer(AddedExternalDataLayerAsset);
		}
	}

	// Handle removed ones
	for (const TObjectPtr<const UExternalDataLayerAsset>& RemovedExternalDataLayerAsset : RemovedExternalDataLayerAssets)
	{
		if (RegisteredExternalDataLayerAssets.Contains(RemovedExternalDataLayerAsset))
		{
			RemoveExternalDataLayer(RemovedExternalDataLayerAsset);
		}
	}

	PreEditUndoExternalDataLayerAssets.Empty();
}

void UExternalDataLayerManager::PrepareEditorGameWorld()
{
	if (IsRunningGameOrInstancedWorldPartition())
	{
		for (const TObjectPtr<const UExternalDataLayerAsset>& ExternalDataLayerAsset : InjectedExternalDataLayerAssets)
		{
			InjectIntoGameWorld(ExternalDataLayerAsset);
		}
	}
}

void UExternalDataLayerManager::ShutdownEditorGameWorld()
{
	// UWorldPartition::Uninitialize() calls ShutdownEditorGameWorld for game worlds, but ExternalDataLayerManager::Deinitialize is also called afterward. 
	// For game world, let UExternalDataLayerManager::Deinitialize do the job.
	if (!GetTypedOuter<UWorld>()->IsGameWorld())
	{
		ExternalStreamingObjects.Empty();
	}
}

UWorldPartitionRuntimeCell* UExternalDataLayerManager::GetCellForCookPackage(const FString& InCookPackageName) const
{
	UWorldPartitionRuntimeCell* FoundCell = nullptr;
	ForEachExternalStreamingObjects([&FoundCell, &InCookPackageName](URuntimeHashExternalStreamingObjectBase* StreamingObject)
	{
		FoundCell = StreamingObject->GetCellForCookPackage(InCookPackageName);
		return !FoundCell;
	});
	return FoundCell;
}

URuntimeHashExternalStreamingObjectBase* UExternalDataLayerManager::GetExternalStreamingObjectForCookPackage(const FString& InCookPackageName) const
{
	URuntimeHashExternalStreamingObjectBase* FoundStreamingObject = nullptr;
	ForEachExternalStreamingObjects([&FoundStreamingObject, &InCookPackageName](URuntimeHashExternalStreamingObjectBase* StreamingObject)
	{
		if (StreamingObject->GetPackageNameToCreate() == InCookPackageName)
		{
			FoundStreamingObject = StreamingObject;
		}
		return !FoundStreamingObject;
	});
	return FoundStreamingObject;
}

UActorDescContainerInstance* UExternalDataLayerManager::RegisterExternalDataLayerActorDescContainer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	check(InExternalDataLayerAsset);
	UWorldPartition* OuterWorldPartition = GetOuterUWorldPartition();
	const UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const UPackage* OuterWorldPackage = OuterWorld->GetPackage();
	const FString OuterWorldPackageName = OuterWorldPackage->GetLoadedPath().GetPackageFName().IsNone() ? OuterWorldPackage->GetName() : OuterWorldPackage->GetLoadedPath().GetPackageName();
	const FString ContainerRootPath = GetExternalDataLayerLevelRootPath(InExternalDataLayerAsset);

	const FSoftObjectPath ExternalDataLayerAssetSoftObjectPath(InExternalDataLayerAsset);
	UActorDescContainerInstance::FInitializeParams InitParams(*ContainerRootPath);
	InitParams.ExternalDataLayerAsset = InExternalDataLayerAsset;
	InitParams.FilterActorDescFunc = [this, InExternalDataLayerAsset, &ExternalDataLayerAssetSoftObjectPath](const FWorldPartitionActorDesc* ActorDesc)
	{
		// Aside from the WorldDataLayer all actor in this containers should be on the ExternalDataLayerAsset
		if ((ActorDesc->GetExternalDataLayerAsset() != ExternalDataLayerAssetSoftObjectPath) && !ActorDesc->GetActorNativeClass()->IsChildOf<AWorldDataLayers>())
		{
			UE_LOGF(LogWorldPartition, Warning, "[EDL %ls] Actor Desc %ls External Data Layer '%ls' doesn't match its container External Data Layer '%ls'.",
				*InExternalDataLayerAsset->GetName(),
				*ActorDesc->GetActorPackage().ToString(),
				*ActorDesc->GetExternalDataLayerAsset().GetAssetPath().ToString(),
				*ExternalDataLayerAssetSoftObjectPath.GetAssetPath().ToString());
			return false;
		}
		return true;
	};

	FWorldPartitionReference WDLReference;
	InitParams.OnInitializedFunc = [InExternalDataLayerAsset, OuterWorld, &WDLReference, OuterWorldPartition, this](UActorDescContainerInstance* ActorDescContainerInstance)
	{
		check(ActorDescContainerInstance->GetExternalDataLayerAsset() == InExternalDataLayerAsset);

		// Find ExternalDataLayerInstance for this container (for container newly created, use FindObject - see GetWorldDataLayers)
		WDLReference = UDataLayerManager::LoadWorldDataLayersActor(ActorDescContainerInstance);
		AWorldDataLayers* WorldDataLayers = WDLReference.IsLoaded() ? CastChecked<AWorldDataLayers>(WDLReference.GetActor()) : GetWorldDataLayers(InExternalDataLayerAsset);
		if (WorldDataLayers)
		{
			WorldDataLayers->OnDataLayerManagerInitialized();
		}
		if (UExternalDataLayerInstance* ExternalDataLayerInstance = WorldDataLayers ? WorldDataLayers->GetExternalDataLayerInstance(InExternalDataLayerAsset) : nullptr)
		{
			// Register the ExternalDataLayerInstance (adds it to the DLManager's transient EDLInstances)
			if (RegisterExternalDataLayerInstance(ExternalDataLayerInstance))
			{
				Modify(false);
				InjectedExternalDataLayerAssets.Add(InExternalDataLayerAsset);
				EDLContainerMap.Add(InExternalDataLayerAsset, ActorDescContainerInstance);
				EDLWorldDataLayersMap.Add(InExternalDataLayerAsset, WDLReference);
			}
		}
	};

	if (UActorDescContainerInstance* ActorDescContainerInstance = OuterWorldPartition->RegisterActorDescContainerInstance(InitParams))
	{
		// Validate that the EDL ActorDescContainer was fully registered
		if (GetExternalDataLayerInstance(InExternalDataLayerAsset))
		{
			return ActorDescContainerInstance;
		}
		WDLReference.Reset();
		OuterWorldPartition->UnregisterActorDescContainerInstance(ActorDescContainerInstance);
	}
	return nullptr;
}

bool UExternalDataLayerManager::RegisterExternalDataLayerInstance(UExternalDataLayerInstance* InExternalDataLayerInstance)
{
	check(InExternalDataLayerInstance);
	const UExternalDataLayerAsset* ExternalDataLayerAsset = InExternalDataLayerInstance->GetExternalDataLayerAsset();
	check(ExternalDataLayerAsset);

	UExternalDataLayerInstance* ExistingExternalDataLayerInstance = GetExternalDataLayerInstance(ExternalDataLayerAsset);
	// When transacting (redo), the ExternalDataLayerInstance part of WorldDataLayers TransientDataLayerInstances is already present.
	ensure(!ExistingExternalDataLayerInstance || IsUndoRedoInProgress());
	if (!ExistingExternalDataLayerInstance)
	{
		AWorldDataLayers* WorldDataLayers = GetDataLayerManager().GetWorldDataLayers();
		if (!WorldDataLayers || !WorldDataLayers->AddExternalDataLayerInstance(InExternalDataLayerInstance))
		{
			UE_LOGF(LogWorldPartition, Error, "[EDL %ls] Failed to register External Data Layer Instance.", *ExternalDataLayerAsset->GetName());
			return false;
		}
	}

	return true;
}

bool UExternalDataLayerManager::UnregisterExternalDataLayerActorDescContainer(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	Modify(false);
	check(!GetTypedOuter<UWorld>()->IsGameWorld() || IsRunningGameOrInstancedWorldPartition());
	check(InExternalDataLayerAsset);

	// Find associated ActorDescContainerInstance for this EDL
	FExternalDataLayerContainerMap::ValueType* ActorDescContainerInstancePtr = EDLContainerMap.Find(InExternalDataLayerAsset);
	UActorDescContainerInstance* ActorDescContainerInstance = ActorDescContainerInstancePtr ? ActorDescContainerInstancePtr->Get() : nullptr;
	ensure(ActorDescContainerInstance || !InjectedExternalDataLayerAssets.Contains(InExternalDataLayerAsset));

	// Remove the EDL from the ActorDescContainerInstance acceleration table
	EDLContainerMap.Remove(InExternalDataLayerAsset);
	EDLWorldDataLayersMap.Remove(InExternalDataLayerAsset);
	InjectedExternalDataLayerAssets.Remove(InExternalDataLayerAsset);

	bool bSuccess = true;
	if (ActorDescContainerInstance)
	{
		// Unregister the ActorDescContainerInstance
		UWorldPartition* WorldPartition = ActorDescContainerInstance->GetOuterWorldPartition();
		if (!WorldPartition || !WorldPartition->UnregisterActorDescContainerInstance(ActorDescContainerInstance))
		{
			UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to unregister the ActorDescContainer of this External Data Layer.", *InExternalDataLayerAsset->GetName());
			bSuccess = false;
		}
	}

	if (!UExternalDataLayerEngineSubsystem::Get().IsExternalDataLayerAssetRegistered(InExternalDataLayerAsset))
	{
		// New unsaved actors from an external data layer are lost/destroyed
		for (TActorIterator<AActor> It(GetTypedOuter<UWorld>()); It; ++It)
		{
			if (AActor* Actor = *It; Actor->GetExternalDataLayerAsset() == InExternalDataLayerAsset)
			{
				// Clear selection before destroying actor (prevents a crash in TypedElement selection)
				GEditor->SelectActor(Actor, false, false);
				GetWorld()->DestroyActor(Actor);
			}
		}
	}

	if (UExternalDataLayerInstance* ExternalDataLayerInstance = GetExternalDataLayerInstance(InExternalDataLayerAsset))
	{
		if (!UnregisterExternalDataLayerInstance(ExternalDataLayerInstance))
		{
			UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to remove External Data Layer.", *InExternalDataLayerAsset->GetName());
			bSuccess = false;
		}
	}
		
	return bSuccess;
}

bool UExternalDataLayerManager::UnregisterExternalDataLayerInstance(UExternalDataLayerInstance* InExternalDataLayerInstance)
{
	check(InExternalDataLayerInstance);
	const UExternalDataLayerAsset* ExternalDataLayerAsset = InExternalDataLayerInstance->GetExternalDataLayerAsset();
	check(ExternalDataLayerAsset);

	// Remove EDLInstance from the WorldDataLayer's EDL transient list
	AWorldDataLayers* WorldDataLayers = GetDataLayerManager().GetWorldDataLayers();
	if (!WorldDataLayers || !WorldDataLayers->RemoveExternalDataLayerInstance(InExternalDataLayerInstance))
	{
		UE_LOGF(LogWorldPartition, Error, "[EDL: %ls] Failed to unregister External Data Layer Instance.", *ExternalDataLayerAsset->GetName());
		return false;
	}
	return true;
}

URuntimeHashExternalStreamingObjectBase* UExternalDataLayerManager::GetExternalStreamingObjectForDataLayerAsset(const UExternalDataLayerAsset* InAsset) const
{
	return ExternalStreamingObjects.FindRef(InAsset);
}

void UExternalDataLayerManager::ForEachExternalStreamingObjects(TFunctionRef<bool(URuntimeHashExternalStreamingObjectBase*)> Func) const
{
	for (auto& [ExternalDataLayerAsset, ExternalStreamingObject] : ExternalStreamingObjects)
	{
		if (!Func(ExternalStreamingObject))
		{
			break;
		}
	}
}

FString UExternalDataLayerManager::GetActorPackageName(const UExternalDataLayerAsset* InExternalDataLayerAsset, const ULevel* InDestinationLevel, const FString& InActorPath) const
{
	const FString ActorPackageName = ULevel::GetActorPackageName(InDestinationLevel->GetPackage(), InDestinationLevel->GetActorPackagingScheme(), InActorPath, InExternalDataLayerAsset);
	return ActorPackageName;
}

bool UExternalDataLayerManager::ValidateOnActorExternalDataLayerAssetChanged(AActor* InActor)
{
	check(InActor);
	check(InActor->IsPackageExternal());
	const UExternalDataLayerAsset* ExternalDataLayerAsset = InActor->GetExternalDataLayerAsset();
	if (ExternalDataLayerAsset)
	{
#if DO_CHECK
		// Validate the corresponding container for this External Data Layer asset
		FExternalDataLayerContainerMap::ValueType ActorDescContainer = EDLContainerMap.FindChecked(ExternalDataLayerAsset);
		check(ActorDescContainer->GetExternalActorPath() == ULevel::GetExternalActorsPath(GetExternalDataLayerLevelRootPath(ExternalDataLayerAsset)));
#endif
		// Validate that the actor package is valid
		const FString ActorPackageName = InActor->GetExternalPackage()->GetName();
		const FString NewActorPackageName = GetActorPackageName(ExternalDataLayerAsset, InActor->GetLevel(), InActor->GetPathName());
		check(NewActorPackageName == ActorPackageName);
		return (NewActorPackageName == ActorPackageName);
	}
	return true;
}

AWorldDataLayers* UExternalDataLayerManager::GetWorldDataLayers(const UExternalDataLayerAsset* InExternalDataLayerAsset, bool bAllowCreate) const
{
	// @todo_ow: Add LevelInstance EDL support (choose the right destination level)
	ULevel* DestinationLevel = GetTypedOuter<UWorld>()->PersistentLevel;
	const FString WorldDataLayersName = FString::Printf(TEXT("%s%s"), *AWorldDataLayers::StaticClass()->GetName(), *ObjectTools::SanitizeObjectName(InExternalDataLayerAsset->GetPathName()));
	const FString WorldDataLayersPathName = DestinationLevel->GetPathName() + TEXT(".") + WorldDataLayersName;
	const FString PackageName = GetActorPackageName(InExternalDataLayerAsset, DestinationLevel, WorldDataLayersPathName);
	UPackage* ActorPackage = FindObject<UPackage>(nullptr, *PackageName);
	AWorldDataLayers* EDLWorldDataLayers = FindObject<AWorldDataLayers>(ActorPackage, *WorldDataLayersName);
	if (IsValid(EDLWorldDataLayers))
	{
		return EDLWorldDataLayers;
	}

	if (bAllowCreate)
	{
		ActorPackage = ULevel::CreateActorPackage(DestinationLevel->GetPackage(), DestinationLevel->GetActorPackagingScheme(), WorldDataLayersPathName, InExternalDataLayerAsset);
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.Name = *WorldDataLayersName;
		SpawnParameters.OverrideLevel = DestinationLevel;
		SpawnParameters.bCreateActorPackage = false;
		SpawnParameters.OverridePackage = ActorPackage;
		EDLWorldDataLayers = AWorldDataLayers::Create(SpawnParameters);
		EDLWorldDataLayers->SetActorLabel(InExternalDataLayerAsset->GetName());
		return EDLWorldDataLayers;
	}
	return nullptr;
}

URuntimeHashExternalStreamingObjectBase* UExternalDataLayerManager::CreateExternalStreamingObjectUsingStreamingGeneration(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	const UExternalDataLayerInstance* ExternalDataLayerInstance = GetExternalDataLayerInstance(InExternalDataLayerAsset);
	if (!ensure(ExternalDataLayerInstance))
	{
		return nullptr;
	}

	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = GetOuterUWorldPartition()->FlushStreamingToExternalStreamingObject();
	if (!ExternalStreamingObject)
	{
		return nullptr;
	}

	ExternalStreamingObjects.Emplace(InExternalDataLayerAsset, ExternalStreamingObject);
	ExternalStreamingObject->ExternalDataLayerAsset = InExternalDataLayerAsset;
	ExternalStreamingObject->SetPackagePathToCreate(GetExternalStreamingObjectPackagePath(InExternalDataLayerAsset));

	// Make a copy of External Data Layer WorldDataLayers DataLayerInstances
	AWorldDataLayers* EDLWorldDataLayers = ExternalDataLayerInstance->GetDirectOuterWorldDataLayers();
	// Remove editor data layers
	AWorldDataLayers* DuplicatedEDLWorldDataLayers = DuplicateObject<AWorldDataLayers>(EDLWorldDataLayers, GetTransientPackage());
	DuplicatedEDLWorldDataLayers->RemoveEditorDataLayers();
	// Outer DataLayerInstances to the ExternalStreamingObject and set its RootExternalDataLayerInstance
	DuplicatedEDLWorldDataLayers->ForEachDataLayerInstance([ExternalStreamingObject](UDataLayerInstance* DataLayerInstance)
	{
		DataLayerInstance->Rename(nullptr, ExternalStreamingObject, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		ExternalStreamingObject->DataLayerInstances.Add(DataLayerInstance);
		if (UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(DataLayerInstance))
		{
			check(!ExternalStreamingObject->RootExternalDataLayerInstance);
			ExternalStreamingObject->RootExternalDataLayerInstance = ExternalDataLayerInstance;
		}
		return true;
	});

	return ExternalStreamingObject;
}

#endif

#undef LOCTEXT_NAMESPACE
