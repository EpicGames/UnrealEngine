// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/ExternalDataLayerUID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalDataLayerUID)

bool FExternalDataLayerUID::IsValid() const
{
	return Value != 0;
}

FString FExternalDataLayerUID::ToString() const
{
	return FString::Printf(TEXT("%X"), Value);
}

#if WITH_EDITOR
FExternalDataLayerUID FExternalDataLayerUID::NewUID()
{
	FExternalDataLayerUID UID;
	UID.Value = GetTypeHash(FGuid::NewGuid());
	return UID;
}

bool FExternalDataLayerUID::Parse(const FString& InUIDString, FExternalDataLayerUID& OutUID)
{
	OutUID.Value = (uint32)FCString::Strtoi(*InUIDString, nullptr, 16);
	return OutUID.IsValid();
}
#endif
