// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldDataLayers.cpp: AWorldDataLayers class implementation
=============================================================================*/

#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "Engine/Level.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/PushModel/PushModel.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldDataLayers)

#if WITH_EDITOR
#include "ActorEditorContext/ScopedActorEditorContextSetExternalDataLayerAsset.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystemInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Interfaces/IPluginManager.h"
#include "Algo/Find.h"
#include "Misc/MessageDialog.h"
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "WorldDataLayers"

static FString JoinDataLayerShortNamesFromInstanceNames(AWorldDataLayers* InWorldDataLayers, const TArray<FName>& InDataLayerInstanceNames)
{
	check(InWorldDataLayers);
	TArray<FString> DataLayerShortNames;
	DataLayerShortNames.Reserve(InDataLayerInstanceNames.Num());
	for (const FName& DataLayerInstanceName : InDataLayerInstanceNames)
	{
		if (const UDataLayerInstance* DataLayerInstance = InWorldDataLayers->GetDataLayerInstance(DataLayerInstanceName))
		{
			DataLayerShortNames.Add(DataLayerInstance->GetDataLayerShortName());
		}
	}
	return FString::Join(DataLayerShortNames, TEXT(","));
}

AWorldDataLayers::AWorldDataLayers(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
#if WITH_EDITORONLY_DATA
	, bUseExternalPackageDataLayerInstances(false)
	, bAllowRuntimeDataLayerEditing(true)
#endif
	, DataLayersStateEpoch(0)
{
	bAlwaysRelevant = true;
	bReplicates = true;
	SetNetDormancy(DORM_Initial);
	bReplayRewindable = true;
}

void AWorldDataLayers::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(AWorldDataLayers, RepLoadedDataLayerNames, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(AWorldDataLayers, RepActiveDataLayerNames, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(AWorldDataLayers, RepEffectiveLoadedDataLayerNames, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(AWorldDataLayers, RepEffectiveActiveDataLayerNames, Params);
}

#define AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(ReplicatedArray, SourceArray) \
	MARK_PROPERTY_DIRTY_FROM_NAME(AWorldDataLayers, ReplicatedArray, this); \
	ReplicatedArray = SourceArray;

bool AWorldDataLayers::CanChangeDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, AWorldDataLayers::ESetDataLayerRuntimeStateError* OutReason) const
{
	if (!InDataLayerInstance->IsRuntime())
	{
		if (OutReason)
		{
			*OutReason = ESetDataLayerRuntimeStateError::NotRuntime;
		}
		return false;
	}

	const ENetMode NetMode = GetNetMode();

	if (InDataLayerInstance->IsClientOnly())
	{
		if (NetMode != NM_Standalone && NetMode != NM_Client)
		{
			if (OutReason)
			{
				*OutReason = ESetDataLayerRuntimeStateError::ClientOnlyFromServer;
			}
			return false;
		}
	}
	else if (InDataLayerInstance->IsServerOnly())
	{
		if (NetMode == NM_Client)
		{
			if (OutReason)
			{
				*OutReason = ESetDataLayerRuntimeStateError::ServerOnlyFromClient;
			}
			return false;
		}
	}
	else if (NetMode == NM_Client)
	{
		if (OutReason)
		{
			*OutReason = ESetDataLayerRuntimeStateError::AuthoritativeFromClient;
		}
		return false;
	}

	return true;
}

void AWorldDataLayers::InitializeDataLayerRuntimeStates()
{
	if (IsExternalDataLayerWorldDataLayers())
	{
		return;
	}

	check(ActiveDataLayerNames.IsEmpty() && LoadedDataLayerNames.IsEmpty());

	if (GetWorld()->IsGameWorld())
	{
		TArray<UDataLayerInstance*> RuntimeDataLayerInstances;
		RuntimeDataLayerInstances.Reserve(GetDataLayerInstances().Num());

		ForEachDataLayerInstance([this, &RuntimeDataLayerInstances](UDataLayerInstance* DataLayerInstance)
		{
			if (CanChangeDataLayerRuntimeState(DataLayerInstance))
			{
				const bool bDataLayerClientOnly = DataLayerInstance->IsClientOnly();
				const bool bDataLayerServerOnly = DataLayerInstance->IsServerOnly();
				const bool bIsLocalDataLayer = bDataLayerClientOnly || bDataLayerServerOnly;
				TSet<FName>& TargetLoadedDataLayerNames = bIsLocalDataLayer ? LocalLoadedDataLayerNames : LoadedDataLayerNames;
				TSet<FName>& TargetActiveDataLayerNames = bIsLocalDataLayer ? LocalActiveDataLayerNames : ActiveDataLayerNames;

				if (DataLayerInstance->IsRuntime())
				{
					if (DataLayerInstance->GetInitialRuntimeState() == EDataLayerRuntimeState::Loaded)
					{
						TargetLoadedDataLayerNames.Add(DataLayerInstance->GetFName());
					}
					else if (DataLayerInstance->GetInitialRuntimeState() == EDataLayerRuntimeState::Activated)
					{
						TargetActiveDataLayerNames.Add(DataLayerInstance->GetFName());
					}
					RuntimeDataLayerInstances.Add(DataLayerInstance);
				}
			}
			return true;
		});

		FlushNetDormancy();
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepActiveDataLayerNames, ActiveDataLayerNames.Array());
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepLoadedDataLayerNames, LoadedDataLayerNames.Array());

		const bool bNotifyChange = false;
		for (UDataLayerInstance* DataLayerInstance : RuntimeDataLayerInstances)
		{
			ResolveEffectiveRuntimeState(DataLayerInstance, bNotifyChange);
		}

		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveActiveDataLayerNames, EffectiveStates.GetReplicatedEffectiveActiveDataLayerNames().Array());
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveLoadedDataLayerNames, EffectiveStates.GetReplicatedEffectiveLoadedDataLayerNames().Array());

		UE_CLOGF(RepEffectiveActiveDataLayerNames.Num() || RepEffectiveLoadedDataLayerNames.Num(), LogWorldPartition, Log, "Initial Data Layer Effective States Activated(%ls) Loaded(%ls)", *JoinDataLayerShortNamesFromInstanceNames(this, RepEffectiveActiveDataLayerNames), *JoinDataLayerShortNamesFromInstanceNames(this, RepEffectiveLoadedDataLayerNames));
	}
}

void AWorldDataLayers::ResetDataLayerRuntimeStates(bool bFlushNet)
{
	ActiveDataLayerNames.Reset();
	LoadedDataLayerNames.Reset();
	LocalActiveDataLayerNames.Reset();
	LocalLoadedDataLayerNames.Reset();

	EffectiveStates.Reset();
	
	if (bFlushNet)
	{
		FlushNetDormancy();
		static const TArray<FName> Empty;		
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepActiveDataLayerNames, Empty);
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepLoadedDataLayerNames, Empty);
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveActiveDataLayerNames, Empty);
		AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveLoadedDataLayerNames, Empty);
	}
}

bool AWorldDataLayers::SetDataLayerRuntimeState(const UDataLayerInstance* InDataLayerInstance, EDataLayerRuntimeState InState, bool bInIsRecursive)
{
	if (!InDataLayerInstance)
	{
		return false;
	}

	const EDataLayerRuntimeState CurrentState = GetDataLayerRuntimeStateByName(InDataLayerInstance->GetFName());

	ESetDataLayerRuntimeStateError Reason;
	if (!CanChangeDataLayerRuntimeState(InDataLayerInstance, &Reason))
	{
		switch (Reason)
		{
		case ESetDataLayerRuntimeStateError::NotRuntime:
			UE_LOGF(LogWorldPartition, Verbose, "Non-Runtime Data Layer '%ls' state change was ignored",
				*InDataLayerInstance->GetDataLayerShortName());
			break;
		case ESetDataLayerRuntimeStateError::ClientOnlyFromServer:
			UE_LOGF(LogWorldPartition, Verbose, "Client Only Data Layer '%ls' state change was ignored: %ls -> %ls",
				*InDataLayerInstance->GetDataLayerShortName(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)CurrentState).ToString(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)InState).ToString());
			break;
		case ESetDataLayerRuntimeStateError::ServerOnlyFromClient:
			UE_LOGF(LogWorldPartition, Verbose, "Server Only Data Layer '%ls' state change was ignored: %ls -> %ls",
				*InDataLayerInstance->GetDataLayerShortName(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)CurrentState).ToString(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)InState).ToString());
			break;
		case ESetDataLayerRuntimeStateError::AuthoritativeFromClient:
				UE_LOGF(LogWorldPartition, Verbose, "Data Layer '%ls' state change was ignored on client: %ls -> %ls",
					*InDataLayerInstance->GetDataLayerShortName(),
					*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)CurrentState).ToString(),
					*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)InState).ToString());
				break;
		}
		return false;
	}

	if (CurrentState != InState)
	{
		const ENetMode NetMode = GetNetMode();
		const bool bDataLayerClientOnly = InDataLayerInstance->IsClientOnly();
		const bool bDataLayerServerOnly = InDataLayerInstance->IsServerOnly();
		const bool bIsLocalDataLayer = bDataLayerClientOnly || bDataLayerServerOnly;
		TSet<FName>& TargetLoadedDataLayerNames = bIsLocalDataLayer ? LocalLoadedDataLayerNames : LoadedDataLayerNames;
		TSet<FName>& TargetActiveDataLayerNames = bIsLocalDataLayer ? LocalActiveDataLayerNames : ActiveDataLayerNames;

		TargetLoadedDataLayerNames.Remove(InDataLayerInstance->GetFName());
		TargetActiveDataLayerNames.Remove(InDataLayerInstance->GetFName());

		if (InState == EDataLayerRuntimeState::Loaded)
		{
			TargetLoadedDataLayerNames.Add(InDataLayerInstance->GetFName());
		}
		else if (InState == EDataLayerRuntimeState::Activated)
		{
			TargetActiveDataLayerNames.Add(InDataLayerInstance->GetFName());
		}

		// Update replicated properties
		if (!bIsLocalDataLayer && (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer || NetMode == NM_Standalone))
		{
			FlushNetDormancy();
			AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepActiveDataLayerNames, ActiveDataLayerNames.Array());
			AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepLoadedDataLayerNames, LoadedDataLayerNames.Array());
		}

		++DataLayersStateEpoch;

#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		UE_LOGF(LogWorldPartition, Log, "Data Layer Instance '%ls' state changed: %ls -> %ls",
			*InDataLayerInstance->GetDataLayerShortName(),
			*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)CurrentState).ToString(),
			*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)InState).ToString());
#endif // !UE_BUILD_TEST && !UE_BUILD_SHIPPING

		CSV_EVENT_GLOBAL(TEXT("DataLayer-%s-%s"), *InDataLayerInstance->GetDataLayerShortName(), *StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)InState).ToString());

		ResolveEffectiveRuntimeState(InDataLayerInstance);
	}

	if (bInIsRecursive)
	{
		InDataLayerInstance->ForEachChild([this, InState, bInIsRecursive](const UDataLayerInstance* Child)
		{
			SetDataLayerRuntimeState(Child, InState, bInIsRecursive);
			return true;
		});
	}

	return true;
}

void AWorldDataLayers::OnDataLayerRuntimeStateChanged_Implementation(const UDataLayerInstance* InDataLayer, EDataLayerRuntimeState InState)
{
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(this))
	{
		DataLayerManager->BroadcastOnDataLayerInstanceRuntimeStateChanged(InDataLayer, InState);
	}
}

void AWorldDataLayers::OnRep_ActiveDataLayerNames()
{
	++DataLayersStateEpoch;
	ActiveDataLayerNames.Reset();
	ActiveDataLayerNames.Append(RepActiveDataLayerNames);
}

void AWorldDataLayers::OnRep_LoadedDataLayerNames()
{
	++DataLayersStateEpoch;
	LoadedDataLayerNames.Reset();
	LoadedDataLayerNames.Append(RepLoadedDataLayerNames);
}

EDataLayerRuntimeState AWorldDataLayers::GetDataLayerRuntimeStateByName(FName InDataLayerName) const
{
	if (ActiveDataLayerNames.Contains(InDataLayerName))
	{
		check(!LoadedDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Activated;
	}
	else if (LoadedDataLayerNames.Contains(InDataLayerName))
	{
		check(!ActiveDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Loaded;
	}
	else if (LocalActiveDataLayerNames.Contains(InDataLayerName))
	{
		check(!LocalLoadedDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Activated;
	}
	else if (LocalLoadedDataLayerNames.Contains(InDataLayerName))
	{
		check(!LocalActiveDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Loaded;
	}

	return EDataLayerRuntimeState::Unloaded;
}

void AWorldDataLayers::OnRep_EffectiveActiveDataLayerNames()
{
	++DataLayersStateEpoch;
	// TSet do not support replication so we replicate an array and update the set here
	EffectiveStates.SetReplicatedEffectiveActiveDataLayerNames(RepEffectiveActiveDataLayerNames);
}

void AWorldDataLayers::OnRep_EffectiveLoadedDataLayerNames()
{
	++DataLayersStateEpoch;
	// TSet do not support replication so we replicate an array and update the set here
	EffectiveStates.SetReplicatedEffectiveLoadedDataLayerNames(RepEffectiveLoadedDataLayerNames);
}

EDataLayerRuntimeState AWorldDataLayers::GetDataLayerEffectiveRuntimeStateByName(FName InDataLayerName) const
{
	return EffectiveStates.GetDataLayerEffectiveRuntimeStateByName(InDataLayerName);
}

const FWorldDataLayersEffectiveStates& AWorldDataLayers::GetEffectiveStates() const
{
	check(IsInGameThread());
	return EffectiveStates;
}

const TSet<FName>& AWorldDataLayers::GetEffectiveActiveDataLayerNames() const
{
	return EffectiveStates.GetAllEffectiveActiveDataLayerNames();
}

const TSet<FName>& AWorldDataLayers::GetEffectiveLoadedDataLayerNames() const
{
	return EffectiveStates.GetAllEffectiveLoadedDataLayerNames();
}

void AWorldDataLayers::ResolveEffectiveRuntimeState(const UDataLayerInstance* InDataLayerInstance, bool bInNotifyChange)
{
	check(InDataLayerInstance);
	const ENetMode NetMode = GetNetMode();
	const bool bDataLayerClientOnly = InDataLayerInstance->IsClientOnly();
	const bool bDataLayerServerOnly = InDataLayerInstance->IsServerOnly();
	const bool bIsLocalDataLayer = (bDataLayerClientOnly || bDataLayerServerOnly);
	const FName DataLayerName = InDataLayerInstance->GetFName();
	EDataLayerRuntimeState NewEffectiveRuntimeState = GetDataLayerRuntimeStateByName(DataLayerName);
	const UDataLayerInstance* Parent = InDataLayerInstance->GetParent();

	while (Parent && (NewEffectiveRuntimeState != EDataLayerRuntimeState::Unloaded))
	{
		if (Parent->IsRuntime())
		{
			// Apply min logic with parent DataLayers
			NewEffectiveRuntimeState = (EDataLayerRuntimeState)FMath::Min((int32)NewEffectiveRuntimeState, (int32)GetDataLayerRuntimeStateByName(Parent->GetFName()));
		}
		Parent = Parent->GetParent();
	}

	EDataLayerRuntimeState OldEffectiveRuntimeState;
	if (EffectiveStates.SetDataLayerEffectiveRuntimeState(DataLayerName, bIsLocalDataLayer, NewEffectiveRuntimeState, OldEffectiveRuntimeState))
	{
		// Update Replicated Properties
		if (!bIsLocalDataLayer && (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer || NetMode == NM_Standalone))
		{
			FlushNetDormancy();
			AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveActiveDataLayerNames, EffectiveStates.GetReplicatedEffectiveActiveDataLayerNames().Array());
			AWORLDDATALAYERS_UPDATE_REPLICATED_DATALAYERS(RepEffectiveLoadedDataLayerNames, EffectiveStates.GetReplicatedEffectiveLoadedDataLayerNames().Array());
		}

		++DataLayersStateEpoch;

		if (bInNotifyChange)
		{
			UE_LOGF(LogWorldPartition, Log, "Data Layer Instance '%ls' effective state changed: %ls -> %ls",
				*InDataLayerInstance->GetDataLayerShortName(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)OldEffectiveRuntimeState).ToString(),
				*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)NewEffectiveRuntimeState).ToString());

			OnDataLayerRuntimeStateChanged(InDataLayerInstance, NewEffectiveRuntimeState);
		}

		InDataLayerInstance->ForEachChild([this](const UDataLayerInstance* Child)
		{
			ResolveEffectiveRuntimeState(Child);
			return true;
		});
	}
}

static TSet<TObjectPtr<UDataLayerInstance>>& GetDataLayerInstancesFromProvider(UDataLayerInstance* DataLayerInstance)
{
	check(DataLayerInstance);
#if WITH_EDITOR
	check(DataLayerInstance->GetRootExternalDataLayerInstance() == DataLayerInstance);
#endif
	IDataLayerInstanceProvider* DataLayerInstanceProvider = DataLayerInstance->GetImplementingOuter<IDataLayerInstanceProvider>();
	if (ensure(DataLayerInstanceProvider))
	{
		return DataLayerInstanceProvider->GetDataLayerInstances();
	}

	static TSet<TObjectPtr<UDataLayerInstance>> Empty;
	return Empty;
}

bool AWorldDataLayers::AddExternalDataLayerInstance(UExternalDataLayerInstance* ExternalDataLayerInstance)
{
#if WITH_EDITOR
	Modify(/*bDirty*/false);
#endif
	check(!IsExternalDataLayerWorldDataLayers());
	if (!TransientDataLayerInstances.Contains(ExternalDataLayerInstance))
	{
		TransientDataLayerInstances.Add(ExternalDataLayerInstance);
		for (UDataLayerInstance* DataLayerInstance : GetDataLayerInstancesFromProvider(ExternalDataLayerInstance))
		{
			SetDataLayerRuntimeState(DataLayerInstance, DataLayerInstance->GetInitialRuntimeState());
#if !WITH_EDITOR
			UpdateAccelerationTable(DataLayerInstance, true);
#endif
		}
		return true;
	}
	
	return false;
}

bool AWorldDataLayers::RemoveExternalDataLayerInstance(UExternalDataLayerInstance* ExternalDataLayerInstance)
{
#if WITH_EDITOR
	Modify(/*bDirty*/false);
#endif
	check(!IsExternalDataLayerWorldDataLayers());
	uint32 Index = TransientDataLayerInstances.IndexOfByKey(ExternalDataLayerInstance);
	if (Index != INDEX_NONE)
	{
		for (UDataLayerInstance* DataLayerInstance : GetDataLayerInstancesFromProvider(ExternalDataLayerInstance))
		{
			SetDataLayerRuntimeState(DataLayerInstance, EDataLayerRuntimeState::Unloaded);
#if !WITH_EDITOR
			UpdateAccelerationTable(DataLayerInstance, false);
#endif
		}
		TransientDataLayerInstances.RemoveAtSwap(Index);
		return true;
	}

	return false;
}

void AWorldDataLayers::DumpDataLayerRecursively(const UDataLayerInstance* DataLayer, FString Prefix, FOutputDevice& OutputDevice) const
{
	auto GetDataLayerRuntimeStateString = [this](const UDataLayerInstance* DataLayer)
	{
		if (DataLayer->IsRuntime())
		{
			if (!DataLayer->GetWorld()->IsGameWorld())
			{
				return FString::Printf(TEXT("(Initial State = %s)"), GetDataLayerRuntimeStateName(DataLayer->GetInitialRuntimeState()));
			}
			else
			{
				return FString::Printf(TEXT("(Effective State = %s | Target State = %s)"),
					GetDataLayerRuntimeStateName(GetDataLayerEffectiveRuntimeStateByName(DataLayer->GetFName())),
					GetDataLayerRuntimeStateName(GetDataLayerRuntimeStateByName(DataLayer->GetFName()))
				);
			}
		}
		return FString();
	};

	OutputDevice.Logf(TEXT(" %s%s%s %s"),
		*Prefix,
		(DataLayer->GetChildren().IsEmpty() && DataLayer->GetParent()) ? TEXT("") : TEXT("[+]"),
		*DataLayer->GetDataLayerShortName(),
		*GetDataLayerRuntimeStateString(DataLayer));

	DataLayer->ForEachChild([this, &Prefix, &OutputDevice](const UDataLayerInstance* Child)
	{
		DumpDataLayerRecursively(Child, Prefix + TEXT(" | "), OutputDevice);
		return true;
	});
}

void AWorldDataLayers::DumpDataLayers(FOutputDevice& OutputDevice) const
{
	OutputDevice.Logf(TEXT("===================================================="));
	OutputDevice.Logf(TEXT(" Data Layers for %s"), *GetName());
	OutputDevice.Logf(TEXT("===================================================="));
	OutputDevice.Logf(TEXT(""));

	if (GetWorld()->IsGameWorld())
	{
		auto DumpDataLayersRuntimeState = [this, &OutputDevice](const TCHAR* InStateName, const TSet<FName>& InDataLayerInstanceNames)
		{
			if (InDataLayerInstanceNames.Num())
			{
				OutputDevice.Logf(TEXT(" - %s Data Layers:"), InStateName);
				for (const FName& DataLayerInstanceName : InDataLayerInstanceNames)
				{
					if (const UDataLayerInstance* DataLayerInstance = GetDataLayerInstance(DataLayerInstanceName))
					{
						OutputDevice.Logf(TEXT("    - %s"), *DataLayerInstance->GetDataLayerShortName());
					}
				}
			}
		};

		if (EffectiveStates.GetAllEffectiveLoadedDataLayerNames().Num() || EffectiveStates.GetAllEffectiveActiveDataLayerNames().Num())
		{
			OutputDevice.Logf(TEXT("----------------------------------------------------"));
			OutputDevice.Logf(TEXT(" Data Layers Runtime States"));
			DumpDataLayersRuntimeState(TEXT("Loaded"), EffectiveStates.GetAllEffectiveLoadedDataLayerNames());
			DumpDataLayersRuntimeState(TEXT("Active"), EffectiveStates.GetAllEffectiveActiveDataLayerNames());
			OutputDevice.Logf(TEXT("----------------------------------------------------"));
			OutputDevice.Logf(TEXT(""));
		}
	}
	
	OutputDevice.Logf(TEXT("----------------------------------------------------"));
	OutputDevice.Logf(TEXT(" Data Layers Hierarchy"));
	ForEachDataLayerInstance([this, &OutputDevice](UDataLayerInstance* DataLayerInstance)
	{
		if (!DataLayerInstance->GetParent())
		{
			DumpDataLayerRecursively(DataLayerInstance, TEXT(""), OutputDevice);
		}
		return true;
	});
	OutputDevice.Logf(TEXT("----------------------------------------------------"));
}

#if WITH_EDITOR
TUniquePtr<class FWorldPartitionActorDesc> AWorldDataLayers::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FWorldDataLayersActorDesc());
}

void AWorldDataLayers::OnLoadedActorRemovedFromLevel()
{
	Super::OnLoadedActorRemovedFromLevel();

	if (IsUsingExternalPackageDataLayerInstances())
	{
		// Validation will sometimes load the world without invoking the full initialization of its systems so
		// perform a minimal initialization on ExternalPackage DataLayerInstances so that we can iterate over them
		InitializeExternalPackageDataLayerInstances();
	}
}

AWorldDataLayers* AWorldDataLayers::Create(UWorld* World, FName InWorldDataLayerName)
{
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name =  InWorldDataLayerName.IsNone() ? AWorldDataLayers::StaticClass()->GetFName() : InWorldDataLayerName;
	SpawnParameters.OverrideLevel = World->PersistentLevel;
	return Create(SpawnParameters);
}

AWorldDataLayers* AWorldDataLayers::Create(const FActorSpawnParameters& SpawnParameters)
{
	check(SpawnParameters.Name != NAME_None);
	check(SpawnParameters.NameMode == FActorSpawnParameters::ESpawnActorNameMode::Required_Fatal);

	AWorldDataLayers* WorldDataLayers = nullptr;

	if (UObject* ExistingObject = StaticFindObject(nullptr, SpawnParameters.OverrideLevel, *SpawnParameters.Name.ToString()))
	{
		WorldDataLayers = CastChecked<AWorldDataLayers>(ExistingObject);
		if (!IsValidChecked(WorldDataLayers))
		{
			// Handle the case where the actor already exists, but it's pending kill
			WorldDataLayers->Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
			WorldDataLayers = nullptr;
		}
	}

	if (!WorldDataLayers)
	{
		// Make sure there's no context while creating the AWorldDataLayers (avoids generating any warnings while spawning)
		FScopedActorEditorContextSetExternalDataLayerAsset ScopeDisableCurrentEDL(nullptr);
		FScopedOverrideSpawningLevelMountPointObject ScopeDisableOverrideEDL(nullptr);
		UWorld* World = SpawnParameters.OverrideLevel->GetWorld();
		WorldDataLayers = World->SpawnActor<AWorldDataLayers>(AWorldDataLayers::StaticClass(), SpawnParameters);
	}
	else
	{
		UE_LOGF(LogWorldPartition, Error, "Failed to create WorldDataLayers Actor. There is already a WorldDataLayer Actor named \"%ls\" ", *SpawnParameters.Name.ToString())
	}

	check(WorldDataLayers);

	return WorldDataLayers;
}

TArray<FName> AWorldDataLayers::GetDataLayerInstanceNames(const TArray<const UDataLayerAsset*>& InDataLayersAssets) const
{
	TArray<FName> OutDataLayerNames;
	OutDataLayerNames.Reserve(InDataLayersAssets.Num());

	for (const UDataLayerInstance* DataLayerInstance : GetDataLayerInstances(InDataLayersAssets))
	{
		OutDataLayerNames.Add(DataLayerInstance->GetFName());
	}

	return OutDataLayerNames;
}

TArray<const UDataLayerInstance*> AWorldDataLayers::GetDataLayerInstances(const TArray<const UDataLayerAsset*>& InDataLayersAssets) const
{
	TArray<const UDataLayerInstance*> OutDataLayers;
	OutDataLayers.Reserve(InDataLayersAssets.Num());

	for (const UDataLayerAsset* DataLayerAsset : InDataLayersAssets)
	{
		if (const UDataLayerInstance* DataLayerObject = GetDataLayerInstance(DataLayerAsset))
		{
			OutDataLayers.AddUnique(DataLayerObject);
		}
	}

	return OutDataLayers;
}

bool AWorldDataLayers::IsEmpty() const
{
	return GetDataLayerInstances().IsEmpty() && TransientDataLayerInstances.IsEmpty();
}

void AWorldDataLayers::AddDataLayerInstance(UDataLayerInstance* InDataLayerInstance)
{
	check(InDataLayerInstance);
	TSet<TObjectPtr<UDataLayerInstance>>& UsedDataLayerInstances = GetDataLayerInstances();
	// Only dirty actor when not using external package
	const bool bDirty = !bUseExternalPackageDataLayerInstances;
	Modify(bDirty);
	check(GetLevel());
	check(GetLevel()->IsUsingExternalObjects());
	check(!UsedDataLayerInstances.Contains(InDataLayerInstance));
	UsedDataLayerInstances.Add(InDataLayerInstance);
	if (InDataLayerInstance->IsPackageExternal())
	{
		InDataLayerInstance->MarkPackageDirty();
	}
	if (UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(InDataLayerInstance))
	{
		check(TransientDataLayerInstances.IsEmpty());
		check(!RootExternalDataLayerInstance);
		RootExternalDataLayerInstance = ExternalDataLayerInstance;
	}
}

int32 AWorldDataLayers::RemoveDataLayers(const TArray<UDataLayerInstance*>& InDataLayerInstances, bool bInResolveActorDescContainers)
{
	int32 RemovedCount = 0;

	TSet<TObjectPtr<UDataLayerInstance>>& UsedDataLayerInstances = GetDataLayerInstances();
	for (UDataLayerInstance* DataLayerInstance : InDataLayerInstances)
	{
		if (UsedDataLayerInstances.Contains(DataLayerInstance))
		{
			// Only dirty actor when not using external package
			const bool bDirty = !bUseExternalPackageDataLayerInstances;
			Modify(bDirty);
			DataLayerInstance->Modify();
			DataLayerInstance->OnRemovedFromWorldDataLayers();
			UsedDataLayerInstances.Remove(DataLayerInstance);
			if (DataLayerInstance->IsPackageExternal())
			{
				DataLayerInstance->MarkAsGarbage();
			}
			RemovedCount++;
		}
	}

	if (RemovedCount > 0)
	{
		if (bInResolveActorDescContainers)
		{
			ResolveActorDescContainers();
		}
	}

	return RemovedCount;
}

bool AWorldDataLayers::RemoveDataLayer(const UDataLayerInstance* InDataLayerInstance, bool bInResolveActorDescContainers)
{
	return RemoveDataLayers({ const_cast<UDataLayerInstance*>(InDataLayerInstance) }, bInResolveActorDescContainers) > 0;
}

void AWorldDataLayers::SetAllowRuntimeDataLayerEditing(bool bInAllowRuntimeDataLayerEditing)
{
	if (bAllowRuntimeDataLayerEditing != bInAllowRuntimeDataLayerEditing)
	{
		Modify();
		bAllowRuntimeDataLayerEditing = bInAllowRuntimeDataLayerEditing;
	}
}

bool AWorldDataLayers::IsActorEditorContextCurrentColorized(const UDataLayerInstance* InDataLayerInstance) const
{
	return InDataLayerInstance && !CurrentDataLayers.CurrentColorizedDataLayerInstanceName.IsNone() && (InDataLayerInstance->GetFName() == CurrentDataLayers.CurrentColorizedDataLayerInstanceName);
}

bool AWorldDataLayers::IsInActorEditorContext(const UDataLayerInstance* InDataLayerInstance) const
{
	for (const FName& DataLayerInstanceName : CurrentDataLayers.DataLayerInstanceNames)
	{
		const UDataLayerInstance* DataLayerInstance = GetDataLayerInstance(DataLayerInstanceName);
		if (DataLayerInstance && (DataLayerInstance == InDataLayerInstance) && !DataLayerInstance->IsReadOnly())
		{
			return true;
		}
	}

	if (const UDataLayerInstance* ExternalDataLayerInstance = GetDataLayerInstance(CurrentDataLayers.ExternalDataLayerName))
	{
		check(ExternalDataLayerInstance->IsA<UExternalDataLayerInstance>());
		if (ExternalDataLayerInstance && ExternalDataLayerInstance == InDataLayerInstance)
		{
			return true;
		}
	}

	return false;
}

void AWorldDataLayers::UpdateCurrentColorizedDataLayerInstance()
{
	Modify(/*bDirty*/false);
	CurrentDataLayers.CurrentColorizedDataLayerInstanceName = NAME_None;
	TArray<UDataLayerInstance*> ContextDataLayerInstances = GetActorEditorContextDataLayers();
	if (ContextDataLayerInstances.Num() == 1)
	{
		CurrentDataLayers.CurrentColorizedDataLayerInstanceName = ContextDataLayerInstances[0]->GetFName();
	}
	else if (ContextDataLayerInstances.Num() == 2)
	{
		for (UDataLayerInstance* DataLayerInstance : ContextDataLayerInstances)
		{
			const UExternalDataLayerInstance* ExternalDataLayerInstance = DataLayerInstance->GetRootExternalDataLayerInstance();
			if (ExternalDataLayerInstance && (ExternalDataLayerInstance != DataLayerInstance) && ContextDataLayerInstances.Contains(ExternalDataLayerInstance))
			{
				CurrentDataLayers.CurrentColorizedDataLayerInstanceName = DataLayerInstance->GetFName();
				break;
			}
		}
	}
}

bool AWorldDataLayers::AddToActorEditorContext(UDataLayerInstance* InDataLayerInstance)
{
	ON_SCOPE_EXIT { UpdateCurrentColorizedDataLayerInstance(); };

	check(InDataLayerInstance->CanBeInActorEditorContext());
	check(ContainsDataLayer(InDataLayerInstance));
	
	bool bSuccess = false;
	if (UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(InDataLayerInstance))
	{
		Modify(/*bDirty*/false);
		CurrentDataLayers.ExternalDataLayerName = ExternalDataLayerInstance->GetFName();

		// Adding an EDL Instance replaces the existing (if any) and removes all DataLayerInstances with a different root EDL Instance
		TArray<FName> ToRemove;
		for (const FName& DataLayerInstanceName : CurrentDataLayers.DataLayerInstanceNames)
		{
			const UDataLayerInstance* DataLayerInstance = GetDataLayerInstance(DataLayerInstanceName);
			const UExternalDataLayerInstance* RootEDLInstance = DataLayerInstance ? DataLayerInstance->GetRootExternalDataLayerInstance() : nullptr;
			if (!DataLayerInstance || (RootEDLInstance && RootEDLInstance != ExternalDataLayerInstance))
			{
				ToRemove.Add(DataLayerInstanceName);
			}
		}
		for (const FName& DataLayerInstanceName : ToRemove)
		{
			CurrentDataLayers.DataLayerInstanceNames.Remove(DataLayerInstanceName);
		}

		// EDL has priority over Content Bundle
		IContentBundleEditorSubsystemInterface::Get()->DeactivateCurrentContentBundleEditing();

		bSuccess = true;
	}
	else if (!CurrentDataLayers.DataLayerInstanceNames.Contains(InDataLayerInstance->GetFName()))
	{
		Modify(/*bDirty*/false);
		CurrentDataLayers.DataLayerInstanceNames.Add(InDataLayerInstance->GetFName());
		bSuccess = true;

		// Adding a Data Layer with a RootExternalDataLayerInstance will set this Root EDL Instance in the context
		if (const UExternalDataLayerInstance* RootEDLInstance = InDataLayerInstance->GetRootExternalDataLayerInstance())
		{
			if (!AddToActorEditorContext(const_cast<UExternalDataLayerInstance*>(RootEDLInstance)))
			{
				bSuccess = false;
			}
		}
	}

	return bSuccess;
}

bool AWorldDataLayers::RemoveFromActorEditorContext(UDataLayerInstance* InDataLayerInstance)
{
	ON_SCOPE_EXIT { UpdateCurrentColorizedDataLayerInstance(); };

	check(ContainsDataLayer(InDataLayerInstance));

	const UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(InDataLayerInstance);
	const FName ExternalDataLayerName = ExternalDataLayerInstance ? ExternalDataLayerInstance->GetFName() : NAME_None;
	if (!ExternalDataLayerName.IsNone() && (CurrentDataLayers.ExternalDataLayerName == ExternalDataLayerName))
	{
		Modify(/*bDirty*/false);
		CurrentDataLayers.ExternalDataLayerName = NAME_None;
		// Removing an EDL Instance removes all DataLayerInstances with a matching root EDL Instance
		TArray<FName> ToRemove;
		for (const FName& DataLayerInstanceName : CurrentDataLayers.DataLayerInstanceNames)
		{
			const UDataLayerInstance* DataLayerInstance = GetDataLayerInstance(DataLayerInstanceName);
			const UExternalDataLayerInstance* RootEDLInstance = DataLayerInstance ? DataLayerInstance->GetRootExternalDataLayerInstance() : nullptr;
			if (!DataLayerInstance || (RootEDLInstance && RootEDLInstance == ExternalDataLayerInstance))
			{
				ToRemove.Add(DataLayerInstanceName);
			}
		}
		for (const FName& DataLayerInstanceName : ToRemove)
		{
			CurrentDataLayers.DataLayerInstanceNames.Remove(DataLayerInstanceName);
		}
		return true;
	}
	else if (CurrentDataLayers.DataLayerInstanceNames.Contains(InDataLayerInstance->GetFName()))
	{
		Modify(/*bDirty*/false);
		CurrentDataLayers.DataLayerInstanceNames.Remove(InDataLayerInstance->GetFName());
		return true;
	}
	return false;
}

void AWorldDataLayers::PushActorEditorContext(int32 InContextID, bool bDuplicateContext)
{
	Modify(/*bDirty*/false);
	CurrentDataLayers.ContextID = InContextID;
	CurrentDataLayersStack.Push(CurrentDataLayers);
	if (!bDuplicateContext)
	{
		CurrentDataLayers.Reset();
	}
}

void AWorldDataLayers::PopActorEditorContext(int32 InContextID)
{
	if (Algo::FindByPredicate(CurrentDataLayersStack, [InContextID](FActorPlacementDataLayers& Element) { return Element.ContextID == InContextID; }))
	{
		Modify(/*bDirty*/false);
		while (!CurrentDataLayersStack.IsEmpty())
		{
			CurrentDataLayers = CurrentDataLayersStack.Pop();
			if (CurrentDataLayers.ContextID == InContextID)
			{
				break;
			}
		}
	}
}

TArray<UDataLayerInstance*> AWorldDataLayers::GetActorEditorContextDataLayers() const
{
	TArray<UDataLayerInstance*> Result;
	for (const FName& DataLayerInstanceName : CurrentDataLayers.DataLayerInstanceNames)
	{
		const UDataLayerInstance* DataLayerInstance = GetDataLayerInstance(DataLayerInstanceName);
		if (DataLayerInstance && !DataLayerInstance->IsReadOnly())
		{
			Result.Add(const_cast<UDataLayerInstance*>(DataLayerInstance));
		}
	}

	if (const UDataLayerInstance* ExternalDataLayerInstance = GetDataLayerInstance(CurrentDataLayers.ExternalDataLayerName))
	{
		check(ExternalDataLayerInstance->IsA<UExternalDataLayerInstance>());
		Result.Add(const_cast<UDataLayerInstance*>(ExternalDataLayerInstance));
	}

	return Result;
}

#endif

bool AWorldDataLayers::ContainsDataLayer(const UDataLayerInstance* InDataLayerInstance) const 
{
	if (GetDataLayerInstances().Contains(InDataLayerInstance) || TransientDataLayerInstances.Contains(InDataLayerInstance))
	{
		return true;
	}
	else if (!TransientDataLayerInstances.IsEmpty())
	{
		for (UDataLayerInstance* DataLayerInstance : TransientDataLayerInstances)
		{
			if (GetDataLayerInstancesFromProvider(DataLayerInstance).Contains(InDataLayerInstance))
			{
				return true;
			}
		}
	}
	return false;
}

const UDataLayerInstance* AWorldDataLayers::GetDataLayerInstance(const FName& InDataLayerInstanceName) const
{
	if (InDataLayerInstanceName.IsNone())
	{
		return nullptr;
	}

#if WITH_EDITOR	
	{
		const UDataLayerInstance* FoundDataLayerInstance = nullptr;
		ForEachDataLayerInstance([InDataLayerInstanceName, &FoundDataLayerInstance](UDataLayerInstance* DataLayerInstance)
		{
			if (DataLayerInstance->GetFName() == InDataLayerInstanceName)
			{
				FoundDataLayerInstance = DataLayerInstance;
				return false;
			}
			return true;
		});
		if (FoundDataLayerInstance)
		{
			return FoundDataLayerInstance;
		}
	}
#else
	if (const UDataLayerInstance* const* FoundDataLayerInstance = InstanceNameToInstance.Find(InDataLayerInstanceName))
	{
		return *FoundDataLayerInstance;
	}
#endif

	return nullptr;
}

const UDataLayerInstance* AWorldDataLayers::GetDataLayerInstanceFromAssetName(const FName& InDataLayerAssetPathName) const
{
	if (InDataLayerAssetPathName.IsNone())
	{
		return nullptr;
	}

#if WITH_EDITOR	
	const UDataLayerInstance* FoundDataLayerInstance = nullptr;
	ForEachDataLayerInstance([InDataLayerAssetPathName, &FoundDataLayerInstance](UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance->GetDataLayerFullName().Equals(InDataLayerAssetPathName.ToString(), ESearchCase::IgnoreCase))
		{
			FoundDataLayerInstance = DataLayerInstance;
			return false;
		}
		return true;
	});
	return FoundDataLayerInstance;
#else
	if (const UDataLayerInstance* const* FoundDataLayerInstance = AssetNameToInstance.Find(InDataLayerAssetPathName.ToString()))
	{
		return *FoundDataLayerInstance;
	}
	return nullptr;
#endif
}

const UDataLayerInstance* AWorldDataLayers::GetDataLayerInstance(const UDataLayerAsset* InDataLayerAsset) const
{
	if (InDataLayerAsset == nullptr)
	{
		return nullptr;
	}

#if WITH_EDITOR	
	const UDataLayerInstance* FoundDataLayerInstance = nullptr;
	ForEachDataLayerInstance([InDataLayerAsset, &FoundDataLayerInstance](UDataLayerInstance* DataLayerInstance)
	{
		if (DataLayerInstance->GetAsset() == InDataLayerAsset)
		{
			FoundDataLayerInstance = DataLayerInstance;
			return false;
		}
		return true;
	});
	return FoundDataLayerInstance;
#else
	if (const UDataLayerInstance* const* FoundDataLayerInstance = AssetNameToInstance.Find(InDataLayerAsset->GetPathName()))
	{
		return *FoundDataLayerInstance;
	}
	return nullptr;
#endif
}

bool AWorldDataLayers::IsExternalDataLayerWorldDataLayers() const
{
	return !!GetRootExternalDataLayerInstance();
}

UExternalDataLayerInstance* AWorldDataLayers::GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset)
{
	if (UDataLayerInstance* DataLayerInstance = const_cast<UDataLayerInstance*>(GetDataLayerInstance(InExternalDataLayerAsset)))
	{
		return CastChecked<UExternalDataLayerInstance>(DataLayerInstance);
	}

	return nullptr;
}

const UExternalDataLayerInstance* AWorldDataLayers::GetExternalDataLayerInstance(const UExternalDataLayerAsset* InExternalDataLayerAsset) const
{
	return const_cast<AWorldDataLayers*>(this)->GetExternalDataLayerInstance(InExternalDataLayerAsset);
}

void AWorldDataLayers::ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func)
{
	for (UDataLayerInstance* DataLayerInstance : GetDataLayerInstances())
	{
		if (DataLayerInstance && !Func(DataLayerInstance))
		{
			return;
		}
	}

	for (UDataLayerInstance* DataLayerInstance : TransientDataLayerInstances)
	{
		for (UDataLayerInstance* TransientDataLayerInstance : GetDataLayerInstancesFromProvider(DataLayerInstance))
		{
			if (TransientDataLayerInstance && !Func(TransientDataLayerInstance))
			{
				return;
			}
		}
	}
}

void AWorldDataLayers::ForEachDataLayerInstance(TFunctionRef<bool(UDataLayerInstance*)> Func) const
{
	const_cast<AWorldDataLayers*>(this)->ForEachDataLayerInstance(Func);
}

TArray<const UDataLayerInstance*> AWorldDataLayers::GetDataLayerInstances(const TArray<FName>& InDataLayerInstanceNames) const
{
	TArray<const UDataLayerInstance*> OutDataLayers;
	OutDataLayers.Reserve(InDataLayerInstanceNames.Num());

	for (const FName& DataLayerInstanceName : InDataLayerInstanceNames)
	{
		if (const UDataLayerInstance* DataLayerObject = GetDataLayerInstance(DataLayerInstanceName))
		{
			OutDataLayers.AddUnique(DataLayerObject);
		}
	}

	return OutDataLayers;
}

TSet<TObjectPtr<UDataLayerInstance>>& AWorldDataLayers::GetDataLayerInstances()
{
#if WITH_EDITOR
	if (IsUsingExternalPackageDataLayerInstances())
	{
		check(DataLayerInstances.IsEmpty());
		check(LoadedExternalPackageDataLayerInstances.IsEmpty());
		return ExternalPackageDataLayerInstances;
	}
	else
	{
		check(ExternalPackageDataLayerInstances.IsEmpty());
	}
#endif
	return DataLayerInstances;
}

#if WITH_EDITOR
void AWorldDataLayers::InitializeExternalPackageDataLayerInstances()
{
	check(IsUsingExternalPackageDataLayerInstances());
	ExternalPackageDataLayerInstances.Append(LoadedExternalPackageDataLayerInstances);
	LoadedExternalPackageDataLayerInstances.Reset();
}
#endif

void AWorldDataLayers::OnDataLayerManagerInitialized()
{
#if WITH_EDITOR
	// At this point, LoadedExternalPackageDataLayerInstances are fully loaded, transfer them to the DataLayerInstances list.
	if (IsUsingExternalPackageDataLayerInstances())
	{
		InitializeExternalPackageDataLayerInstances();
	}
	
	if (IsRunningCookCommandlet())
	{
		// Embed external DataLayerInstances when cooking
		UE_CLOGF(IsUsingExternalPackageDataLayerInstances(), LogWorldPartition, Display, "Internalizing DataLayerInstances in %ls in package (%ls)", *GetPathName(), *GetPackage()->GetName());
		SetUseExternalPackageDataLayerInstances(false);
		UE_CLOGF(IsUsingExternalPackageDataLayerInstances(), LogWorldPartition, Error, "Error while internalizing DataLayerInstances.");
	}

	if (RootExternalDataLayerInstance)
	{
		TArray<UDataLayerInstance*> ToDelete;
		ForEachDataLayerInstance([&ToDelete, this](UDataLayerInstance* DataLayerInstance)
		{
			if (DataLayerInstance->IsA<UExternalDataLayerInstance>() && RootExternalDataLayerInstance != DataLayerInstance)
			{
				ToDelete.Add(DataLayerInstance);
			}
			return true;
		});
		RemoveDataLayers(ToDelete);
	}

	// Remove all Editor Data Layers when cooking or when in a game world
	if (IsRunningCookCommandlet() || GetWorld()->IsGameWorld())
	{
		RemoveEditorDataLayers();
	}

	TSet<const UDataLayerAsset*> SettingsDataLayerAssetsLoadedInEditor;
	TSet<const UDataLayerAsset*> SettingsDataLayerAssetsNotLoadedInEditor;
	SettingsDataLayerAssetsLoadedInEditor.Append(GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetWorldDataLayerAssetsLoadedInEditor(GetWorld()));
	SettingsDataLayerAssetsNotLoadedInEditor.Append(GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetWorldDataLayerAssetsNotLoadedInEditor(GetWorld()));

	// Setup IsLoadedInEditor based on default value and DataLayerEditorPerProjectUserSettings
	ForEachDataLayerInstance([&SettingsDataLayerAssetsLoadedInEditor, &SettingsDataLayerAssetsNotLoadedInEditor](UDataLayerInstance* DataLayerInstance)
	{
		bool bIsLoadedInEditor = DataLayerInstance->IsInitiallyLoadedInEditor();
		const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset();
		if (SettingsDataLayerAssetsLoadedInEditor.Contains(DataLayerAsset))
		{
			bIsLoadedInEditor = true;
		}
		else if (SettingsDataLayerAssetsNotLoadedInEditor.Contains(DataLayerAsset))
		{
			bIsLoadedInEditor = false;
		}
		DataLayerInstance->SetIsLoadedInEditor(bIsLoadedInEditor, /*bFromUserChange*/false);
		return true;
	});
#endif

	InitializeDataLayerRuntimeStates();
}

void AWorldDataLayers::OnDataLayerManagerDeinitialized()
{
    ResetDataLayerRuntimeStates(false);
}

void AWorldDataLayers::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	// This handle both duplication for PIE (which includes unsaved modifications) and regular duplicate
	if ((Ar.GetPortFlags() & PPF_Duplicate) && Ar.IsPersistent())
	{
		Ar << ExternalPackageDataLayerInstances;
	}
#endif
}

void AWorldDataLayers::PostLoad()
{
	Super::PostLoad();

	ULevel* Level = GetLevel();
#if WITH_EDITOR
	// When duplicating the EDL WorldDataLayers for PIE/Cook, the outer is the GObjTransientPkg.
	// In this case, there's nothing to do in the PostLoad called by DuplicateObject.
	// (see UExternalDataLayerManager::CreateExternalStreamingObjectUsingStreamingGeneration for details)
	check(Level || IsExternalDataLayerWorldDataLayers());
	if (Level)
#endif
	{
		Level->ConditionalPostLoad();

		// Patch WorldDataLayer in UWorld.
		// Only the "main" world data Layer is named AWorldDataLayers::StaticClass()->GetFName() for a given world.
		if ((GetTypedOuter<UWorld>()->GetWorldDataLayers() == nullptr) && (GetFName() == GetWorldPartitionWorldDataLayersName()))
		{
			GetTypedOuter<UWorld>()->SetWorldDataLayers(this);
		}

#if WITH_EDITOR
		if (!Level->bWasDuplicated && IsUsingExternalPackageDataLayerInstances())
		{
			// Load all folders for this level
			FExternalPackageHelper::LoadObjectsFromExternalPackages<UDataLayerInstance>(this, [this](UDataLayerInstance* LoadedDataLayerInstance)
			{
				check(IsValid(LoadedDataLayerInstance));
				LoadedExternalPackageDataLayerInstances.Add(LoadedDataLayerInstance);
			});
		}
#endif
	}

#if WITH_EDITOR
	bListedInSceneOutliner = true;
#else
	// Build acceleration tables
	ForEachDataLayerInstance([this](const UDataLayerInstance* DataLayerInstance)
	{
		UpdateAccelerationTable(DataLayerInstance, true);
		return true;
	});
#endif
}

#if !WITH_EDITOR
void AWorldDataLayers::UpdateAccelerationTable(const UDataLayerInstance* DataLayerInstance, bool bIsAdding)
{
	if (bIsAdding)
	{
		InstanceNameToInstance.Add(DataLayerInstance->GetFName(), DataLayerInstance);
	}
	else
	{
		InstanceNameToInstance.Remove(DataLayerInstance->GetFName());
	}

	if (const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset())
	{
		if (bIsAdding)
		{
			AssetNameToInstance.Add(DataLayerAsset->GetPathName(), DataLayerInstance);
		}
		else
		{
			AssetNameToInstance.Remove(DataLayerAsset->GetPathName());
		}
	}
}
#endif

#if WITH_EDITOR

bool AWorldDataLayers::CanReferenceDataLayerAsset(const UDataLayerAsset* InDataLayerAsset, FText* OutFailureReason) const
{
	auto PassesAssetReferenceFiltering = [](const UObject* InReferencingObject, const UDataLayerAsset* InDataLayerAsset, FText* OutReason)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAsset(FAssetData(InReferencingObject));
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
		return AssetReferenceFilter.IsValid() ? AssetReferenceFilter->PassesFilter(FAssetData(InDataLayerAsset), OutReason) : true;
	};

	const UExternalDataLayerAsset* RootExternalDataLayerAsset = GetRootExternalDataLayerAsset();
	const UObject* ReferencingObject = RootExternalDataLayerAsset ? Cast<UObject>(RootExternalDataLayerAsset) : Cast<UObject>(this);
	if (!PassesAssetReferenceFiltering(ReferencingObject, InDataLayerAsset, OutFailureReason))
	{
		return false;
	}
	// Validate that AWorldDataLayers and its EDL Asset are part of the same plugin
	else if (InDataLayerAsset && (RootExternalDataLayerAsset == InDataLayerAsset))
	{
		const UPackage* Package = GetPackage();
		const FString PackageName = !Package->GetLoadedPath().IsEmpty() ? Package->GetLoadedPath().GetPackageName() : Package->GetName();

		IPluginManager& PluginManager = IPluginManager::Get();
		const FString AssetMountPoint = FPackageName::GetPackageMountPoint(InDataLayerAsset->GetPackage()->GetName()).ToString();
		const FString ThisMountPoint = FPackageName::GetPackageMountPoint(PackageName).ToString();
		TSharedPtr<IPlugin> AssetPlugin = PluginManager.FindPluginFromPath(AssetMountPoint);
		TSharedPtr<IPlugin> ThisPlugin = PluginManager.FindPluginFromPath(ThisMountPoint);
		if (AssetPlugin != ThisPlugin)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FText::Format(LOCTEXT("WorldDataLayerAndDataLayerAssetPluginMismatch", "{0} part of {1}'{2}' cannot be referenced by {3} part of {4}'{5}'."),
					FText::FromString(InDataLayerAsset->GetName()),
					FText::FromString(AssetPlugin.IsValid() ? TEXT("plugin ") : TEXT("")),
					FText::FromString(AssetMountPoint),
					FText::FromString(GetActorLabel()),
					FText::FromString(ThisPlugin.IsValid() ? TEXT("plugin ") : TEXT("")),
					FText::FromString(ThisMountPoint));
			}
			return false;
		}
	}
	return true;
}

void AWorldDataLayers::RemoveEditorDataLayers()
{
	TArray<UDataLayerInstance*> EditorDataLayers;
	ForEachDataLayerInstance([&EditorDataLayers](UDataLayerInstance* DataLayerInstance)
	{
		if (!DataLayerInstance->IsRuntime())
		{
			EditorDataLayers.Add(DataLayerInstance);
		}
		return true;
	});
	RemoveDataLayers(EditorDataLayers);
}

bool AWorldDataLayers::IsSubWorldDataLayers() const
{
	UWorld* ActorWorld = GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	return ActorWorld != nullptr && OuterWorld != nullptr && (OuterWorld->GetFName() != ActorWorld->GetFName());
}

bool AWorldDataLayers::IsReadOnly(FText* OutReason) const
{
	if (IsSubWorldDataLayers())
	{
		const UWorld* ActorWorld = GetWorld();
		const ULevel* CurrentLevel = (ActorWorld && ActorWorld->GetCurrentLevel() && !ActorWorld->GetCurrentLevel()->IsPersistentLevel()) ? ActorWorld->GetCurrentLevel() : nullptr;
		const bool bIsCurrentLevelWorldDataLayers = CurrentLevel && (CurrentLevel == GetLevel());
		if (!bIsCurrentLevelWorldDataLayers)
		{
			if (OutReason)
			{
				*OutReason = LOCTEXT("WorldDataLayersIsReadOnly", "WorldDataLayers actor is read-only, it's not the Current Level's WorldDataLayers.");
			}
			return true;
		}
	}
	return false;
}

bool AWorldDataLayers::SupportsExternalPackageDataLayerInstances() const
{
	ULevel* Level = GetLevel();
	return Level && Level->bIsPartitioned && Level->IsUsingExternalObjects();
}

bool AWorldDataLayers::SetUseExternalPackageDataLayerInstances(bool bInNewValue, bool bInInteractiveMode)
{
	if (HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		return false;
	}

	if (bUseExternalPackageDataLayerInstances == bInNewValue)
	{
		return false;
	}

	check(SupportsExternalPackageDataLayerInstances());
	const bool bInteractiveMode = bInInteractiveMode && !IsRunningCommandlet();

	// Validate we have a saved map
	UPackage* Package = GetExternalPackage();
	if (Package == GetTransientPackage()
		|| Package->HasAnyFlags(RF_Transient)
		|| !FPackageName::IsValidLongPackageName(Package->GetName()))
	{
		if (bInteractiveMode)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UseExternalPackageDataLayerInstancesSaveActor", "You need to save the WorldDataLayers actor before changing the `Use External Package Data Layer Instances` option."));
		}
		UE_LOGF(LogWorldPartition, Warning, "You need to save the WorldDataLayers actor before changing the `Use External Package Data Layer Instances` option.");
		return false;
	}

	if (bInteractiveMode && !IsEmpty())
	{
		FText MessageTitle(LOCTEXT("ConvertDataLayerInstancesExternalPackagingDialog", "Convert Data Layer Instances External Packaging"));
		FText Message(LOCTEXT("ConvertDataLayerInstancesExternalPackagingMsg", "Do you want to convert external packaging for all data layer instances?"));
		EAppReturnType::Type ConvertAnswer = FMessageDialog::Open(EAppMsgType::YesNo, Message, MessageTitle);
		if (ConvertAnswer != EAppReturnType::Yes)
		{
			return false;
		}
	}

	Modify();
	// Only change packaging for owned data layer instances
	for (UDataLayerInstance* DataLayerInstance : GetDataLayerInstances())
	{
		check(DataLayerInstance->GetDirectOuterWorldDataLayers() == this);
		UPackage* PreviousPackage = DataLayerInstance->GetPackage();
		FExternalPackageHelper::SetPackagingMode(DataLayerInstance, this, bInNewValue);
		UE_LOGF(LogWorldPartition, Display, "DataLayerInstance %ls changed package from %ls to %ls",
			*DataLayerInstance->GetPathName(),
			*PreviousPackage->GetName(),
			*DataLayerInstance->GetPackage()->GetName());
	}
	bUseExternalPackageDataLayerInstances = bInNewValue;
	Swap(DataLayerInstances, ExternalPackageDataLayerInstances);

	// Operation cannot be undone
	GEditor->ResetTransaction(LOCTEXT("EActorFolderObjectsResetTrans", "WorldDataLayers Use External Package Data Layer Instances"));

	return true;
}

void AWorldDataLayers::ResolveActorDescContainers()
{
	// Always use actor owning world to find DataLayerManager for resolving of data layers as partitioned 
	// LevelInstance DataLayerManager can't resolve.
	if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
	{
		DataLayerManager->ResolveActorDescContainersDataLayers();
	}
}

void AWorldDataLayers::PreEditUndo()
{
	Super::PreEditUndo();
	CachedDataLayerInstances = GetDataLayerInstances();
}

void AWorldDataLayers::PostEditUndo()
{
	Super::PostEditUndo();

	bool bNeedResolve = CachedDataLayerInstances.Num() != GetDataLayerInstances().Num();
	if (!bNeedResolve)
	{
		ForEachDataLayerInstance([this, &bNeedResolve](UDataLayerInstance* DataLayerInstance)
		{
			if (!CachedDataLayerInstances.Contains(DataLayerInstance))
			{
				bNeedResolve = true;
				return false;
			}
			return true;
		});
	}
	if (bNeedResolve)
	{
		ResolveActorDescContainers();
	}
	CachedDataLayerInstances.Empty();
}

bool AWorldDataLayers::ShouldLevelKeepRefIfExternal() const
{
	return !IsExternalDataLayerWorldDataLayers();
}

bool AWorldDataLayers::IsEditorOnly() const
{
	return IsExternalDataLayerWorldDataLayers();
}

#endif

FWorldDataLayersEffectiveStates::FWorldDataLayersEffectiveStates()
	: UpdateEpoch(0)
	, AllEffectiveActiveDataLayerNamesEpoch(MAX_int32)
	, AllEffectiveLoadedDataLayerNamesEpoch(MAX_int32)
{}

void FWorldDataLayersEffectiveStates::SetReplicatedEffectiveActiveDataLayerNames(const TArray<FName>& InRepEffectiveActiveDataLayerNames)
{
	ReplicatedEffectiveActiveDataLayerNames.Reset();
	ReplicatedEffectiveActiveDataLayerNames.Append(InRepEffectiveActiveDataLayerNames);
	++UpdateEpoch;
}

void FWorldDataLayersEffectiveStates::SetReplicatedEffectiveLoadedDataLayerNames(const TArray<FName>& InRepEffectiveLoadedDataLayerNames)
{
	ReplicatedEffectiveLoadedDataLayerNames.Reset();
	ReplicatedEffectiveLoadedDataLayerNames.Append(InRepEffectiveLoadedDataLayerNames);
	++UpdateEpoch;
}

bool FWorldDataLayersEffectiveStates::SetDataLayerEffectiveRuntimeState(FName InDataLayerName, bool bIsLocalDataLayer, EDataLayerRuntimeState NewEffectiveRuntimeState, EDataLayerRuntimeState& OutOldEffectiveRuntimeState)
{
	OutOldEffectiveRuntimeState = GetDataLayerEffectiveRuntimeStateByName(InDataLayerName);
	if (OutOldEffectiveRuntimeState != NewEffectiveRuntimeState)
	{
		TSet<FName>& TargetEffectiveLoadedDataLayerNames = bIsLocalDataLayer ? LocalEffectiveLoadedDataLayerNames : ReplicatedEffectiveLoadedDataLayerNames;
		TSet<FName>& TargetEffectiveActiveDataLayerNames = bIsLocalDataLayer ? LocalEffectiveActiveDataLayerNames : ReplicatedEffectiveActiveDataLayerNames;

		TargetEffectiveLoadedDataLayerNames.Remove(InDataLayerName);
		TargetEffectiveActiveDataLayerNames.Remove(InDataLayerName);

		if (NewEffectiveRuntimeState == EDataLayerRuntimeState::Loaded)
		{
			TargetEffectiveLoadedDataLayerNames.Add(InDataLayerName);
		}
		else if (NewEffectiveRuntimeState == EDataLayerRuntimeState::Activated)
		{
			TargetEffectiveActiveDataLayerNames.Add(InDataLayerName);
		}

		++UpdateEpoch;
		return true;
	}
	return false;
}

void FWorldDataLayersEffectiveStates::Reset()
{
	ReplicatedEffectiveActiveDataLayerNames.Reset();
	ReplicatedEffectiveLoadedDataLayerNames.Reset();
	LocalEffectiveActiveDataLayerNames.Reset();
	LocalEffectiveLoadedDataLayerNames.Reset();
	++UpdateEpoch;
}

EDataLayerRuntimeState FWorldDataLayersEffectiveStates::GetDataLayerEffectiveRuntimeStateByName(FName InDataLayerName) const
{
	if (ReplicatedEffectiveActiveDataLayerNames.Contains(InDataLayerName))
	{
		check(!ReplicatedEffectiveLoadedDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Activated;
	}
	else if (ReplicatedEffectiveLoadedDataLayerNames.Contains(InDataLayerName))
	{
		check(!ReplicatedEffectiveActiveDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Loaded;
	}
	else if (LocalEffectiveActiveDataLayerNames.Contains(InDataLayerName))
	{
		check(!LocalEffectiveLoadedDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Activated;
	}
	else if (LocalEffectiveLoadedDataLayerNames.Contains(InDataLayerName))
	{
		check(!LocalEffectiveActiveDataLayerNames.Contains(InDataLayerName));
		return EDataLayerRuntimeState::Loaded;
	}

	return EDataLayerRuntimeState::Unloaded;
}

const TSet<FName>& FWorldDataLayersEffectiveStates::GetAllEffectiveActiveDataLayerNames() const
{
	if (AllEffectiveActiveDataLayerNamesEpoch != UpdateEpoch)
	{
		AllEffectiveActiveDataLayerNames = ReplicatedEffectiveActiveDataLayerNames;
		AllEffectiveActiveDataLayerNames.Append(LocalEffectiveActiveDataLayerNames);
		AllEffectiveActiveDataLayerNamesEpoch = UpdateEpoch;
	}
	return AllEffectiveActiveDataLayerNames;
}

const TSet<FName>& FWorldDataLayersEffectiveStates::GetAllEffectiveLoadedDataLayerNames() const
{
	if (AllEffectiveLoadedDataLayerNamesEpoch != UpdateEpoch)
	{
		AllEffectiveLoadedDataLayerNames = ReplicatedEffectiveLoadedDataLayerNames;
		AllEffectiveLoadedDataLayerNames.Append(LocalEffectiveLoadedDataLayerNames);
		AllEffectiveLoadedDataLayerNamesEpoch = UpdateEpoch;
	}
	return AllEffectiveLoadedDataLayerNames;
}

#undef LOCTEXT_NAMESPACE
