// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"

#if WITH_EDITOR

#include "UObject/TopLevelAssetPath.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/MetaData.h"
#include "ExternalPackageHelper.h"

FArchive& operator<<(FArchive& Ar, FDataLayerInstanceDesc& Desc)
{
	Ar << Desc.Name << Desc.ParentName << Desc.bIsUsingAsset;

	if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::FixedDataLayerInstanceDesc)
	{
		Ar << Desc.PrivateShortName;
		if (Desc.bIsUsingAsset)
		{
			Ar << Desc.AssetPath << Desc.bDeprecatedIsRuntime;
		}
	}
	else
	{
		if (Desc.bIsUsingAsset)
		{
			Ar << Desc.AssetPath;
		}
		else
		{
			Ar << Desc.bDeprecatedIsRuntime << Desc.PrivateShortName;
		}
	}

	// Fixup redirected data layer asset path
	if (Ar.IsLoading() && Desc.bIsUsingAsset)
	{
		UAssetRegistryHelpers::FixupRedirectedAssetPath(Desc.AssetPath);
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorFilter)
	{
		Ar << Desc.bIsIncludedInActorFilterDefault;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionPrivateDataLayers)
	{
		Ar << Desc.bIsPrivate;

		if (Desc.bIsPrivate)
		{
			Ar << Desc.PrivateShortName;
			Ar << Desc.bPrivateDataLayerSupportsActorFilter;
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionDataLayerDefaultRuntimeState)
	{
		Ar << Desc.InitialRuntimeState;
	}

	return Ar;
}

bool operator == (const FDataLayerInstanceDesc& Lhs, const FDataLayerInstanceDesc& Rhs)
{
	return (Lhs.Name == Rhs.Name) &&
		(Lhs.ParentName == Rhs.ParentName) &&
		(Lhs.bIsUsingAsset == Rhs.bIsUsingAsset) &&
		(Lhs.AssetPath == Rhs.AssetPath) &&
		(Lhs.bDeprecatedIsRuntime == Rhs.bDeprecatedIsRuntime) &&
		(Lhs.PrivateShortName == Rhs.PrivateShortName) &&
		(Lhs.bIsIncludedInActorFilterDefault == Rhs.bIsIncludedInActorFilterDefault) &&
		(Lhs.bIsPrivate == Rhs.bIsPrivate) &&
		(Lhs.bPrivateDataLayerSupportsActorFilter == Rhs.bPrivateDataLayerSupportsActorFilter) &&
		(Lhs.InitialRuntimeState == Rhs.InitialRuntimeState);
}

bool operator < (const FDataLayerInstanceDesc& Lhs, const FDataLayerInstanceDesc& Rhs)
{
	if (Lhs.Name == Rhs.Name)
	{
		if (Lhs.ParentName == Rhs.ParentName)
		{
			if (Lhs.bIsUsingAsset == Rhs.bIsUsingAsset)
			{
				if (Lhs.AssetPath == Rhs.AssetPath)
				{
					if (Lhs.bDeprecatedIsRuntime == Rhs.bDeprecatedIsRuntime)
					{
						if (Lhs.PrivateShortName == Rhs.PrivateShortName)
						{
							if (Lhs.bIsIncludedInActorFilterDefault == Rhs.bIsIncludedInActorFilterDefault)
							{
								if (Lhs.bIsPrivate == Rhs.bIsPrivate)
								{
									if (Lhs.bPrivateDataLayerSupportsActorFilter == Rhs.bPrivateDataLayerSupportsActorFilter)
									{
										return (int)Lhs.InitialRuntimeState < (int)Rhs.InitialRuntimeState;
									}
									return (int)Lhs.bPrivateDataLayerSupportsActorFilter < (int)Rhs.bPrivateDataLayerSupportsActorFilter;
								}
								return (int)Lhs.bIsPrivate < (int)Rhs.bIsPrivate;
							}
							return (int)Lhs.bIsIncludedInActorFilterDefault < (int)Rhs.bIsIncludedInActorFilterDefault;
						}
						return Lhs.PrivateShortName < Rhs.PrivateShortName;
					}
					return (int)Lhs.bDeprecatedIsRuntime < (int)Rhs.bDeprecatedIsRuntime;
				}
				return Lhs.AssetPath.LexicalLess(Rhs.AssetPath);
			}
			return (int)Lhs.bIsUsingAsset < (int)Rhs.bIsUsingAsset;
		}
		return Lhs.ParentName.LexicalLess(Rhs.ParentName);
	}
	return Lhs.Name.LexicalLess(Rhs.Name);
}

UDataLayerAsset* FDataLayerInstanceDesc::GetAsset() const
{
	if (!bIsPrivate && IsUsingAsset() && !AssetPath.IsNone())
	{
		UDataLayerAsset* Asset = LoadObject<UDataLayerAsset>(nullptr, *AssetPath.ToString());
		return Asset;
	}
	return nullptr;
}

EDataLayerType FDataLayerInstanceDesc::GetDataLayerType() const
{
	if (bIsPrivate)
	{
		return EDataLayerType::Editor;
	}

	if (IsUsingAsset())
	{
		if (UDataLayerAsset* DataLayerAsset = GetAsset())
		{
			return DataLayerAsset->IsRuntime() ? EDataLayerType::Runtime : EDataLayerType::Editor;
		}
		return EDataLayerType::Unknown;
	}
	return bDeprecatedIsRuntime ? EDataLayerType::Runtime : EDataLayerType::Editor;
}

FString FDataLayerInstanceDesc::GetShortName() const
{
	if (!bIsPrivate && IsUsingAsset())
	{
		static FString UnknownString("Unknown");
		UDataLayerAsset* DataLayerAsset = GetAsset();
		return DataLayerAsset ? DataLayerAsset->GetName() : UnknownString;
	}
	return PrivateShortName;
}

EDataLayerRuntimeState FDataLayerInstanceDesc::GetInitialRuntimeState() const
{
	return InitialRuntimeState;
}

bool FDataLayerInstanceDesc::SupportsActorFilters() const
{
	if (bIsPrivate)
	{
		return bPrivateDataLayerSupportsActorFilter;
	}

	if (IsUsingAsset())
	{
		if (UDataLayerAsset* DataLayerAsset = GetAsset())
		{
			return DataLayerAsset->SupportsActorFilters();
		}
	}
	
	return false;
}

FDataLayerInstanceDesc::FDataLayerInstanceDesc()
: bIsUsingAsset(false)
, bIsIncludedInActorFilterDefault(true)
, bIsPrivate(false)
, InitialRuntimeState(EDataLayerRuntimeState::Unloaded)
, bPrivateDataLayerSupportsActorFilter(false)
, bDeprecatedIsRuntime(false)
{
}

void FDataLayerInstanceDesc::Init(UDataLayerInstance* InDataLayerInstance)
{
	Name = InDataLayerInstance->GetFName();
	ParentName = InDataLayerInstance->GetParent() ? InDataLayerInstance->GetParent()->GetFName() : NAME_None;
	const UDataLayerInstanceWithAsset* DataLayerWithAsset = Cast<UDataLayerInstanceWithAsset>(InDataLayerInstance);
	const UDataLayerAsset* DataLayerAsset = InDataLayerInstance->GetAsset();
	bIsUsingAsset = InDataLayerInstance->Implements<UDataLayerInstanceWithAsset>() || (DataLayerAsset != nullptr);
	if (bIsUsingAsset && DataLayerAsset)
	{
		AssetPath = FName(DataLayerAsset->GetPathName());
		bIsPrivate = DataLayerAsset->IsPrivate();
	}
	
	bIsIncludedInActorFilterDefault = InDataLayerInstance->IsIncludedInActorFilterDefault();
	bPrivateDataLayerSupportsActorFilter = InDataLayerInstance->SupportsActorFilters();
	PrivateShortName = InDataLayerInstance->GetDataLayerShortName();
	bDeprecatedIsRuntime = InDataLayerInstance->IsRuntime();
	InitialRuntimeState = InDataLayerInstance->GetInitialRuntimeState();
}

FWorldDataLayersActorDesc::FWorldDataLayersActorDesc()
: bIsValid(false)
, bIsExternalDataLayerWorldDataLayers(false)
, bUseExternalPackageDataLayerInstances(false)
{}

void FWorldDataLayersActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const AWorldDataLayers* WorldDataLayers = CastChecked<AWorldDataLayers>(InActor);
	bUseExternalPackageDataLayerInstances = WorldDataLayers->IsUsingExternalPackageDataLayerInstances();
	bIsExternalDataLayerWorldDataLayers = WorldDataLayers->IsExternalDataLayerWorldDataLayers();
	const IDataLayerInstanceProvider* DataLayerInstanceProvider = CastChecked<IDataLayerInstanceProvider>(WorldDataLayers);
	for (UDataLayerInstance* DataLayerInstance : DataLayerInstanceProvider->GetDataLayerInstances())
	{
		FDataLayerInstanceDesc& DataLayerInstanceDesc = DataLayerInstances.Emplace_GetRef();
		DataLayerInstanceDesc.Init(DataLayerInstance);
	}

	bIsValid = true;
}

bool FWorldDataLayersActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FWorldDataLayersActorDesc* OtherDesc = (FWorldDataLayersActorDesc*)Other;
		return (bUseExternalPackageDataLayerInstances == OtherDesc->bUseExternalPackageDataLayerInstances) && 
			   (bIsExternalDataLayerWorldDataLayers == OtherDesc->bIsExternalDataLayerWorldDataLayers) &&
			   CompareUnsortedArrays(DataLayerInstances, OtherDesc->DataLayerInstances);
	}
	return false;
}

void FWorldDataLayersActorDesc::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	FWorldPartitionActorDesc::Serialize(Ar);

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::AddedWorldDataLayersActorDesc)
		{
			Ar << DataLayerInstances;
			bIsValid = true;
		}
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddDataLayerInstanceExternalPackage)
		{
			Ar << bUseExternalPackageDataLayerInstances;
		}
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionExternalDataLayers)
		{
			Ar << bIsExternalDataLayerWorldDataLayers;
		}
	}
}

bool FWorldDataLayersActorDesc::IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const 
{
	if (!FWorldPartitionActorDesc::IsRuntimeRelevant(InActorDescInstance))
	{
		return false;
	}

	// ExternalDataLayer WorldDataLayers (used to store data layers for an ExternalDataLayer) are not used at runtime.
	return !bIsExternalDataLayerWorldDataLayers;
}

const TArray<FDataLayerInstanceDesc>& FWorldDataLayersActorDesc::GetExternalPackageDataLayerInstances() const
{
	check(bUseExternalPackageDataLayerInstances);
	if (!ExternalPackageDataLayerInstances.IsSet())
	{
		TArray<FDataLayerInstanceDesc> FoundDataLayerInstances;
		if (ULevel::GetIsLevelPartitionedFromPackage(ActorPath.GetLongPackageFName()))
		{
			FTopLevelAssetPath MapAssetName = ActorPath.GetAssetPath();
			FString ExternalObjectsPath = FExternalPackageHelper::GetExternalObjectsPath(MapAssetName.GetPackageName().ToString());
			
			// Do a synchronous scan of the world external objects path.			
			IAssetRegistry & AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			AssetRegistry.ScanSynchronous({ ExternalObjectsPath }, TArray<FString>());

			FARFilter Filter;
			Filter.bRecursivePaths = true;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.ClassPaths.Add(UDataLayerInstance::StaticClass()->GetClassPathName());
			Filter.bRecursiveClasses = true;
			Filter.PackagePaths.Add(*ExternalObjectsPath);
			TArray<FAssetData> Assets;
			FExternalPackageHelper::GetSortedAssets(Filter, Assets);

			for (const FAssetData& Asset : Assets)
			{
				FDataLayerInstanceDesc DataLayerInstanceDesc;
				if (UDataLayerInstance::GetAssetRegistryInfoFromPackage(Asset, DataLayerInstanceDesc))
				{
					FoundDataLayerInstances.Add(DataLayerInstanceDesc);
				}
			}
		}
		ExternalPackageDataLayerInstances = MoveTemp(FoundDataLayerInstances);
	}
	return ExternalPackageDataLayerInstances.GetValue();
}

const TArray<FDataLayerInstanceDesc>& FWorldDataLayersActorDesc::GetDataLayerInstances() const
{
	return bUseExternalPackageDataLayerInstances ? GetExternalPackageDataLayerInstances() : DataLayerInstances;
}

void FWorldDataLayersActorDesc::ForEachDataLayerInstanceDesc(TFunctionRef<bool(const FDataLayerInstanceDesc&)> Func) const
{
	for (const FDataLayerInstanceDesc& DataLayerInstance : GetDataLayerInstances())
	{
		if (!Func(DataLayerInstance))
		{
			return;
		}
	}
}

const FDataLayerInstanceDesc* FWorldDataLayersActorDesc::GetDataLayerInstanceFromInstanceName(FName InDataLayerInstanceName) const
{
	const FDataLayerInstanceDesc* FoundDataLayerInstanceDesc = nullptr;
	ForEachDataLayerInstanceDesc([InDataLayerInstanceName, &FoundDataLayerInstanceDesc](const FDataLayerInstanceDesc& DataLayerInstance)
	{
		if (DataLayerInstance.GetName().IsEqual(InDataLayerInstanceName, ENameCase::CaseSensitive))
		{
			FoundDataLayerInstanceDesc = &DataLayerInstance;
			return false;
		}
		return true;
	});
	return FoundDataLayerInstanceDesc;
}

const FDataLayerInstanceDesc* FWorldDataLayersActorDesc::GetDataLayerInstanceFromAssetPath(FName InDataLayerAssetPath) const
{
	const FDataLayerInstanceDesc* FoundDataLayerInstanceDesc = nullptr;
	ForEachDataLayerInstanceDesc([InDataLayerAssetPath, &FoundDataLayerInstanceDesc](const FDataLayerInstanceDesc& DataLayerInstance)
	{
		if (FName(DataLayerInstance.GetAssetPath()) == InDataLayerAssetPath)
		{
			FoundDataLayerInstanceDesc = &DataLayerInstance;
			return false;
		}
		return true;
	});
	return FoundDataLayerInstanceDesc;
}

#endif
