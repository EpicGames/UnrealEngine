// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"
#include "WorldPartition/WorldPartitionActorDescInstanceViewInterface.h"

FString IStreamingGenerationErrorHandler::GetActorName(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	const FString ActorPath = ActorDescView.GetActorSoftPath().GetLongPackageName();
	const FString ActorLabel = ActorDescView.GetActorLabelOrName().ToString();
	return ActorPath + TEXT(".") + ActorLabel;
}

FString IStreamingGenerationErrorHandler::GetFullActorName(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	const FString ActorPackage = ActorDescView.GetActorPackage().ToString();
	return GetActorName(ActorDescView) + TEXT(" (") + ActorPackage + TEXT(")");
}
#endif
