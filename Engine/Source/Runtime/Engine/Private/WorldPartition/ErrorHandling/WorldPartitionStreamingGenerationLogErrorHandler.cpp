// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationLogErrorHandler.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#define UE_ASSET_LOG_ACTORDESCVIEW(CategoryName, Verbosity, ActorDescView, Format, ...) \
	UE_ASSET_LOG(LogWorldPartition, Log, *ActorDescView.GetActorPackage().ToString(), Format, ##__VA_ARGS__)

void FStreamingGenerationLogErrorHandler::OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s has an invalid runtime grid %s"), *GetActorName(ActorDescView), *GridName.ToString());
}

void FStreamingGenerationLogErrorHandler::OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s has an invalid reference to %s"), *GetActorName(ActorDescView), ReferenceActorDescView ? *GetActorName(*ReferenceActorDescView) : *ReferenceGuid.ToString());
}

void FStreamingGenerationLogErrorHandler::OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	static const FString SpatiallyLoadedActor("Spatially loaded actor");
	static const FString NonSpatiallyLoadedActor("Non-spatially loaded actor");

	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("%s %s reference %s %s"), ActorDescView.GetIsSpatiallyLoaded() ? *SpatiallyLoadedActor : *NonSpatiallyLoadedActor, *GetActorName(ActorDescView), ReferenceActorDescView.GetIsSpatiallyLoaded() ? *SpatiallyLoadedActor : *NonSpatiallyLoadedActor, *GetActorName(ReferenceActorDescView));
}

void FStreamingGenerationLogErrorHandler::OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason)
{
	const FString ActorName = GetActorName(ActorDescView);
	const FString ReferenceActorName = GetActorName(ReferenceActorDescView);

	switch (Reason)
	{
	case EDataLayerInvalidReason::ReferencedActorDifferentRuntimeDataLayers:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s references an actor %s in a different set of runtime data layers"), *ActorName, *ReferenceActorName);
		break;
	case EDataLayerInvalidReason::ReferencedActorDifferentExternalDataLayer:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s references an actor %s with a different external data layer"), *ActorName, *ReferenceActorName);
		break;
	}
}

void FStreamingGenerationLogErrorHandler::OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s references an actor in a different runtime grid %s"), *GetActorName(ActorDescView), *GetActorName(ReferenceActorDescView));
}

void FStreamingGenerationLogErrorHandler::OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s contains runtime data layers with different types of Load Filter"), *GetActorName(ActorDescView));
}

void FStreamingGenerationLogErrorHandler::OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason)
{
	switch(Reason)
	{
	case EWorldReferenceInvalidReason::ReferencedActorIsSpatiallyLoaded:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("World references spatially loaded actor %s"), *GetActorName(ActorDescView));
		break;
	case EWorldReferenceInvalidReason::ReferencedActorHasDataLayers:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("World references actor %s with data layers"), *GetActorName(ActorDescView));
		break;
	}	
}

void FStreamingGenerationLogErrorHandler::OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance)
{
	UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Data Layer does not have a Data Layer asset"));
}

void FStreamingGenerationLogErrorHandler::OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset)
{
	UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Data Layer is not compatible with Data Layer asset %s type %s"), *DataLayerAsset->GetName(), *DataLayerAsset->GetClass()->GetName());
}

void FStreamingGenerationLogErrorHandler::OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason)
{
	switch (Reason)
	{
	case EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild:
		UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Client-only Data Layer %s can't be child of parent %s"), *DataLayerInstance->GetDataLayerFullName(), *Parent->GetDataLayerFullName());
		break;
	case EDataLayerHierarchyInvalidReason::ServerOnlyDataLayerCantBeChild:
		UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Server-only Data Layer %s can't be child of parent %s"), *DataLayerInstance->GetDataLayerFullName(), *Parent->GetDataLayerFullName());
		break;
	case EDataLayerHierarchyInvalidReason::IncompatibleDataLayerType:
		UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Data Layer %s is of Type %s and its parent %s is of type %s"), *DataLayerInstance->GetDataLayerFullName(), *UEnum::GetValueAsString(DataLayerInstance->GetType()), *Parent->GetDataLayerFullName(), *UEnum::GetValueAsString(Parent->GetType()));
		break;
	}
}

void FStreamingGenerationLogErrorHandler::OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason)
{
	UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Actor %s can't reference data layer %s because of asset reference restrictions (%s)"), *WorldDataLayers->GetName(), *DataLayerInstance->GetDataLayerFullName(), *Reason.ToString());
}

void FStreamingGenerationLogErrorHandler::OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance)
{
	UE_ASSET_LOG(LogWorldPartition, Log, DataLayerInstance, TEXT("Data Layer Instance %s and Data Layer Instance %s are both referencing Data Layer Asset %s"), *DataLayerInstance->GetFName().ToString(), *ConflictingDataLayerInstance->GetFName().ToString(), *DataLayerInstance->GetAsset()->GetFullName());
}

void FStreamingGenerationLogErrorHandler::OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s needs to be resaved"), *GetActorName(ActorDescView));
}

void FStreamingGenerationLogErrorHandler::OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason)
{
	const FString ActorName = GetActorName(ActorDescView);

	switch (Reason)
	{
	case ELevelInstanceInvalidReason::WorldAssetNotFound:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s has an invalid world asset %s"), *ActorName, *WorldAsset.ToString());
		break;
	case ELevelInstanceInvalidReason::WorldAssetDontContainActorsMetadata:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s world asset %s is not using external actors, resave level to add compatibility."), *ActorName, *WorldAsset.ToString());
		break;
	case ELevelInstanceInvalidReason::WorldAssetIncompatiblePartitioned:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s world asset %s is partitioned but not marked as compatible"), *ActorName, *WorldAsset.ToString());
		break;
	case ELevelInstanceInvalidReason::WorldAssetHasInvalidContainer:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s world asset %s has an invalid container"), *ActorName, *WorldAsset.ToString());
		break;
	case ELevelInstanceInvalidReason::CirculalReference:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s world asset %s has a circular reference"), *ActorName, *WorldAsset.ToString());
		break;
	};
}

void FStreamingGenerationLogErrorHandler::OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ReferenceActorDescView, TEXT("Actor %s will not be filtered out because it is referenced by Actor %s not part of the filter"), *GetActorName(ReferenceActorDescView), *GetActorName(ActorDescView));
}

void FStreamingGenerationLogErrorHandler::OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s has an invalid HLOD layer %s"), *GetActorName(ActorDescView), *ActorDescView.GetHLODLayer().ToString());
}

void FStreamingGenerationLogErrorHandler::OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason)
{
	FString LayerType = HLODLayer ? StaticEnum<EHLODLayerType>()->GetDisplayNameTextByValue((int64)HLODLayer->GetLayerType()).ToString() : TEXT("Unknown");

	switch (Reason)
	{
	case EHLODLayerUnsupportedReason::ActorClassUnsupportedByLayerType:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Actor %s of class '%s' is unsupported by HLOD Layer %s of type '%s'"), 
			*GetActorName(ActorDescView), *ActorDescView.GetActorNativeClass()->GetName(), *ActorDescView.GetHLODLayer().ToString(), *LayerType);
		break;
	}
}

void FStreamingGenerationLogErrorHandler::OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason)
{
	switch (Reason)
	{
	case EActorDescViewMutatorRejectedReason::DivergesClusterRuntimeGrid:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Mutator override rejected: cannot set actor %s RuntimeGrid to %s because it would split a reference-connected cluster"),
			*GetActorName(ActorDescView), *RejectedOverrides.RuntimeGrid.GetValue().ToString());
		break;
	case EActorDescViewMutatorRejectedReason::DivergesClusterIsSpatiallyLoaded:
		UE_ASSET_LOG_ACTORDESCVIEW(LogWorldPartition, Log, ActorDescView, TEXT("Mutator override rejected: cannot set actor %s IsSpatiallyLoaded to %s because it would split a reference-connected cluster"),
			*GetActorName(ActorDescView), RejectedOverrides.bIsSpatiallyLoaded.GetValue() ? TEXT("true") : TEXT("false"));
		break;
	}
}
#endif
