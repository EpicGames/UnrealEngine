// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationMapCheckErrorHandler.h"
#include "Logging/MessageLog.h"

void FStreamingGenerationMapCheckErrorHandler::HandleTokenizedMessage(TSharedRef<FTokenizedMessage>&& ErrorMessage)
{
	FMessageLog("MapCheck").AddMessage(ErrorMessage);
}

#endif
