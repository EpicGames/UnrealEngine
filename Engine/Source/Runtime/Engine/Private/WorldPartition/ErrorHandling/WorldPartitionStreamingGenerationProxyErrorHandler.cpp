// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationProxyErrorHandler.h"

void FStreamingGenerationProxyErrorHandler::OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName)
{
	InnerErrorHandler->OnInvalidRuntimeGrid(ActorDescView, GridName);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView)
{
	InnerErrorHandler->OnInvalidReference(ActorDescView, ReferenceGuid, ReferenceActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	InnerErrorHandler->OnInvalidReferenceGridPlacement(ActorDescView, ReferenceActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason)
{
	InnerErrorHandler->OnInvalidReferenceDataLayers(ActorDescView, ReferenceActorDescView, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	InnerErrorHandler->OnInvalidReferenceRuntimeGrid(ActorDescView, ReferenceActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	InnerErrorHandler->OnDataLayersLoadFilterMismatch(ActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason)
{
	InnerErrorHandler->OnInvalidWorldReference(ActorDescView, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance)
{
	InnerErrorHandler->OnInvalidReferenceDataLayerAsset(DataLayerInstance);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset) 
{
	InnerErrorHandler->OnInvalidDataLayerAssetType(DataLayerInstance, DataLayerAsset);
}

void FStreamingGenerationProxyErrorHandler::OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason)
{
	InnerErrorHandler->OnDataLayerHierarchyTypeMismatch(DataLayerInstance, Parent, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason)
{
	InnerErrorHandler->OnInvalidWorldDataLayersReference(WorldDataLayers, DataLayerInstance, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance)
{
	InnerErrorHandler->OnDataLayerAssetConflict(DataLayerInstance, ConflictingDataLayerInstance);
}

void FStreamingGenerationProxyErrorHandler::OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	InnerErrorHandler->OnActorNeedsResave(ActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason)
{
	InnerErrorHandler->OnLevelInstanceInvalidWorldAsset(ActorDescView, WorldAsset, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	InnerErrorHandler->OnInvalidActorFilterReference(ActorDescView, ReferenceActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	InnerErrorHandler->OnInvalidHLODLayer(ActorDescView);
}

void FStreamingGenerationProxyErrorHandler::OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason)
{
	InnerErrorHandler->OnUnsupportedHLODLayer(ActorDescView, HLODLayer, Reason);
}

void FStreamingGenerationProxyErrorHandler::OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason)
{
	InnerErrorHandler->OnRejectedActorDescViewMutator(ActorDescView, RejectedOverrides, Reason);
}
#endif
