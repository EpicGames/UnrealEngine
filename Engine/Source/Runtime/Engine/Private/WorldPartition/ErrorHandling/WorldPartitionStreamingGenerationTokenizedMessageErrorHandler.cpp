// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationTokenizedMessageErrorHandler.h"

#include "Misc/PackageName.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#define LOCTEXT_NAMESPACE "WorldPartition"

void ITokenizedMessageErrorHandler::OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_HaveInvalidRuntimeGrid", "has an invalid runtime grid")))
		->AddToken(FTextToken::Create(FText::FromName(GridName)))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_InvalidRuntimeGrid_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_HaveMissingRefsTo", "has an invalid reference to")));

	if (ReferenceActorDescView)
	{
		Message->AddToken(FActorToken::Create(ReferenceActorDescView->GetActorSoftPath().ToString(), ReferenceActorDescView->GetGuid(), FText::FromString(GetActorName(*ReferenceActorDescView))));
	}
	else
	{
		Message->AddToken(FTextToken::Create(FText::FromString(ReferenceGuid.ToString())));
	}

	Message->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_MissingActorReference_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	const FText SpatiallyLoadedActor(LOCTEXT("TokenMessage_WorldPartition_SpatiallyLoadedActor", "Spatially loaded actor"));
	const FText NonSpatiallyLoadedActor(LOCTEXT("TokenMessage_WorldPartition_NonSpatiallyLoadedActor", "Non-spatially loaded actor"));
	const EMessageSeverity::Type MessageSeverity = ActorDescView.GetIsSpatiallyLoaded() ? EMessageSeverity::Warning : EMessageSeverity::Error;

	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(MessageSeverity);
	Message->AddToken(FTextToken::Create(ActorDescView.GetIsSpatiallyLoaded() ? SpatiallyLoadedActor : NonSpatiallyLoadedActor))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_References", "references")))
		->AddToken(FTextToken::Create(ReferenceActorDescView.GetIsSpatiallyLoaded() ? SpatiallyLoadedActor : NonSpatiallyLoadedActor))
		->AddToken(FActorToken::Create(ReferenceActorDescView.GetActorSoftPath().ToString(), ReferenceActorDescView.GetGuid(), FText::FromString(GetActorName(ReferenceActorDescView))))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_StreamedActorReferenceAlwaysLoadedActor_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason)
{
	FText ReasonText;

	switch (Reason)
	{
	case EDataLayerInvalidReason::ReferencedActorDifferentRuntimeDataLayers:
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_ReferenceActorInOtherDataLayers", "references an actor in a different set of runtime data layers");
		break;
	case EDataLayerInvalidReason::ReferencedActorDifferentExternalDataLayer:
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_ReferenceActorInOtherExternalDataLayer", "references an actor assigned to a different external data layer");
		break;
	};

	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);

	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(ReasonText))
		->AddToken(FActorToken::Create(ReferenceActorDescView.GetActorSoftPath().ToString(), ReferenceActorDescView.GetGuid(), FText::FromString(GetActorName(ReferenceActorDescView))))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_ActorInvalidReferenceDataLayers_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_ReferenceActorInOtherRuntimeGrid", "references an actor in a different runtime grid")))
		->AddToken(FActorToken::Create(ReferenceActorDescView.GetActorSoftPath().ToString(), ReferenceActorDescView.GetGuid(), FText::FromString(GetActorName(ReferenceActorDescView))))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_ActorReferenceActorInAnotherRuntimeGrid_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_ContainsRuntimeDataLayersWithDifferentTypesOfLoadFilter", "contains runtime data layers with different types of Load Filter")))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_DataLayersLoadFilterMismatch_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);

	switch(Reason)
	{
	case EWorldReferenceInvalidReason::ReferencedActorIsSpatiallyLoaded:
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_WorldReferenceSpatiallyLoadedActor", "World references spatially loaded actor")))
			->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
			->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_WorldReferenceSpatiallyLoadedActor_CheckForErrors")));
		break;
	case EWorldReferenceInvalidReason::ReferencedActorHasDataLayers:
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_WorldReferenceActorWithDataLayers", "World references actor with data layers")))
			->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
			->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_WorldReferenceActorWithDataLayers_CheckForErrors")));
		break;
	}

	HandleTokenizedMessage(MoveTemp(Message));	
}

void ITokenizedMessageErrorHandler::OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_DataLayer", "Data layer")))
		->AddToken(FUObjectToken::Create(DataLayerInstance))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_NullAsset", "Does not have Data Layer Asset")))
		->AddToken(FMapErrorToken::Create(TEXT("DataLayers_InvalidAsset_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_DataLayer", "Data layer")))
		->AddToken(FUObjectToken::Create(DataLayerInstance))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_InvalidAssetType_NotCompatible", "is not compatible with its Data Layer Asset")))
		->AddToken(FUObjectToken::Create(DataLayerAsset))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_InvalidAssetType_type", "type")))
		->AddToken(FMapErrorToken::Create(TEXT("DataLayers_InvalidDataLayerAssetType_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);

	switch (Reason)
	{
	case EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild:
	case EDataLayerHierarchyInvalidReason::ServerOnlyDataLayerCantBeChild:
		Message->AddToken(FTextToken::Create((Reason == EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild) ? LOCTEXT("TokenMessage_DataLayers_ClientOnlyDataLayer", "Client-Only Data Layer") : LOCTEXT("TokenMessage_DataLayers_ServerOnlyDataLayer", "Server-Only Data Layer")))
			->AddToken(FTextToken::Create(FText::FromString(DataLayerInstance->GetDataLayerShortName())))
			->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_CantBeChild", "can't be child of")))
			->AddToken(FTextToken::Create(FText::FromString(Parent->GetDataLayerShortName())))
			->AddToken(FMapErrorToken::Create(TEXT("DataLayers_HierarchyTypeMismatch_CheckForErrors")));
		break;
	case EDataLayerHierarchyInvalidReason::IncompatibleDataLayerType:
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_DataLayer", "Data layer")))
			->AddToken(FTextToken::Create(FText::FromString(DataLayerInstance->GetDataLayerShortName())))
			->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_IsEditor", "is Editor but its parent data layer")))
			->AddToken(FTextToken::Create(FText::FromString(Parent->GetDataLayerShortName())))
			->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_IsRuntime", "is Runtime")))
			->AddToken(FMapErrorToken::Create(TEXT("DataLayers_HierarchyTypeMismatch_CheckForErrors")));
		break;
	}
	
	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_Actor", "Actor")))
		->AddToken(FTextToken::Create(FText::FromString(WorldDataLayers->GetName())))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_CantReferenceDataLayer", "can't reference data layer")))
		->AddToken(FTextToken::Create(FText::FromString(DataLayerInstance->GetDataLayerShortName())))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_AssetReferenceRestrictions", "because of asset reference restrictions")))
		->AddToken(FTextToken::Create(Reason))
		->AddToken(FMapErrorToken::Create(TEXT("DataLayers_HierarchyTypeMismatch_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_DataLayerInstance", "Data layer Instance")))
		->AddToken(FUObjectToken::Create(DataLayerInstance))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_AndDataLayerInstance", "and Data Layer Instance")))
		->AddToken(FUObjectToken::Create(ConflictingDataLayerInstance))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_DataLayers_BothReferencing", "are both referencing Data Layer Asset")))
		->AddToken(FUObjectToken::Create(DataLayerInstance->GetAsset()))
		->AddToken(FMapErrorToken::Create(TEXT("DataLayers_AssetConflict_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_ActorNeedsResave", "Actor needs resave")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_ActorNeedsResave_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason)
{
	FSoftObjectPath ActorPath(ActorDescView.GetActorSoftPath());
	EMessageSeverity::Type MessageSeverity = EMessageSeverity::Info;
	FText ReasonText;

	switch (Reason)
	{
	case ELevelInstanceInvalidReason::WorldAssetNotFound:
		MessageSeverity = EMessageSeverity::Error;
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_HasInvalidWorldAsset", "has an invalid world asset");
		break;
	case ELevelInstanceInvalidReason::WorldAssetDontContainActorsMetadata:
		MessageSeverity = EMessageSeverity::Error;
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_WorldAssetIsNotUsingExternalActors", "is not using external actors, resave level to add compatibility");
		break;
	case ELevelInstanceInvalidReason::WorldAssetIncompatiblePartitioned:
		MessageSeverity = EMessageSeverity::Error;
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_WorldAssetIsPartitionedIncompatible", "is partitioned but not marked as compatible");
		break;
	case ELevelInstanceInvalidReason::WorldAssetHasInvalidContainer:
		MessageSeverity = EMessageSeverity::Error;
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_WorldAssetHasInvalidContainer", "has an invalid container");
		break;
	case ELevelInstanceInvalidReason::CirculalReference:
		MessageSeverity = EMessageSeverity::Error;
		ReasonText = LOCTEXT("TokenMessage_WorldPartition_WorldAssetHasCircularReference", "has a circular reference");
		break;
	};

	const FString WorldAssetStr = WorldAsset.ToString();
	const FString WorldAssetPath = WorldAssetStr + TEXT(".") + FPackageName::GetShortName(WorldAssetStr);

	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(MessageSeverity);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_LevelInstance", "Level Instance")))
		->AddToken(FTextToken::Create(FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(ReasonText))
		->AddToken(FAssetNameToken::Create(WorldAssetPath))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_LevelInstanceInvalidWorldAsset_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ReferenceActorDescView.GetActorSoftPath().ToString(), ReferenceActorDescView.GetGuid(), FText::FromString(GetActorName(ReferenceActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_ActorFilterFailed", "will not be filtered out because it is referenced by actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_MissingActorReference_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
		->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_HaveInvalidHLODLayer", "has an invalid HLOD layer")))
		->AddToken(FAssetNameToken::Create(ActorDescView.GetHLODLayer().ToString()))
		->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_InvalidActorHLODLayer_CheckForErrors")));

	HandleTokenizedMessage(MoveTemp(Message));
}

void ITokenizedMessageErrorHandler::OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason)
{
	FText LayerTypeText = HLODLayer ? StaticEnum<EHLODLayerType>()->GetDisplayNameTextByValue((int64)HLODLayer->GetLayerType()) : FText::FromString(TEXT("Unknown"));

	switch (Reason)
	{
	case EHLODLayerUnsupportedReason::ActorClassUnsupportedByLayerType:
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning);
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_Actor", "Actor")))
			->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("TokenMessage_WorldPartition_ActorClassUnsupportedByLayerType_Warning", "of class '{0}' is unsupported by HLOD Layer"), FText::FromString(ActorDescView.GetActorNativeClass()->GetName()))))
			->AddToken(FAssetNameToken::Create(ActorDescView.GetHLODLayer().ToString()))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("TokenMessage_WorldPartition_ActorClassUnsupportedByLayerType_LayerType", "of type '{0}'"), LayerTypeText)));
		HandleTokenizedMessage(MoveTemp(Message));
		break;
	}
}

void ITokenizedMessageErrorHandler::OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning);
	Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_MutatorRejected", "Mutator override rejected: cannot set actor")))
		->AddToken(FActorToken::Create(ActorDescView.GetActorSoftPath().ToString(), ActorDescView.GetGuid(), FText::FromString(GetActorName(ActorDescView))));

	switch (Reason)
	{
	case EActorDescViewMutatorRejectedReason::DivergesClusterRuntimeGrid:
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_MutatorRejected_RuntimeGridTo", "RuntimeGrid to")))
			->AddToken(FTextToken::Create(FText::FromName(RejectedOverrides.RuntimeGrid.GetValue())))
			->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_MutatorRejected_WouldSplitCluster", "because it would split a reference-connected cluster")));
		break;
	case EActorDescViewMutatorRejectedReason::DivergesClusterIsSpatiallyLoaded:
		Message->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_MutatorRejected_IsSpatiallyLoadedTo", "IsSpatiallyLoaded to")))
			->AddToken(FTextToken::Create(RejectedOverrides.bIsSpatiallyLoaded.GetValue()
				? LOCTEXT("TokenMessage_WorldPartition_True", "true")
				: LOCTEXT("TokenMessage_WorldPartition_False", "false")))
			->AddToken(FTextToken::Create(LOCTEXT("TokenMessage_WorldPartition_MutatorRejected_WouldSplitCluster", "because it would split a reference-connected cluster")));
		break;
	}

	Message->AddToken(FMapErrorToken::Create(TEXT("WorldPartition_ActorDescViewMutatorRejected_CheckForErrors")));
	HandleTokenizedMessage(MoveTemp(Message));
}

#undef LOCTEXT_NAMESPACE

#endif
