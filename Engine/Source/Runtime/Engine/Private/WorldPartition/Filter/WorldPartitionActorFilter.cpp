// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionActorFilter)

#if WITH_EDITORONLY_DATA
FWorldPartitionActorFilter::FOnWorldPartitionActorFilterChanged FWorldPartitionActorFilter::OnWorldPartitionActorFilterChanged;
#endif

FWorldPartitionActorFilter::~FWorldPartitionActorFilter()
{
#if WITH_EDITORONLY_DATA
	ClearChildFilters();
#endif
}

FWorldPartitionActorFilter::FWorldPartitionActorFilter(const FWorldPartitionActorFilter& Other)
{
	*this = Other;
}
FWorldPartitionActorFilter::FWorldPartitionActorFilter(FWorldPartitionActorFilter&& Other)
{
	*this = MoveTemp(Other);
}

FWorldPartitionActorFilter& FWorldPartitionActorFilter::operator=(const FWorldPartitionActorFilter& Other)
{
#if WITH_EDITORONLY_DATA
	Parent = nullptr;
	DisplayName = Other.DisplayName;
	DataLayerFilters = Other.DataLayerFilters;
	ClearChildFilters();
	for (const auto& [ActorGuid, FilterPtr] : Other.ChildFilters)
	{
		AddChildFilter(ActorGuid, new FWorldPartitionActorFilter(*FilterPtr));
	}
#endif
	return *this;
}

FWorldPartitionActorFilter& FWorldPartitionActorFilter::operator=(FWorldPartitionActorFilter&& Other)
{
#if WITH_EDITORONLY_DATA
	Parent = nullptr;
	DisplayName = MoveTemp(Other.DisplayName);
	DataLayerFilters = MoveTemp(Other.DataLayerFilters);
	ClearChildFilters();
	ChildFilters = MoveTemp(Other.ChildFilters);
	for (const auto& [ActorGuid, FilterPtr] : ChildFilters)
	{
		FilterPtr->Parent = this;
	}
#endif
	return *this;
}

#if WITH_EDITORONLY_DATA
void FWorldPartitionActorFilter::RequestFilterRefresh(bool bIsFromUserChange)
{
	IWorldPartitionActorLoaderInterface::RefreshLoadedState(bIsFromUserChange);
}

void FWorldPartitionActorFilter::AddChildFilter(const FGuid& InGuid, FWorldPartitionActorFilter* InChildFilter)
{
	InChildFilter->Parent = this;
	ChildFilters.Add(InGuid, InChildFilter);
}

void FWorldPartitionActorFilter::RemoveChildFilter(const FGuid& InGuid)
{
	FWorldPartitionActorFilter* ChildFilter = nullptr;
	if (ChildFilters.RemoveAndCopyValue(InGuid, ChildFilter))
	{
		delete ChildFilter;
	}
}

void FWorldPartitionActorFilter::ClearChildFilters()
{
	for (auto& [ActorGuid, ChildFilter] : ChildFilters)
	{
		delete ChildFilter;
	}
	ChildFilters.Empty();
}

void FWorldPartitionActorFilter::Override(const FWorldPartitionActorFilter& Other)
{
	for (const auto& [AssetPath, OtherDataLayerFilter] : Other.DataLayerFilters)
	{
		if (FDataLayerFilter* ExistingFilter = DataLayerFilters.Find(AssetPath))
		{
			ExistingFilter->bIncluded = OtherDataLayerFilter.bIncluded;
		}
	}
	
	for (const auto& [ActorGuid, OtherFilter] : Other.GetChildFilters())
	{
		if (FWorldPartitionActorFilter** ExistingFilter = ChildFilters.Find(ActorGuid))
		{
			(*ExistingFilter)->Override(*OtherFilter);
		}
	}
}

bool FWorldPartitionActorFilter::operator==(const FWorldPartitionActorFilter& Other) const
{
	if (DataLayerFilters.Num() != Other.DataLayerFilters.Num() || ChildFilters.Num() != Other.ChildFilters.Num())
	{
		return false;
	}

	for (const auto& [AssetPath, DataLayerFilter] : DataLayerFilters)
	{
		if (const FDataLayerFilter* OtherDataLayerFilter = Other.DataLayerFilters.Find(AssetPath))
		{
			if (DataLayerFilter.bIncluded != OtherDataLayerFilter->bIncluded)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	for (const auto& [ActorGuid, WorldPartitionActorFilter] : ChildFilters)
	{
		if (const FWorldPartitionActorFilter*const* OtherChildFilter = Other.GetChildFilters().Find(ActorGuid))
		{
			if (*WorldPartitionActorFilter != **OtherChildFilter)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FWorldPartitionActorFilter::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	uint32 DataLayerFilterCount = DataLayerFilters.Num();
	Ar << DataLayerFilterCount;

	if (Ar.IsLoading())
	{
		DataLayerFilters.Empty(DataLayerFilterCount);

		for (uint32 i = 0; i < DataLayerFilterCount; ++i)
		{
			FSoftObjectPath AssetPath;

			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeSoftObjectPathSupport || 
				Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorFilterStringAssetPath)
			{
				FString AssetPathStr;
				Ar << AssetPathStr;
				AssetPath = FSoftObjectPath(AssetPathStr);
			}
			else
			{
				Ar << AssetPath;
			}
			UAssetRegistryHelpers::FixupRedirectedAssetPath(AssetPath);

			bool bIncluded;
			Ar << bIncluded;
			
			DataLayerFilters.Add(AssetPath).bIncluded = bIncluded;
		}
	}
	else
	{
		for (auto& [AssetPath, DataLayerFilter] : DataLayerFilters)
		{
			FString AssetPathStr = AssetPath.ToString();
			Ar << AssetPathStr << DataLayerFilter.bIncluded;
		}
	}

	uint32 ChildFilterCount = ChildFilters.Num();
	Ar << ChildFilterCount;

	if (Ar.IsLoading())
	{
		// with undo/redo serialization we can be serializing over existing childs
		ClearChildFilters();

		for (uint32 i = 0; i < ChildFilterCount; ++i)
		{
			FGuid ActorGuid;
			Ar << ActorGuid;
			FWorldPartitionActorFilter* ChildFilter = new FWorldPartitionActorFilter();
			Ar << *ChildFilter;
			AddChildFilter(ActorGuid, ChildFilter);
		}
	}
	else
	{
		for (auto& [ActorGuid, ChildFilter] : ChildFilters)
		{
			Ar << ActorGuid;
			Ar << *ChildFilter;
		}
	}

	return true;
}

FArchive& operator<<(FArchive& Ar, FWorldPartitionActorFilter& Filter)
{
	Filter.Serialize(Ar);
	return Ar;
}

FString FWorldPartitionActorFilter::ToString() const
{
	FString ReturnValue;
	FWorldPartitionActorFilter Default;
	if (ExportTextItem(ReturnValue, Default, nullptr, 0, nullptr))
	{
		return ReturnValue;
	}

	return FString();
}

bool FWorldPartitionActorFilter::ExportTextItem(FString& ValueStr, FWorldPartitionActorFilter const& DefaultValue, UObject* ParentObject, int32 PortFlags, UObject* ExportRootScope) const
{
	const uint32 OriginalLen = ValueStr.Len();
	const FSoftObjectPath DefaultPath;

	ValueStr += TEXT("(");
	bool bFirst = true;
	
	for (const auto& [AssetPath, DataLayerFilter] : DataLayerFilters)
	{
		if (!bFirst)
		{
			ValueStr += TEXT(",");
		}
		bFirst = false;

		if (!AssetPath.ExportTextItem(ValueStr, DefaultPath, ParentObject, PortFlags, ExportRootScope))
		{
			ValueStr.LeftInline(OriginalLen);
			return false;
		}

		ValueStr += TEXT(";");
		ValueStr += DataLayerFilter.bIncluded ? TEXT("1") : TEXT("0");
	}
	ValueStr += TEXT(")(");

	bFirst = true;
	const FGuid DefaultGuid;
	for (const auto& [ActorGuid, WorldPartitionActorFilter] : ChildFilters)
	{
		if (!bFirst)
		{
			ValueStr += TEXT(",");
		}
		bFirst = false;
				
		if (!ActorGuid.ExportTextItem(ValueStr, DefaultGuid, ParentObject, PortFlags, ExportRootScope))
		{
			ValueStr.LeftChopInline(OriginalLen);
			return false;
		}
		ValueStr += TEXT(";");

		if (!WorldPartitionActorFilter->ExportTextItem(ValueStr, DefaultValue, ParentObject, PortFlags, ExportRootScope))
		{
			ValueStr.LeftInline(OriginalLen);
			return false;
		}
	}

	ValueStr += TEXT(")");
	return true;
}

bool FWorldPartitionActorFilter::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* ParentObject, FOutputDevice* ErrorText)
{
	FWorldPartitionActorFilter ImportedFilter;
	if (*Buffer != '(')
	{
		ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing opening \'(\' while importing property values."));

		// Parse error
		return false;
	}
	Buffer++;

	while (*Buffer != ')')
	{
		FSoftObjectPath AssetPath;
		if (!AssetPath.ImportTextItem(Buffer, PortFlags, ParentObject, ErrorText))
		{
			return false;
		}

		if (*Buffer != ';')
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing \';\' while importing property values."));
			return false;
		}
		Buffer++;
		bool bValue;
		if (*Buffer == '0' || *Buffer == '1')
		{
			bValue = *Buffer == '1';
		}
		else
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing \'0\' or \'1\' while importing property values."));
			return false;
		}
		Buffer++;

		if (*Buffer == ',')
		{
			Buffer++;
		}
		else if(*Buffer != ')')
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing \')\' while importing property values."));
			return false;
		}
		ImportedFilter.DataLayerFilters.Add(AssetPath).bIncluded = bValue;
	}
	Buffer++;
	
	if (*Buffer != '(')
	{
		ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing opening \'(\' while importing property values."));

		// Parse error
		return false;
	}
	Buffer++;

	while (*Buffer != ')')
	{
		FGuid ActorGuid;
		if (!ActorGuid.ImportTextItem(Buffer, PortFlags, ParentObject, ErrorText))
		{
			return false;
		}

		if (*Buffer != ';')
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing \';\' while importing property values."));
			return false;
		}
		Buffer++;

		FWorldPartitionActorFilter ChildFilter;
		if (!ChildFilter.ImportTextItem(Buffer, PortFlags, ParentObject, ErrorText))
		{
			return false;
		}

		if (*Buffer == ',')
		{
			Buffer++;
		}
		else if (*Buffer != ')')
		{
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("FWorldPartitionActorFilter: Missing \')\' while importing property values."));
			return false;
		}

		ImportedFilter.AddChildFilter(ActorGuid, new FWorldPartitionActorFilter(MoveTemp(ChildFilter)));
	}
	Buffer++;

	*this = MoveTemp(ImportedFilter);
	return true;
}

#endif 
