// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/CustomHLODActor.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ObjectSaveContext.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomHLODActor)

AWorldPartitionCustomHLOD::AWorldPartitionCustomHLOD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("RootComponent"));
	StaticMeshComponent->Mobility = EComponentMobility::Static;
	RootComponent = StaticMeshComponent;
}

UObject* AWorldPartitionCustomHLOD::GetUObject() const
{
	return const_cast<AWorldPartitionCustomHLOD*>(this);
}

ULevel* AWorldPartitionCustomHLOD::GetHLODLevel() const
{
	return GetLevel();
}

FString AWorldPartitionCustomHLOD::GetHLODNameOrLabel() const
{
	return GetActorNameOrLabel();
}

bool AWorldPartitionCustomHLOD::DoesRequireWarmup() const
{
	return true;
}

TSet<UObject*> AWorldPartitionCustomHLOD::GetAssetsToWarmup() const
{
	TSet<UObject*> AssetsToWarmup;
	
	ForEachComponent<UStaticMeshComponent>(false, [&](UStaticMeshComponent* SMC)
	{
		for (int32 iMaterialIndex = 0; iMaterialIndex < SMC->GetNumMaterials(); ++iMaterialIndex)
		{
			if (UMaterialInterface* Material = SMC->GetMaterial(iMaterialIndex))
			{
				AssetsToWarmup.Add(Material);
			}
		}

		if (UStaticMesh* StaticMesh = SMC->GetStaticMesh())
		{
			AssetsToWarmup.Add(StaticMesh);
		}
	});

	return AssetsToWarmup;
}

void AWorldPartitionCustomHLOD::SetVisibility(bool bInVisible)
{
	ForEachComponent<USceneComponent>(false, [bInVisible](USceneComponent* SceneComponent)
	{
		if (SceneComponent && (SceneComponent->GetVisibleFlag() != bInVisible))
		{
			SceneComponent->SetVisibility(bInVisible, false);
		}
	});
}

const FGuid& AWorldPartitionCustomHLOD::GetSourceCellGuid() const
{
	static const FGuid InvalidGuid;
	return InvalidGuid;
}

bool AWorldPartitionCustomHLOD::IsStandalone() const
{
	return false;
}

const FGuid& AWorldPartitionCustomHLOD::GetStandaloneHLODGuid() const
{
	static const FGuid InvalidGuid;
	return InvalidGuid;
}

bool AWorldPartitionCustomHLOD::IsCustomHLOD() const
{
	return true;
}

const FGuid& AWorldPartitionCustomHLOD::GetCustomHLODGuid() const
{
#if WITH_EDITOR
	return GetActorInstanceGuid();
#else
	return HLODInstanceGuid;
#endif
}

void AWorldPartitionCustomHLOD::BeginPlay()
{
	Super::BeginPlay();
	GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->RegisterHLODObject(this);
}

void AWorldPartitionCustomHLOD::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->UnregisterHLODObject(this);
	Super::EndPlay(EndPlayReason);
}

#if WITH_EDITOR
void AWorldPartitionCustomHLOD::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking())
	{
		HLODInstanceGuid = GetActorInstanceGuid();
	}

	SetActorEnableCollision(false);
}
#endif