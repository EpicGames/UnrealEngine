// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/DestructibleHLODComponent.h"
#include "WorldPartition/HLOD/HLODInstancedStaticMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Texture2DDynamic.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Net/UnrealNetwork.h"
#include "RenderingThread.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DestructibleHLODComponent)

#define LOCTEXT_NAMESPACE "DestructibleHLODComponent"

DEFINE_LOG_CATEGORY_STATIC(LogHLODDestruction, Log, All);

UWorldPartitionDestructibleHLODComponent::UWorldPartitionDestructibleHLODComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VisibilityTextureSize(0)
{
	SetIsReplicatedByDefault(true);
}

const TArray<FName>& UWorldPartitionDestructibleHLODComponent::GetDestructibleActors() const
{
	return DestructibleActors;
}

void FWorldPartitionDestructibleHLODState::Initialize(UWorldPartitionDestructibleHLODComponent* InDestructibleHLODComponent)
{
	OwnerComponent = InDestructibleHLODComponent;
	
	NumDestructibleActors = InDestructibleHLODComponent->GetDestructibleActors().Num();

	const ENetMode NetMode = OwnerComponent->GetNetMode();
	bIsClient = NetMode != ENetMode::NM_DedicatedServer;
	bIsServer = NetMode == ENetMode::NM_DedicatedServer || NetMode == ENetMode::NM_ListenServer;

	if (bIsServer)
	{
		DamagedActors.Reserve(NumDestructibleActors);
		ActorsToDamagedActorsMapping.SetNum(NumDestructibleActors);
		for (auto& Entry : ActorsToDamagedActorsMapping)
		{
			Entry = INDEX_NONE;
		}
	}
	
	if (bIsClient)
	{
		// In case replication occured before this initialization, process all entries in the DamagedActors array
		if (!DamagedActors.IsEmpty())
		{
			for (int32 DamagedActorIndex = 0; DamagedActorIndex < DamagedActors.Num(); ++DamagedActorIndex)
			{
				ApplyDamagedActorState(DamagedActorIndex);
			}

			OwnerComponent->OnDestructionStateUpdated();
		}
	}
}

void FWorldPartitionDestructibleHLODState::SetActorHealth(int32 InActorIndex, uint8 InActorHealth)
{
	if (InActorIndex < 0 || InActorIndex >= NumDestructibleActors)
	{
		UE_LOGF(LogHLODDestruction, Error, "Invalid actor index provided to SetActorHealth() (%d, max = %d)", InActorIndex, NumDestructibleActors);
		return;
	}

	// If we are the server, replicate damage to clients
	if (IsServer())
	{
		// Perform replication only if necessary
		bool bReplicationNeeded = false;
		auto ReplicationNeeded = [this, &bReplicationNeeded]()
		{
			if (!bReplicationNeeded)
			{
				OwnerComponent->GetOwner()->FlushNetDormancy();
				bReplicationNeeded = true;
			}
		};

		int32 DamagedActorIdx = ActorsToDamagedActorsMapping[InActorIndex];

		// Add a new damaged actor entry
		if (DamagedActorIdx == INDEX_NONE)
		{
			ReplicationNeeded();
			DamagedActorIdx = DamagedActors.Num();
			ActorsToDamagedActorsMapping[InActorIndex] = DamagedActorIdx;
			DamagedActors.Emplace(InActorIndex);
		}

		FWorldPartitionDestructibleHLODDamagedActorState& DamagedActor = DamagedActors[DamagedActorIdx];
		check(DamagedActor.ActorIndex == InActorIndex);
		if (DamagedActor.ActorHealth != InActorHealth)
		{
			ReplicationNeeded();
			DamagedActor.ActorHealth = InActorHealth;
		}

		if (bReplicationNeeded)
		{
			MarkItemDirty(DamagedActor);
		}
	}
}

void FWorldPartitionDestructibleHLODState::PostReplicatedAdd(const TArrayView<int32>& AddedIndices, int32 FinalSize)
{
	PostReplicatedChange(AddedIndices, FinalSize);
}

void FWorldPartitionDestructibleHLODState::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	// Component may still be null if replication occurs before the component BeginPlay() is called, 
	// in which case FWorldPartitionDestructibleHLODState::Initialize() will not have been called.
	// DamagedActors will be process directly in Initialize() if that's the case
	if (OwnerComponent)
	{
		check(IsClient());

		for (int32 ChangedIndex : ChangedIndices)
		{			
			ApplyDamagedActorState(ChangedIndex);
		}

		OwnerComponent->OnDestructionStateUpdated();
	}
}

void FWorldPartitionDestructibleHLODState::ApplyDamagedActorState(int32 DamagedActorIndex)
{
	if (DamagedActors.IsValidIndex(DamagedActorIndex))
	{
		const FWorldPartitionDestructibleHLODDamagedActorState& DamagedActorState = DamagedActors[DamagedActorIndex];
		OwnerComponent->ApplyDamagedActorState(DamagedActorState.ActorIndex, DamagedActorState.ActorHealth);
	}
	else
	{
		UE_LOGF(LogHLODDestruction, Error, "Invalid damaged actor index %d (num = %d)", DamagedActorIndex, DamagedActors.Num());
	}
}

void UWorldPartitionDestructibleHLODComponent::GetLifetimeReplicatedProps(TArray< class FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	// Turn on push based replication for low-frequency variables. This saves the server time in determining what to replicate.
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(UWorldPartitionDestructibleHLODComponent, DestructibleHLODState, Params);
}

void UWorldPartitionDestructibleHLODComponent::BeginPlay()
{
	Super::BeginPlay();

	const bool bIsClient = GetNetMode() != ENetMode::NM_DedicatedServer;
	if (bIsClient)
	{
		VisibilityBuffer.SetNumUninitialized(FMath::RoundUpToPowerOfTwo(GetDestructibleActors().Num()));
		FMemory::Memset(VisibilityBuffer.GetData(), FWorldPartitionDestructibleHLODDamagedActorState::MAX_HEALTH, VisibilityBuffer.Num());
	}

	DestructibleHLODState.Initialize(this);
}

void UWorldPartitionDestructibleHLODComponent::OnDestructionStateUpdated()
{
	UpdateVisibilityTexture();
}

void UWorldPartitionDestructibleHLODComponent::DestroyActor(int32 ActorIndex)
{
	DamageActor(ActorIndex, 0.0f);
}

void UWorldPartitionDestructibleHLODComponent::DamageActor(int32 ActorIndex, float RemainingHealthPercent)
{
	// Set percentage health as uint8 in the visibility texture
	const uint8 ActorHealth = FMath::Clamp(RemainingHealthPercent, 0, 1) * FWorldPartitionDestructibleHLODDamagedActorState::MAX_HEALTH;

	DestructibleHLODState.SetActorHealth(ActorIndex, ActorHealth);

	// If we are the client, directly update the visibility buffer
	const bool bIsClient = GetNetMode() != ENetMode::NM_DedicatedServer;
	if (bIsClient && FApp::CanEverRender())
	{
		ApplyDamagedActorState(ActorIndex, ActorHealth);
		OnDestructionStateUpdated();
	}
}

void UWorldPartitionDestructibleHLODComponent::ApplyDamagedActorState(int32 ActorIndex, const uint8 ActorHealth)
{
	if (VisibilityBuffer.IsValidIndex(ActorIndex))
	{
		VisibilityBuffer[ActorIndex] = ActorHealth;
	}

	if (ActorHealth == 0)
	{
		static const FTransform ZeroedTransform = FTransform::Identity.GetScaled(0);

		// For each HLOD ISM component in which the destroyed actor has instances
		DestructibleHLODInstancesMappingData.ForEachActorInstancingMapping(ActorIndex, [](UHLODInstancedStaticMeshComponent* HLODComponent, uint32 InstancesStart, uint32 InstancesCount)
		{
			// Clear all instances associated with the actor
			if (ensure(IsValid(HLODComponent)))
			{
				HLODComponent->BatchUpdateInstancesTransform(InstancesStart, InstancesCount, ZeroedTransform, false, true, false);
			}
		});
	}
}

void UWorldPartitionDestructibleHLODComponent::SetupVisibilityTexture()
{
	if (!VisibilityTexture && DestructibleHLODMaterial)
	{
		// Retrieve number of instance stored inside of this LOD actor
		float NumberOfInstances = 0.0f;
		if (DestructibleHLODMaterial->GetScalarParameterValue(FMaterialParameterInfo("NumInstances"), NumberOfInstances, true))
		{
			// Create dynamic texture size of (NumInstances, 1) if required
			uint32 TextureSize = FMath::TruncToInt(NumberOfInstances);

			FTexture2DDynamicCreateInfo CreateInfo;
			CreateInfo.Format = PF_G8;
			CreateInfo.Filter = TF_Nearest;
			CreateInfo.SamplerAddressMode = AM_Clamp;
			CreateInfo.bSRGB = false;

			UTexture2DDynamic* DynamicInstanceTexture = UTexture2DDynamic::Create(TextureSize, 1, CreateInfo);
			if (DynamicInstanceTexture)
			{
				VisibilityTexture = DynamicInstanceTexture;
				VisibilityTextureSize = TextureSize;
			}

			if (VisibilityTexture)
			{
				// Create dynamic material instance if required and set it to us the dynamic texture
				UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(DestructibleHLODMaterial, this);
				MaterialInstance->SetTextureParameterValue("InstanceVisibilityTexture", VisibilityTexture);
				VisibilityMaterial = MaterialInstance;
			}

			if (VisibilityMaterial)
			{
				// For components that are referencing the destructible material, override it with our MID
				GetOwner()->ForEachComponent<UStaticMeshComponent>(false, [this](UStaticMeshComponent* SMComponent)
				{
					if (UMaterialInterface* Material = SMComponent->GetMaterial(0))
					{
						if (Material == DestructibleHLODMaterial)
						{
							SMComponent->SetMaterial(0, VisibilityMaterial);
						}
					}
				});
			}
		}
	}
}

void UWorldPartitionDestructibleHLODComponent::UpdateVisibilityTexture()
{
	if (!DestructibleHLODState.IsClient() || !FApp::CanEverRender())
	{
		return;
	}

	SetupVisibilityTexture();

	if (VisibilityTexture)
	{
		FTexture2DDynamicResource* TextureResource = static_cast<FTexture2DDynamicResource*>(VisibilityTexture->GetResource());

		if (FApp::CanEverRender() && ensure(TextureResource))
		{
			ENQUEUE_RENDER_COMMAND(FUpdateHLODVisibilityTexture)(
				[TextureResource, VisibilityTextureSize = VisibilityTextureSize, VisibilityBuffer = VisibilityBuffer](FRHICommandListImmediate& RHICmdList)
				{
					FRHITexture* TextureRHI = TextureResource->GetTexture2DRHI();
					FRHILockTextureArgs LockArgs = FRHILockTextureArgs::Lock2D(TextureRHI, 0, RLM_WriteOnly, false, false);
					FRHILockTextureResult LockResult = RHICmdList.LockTexture(LockArgs);
					{
						checkf(VisibilityTextureSize <= LockResult.Stride, TEXT("UpdateVisibilityTexture: copy dest (%d) is smaller than source (%d)"), LockResult.Stride, VisibilityTextureSize);
						checkf(VisibilityTextureSize <= (uint32)VisibilityBuffer.Num(), TEXT("UpdateVisibilityTexture: Visibility texture size (%d) is greater than the visibility buffer size (%d)"), VisibilityTextureSize, VisibilityBuffer.Num());
						uint32 CopyLen = FMath::Min(VisibilityTextureSize, FMath::Min(LockResult.Stride, (uint32)VisibilityBuffer.Num()));
						FMemory::Memcpy((uint8*)LockResult.Data, VisibilityBuffer.GetData(), CopyLen);
					}
					RHICmdList.UnlockTexture(LockArgs);
				});
		}
	}
}

#if WITH_EDITOR
void UWorldPartitionDestructibleHLODComponent::SetDestructibleActors(const TArray<FName>& InDestructibleActors)
{
	DestructibleActors = InDestructibleActors;
}

void UWorldPartitionDestructibleHLODComponent::SetDestructibleHLODMaterial(UMaterialInterface* InDestructibleMaterial)
{
	DestructibleHLODMaterial = InDestructibleMaterial;
}

void UWorldPartitionDestructibleHLODComponent::SetHLODInstancingPackedMappingData(FHLODInstancingPackedMappingData&& InHLODInstancingPackedMappingData)
{
	DestructibleHLODInstancesMappingData = MoveTemp(InHLODInstancingPackedMappingData);
}
#endif

UDEPRECATED_UWorldPartitionDestructibleHLODMeshComponent::UDEPRECATED_UWorldPartitionDestructibleHLODMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE
