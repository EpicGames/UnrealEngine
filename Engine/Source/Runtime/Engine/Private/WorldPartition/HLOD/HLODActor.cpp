// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODActor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"
#include "Components/PrimitiveComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Misc/PackageName.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODActor)

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Engine/StaticMeshSocket.h"
#include "Engine/Texture.h"
#include "HAL/FileManager.h"
#include "MeshDescription.h"
#include "Misc/ArchiveMD5.h"
#include "Modules/ModuleManager.h"
#include "PhysicsEngine/BodySetup.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODStats.h"
#include "WorldPartition/HLOD/HLODUtils.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"
#include "WorldPartition/HLOD/HLODSourceActorsFromCell.h"
#endif

DEFINE_LOG_CATEGORY(LogHLODHash);

static int32 GWorldPartitionHLODForceDisableShadows = 0;
static FAutoConsoleVariableRef CVarWorldPartitionHLODForceDisableShadows(
	TEXT("wp.Runtime.HLOD.ForceDisableShadows"),
	GWorldPartitionHLODForceDisableShadows,
	TEXT("Force disable CastShadow flag on World Partition HLOD actors"),
	ECVF_Scalability);

#if WITH_EDITOR
const FName AWorldPartitionHLOD::NAME_HLODHash_AssetTag(TEXT("HLODActor_HLODHash"));
FWorldPartitionHLODBuildEventDelegate AWorldPartitionHLOD::HLODBuildEventDelegate;
FWorldPartitionHLODActorEditorRegisteredDelegate AWorldPartitionHLOD::HLODActorEditorRegisteredDelegate;
FWorldPartitionHLODActorEditorUnregisteredDelegate AWorldPartitionHLOD::HLODActorEditorUnregisteredDelegate;
#endif

AWorldPartitionHLOD::AWorldPartitionHLOD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRequireWarmup(false)
{
	SetCanBeDamaged(false);
	SetActorEnableCollision(false);

	// Set HLOD actors to replicate by default, since the CDO's GetIsReplicated() is used to tell if a class type might replicate or not.
	// The real need for replication will be adjusted depending on the presence of owned components that
	// needs to be replicated.
	bReplicates = true;

	NetDormancy = DORM_Initial;
	SetNetUpdateFrequency(1.f);

#if WITH_EDITORONLY_DATA
	HLODBounds = FBox(EForceInit::ForceInit);
#endif

#if WITH_EDITOR
	if (!IsInAsyncLoadingThread())
	{
		FWorldDelegates::OnWorldCleanup.AddUObject(this, &AWorldPartitionHLOD::OnWorldCleanup);
	}
#endif
}

ULevel* AWorldPartitionHLOD::GetHLODLevel() const
{
	return GetLevel();
}

FString AWorldPartitionHLOD::GetHLODNameOrLabel() const
{
	return GetActorNameOrLabel();
}

const FGuid& AWorldPartitionHLOD::GetSourceCellGuid() const
{
	// When no source cell guid was set, try resolving it through its associated world partition runtime cell
	// This is necessary for any HLOD actor part of a level that is instanced multiple times (shared amongst multiple cells)
	if (!SourceCellGuid.IsValid())
	{
		const UWorldPartitionRuntimeCell* Cell = Cast<UWorldPartitionRuntimeCell>(GetLevel()->GetWorldPartitionRuntimeCell());
		if (Cell && Cell->GetIsHLOD())
		{
			const_cast<AWorldPartitionHLOD*>(this)->SourceCellGuid = Cell->GetSourceCellGuid();
		}
	}
	return SourceCellGuid;
}

bool AWorldPartitionHLOD::IsStandalone() const
{
	return StandaloneHLODGuid.IsValid();
}

const FGuid& AWorldPartitionHLOD::GetStandaloneHLODGuid() const
{
#if WITH_EDITOR
	if (IsStandalone())
	{
		return GetActorInstanceGuid();
	}
#endif
	return StandaloneHLODGuid;
}

bool AWorldPartitionHLOD::IsCustomHLOD() const
{
	return false;
}

const FGuid& AWorldPartitionHLOD::GetCustomHLODGuid() const
{
	static const FGuid InvalidGuid;
	return InvalidGuid;
}

void AWorldPartitionHLOD::SetVisibility(bool bInVisible)
{
	ForEachComponent<USceneComponent>(false, [bInVisible](USceneComponent* SceneComponent)
	{
		if (SceneComponent && (SceneComponent->GetVisibleFlag() != bInVisible))
		{
			SceneComponent->SetVisibility(bInVisible, false);
		}
	});
}

TSet<UObject*> AWorldPartitionHLOD::GetAssetsToWarmup() const
{
	TSet<UObject*> AssetsToWarmup;
	
	ForEachComponent<UStaticMeshComponent>(false, [&](UStaticMeshComponent* SMC)
	{
		// Assume ISM HLOD don't need warmup, as they are actually found in the source level
		if (SMC->IsA<UInstancedStaticMeshComponent>())
		{
			return;
		}

		for (int32 iMaterialIndex = 0; iMaterialIndex < SMC->GetNumMaterials(); ++iMaterialIndex)
		{
			if (UMaterialInterface* Material = SMC->GetMaterial(iMaterialIndex))
			{
				AssetsToWarmup.Add(Material);
			}
		}

		if (UStaticMesh* StaticMesh = SMC->GetStaticMesh())
		{
			AssetsToWarmup.Add(StaticMesh);
		}
	});

	return AssetsToWarmup;
}

void AWorldPartitionHLOD::BeginPlay()
{
	Super::BeginPlay();
	GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->RegisterHLODObject(this);
}

void AWorldPartitionHLOD::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->UnregisterHLODObject(this);
	Super::EndPlay(EndPlayReason);
}

void AWorldPartitionHLOD::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionStreamingCellsNamingShortened)
		{
			SourceCell_DEPRECATED = SourceCell_DEPRECATED.ToString().Replace(TEXT("WPRT_"), TEXT(""), ESearchCase::CaseSensitive).Replace(TEXT("Cell_"), TEXT(""), ESearchCase::CaseSensitive);
		}

		if (Ar.CustomVer(FUE5SpecialProjectStreamObjectVersion::GUID) < FUE5SpecialProjectStreamObjectVersion::ConvertWorldPartitionHLODsCellsToName)
		{
			FString CellName;
			FString CellContext;
			const FString CellPath = FPackageName::GetShortName(SourceCell_DEPRECATED.ToSoftObjectPath().GetSubPathString());
			if (!CellPath.Split(TEXT("."), &CellContext, &CellName, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
			{
				CellName = CellPath;
			}
			SourceCellName_DEPRECATED = *CellName;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODSourceActorsRefactor)
		{
			check(!SourceActors)
			UWorldPartitionHLODSourceActorsFromCell* SourceActorsFromCell = NewObject<UWorldPartitionHLODSourceActorsFromCell>(this);
			SourceActorsFromCell->SetActors(MoveTemp(HLODSubActors_DEPRECATED));
			SourceActorsFromCell->SetHLODLayer(SubActorsHLODLayer_DEPRECATED);
			SourceActors = SourceActorsFromCell;
		}
	}
#endif
}

bool AWorldPartitionHLOD::IsEditorOnly() const
{
	// Treat HLOD actors which were never built (or failed to build components for various reasons) as editor only.
	if (!IsTemplate() && GetRootComponent() == nullptr)
	{
		return true;
	}

	return Super::IsEditorOnly();
}

bool AWorldPartitionHLOD::NeedsLoadForServer() const
{
	// Only needed on server if this HLOD actor has anything to replicate to clients
	return GetIsReplicated();
}

void AWorldPartitionHLOD::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionStreamingCellsNamingShortened)
	{
		if (!HLODSubActors_DEPRECATED.IsEmpty())
		{
			// As we may be dealing with an unsaved world created from a template map, get
			// the source package name of this HLOD actor and figure out the world name from there
			FName ExternalActorsPath = HLODSubActors_DEPRECATED[0].ContainerPackage;
			FString WorldName = FPackageName::GetShortName(ExternalActorsPath);

			// Strip "WorldName_" from the cell name
			FString CellName = SourceCellName_DEPRECATED.ToString();
			bool bRemoved = CellName.RemoveFromStart(WorldName + TEXT("_"), ESearchCase::CaseSensitive);
			if (bRemoved)
			{
				SourceCellName_DEPRECATED = *CellName;
			}
		}
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODActorUseSourceCellGuid)
	{
		check(!SourceCellName_DEPRECATED.IsNone());
		check(!SourceCellGuid.IsValid());

		FString GridName;
		int64 CellGlobalCoord[3];
		uint32 DataLayerID;
		uint32 ContentBundleID;

		// Input format should be GridName_Lx_Xx_Yx_DLx[_CBx]
		TArray<FString> Tokens;
		if (SourceCellName_DEPRECATED.ToString().ParseIntoArray(Tokens, TEXT("_")) >= 4)
		{
			int32 CurrentIndex = 0;
			GridName = Tokens[CurrentIndex++];

			// Since GridName can contain underscores, we do our best to extract it
			while(Tokens.IsValidIndex(CurrentIndex))
			{
				if ((Tokens[CurrentIndex][0] == TEXT('L')) && (Tokens[CurrentIndex].Len() > 1))
				{
					if (FCString::IsNumeric(*Tokens[CurrentIndex] + 1))
					{
						break;
					}
				}

				GridName += TEXT("_");
				GridName += Tokens[CurrentIndex++];
			}

			GridName = GridName.ToLower();

			CellGlobalCoord[2] = Tokens.IsValidIndex(CurrentIndex) ? FCString::Strtoui64(*Tokens[CurrentIndex++] + 1, nullptr, 10) : 0;
			CellGlobalCoord[0] = Tokens.IsValidIndex(CurrentIndex) ? FCString::Strtoui64(*Tokens[CurrentIndex++] + 1, nullptr, 10) : 0;
			CellGlobalCoord[1] = Tokens.IsValidIndex(CurrentIndex) ? FCString::Strtoui64(*Tokens[CurrentIndex++] + 1, nullptr, 10) : 0;
			DataLayerID = Tokens.IsValidIndex(CurrentIndex) ? FCString::Strtoui64(*Tokens[CurrentIndex++] + 2, nullptr, 16) : 0;
			ContentBundleID = Tokens.IsValidIndex(CurrentIndex) ? FCString::Strtoui64(*Tokens[CurrentIndex++] + 2, nullptr, 16) : 0;
		}

		FArchiveMD5 ArMD5;
		ArMD5 << GridName << CellGlobalCoord[0] << CellGlobalCoord[1] << CellGlobalCoord[2] << DataLayerID << ContentBundleID;

		SourceCellGuid = ArMD5.GetGuidFromHash();
		check(SourceCellGuid.IsValid());
	}

	// CellGuid taking the cell size into account
	if (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::WorldPartitionRuntimeCellGuidWithCellSize)
	{
		if (SourceCellGuid.IsValid())
		{
			int32 CellSize = (int32)FMath::RoundToInt(HLODBounds.GetSize().X);

			FArchiveMD5 ArMD5;
			ArMD5 << SourceCellGuid << CellSize;

			SourceCellGuid = ArMD5.GetGuidFromHash();
			check(SourceCellGuid.IsValid());
		}
	}

	FWorldDelegates::OnWorldCleanup.AddUObject(this, &AWorldPartitionHLOD::OnWorldCleanup);
#endif
}

#if WITH_EDITOR
void AWorldPartitionHLOD::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (ObjectSaveContext.IsFromAutoSave())
	{
		return;
	}

	// Always disable collisions on HLODs
	SetActorEnableCollision(false);

	ForEachComponent<UPrimitiveComponent>(false, [this, &ObjectSaveContext](UPrimitiveComponent* PrimitiveComponent)
	{
		// Disable collision on HLOD components
		PrimitiveComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

		// When cooking, get rid of collision data
		if (ObjectSaveContext.IsCooking())
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
			{
				if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					// If the HLOD process did create this static mesh
					if (StaticMesh->GetPackage() == GetPackage())
					{
						UE::Private::HLOD::RemoveCollisionData(StaticMesh);
					}
				}
			}
		}
	});

	if (ObjectSaveContext.IsCooking() && IsStandalone())
	{
		StandaloneHLODGuid = GetActorInstanceGuid();
	}
}
#endif

void AWorldPartitionHLOD::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<AWorldPartitionHLOD>(this))
	{
		CVarWorldPartitionHLODForceDisableShadows->OnChangedDelegate().RemoveAll(this);
		CVarWorldPartitionHLODForceDisableShadows->OnChangedDelegate().AddUObject(this, &AWorldPartitionHLOD::OnForceDisableShadowsCVarChanged);
	}
}

void AWorldPartitionHLOD::BeginDestroy()
{
	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<AWorldPartitionHLOD>(this))
	{
		CVarWorldPartitionHLODForceDisableShadows->OnChangedDelegate().RemoveAll(this);
	}

	Super::BeginDestroy();
}

void AWorldPartitionHLOD::OnForceDisableShadowsCVarChanged(IConsoleVariable* CVar)
{
	const bool bShouldCastShadows = (CVar != nullptr) ? !CVar->GetBool() : true;

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		UWorld* World = WorldContext.World();
		if (!World || !World->IsGameWorld())
		{
			continue;
		}

		for (TActorIterator<AWorldPartitionHLOD> It(World); It; ++It)
		{
			AWorldPartitionHLOD * HLODActor = *It;
			if (IsValid(HLODActor))
			{
				HLODActor->ApplyCastShadowToComponents(bShouldCastShadows);
			}
		}
	}
}

void AWorldPartitionHLOD::ApplyCastShadowToComponents(bool bInShouldCastShadows)
{
	ForEachComponent<UPrimitiveComponent>(false, [bInShouldCastShadows](UPrimitiveComponent* PrimitiveComponent)
	{
		PrimitiveComponent->SetCastShadow(bInShouldCastShadows);
	});
}

void AWorldPartitionHLOD::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	if (GWorldPartitionHLODForceDisableShadows && GetWorld() && GetWorld()->IsGameWorld())
	{
		ApplyCastShadowToComponents(false);
	}

#if WITH_EDITOR
	// In editor, turn on collision on HLODs in order to enable some useful editor features on HLODs (Actor placement, Play from here, Go Here, etc)
	// In PIE, collisions should be disabled
	if (GetWorld() && !IsRunningCommandlet() && !FApp::IsUnattended())
	{
		bool bShouldEnableCollision = !GetWorld()->IsGameWorld();
		if (GetActorEnableCollision() != bShouldEnableCollision)
		{
			SetActorEnableCollision(bShouldEnableCollision);
			ForEachComponent<UPrimitiveComponent>(false, [bShouldEnableCollision](UPrimitiveComponent* PrimitiveComponent)
			{
				bool bShouldEnableCollisionForComponent = bShouldEnableCollision;
				if (bShouldEnableCollisionForComponent)
				{
					if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
					{
						UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
						int32 NumSectionsWithCollision = StaticMesh ? StaticMesh->GetNumSectionsWithCollision() : 0;
						int32 NumCollisionPrims = StaticMesh ? (StaticMesh->GetBodySetup() ? StaticMesh->GetBodySetup()->AggGeom.GetElementCount() : 0) : 0;
						bShouldEnableCollisionForComponent = NumSectionsWithCollision != 0 || NumCollisionPrims != 0;
					}
				}

				PrimitiveComponent->SetCollisionEnabled(bShouldEnableCollisionForComponent ? ECollisionEnabled::QueryOnly : ECollisionEnabled::NoCollision);
				PrimitiveComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
				PrimitiveComponent->SetCollisionResponseToChannel(ECC_Visibility, bShouldEnableCollisionForComponent ? ECR_Block : ECR_Ignore);
				PrimitiveComponent->SetCollisionResponseToChannel(ECC_Camera, bShouldEnableCollisionForComponent ? ECR_Block : ECR_Ignore);
			});
		}
	}	
#endif

	// If world is instanced, we need to recompute our bounds since they are in the instanced-world space
	if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(this))
	{
		const bool bIsInstancedLevel = WorldPartition->GetTypedOuter<ULevel>()->IsInstancedLevel();
		if (bIsInstancedLevel)
		{
			ForEachComponent<USceneComponent>(false, [](USceneComponent* SceneComponent)
			{
				// Clear bComputedBoundsOnceForGame so that the bounds are recomputed once
				SceneComponent->bComputedBoundsOnceForGame = false;
			});
		}
	}
}

void AWorldPartitionHLOD::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();
#if WITH_EDITOR
	if (UWorld* World = GetWorld())
	{
		if (World->IsEditorWorld())
		{
			HLODActorEditorRegisteredDelegate.Broadcast(this);
		}
	}
#endif
}

void AWorldPartitionHLOD::PostUnregisterAllComponents()
{
#if WITH_EDITOR
	if (UWorld* World = GetWorld())
	{
		if (World->IsEditorWorld())
		{
			HLODActorEditorUnregisteredDelegate.Broadcast(this);
		}
	}
#endif
	Super::PostUnregisterAllComponents();
}

#if WITH_EDITOR
void AWorldPartitionHLOD::RerunConstructionScripts()
{}

void AWorldPartitionHLOD::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	// Close all asset editors associated with this HLOD actor
	const UWorld* World = GetWorld();
	if (World == InWorld)
	{
		if (!World->IsGameWorld())
		{
			UPackage* HLODResourcesPackage = GetHLODResourcesPackage();

			// Find all assets being edited
			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			TArray<UObject*> AllAssets = AssetEditorSubsystem->GetAllEditedAssets();

			for (UObject* Asset : AllAssets)
			{
				if (Asset->GetPackage() == HLODResourcesPackage)
				{
					AssetEditorSubsystem->CloseAllEditorsForAsset(Asset);
				}
			}
		}
	}
}

TUniquePtr<FWorldPartitionActorDesc> AWorldPartitionHLOD::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FHLODActorDesc());
}

void AWorldPartitionHLOD::SetHLODComponents(const TArray<UActorComponent*>& InHLODComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AWorldPartitionHLOD::SetHLODComponents);

	Modify();

	TArray<UActorComponent*> ComponentsToRemove = GetInstanceComponents();
	for (UActorComponent* ComponentToRemove : ComponentsToRemove)
	{
		if (ComponentToRemove)
		{
			ComponentToRemove->DestroyComponent();
		}
	}

	// We'll turn on replication for this actor only if it contains a replicated component
	check(!IsActorInitialized());
	bReplicates = false;

	for(UActorComponent* Component : InHLODComponents)
	{
		Component->Rename(nullptr, this);
		AddInstanceComponent(Component);

		const bool ComponentReplicates = Component->GetIsReplicated();
		bReplicates |= ComponentReplicates;

		// Avoid using a dummy scene root component (for efficiency), choose one component as the root
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			// If we have one, prefer a replicated component as our root.
			// This is required, otherwise the actor won't even be considered for replication
			if (!RootComponent || (!RootComponent->GetIsReplicated() && ComponentReplicates))
			{
				RootComponent = SceneComponent;
			}
		}
	
		Component->RegisterComponent();
	}

	// Attach all scene components to our root.
	const bool bIncludeFromChildActors = false;
	ForEachComponent<USceneComponent>(bIncludeFromChildActors, [&](USceneComponent* Component)
	{
		// Skip the root component
		if (Component != GetRootComponent())
		{
			// Keep world transform intact while attaching to root component
			Component->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);
		}
	});
}

void AWorldPartitionHLOD::SetSourceActors(UWorldPartitionHLODSourceActors* InHLODSourceActors)
{
	SourceActors = InHLODSourceActors;
	InputStats.BuildersReferencedAssets.Reset();
}

const UWorldPartitionHLODSourceActors* AWorldPartitionHLOD::GetSourceActors() const
{
	return SourceActors;
}

UWorldPartitionHLODSourceActors* AWorldPartitionHLOD::GetSourceActors()
{
	return SourceActors;
}

void AWorldPartitionHLOD::SetInputStats(const FHLODBuildInputStats& InInputStats)
{
	InputStats = InInputStats;
}

const FHLODBuildInputStats& AWorldPartitionHLOD::GetInputStats() const
{
	return InputStats;
}

void AWorldPartitionHLOD::SetSourceCellGuid(const FGuid& InSourceCellGuid)
{
	SourceCellGuid = InSourceCellGuid;
}

void AWorldPartitionHLOD::SetIsStandalone(bool bInIsStandalone)
{
	StandaloneHLODGuid = bInIsStandalone ? GetActorGuid() : FGuid();
}

const FBox& AWorldPartitionHLOD::GetHLODBounds() const
{
	return HLODBounds;
}

void AWorldPartitionHLOD::SetHLODBounds(const FBox& InBounds)
{
	HLODBounds = InBounds;
}

UPackage* AWorldPartitionHLOD::GetHLODResourcesPackage() const
{
	if (!HLODResourcesPackagePath.IsNone())
	{
		// If HLODResourcesPackagePath is set, HLOD resources are in that package
		return FindPackage(nullptr, *HLODResourcesPackagePath.ToString());
	}
	// If HLODResourcesPackagePath is not set, HLOD resources are in the actor's package
	return GetPackage();
}

void AWorldPartitionHLOD::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	OutRuntimeBounds = OutEditorBounds = HLODBounds;
}

int64 AWorldPartitionHLOD::GetStat(FName InStatName) const
{
	if (InStatName == FWorldPartitionHLODStats::MemoryDiskSizeBytes)
	{
		const FString PackageFileName = GetPackage()->GetLoadedPath().GetLocalFullPath();
		return IFileManager::Get().FileSize(*PackageFileName);
	}
	return HLODStats.FindRef(InStatName);
}

bool AWorldPartitionHLOD::BuildHLOD(bool bForceBuild)
{
	bool bBuiltHLOD = false;

	GetHLODBuildEventDelegate().Broadcast({ FWorldPartitionHLODBuildEvent::EEventType::BeginBuild, this });

	IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
	if (IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr)
	{
		FHLODBuildParams BuildParams;
		BuildParams.HLODActor = this;
		BuildParams.bForceBuild = bForceBuild;
		bBuiltHLOD = WPHLODUtilities->BuildHLOD(BuildParams);
	}

	// When generating WorldPartition HLODs, we have the renderer initialized.
	// Take advantage of this and generate texture streaming built data (local to the actor).
	// This built data will be used by the cooking (it will convert it to level texture streaming built data).
	// Use same quality level and feature level as FEditorBuildUtils::EditorBuildTextureStreaming
	BuildActorTextureStreamingData(this, EMaterialQualityLevel::High, GMaxRHIFeatureLevel);

	GetHLODBuildEventDelegate().Broadcast({ FWorldPartitionHLODBuildEvent::EEventType::EndBuild, this });

	return bBuiltHLOD;
}

bool AWorldPartitionHLOD::ShouldBuildHLOD() const
{
	bool bDirty = false;

	IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
	if (IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr)
	{
		FHLODBuildParams BuildParams;
		BuildParams.HLODActor = const_cast<AWorldPartitionHLOD*>(this);
		BuildParams.bForceBuild = false;
		BuildParams.bTestOnly = true;
		bDirty = WPHLODUtilities->BuildHLOD(BuildParams);
	}

	return bDirty;
}

FWorldPartitionHLODBuildEventDelegate& AWorldPartitionHLOD::GetHLODBuildEventDelegate()
{
	return HLODBuildEventDelegate;
}

FWorldPartitionHLODActorEditorRegisteredDelegate& AWorldPartitionHLOD::GetHLODActorEditorRegisteredDelegate()
{
	return HLODActorEditorRegisteredDelegate;
}

FWorldPartitionHLODActorEditorUnregisteredDelegate& AWorldPartitionHLOD::GetHLODActorEditorUnregisteredDelegate()
{
	return HLODActorEditorUnregisteredDelegate;
}

const TCHAR* HLOD_REPORT_BEGIN = TEXT("### HLOD_REPORT_BEGIN ###");
const TCHAR* HLODLastBuildInfoSectionHeader = TEXT("### Last Build Info ###");
const TCHAR* HLODActorDetailsSectionHeader = TEXT("### HLOD Actor Details ###");
const TCHAR* HLODBuildDetailsSectionHeader = TEXT("### HLOD Build Details ###");
const TCHAR* HLOD_REPORT_END = TEXT("### HLOD_REPORT_END ###");

FString AWorldPartitionHLOD::GenerateHLODBuildReportHeaderString() const
{
	TStringBuilder<1024> BuildReportHeader;

	// add constant metadata
	FString BuildVersion = FApp::GetBuildVersion();
	FString EngineVersion = FEngineVersion::Current().ToString();
	FString ExecutingJobURL = FApp::GetExecutingJobURL();
	FString Platform = ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
	FString BuildConfiguration = LexToString(FApp::GetBuildConfiguration());
	FString CommandLine = FCommandLine::Get();
	CommandLine.TrimStartAndEndInline();
	CommandLine.ReplaceInline(TEXT("\n"), TEXT(""));
	CommandLine.ReplaceInline(TEXT("\r"), TEXT(""));
	FString EngineMode = FGenericPlatformMisc::GetEngineMode();
	FString GraphicsRHI = !FApp::GetGraphicsRHI().IsEmpty() ? FApp::GetGraphicsRHI() : TEXT("NullRHI");
	FString DateTimeUTC = FDateTime::UtcNow().ToString(TEXT("%Y-%m-%d %H:%M:%S"));

	BuildReportHeader << HLODLastBuildInfoSectionHeader << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * BuildVersion:       ") << BuildVersion       << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * EngineVersion:      ") << EngineVersion      << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * ExecutingJobURL:    ") << ExecutingJobURL    << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * Platform:           ") << Platform           << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * BuildConfiguration: ") << BuildConfiguration << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * CommandLine:        ") << CommandLine        << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * EngineMode:         ") << EngineMode         << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * GraphicsRHI:        ") << GraphicsRHI        << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * DateTimeUTC:        ") << DateTimeUTC        << LINE_TERMINATOR;
	BuildReportHeader << LINE_TERMINATOR;
	// Actor Details
	FString ActorDesc = CreateActorDesc()->ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff).TrimEnd();

	BuildReportHeader << HLODActorDetailsSectionHeader << LINE_TERMINATOR;
	BuildReportHeader << TEXT(" * HLOD Actor Descriptor:   ") << LINE_TERMINATOR << TEXT("\t") << ActorDesc << LINE_TERMINATOR; // Will log on multiple lines
	BuildReportHeader << TEXT(" * HLOD Actor IsStandalone: ") << (IsStandalone() ? TEXT("1") : TEXT("0")) << LINE_TERMINATOR;

	return FString(BuildReportHeader);
}

void AWorldPartitionHLOD::UpdateHLODBuildReportHeader()
{
	// The HLOD report is formatted as such:
	//
	// ### HLOD_REPORT_BEGIN ###
	// ### Last Build Info ###
	// * Info1:
	// * Info2:
	// * Info3:
	// * ...
	// 
	// ### HLOD Actor Details ###
	// * Detail1:
	// * Detail2:
	// 
	// ### HLOD Build Details ###
	// >> HLODBuildReportContent <<
	// ### HLOD_REPORT_END ###

	// Extract existing content section
	FString BuildReportContent;
	int32 Pos = HLODBuildReport.Find(HLODBuildDetailsSectionHeader);
	if (Pos != INDEX_NONE)
	{
		HLODBuildReport.RightChopInline(Pos, EAllowShrinking::No);
		BuildReportContent = MoveTemp(HLODBuildReport);
	}
	else
	{
		// No report content was ever written, so create an empty one
		BuildReportContent = LINE_TERMINATOR;
		BuildReportContent += HLODBuildDetailsSectionHeader;
		BuildReportContent += LINE_TERMINATOR;
		BuildReportContent += HLOD_REPORT_END;
		BuildReportContent += LINE_TERMINATOR;
	}

	HLODBuildReport.Reset();

	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += HLOD_REPORT_BEGIN;
	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += GenerateHLODBuildReportHeaderString();
	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += BuildReportContent;
}

void AWorldPartitionHLOD::UpdateHLODBuildReportContent()
{
	HLODBuildReport.Reset();

	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += HLOD_REPORT_BEGIN;
	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += GenerateHLODBuildReportHeaderString();
	HLODBuildReport += LINE_TERMINATOR;
	HLODBuildReport += HLODBuildDetailsSectionHeader;
	HLODBuildReport += LINE_TERMINATOR;

	for (const TObjectPtr<UHLODRebuildPolicyData>& HLODRebuildPolicyData : HLODRebuildPolicyDataSet)
	{
		FString HLODPolicyDataString = HLODRebuildPolicyData->ToString();
		if (!HLODPolicyDataString.IsEmpty())
		{
			HLODBuildReport += HLODPolicyDataString.TrimEnd();
			HLODBuildReport += LINE_TERMINATOR;
		}
	}

	HLODBuildReport += HLOD_REPORT_END;
	HLODBuildReport += LINE_TERMINATOR;
}

TArray<UObject*> AWorldPartitionHLOD::ExportHLODAssets(const FExportHLODAssetsParams& ExportHLODAssetsParams, FString& OutErrorMessage) const
{
	TArray<UObject*> ExportedAssets;
	OutErrorMessage = TEXT("");
		
	auto ShouldExportAsset = [ResourcesPackage = GetHLODResourcesPackage()](const UObject* InAsset)
	{
		// Skip null entries and assets that are not in the HLOD package (no need for export)
		return InAsset && InAsset->IsInPackage(ResourcesPackage);
	};

	// Unbuilt HLODs will have no static mesh components, we can ignore them
	UStaticMeshComponent* MeshComp = FindComponentByClass<UStaticMeshComponent>();
	if (!MeshComp)
	{
		return ExportedAssets;
	}

	UStaticMesh* StaticMesh = MeshComp->GetStaticMesh();
	if (!ShouldExportAsset(StaticMesh))
	{
		return ExportedAssets;
	}

	FString ExportRootPath = ExportHLODAssetsParams.ExportRootPath.Path;
	FPaths::NormalizeDirectoryName(ExportRootPath);

	if (ExportRootPath.IsEmpty() || !FPackageName::IsValidLongPackageName(ExportRootPath))
	{
		OutErrorMessage = FString::Printf(TEXT("Invalid export path: %s"), *ExportRootPath);
		return ExportedAssets;
	}	

	UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();

	FString MeshPackagePath = ExportRootPath / GetActorLabel();
	FString MeshAssetName = FPackageName::GetLongPackageAssetName(MeshPackagePath);
	FString AssetsRootPath = FPackageName::GetLongPackagePath(MeshPackagePath);

	if (EditorAssetSubsystem->DoesAssetExist(MeshPackagePath))
	{
		OutErrorMessage = FString::Printf(TEXT("Mesh asset already exists at %s"), *MeshPackagePath);
		return ExportedAssets;
	}


	for (int32 MatIndex = 0; MatIndex < StaticMesh->GetStaticMaterials().Num(); ++MatIndex)
	{
		UMaterialInterface* Mat = MeshComp->GetMaterial(MatIndex);

		// Skip null entries and materials that are not in the HLOD package (no need for export)
		if (!ShouldExportAsset(Mat))
		{
			continue;
		}

		FString MatPath = AssetsRootPath / Mat->GetName();
		if (EditorAssetSubsystem->DoesAssetExist(MatPath))
		{
			OutErrorMessage = FString::Printf(TEXT("Material asset already exists at %s"), *MatPath);
			return ExportedAssets;
		}

		TArray<UTexture*> UsedTextures;
		Mat->GetUsedTextures(UsedTextures);
		for (UTexture* Tex : UsedTextures)
		{
			// Skip null entries and textures that are not in the HLOD package (no need for export)
			if (!ShouldExportAsset(Tex))
			{
				continue;
			}

			FString TexPath = AssetsRootPath / Tex->GetName();
			if (EditorAssetSubsystem->DoesAssetExist(TexPath))
			{
				OutErrorMessage = FString::Printf(TEXT("Texture asset already exists at %s"), *TexPath);
				return ExportedAssets;
			}
		}
	}

	if (ExportHLODAssetsParams.bTestExportOnly)
	{
		OutErrorMessage = TEXT("");
		return ExportedAssets;
	}

	// Duplicate mesh
	UPackage* MeshPackage = CreatePackage(*MeshPackagePath);
	UStaticMesh* NewMesh = DuplicateObject<UStaticMesh>(StaticMesh, MeshPackage, *MeshAssetName);
	if (!NewMesh)
	{
		OutErrorMessage = TEXT("Failed to duplicate static mesh.");
		return ExportedAssets;
	}

	// Offset calculation	
	if (ExportHLODAssetsParams.MeshOrigin == EExportHLODMeshOrigin::Actor)
	{
		FVector Offset = -GetActorLocation();

		// Offset vertex positions
		if (NewMesh->IsMeshDescriptionValid(0))
		{
			FMeshDescription* MeshDesc = NewMesh->GetMeshDescription(0);
			if (MeshDesc)
			{
				TVertexAttributesRef<FVector3f> VertexPositions = MeshDesc->GetVertexPositions();
				for (const FVertexID VertexID : MeshDesc->Vertices().GetElementIDs())
				{
					VertexPositions[VertexID] += (FVector3f)Offset;
				}
				NewMesh->CommitMeshDescription(0);
			}
		}

		// Offset sockets
		for (UStaticMeshSocket* Socket : NewMesh->Sockets)
		{
			if (Socket)
			{
				Socket->RelativeLocation += Offset;
			}
		}

		// Offset collision data
		if (NewMesh->GetBodySetup())
		{
			UBodySetup* Body = NewMesh->GetBodySetup();

			for (FKConvexElem& Convex : Body->AggGeom.ConvexElems)
			{
				for (FVector& V : Convex.VertexData)
				{
					V += Offset;
				}
				Convex.UpdateElemBox();
			}
			for (FKBoxElem& Elem : Body->AggGeom.BoxElems)
			{
				Elem.Center += Offset;
			}

			for (FKSphereElem& Elem : Body->AggGeom.SphereElems)
			{
				Elem.Center += Offset;
			}

			for (FKSphylElem& Elem : Body->AggGeom.SphylElems)
			{
				Elem.Center += Offset;
			}

			Body->InvalidatePhysicsData();
			Body->CreatePhysicsMeshes();
		}
	}

	// Helper to finalize asset export
	auto OnAssetExported = [&ExportedAssets](UObject* InAsset)
	{
		InAsset->SetFlags(RF_Public | RF_Standalone);
		FAssetRegistryModule::AssetCreated(InAsset);
		InAsset->MarkPackageDirty();
		InAsset->PostEditChange();
		ExportedAssets.Add(InAsset);
	};

	// Duplicate materials and textures
	TMap<UMaterialInterface*, UMaterialInterface*> DuplicatedMaterials;
	TMap<UTexture*, UTexture*> DuplicatedTextures;
	for (int32 MatIndex = 0; MatIndex < StaticMesh->GetStaticMaterials().Num(); ++MatIndex)
	{
		UMaterialInterface* Mat = MeshComp->GetMaterial(MatIndex);

		// Skip null entries and materials that are not in the HLOD package (no need for export)
		if (!ShouldExportAsset(Mat))
		{
			continue;
		}

		// Skip materials that were already exported
		if (DuplicatedMaterials.Contains(Mat))
		{
			continue;
		}

		FString MatPath = AssetsRootPath / Mat->GetName();
		UPackage* MatPackage = CreatePackage(*MatPath);
		UMaterialInterface* NewMat = DuplicateObject<UMaterialInterface>(Mat, MatPackage, *Mat->GetName());

		if (NewMat)
		{
			DuplicatedMaterials.Emplace(Mat, NewMat);

			TArray<UTexture*> UsedTextures;
			NewMat->GetUsedTextures(UsedTextures);

			for (UTexture* Tex : UsedTextures)
			{
				// Skip null entries and textures that are not in the HLOD package (no need for export)
				if (!ShouldExportAsset(Tex))
				{
					continue;
				}

				// Skip textures that were already exported
				if (DuplicatedTextures.Contains(Tex))
				{
					continue;
				}

				FString TexPath = AssetsRootPath / Tex->GetName();
				UPackage* TexPackage = CreatePackage(*TexPath);
				UTexture* NewTex = DuplicateObject<UTexture>(Tex, TexPackage, *Tex->GetName());
				if (NewTex)
				{
					DuplicatedTextures.Emplace(Tex, NewTex);
					OnAssetExported(NewTex);
				}
			}

			// Replace all the original textures by their duplicate in the new material
			FArchiveReplaceObjectRef<UTexture> MeshReplacer(NewMat, DuplicatedTextures);

			OnAssetExported(NewMat);
		}
	}

	// Replace all the original materials by their duplicate in the new mesh
	FArchiveReplaceObjectRef<UMaterialInterface> MeshReplacer(NewMesh, DuplicatedMaterials);

	OnAssetExported(NewMesh);

	OutErrorMessage = TEXT("");

	return ExportedAssets;
}

const FHLODRebuildPolicyDataSet& AWorldPartitionHLOD::GetHLODRebuildPolicyDataSet() const
{
	return HLODRebuildPolicyDataSet;
}

void AWorldPartitionHLOD::SetHLODRebuildPolicyDataSet(const FHLODRebuildPolicyDataSet& InHLODRebuildPolicyDataSet)
{
	HLODRebuildPolicyDataSet = InHLODRebuildPolicyDataSet;

	for (TObjectPtr<UHLODRebuildPolicyData>& HLODRebuildPolicyData : HLODRebuildPolicyDataSet)
	{
		HLODRebuildPolicyData->Rename(nullptr, this, REN_None);
	}

	UpdateHLODBuildReportContent();
}

#endif // WITH_EDITOR