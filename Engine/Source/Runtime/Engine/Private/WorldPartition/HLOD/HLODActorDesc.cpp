// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "UObject/Package.h"
#include "WorldPartition/HLOD/HLODStats.h"

#if WITH_EDITOR

#include "Engine/World.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODSourceActorsFromCell.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

FHLODActorDesc::FHLODActorDesc()
{}

int64 FHLODActorDesc::GetStat(FName InStatName) const
{
	if (InStatName == FWorldPartitionHLODStats::MemoryDiskSizeBytes)
	{
		FString PackageFileName = GetActorPackage().ToString();
		FPackageName::TryConvertLongPackageNameToFilename(GetActorPackage().ToString(), PackageFileName, FPackageName::GetAssetPackageExtension());
		return IFileManager::Get().FileSize(*PackageFileName);
	}
	return HLODStats.FindRef(InStatName);
}

void FHLODActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const UWorldPartition* WorldPartition = InActor->GetWorld() ? InActor->GetWorld()->GetWorldPartition() : nullptr;
	const AWorldPartitionHLOD* HLODActor = CastChecked<AWorldPartitionHLOD>(InActor);
	const UWorldPartitionHLODSourceActors* HLODSourceActors = HLODActor->GetSourceActors();
	const UWorldPartitionHLODSourceActorsFromCell* HLODCellSourceActors = Cast<UWorldPartitionHLODSourceActorsFromCell>(HLODSourceActors);

	if (HLODCellSourceActors && WorldPartition)
	{
		for (const FWorldPartitionRuntimeCellObjectMapping& SubActor : HLODCellSourceActors->GetActors())
		{
			if (SubActor.ContainerID.IsMainContainer())
			{
				const FWorldPartitionActorDescInstance* SubActorDescInstance = WorldPartition->GetActorDescInstance(SubActor.ActorInstanceGuid);
				if (SubActorDescInstance && SubActorDescInstance->GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>())
				{
					ChildHLODActors.Add(SubActor.ActorInstanceGuid);
				}
			}
			else if (WorldPartition->IsStandaloneHLODWorld())
			{
				UClass* NativeClassPtr = FWorldPartitionActorDescUtils::GetActorNativeClassFromString(SubActor.NativeClass.ToString());
				if (NativeClassPtr && NativeClassPtr->IsChildOf<AWorldPartitionHLOD>())
				{
					ExternalChildHLODActors.Add(SubActor.ActorInstanceGuid);
				}
			}
		}
	}

	if (HLODSourceActors)
	{
		SourceHLODLayer = FTopLevelAssetPath(HLODSourceActors->GetHLODLayer());
	}
	
	HLODStats = HLODActor->GetStats();

	FVector Origin, Extent;
	HLODActor->GetActorBounds(false, Origin, Extent);
	SetEditorBounds(FBox::BuildAABB(Origin, Extent));
}

struct FHLODSubActorDescDeprecated
{
	friend FArchive& operator<<(FArchive& Ar, FHLODSubActorDescDeprecated& SubActor);
	FGuid ActorGuid;
	FActorContainerID ContainerID;
};

FArchive& operator<<(FArchive& Ar, FHLODSubActorDescDeprecated& SubActor)
{
	Ar << SubActor.ActorGuid;

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionFActorContainerIDu64ToGuid)
	{
		uint64 ContainerID;
		Ar << ContainerID;
	}
	else
	{
		Ar << SubActor.ContainerID;
	}

	return Ar;
}

void FHLODActorDesc::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	FWorldPartitionActorDesc::Serialize(Ar);

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::WorldPartitionHLODActorDescSerializeHLODSubActors)
		{
			TArray<FGuid> SubActors;
			Ar << SubActors;
		}
		else if(Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODSourceActorsRefactor)
		{
			TArray<FHLODSubActorDescDeprecated> HLODSubActors;
			Ar << HLODSubActors;
		}
		else
		{
			Ar << ChildHLODActors;
		}
	
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionHLODActorDescSerializeHLODLayer)
		{
			FString HLODLayer_Deprecated;
			Ar << HLODLayer_Deprecated;
		}

		const bool bSerializeCellHash = Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::WorldPartitionHLODActorDescSerializeCellHash &&
										Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODActorDescSerializeStats;
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::WorldPartitionHLODActorDescSerializeCellHash)
		{
			uint64 CellHash_Deprecated;
			Ar << CellHash_Deprecated;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeActorIsRuntimeOnly)
		{
			bActorIsRuntimeOnly = true;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionHLODActorDescSerializeStats)
		{
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODActorUseSourceCellGuid)
			{
				FName SourceCellName;
				Ar << SourceCellName;
			}

			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionHLODActorDescSerializeSourceHLODLayer)
			{
				FString SourceHLODLayerName;
				Ar << SourceHLODLayerName;
			}

			Ar << HLODStats;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionHLODActorDescSerializeSourceHLODLayer)
		{
			Ar << SourceHLODLayer;
		}
		
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionHLODActorDescSerializeEditorBounds)
		{
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeEditorBounds)
			{
				Ar << EditorBounds;
			}
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionAddStandaloneHLODSupport)
		{
			Ar << ExternalChildHLODActors;
		}
	}
}

bool FHLODActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)Other;
		return SourceHLODLayer == HLODActorDesc.SourceHLODLayer &&
			   HLODStats.OrderIndependentCompareEqual(HLODActorDesc.HLODStats) &&
			   CompareUnsortedArrays(ChildHLODActors, HLODActorDesc.ChildHLODActors) &&
			   CompareUnsortedArrays(ExternalChildHLODActors, HLODActorDesc.ExternalChildHLODActors) &&
			   EditorBounds == HLODActorDesc.EditorBounds;
	}
	return false;
}

bool FHLODActorDesc::EqualsUnbuilt(const FHLODActorDesc& Other) const
{
	return Guid == Other.Guid &&
		BaseClass == Other.BaseClass &&
		NativeClass == Other.NativeClass &&
		ActorLabel == Other.ActorLabel &&
		RuntimeGrid == Other.RuntimeGrid &&
		bIsSpatiallyLoaded == Other.bIsSpatiallyLoaded &&
		bActorIsRuntimeOnly == Other.bActorIsRuntimeOnly &&
		bActorIsMainWorldOnly == Other.bActorIsMainWorldOnly &&
		HLODLayer == Other.HLODLayer &&
		ContentBundleGuid == Other.ContentBundleGuid &&
		ComponentDescs == Other.ComponentDescs &&
		CompareUnsortedArrays(DataLayers, Other.DataLayers) &&
		CompareUnsortedArrays(Tags, Other.Tags) &&
		Properties == Other.Properties &&
		ExternalDataLayerAsset == Other.ExternalDataLayerAsset &&
		SourceHLODLayer == Other.SourceHLODLayer;
}

bool FHLODActorDesc::IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	return !InActorDescInstance->GetForceNonSpatiallyLoaded();
}

#endif
