// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODActorExternalResources.h"

#include "Engine/StaticMesh.h"
#include "WorldPartition/HLOD/HLODUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODActorExternalResources)

#if WITH_EDITOR
void UHLODActorExternalResources::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking())
	{
		for (UObject* Resource : Resources)
		{
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Resource))
			{
				UE::Private::HLOD::RemoveCollisionData(StaticMesh);
			}
		}
	}
}
#endif // WITH_EDITOR