// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODBlueprintLibrary.h"

#if WITH_EDITOR
#include "Editor.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/WorldPartitionHLODsBuilder.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBlueprintLibrary)

bool UWorldPartitionHLODBlueprintLibrary::BuildHLODForActors(const TArray<AActor*>& InActors)
{
#if WITH_EDITOR
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		TArray<FBox> FilterVolumes;
		for (const AActor* Actor : InActors)
		{
			if (IsValid(Actor))
			{
				FBox RuntimeBounds;
				FBox EditorBounds;
				Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);
				FilterVolumes.Add(RuntimeBounds);
			}
		}

		if (FilterVolumes.Num() > 0)
		{
			return UWorldPartitionHLODsBuilder::BuildWithFilters(World, { MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes)) });
		}
	}
#endif
	return false;
}

bool UWorldPartitionHLODBlueprintLibrary::BuildHLODForVolume(const FBox& InBox)
{
#if WITH_EDITOR
	if (UWorld* World = GEditor->GetEditorWorldContext().World())
	{
		TArray<FBox> FilterVolumes = { InBox };
		return UWorldPartitionHLODsBuilder::BuildWithFilters(World, { MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes)) });
	}
#endif
	return false;
}