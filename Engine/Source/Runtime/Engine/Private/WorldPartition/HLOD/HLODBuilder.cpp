// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODBuilder.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/HLODProxy.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "HLOD/HLODBatchingPolicy.h"
#include "ISMPartition/ISMComponentBatcher.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Materials/MaterialInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Package.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "WorldPartition/HLOD/HLODInstancedStaticMeshComponent.h"
#include "WorldPartition/HLOD/HLODInstancedSkinnedMeshComponent.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODBuilder)


DEFINE_LOG_CATEGORY(LogHLODBuilder);


UHLODBuilder::UHLODBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, HLODInstancedStaticMeshComponentClass(UHLODInstancedStaticMeshComponent::StaticClass())
	, HLODInstancedSkinnedMeshComponentClass(UHLODInstancedSkinnedMeshComponent::StaticClass())
#endif
{
}

UNullHLODBuilder::UNullHLODBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODBuilderSettings::UHLODBuilderSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

TSubclassOf<UHLODBuilderSettings> UHLODBuilder::GetSettingsClass() const
{
	return UHLODBuilderSettings::StaticClass();
}

void UHLODBuilder::SetHLODBuilderSettings(const UHLODBuilderSettings* InHLODBuilderSettings)
{
	check(InHLODBuilderSettings->IsA(GetSettingsClass()));
	HLODBuilderSettings = InHLODBuilderSettings;
}

bool UHLODBuilder::RequiresWarmup() const
{
	return true;
}

bool UHLODBuilder::ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const
{
	InSourceComponent->ComputeHLODHash(HashBuilder);
	return true;
}

void UHLODBuilder::ComputeHLODHash(FHLODHashBuilder& HashBuilder, const TArray<UActorComponent*>& InSourceComponents)
{
	// We get the hash of each component
	TArray<uint32> ComponentsHashes;

	for (UActorComponent* SourceComponent : InSourceComponents)
	{
		FHLODHashScope HashScope(HashBuilder, SourceComponent, FHLODHashScope::EFlags::ResetHash);

		TSubclassOf<UHLODBuilder> HLODBuilderClass = SourceComponent->GetCustomHLODBuilderClass();
		if (!HLODBuilderClass)
		{
			HLODBuilderClass = UHLODBuilder::StaticClass();
		}

		bool bValidHash = HLODBuilderClass->GetDefaultObject<UHLODBuilder>()->ComputeHLODHash(HashBuilder, SourceComponent);
		if (bValidHash)
		{
			ComponentsHashes.Add(HashBuilder.GetCrc());
		}
		else
		{
			UE_LOGF(LogHLODBuilder, Warning, "Can't compute HLOD hash for component of type %ls, assuming it is dirty.", *SourceComponent->GetClass()->GetName());
			ComponentsHashes.Add(FMath::Rand());
		}
	}

	// Sort the components hashes to ensure the order of components won't have an impact on the final hash
	ComponentsHashes.Sort();

	// Incorporate all hashes
	HashBuilder << ComponentsHashes;
}

TSubclassOf<UHLODInstancedStaticMeshComponent> UHLODBuilder::GetInstancedStaticMeshComponentClass()
{
	TSubclassOf<UHLODInstancedStaticMeshComponent> ISMClass = StaticClass()->GetDefaultObject<UHLODBuilder>()->HLODInstancedStaticMeshComponentClass;
	if (!ISMClass)
	{
		FString ConfigValue;
		GConfig->GetString(TEXT("/Script/Engine.HLODBuilder"), TEXT("HLODInstancedStaticMeshComponentClass"), ConfigValue, GEditorIni);
		UE_LOGF(LogHLODBuilder, Error, "Could not resolve the class specified for HLODInstancedStaticMeshComponentClass. Config value was %ls", *ConfigValue);

		// Fallback to standard HLOD ISMC
		ISMClass = UHLODInstancedStaticMeshComponent::StaticClass();
	}
	return ISMClass;
}

TSubclassOf<UHLODInstancedSkinnedMeshComponent> UHLODBuilder::GetInstancedSkinnedMeshComponentClass()
{
	TSubclassOf<UHLODInstancedSkinnedMeshComponent> ISMClass = StaticClass()->GetDefaultObject<UHLODBuilder>()->HLODInstancedSkinnedMeshComponentClass;
	if (!ISMClass)
	{
		FString ConfigValue;
		GConfig->GetString(TEXT("/Script/Engine.HLODBuilder"), TEXT("HLODInstancedSkinnedMeshComponentClass"), ConfigValue, GEditorIni);
		UE_LOGF(LogHLODBuilder, Error, "Could not resolve the class specified for HLODInstancedSkinnedMeshComponentClass. Config value was %ls", *ConfigValue);

		// Fallback to standard HLOD ISMC
		ISMClass = UHLODInstancedSkinnedMeshComponent::StaticClass();
	}
	return ISMClass;
}

namespace
{
	struct FInstancedStaticMeshBatch
	{
		TUniquePtr<FISMComponentDescriptor>		ISMComponentDescriptor;
		FISMComponentBatcher					ISMComponentBatcher;

		FInstancedStaticMeshBatch()
			: ISMComponentBatcher(/*bBuildMappingInfo=*/true)
		{
		}
	};

	struct FInstancedSkinnedMeshBatch
	{
		TUniquePtr<FSkinnedMeshComponentDescriptor>	ISMComponentDescriptor;
		FISMComponentBatcher						ISMComponentBatcher;
	};
}

TArray<UActorComponent*> UHLODBuilder::BatchInstances(const TArray<UActorComponent*>& InSourceComponents)
{
	return BatchInstances(InSourceComponents, [](const FBox& InBox) { return true; });
}

TArray<UActorComponent*> UHLODBuilder::BatchInstances(const TArray<UActorComponent*>& InSourceComponents, TFunctionRef<bool(const FBox&)> InFilterFunc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHLODBuilderInstancing::Build);

	TArray<UActorComponent*> HLODComponents;

	TArray<UStaticMeshComponent*> SourceStaticMeshComponents;
	TArray<UInstancedSkinnedMeshComponent*> SourceInstancedSkinnedMeshComponents;
	TArray<UActorComponent*> FilteredOutComponents;

	for (UActorComponent* SourceComponent : InSourceComponents)
	{
		if (UStaticMeshComponent* SourceStaticMeshComponent = Cast<UStaticMeshComponent>(SourceComponent))
		{
			SourceStaticMeshComponents.Add(SourceStaticMeshComponent);
		}
		else if (UInstancedSkinnedMeshComponent* SourceInstancedSkinnedMeshComponent = Cast<UInstancedSkinnedMeshComponent>(SourceComponent))
		{
			SourceInstancedSkinnedMeshComponents.Add(SourceInstancedSkinnedMeshComponent);
		}
		else if (SourceComponent)
		{
			FilteredOutComponents.Add(SourceComponent);
		}
	}

	// Log filtered out components, this is unexpected
	if (!FilteredOutComponents.IsEmpty())
	{
		UE_LOGF(LogHLODBuilder, Warning, "UHLODBuilder::BatchInstances - Excluding %d unsupported components:", FilteredOutComponents.Num());
		for (UActorComponent* FilteredOutComponent : FilteredOutComponents)
		{
			UE_LOGF(LogHLODBuilder, Warning, "    -> (%ls) %ls", *FilteredOutComponent->GetClass()->GetName(), *FilteredOutComponent->GetName());
		}
	}

	auto IsAssetValidForInstancing = [](UActorComponent* InComponent, UObject* InObj)
	{
		const bool bPrivateAsset = InObj && !InObj->HasAnyFlags(RF_Public);
		const bool bTransientAsset = InObj && InObj->HasAnyFlags(RF_Transient);
		if (!InObj || bPrivateAsset || bTransientAsset)
		{
			UE_LOGF(LogHLODBuilder, Warning, "Instanced HLOD source component %ls points to a %ls mesh, ignoring.", *InComponent->GetPathName(), !InObj ? TEXT("null") : bPrivateAsset ? TEXT("private") : TEXT("transient"));
			return false;
		}

		return true;
	};

	// Static meshes batching
	{		
		TSubclassOf<UHLODInstancedStaticMeshComponent> ComponentClass = UHLODBuilder::GetInstancedStaticMeshComponentClass();

		// Prepare instance batches
		TMap<uint32, FInstancedStaticMeshBatch> InstancesData;
		for (UStaticMeshComponent* SMC : SourceStaticMeshComponents)
		{
			if (!IsAssetValidForInstancing(SMC, SMC->GetStaticMesh()))
			{
				continue;
			}

			TUniquePtr<FISMComponentDescriptor> ISMComponentDescriptor = ComponentClass->GetDefaultObject<UHLODInstancedStaticMeshComponent>()->AllocateISMComponentDescriptor();
			ISMComponentDescriptor->InitFrom(SMC, false);

			FInstancedStaticMeshBatch& InstanceBatch = InstancesData.FindOrAdd(ISMComponentDescriptor->GetTypeHash());
			if (!InstanceBatch.ISMComponentDescriptor.IsValid())
			{
				InstanceBatch.ISMComponentDescriptor = MoveTemp(ISMComponentDescriptor);
			}

			InstanceBatch.ISMComponentBatcher.Add(SMC, InFilterFunc);
		}

		// Create an ISMC for each SM asset we found		
		for (auto& Entry : InstancesData)
		{
			FInstancedStaticMeshBatch& InstanceBatch = Entry.Value;
			if (InstanceBatch.ISMComponentBatcher.GetNumInstances() > 0)
			{
				UHLODInstancedStaticMeshComponent* ISMComponent = CastChecked<UHLODInstancedStaticMeshComponent>(InstanceBatch.ISMComponentDescriptor->CreateComponent(GetTransientPackage()));
				InstanceBatch.ISMComponentBatcher.InitComponent(ISMComponent);

				ISMComponent->SetSourceComponentsToInstancesMap(InstanceBatch.ISMComponentBatcher.GetComponentsToInstancesMap());

				HLODComponents.Add(ISMComponent);
			}
		};
	}

	// Skinned meshes batching
	{
		TSubclassOf<UHLODInstancedSkinnedMeshComponent> ComponentClass = UHLODBuilder::GetInstancedSkinnedMeshComponentClass();

		// Prepare instance batches
		TMap<uint32, FInstancedSkinnedMeshBatch> InstancesData;
		for (UInstancedSkinnedMeshComponent* ISKMC : SourceInstancedSkinnedMeshComponents)
		{
			if (!IsAssetValidForInstancing(ISKMC, ISKMC->GetSkinnedAsset()))
			{
				continue;
			}

			TUniquePtr<FSkinnedMeshComponentDescriptor> ISMComponentDescriptor = ComponentClass->GetDefaultObject<UHLODInstancedSkinnedMeshComponent>()->AllocateISMComponentDescriptor();
			ISMComponentDescriptor->InitFrom(ISKMC, false);

			FInstancedSkinnedMeshBatch& InstanceBatch = InstancesData.FindOrAdd(ISMComponentDescriptor->GetTypeHash());
			if (!InstanceBatch.ISMComponentDescriptor.IsValid())
			{
				InstanceBatch.ISMComponentDescriptor = MoveTemp(ISMComponentDescriptor);
			}

			InstanceBatch.ISMComponentBatcher.Add(ISKMC, InFilterFunc);
		}

		// Create an ISKMC for each SKM asset we found
		for (auto& Entry : InstancesData)
		{
			const FInstancedSkinnedMeshBatch& InstanceBatch = Entry.Value;
			if (InstanceBatch.ISMComponentBatcher.GetNumInstances() > 0)
			{
				UInstancedSkinnedMeshComponent* ISMComponent = InstanceBatch.ISMComponentDescriptor->CreateComponent(GetTransientPackage());
				InstanceBatch.ISMComponentBatcher.InitComponent(ISMComponent);
				HLODComponents.Add(ISMComponent);
			}
		};
	}

	return HLODComponents;
}

static bool ShouldBatchComponent(UActorComponent* ActorComponent)
{
	bool bShouldBatch = false;

	if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ActorComponent))
	{
		switch (PrimitiveComponent->HLODBatchingPolicy)
		{
		case EHLODBatchingPolicy::None:
			break;
		case EHLODBatchingPolicy::Instancing:
			bShouldBatch = true;
			break;
		case EHLODBatchingPolicy::MeshSection:
		{
			bShouldBatch = true;
			FString LogDetails = FString::Printf(TEXT("%s %s (from actor %s)"), *PrimitiveComponent->GetClass()->GetName(), *ActorComponent->GetName(), *ActorComponent->GetOwner()->GetActorLabel());
			if (UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
			{
				LogDetails += FString::Printf(TEXT(" using static mesh %s"), SMComponent->GetStaticMesh() ? *SMComponent->GetStaticMesh()->GetName() : TEXT("<null>"));
			}
			UE_LOGF(LogHLODBuilder, Display, "EHLODBatchingPolicy::MeshSection is not yet supported by the HLOD builder, falling back to EHLODBatchingPolicy::Instancing for %ls.", *LogDetails);
			break;
		}
		default:
			checkNoEntry();
		}
	}

	return bShouldBatch;
}

FHLODBuildResult UHLODBuilder::Build(const FHLODBuildContext& InHLODBuildContext) const
{
	// Handle components using a batching policy separately
	TArray<UActorComponent*> InputComponents;
	TArray<UActorComponent*> ComponentsToBatch;

	if (!ShouldIgnoreBatchingPolicy())
	{				
		for (UActorComponent* SourceComponent : InHLODBuildContext.SourceComponents)
		{
			if (ShouldBatchComponent(SourceComponent))
			{
				ComponentsToBatch.Add(SourceComponent);
			}
			else
			{
				InputComponents.Add(SourceComponent);
			}
		}
	}
	else
	{
		InputComponents = InHLODBuildContext.SourceComponents;
	}

	TMap<TSubclassOf<UHLODBuilder>, TArray<UActorComponent*>> HLODBuildersForComponents;

	// Gather custom HLOD builders, and regroup all components by builders
	for (UActorComponent* SourceComponent : InputComponents)
	{
		TSubclassOf<UHLODBuilder> HLODBuilderClass = SourceComponent->GetCustomHLODBuilderClass();
		HLODBuildersForComponents.FindOrAdd(HLODBuilderClass).Add(SourceComponent);
	}
	
	FHLODBuildResult BuildResult;

	auto AddReferencedAssetsToStats = [&BuildResult](FName HLODBuilderClassName, TArray<UActorComponent*> InSourceComponents)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		const FTopLevelAssetPath StaticMeshAssetClassPath(UStaticMesh::StaticClass());

		FHLODBuildInputReferencedAssets& ReferencedAssetsStats = BuildResult.InputStats.BuildersReferencedAssets.FindOrAdd(HLODBuilderClassName);

		for (UActorComponent* SourceComponent : InSourceComponents)
		{
			// At the moment we only care about static meshes for our stats
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SourceComponent))
			{
				FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(StaticMeshComponent->GetStaticMesh()));
				if (AssetData.IsUAsset())
				{					
					if (AssetData.AssetClassPath == StaticMeshAssetClassPath)
					{
						FTopLevelAssetPath StaticMeshAssetPath(AssetData.PackageName, AssetData.AssetName);
						ReferencedAssetsStats.StaticMeshes.FindOrAdd(StaticMeshAssetPath)++;
					}
				}
			}
		}	
	};
	
	// Build HLOD components by sending source components to the individual builders, in batch
	TArray<UActorComponent*> HLODComponents;
	for (const auto& HLODBuilderPair : HLODBuildersForComponents)
	{
		// If no custom HLOD builder is provided, use this current builder.
		const UHLODBuilder* HLODBuilder = HLODBuilderPair.Key ? HLODBuilderPair.Key->GetDefaultObject<UHLODBuilder>() : this;
		const TArray<UActorComponent*>& SourceComponents = HLODBuilderPair.Value;

		AddReferencedAssetsToStats(HLODBuilder->GetClass()->GetFName(), SourceComponents);

		TArray<UActorComponent*> NewComponents = HLODBuilder->Build(InHLODBuildContext, SourceComponents);
		BuildResult.HLODComponents.Append(NewComponents);
	}

	// Append batched components
	if (!ComponentsToBatch.IsEmpty())
	{
		FName HLODBuilderInstancingClassName("HLODBuilderInstancing");
		AddReferencedAssetsToStats(HLODBuilderInstancingClassName, ComponentsToBatch);
		BuildResult.HLODComponents.Append(BatchInstances(ComponentsToBatch));
	}

	// In case a builder returned null entries, clean the array.
	BuildResult.HLODComponents.RemoveSwap(nullptr);

	return BuildResult;
}

#endif // WITH_EDITOR

