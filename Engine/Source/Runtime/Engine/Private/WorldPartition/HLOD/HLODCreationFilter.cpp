// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODCreationFilter.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"

namespace UE::HLOD::CreationFilter
{
	bool HasAnyActiveFilters(const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters)
	{
		return Algo::AnyOf(InFilters, [](const TSharedPtr<IHLODCreationFilter>& Filter)
		{
			return (Filter.IsValid() && Filter->IsActive());
		});
	}

	bool PassesFilters(const TArray<TSharedPtr<IHLODCreationFilter>>& InFilters, const FHLODCreationFilterContext& InFilterContext)
	{
		return Algo::AllOf(InFilters, [&InFilterContext](const TSharedPtr<IHLODCreationFilter>& Filter)
		{
			return (Filter.IsValid() == false || Filter->PassesFilter(InFilterContext));
		});
	}
}

FHLODCreationVolumeFilter::FHLODCreationVolumeFilter(TArray<FBox> && InFilterVolumes)
	: FilterVolumes(MoveTemp(InFilterVolumes))
{
}

bool FHLODCreationVolumeFilter::IsActive() const
{
	return !FilterVolumes.IsEmpty();
}

bool FHLODCreationVolumeFilter::PassesFilter(const FHLODCreationFilterContext& InFilterContext) const
{
	if (InFilterContext.Bounds.IsSet() == false)
	{
		return false;
	}

	const FBox& Bounds = InFilterContext.Bounds.GetValue();
	return (IsActive() == false || Algo::AnyOf(FilterVolumes, [&Bounds](const FBox& FilterVolume)
	{
		return FilterVolume.Intersect(Bounds);
	}));
}