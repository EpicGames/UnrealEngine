// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODDestruction.h"

#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "LevelUtils.h"
#include "UObject/ScriptInterface.h"
#include "WorldPartition/HLOD/DestructibleHLODComponent.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODDestruction)

namespace
{
	static const UWorldPartitionRuntimeCell* GetActorRuntimeCell(AActor* InActor)
	{
		const ULevel* ActorLevel = InActor->GetLevel();
		const ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(ActorLevel);
		return LevelStreaming ? Cast<UWorldPartitionRuntimeCell>(LevelStreaming->GetWorldPartitionCell()) : nullptr;
	}

	static FWorldPartitionHLODDestructionTag ResolveHLODDestructionTagForActor(AActor* InActor)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ResolveHLODDestructionTagForActor);

		const UWorldPartitionRuntimeCell* RuntimeCell = GetActorRuntimeCell(InActor);

		const FName ActorFName = InActor->GetFName();

		FWorldPartitionHLODDestructionTag HLODDestructionTag;

		if (RuntimeCell)
		{
			for (IWorldPartitionHLODObject* HLODObject : InActor->GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->GetHLODObjectsForCell(RuntimeCell))
			{
				// This is the only use of IWorldPartitionHLODObject::GetUObject() for now... Once we solve destruction for FastGeo HLODs, we will be able to remove it.
				if (AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(HLODObject->GetUObject()))
				{
					for (UActorComponent* ActorComponent : HLODActor->GetComponents())
					{
						if (UWorldPartitionDestructibleHLODComponent* DestructibleHLODComponent = Cast<UWorldPartitionDestructibleHLODComponent>(ActorComponent))
						{
							int32 ActorIdx = DestructibleHLODComponent->GetDestructibleActors().IndexOfByKey(ActorFName);
							if (ActorIdx != INDEX_NONE)
							{
								HLODDestructionTag.ActorIndex = ActorIdx;
								HLODDestructionTag.HLODDestructionComponent = DestructibleHLODComponent;
								break;
							}
						}
					}
				}

				if (HLODDestructionTag.IsValid())
				{
					break;
				}
			}
		}

		return HLODDestructionTag;
	}

	FWorldPartitionHLODDestructionTag GetDestructionTagForActor(AActor* SourceActor)
	{
		FWorldPartitionHLODDestructionTag Tag = IWorldPartitionDestructibleInHLODInterface::Execute_GetHLODDestructionTag(SourceActor);
		if (!Tag.IsValid())
		{
			Tag = ResolveHLODDestructionTagForActor(SourceActor);
			IWorldPartitionDestructibleInHLODInterface::Execute_SetHLODDestructionTag(SourceActor, Tag);
		}
		return Tag;
	}
}

void UWorldPartitionDestructibleInHLODSupportLibrary::DestroyInHLOD(const TScriptInterface<IWorldPartitionDestructibleInHLODInterface>& DestructibleInHLOD)
{
	AActor* SourceActor = CastChecked<AActor>(DestructibleInHLOD.GetObject());

	if (UWorld* World = SourceActor->GetWorld())
	{
		if (World->IsGameWorld() && SourceActor->HasAuthority())
		{
			FWorldPartitionHLODDestructionTag Tag = GetDestructionTagForActor(SourceActor);
			if (Tag.IsValid())
			{
				Tag.HLODDestructionComponent->DestroyActor(Tag.ActorIndex);
			}
		}
	}
}

void UWorldPartitionDestructibleInHLODSupportLibrary::DamageInHLOD(const TScriptInterface<IWorldPartitionDestructibleInHLODInterface>& DestructibleInHLOD, float DamagePercent)
{
	AActor* SourceActor = CastChecked<AActor>(DestructibleInHLOD.GetObject());

	if (UWorld* World = SourceActor->GetWorld())
	{
		if (World->IsGameWorld() && SourceActor->HasAuthority())
		{
			FWorldPartitionHLODDestructionTag Tag = GetDestructionTagForActor(SourceActor);
			if (Tag.IsValid())
			{
				Tag.HLODDestructionComponent->DamageActor(Tag.ActorIndex, DamagePercent);
			}
		}
	}
}
