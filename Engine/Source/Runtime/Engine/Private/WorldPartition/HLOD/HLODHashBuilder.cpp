// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODHashBuilder.h"

#if WITH_EDITOR

#include "Engine/HLODProxy.h"
#include "Engine/Level.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "WorldPartition/HLOD/HLODBuilder.h"

// For most object, use the path name as the key
// For MaterialInstanceDynamic, which are created dynamically, the name will not be stable from one execution to the next.
// In that case, use the parent material name + a hash representing the dynamic variation.
// This is purely for report stability, this doesn't affect the resulting HLOD hash
static FString GetReportObjectKey(const UObject* InObject)
{
	FString ObjectKey;

	if (const UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<const UMaterialInstanceDynamic>(InObject))
	{
		uint32 HashValue = UHLODProxy::GetCRC(const_cast<UMaterialInstanceDynamic*>(MaterialInstanceDynamic));

		TArray<UTexture*> Textures;
		TArray<uint32> TexturesHashes;
		MaterialInstanceDynamic->GetUsedTextures(Textures);
		for (UTexture* Texture : Textures)
		{
			TexturesHashes.Add(UHLODProxy::GetCRC(Texture));
		}
		TexturesHashes.Sort();
		for (uint32 TextureHash : TexturesHashes)
		{
			HashValue = HashCombine(HashValue, TextureHash);
		}

		ObjectKey = FString::Printf(TEXT("%s (MID Key=%08X)"), *MaterialInstanceDynamic->Parent->GetPathName(), HashValue);
	}
	else
	{
		UObject* StopOuter = InObject->IsA<UActorComponent>() ? InObject->GetTypedOuter<ULevel>() : nullptr;
		ObjectKey = InObject->GetPathName(StopOuter);
	}

	return ObjectKey;
}

void FHLODHashBuilder::PushObjectContext(const UObject* InObjectContext)
{
	ObjectContextStack.Push(InObjectContext->GetClass()->GetName() + TEXT(" ") + GetReportObjectKey(InObjectContext));
}

void FHLODHashBuilder::PopObjectContext()
{
	check(!ObjectContextStack.IsEmpty());
	ObjectsHashes.FindOrAdd(ObjectContextStack.Top()).Hash = GetCrc();
	ObjectContextStack.Pop();
}

FArchive& FHLODHashBuilder::operator<<(FTransform InTransform)
{
	*this << TransformUtilities::GetRoundedTransformCRC32(InTransform);
	return *this;
}

uint32 FHLODHashBuilder::AddAssetReference(UObject* Asset, TFunctionRef<uint32()> GetHashFunc)
{
	check(Asset != nullptr);

	FName AssetName(GetReportObjectKey(Asset));

	FAssetHash& AssetHash = AssetsHashes.FindOrAdd(AssetName);
	if (AssetHash.Hash == 0)
	{
		AssetHash.Hash = GetHashFunc();
		AssetHash.AssetType = Asset->GetClass()->GetFName();
	}

	if (!ObjectContextStack.IsEmpty())
	{		
		ObjectsHashes.FindOrAdd(ObjectContextStack.Top()).ReferencedAssets.Add(AssetName);
	}

	return AssetHash.Hash;
}

FArchive& FHLODHashBuilder::operator<<(UObject*& InObject)
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InObject))
	{
		*this << AddAssetReference(MaterialInterface, [MaterialInterface]() { return UHLODProxy::GetCRC(MaterialInterface); });

		FHLODHashScope HashScopeMaterial(*this, MaterialInterface);

		// Add the whole material chain as references
		UMaterialInterface* Parent = Cast<UMaterialInstance>(MaterialInterface) ? Cast<UMaterialInstance>(MaterialInterface)->Parent : nullptr;
		while (Parent)
		{
			AddAssetReference(Parent, [Parent]() { return UHLODProxy::GetCRC(Parent); });
			Parent = Cast<UMaterialInstance>(Parent) ? Cast<UMaterialInstance>(Parent)->Parent : nullptr;
		}

		// Textures
		TArray<UTexture*> Textures;
		MaterialInterface->GetUsedTextures(Textures);
		*this << Textures;
	
		if (UMaterialInterface* NaniteOverride = MaterialInterface->GetNaniteOverride())
		{ 
			*this << AddAssetReference(NaniteOverride, [NaniteOverride]() { return UHLODProxy::GetCRC(NaniteOverride); });

			FHLODHashScope HashScopeNaniteMaterial(*this, NaniteOverride);

			Textures.Reset();
			NaniteOverride->GetUsedTextures(Textures);
			*this << Textures;
		}
	}
	else if (UTexture* Texture = Cast<UTexture>(InObject))
	{
		*this << AddAssetReference(Texture, [Texture]() { return UHLODProxy::GetCRC(Texture); });
	}
	else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(InObject))
	{
		*this << AddAssetReference(StaticMesh, [StaticMesh]() { return UHLODProxy::GetCRC(StaticMesh); });
	}
	else
	{
		FArchive::operator<<(InObject);
	}
	
	return *this;
}

FArchive& FHLODHashBuilder::operator<<(UMaterialInterface* InMaterialInterface)
{
	UObject* Object = InMaterialInterface;
	return *this << Object;
}

FArchive& FHLODHashBuilder::operator<<(UTexture* InTexture)
{
	UObject* Object = InTexture;
	return *this << Object;
}

FArchive& FHLODHashBuilder::operator<<(UStaticMesh* InStaticMesh)
{
	UObject* Object = InStaticMesh;
	return *this << Object;
}

FArchive& FHLODHashBuilder::operator<<(USkinnedAsset* InSkinnedAsset)
{
	UObject* Object = InSkinnedAsset;
	return *this << Object;
}

static FString GetTypePrefixFromClassName(const FName& ClassFName)
{
	const FString C = ClassFName.ToString();
	if (C.Contains(TEXT("MaterialInstanceDynamic")))                     return TEXT("MID");
	if (C.Contains(TEXT("MaterialInstanceConstant")))                    return TEXT("MIC");
	if (C.Contains(TEXT("MaterialInstance")))                    		 return TEXT("MI");
	if (C.Contains(TEXT("Material")))                             		 return TEXT("MAT");
	if (C.Contains(TEXT("Texture")))                              		 return TEXT("TEX");
	if (C.Contains(TEXT("StaticMesh")))                           		 return TEXT("SM");
	if (C.Contains(TEXT("SkeletalMesh")) || C.Contains(TEXT("Skinned"))) return TEXT("SK");
	return TEXT("OBJ");
}

static uint16 StablePathId16(const FName& AssetPath)
{
	FString Norm = AssetPath.ToString();
	Norm.ToLowerInline();
	Norm.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	return static_cast<uint16>(FCrc::StrCrc32(*Norm) & 0xFFFF);
}

// Build final label from type prefix + 16-bit id.
static FString MakeTypeAwareLabel16(const FString& TypePrefix, uint16 Id16)
{
	return FString::Printf(TEXT("@%s-%04X"), *TypePrefix, Id16);
}

FString FHLODHashBuilder::BuildHashReport() const
{
	FString Out;
	static constexpr int32 ReserveSize = 64 * 1024;
	Out.Reserve(ReserveSize);

	// Collect & sort asset paths
	TArray<FName> SortedAssetNames;
	SortedAssetNames.Reserve(AssetsHashes.Num());
	AssetsHashes.GenerateKeyArray(SortedAssetNames);
	SortedAssetNames.Sort([](const FName& A, const FName& B)
	{
		return A.ToString().Compare(B.ToString(), ESearchCase::IgnoreCase) < 0;
	});

	// Generate short asset labels ("@TYP-XXXX")
	TMap<FName, FString> AssetLabels;	// AssetName to label
	TSet<FString>        UsedLabels;	// "@TYP-XXXX" taken?
	AssetLabels.Reserve(SortedAssetNames.Num());
	UsedLabels.Reserve(SortedAssetNames.Num());

	for (const FName& AssetName : SortedAssetNames)
	{
		const FAssetHash* Entry = AssetsHashes.Find(AssetName);
		const FName AssetType = Entry ? Entry->AssetType : NAME_None;

		const FString Prefix = GetTypePrefixFromClassName(AssetType);
		uint16 Id16 = StablePathId16(AssetName);

		// Handle collisions - increment id until a free one is found
		// This is deterministic because input order is the sorted asset path order.
		FString Token;
		for (uint32 Attempt = 0; Attempt < 65536u; ++Attempt)
		{
			Token = MakeTypeAwareLabel16(Prefix, Id16);
			if (!UsedLabels.Contains(Token))
			{
				UsedLabels.Add(Token);
				break;
			}
			Id16 = static_cast<uint16>(Id16 + 1);
		}

		AssetLabels.Add(AssetName, Token);
	}

	Out += TEXT("## Global Fields ##\n");
	for (const TTuple<FName, FString>& Field : GlobalFields)
	{
		Out += FString::Printf(TEXT("\n    * %s: %s"), *Field.Key.ToString(), *Field.Value);
	}

	Out += TEXT("\n\n## Referenced Assets ##\n");

 	// Referenced assets have no fields, we only track their hashes. We don't want to show them in the Objects section.
	// Keep a list of the keys we consume here so we can skip them later
	TSet<FString> SuppressedObjectKeys;
	SuppressedObjectKeys.Reserve(SortedAssetNames.Num());

	// Get a string containing all referenced assets (sorted)
	auto GetReferencesString = [&AssetLabels](const FObjectHash& ObjectHashes)
	{
		TArray<FString> Labels;
		Labels.Reserve(ObjectHashes.ReferencedAssets.Num());
		for (const FName& RefAssetName : ObjectHashes.ReferencedAssets)
		{
			if (const FString* Label = AssetLabels.Find(RefAssetName))
			{
				Labels.Add(*Label);
			}
		}
		Labels.Sort([](const FString& A, const FString& B)
		{
			return A.Compare(B, ESearchCase::CaseSensitive) < 0;
		});

		FString RefsStr;
		for (int32 i = 0; i < Labels.Num(); ++i)
		{
			if (i > 0)
			{
				RefsStr += TEXT(" ");
			}
			RefsStr += Labels[i];
		}
		return RefsStr;
	};

	// Assets section (sorted by asset path)
	for (const FName& AssetName : SortedAssetNames)
	{
		const FAssetHash* AssetEntry = AssetsHashes.Find(AssetName);
		const uint32 Hash = AssetEntry ? AssetEntry->Hash : 0u;
		const FName  Type = AssetEntry ? AssetEntry->AssetType : NAME_None;
		const FString& Label = AssetLabels.FindChecked(AssetName);

		FString RefsStr;
		const FString ContextKey = Type.ToString() + TEXT(" ") + AssetName.ToString();
		if (const FObjectHash* ContextObj = ObjectsHashes.Find(ContextKey))
		{
			if (!ContextObj->ReferencedAssets.IsEmpty())
			{
				RefsStr = TEXT(", References=") + GetReferencesString(*ContextObj);
			}
			SuppressedObjectKeys.Add(ContextKey);
		}

		Out += FString::Printf(TEXT("%s: %s %s (Hash=%08X%s)\n"),
			*Label,
			*Type.ToString(),
			*AssetName.ToString(),
			Hash,
			*RefsStr);
	}

	// Objects section (sorted), skipping any keys consumed above
	TArray<FString> SortedObjectPaths;
	SortedObjectPaths.Reserve(ObjectsHashes.Num());
	ObjectsHashes.GenerateKeyArray(SortedObjectPaths);
	SortedObjectPaths.Sort([](const FString& A, const FString& B)
	{
		return A.Compare(B, ESearchCase::IgnoreCase) < 0;
	});

	Out += TEXT("\n## Source Components ##\n");

	for (const FString& ObjPath : SortedObjectPaths)
	{
		if (SuppressedObjectKeys.Contains(ObjPath))
		{
			continue; // Object already shown in the "Referenced Assets" section
		}

		const FObjectHash& ObjectHash = ObjectsHashes.FindChecked(ObjPath);

		Out += FString::Printf(TEXT("%s (Hash=%08X)\n"), *ObjPath, ObjectHash.Hash);

		for (const TTuple<FName, FString>& Field : ObjectHash.Fields)
		{
			Out += FString::Printf(TEXT("    * %s: %s\n"), *Field.Key.ToString(), *Field.Value);
		}

		if (!ObjectHash.ReferencedAssets.IsEmpty())
		{
			Out += TEXT("    * References: ");
			Out += GetReferencesString(ObjectHash);
			Out += TEXT("\n");
		}

		Out += TEXT("\n");
	}

	return Out;
}

#endif // #if WITH_EDITOR
