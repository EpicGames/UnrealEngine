// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODInstancedSkinnedMeshComponent.h"
#include "WorldPartition/HLOD/HLODBuilder.h"

#include "Engine/SkinnedAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODInstancedSkinnedMeshComponent)

UHLODInstancedSkinnedMeshComponent::UHLODInstancedSkinnedMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

TUniquePtr<FSkinnedMeshComponentDescriptor> UHLODInstancedSkinnedMeshComponent::AllocateISMComponentDescriptor() const
{
	return MakeUnique<FHLODSkinnedMeshComponentDescriptor>();
}

FHLODSkinnedMeshComponentDescriptor::FHLODSkinnedMeshComponentDescriptor()
{
	ComponentClass = UHLODBuilder::GetInstancedSkinnedMeshComponentClass();
}

void FHLODSkinnedMeshComponentDescriptor::InitFrom(const UInstancedSkinnedMeshComponent* Component, bool bInitBodyInstance)
{
	Super::InitFrom(Component, bInitBodyInstance);

	// Stationnary can be considered as static for the purpose of HLODs
	if (Mobility == EComponentMobility::Stationary)
	{
		Mobility = EComponentMobility::Static;
	}
}


#endif
