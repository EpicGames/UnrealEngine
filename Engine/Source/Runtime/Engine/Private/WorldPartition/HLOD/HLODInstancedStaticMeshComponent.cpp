// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODInstancedStaticMeshComponent.h"
#include "WorldPartition/HLOD/HLODBuilder.h"

#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODInstancedStaticMeshComponent)


UHLODInstancedStaticMeshComponent::UHLODInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

TUniquePtr<FISMComponentDescriptor> UHLODInstancedStaticMeshComponent::AllocateISMComponentDescriptor() const
{
	return MakeUnique<FHLODISMComponentDescriptor>();
}

void UHLODInstancedStaticMeshComponent::SetSourceComponentsToInstancesMap(UHLODInstancedStaticMeshComponent::FSourceComponentsToInstancesMap&& InSourceComponentsToInstances)
{
	SourceComponentsToInstances = MoveTemp(InSourceComponentsToInstances);
}

const UHLODInstancedStaticMeshComponent::FSourceComponentsToInstancesMap& UHLODInstancedStaticMeshComponent::GetSourceComponentsToInstancesMap() const
{
	return SourceComponentsToInstances;
}

FHLODISMComponentDescriptor::FHLODISMComponentDescriptor()
{
	ComponentClass = UHLODBuilder::GetInstancedStaticMeshComponentClass();
}

void FHLODISMComponentDescriptor::InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance)
{
	Super::InitFrom(Component, bInitBodyInstance);

	// Improve instance batching by ignoring some of the fields that aren't relevant for HLOD
	Mobility = EComponentMobility::Static;
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::No;
	bSelectable = true;
	bHasPerInstanceHitProxies = false;
	bConsiderForActorPlacementWhenHidden = false;
	bUseDefaultCollision = true;
	bGenerateOverlapEvents = false;
	bOverrideNavigationExport = false;
	bForceNavigationObstacle = false;
	bCanEverAffectNavigation = false;
	bFillCollisionUnderneathForNavmesh = false;

	// Force the correct ISM component class as it can be changed by the parent InitFrom() implementation
	ComponentClass = UHLODBuilder::GetInstancedStaticMeshComponentClass();
}


void FHLODISMComponentDescriptor::InitComponent(UInstancedStaticMeshComponent* ISMComponent) const
{
	Super::InitComponent(ISMComponent);

	if (ISMComponent->GetStaticMesh())
	{
		ISMComponent->SetForcedLodModel(ISMComponent->GetStaticMesh()->GetNumLODs());
	}
}

#endif
