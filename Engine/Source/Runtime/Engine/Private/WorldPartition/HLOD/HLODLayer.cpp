// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HLODLayer.cpp: UHLODLayer class implementation
=============================================================================*/

#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODActor.h"

#if WITH_EDITOR
#include "Dialogs/Dialogs.h"
#include "Engine/World.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataValidation.h"
#include "Misc/StringFormatArg.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "WorldPartition/HLOD/HLODModifier.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODLayer)

DEFINE_LOG_CATEGORY_STATIC(LogHLODLayer, Log, All);

#define LOCTEXT_NAMESPACE "HLODLayer"

UHLODLayer::UHLODLayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bIsSpatiallyLoaded(true)
	, CellSize(25600)
	, LoadingRange(51200)
	, HLODActorClass(AWorldPartitionHLOD::StaticClass())
	, EditorLoadingBehavior(EHLODLayerEditorLoadingBehavior::Default)
#endif
{
}

#if WITH_EDITOR
bool UHLODLayer::DoesRequireWarmup() const
{
	IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
	if (IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr)
	{
		return WPHLODUtilities->GetHLODBuilderClass(this)->GetDefaultObject<UHLODBuilder>()->RequiresWarmup();
	}

	return false;
}

UHLODLayer* UHLODLayer::GetEngineDefaultHLODLayersSetup()
{
	UHLODLayer* Result = nullptr;

	if (FConfigFile* EngineConfig = GConfig->FindConfigFileWithBaseName(TEXT("Engine")))
	{
		FString DefaultHLODLayerName;
		if (EngineConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultWorldPartitionHLODLayer"), DefaultHLODLayerName))
		{
			FSoftObjectPath DefaultHLODLayerPath(*DefaultHLODLayerName);
			TSoftObjectPtr<UHLODLayer> EngineHLODLayerPath(DefaultHLODLayerPath);

			if (UHLODLayer* EngineHLODLayer = EngineHLODLayerPath.LoadSynchronous())
			{
				Result = EngineHLODLayer;
			}
		}
	}

	return Result;
}

UHLODLayer* UHLODLayer::DuplicateHLODLayersSetup(UHLODLayer* HLODLayer, const FString& DestinationPath, const FString& Prefix)
{
	UHLODLayer* Result = nullptr;

	UHLODLayer* LastHLODLayer = nullptr;
	UHLODLayer* CurrentHLODLayer = HLODLayer;

	while (CurrentHLODLayer)
	{
		const FString PackageName = DestinationPath + TEXT("_") + CurrentHLODLayer->GetName();
		UPackage* Package = CreatePackage(*PackageName);
		// In case Package already exists setting this flag will allow overwriting it
		Package->MarkAsFullyLoaded();

		FObjectDuplicationParameters ObjParameters(CurrentHLODLayer, Package);
		ObjParameters.DestName = FName(Prefix + TEXT("_") + CurrentHLODLayer->GetName());
		ObjParameters.ApplyFlags = RF_Public | RF_Standalone;

		UHLODLayer* NewHLODLayer = CastChecked<UHLODLayer>(StaticDuplicateObjectEx(ObjParameters));
		check(NewHLODLayer);

		if (LastHLODLayer)
		{
			LastHLODLayer->SetParentLayer(NewHLODLayer);
		}
		else
		{
			Result = NewHLODLayer;
		}

		LastHLODLayer = NewHLODLayer;
		CurrentHLODLayer = CurrentHLODLayer->GetParentLayer();
	}

	return Result;
}

void UHLODLayer::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
		if (IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr)
		{
			const UClass* BuilderClass = WPHLODUtilities->GetHLODBuilderClass(this);
			const UClass* BuilderSettingsClass = BuilderClass ? BuilderClass->GetDefaultObject<UHLODBuilder>()->GetSettingsClass() : nullptr;

			if (!HLODBuilderSettings || (BuilderSettingsClass && !HLODBuilderSettings->IsA(BuilderSettingsClass)))
			{
				HLODBuilderSettings = WPHLODUtilities->CreateHLODBuilderSettings(this);
			}
		}

		if (bAlwaysLoaded_DEPRECATED)
		{
			bIsSpatiallyLoaded = false;
		}
	}
}

#if WITH_EDITORONLY_DATA
void UHLODLayer::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/WorldPartitionHLODUtilities.HLODBuilderInstancingSettings")));
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/WorldPartitionHLODUtilities.HLODBuilderMeshMerge")));
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/WorldPartitionHLODUtilities.HLODBuilderMeshSimplify")));
	OutConstructClasses.Add(FTopLevelAssetPath(TEXT("/Script/WorldPartitionHLODUtilities.HLODBuilderMeshApproximate")));
}
#endif

// Some types of HLOD layers aren't meant to be used as parent layers of others.
// Instancing for example is a good example - using instancing for HLOD1 when HLOD0 is built out of merged meshes makes no sense. 
// The merged assets are private inside the OFPA packages (and the HLOD0 meshes are all unique so wouldn't benefit from instancing anyway).
// Some configuration of HLOD layers are invalid. For example, using merged meshes without merging materials for an HLOD1 would mean
// that the generated meshes for that layer would try to use the HLOD0 materials directly.
static bool IsInvalidSourceMaterialReuse(const UHLODLayer* ParentLayer, const UHLODLayer* Layer)
{
	if (ParentLayer && ParentLayer->GetHLODBuilderSettings() && Layer->GetHLODBuilderSettings())
	{
		if (ParentLayer->GetHLODBuilderSettings()->IsReusingSourceMaterials() && !Layer->GetHLODBuilderSettings()->IsReusingSourceMaterials())
		{
			return true;
		}
	}

	return false;
}

void UHLODLayer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : FName();


	if (PropertyName == GET_MEMBER_NAME_CHECKED(UHLODLayer, LayerType) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UHLODLayer, HLODBuilderClass))
	{
		IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
		IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr;
		if (WPHLODUtilities != nullptr)
		{
			HLODBuilderSettings = WPHLODUtilities->CreateHLODBuilderSettings(this);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UHLODLayer, ParentLayer))
	{
		bool bInvalidParentLayer = false;

		TSet<const UHLODLayer*> VisitedHLODLayers;
		const UHLODLayer* CurHLODLayer = ParentLayer;
		while (CurHLODLayer)
		{
			bool bHLODLayerWasAlreadyInSet;
			VisitedHLODLayers.Add(CurHLODLayer, &bHLODLayerWasAlreadyInSet);
			if (bHLODLayerWasAlreadyInSet)
			{
				bInvalidParentLayer = true;
				UE_LOGF(LogHLODLayer, Error, "Circular HLOD parent chain detected: HLODLayer=%ls ParentLayer=%ls", *GetName(), *ParentLayer->GetName());
				break;
			}
			CurHLODLayer = CurHLODLayer->GetParentLayer();
		}

		if (IsInvalidSourceMaterialReuse(ParentLayer, this))
		{
			UE_LOGF(LogHLODLayer, Error, "Invalid HLOD settings. Parent layer %ls will reuse private materials created for this HLOD layer. Common error is to use an \"instancing\" parent layer, or a \"merged\" parent layer set to not merge materials. Change the type of HLOD generated by the parent layer or assign a new parent layer.", *ParentLayer->GetPathName());
			bInvalidParentLayer = true;
		}

		if (bInvalidParentLayer)
		{
			ParentLayer = nullptr;

			FText FormattedMessage = LOCTEXT("NotifyInvalidHLODParentLayer", "Invalid Parent HLOD Layer specified, see log for more details.");

			// Show toast.
			FNotificationInfo Info(FormattedMessage);
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UHLODLayer, EditorLoadingBehavior))
	{
		// Warn when enabling loading for Instancing layers, as this can have performance/memory implications
		if (EditorLoadingBehavior == EHLODLayerEditorLoadingBehavior::ForceLoaded && LayerType == EHLODLayerType::Instancing)
		{
			FSuppressableWarningDialog::FSetupInfo Info(
				LOCTEXT("InstancingHLODLoadWarning_Message", "Loading Instancing HLOD actors in the editor can have significant performance and memory implications, as this may lead to a lot of assets being loaded.\n\nAre you sure you want to enable editor loading for this HLOD layer?"),
				LOCTEXT("InstancingHLODLoadWarning_Title", "Warning: Loading Instancing HLOD Layer"),
				TEXT("InstancingHLODLayerEditorLoadingWarning")
			);
			Info.ConfirmText = LOCTEXT("InstancingHLODLoadWarning_Confirm", "Yes");
			Info.CancelText = LOCTEXT("InstancingHLODLoadWarning_Cancel", "No");

			FSuppressableWarningDialog WarningDialog(Info);
			if (WarningDialog.ShowModal() == FSuppressableWarningDialog::Cancel)
			{
				// User cancelled, revert to default
				EditorLoadingBehavior = EHLODLayerEditorLoadingBehavior::Default;
			}
		}

		// Refresh all loaders to re-evaluate which HLODs should be loaded
		IWorldPartitionActorLoaderInterface::RefreshLoadedState(true);
	}
}

EDataValidationResult UHLODLayer::IsDataValid(class FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	if (IsInvalidSourceMaterialReuse(ParentLayer, this))
	{
		Result = EDataValidationResult::Invalid;
		Context.AddError(LOCTEXT("ParentReusingSourceMaterial", "Invalid HLOD settings. Assigned parent layer will reuse private materials created for this HLOD layer. Common error is to use an \"instancing\" parent layer, or a \"merged\" parent layer set to not merge materials. Change the type of HLOD generated by the parent layer or assign a new parent layer."));
	}

	return Result;
}

FName UHLODLayer::GetRuntimeGridName(uint32 InLODLevel, int32 InCellSize, double InLoadingRange)
{
	return *FString::Format(TEXT("HLOD{0}_{1}m_{2}m"), { InLODLevel, int32(InCellSize * 0.01f), int32(InLoadingRange * 0.01f)});
}

FName UHLODLayer::GetRuntimeGrid(uint32 InHLODLevel) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return !IsSpatiallyLoaded() ? NAME_None : GetRuntimeGridName(InHLODLevel, CellSize, LoadingRange);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UHLODLayer::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	// HLOD Layer
	InHashBuilder.HashField(LayerType, GET_MEMBER_NAME_CHECKED(UHLODLayer, LayerType));
	
	if (HLODBuilderSettings)
	{
		HLODBuilderSettings->ComputeHLODHash(InHashBuilder);
	}

	if (HLODModifierClass.Get())
	{
		InHashBuilder.HashField(HLODModifierClass->GetPathName(), GET_MEMBER_NAME_CHECKED(UHLODLayer, HLODModifierClass));
	}

	if (bForceRayTracingFarField)
	{
		InHashBuilder.HashField(bForceRayTracingFarField, GET_MEMBER_NAME_CHECKED(UHLODLayer, bForceRayTracingFarField));
	}
}

bool UHLODLayer::MustBeLoadedOnServer() const
{
	return HLODModifierClass && HLODModifierClass->GetDefaultObject<UWorldPartitionHLODModifier>()->MustBeLoadedOnServer();
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE