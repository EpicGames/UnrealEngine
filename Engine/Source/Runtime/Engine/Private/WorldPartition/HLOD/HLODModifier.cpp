// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODModifier)


UWorldPartitionHLODModifier::UWorldPartitionHLODModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
