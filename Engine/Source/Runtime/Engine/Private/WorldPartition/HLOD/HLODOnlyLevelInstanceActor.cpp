// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODOnlyLevelInstanceActor.h"

#include "Engine/World.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODOnlyLevelInstanceActor)

AWorldPartitionHLODOnlyLevelInstance::AWorldPartitionHLODOnlyLevelInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LevelInstanceActorGuid(this)
	, LevelInstanceActorImpl(this)
{
	RootComponent = CreateDefaultSubobject<ULevelInstanceComponent>(TEXT("Root"));
	RootComponent->Mobility = EComponentMobility::Static;

	bIsEditorOnlyActor = true;
}

const FLevelInstanceID& AWorldPartitionHLODOnlyLevelInstance::GetLevelInstanceID() const
{
	return LevelInstanceActorImpl.GetLevelInstanceID();
}

bool AWorldPartitionHLODOnlyLevelInstance::HasValidLevelInstanceID() const
{
	return LevelInstanceActorImpl.HasValidLevelInstanceID();
}

const FGuid& AWorldPartitionHLODOnlyLevelInstance::GetLevelInstanceGuid() const
{
	return LevelInstanceActorGuid.GetGuid();
}

const TSoftObjectPtr<UWorld>& AWorldPartitionHLODOnlyLevelInstance::GetWorldAsset() const
{
	return WorldAsset;
}

bool AWorldPartitionHLODOnlyLevelInstance::IsLoadingEnabled() const
{
	return false;
}

bool AWorldPartitionHLODOnlyLevelInstance::SetWorldAsset(TSoftObjectPtr<UWorld> InWorldAsset)
{
	WorldAsset = InWorldAsset;
	return true;
}

#if WITH_EDITOR
ULevelInstanceComponent* AWorldPartitionHLODOnlyLevelInstance::GetLevelInstanceComponent() const
{
	return Cast<ULevelInstanceComponent>(RootComponent);
}

ELevelInstanceRuntimeBehavior AWorldPartitionHLODOnlyLevelInstance::GetDesiredRuntimeBehavior() const
{
	return ELevelInstanceRuntimeBehavior::LevelStreaming;
}

ELevelInstanceRuntimeBehavior AWorldPartitionHLODOnlyLevelInstance::GetDefaultRuntimeBehavior() const
{
	return ELevelInstanceRuntimeBehavior::LevelStreaming;
}

bool AWorldPartitionHLODOnlyLevelInstance::HasStandaloneHLOD() const
{
	return true;
}

bool AWorldPartitionHLODOnlyLevelInstance::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty && InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(AActor, bIsEditorOnlyActor))
	{
		return false;
	}

	return true;
}
#endif // WITH_EDITOR

void AWorldPartitionHLODOnlyLevelInstance::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (LevelInstanceActorGuid.IsValid())
	{
		LevelInstanceActorImpl.RegisterLevelInstance();
	}
}

void AWorldPartitionHLODOnlyLevelInstance::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	LevelInstanceActorImpl.UnregisterLevelInstance();
}

#if WITH_EDITOR
TUniquePtr<FWorldPartitionActorDesc> AWorldPartitionHLODOnlyLevelInstance::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FLevelInstanceActorDesc());
}
#endif
