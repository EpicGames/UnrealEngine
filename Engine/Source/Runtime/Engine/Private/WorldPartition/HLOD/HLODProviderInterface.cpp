// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODProviderInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODProviderInterface)


UWorldPartitionHLODProvider::UWorldPartitionHLODProvider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
