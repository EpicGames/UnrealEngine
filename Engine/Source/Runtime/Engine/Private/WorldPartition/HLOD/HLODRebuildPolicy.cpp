// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODRebuildPolicy.h"
#include "WorldPartition/HLOD/HLODActor.h"

UHLODRebuildPolicyData::UHLODRebuildPolicyData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UHLODRebuildPolicy::UHLODRebuildPolicy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

TArray<TSubclassOf<UHLODRebuildPolicy>> UHLODRebuildPolicy::GetRebuildPoliciesClasses()
{
	TArray<TSubclassOf<UHLODRebuildPolicy>> HLODRebuildPoliciesClasses;

	auto AddPolicyClass = [&HLODRebuildPoliciesClasses](const FSoftObjectPath& ObjectPath)
	{
		UObject* FoundObject = ObjectPath.TryLoad(nullptr, LOAD_Quiet);
		if (FoundObject)
		{
			TSubclassOf<UHLODRebuildPolicy> PolicyClass = Cast<UClass>(FoundObject);
			if (PolicyClass)
			{
				HLODRebuildPoliciesClasses.AddUnique(PolicyClass);
			}
			else
			{
				UE_LOGF(LogHLODBuilder, Warning, "Invalid rebuild policy class specified: %ls", *ObjectPath.ToString());
			}
		}
		else
		{
			UE_LOGF(LogHLODBuilder, Warning, "Rebuild policy class not found: %ls", *ObjectPath.ToString());
		}
	};

	for (const FSoftObjectPath& HLODRebuildPolicyClassPath : UHLODRebuildPolicy::StaticClass()->GetDefaultObject<UHLODRebuildPolicy>()->HLODRebuildPolicies)
	{
		AddPolicyClass(HLODRebuildPolicyClassPath);
	}

	if (HLODRebuildPoliciesClasses.IsEmpty())
	{
		AddPolicyClass(FSoftObjectPath(TEXT("/Script/WorldPartitionHLODUtilities.HLODRebuildPolicyHashCompare")));
	}

	return HLODRebuildPoliciesClasses;
}

FHLODRebuildPolicyDataSet UHLODRebuildPolicy::ComputeDataForRebuildPolicies(const AWorldPartitionHLOD* InHLODActor, const TArray<UActorComponent*>& InSourceComponents, const bool bInForComparison)
{
	FHLODRebuildPolicyDataSet HLODRebuildPolicyDataSet;

	for (TSubclassOf<UHLODRebuildPolicy> HLODRebuildPolicyClass : GetRebuildPoliciesClasses())
	{
		UHLODRebuildPolicy* UHLODRebuildPolicyCDO = HLODRebuildPolicyClass.GetDefaultObject();
		check(UHLODRebuildPolicyCDO);

		UHLODRebuildPolicyData* OldData = nullptr;
		if (bInForComparison)
		{
			const FHLODRebuildPolicyDataSet& BaselineHLODRebuildPolicyDataSet = InHLODActor->GetHLODRebuildPolicyDataSet();
			const TObjectPtr<UHLODRebuildPolicyData>* OldDataPtr = BaselineHLODRebuildPolicyDataSet.FindByPredicate([UHLODRebuildPolicyCDO](const TObjectPtr<UHLODRebuildPolicyData>& Entry) { return Entry.IsA(UHLODRebuildPolicyCDO->GetRebuildPolicyDataType()); });
			if (OldDataPtr)
			{
				OldData = *OldDataPtr;
			}
		}
		
		UHLODRebuildPolicyData* NewData = UHLODRebuildPolicyCDO->ComputeDataForRebuildPolicy(InHLODActor, InSourceComponents, OldData);
		if (NewData)
		{
			HLODRebuildPolicyDataSet.Emplace(NewData);
		}
	}

	return HLODRebuildPolicyDataSet;
}

EHLODRebuildPolicyDecision UHLODRebuildPolicy::Evaluate(const AWorldPartitionHLOD* InHLODActor, const FHLODRebuildPolicyDataSet& InOldData, const FHLODRebuildPolicyDataSet& InNewData)
{
	EHLODRebuildPolicyDecision FinalDecision = EHLODRebuildPolicyDecision::None;

	UE_LOGF(LogHLODBuilder, Display, "Evaluating HLOD rebuild policies...");

	for (TSubclassOf<UHLODRebuildPolicy> HLODRebuildPolicyClass : GetRebuildPoliciesClasses())
	{
		UHLODRebuildPolicy* UHLODRebuildPolicyCDO = HLODRebuildPolicyClass.GetDefaultObject();
		check(UHLODRebuildPolicyCDO);

		// Find old data
		const TObjectPtr<UHLODRebuildPolicyData>* OldHLODRebuildPolicyDataPtr = InOldData.FindByPredicate([UHLODRebuildPolicyCDO](const TObjectPtr<UHLODRebuildPolicyData>& Entry) { return Entry.IsA(UHLODRebuildPolicyCDO->GetRebuildPolicyDataType()); });
		const UHLODRebuildPolicyData* OldHLODRebuildPolicyData = nullptr;
		if (OldHLODRebuildPolicyDataPtr)
		{
			OldHLODRebuildPolicyData = *OldHLODRebuildPolicyDataPtr;
		}

		// Find new Data
		const TObjectPtr<UHLODRebuildPolicyData>* NewHLODRebuildPolicyDataPtr = InNewData.FindByPredicate([UHLODRebuildPolicyCDO](const TObjectPtr<UHLODRebuildPolicyData>& Entry) { return Entry.IsA(UHLODRebuildPolicyCDO->GetRebuildPolicyDataType()); });
		const UHLODRebuildPolicyData* NewHLODRebuildPolicyData = nullptr;
		if (NewHLODRebuildPolicyDataPtr)
		{
			NewHLODRebuildPolicyData = *NewHLODRebuildPolicyDataPtr;
		}

		if (!NewHLODRebuildPolicyData)
		{
			continue;
		}
		
		FString Reason;
		EHLODRebuildPolicyDecision Decision = UHLODRebuildPolicyCDO->Evaluate(InHLODActor, OldHLODRebuildPolicyData, NewHLODRebuildPolicyData, Reason);

		FString PolicyName = UHLODRebuildPolicyCDO->GetClass()->GetName();
		FString DecisionString = StaticEnum<EHLODRebuildPolicyDecision>()->GetNameStringByValue(static_cast<int64>(Decision));
		
		UE_CLOGF( Reason.IsEmpty(), LogHLODBuilder, Display, " * %ls -> %ls", *PolicyName, *DecisionString);
		UE_CLOGF(!Reason.IsEmpty(), LogHLODBuilder, Display, " * %ls -> %ls (%ls)", *PolicyName, *DecisionString, *Reason);
		
		if (Decision != EHLODRebuildPolicyDecision::None)
		{
			FinalDecision = Decision;
		}

		// If the build was rejected, stop processing
		if (FinalDecision == EHLODRebuildPolicyDecision::RejectRebuild)
		{
			break;
		}
	}

	FString FinalDecisionString = StaticEnum<EHLODRebuildPolicyDecision>()->GetNameStringByValue(static_cast<int64>(FinalDecision));
	UE_LOGF(LogHLODBuilder, Display, "Evaluated HLOD rebuild policies, final decision: %ls", *FinalDecisionString);

	return FinalDecision;
}

#endif