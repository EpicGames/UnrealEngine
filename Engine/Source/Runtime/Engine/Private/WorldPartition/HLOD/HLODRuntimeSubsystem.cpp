// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"
#include "Engine/Level.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "WorldPartition/WorldPartition.h"
#include "RenderUtils.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "EngineModule.h"
#include "LevelUtils.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "SceneViewExtension.h"
#include "StaticMeshResources.h"
#include "Subsystems/Subsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODRuntimeSubsystem)

#define LOCTEXT_NAMESPACE "HLODRuntimeSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogHLODRuntimeSubsystem, Log, All);

static TAutoConsoleVariable<int32> CVarHLODWarmupEnabled(
	TEXT("wp.Runtime.HLOD.WarmupEnabled"),
	1,
	TEXT("Enable HLOD assets warmup. Will delay unloading of cells & transition to HLODs for wp.Runtime.HLOD.WarmupNumFrames frames."));

static TAutoConsoleVariable<int32> CVarHLODWarmupVT(
	TEXT("wp.Runtime.HLOD.WarmupVT"),
	1,
	TEXT("Enable virtual texture warmup for HLOD assets. Requires wp.Runtime.HLOD.WarmupEnabled to be 1."));

static TAutoConsoleVariable<int32> CVarHLODWarmupNanite(
	TEXT("wp.Runtime.HLOD.WarmupNanite"),
	1,
	TEXT("Enable Nanite warmup for HLOD assets. Requires wp.Runtime.HLOD.WarmupEnabled to be 1."));

static TAutoConsoleVariable<int32> CVarHLODWarmupNumFrames(
	TEXT("wp.Runtime.HLOD.WarmupNumFrames"),
	5,
	TEXT("Delay unloading of a cell for this amount of frames to ensure HLOD assets are ready to be shown at the proper resolution. Set to 0 to force disable warmup."));

static TAutoConsoleVariable<int32> CVarHLODWarmupDebugDraw(
	TEXT("wp.Runtime.HLOD.WarmupDebugDraw"),
	0,
	TEXT("Draw debug display for the warmup requests"));

static TAutoConsoleVariable<float> CVarHLODWarmupVTScaleFactor(
	TEXT("wp.Runtime.HLOD.WarmupVTScaleFactor"),
	2.0f,
	TEXT("Scale the VT size we ask to prefetch by this factor."));

static TAutoConsoleVariable<int32> CVarHLODWarmupVTSizeClamp(
	TEXT("wp.Runtime.HLOD.WarmupVTSizeClamp"),
	2048,
	TEXT("Clamp VT warmup requests for safety."));

namespace FHLODRuntimeSubsystem
{
	//Cached value for UWorldPartitionHLODRuntimeSubsystem::IsWarmupEnabled
	static bool bWarmupEnabled = true;
}

static void HLODRuntimeSubsystemCVarSinkFunction()
{
	// Wait to evaluate if warmup is enabled until after RHI is initialized. Sink functions get called very early in engine init
	if (!GIsRHIInitialized)
	{
		return;
	}

	// Check if any cvars affecting warmup have changed
	bool bNewWarmupEnabled = UWorldPartitionHLODRuntimeSubsystem::IsWarmupEnabled();
	if (FHLODRuntimeSubsystem::bWarmupEnabled == bNewWarmupEnabled)
	{
		return;
	}

	// Update the static cached value and every subsystem instance
	FHLODRuntimeSubsystem::bWarmupEnabled = bNewWarmupEnabled;
	for (UWorldPartitionHLODRuntimeSubsystem* HLODRuntimeSubsystem : TObjectRange<UWorldPartitionHLODRuntimeSubsystem>(RF_ClassDefaultObject | RF_ArchetypeObject, false, EInternalObjectFlags::Garbage))
	{
		HLODRuntimeSubsystem->OnCVarsChanged();
	}
}

static FAutoConsoleVariableSink CVarHLODSink(FConsoleCommandDelegate::CreateStatic(&HLODRuntimeSubsystemCVarSinkFunction));

namespace FHLODRuntimeSubsystem
{
    static const UWorldPartitionRuntimeCell* GetRuntimeCell(IWorldPartitionHLODObject* InWorldPartitionHLOD)
    {
	    const ULevel* Level = InWorldPartitionHLOD->GetHLODLevel();
	    const ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level);
	    return LevelStreaming ? Cast<const UWorldPartitionRuntimeCell>(LevelStreaming->GetWorldPartitionCell()) : nullptr;
    }
    
    static UWorldPartition* GetWorldPartition(IWorldPartitionHLODObject* InWorldPartitionHLOD)
    {
	    // Alwaysloaded Cell level will have a WorldPartition
	    if (UWorldPartition* WorldPartition = InWorldPartitionHLOD->GetHLODLevel()->GetWorldPartition())
	    {
		    return WorldPartition;
	    } // If not find it through the cell
	    else if (const UWorldPartitionRuntimeCell* RuntimeCell = GetRuntimeCell(InWorldPartitionHLOD))
	    {
		    return RuntimeCell->GetOuterWorld()->GetWorldPartition();
	    }
    
	    return nullptr;
    }
}


class FHLODResourcesResidencySceneViewExtension : public FWorldSceneViewExtension
{
public:
	FHLODResourcesResidencySceneViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld)
		: FWorldSceneViewExtension(AutoRegister, InWorld)
	{
	}

	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override {}
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override {}
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override
	{
		GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()->OnBeginRenderViews(InViewFamily);
	}
};


UWorldPartitionHLODRuntimeSubsystem::UWorldPartitionHLODRuntimeSubsystem()
	: UWorldSubsystem()
	, bCachedShouldPerformWarmup(FHLODRuntimeSubsystem::bWarmupEnabled)
{
}

bool UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled = true;

FAutoConsoleCommand UWorldPartitionHLODRuntimeSubsystem::EnableHLODCommand(
	TEXT("wp.Runtime.HLOD"),
	TEXT("Turn on/off loading & rendering of world partition HLODs."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled = (Args.Num() != 1) || (Args[0] != TEXT("0"));
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (UWorldPartitionHLODRuntimeSubsystem* HLODSubSystem = World->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>()) 
				{
					for (const auto& KeyValuePair : HLODSubSystem->WorldPartitionsHLODRuntimeData)
					{
						for (const auto& CellHLODMapping : KeyValuePair.Value.CellsData)
						{
							const FCellData& CellData = CellHLODMapping.Value;
							bool bIsHLODVisible = UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !CellData.bIsCellVisible;
							for (IWorldPartitionHLODObject* HLODObject : CellData.LoadedHLODs)
							{
								HLODObject->SetVisibility(bIsHLODVisible);
							}
						}
					}

					// Update visibility for Standalone HLOD actors referencing unloaded cells
					for (const TPair<FGuid, TSet<IWorldPartitionHLODObject*>>& Pair : HLODSubSystem->StandaloneHLODObjectsReferencingUnloadedCells)
					{
						for (IWorldPartitionHLODObject* HLODObject : Pair.Value)
						{
							HLODObject->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled);
						}
					}
				}
			}
		}
	})
);

bool UWorldPartitionHLODRuntimeSubsystem::IsHLODEnabled()
{
	return UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled;
}

bool UWorldPartitionHLODRuntimeSubsystem::IsWarmupEnabled()
{
	// Test if warmup is disabled globally.
	const bool bWarmupEnabled = CVarHLODWarmupEnabled.GetValueOnGameThread() != 0;
	if (!bWarmupEnabled)
	{
		return false;
	}

	// If warmup num of frames is invalid, no warmup needed
	if (CVarHLODWarmupNumFrames.GetValueOnGameThread() <= 0)
	{
		return false;
	}

	// If warmup num of frames is invalid, no warmup needed
	const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	const bool bNaniteEnabled = UseNanite(ShaderPlatform);
	const bool bVirtualTextureEnabled = UseVirtualTexturing(ShaderPlatform);
	const bool bWarmupNanite = CVarHLODWarmupNanite.GetValueOnGameThread() != 0;
	const bool bWarmupVT = CVarHLODWarmupVT.GetValueOnGameThread() != 0;
	const bool bWarmupNeeded = (bNaniteEnabled && bWarmupNanite) || (bVirtualTextureEnabled && bWarmupVT);
	if (!bWarmupNeeded)
	{
		return false;
	}

	return true;
}

bool UWorldPartitionHLODRuntimeSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Game || WorldType == EWorldType::PIE;
}

void UWorldPartitionHLODRuntimeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	// Ensure the WorldPartitionSubsystem gets created before the HLODRuntimeSubsystem
	Collection.InitializeDependency<UWorldPartitionSubsystem>();

	Super::Initialize(Collection);

	UWorld* World = GetWorld();
	check(World->IsGameWorld());

	GetWorld()->OnWorldPartitionInitialized().AddUObject(this, &UWorldPartitionHLODRuntimeSubsystem::OnWorldPartitionInitialized);
	GetWorld()->OnWorldPartitionUninitialized().AddUObject(this, &UWorldPartitionHLODRuntimeSubsystem::OnWorldPartitionUninitialized);

	bCachedShouldPerformWarmup = ShouldPerformWarmup();

	SceneViewExtension = FSceneViewExtensions::NewExtension<FHLODResourcesResidencySceneViewExtension>(World);
}

void UWorldPartitionHLODRuntimeSubsystem::Deinitialize()
{
	Super::Deinitialize();

	GetWorld()->OnWorldPartitionInitialized().RemoveAll(this);
	GetWorld()->OnWorldPartitionUninitialized().RemoveAll(this);
}

void UWorldPartitionHLODRuntimeSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	if (InWorldPartition && InWorldPartition->IsStreamingEnabled())
	{
		check(!WorldPartitionsHLODRuntimeData.Contains(InWorldPartition));

		FWorldPartitionHLODRuntimeData& WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Add(InWorldPartition, FWorldPartitionHLODRuntimeData());
	
		// Build cell to HLOD mapping
		if (InWorldPartition && InWorldPartition->RuntimeHash)
		{
			InWorldPartition->RuntimeHash->ForEachStreamingCells([this, &WorldPartitionHLODRuntimeData, &InWorldPartition](const UWorldPartitionRuntimeCell* Cell)
			{
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering cell %ls - %ls", *Cell->GetGuid().ToString(), *Cell->GetDebugName());

				FCellData& CellData = WorldPartitionHLODRuntimeData.CellsData.Emplace(Cell->GetGuid());

				if (InWorldPartition->HasStandaloneHLOD())
				{
					StandaloneHLODCellToWorldPartitionMap.Add(Cell->GetGuid(), InWorldPartition);

					// If we have Standalone HLOD Actor for this cell, attach it to the cell
					if (TSet<IWorldPartitionHLODObject*>* StandaloneHLODObjects = StandaloneHLODObjectsReferencingUnloadedCells.Find(Cell->GetGuid()))
					{
						for (IWorldPartitionHLODObject* HLODObject : (*StandaloneHLODObjects))
						{
							CellData.LoadedHLODs.Add(HLODObject);
							HLODObject->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !CellData.bIsCellVisible);
							UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Attached Standalone HLOD %ls to cell %ls", *HLODObject->GetHLODNameOrLabel(), *Cell->GetGuid().ToString());
						}
						StandaloneHLODObjectsReferencingUnloadedCells.Remove(Cell->GetGuid());
					}
				}
				return true;
			});
		}
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	if (InWorldPartition && InWorldPartition->IsStreamingEnabled())
	{
		check(WorldPartitionsHLODRuntimeData.Contains(InWorldPartition));

		FWorldPartitionHLODRuntimeData& WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.FindChecked(InWorldPartition);

		if (InWorldPartition->HasStandaloneHLOD())
		{
			for (const TPair<FGuid, FCellData>& It : WorldPartitionHLODRuntimeData.CellsData)
			{
				const FCellData& CellData = It.Value;
				FGuid CellGuid = It.Key;

				StandaloneHLODCellToWorldPartitionMap.Remove(CellGuid);

				for (IWorldPartitionHLODObject* HLODObject : CellData.LoadedHLODs)
				{
					if (HLODObject->IsStandalone())
					{
						StandaloneHLODObjectsReferencingUnloadedCells.FindOrAdd(CellGuid).Add(HLODObject);
						HLODObject->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled);
						UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Detached Standalone HLOD %ls from cell %ls", *HLODObject->GetHLODNameOrLabel(), *CellGuid.ToString());
					}
				}
			}
		}

#if !NO_LOGGING
		InWorldPartition->RuntimeHash->ForEachStreamingCells([&WorldPartitionHLODRuntimeData](const UWorldPartitionRuntimeCell* Cell)
		{
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering cell %ls - %ls", *Cell->GetGuid().ToString(), *Cell->GetDebugName());
			return true;
		});
#endif

		WorldPartitionsHLODRuntimeData.Remove(InWorldPartition);
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnExternalStreamingObjectInjected(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	UWorldPartition* OwnerPartition = ExternalStreamingObject->GetOuterWorld()->GetWorldPartition();
	FWorldPartitionHLODRuntimeData* WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Find(OwnerPartition);
	if (WorldPartitionHLODRuntimeData)
	{
		ExternalStreamingObject->ForEachStreamingCells([WorldPartitionHLODRuntimeData](const UWorldPartitionRuntimeCell& Cell)
		{
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering external cell %ls - %ls", *Cell.GetGuid().ToString(), *Cell.GetDebugName());
			WorldPartitionHLODRuntimeData->CellsData.Emplace(Cell.GetGuid());
			return true;
		});
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnExternalStreamingObjectRemoved(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	UWorldPartition* OwnerPartition = ExternalStreamingObject->GetOuterWorld()->GetWorldPartition();
	FWorldPartitionHLODRuntimeData* WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Find(OwnerPartition);
	if (WorldPartitionHLODRuntimeData)
	{
		ExternalStreamingObject->ForEachStreamingCells([WorldPartitionHLODRuntimeData](const UWorldPartitionRuntimeCell& Cell)
		{
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering external cell %ls - %ls", *Cell.GetGuid().ToString(), *Cell.GetDebugName());
			WorldPartitionHLODRuntimeData->CellsData.Remove(Cell.GetGuid());
			return true;
		});
	}
}

const UWorldPartitionHLODRuntimeSubsystem::FCellData* UWorldPartitionHLODRuntimeSubsystem::GetCellData(const UWorldPartitionRuntimeCell* InCell) const
{
	return const_cast<UWorldPartitionHLODRuntimeSubsystem*>(this)->GetCellData(InCell);
}

UWorldPartitionHLODRuntimeSubsystem::FCellData* UWorldPartitionHLODRuntimeSubsystem::GetCellData(const UWorldPartitionRuntimeCell* InCell)
{
	const UWorldPartition* WorldPartition = InCell->GetOuterWorld()->GetWorldPartition();
	if (WorldPartition)
	{
		FWorldPartitionHLODRuntimeData* WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Find(WorldPartition);
		if (WorldPartitionHLODRuntimeData)
		{
			check(WorldPartition->IsStreamingEnabled());
			return WorldPartitionHLODRuntimeData->CellsData.Find(InCell->GetGuid());
		}
	}

	return nullptr;
}

UWorldPartitionHLODRuntimeSubsystem::FCellData* UWorldPartitionHLODRuntimeSubsystem::GetCellData(IWorldPartitionHLODObject* InWorldPartitionHLOD)
{
	const UWorldPartition* WorldPartition = FHLODRuntimeSubsystem::GetWorldPartition(InWorldPartitionHLOD);
	if (WorldPartition)
	{
		FWorldPartitionHLODRuntimeData* WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Find(WorldPartition);
		if (WorldPartitionHLODRuntimeData)
		{
			check(WorldPartition->IsStreamingEnabled());
			const FGuid CellGuid = InWorldPartitionHLOD->GetSourceCellGuid();
			return WorldPartitionHLODRuntimeData->CellsData.Find(CellGuid);
		}
	}

	return nullptr;		
}

UWorldPartitionHLODRuntimeSubsystem::FCellData* UWorldPartitionHLODRuntimeSubsystem::GetCellData(const UWorldPartition* InWorldPartition, const FGuid& InCellGuid)
{
	if (InWorldPartition)
	{
		FWorldPartitionHLODRuntimeData* WorldPartitionHLODRuntimeData = WorldPartitionsHLODRuntimeData.Find(InWorldPartition);
		if (WorldPartitionHLODRuntimeData)
		{
			check(InWorldPartition->IsStreamingEnabled());
			return WorldPartitionHLODRuntimeData->CellsData.Find(InCellGuid);
		}
	}

	return nullptr;
}

const TArray<IWorldPartitionHLODObject*>& UWorldPartitionHLODRuntimeSubsystem::GetHLODObjectsForCell(const UWorldPartitionRuntimeCell* InCell) const
{
	if (const FCellData* CellData = GetCellData(InCell))
	{
		return CellData->LoadedHLODs;
	}

	// No HLOD found for the given cell, return a dummy array
	static const TArray<IWorldPartitionHLODObject*> DummyArray;
	return DummyArray;
}

void UWorldPartitionHLODRuntimeSubsystem::RegisterHLODObject(IWorldPartitionHLODObject* InWorldPartitionHLOD)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::RegisterHLODObject);
	
	if (InWorldPartitionHLOD->IsCustomHLOD())
	{
		if (const UWorldPartition* WorldPartition = FHLODRuntimeSubsystem::GetWorldPartition(InWorldPartitionHLOD))
		{
			if (const FGuid* CustomHLODSourceCellGuidPtr = WorldPartition->RuntimeHash->GetCustomHLODActorSourceCellOverride(InWorldPartitionHLOD->GetCustomHLODGuid()))
			{
				FGuid CustomHLODSourceCellGuid = *CustomHLODSourceCellGuidPtr;
				if (FCellData* CustomHLODSourceCellData = GetCellData(WorldPartition, CustomHLODSourceCellGuid))
				{
					// Overriden source cell found in WorldPartitionHLODRuntimeData. Add the HLOD Actor to its LoadedHLODs.
					UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering Custom HLOD %ls for overriden source cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *CustomHLODSourceCellGuid.ToString());

					CustomHLODSourceCellData->LoadedHLODs.Add(InWorldPartitionHLOD);
					InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !CustomHLODSourceCellData->bIsCellVisible);
				}
				else
				{
					UE_LOGF(LogHLODRuntimeSubsystem, Log, "Override source cell '%ls' not found for Custom HLOD actor %ls", *CustomHLODSourceCellGuid.ToString(), *InWorldPartitionHLOD->GetHLODNameOrLabel());
				}
			}
			else
			{
				UE_LOGF(LogHLODRuntimeSubsystem, Log, "Override source cell not found for Custom HLOD actor %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel());
			}
		}
	}
	else if (FCellData* SourceCellData = GetCellData(InWorldPartitionHLOD))
	{
		UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering HLOD %ls for cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

		// Register this HLOD as a representation of the source cell - Needed to warmup an HLOD object when it's source cell is unloading
		SourceCellData->LoadedHLODs.Add(InWorldPartitionHLOD);
		InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !SourceCellData->bIsCellVisible);
	}
	else if (InWorldPartitionHLOD->IsStandalone())
	{	
		const UWorldPartition* WorldPartition = FHLODRuntimeSubsystem::GetWorldPartition(InWorldPartitionHLOD);

		// Standalone HLOD actors are embedded in the main world streaming cells, so for Standalone HLOD actors that are using other Standalone HLOD actors as source actors,
		// source cell GUID won't be correct. During streaming generation, we save the actual source cell GUID, so that we can use that override here.
		if (const FGuid* StandaloneHLODSourceCellsGuidPtr = WorldPartition->RuntimeHash->GetStandaloneHLODActorSourceCellOverride(InWorldPartitionHLOD->GetStandaloneHLODGuid()))
		{
			// Source cell GUID override found
			FGuid StandaloneHLODSourceCellGuid = *StandaloneHLODSourceCellsGuidPtr;
			if (FCellData* StandaloneHLODSourceCellData = GetCellData(WorldPartition, StandaloneHLODSourceCellGuid))
			{
				// Overriden source cell found in WorldPartitionHLODRuntimeData. Add the HLOD Actor to its LoadedHLODs.
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering Standalone HLOD %ls for overriden source cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *StandaloneHLODSourceCellGuid.ToString());

				StandaloneHLODSourceCellData->LoadedHLODs.Add(InWorldPartitionHLOD);
				InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !StandaloneHLODSourceCellData->bIsCellVisible);
			}
			else
			{
				// Overriden source cell not found in WorldPartitionHLODRuntimeData. This can happen when Standalone LI is not currently loaded.
				// Storing the HLOD object in StandaloneHLODObjectsReferencingUnloadedCells so that we can associate it later with the overridden source cell, once it's loaded.
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering Standalone HLOD %ls referencing currently not loaded overriden source cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *StandaloneHLODSourceCellGuid.ToString());

				TSet<IWorldPartitionHLODObject*>& StandaloneHLODs = StandaloneHLODObjectsReferencingUnloadedCells.FindOrAdd(StandaloneHLODSourceCellGuid);
				StandaloneHLODs.Add(InWorldPartitionHLOD);
				InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled);
			}
		}
		else if (TObjectPtr<UWorldPartition>* StandaloneWorldPartitionPtr = StandaloneHLODCellToWorldPartitionMap.Find(InWorldPartitionHLOD->GetSourceCellGuid()))
		{
			UWorldPartition* StandaloneWorldPartition = *StandaloneWorldPartitionPtr;
			if (FCellData* StandaloneHLODSourceCellData = GetCellData(StandaloneWorldPartition, InWorldPartitionHLOD->GetSourceCellGuid()))
			{
				// Source cell found in WorldPartitionHLODRuntimeData
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering Standalone HLOD %ls for Standalone LI source cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

				StandaloneHLODSourceCellData->LoadedHLODs.Add(InWorldPartitionHLOD);
				InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled && !StandaloneHLODSourceCellData->bIsCellVisible);
			}
		}
		else
		{
			// Source cell not found in WorldPartitionHLODRuntimeData. This can happen when Standalone LI is not currently loaded.
			// Storing the HLOD object in StandaloneHLODObjectsReferencingUnloadedCells so that we can associate it later with the source cell, once it's loaded.
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Registering Standalone HLOD %ls referencing currently not loaded cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

			TSet<IWorldPartitionHLODObject*>& StandaloneHLODs = StandaloneHLODObjectsReferencingUnloadedCells.FindOrAdd(InWorldPartitionHLOD->GetSourceCellGuid());
			StandaloneHLODs.Add(InWorldPartitionHLOD);
			InWorldPartitionHLOD->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled);
		}
	}
	else
	{
		UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Found HLOD %ls referencing nonexistent cell '%ls'", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());
		InWorldPartitionHLOD->SetVisibility(false);

#if !UE_BUILD_SHIPPING
		OutdatedHLODObjects.Add(InWorldPartitionHLOD);
#endif
	}

	RemoveHLODObjectFromWarmup(InWorldPartitionHLOD);

	HLODObjectRegisteredEvent.Broadcast(InWorldPartitionHLOD);
}

void UWorldPartitionHLODRuntimeSubsystem::UnregisterHLODObject(IWorldPartitionHLODObject* InWorldPartitionHLOD)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::UnregisterHLODObject);

	if (InWorldPartitionHLOD->IsCustomHLOD())
	{
		if (const UWorldPartition* WorldPartition = FHLODRuntimeSubsystem::GetWorldPartition(InWorldPartitionHLOD))
		{
			if (const FGuid* CustomHLODSourceCellGuidPtr = WorldPartition->RuntimeHash->GetCustomHLODActorSourceCellOverride(InWorldPartitionHLOD->GetCustomHLODGuid()))
			{
				const FGuid CustomHLODSourceCellGuid = *CustomHLODSourceCellGuidPtr;
				if (FCellData* CustomHLODSourceCellData = GetCellData(WorldPartition, CustomHLODSourceCellGuid))
				{
					// Overriden source cell found. Remove the HLOD object from its LoadedHLODs.
					UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering Custom HLOD %ls for overriden source cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *CustomHLODSourceCellGuid.ToString());

					verify(CustomHLODSourceCellData->LoadedHLODs.Remove(InWorldPartitionHLOD));
				}
			}
		}
	}
	else if (FCellData* CellData = GetCellData(InWorldPartitionHLOD))
	{
		UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering HLOD %ls for cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

		verify(CellData->LoadedHLODs.Remove(InWorldPartitionHLOD));
	}
	else if (InWorldPartitionHLOD->IsStandalone())
	{
		const UWorldPartition* WorldPartition = FHLODRuntimeSubsystem::GetWorldPartition(InWorldPartitionHLOD);

		// Check if we have source cell override. See RegisterHLODActor for more info.
		if (const FGuid* SourceCellGuidPtr = WorldPartition->RuntimeHash->GetStandaloneHLODActorSourceCellOverride(InWorldPartitionHLOD->GetStandaloneHLODGuid()))
		{
			// Source cell GUID override found
			const FGuid SourceCellGuid = *SourceCellGuidPtr;
			if (FCellData* SourceCellData = GetCellData(WorldPartition, SourceCellGuid))
			{
				// Overriden source cell found. Remove the HLOD object from its LoadedHLODs.
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering Standalone HLOD %ls for overriden source cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *SourceCellGuid.ToString());

				verify(SourceCellData->LoadedHLODs.Remove(InWorldPartitionHLOD));
			}
			else
			{
				// Overriden source cell not found. Remove the HLOD object from StandaloneHLODObjectsReferencingUnloadedCells map
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering Standalone HLOD %ls for currently not loaded overriden source cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *SourceCellGuid.ToString());

				if (TSet<IWorldPartitionHLODObject*>* StandaloneHLODActorsPtr = StandaloneHLODObjectsReferencingUnloadedCells.Find(SourceCellGuid))
				{
					verify(StandaloneHLODActorsPtr->Remove(InWorldPartitionHLOD));
				}
			}
		}
		else if (TObjectPtr<UWorldPartition>* StandaloneWorldPartitionPtr = StandaloneHLODCellToWorldPartitionMap.Find(InWorldPartitionHLOD->GetSourceCellGuid()))
		{
			UWorldPartition* StandaloneWorldPartition = *StandaloneWorldPartitionPtr;
			if (FCellData* SourceCellData = GetCellData(StandaloneWorldPartition, InWorldPartitionHLOD->GetSourceCellGuid()))
			{
				// Source cell found. Remove the HLOD object from its LoadedHLODs.
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering Standalone HLOD %ls for Standalone LI source cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

				verify(SourceCellData->LoadedHLODs.Remove(InWorldPartitionHLOD));
			}
		}
		else
		{
			// Source cell not found. Remove the HLOD actor from StandaloneHLODObjectsReferencingUnloadedCells map
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Unregistering Standalone HLOD %ls for currently not loaded cell %ls", *InWorldPartitionHLOD->GetHLODNameOrLabel(), *InWorldPartitionHLOD->GetSourceCellGuid().ToString());

			if (TSet<IWorldPartitionHLODObject*>* StandaloneHLODActorsPtr = StandaloneHLODObjectsReferencingUnloadedCells.Find(InWorldPartitionHLOD->GetSourceCellGuid()))
			{
				verify(StandaloneHLODActorsPtr->Remove(InWorldPartitionHLOD));
			}
		}
	}
	else
	{
#if !UE_BUILD_SHIPPING
		OutdatedHLODObjects.Remove(InWorldPartitionHLOD);
#endif
	}

	RemoveHLODObjectFromWarmup(InWorldPartitionHLOD);

	HLODObjectUnregisteredEvent.Broadcast(InWorldPartitionHLOD);
}

void UWorldPartitionHLODRuntimeSubsystem::OnCellShown(const UWorldPartitionRuntimeCell* InCell)
{
	if (FCellData* CellData = GetCellData(InCell))
	{
		CellData->bIsCellVisible = true;

		if (!CellData->LoadedHLODs.IsEmpty())
		{
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Cell shown - %ls - hiding %d HLOD objects", *InCell->GetGuid().ToString(), CellData->LoadedHLODs.Num());

			for (IWorldPartitionHLODObject* HLODObject : CellData->LoadedHLODs)
			{
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "\t\t* %ls", *HLODObject->GetHLODNameOrLabel());
				HLODObject->SetVisibility(false);
			}
		}
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnCellHidden(const UWorldPartitionRuntimeCell* InCell)
{
	if (FCellData* CellData = GetCellData(InCell))
	{
		CellData->bIsCellVisible = false;

		if (!CellData->LoadedHLODs.IsEmpty())
		{
			UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "Cell hidden - %ls - showing %d HLOD objects", *InCell->GetGuid().ToString(), CellData->LoadedHLODs.Num());

			for (IWorldPartitionHLODObject* HLODObject : CellData->LoadedHLODs)
			{
				UE_LOGF(LogHLODRuntimeSubsystem, Verbose, "\t\t* %ls", *HLODObject->GetHLODNameOrLabel());
				HLODObject->SetVisibility(UWorldPartitionHLODRuntimeSubsystem::WorldPartitionHLODEnabled);

				RemoveHLODObjectFromWarmup(HLODObject);
			}
		}
	}
}

static void PrepareVTRequests(TMap<UMaterialInterface*, float>& InOutVTRequests, UMaterialInterface* InMaterialInterface, float InPixelSize)
{
	check(InMaterialInterface);

	float PixelSize = InPixelSize;

	// Assume the texture is wrapped around the object, so the screen size is actually less than the resolution we require.
	PixelSize *= CVarHLODWarmupVTScaleFactor.GetValueOnAnyThread();

	// Clamp for safety
	PixelSize = FMath::Min(PixelSize, CVarHLODWarmupVTSizeClamp.GetValueOnAnyThread());

	// We have a VT we'd like to prefetch, add or update a request in our request map.
	// If the texture was already requested by another component, fetch the highest required resolution only.
	float& CurrentMaxPixel = InOutVTRequests.FindOrAdd(InMaterialInterface);
	CurrentMaxPixel = FMath::Max(CurrentMaxPixel, PixelSize);
}

static void PrepareNaniteRequests(TMap<Nanite::FResources*, int32>& InOutNaniteRequests, UStaticMesh* InStaticMesh, int32 InNumFramesUntilRender)
{
	check(InStaticMesh);

	if (InStaticMesh->HasValidNaniteData())
	{
		int32& NumFramesUntilRender = InOutNaniteRequests.FindOrAdd(InStaticMesh->GetRenderData()->NaniteResourcesPtr.Get());
		NumFramesUntilRender = FMath::Max(InNumFramesUntilRender, 1);
	}
}

UWorldPartitionHLODRuntimeSubsystem::FWorldPartitionHLODWarmupState& UWorldPartitionHLODRuntimeSubsystem::AddHLODObjectToWarmup(IWorldPartitionHLODObject* InHLODObject)
{
	UWorldPartitionHLODRuntimeSubsystem::FWorldPartitionHLODWarmupState& WarmupState = HLODObjectsToWarmup.FindOrAdd(InHLODObject);

	// HLOD objects that are warming up can sometime linger in the HLODObjectsToWarmup array in the case where the level is actually never added to the world.
	// To prevent accessing a dangling pointer when that level is unloaded, we hook ourselves to the OnCleanupLevel delegate.
	ULevel* HLODLevel = InHLODObject->GetHLODLevel();
	check(HLODLevel);
	FHLODLevelState& LevelState = HLODLevelWarmupStates.FindOrAdd(HLODLevel);
	if (!LevelState.OnCleanupLevelDelegateHandle.IsValid())
	{
		check(LevelState.HLODObjectsWarmingUp.IsEmpty());
		LevelState.OnCleanupLevelDelegateHandle = HLODLevel->OnCleanupLevel.AddUObject(this, &UWorldPartitionHLODRuntimeSubsystem::OnCleanupLevel, HLODLevel);
	}
	LevelState.HLODObjectsWarmingUp.Add(InHLODObject);

	return WarmupState;
}

void UWorldPartitionHLODRuntimeSubsystem::RemoveHLODObjectFromWarmup(IWorldPartitionHLODObject* InHLODObject)
{
	const FSetElementId ElementId = HLODObjectsToWarmup.FindId(InHLODObject);
	if (ElementId.IsValidId())
	{
		FWorldPartitionHLODWarmupState& WarmupState = HLODObjectsToWarmup.Get(ElementId).Value;

		ULevel* HLODLevel = InHLODObject->GetHLODLevel();

		const FSetElementId LevelElementId = HLODLevelWarmupStates.FindId(HLODLevel);
		check(LevelElementId.IsValidId());

		TPair<ULevel*, FHLODLevelState>& LevelStatePair = HLODLevelWarmupStates.Get(LevelElementId);
		FHLODLevelState& LevelState = LevelStatePair.Value;

		const int32 NumRemoved = LevelState.HLODObjectsWarmingUp.Remove(InHLODObject);
		check(NumRemoved == 1);

		if (LevelState.HLODObjectsWarmingUp.IsEmpty())
		{
			check(LevelState.OnCleanupLevelDelegateHandle.IsValid());
			HLODLevel->OnCleanupLevel.Remove(LevelState.OnCleanupLevelDelegateHandle);
			HLODLevelWarmupStates.Remove(LevelElementId);
		}

		HLODObjectsToWarmup.Remove(ElementId);
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnCleanupLevel(ULevel* InLevel)
{
	FHLODLevelState LevelState = HLODLevelWarmupStates.FindAndRemoveChecked(InLevel);

	for (IWorldPartitionHLODObject* HLODObject : LevelState.HLODObjectsWarmingUp)
	{
		HLODObjectsToWarmup.FindAndRemoveChecked(HLODObject);
	}

	InLevel->OnCleanupLevel.Remove(LevelState.OnCleanupLevelDelegateHandle);
}

bool UWorldPartitionHLODRuntimeSubsystem::PrepareToWarmup(const UWorldPartitionRuntimeCell* InCell, IWorldPartitionHLODObject* InHLODObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::PrepareToWarmup)

	bool bHLODObjectNeedsWarmUp = false;

	if (InHLODObject->DoesRequireWarmup())
	{
		FWorldPartitionHLODWarmupState& WarmupState = AddHLODObjectToWarmup(InHLODObject);

		// Trigger warmup for CVarHLODWarmupNumFrames frames on the first request, or if a warmup wasn't requested in the last frame
		const bool bResetWarmup =  WarmupState.WarmupLastRequestedFrame == INDEX_NONE ||
								  (WarmupState.WarmupLastRequestedFrame + 1) < GFrameNumber;
				
		if (bResetWarmup)
		{
			WarmupState.WarmupCallsUntilReady = CVarHLODWarmupNumFrames.GetValueOnGameThread();
			WarmupState.WarmupBounds = InCell->GetContentBounds();

			// If we're dealing with an instanced world partition, take the instance transform into account
			const UWorldPartition* WorldPartition = InCell->GetOuterWorld()->GetWorldPartition();
			if (ensure(WorldPartition) && !WorldPartition->IsMainWorldPartition())
			{
				WarmupState.WarmupBounds = WarmupState.WarmupBounds.TransformBy(WorldPartition->GetInstanceTransform());
			}			
		}
		else if (WarmupState.WarmupCallsUntilReady != 0)
		{	
			// Progress toward warmup readiness
			WarmupState.WarmupCallsUntilReady--;
		}
		
		bHLODObjectNeedsWarmUp = WarmupState.WarmupCallsUntilReady != 0;
		WarmupState.WarmupLastRequestedFrame = GFrameNumber;
	}

	return bHLODObjectNeedsWarmUp;
}

void UWorldPartitionHLODRuntimeSubsystem::OnCVarsChanged()
{
	bCachedShouldPerformWarmup = ShouldPerformWarmup();
}

bool UWorldPartitionHLODRuntimeSubsystem::ShouldPerformWarmup() const
{
	// If warmup is disabled by cvar/platform setting, no warmup is needed
	if (!FHLODRuntimeSubsystem::bWarmupEnabled)
	{
		return false;
	}

	// If we're running a dedicated server, no warmup needed
	const bool bIsDedicatedServer = GetWorld()->GetNetMode() == NM_DedicatedServer;
	if (bIsDedicatedServer)
	{
		return false;
	}

	return true;
}

bool UWorldPartitionHLODRuntimeSubsystem::ShouldPerformWarmupForCell(const UWorldPartitionRuntimeCell* InCell) const
{
	if (!bCachedShouldPerformWarmup)
	{
		return false;
	}

	const UWorld* World = GetWorld();

	// Blocking loading shouldn't trigger warmup
	const bool bIsInBlockingLoading = World->GetIsInBlockTillLevelStreamingCompleted();
	if (bIsInBlockingLoading)
	{
		return false;
	}

	// If processing for this cell in Add/RemoveFromWorld() has already started, it's too late for warmup
	const ULevel* CellLevel = InCell->GetLevel();
	const bool bCurrentlyProcessingLevel = World->IsLevelMakingVisible(CellLevel) || World->IsLevelMakingInvisible(CellLevel);
	if (bCurrentlyProcessingLevel)
	{
		return false;
	}

	return true;
}

bool UWorldPartitionHLODRuntimeSubsystem::CanMakeVisible(const UWorldPartitionRuntimeCell* InCell)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::CanMakeVisible)

	if (!ShouldPerformWarmupForCell(InCell))
	{
		return true;
	}

	bool bCanMakeVisible = true;

	// Prevent cells containing HLODs objects from being made visible until warmup has been performed
	if (InCell->GetIsHLOD() && InCell->GetLevel())
	{
		ForEachHLODObjectInCell(InCell, [&bCanMakeVisible, InCell, this](IWorldPartitionHLODObject* HLODObject)
		{
			const bool bHLODObjectNeedsWarmup = PrepareToWarmup(InCell, HLODObject);
			bCanMakeVisible &= !bHLODObjectNeedsWarmup;
		});
	}

	return bCanMakeVisible;
}

bool UWorldPartitionHLODRuntimeSubsystem::CanMakeInvisible(const UWorldPartitionRuntimeCell* InCell)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::CanMakeInvisible)

	if (!ShouldPerformWarmupForCell(InCell))
	{
		return true;
	}

	bool bCanMakeInvisible = true;

	// Prevent cells from being hidden until their matching HLOD objects have been warmed up
	if (FCellData* CellData = GetCellData(InCell))
	{
		for (IWorldPartitionHLODObject* HLODObject : CellData->LoadedHLODs)
		{
			const bool bHLODObjectNeedsWarmup = PrepareToWarmup(InCell, HLODObject);
			bCanMakeInvisible &= !bHLODObjectNeedsWarmup;
		}
	}

	return bCanMakeInvisible;
}

FWorldPartitionHLODForEachHLODObjectInCellEvent& UWorldPartitionHLODRuntimeSubsystem::GetForEachHLODObjectInCellEvent()
{
	return ForEachHLODObjectInCellEvent;
}

void UWorldPartitionHLODRuntimeSubsystem::ForEachHLODObjectInCell(const UWorldPartitionRuntimeCell* InCell, TFunction<void(IWorldPartitionHLODObject*)> InFunc)
{
	if (InCell->GetIsHLOD() && InCell->GetLevel())
	{
		for (AActor* Actor : InCell->GetLevel()->Actors)
		{
			if (AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(Actor))
			{
				InFunc(HLODActor);
			}
		}

		ForEachHLODObjectInCellEvent.Broadcast(InCell, InFunc);
	}
}

static float IsInView(const FVector& BoundsOrigin, const FVector& BoundsExtent, const FSceneViewFamily& InViewFamily, bool bComputeScreenSize, float& MaxScreenSizePixels)
{
	MaxScreenSizePixels = 0;

	// Estimate the highest screen pixel size of this component in the provided views
	for (const FSceneView* View : InViewFamily.Views)
	{
		// Make sure the HLOD Object we're about to show is actually in the frustum
		if (View->ViewFrustum.IntersectBox(BoundsOrigin, BoundsExtent))
		{
			if (bComputeScreenSize)
			{
				float ScreenDiameter = ComputeBoundsScreenSize(BoundsOrigin, BoundsExtent.Size(), *View);
				float ScreenSizePixels = ScreenDiameter * View->ViewMatrices.GetScreenScale() * 2.0f;

				MaxScreenSizePixels = FMath::Max(MaxScreenSizePixels, ScreenSizePixels);
			}
			else
			{
				return true;
			}
		}
	}

	return MaxScreenSizePixels > 0;
}

static void MakeHLODRenderResourcesResident(TMap<UMaterialInterface*, float>& VTRequests, TMap<Nanite::FResources*, int32>& NaniteRequests, const FSceneViewFamily& InViewFamily)
{
	if (!VTRequests.IsEmpty() || !NaniteRequests.IsEmpty())
	{
		ENQUEUE_RENDER_COMMAND(MakeHLODRenderResourcesResident)(
			[VTRequests = MoveTemp(VTRequests), NaniteRequests = MoveTemp(NaniteRequests), FeatureLevel = InViewFamily.GetFeatureLevel()](FRHICommandListImmediate& RHICmdList)
		{
			for (const TPair<UMaterialInterface*, float>& VTRequest : VTRequests)
			{
				UMaterialInterface* Material = VTRequest.Key;
				FMaterialRenderProxy* MaterialRenderProxy = Material->GetRenderProxy();

				GetRendererModule().RequestVirtualTextureTiles(MaterialRenderProxy, FVector2D(VTRequest.Value, VTRequest.Value), FeatureLevel);
			}

			for (const TPair<Nanite::FResources*, int32> NaniteRequest : NaniteRequests)
			{
				const Nanite::FResources* NaniteResource = NaniteRequest.Key;
				const int32 NumFramesUntilRender = NaniteRequest.Value;

				GetRendererModule().PrefetchNaniteResource(NaniteResource, NumFramesUntilRender);
			}
		});
	}
}

void UWorldPartitionHLODRuntimeSubsystem::OnBeginRenderViews(const FSceneViewFamily& InViewFamily)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODRuntimeSubsystem::OnBeginRenderViews)

	TMap<UMaterialInterface*, float> VTRequests;
	TMap<Nanite::FResources*, int32> NaniteRequests;

	const bool bWarmupNanite = CVarHLODWarmupNanite.GetValueOnGameThread() != 0;
	const bool bWarmupVT = CVarHLODWarmupVT.GetValueOnGameThread() != 0;

	for (FHLODWarmupStateMap::TIterator HLODObjectWarmupStateIt(HLODObjectsToWarmup); HLODObjectWarmupStateIt; ++HLODObjectWarmupStateIt)
	{
		const IWorldPartitionHLODObject* HLODObject = HLODObjectWarmupStateIt.Key();
		FWorldPartitionHLODWarmupState& HLODWarmupState = HLODObjectWarmupStateIt.Value();

		// Retrieve this component's bound - we must support getting the bounds before the component is even registered.
		FVector BoundsOrigin;
		FVector BoundsExtent;
		HLODWarmupState.WarmupBounds.GetCenterAndExtents(BoundsOrigin, BoundsExtent);

		float ScreenSizePixels = 0;
		if (IsInView(BoundsOrigin, BoundsExtent, InViewFamily, bWarmupVT, ScreenSizePixels))
		{
			for (UObject* AssetToWarmup : HLODObject->GetAssetsToWarmup())
			{
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetToWarmup))
				{
				    if (bWarmupVT)
				    {
					    PrepareVTRequests(VTRequests, MaterialInterface, ScreenSizePixels);
				    }
				}

				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetToWarmup))
				{
				    if (bWarmupNanite)
				    {
					    if (HLODWarmupState.WarmupCallsUntilReady == CVarHLODWarmupNumFrames.GetValueOnGameThread())
					    {
						    // Send a nanite request to prepare for visibility in CVarHLODWarmupNumFrames frames
						    PrepareNaniteRequests(NaniteRequests, StaticMesh, CVarHLODWarmupNumFrames.GetValueOnGameThread());
					    }
					    else if (HLODWarmupState.WarmupCallsUntilReady == 0)
					    {
						    // We expect HLOD to be visible at any moment (likely waiting for server visibility ack)
						    PrepareNaniteRequests(NaniteRequests, StaticMesh, 1);
					    }
				    }
				}
			}

#if ENABLE_DRAW_DEBUG
			if (CVarHLODWarmupDebugDraw.GetValueOnAnyThread())
			{
				DrawDebugBox(HLODObject->GetHLODLevel()->GetWorld(), BoundsOrigin, BoundsExtent, HLODWarmupState.WarmupCallsUntilReady > 0 ? FColor::Yellow : FColor::Green, /*bPersistentLine*/ false, /*Lifetime*/ 0.5f);
			}
#endif
		}
	}

	MakeHLODRenderResourcesResident(VTRequests, NaniteRequests, InViewFamily);
}

#undef LOCTEXT_NAMESPACE

