// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODSourceActors.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"

#if WITH_EDITOR
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODSourceActors)

UWorldPartitionHLODSourceActors::UWorldPartitionHLODSourceActors(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

void UWorldPartitionHLODSourceActors::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	if (HLODLayer)
	{
		HLODLayer->ComputeHLODHash(InHashBuilder);
	}
}

void UWorldPartitionHLODSourceActors::SetHLODLayer(const UHLODLayer* InHLODLayer)
{
	HLODLayer = InHLODLayer;
}

const UHLODLayer* UWorldPartitionHLODSourceActors::GetHLODLayer() const
{
	return HLODLayer;
}

#endif // #if WITH_EDITOR
