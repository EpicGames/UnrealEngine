// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODSourceActorsFromCell.h"

#if WITH_EDITOR
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "Serialization/ArchiveCrc32.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"

#include "Editor/UnrealEd/Public/EditorLevelUtils.h" // MoveActorsToLevel
#include "EngineUtils.h"
#include "UObject/Linker.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"

#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODSourceActorsFromCell)

UWorldPartitionHLODSourceActorsFromCell::UWorldPartitionHLODSourceActorsFromCell(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

bool UWorldPartitionHLODSourceActorsFromCell::LoadSourceActors(bool& bOutDirty, UWorld* TargetWorld) const
{
	UPackage::WaitForAsyncFileWrites();

	bOutDirty = false;
	DirtyDuplicateActorsToOriginalGuids.Reset();
	AWorldPartitionHLOD* HLODActor = CastChecked<AWorldPartitionHLOD>(GetOuter());
	UWorld* SourceWorld = HLODActor->GetWorld();

	FLinkerInstancingContext InstancingContext;
	InstancingContext.AddTag(ULevel::DontLoadExternalObjectsTag);
	InstancingContext.AddTag(ULevel::DontLoadExternalFoldersTag);

	// Don't do SoftObjectPath remapping for PersistentLevel actors because references can end up in different cells
	InstancingContext.SetSoftObjectPathRemappingEnabled(false);

	UPackage* SourcePackage = SourceWorld->GetPackage();
	UPackage* TargetPackage = TargetWorld->GetPackage();

	FName SourcePackageName = SourcePackage->GetFName();
	FName TargetPackageName = TargetPackage->GetFName();
	InstancingContext.AddPackageMapping(SourcePackageName, TargetPackageName);

	for (const FWorldPartitionRuntimeCellObjectMapping& CellObjectMapping : Actors)
	{
		if (ContentBundlePaths::IsAContentBundlePath(CellObjectMapping.ContainerPackage.ToString()) ||
			FExternalDataLayerHelper::IsExternalDataLayerPath(CellObjectMapping.ContainerPackage.ToString()))
		{
			check(CellObjectMapping.ContainerPackage != CellObjectMapping.WorldPackage);
			bool bIsContainerPackageAlreadyRemapped = InstancingContext.RemapPackage(CellObjectMapping.ContainerPackage) != CellObjectMapping.ContainerPackage;
			if (!bIsContainerPackageAlreadyRemapped)
			{
				InstancingContext.AddPackageMapping(CellObjectMapping.ContainerPackage, TargetPackageName);
			}
		}
	}

	TArray<FWorldPartitionRuntimeCellObjectMapping> ActorsCopy = Actors;

	// Find dirty source HLOD actors and duplicate them instead of loading them from disk
	TArray<AActor*> DirtySourceActors;
	for (TActorIterator<AWorldPartitionHLOD> It(SourceWorld); It; ++It)
	{
		AActor* SourceActor = *It;
		if (UPackage* SourceActorPackage = SourceActor->GetPackage())
		{
			if (SourceActorPackage->IsDirty())
			{
				int32 Index = ActorsCopy.IndexOfByPredicate([UnsavedActorGuid = SourceActor->GetActorInstanceGuid()](FWorldPartitionRuntimeCellObjectMapping Actor)
				{
					return UnsavedActorGuid == Actor.ActorInstanceGuid;
				});

				if (Index != INDEX_NONE)
				{
					DirtySourceActors.Add(SourceActor);
					ActorsCopy.RemoveAtSwap(Index);
				}
			}
		}
	}

	if (DirtySourceActors.Num())
	{
		UActorContainer* ActorContainer = NewObject<UActorContainer>(SourceWorld->PersistentLevel);

		for (AActor* DirtySourceActor : DirtySourceActors)
		{
			DirtySourceActor->UObject::Rename(nullptr, ActorContainer, REN_DoNotDirty);
			ActorContainer->Actors.Add(DirtySourceActor->GetFName(), DirtySourceActor);
		}

		FObjectDuplicationParameters Parameters(ActorContainer, TargetWorld->PersistentLevel);
		Parameters.DestClass = ActorContainer->GetClass();
		Parameters.FlagMask = RF_AllFlags & ~(RF_MarkAsRootSet | RF_MarkAsNative | RF_HasExternalPackage);
		Parameters.InternalFlagMask = EInternalObjectFlags_AllFlags;
		Parameters.DuplicationSeed.Add(SourceWorld->PersistentLevel, TargetWorld->PersistentLevel);

		UActorContainer* ActorContainerDup = (UActorContainer*)StaticDuplicateObjectEx(Parameters);
		check(ActorContainerDup);

		for (AActor* OriginalActor : DirtySourceActors)
		{
			if (TObjectPtr<AActor>* DuplicateActorPtr = ActorContainerDup->Actors.Find(OriginalActor->GetFName()))
			{
				DirtyDuplicateActorsToOriginalGuids.Add(*DuplicateActorPtr, OriginalActor->GetActorInstanceGuid());
				if (AWorldPartitionHLOD* DuplicatedHLODActor = Cast<AWorldPartitionHLOD>(DuplicateActorPtr->Get()))
				{
					DuplicatedHLODActor->SetVisibility(true);
				}
			}
		}
		
		for (TPair<FName, AActor*> Pair : ActorContainerDup->Actors)
		{
			AActor* DuplicatedActor = Pair.Value;
			DuplicatedActor->Rename(nullptr, TargetWorld->PersistentLevel, REN_DoNotDirty);
		}

		for (AActor* DirtySourceActor : DirtySourceActors)
		{
			DirtySourceActor->UObject::Rename(nullptr, SourceWorld->PersistentLevel, REN_DoNotDirty);
		}

		ActorContainer->MarkAsGarbage();
		ActorContainerDup->MarkAsGarbage();
	}

	FWorldPartitionLevelHelper::FPackageReferencer PackageReferencer;
	FWorldPartitionLevelHelper::FLoadActorsParams Params = FWorldPartitionLevelHelper::FLoadActorsParams()
		.SetOuterWorld(TargetWorld)
		.SetDestLevel(nullptr)
		.SetActorPackages(ActorsCopy)
		.SetPackageReferencer(&PackageReferencer)
		.SetCompletionCallback([&bOutDirty](bool bSucceeded) { bOutDirty = !bSucceeded; })
		.SetLoadAsync(false)
		.SetInstancingContext(MoveTemp(InstancingContext))
		.SetSilenceLoadFailures(true);

	if (FWorldPartitionLevelHelper::LoadActors(MoveTemp(Params)))
	{
		TSet<UPackage*> LevelPackagesMadeTransient;
		TArray<UObject*> ActorsToResetLoaders;
		ActorsToResetLoaders.Reserve(ActorsCopy.Num());
		for (const FWorldPartitionRuntimeCellObjectMapping& ActorMapping : ActorsCopy)
		{
			AActor* Actor = FindObject<AActor>(nullptr, *ActorMapping.LoadedPath.ToString());
			if (Actor)
			{
				ActorsToResetLoaders.Add(Actor);

				// Mark levels' packages as transient, so that they don't get dirtied during MoveExternalActorsToLevel
				if (ULevel* ActorLevel = Actor->GetLevel())
				{
					if (UPackage* ActorLevelPackage = ActorLevel->GetPackage())
					{
						if (IsValid(ActorLevelPackage) && !ActorLevelPackage->HasAnyFlags(RF_Transient))
						{
							ActorLevelPackage->SetFlags(RF_Transient);
							LevelPackagesMadeTransient.Add(ActorLevelPackage);
						}
					}
				}
			}
		}
		ResetLoaders(ActorsToResetLoaders);
	
		TArray<UPackage*> ModifiedPackages;
		FWorldPartitionLevelHelper::MoveExternalActorsToLevel(ActorsCopy, TargetWorld->PersistentLevel, ModifiedPackages);

		for (UPackage* Package : LevelPackagesMadeTransient)
		{
			Package->ClearFlags(RF_Transient);
		}

		return true;
	}

	return false;
}

uint32 UWorldPartitionHLODSourceActorsFromCell::GetSourceActorsHash(const TArray<FWorldPartitionRuntimeCellObjectMapping>& InSourceActors)
{
	TArray<FWorldPartitionRuntimeCellObjectMapping>& MutableSourceActors(const_cast<TArray<FWorldPartitionRuntimeCellObjectMapping>&>(InSourceActors));

	FArchiveCrc32 Ar;
	for (FWorldPartitionRuntimeCellObjectMapping& Mapping : MutableSourceActors)
	{
		Ar << Mapping;
	}
	
	return Ar.GetCrc();
}

bool UWorldPartitionHLODSourceActorsFromCell::IsHLODRelevant(AActor* InActor) const
{
	check(InActor);
	const FGuid* OverrideGuid = DirtyDuplicateActorsToOriginalGuids.Find(InActor);
	const FGuid& ActorInstanceGuid = OverrideGuid ? *OverrideGuid : InActor->GetActorInstanceGuid();
	return !!Actors.FindByPredicate([&ActorInstanceGuid](const FWorldPartitionRuntimeCellObjectMapping& Actor)
	{
		return Actor.ActorInstanceGuid == ActorInstanceGuid;
	});
}

void UWorldPartitionHLODSourceActorsFromCell::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	Super::ComputeHLODHash(InHashBuilder);

	// Source Actors
	uint32 SourceActorsHash = GetSourceActorsHash(Actors);
	InHashBuilder.HashField(SourceActorsHash, TEXT("SourceActorsHash"));
}

void UWorldPartitionHLODSourceActorsFromCell::SetActors(const TArray<FWorldPartitionRuntimeCellObjectMapping>&& InSourceActors)
{
	Actors = InSourceActors;
}

const TArray<FWorldPartitionRuntimeCellObjectMapping>& UWorldPartitionHLODSourceActorsFromCell::GetActors() const
{
	return Actors;
}

#endif // #if WITH_EDITOR
