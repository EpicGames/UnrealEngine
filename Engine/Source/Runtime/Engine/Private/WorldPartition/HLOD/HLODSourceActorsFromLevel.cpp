// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODSourceActorsFromLevel.h"

#if WITH_EDITOR
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODHashBuilder.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/World.h"
#include "UObject/Linker.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODSourceActorsFromLevel)

UWorldPartitionHLODSourceActorsFromLevel::UWorldPartitionHLODSourceActorsFromLevel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

bool UWorldPartitionHLODSourceActorsFromLevel::LoadSourceActors(bool& bOutDirty, UWorld* TargetWorld) const
{
	bool bSuccess = false;
	ULevelStreamingDynamic* LevelStreaming = ULevelStreamingDynamic::LoadLevelInstanceBySoftObjectPtr(TargetWorld, SourceLevel, FTransform::Identity, bSuccess);
	TargetWorld->FlushLevelStreaming();

	ULevel* LoadedLevel = LevelStreaming ? LevelStreaming->GetLoadedLevel() : nullptr;
	if (LoadedLevel != nullptr)
	{
		ResetLoaders(LoadedLevel->GetPackage());
	}

	return bSuccess && LoadedLevel != nullptr;
}

void UWorldPartitionHLODSourceActorsFromLevel::ComputeHLODHash(FHLODHashBuilder& InHashBuilder) const
{
	Super::ComputeHLODHash(InHashBuilder);

	InHashBuilder.HashField(SourceLevel.GetUniqueID().GetLongPackageName(), GET_MEMBER_NAME_CHECKED(UWorldPartitionHLODSourceActorsFromLevel, SourceLevel));
}

bool UWorldPartitionHLODSourceActorsFromLevel::IsHLODRelevant(AActor* InActor) const
{
	check(InActor);
	return InActor->IsHLODRelevant();
}

void UWorldPartitionHLODSourceActorsFromLevel::SetSourceLevel(const UWorld* InSourceLevel)
{
	SourceLevel = const_cast<UWorld*>(InSourceLevel);
}

const TSoftObjectPtr<UWorld>& UWorldPartitionHLODSourceActorsFromLevel::GetSourceLevel() const
{
	return SourceLevel;
}

#endif // #if WITH_EDITOR
