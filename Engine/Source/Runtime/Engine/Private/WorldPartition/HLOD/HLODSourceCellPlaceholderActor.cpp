// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"

#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActorDesc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODSourceCellPlaceholderActor)

AWorldPartitionHLODSourceCellPlaceholder::AWorldPartitionHLODSourceCellPlaceholder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, HLODActorDescInstance(nullptr)
#endif
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
}

#if WITH_EDITOR
void AWorldPartitionHLODSourceCellPlaceholder::InitFrom(const FWorldPartitionActorDescInstance* InHLODActorDescInstance)
{
	HLODActorDescInstance = InHLODActorDescInstance;
	HLODActorGuid = HLODActorDescInstance->GetContainerInstance()->GetContainerID().GetActorGuid(HLODActorDescInstance->GetGuid());

	SetActorTransform(HLODActorDescInstance->GetActorTransform());
	SetRuntimeGrid(HLODActorDescInstance->GetRuntimeGrid());
	SetIsSpatiallyLoaded(HLODActorDescInstance->GetActorDesc()->GetIsSpatiallyLoadedRaw());
	bIsEditorOnlyActor = HLODActorDescInstance->GetActorIsEditorOnly();
}

const FGuid& AWorldPartitionHLODSourceCellPlaceholder::GetHLODActorGuid() const
{
	return HLODActorGuid;
}

void AWorldPartitionHLODSourceCellPlaceholder::SetPlaceholderType(EHLODSourceCellPlaceholderType InPlaceholderType)
{
	PlaceholderType = InPlaceholderType;
}

EHLODSourceCellPlaceholderType AWorldPartitionHLODSourceCellPlaceholder::GetPlaceholderType() const
{
	return PlaceholderType;
}

void AWorldPartitionHLODSourceCellPlaceholder::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	if (HLODActorDescInstance)
	{
		if (PlaceholderType == EHLODSourceCellPlaceholderType::StandaloneHLOD)
		{
			// Use instance bounds including container transform, because the Standalone HLOD placeholder is in the main container while the source is in a child container.
			OutRuntimeBounds = HLODActorDescInstance->GetRuntimeBounds();
			OutEditorBounds = HLODActorDescInstance->GetEditorBounds();
		}
		else
		{
			// Use desc bounds without container transform, because the Custom HLOD placeholder is registered in the same container as the source.
			OutRuntimeBounds = HLODActorDescInstance->GetActorDesc()->GetRuntimeBounds();
			OutEditorBounds = HLODActorDescInstance->GetActorDesc()->GetEditorBounds();
		}
	}
	else
	{
		OutRuntimeBounds = OutEditorBounds = FBox(EForceInit::ForceInit);
	}
}

TUniquePtr<FWorldPartitionActorDesc> AWorldPartitionHLODSourceCellPlaceholder::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FHLODSourceCellPlaceholderActorDesc());
}
#endif
