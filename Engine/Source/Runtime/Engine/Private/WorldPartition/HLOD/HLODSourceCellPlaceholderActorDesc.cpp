// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActorDesc.h"

#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"

#if WITH_EDITOR
void FHLODSourceCellPlaceholderActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	if (!bIsDefaultActorDesc)
	{
		const AWorldPartitionHLODSourceCellPlaceholder* PlaceholderActor = CastChecked<AWorldPartitionHLODSourceCellPlaceholder>(InActor);
		HLODActorGuid = PlaceholderActor->GetHLODActorGuid();
		PlaceholderType = PlaceholderActor->GetPlaceholderType();
	}
}

bool FHLODSourceCellPlaceholderActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FHLODSourceCellPlaceholderActorDesc* OtherDesc = static_cast<const FHLODSourceCellPlaceholderActorDesc*>(Other);
		return HLODActorGuid == OtherDesc->HLODActorGuid
			&& PlaceholderType == OtherDesc->PlaceholderType;
	}
	return false;
}

const FGuid& FHLODSourceCellPlaceholderActorDesc::GetHLODActorGuid() const
{
	return HLODActorGuid;
}

EHLODSourceCellPlaceholderType FHLODSourceCellPlaceholderActorDesc::GetPlaceholderType() const
{
	return PlaceholderType;
}

uint32 FHLODSourceCellPlaceholderActorDesc::GetSizeOf() const
{
	return sizeof(FHLODSourceCellPlaceholderActorDesc);
}

void FHLODSourceCellPlaceholderActorDesc::Serialize(FArchive& Ar)
{
	ensureMsgf(false, TEXT("FHLODSourceCellPlaceholderActorDesc is not expected to be serialized"));
}
#endif