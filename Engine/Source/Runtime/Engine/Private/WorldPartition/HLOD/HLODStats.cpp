// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODStats.h"
#include "UObject/NameTypes.h"


const FName FWorldPartitionHLODStats::InputActorCount(TEXT("InputActorCount"));
const FName FWorldPartitionHLODStats::InputTriangleCount(TEXT("InputTriangleCount"));
const FName FWorldPartitionHLODStats::InputVertexCount(TEXT("InputVertexCount"));

const FName FWorldPartitionHLODStats::MeshInstanceCount(TEXT("MeshInstanceCount"));
const FName FWorldPartitionHLODStats::MeshNaniteTriangleCount(TEXT("MeshNaniteTriangleCount"));
const FName FWorldPartitionHLODStats::MeshNaniteVertexCount(TEXT("MeshNaniteVertexCount"));
const FName FWorldPartitionHLODStats::MeshTriangleCount(TEXT("MeshTriangleCount"));
const FName FWorldPartitionHLODStats::MeshVertexCount(TEXT("MeshVertexCount"));
const FName FWorldPartitionHLODStats::MeshUVChannelCount(TEXT("MeshUVChannelCount"));

const FName FWorldPartitionHLODStats::MaterialBaseColorTextureSize(TEXT("MaterialBaseColorTextureSize"));
const FName FWorldPartitionHLODStats::MaterialNormalTextureSize(TEXT("MaterialNormalTextureSize"));
const FName FWorldPartitionHLODStats::MaterialEmissiveTextureSize(TEXT("MaterialEmissiveTextureSize"));
const FName FWorldPartitionHLODStats::MaterialMetallicTextureSize(TEXT("MaterialMetallicTextureSize"));
const FName FWorldPartitionHLODStats::MaterialRoughnessTextureSize(TEXT("MaterialRoughnessTextureSize"));
const FName FWorldPartitionHLODStats::MaterialSpecularTextureSize(TEXT("MaterialSpecularTextureSize"));

const FName FWorldPartitionHLODStats::MemoryMeshResourceSizeBytes(TEXT("MemoryMeshResourceSizeBytes"));
const FName FWorldPartitionHLODStats::MemoryTexturesResourceSizeBytes(TEXT("MemoryTexturesResourceSizeBytes"));
const FName FWorldPartitionHLODStats::MemoryDiskSizeBytes(TEXT("MemoryDiskSizeBytes"));

const FName FWorldPartitionHLODStats::BuildTimeLoadMilliseconds(TEXT("BuildTimeLoadMilliseconds"));
const FName FWorldPartitionHLODStats::BuildTimeBuildMilliseconds(TEXT("BuildTimeBuildMilliseconds"));
const FName FWorldPartitionHLODStats::BuildTimeTotalMilliseconds(TEXT("BuildTimeTotalMilliseconds"));
