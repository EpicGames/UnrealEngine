// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODUtils.h"

#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/BodySetup.h"

namespace UE::Private::HLOD
{
	void RemoveCollisionData(UStaticMesh* InStaticMesh)
	{
		if (UBodySetup* BodySetup = InStaticMesh->GetBodySetup())
		{
			// To ensure a deterministic cook, save the current GUID and restore it below
			FGuid PreviousBodySetupGuid = BodySetup->BodySetupGuid;
			BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			BodySetup->bNeverNeedsCookedCollisionData = true;
			BodySetup->bHasCookedCollisionData = false;
			BodySetup->InvalidatePhysicsData();
			BodySetup->BodySetupGuid = PreviousBodySetupGuid;
		}
	}
}