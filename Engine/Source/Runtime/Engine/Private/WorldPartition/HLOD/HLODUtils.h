// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UStaticMesh;

namespace UE::Private::HLOD
{
	void RemoveCollisionData(UStaticMesh* InStaticMesh);
}