// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/MaterialParameterCollectionHLODModifier.h"
#include "Engine/World.h"
#include "Materials/MaterialParameterCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialParameterCollectionHLODModifier)


UMaterialParameterCollectionHLODModifier::UMaterialParameterCollectionHLODModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}



bool UMaterialParameterCollectionHLODModifier::CanModifyHLOD(TSubclassOf<UHLODBuilder> InHLODBuilderClass) const
{
	return true;
}

void UMaterialParameterCollectionHLODModifier::BeginHLODBuild(const FHLODBuildContext& InHLODBuildContext)
{
	//Cache off the build context, all of these seems to be executed in one frame so it should be fine to dereference in EndHLODBuild
	HLODBuildContext = &InHLODBuildContext;
	if (MPC != nullptr)
	{
		if (UMaterialParameterCollectionInstance* Instance = HLODBuildContext->BuildWorld->GetParameterCollectionInstance(MPC))
		{
			ApplyScalarParameterOverrides(Instance);
		}
	}
}

void UMaterialParameterCollectionHLODModifier::EndHLODBuild(TArray<UActorComponent*>& InOutComponents)
{
	if (MPC != nullptr)
	{
		if (UMaterialParameterCollectionInstance* Instance = HLODBuildContext->BuildWorld->GetParameterCollectionInstance(MPC))
		{
			RestoreScalarParameterValues(Instance);
		}
	}
}

void UMaterialParameterCollectionHLODModifier::ApplyScalarParameterOverrides(UMaterialParameterCollectionInstance* Instance)
{
	for (int32 ParameterIndex = 0; ParameterIndex < ScalarParameters.Num(); ParameterIndex++)
	{
		const FHLODModifierScalarParameter& Parameter = ScalarParameters[ParameterIndex];
		float CurScalarVal;
		Instance->GetScalarParameterValue(Parameter.ParameterName, CurScalarVal);
		Instance->SetScalarParameterValue(Parameter.ParameterName, Parameter.OverrideValue);
		CachedScalarParameterValues.Add(CurScalarVal);
	}
}

void UMaterialParameterCollectionHLODModifier::RestoreScalarParameterValues(UMaterialParameterCollectionInstance* Instance)
{
	// Revert the overrides in the reverse order, this supports a stack style parameter overrrides, so we can have the same param multiple times in the stack, so only the last value is applied but the original value is reverted to
	for (int32 ParameterIndex = ScalarParameters.Num() - 1; ParameterIndex >= 0; ParameterIndex--)
	{
		const FHLODModifierScalarParameter& Parameter = ScalarParameters[ParameterIndex];
		Instance->SetScalarParameterValue(Parameter.ParameterName, CachedScalarParameterValues[ParameterIndex]);
	}
}
