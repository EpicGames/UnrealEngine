// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/StandaloneHLODActor.h"

#include "Engine/World.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StandaloneHLODActor)

AWorldPartitionStandaloneHLOD::AWorldPartitionStandaloneHLOD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LevelInstanceActorGuid(this)
	, LevelInstanceActorImpl(this)
{
	RootComponent = CreateDefaultSubobject<ULevelInstanceComponent>(TEXT("Root"));
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	bDefaultOutlinerExpansionState = false;
#endif
}

const FLevelInstanceID& AWorldPartitionStandaloneHLOD::GetLevelInstanceID() const
{
	return LevelInstanceActorImpl.GetLevelInstanceID();
}

bool AWorldPartitionStandaloneHLOD::HasValidLevelInstanceID() const
{
	return LevelInstanceActorImpl.HasValidLevelInstanceID();
}

const FGuid& AWorldPartitionStandaloneHLOD::GetLevelInstanceGuid() const
{
	return LevelInstanceActorGuid.GetGuid();
}

const TSoftObjectPtr<UWorld>& AWorldPartitionStandaloneHLOD::GetWorldAsset() const
{
	return WorldAsset;
}

bool AWorldPartitionStandaloneHLOD::IsLoadingEnabled() const
{
	return LevelInstanceActorImpl.IsLoadingEnabled();
}

bool AWorldPartitionStandaloneHLOD::SetWorldAsset(TSoftObjectPtr<UWorld> InWorldAsset)
{
	WorldAsset = InWorldAsset;
	return true;
}

#if WITH_EDITOR
ULevelInstanceComponent* AWorldPartitionStandaloneHLOD::GetLevelInstanceComponent() const
{
	return Cast<ULevelInstanceComponent>(RootComponent);
}

ELevelInstanceRuntimeBehavior AWorldPartitionStandaloneHLOD::GetDesiredRuntimeBehavior() const
{
	return ELevelInstanceRuntimeBehavior::Partitioned;
}

ELevelInstanceRuntimeBehavior AWorldPartitionStandaloneHLOD::GetDefaultRuntimeBehavior() const
{
	return ELevelInstanceRuntimeBehavior::Partitioned;
}

bool AWorldPartitionStandaloneHLOD::CanEditChange(const FProperty* InProperty) const
{
	return false;
}

bool AWorldPartitionStandaloneHLOD::CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const
{
	return false;
}
#endif // WITH_EDITOR

void AWorldPartitionStandaloneHLOD::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (GetLocalRole() == ENetRole::ROLE_Authority && GetWorld()->IsGameWorld())
	{
#if !WITH_EDITOR
		// If the level instance was spawned, not loaded
		LevelInstanceActorGuid.AssignIfInvalid();
#endif
		LevelInstanceSpawnGuid = LevelInstanceActorGuid.GetGuid();
	}

	if (LevelInstanceActorGuid.IsValid())
	{
		LevelInstanceActorImpl.RegisterLevelInstance();
	}
}

void AWorldPartitionStandaloneHLOD::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	LevelInstanceActorImpl.UnregisterLevelInstance();
}

#if WITH_EDITOR
TUniquePtr<FWorldPartitionActorDesc> AWorldPartitionStandaloneHLOD::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FLevelInstanceActorDesc());
}
#endif
