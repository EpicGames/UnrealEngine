// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/StandaloneHLODSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StandaloneHLODSubsystem)

#if WITH_EDITOR

#include "FileHelpers.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "Settings/EditorExperimentalSettings.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/HLOD/StandaloneHLODActor.h"
#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogStandaloneHLODSubsystem, Log, All);

void UWorldPartitionStandaloneHLODSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	bRefreshCachedHLODSetups = true;

	Collection.InitializeDependency<UWorldPartitionSubsystem>();

	Super::Initialize(Collection);

	GetWorld()->OnWorldPartitionInitialized().AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnWorldPartitionInitialized);
	GetWorld()->OnWorldPartitionUninitialized().AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnWorldPartitionUninitialized);

	GEngine->OnLevelActorAdded().AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnActorChanged);
	GEngine->OnActorMoved().AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnActorChanged);
	GEngine->OnLevelActorDeleted().AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnActorDeleted);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnObjectPropertyChanged);
	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnLevelAddedToWorld);
}

void UWorldPartitionStandaloneHLODSubsystem::Deinitialize()
{
	Super::Deinitialize();

	GetWorld()->OnWorldPartitionInitialized().RemoveAll(this);
	GetWorld()->OnWorldPartitionUninitialized().RemoveAll(this);

	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnActorMoved().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

bool UWorldPartitionStandaloneHLODSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return WorldType == EWorldType::Editor || WorldType == EWorldType::Inactive;
}

bool UWorldPartitionStandaloneHLODSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return Super::ShouldCreateSubsystem(Outer) && GetDefault<UEditorExperimentalSettings>()->bEnableStandaloneHLOD;
}

void UWorldPartitionStandaloneHLODSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	InWorldPartition->OnActorDescContainerInstanceRegistered.AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnActorDescContainerInstanceRegistered);
	InWorldPartition->OnActorDescContainerInstanceUnregistered.AddUObject(this, &UWorldPartitionStandaloneHLODSubsystem::OnActorDescContainerInstanceUnregistered);

	// Since this is created upon WP init, we missed the first broadcasts for existing container instances. Register them manually.
	InWorldPartition->ForEachActorDescContainerInstance([this](UActorDescContainerInstance* InContainerInstance)
	{
		OnActorDescContainerInstanceRegistered(InContainerInstance);
	});

	if (InWorldPartition->HasStandaloneHLOD())
	{
		FStandaloneHLODActorParams Params;

		Params.Guid = FGuid();
		Params.Transform = FTransform::Identity;
		Params.WorldPackageName = InWorldPartition->GetPackage()->GetPathName();
		Params.ActorLabel = InWorldPartition->GetWorld()->GetName();

		UpdateStandaloneHLODActors(Params);
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	InWorldPartition->OnActorDescContainerInstanceRegistered.RemoveAll(this);
	InWorldPartition->OnActorDescContainerInstanceUnregistered.RemoveAll(this);
}

void UWorldPartitionStandaloneHLODSubsystem::OnActorChanged(AActor* InActor)
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor))
	{
		// If the changed actor is a SubWP, update its Standalone HLOD actors
		if (LevelInstance->GetDesiredRuntimeBehavior() == ELevelInstanceRuntimeBehavior::LevelStreaming)
		{
			FStandaloneHLODActorParams Params;

			Params.Guid = InActor->GetActorGuid();
			Params.Transform = InActor->GetActorTransform();
			Params.WorldPackageName = LevelInstance->GetWorldAssetPackage();
			Params.ActorLabel = InActor->GetActorLabel();

			UpdateStandaloneHLODActors(Params);
		}

		// Propagate the actor change to currently loaded level instances
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
		LevelInstanceSubsystem->ForEachLevelInstanceChild(LevelInstance, /*bRecursive=*/false, [this](ILevelInstanceInterface* ChildLevelInstance)
		{
			OnActorChanged(CastChecked<AActor>(ChildLevelInstance));
			return true;
		});

		// Propagate the actor change to currently unloaded level instances
		if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
		{
			if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(InActor->GetActorGuid()))
			{
				if (FLevelInstanceActorDesc* LevelInstanceActorDesc = (FLevelInstanceActorDesc*)ActorDescInstance->GetActorDesc())
				{
					UpdateStandaloneHLODActorsRecursive(*LevelInstanceActorDesc, InActor->GetActorTransform(), /*bChildrenOnly=*/true);
				}
			}
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnActorDeleted(AActor* InActor)
{
	if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActor))
	{
		// Delete LI Standalone HLOD actors
		DeleteStandaloneHLODActors(InActor->GetActorGuid());

		// Propagate actor deletion to currently loaded level instances
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
		LevelInstanceSubsystem->ForEachLevelInstanceChild(LevelInstance, /*bRecursive=*/false, [this](ILevelInstanceInterface* ChildLevelInstance)
		{
			OnActorDeleted(CastChecked<AActor>(ChildLevelInstance));
			return true;
		});

		// Propagate actor deletion to currently unloaded level instances
		if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
		{
			if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(InActor->GetActorGuid()))
			{
				if (FLevelInstanceActorDesc* LevelInstanceActorDesc = (FLevelInstanceActorDesc*)ActorDescInstance->GetActorDesc())
				{
					DeleteStandaloneHLODActorsRecursive(*LevelInstanceActorDesc);
				}
			}
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InObject))
	{
		if (AWorldPartitionStandaloneHLOD* StandaloneHLOD = Cast<AWorldPartitionStandaloneHLOD>(InObject))
		{
			return;
		}

		if (AActor* Actor = Cast<AActor>(InObject))
		{
			OnActorDeleted(Actor);
			OnActorChanged(Actor);
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnActorDescContainerInstanceRegistered(UActorDescContainerInstance* InContainerInstance)
{
	for (UActorDescContainerInstance::TConstIterator<> Iterator(InContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->GetActorDesc()->HasStandaloneHLOD())
		{
			const FLevelInstanceActorDesc& LevelInstanceActorDesc = *(FLevelInstanceActorDesc*)Iterator->GetActorDesc();
			FTransform ContainerTransform = FTransform::Identity;
			if (const UActorDescContainerInstance* ParentContainerInstance = InContainerInstance->GetParentContainerInstance())
			{
				if (FWorldPartitionActorDescInstance* ParentContainerActorDescInstance = ParentContainerInstance->GetActorDescInstance(InContainerInstance->GetContainerActorGuid()))
				{
					ContainerTransform = ContainerTransform * ParentContainerActorDescInstance->GetActorTransform();
				}
			}
			UpdateStandaloneHLODActorsRecursive(LevelInstanceActorDesc, ContainerTransform * LevelInstanceActorDesc.GetActorTransform(), /*bChildrenOnly=*/false);
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnActorDescContainerInstanceUnregistered(UActorDescContainerInstance* InContainerInstance)
{
	for (UActorDescContainerInstance::TConstIterator<> Iterator(InContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->GetActorDesc()->HasStandaloneHLOD())
		{
			if (FLevelInstanceActorDesc* LevelInstanceActorDesc = (FLevelInstanceActorDesc*)Iterator->GetActorDesc())
			{
				DeleteStandaloneHLODActorsRecursive(*LevelInstanceActorDesc);
			}
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::OnLevelAddedToWorld(ULevel* Level, UWorld* World)
{
	const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
	if (ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(Level) : nullptr)
	{
		LevelInstanceSubsystem->ForEachLevelInstanceChild(LevelInstance, /*bRecursive=*/false, [this](ILevelInstanceInterface* ChildLevelInstance)
		{
			OnActorChanged(CastChecked<AActor>(ChildLevelInstance));
			return true;
		});
	}
}

void UWorldPartitionStandaloneHLODSubsystem::UpdateStandaloneHLODActorsRecursive(const FLevelInstanceActorDesc& InLevelInstanceActorDesc, const FTransform InActorTransform, bool bChildrenOnly)
{
	if (!bChildrenOnly && InLevelInstanceActorDesc.GetDesiredRuntimeBehavior() == ELevelInstanceRuntimeBehavior::LevelStreaming)
	{
		FStandaloneHLODActorParams Params;
		Params.Guid = InLevelInstanceActorDesc.GetGuid();
		Params.Transform = InActorTransform;
		Params.WorldPackageName = InLevelInstanceActorDesc.GetChildContainerPackage().ToString();
		Params.ActorLabel = InLevelInstanceActorDesc.GetActorLabel().ToString();

		UpdateStandaloneHLODActors(Params);
	}

	UActorDescContainer* Container = UActorDescContainerSubsystem::GetChecked().RegisterContainer({InLevelInstanceActorDesc.GetChildContainerPackage()});
	for (FActorDescList::TConstIterator<> ActorDescIt(Container); ActorDescIt; ++ActorDescIt)
	{
		if (ActorDescIt->HasStandaloneHLOD())
		{
			const FLevelInstanceActorDesc& LevelInstanceActorDesc = *static_cast<const FLevelInstanceActorDesc*>(*ActorDescIt);

			UpdateStandaloneHLODActorsRecursive(LevelInstanceActorDesc, InActorTransform * LevelInstanceActorDesc.GetActorTransform(), /*bChildrenOnly=*/false);
		}
	}
	UActorDescContainerSubsystem::GetChecked().UnregisterContainer(Container);
}

void UWorldPartitionStandaloneHLODSubsystem::UpdateStandaloneHLODActors(UWorldPartitionStandaloneHLODSubsystem::FStandaloneHLODActorParams InStandaloneHLODActorParams)
{
	const FString HLODSuffix = TEXT("_HLOD");

	TArray<AWorldPartitionStandaloneHLOD*>* HLODActorsPtr = StandaloneHLODActors.Find(InStandaloneHLODActorParams.Guid);
	if (HLODActorsPtr)
	{
		TArray<AWorldPartitionStandaloneHLOD*>& HLODActors = *HLODActorsPtr;

		for (AWorldPartitionStandaloneHLOD* HLODActor : HLODActors)
		{
			int32 Position = HLODActor->GetActorLabel().Find(HLODSuffix, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			check(Position != INDEX_NONE);
			FString NameSuffix = HLODActor->GetActorLabel().RightChop(Position);
			FString NewActorLabel = InStandaloneHLODActorParams.ActorLabel + NameSuffix;

			HLODActor->SetActorTransform(InStandaloneHLODActorParams.Transform);
			HLODActor->SetActorLabel(NewActorLabel);
		}
	}
	else
	{
		FString FolderPath, LevelPackagePrefix;
		UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(InStandaloneHLODActorParams.WorldPackageName, FolderPath, LevelPackagePrefix);

		TArray<FString> Packages;
		FPackageName::FindPackagesInDirectory(Packages, FolderPath);

		TMap<int32, FActorContainerID> HLODIndexToContainer;

		for (const FString& Package : Packages)
		{
			if (!Package.Contains(LevelPackagePrefix))
			{
				continue;
			}

			FString PackageName = FPackageName::FilenameToLongPackageName(Package);
			if (!FEditorFileUtils::IsMapPackageAsset(PackageName))
			{
				continue;
			}

			const FSoftObjectPath LODWorldPackagePath(FTopLevelAssetPath(FName(*PackageName), FName(*FPackageName::GetLongPackageAssetName(PackageName))));

			int32 Position = PackageName.Find(HLODSuffix, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			check(Position != INDEX_NONE); // Guaranteed by LevelPackagePrefix returned by GetStandaloneHLODFolderPathAndPackagePrefix
			FString NameSuffix = PackageName.RightChop(Position);
			FString ActorLabel = InStandaloneHLODActorParams.ActorLabel + NameSuffix;
			int32 HLODIndex = FCString::Atoi(*NameSuffix.RightChop(HLODSuffix.Len()));

			FActorSpawnParameters SpawnParams;
			SpawnParams.ObjectFlags = RF_Transient;
			SpawnParams.Name = FName(ActorLabel);
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

			UWorldPartitionRuntimeHash* WorldHash = GetWorld()->GetWorldPartition() ? GetWorld()->GetWorldPartition()->RuntimeHash.Get() : nullptr;
			if (!WorldHash)
			{
				UE_LOGF(LogStandaloneHLODSubsystem, Log, "RuntimeHash not found in World %ls", *GetWorld()->GetName());
				return;
			}

			if (bRefreshCachedHLODSetups)
			{
				WorldHash->ForEachHLODLayer([this](FName RuntimePartitionName, FName HLODSetupName, int32 HLODSetupIndex)
				{
					FString RuntimeGrid = FString::Printf(TEXT("%s:%s"), *RuntimePartitionName.ToString(), *HLODSetupName.ToString());
					TMap<int32, FName>& HLODSetups = CachedHLODSetups.FindOrAdd(RuntimePartitionName);
					HLODSetups.Add(HLODSetupIndex, FName(RuntimeGrid));
					return true;
				});
				bRefreshCachedHLODSetups = false;
			}

			FName RuntimeGrid = NAME_None;
			if (TMap<int32, FName>* HLODSetups = CachedHLODSetups.Find(WorldHash->GetDefaultGrid()))
			{
				if (FName* RuntimeGridPtr = HLODSetups->Find(HLODIndex))
				{
					RuntimeGrid = *RuntimeGridPtr;
				}
			}

			if (RuntimeGrid == NAME_None)
			{
				UE_LOGF(LogStandaloneHLODSubsystem, Log, "Couldn't resolve Runtime Grid for %ls, HLODIndex %d", *WorldHash->GetDefaultGrid().ToString(), HLODIndex);
				return;
			}

			AWorldPartitionStandaloneHLOD* HLODActor = GetWorld()->SpawnActor<AWorldPartitionStandaloneHLOD>(AWorldPartitionStandaloneHLOD::StaticClass(), InStandaloneHLODActorParams.Transform, SpawnParams);
			HLODActor->SetWorldAsset(TSoftObjectPtr<UWorld>(LODWorldPackagePath));
			HLODActor->SetActorLabel(ActorLabel);
			HLODActor->SetFolderPath(TEXT("HLOD"));
			HLODActor->RegisterAllComponents();

			StandaloneHLODActors.FindOrAdd(InStandaloneHLODActorParams.Guid).Add(HLODActor);

			HLODIndexToContainer.Add(HLODIndex, HLODActor->GetLevelInstanceID().GetContainerID());
		}

		// Create a mapping from HLOD container to container with child HLOD actors
		// e.g. container with HLOD1 actors maps to the container with HLOD0 actors
		HLODIndexToContainer.KeySort(TLess<int32>());
		const FActorContainerID* ChildContainer = nullptr;
		for (const TPair<int32, FActorContainerID>& Entry : HLODIndexToContainer)
		{
			if (ChildContainer)
			{
				HLODContainerToChildHLODActorsContainer.Add(Entry.Value, *ChildContainer);
			}
			ChildContainer = &Entry.Value;
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::DeleteStandaloneHLODActorsRecursive(const FLevelInstanceActorDesc& InLevelInstanceActorDesc)
{
	DeleteStandaloneHLODActors(InLevelInstanceActorDesc.GetGuid());

	UActorDescContainer* Container = UActorDescContainerSubsystem::GetChecked().RegisterContainer({ InLevelInstanceActorDesc.GetChildContainerPackage() });
	for (FActorDescList::TConstIterator<> ActorDescIt(Container); ActorDescIt; ++ActorDescIt)
	{
		if (ActorDescIt->HasStandaloneHLOD())
		{
			const FLevelInstanceActorDesc& LevelInstanceActorDesc = *static_cast<const FLevelInstanceActorDesc*>(*ActorDescIt);
			DeleteStandaloneHLODActorsRecursive(LevelInstanceActorDesc);
		}
	}
	UActorDescContainerSubsystem::GetChecked().UnregisterContainer(Container);
}

void UWorldPartitionStandaloneHLODSubsystem::DeleteStandaloneHLODActors(FGuid InGuid)
{
	if (TArray<AWorldPartitionStandaloneHLOD*>* HLODActorsPtr = StandaloneHLODActors.Find(InGuid))
	{
		TArray<AWorldPartitionStandaloneHLOD*>& HLODActors = *HLODActorsPtr;
		for (AWorldPartitionStandaloneHLOD* HLODActor : HLODActors)
		{
			GetWorld()->DestroyActor(HLODActor);
		}
		HLODActors.Empty();
		StandaloneHLODActors.Remove(InGuid);
	}
}

void UWorldPartitionStandaloneHLODSubsystem::ForEachStandaloneHLODActor(TFunctionRef<void(AWorldPartitionStandaloneHLOD*)> Func) const
{
	for (const TPair<FGuid, TArray<AWorldPartitionStandaloneHLOD*>>& Pair : StandaloneHLODActors)
	{
		const TArray<AWorldPartitionStandaloneHLOD*>& HLODActors = Pair.Value;
		for (AWorldPartitionStandaloneHLOD* HLODActor : HLODActors)
		{
			Func(HLODActor);
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::ForEachStandaloneHLODActorWithLevelInstanceGuid(TFunctionRef<void(AWorldPartitionStandaloneHLOD*, const FGuid& LevelInstanceGuid)> Func) const
{
	for (const TPair<FGuid, TArray<AWorldPartitionStandaloneHLOD*>>& Pair : StandaloneHLODActors)
	{
		const FGuid& LevelInstanceGuid = Pair.Key;
		const TArray<AWorldPartitionStandaloneHLOD*>& HLODActors = Pair.Value;
		for (AWorldPartitionStandaloneHLOD* HLODActor : HLODActors)
		{
			Func(HLODActor, LevelInstanceGuid);
		}
	}
}

void UWorldPartitionStandaloneHLODSubsystem::ForEachStandaloneHLODActorFiltered(FGuid InGuid, TFunctionRef<void(AWorldPartitionStandaloneHLOD*)> Func) const
{
	const TArray<AWorldPartitionStandaloneHLOD*>* HLODActorsPtr = StandaloneHLODActors.Find(InGuid);
	if (HLODActorsPtr)
	{
		const TArray<AWorldPartitionStandaloneHLOD*>& HLODActors = *HLODActorsPtr;
		for (AWorldPartitionStandaloneHLOD* HLODActor : HLODActors)
		{
			Func(HLODActor);
		}
	}
}

bool UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(const FString& InWorldPackageName, FString& OutFolderPath, FString& OutPackagePrefix)
{
	const FString SourceLongPackagePath = FPackageName::GetLongPackagePath(InWorldPackageName);
	const FString SourceShortPackageName = FPackageName::GetShortName(InWorldPackageName);
	OutFolderPath = FString::Printf(TEXT("%s/HLOD"), *SourceLongPackagePath);
	OutPackagePrefix = FString::Printf(TEXT("%s_HLOD"), *SourceShortPackageName);

	return true;
}

bool UWorldPartitionStandaloneHLODSubsystem::GetChildHLODActorsContainer(const FActorContainerID& InContainerID, FActorContainerID& OutChildContainerID) const
{
	if (const FActorContainerID* ChildContainerID = HLODContainerToChildHLODActorsContainer.Find(InContainerID))
	{
		OutChildContainerID = *ChildContainerID;
		return true;
	}
	return false;
}
#endif