// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Landscape/LandscapeActorDesc.h"

#if WITH_EDITOR
#include "Landscape.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

void FLandscapeActorDesc::Init(const AActor* InActor)
{
	FPartitionActorDesc::Init(InActor);

	if (!bIsDefaultActorDesc)
	{
		const ALandscapeProxy* LandscapeProxy = CastChecked<ALandscapeProxy>(InActor);
		check(LandscapeProxy);

		SetGridIndices(LandscapeProxy->GetSectionBase().X, LandscapeProxy->GetSectionBase().Y, 0);

		const ALandscape* LandscapeActor = LandscapeProxy->GetLandscapeActor();
		if (LandscapeActor)
		{
			LandscapeActorGuid = LandscapeActor->GetActorGuid();
		}

		// FLandscapeActorDesc derives from FPartitionActorDesc but doesn't use the cell bounds as the parent class was designed for.
		// @todo_ow: make FLandscapeActorDesc derives from FWorldPartitionActorDesc instead?
		FBox NewRuntimeBounds = RuntimeBounds;
		FBox NewEditorBounds = EditorBounds;
		InActor->GetStreamingBounds(NewRuntimeBounds, NewEditorBounds);
		SetRuntimeBounds(NewRuntimeBounds);
		SetEditorBounds(NewEditorBounds);
	}
}

void FLandscapeActorDesc::Serialize(FArchive& Ar)
{
	FPartitionActorDesc::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::FLandscapeActorDescFixupGridIndices)
		{
			SetGridIndices(GridIndexX * GridSize, GridIndexY * GridSize, 0);
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionLandscapeActorDescSerializeLandscapeActorGuid)
		{
			Ar << LandscapeActorGuid;
		}
	}
}

bool FLandscapeActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FPartitionActorDesc::Equals(Other))
	{
		const FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)Other;
		return LandscapeActorGuid == LandscapeActorDesc->LandscapeActorGuid;
	}

	return false;
}

const FGuid& FLandscapeActorDesc::GetSceneOutlinerParent() const
{
	// Landscape can't parent itself
	if (LandscapeActorGuid != GetGuid())
	{
		return LandscapeActorGuid;
	}

	static FGuid NoParent;
	return NoParent;
}

#endif
