// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LevelInstance/LevelInstanceActorDesc.h"

#if WITH_EDITOR
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "Misc/PackageName.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstanceViewInterface.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/LevelInstance/LevelInstanceContainerInstance.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideContainer.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

static int32 GLevelInstanceDebugForceLevelStreaming = 0;
static FAutoConsoleVariableRef CVarForceLevelStreaming(
	TEXT("levelinstance.debug.forcelevelstreaming"),
	GLevelInstanceDebugForceLevelStreaming,
	TEXT("Set to 1 to force Level Instance to be streamed instead of embedded in World Partition grid."));

FLevelInstanceActorDesc::FLevelInstanceActorDesc()
	: DesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior::Partitioned)
	, ChildContainer(nullptr)
	, bIsChildContainerInstance(false)
	, bHasStandaloneHLOD(false)
{}

FLevelInstanceActorDesc::~FLevelInstanceActorDesc()
{
	if (!IsEngineExitRequested())
	{
		UnregisterChildContainer();
		check(!ChildContainer.IsValid());
	}
}

void FLevelInstanceActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const ILevelInstanceInterface* LevelInstance = CastChecked<ILevelInstanceInterface>(InActor);
	WorldAsset = LevelInstance->GetWorldAsset().ToSoftObjectPath();
	DesiredRuntimeBehavior = LevelInstance->GetDesiredRuntimeBehavior();
	Filter = LevelInstance->GetFilter();
	
	bIsChildContainerInstance = IsChildContainerInstanceInternal();

	bHasStandaloneHLOD = LevelInstance->HasStandaloneHLOD();

	if (LevelInstance->SupportsPropertyOverrides())
	{
		if (ULevelInstancePropertyOverrideAsset* Asset = LevelInstance->GetPropertyOverrideAsset())
		{
			FLevelInstancePropertyOverrideDesc* ExistingOverrideDesc = nullptr;
			
			// PropertyOverrides can be nested and we can only override properties one level instance at a time.
			// So when we do save an override, we need to keep the previous actors descs that weren't part of the current property override edit
			// So we find the existing ActorDesc and then check if it has an existing LevelInstanceOverrideActorDesc and transfer from it
			if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(InActor))
			{
				if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(InActor->GetActorGuid()))
				{
					if (FLevelInstanceActorDesc* ExistingActorDesc = (FLevelInstanceActorDesc*)ActorDescInstance->GetActorDesc())
					{
						ExistingOverrideDesc = ExistingActorDesc->GetOverrideDesc();
					}
				}
			}

			PropertyOverrideAsset = FSoftObjectPath(Asset);
			PropertyOverrideDesc = MakeShared<FLevelInstancePropertyOverrideDesc>();
			PropertyOverrideDesc->Init(Asset);

			// Transfer from previous override if we are sharing the same world package
			if (ExistingOverrideDesc && ExistingOverrideDesc->GetWorldPackage() == PropertyOverrideDesc->GetWorldPackage())
			{
				// We always need to transfer from here, the reason is that the PropertyOverride data is incomplete for unloaded Actors.
				// So we take the loaded data from the last Desc and transfer it to the latest data
				PropertyOverrideDesc->TransferNonEditedContainers(ExistingOverrideDesc);
			}

			// Heal the asset to match the desc.
			// ExistingOverrideDesc was pruned by ULevelInstancePropertyOverrideContainer::Initialize
			// when it was first registered; TransferNonEditedContainers above carried that pruned
			// state into the new desc. So PropertyOverrideDesc here is the authoritative survivor set.
			// Do NOT call Modify() on the asset: dirtying here would surprise the user with a save prompt or undo entry.
			if (Asset->GetPropertyOverridesPerContainer().Num() > 0)
			{
				TArray<FActorContainerPath> AssetPaths;
				// Need to capture the asset paths here because we mutate the map below
				Asset->GetPropertyOverridesPerContainer().GetKeys(AssetPaths);
				for (const FActorContainerPath& Path : AssetPaths)
				{
					if (const TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>* DescActorDescs = PropertyOverrideDesc->GetActorDescsPerContainer().Find(Path))
					{
						// Might find stale overrides
						Asset->PruneStaleActorOverrides(Path, *DescActorDescs);
					}
					else
					{
						// This whole storage path is no longer relevant
						Asset->RemoveContainerOverride(Path);
					}
				}
			}
		}
	}
}

void FLevelInstanceActorDesc::Init(const FWorldPartitionActorDescInitData& DescData)
{
	AActor* CDO = DescData.NativeClass->GetDefaultObject<AActor>();
	ILevelInstanceInterface* LevelInstanceCDO = CastChecked<ILevelInstanceInterface>(CDO);
	DesiredRuntimeBehavior = LevelInstanceCDO->GetDefaultRuntimeBehavior();

	FWorldPartitionActorDesc::Init(DescData);
	
	bIsChildContainerInstance = IsChildContainerInstanceInternal();
}

FString FLevelInstanceActorDesc::GetChildContainerName() const
{
	return (ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled() && PropertyOverrideAsset.IsValid()) ? PropertyOverrideAsset.ToString() : GetChildContainerPackage().ToString();
}

bool FLevelInstanceActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FLevelInstanceActorDesc* LevelInstanceActorDesc = (FLevelInstanceActorDesc*)Other;

		return
			WorldAsset == LevelInstanceActorDesc->WorldAsset &&
			PropertyOverrideAsset == LevelInstanceActorDesc->PropertyOverrideAsset &&
			DesiredRuntimeBehavior == LevelInstanceActorDesc->DesiredRuntimeBehavior &&
			bHasStandaloneHLOD == LevelInstanceActorDesc->bHasStandaloneHLOD;
	}

	return false;
}

void FLevelInstanceActorDesc::UpdateBounds()
{
	if (UActorDescContainer* ChildContainerPtr = ChildContainer.Get())
	{
		SetRuntimeBounds(UActorDescContainerSubsystem::GetChecked().GetContainerBounds(GetChildContainerName(), false).TransformBy(GetChildContainerTransform()));
		SetEditorBounds(UActorDescContainerSubsystem::GetChecked().GetContainerBounds(GetChildContainerName(), true).TransformBy(GetChildContainerTransform()));
	}
}

void FLevelInstanceActorDesc::RegisterChildContainer()
{
	check(!ChildContainer.IsValid());
	if (IsChildContainerInstance())
	{
		if (!PropertyOverrideAsset.IsValid() || !ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled())
		{
			ChildContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer({ GetChildContainerName(), GetChildContainerPackage() });
		}
		else
		{
			UActorDescContainer::FInitializeParams InitParams(PropertyOverrideAsset.ToString(), PropertyOverrideDesc->GetWorldPackage());
			InitParams.PreInitialize = [this](UActorDescContainer* InNewContainer)
			{
				ULevelInstancePropertyOverrideContainer* OverrideContainer = CastChecked<ULevelInstancePropertyOverrideContainer>(InNewContainer);
				OverrideContainer->SetPropertyOverrideDesc(PropertyOverrideDesc);
			};

			ChildContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer<ULevelInstancePropertyOverrideContainer>(InitParams);
		}

		UpdateBounds();
	}
}

void FLevelInstanceActorDesc::UnregisterChildContainer()
{
	if (UActorDescContainer* ChildContainerPtr = ChildContainer.Get())
	{
		UActorDescContainerSubsystem::GetChecked().UnregisterContainer(ChildContainerPtr);
	}
	ChildContainer = nullptr;
}

void FLevelInstanceActorDesc::SetContainer(UActorDescContainer* InContainer)
{
	FWorldPartitionActorDesc::SetContainer(InContainer);

	if (Container)
	{
		RegisterChildContainer();
	}
	else
	{
		UnregisterChildContainer();
	}
}

bool FLevelInstanceActorDesc::IsChildContainerInstance() const
{
	return bIsChildContainerInstance;
}

bool FLevelInstanceActorDesc::IsChildContainerInstanceInternal() const
{
	if (DesiredRuntimeBehavior != ELevelInstanceRuntimeBehavior::Partitioned)
	{
		return false;
	}
	
	if (GLevelInstanceDebugForceLevelStreaming)
	{
		return false;
	}

	if (WorldAsset.IsNull())
	{
		return false;
	}

	if (!ULevel::GetIsLevelUsingExternalActorsFromPackage(GetChildContainerPackage()) && 
		!ULevel::GetIsLevelUsingActorsDescsFromPackage(GetChildContainerPackage()))
	{
		return false;
	}

	return true;
}

FTransform FLevelInstanceActorDesc::GetChildContainerTransform() const
{
	FTransform LevelInstancePivotOffsetTransform = FTransform(ULevel::GetLevelInstancePivotOffsetFromPackage(GetChildContainerPackage()));
	return LevelInstancePivotOffsetTransform * ActorTransform;
}

bool FLevelInstanceActorDesc::GetChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance, FContainerInstance& OutContainerInstance) const
{
	if (UActorDescContainerInstance* ContainerInstance = InActorDescInstance->GetChildContainerInstance())
	{
		OutContainerInstance.ContainerInstance = ContainerInstance;
		OutContainerInstance.ClusterMode = EContainerClusterMode::Partitioned;

		// @todo_ow: this is to validate that new parenting of container instance code is equivalent
		OutContainerInstance.Transform = GetChildContainerTransform();

		return true;
	}

	return false;
}

void FLevelInstanceActorDesc::CheckForErrors(const IWorldPartitionActorDescInstanceView* InActorDescView, IStreamingGenerationErrorHandler* ErrorHandler) const
{
	FWorldPartitionActorDesc::CheckForErrors(InActorDescView, ErrorHandler);

	const FName ChildContainerPackage = InActorDescView->GetChildContainerPackage();

	FPackagePath WorldAssetPath;
	if (!FPackagePath::TryFromPackageName(ChildContainerPackage, WorldAssetPath) || !FPackageName::DoesPackageExist(WorldAssetPath))
	{
		ErrorHandler->OnLevelInstanceInvalidWorldAsset(*InActorDescView, ChildContainerPackage, IStreamingGenerationErrorHandler::ELevelInstanceInvalidReason::WorldAssetNotFound);
	}
	else
	{
		if (!ULevel::GetIsLevelUsingExternalActorsFromPackage(ChildContainerPackage) && !ULevel::GetIsLevelCookedFromPackage(ChildContainerPackage))
		{
			if (DesiredRuntimeBehavior != ELevelInstanceRuntimeBehavior::LevelStreaming)
			{
				if (!ULevel::GetIsLevelUsingActorsDescsFromPackage(ChildContainerPackage))
				{
					ErrorHandler->OnLevelInstanceInvalidWorldAsset(*InActorDescView, ChildContainerPackage, IStreamingGenerationErrorHandler::ELevelInstanceInvalidReason::WorldAssetDontContainActorsMetadata);
				}
			}
		}
	
		if (!ValidateCircularReference(InActorDescView->GetContainerInstance(), ChildContainerPackage))
		{
			ErrorHandler->OnLevelInstanceInvalidWorldAsset(*InActorDescView, ChildContainerPackage, IStreamingGenerationErrorHandler::ELevelInstanceInvalidReason::CirculalReference);
		}
	}
}

void FLevelInstanceActorDesc::TransferFrom(const FWorldPartitionActorDesc* From)
{
	FWorldPartitionActorDesc::TransferFrom(From);

	FLevelInstanceActorDesc* FromLevelInstanceActorDesc = (FLevelInstanceActorDesc*)From;

	RegisterChildContainer();
	FromLevelInstanceActorDesc->UnregisterChildContainer();

	if (ULevelInstancePropertyOverrideContainer* OverrideContainer = Cast<ULevelInstancePropertyOverrideContainer>(ChildContainer.Get()))
	{
		OverrideContainer->SetPropertyOverrideDesc(PropertyOverrideDesc);
	}
}

UWorldPartition* FLevelInstanceActorDesc::GetLoadedChildWorldPartition(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InActorDescInstance->GetActor()))
	{
		if (ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
		{
			return LoadedLevel->GetWorldPartition();
		}
	}

	return nullptr;
}

bool FLevelInstanceActorDesc::ValidateCircularReference(const UActorDescContainerInstance* InParentContainer, FName InChildContainerPackage)
{
	const UActorDescContainerInstance* CurrentParentContainerInstance = InParentContainer;
	while (CurrentParentContainerInstance)
	{
		if (CurrentParentContainerInstance->GetContainerPackage() == InChildContainerPackage)
		{
			// found a circular reference
			return false; 
		}
		CurrentParentContainerInstance = Cast<UActorDescContainerInstance>(CurrentParentContainerInstance->GetOuter());
	}

	return true;
}

UActorDescContainerInstance* FLevelInstanceActorDesc::CreateChildContainerInstance(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	// Update Actor Desc Container in case of rename
	if (UActorDescContainer* PreviousChildContainer = ChildContainer.Get(); PreviousChildContainer->GetContainerName() != GetChildContainerName())
	{
		FLevelInstanceActorDesc* NonConstThis = const_cast<FLevelInstanceActorDesc*>(this);
		NonConstThis->UnregisterChildContainer();
		NonConstThis->RegisterChildContainer();
	}

	UActorDescContainerInstance* ContainerInstance = InActorDescInstance->GetContainerInstance();
	if (!ValidateCircularReference(ContainerInstance, InActorDescInstance->GetChildContainerPackage()))
	{
		return nullptr;
	}
	
	// Create ChildContainerInstance
	ULevelInstanceContainerInstance* ChildContainerInstance = NewObject<ULevelInstanceContainerInstance>(ContainerInstance, NAME_None, RF_Transient);

	// Set Override Container which might be a regular UActorDescContainer if no overrides exist on this Container Instance
	ChildContainerInstance->SetOverrideContainerAndAsset(GetChildContainer(), nullptr);
		
	// When a child container instance is created we create the whole hierarchy (for generate streaming)
	const bool bInCreateContainerInstanceHierarchy = true;
	UActorDescContainerInstance::FInitializeParams InitParams(InActorDescInstance->GetChildContainerPackage(), bInCreateContainerInstanceHierarchy);
	InitParams.SetParent(ContainerInstance, InActorDescInstance->GetGuid())
		.SetTransform(GetChildContainerTransform() * ContainerInstance->GetTransform());

	ChildContainerInstance->Initialize(InitParams);
		
	return ChildContainerInstance;
}

void FLevelInstanceActorDesc::Serialize(FArchive& Ar)
{
	FWorldPartitionActorDesc::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeSoftObjectPathSupport)
	{
		Ar << TDeltaSerialize<FSoftObjectPath, FName>(WorldAsset, [](FSoftObjectPath& Value, const FName& DeprecatedValue)
		{
			Value = FSoftObjectPath(*DeprecatedValue.ToString());
		});
	}
	else
	{
		Ar << TDeltaSerialize<FSoftObjectPath>(WorldAsset);
	}

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::LargeWorldCoordinates)
		{
			FTransform3f ActorTransformFlt;
			Ar << ActorTransformFlt;
			ActorTransform = FTransform(ActorTransformFlt);
		}
		else if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescActorTransformSerialization)
		{
			Ar << ActorTransform;
		}
	}

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::LevelInstanceSerializeRuntimeBehavior)
	{
		Ar << TDeltaSerialize<ELevelInstanceRuntimeBehavior>(DesiredRuntimeBehavior);

		if (Ar.IsLoading() && DesiredRuntimeBehavior == ELevelInstanceRuntimeBehavior::Embedded_Deprecated)
		{
			DesiredRuntimeBehavior = ELevelInstanceRuntimeBehavior::Partitioned;
		}
	}

	if (!bIsDefaultActorDesc)
	{
		if (Ar.IsLoading())
		{
			const bool bFixupOldVersion = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::PackedLevelInstanceBoundsFix) && 
										  (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::PackedLevelInstanceBoundsFix);

			const AActor* CDO = GetActorNativeClass()->GetDefaultObject<AActor>();
			const ILevelInstanceInterface* LevelInstanceCDO = CastChecked<ILevelInstanceInterface>(CDO);
			if (WorldAsset.IsValid() && (LevelInstanceCDO->IsLoadingEnabled() || bFixupOldVersion))
			{
				if (!IsChildContainerInstance())
				{
					FBox NewRuntimeBounds;
					if (ULevelInstanceSubsystem::GetLevelInstanceBoundsFromPackage(ActorTransform, GetChildContainerPackage(), NewRuntimeBounds))
					{
						SetRuntimeBounds(NewRuntimeBounds);
					}
				}
			}
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorFilter)
	{
		if(Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::LevelInstanceActorDescDeltaSerializeFilter)
		{
			Ar << TDeltaSerialize<FWorldPartitionActorFilter>(Filter);
		}
		else
		{
			Ar << Filter;
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::LevelInstancePropertyOverrides)
	{
		FString PropertyOverrideAssetString = PropertyOverrideAsset.ToString();
		Ar << PropertyOverrideAssetString;

		if (Ar.IsLoading())
		{
			PropertyOverrideAsset = FSoftObjectPath(PropertyOverrideAssetString);
			UAssetRegistryHelpers::FixupRedirectedAssetPath(PropertyOverrideAsset);
		}

		if (PropertyOverrideAsset.IsValid())
		{
			if (Ar.IsLoading())
			{
				PropertyOverrideDesc = MakeShared<FLevelInstancePropertyOverrideDesc>();
			}
		
			FLevelInstancePropertyOverrideDesc* PropertyOverrideDescPtr = PropertyOverrideDesc.Get();
			Ar << *PropertyOverrideDescPtr;

			PropertyOverrideDescPtr->AssetPath = PropertyOverrideAsset;
			PropertyOverrideDescPtr->PackageName = ActorPackage;
		}
	}

	// As this property was previously ignored for Level Instances, make sure not to break existing content that may have modified that flag.
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ResetLevelInstanceHLODRelevancy)
	{
		bActorIsHLODRelevant = true;
	}

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionAddStandaloneHLODSupport)
		{
			Ar << bHasStandaloneHLOD;
		}
	}
}
#endif
