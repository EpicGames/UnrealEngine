// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LevelInstance/LevelInstanceContainerInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstanceContainerInstance)

#if WITH_EDITOR
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideContainer.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideDesc.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "LevelInstance/LevelInstanceSettings.h"
#include "LevelInstance/LevelInstanceInterface.h"

void ULevelInstanceContainerInstance::Initialize(const FInitializeParams& InParams)
{
	Super::Initialize(InParams);

	// Add References to parent Container Instance(s). This prevents them from being unloaded before this Container instance is uninitialized (can happen on a map change where Worlds are unloaded in random order through CleanupWorld)
	UActorDescContainerInstance* CurrentParentContainerInstance = const_cast<UActorDescContainerInstance*>(GetParentContainerInstance());
	while (CurrentParentContainerInstance)
	{
		UActorDescContainer* ParentContainer = CurrentParentContainerInstance->GetContainer();
		UActorDescContainerSubsystem::GetChecked().RegisterContainer(ParentContainer);
		ParentContainerReferences.Add(ParentContainer);
		CurrentParentContainerInstance = const_cast<UActorDescContainerInstance*>(CurrentParentContainerInstance->GetParentContainerInstance());
	}

	// Build mapping from ContainerID to ContainerPath
	if (const ULevelInstancePropertyOverrideContainer* LocalOverrideContainer = Cast<ULevelInstancePropertyOverrideContainer>(OverrideContainer))
	{
		if (const FLevelInstancePropertyOverrideDesc* PropertyOverrideDesc = LocalOverrideContainer->GetPropertyOverrideDesc())
		{
			for (auto const& [ContainerPath, OverridesPerActor] : PropertyOverrideDesc->GetActorDescsPerContainer())
			{
				FActorContainerID OverrideContainerID(GetContainerID(), ContainerPath);
				ContainerIDToContainerPath.Add(OverrideContainerID, ContainerPath);
			}
		}
	}
}

void ULevelInstanceContainerInstance::Uninitialize()
{
	Super::Uninitialize();

	if (UActorDescContainerSubsystem* ActorDescContainerSubsystem = UActorDescContainerSubsystem::Get())
	{
		for (UActorDescContainer* ParentContainer : ParentContainerReferences)
		{
			ActorDescContainerSubsystem->UnregisterContainer(ParentContainer);
		}
	}
	ParentContainerReferences.Empty();
}

void ULevelInstanceContainerInstance::SetOverrideContainerAndAsset(UActorDescContainer* InOverrideContainer, ULevelInstancePropertyOverrideAsset* InAsset)
{
	check(!IsInitialized());
	OverrideContainer = InOverrideContainer;
	PropertyOverrideAsset = InAsset;
}

void ULevelInstanceContainerInstance::RegisterContainer(const FInitializeParams& InParams)
{
	if (OverrideContainer)
	{
		UActorDescContainerSubsystem::GetChecked().RegisterContainer(OverrideContainer);
		Super::SetContainer(OverrideContainer);
	}
	else
	{
		Super::RegisterContainer(InParams);
	}
}

void ULevelInstanceContainerInstance::UnregisterContainer()
{
	if (!IsEngineExitRequested())
	{
		if (OverrideContainer)
		{
			UActorDescContainerSubsystem::GetChecked().UnregisterContainer(OverrideContainer);
			Super::SetContainer(nullptr);
		}
		else
		{
			Super::UnregisterContainer();
		}
	}
	OverrideContainer = nullptr;
}

FWorldPartitionActorDesc* ULevelInstanceContainerInstance::GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath) const
{
	if (const ULevelInstanceContainerInstance* ParentLevelInstanceContainer = Cast<ULevelInstanceContainerInstance>(GetParentContainerInstance()))
	{
		FActorContainerPath ContainerPath;
		ContainerPath.ContainerGuids.Add(ContainerActorGuid);
		ContainerPath.ContainerGuids.Append(InContainerPath.ContainerGuids);
		if (FWorldPartitionActorDesc* OverrideActorDesc = ParentLevelInstanceContainer->GetOverrideActorDesc(InActorGuid, ContainerPath))
		{
			return OverrideActorDesc;
		}
	}

	if (const ULevelInstancePropertyOverrideContainer* CurrentOverrideContainer = Cast<const ULevelInstancePropertyOverrideContainer>(GetContainer()))
	{
		return CurrentOverrideContainer->GetOverrideActorDesc(InActorGuid, InContainerPath);
	}

	return nullptr;
}

FWorldPartitionActorDesc* ULevelInstanceContainerInstance::GetActorDesc(const FGuid& InActorGuid) const
{
	// Check if we have an override desc in our parent hierarchy for this actor and return it if we do
	if (FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(InActorGuid))
	{
		return OverrideActorDesc;
	}

	// If not, call the base class GetActorDesc which might still find an override through our own ULevelInstancePropertyOverrideContainer
	return Super::GetActorDesc(InActorGuid);
}

FWorldPartitionActorDesc* ULevelInstanceContainerInstance::GetActorDescChecked(const FGuid& InActorGuid) const
{
	if (FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(InActorGuid))
	{
		return OverrideActorDesc;
	}

	return Super::GetActorDescChecked(InActorGuid);
}

void ULevelInstanceContainerInstance::GetPropertyOverridesForActor(const FActorContainerID& InContainerID, const FGuid& InActorGuid, TArray<FWorldPartitionRuntimeCellPropertyOverride>& OutPropertyOverrides) const
{
	if (!ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled())
	{
		return;
	}

	if (const FActorContainerPath* FoundContainerPath = ContainerIDToContainerPath.Find(InContainerID))
	{
		const ULevelInstancePropertyOverrideContainer* LocalOverrideContainer = Cast<ULevelInstancePropertyOverrideContainer>(OverrideContainer);
		check(LocalOverrideContainer);

		const FLevelInstancePropertyOverrideDesc* PropertyOverrideDesc = LocalOverrideContainer->GetPropertyOverrideDesc();
		check(PropertyOverrideDesc);

		const TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>& OverridesPerActor = PropertyOverrideDesc->GetActorDescsPerContainer().FindChecked(*FoundContainerPath);

		// Found an override for actor
		if (OverridesPerActor.Contains(InActorGuid))
		{
			// Return as string to avoid path remapping in PIE
			// Asset Path is the ObjectPath to the Property Override Asset
			// Asset Package in this case is the Outer Actor Package to load
			OutPropertyOverrides.Add({GetContainerID(), PropertyOverrideDesc->GetAssetPath().ToString(), PropertyOverrideDesc->GetAssetPackage(), *FoundContainerPath });
		}
	}

	if (const ULevelInstanceContainerInstance* ParentLevelInstanceContainer = Cast<ULevelInstanceContainerInstance>(GetParentContainerInstance()))
	{
		ParentLevelInstanceContainer->GetPropertyOverridesForActor(InContainerID, InActorGuid, OutPropertyOverrides);
	}
}

void ULevelInstanceContainerInstance::GetPropertyOverridesForActor(const FActorContainerID& InContainerID, const FActorContainerID& InContextContainerID, const FGuid& InActorGuid, TArray<FLevelInstanceActorPropertyOverride>& OutPropertyOverrides) const
{
	if (!ULevelInstanceSettings::Get()->IsPropertyOverrideEnabled())
	{
		return;
	}

	// Context to which we want to limit the overrides:
	// When a Level Instance is part of a non editing hierarchy then the context will be the main container. Meaning we will gather all overrides in the hierarchy
	// When a Level Instance is part of a editing hierarchy we want to apply overrides up to a certain parent to remain in context of that edit
	if (GetContainerID() == InContextContainerID)
	{
		return;
	}
	
	if (ULevelInstancePropertyOverrideAsset* PropertyOverrideAssetPtr = PropertyOverrideAsset.Get())
	{
		if (const FActorContainerPath* FoundContainerPath = ContainerIDToContainerPath.Find(InContainerID))
		{
			if (const FContainerPropertyOverride* FoundContainerOverride = PropertyOverrideAssetPtr->GetPropertyOverridesPerContainer().Find(*FoundContainerPath))
			{
				if (const FActorPropertyOverride* FoundActorOverride = FoundContainerOverride->ActorOverrides.Find(InActorGuid))
				{
					if (const UActorDescContainerInstance* ParentContainerInstancePtr = ParentContainerInstance.Get())
					{
						if (const FWorldPartitionActorDescInstance* ParentActorDescInstance = ParentContainerInstance->GetActorDescInstance(GetContainerActorGuid()))
						{
							if (const ILevelInstanceInterface* ParentLevelInstance = Cast<ILevelInstanceInterface>(ParentActorDescInstance->GetActor()))
							{
								OutPropertyOverrides.Add(FLevelInstanceActorPropertyOverride(ParentLevelInstance->GetLevelInstanceID(), FoundActorOverride));
							}
						}
					}
				}
			}
		}
	}

	if (const ULevelInstanceContainerInstance* ParentLevelInstanceContainer = Cast<ULevelInstanceContainerInstance>(GetParentContainerInstance()))
	{
		ParentLevelInstanceContainer->GetPropertyOverridesForActor(InContainerID, InContextContainerID, InActorGuid, OutPropertyOverrides);
	}
}

#endif
