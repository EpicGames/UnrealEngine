// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideContainer.h"

#if WITH_EDITOR
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/WorldPartitionLog.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelInstancePropertyOverrideContainer)

#if WITH_EDITOR

void ULevelInstancePropertyOverrideContainer::Initialize(const FInitializeParams& InitParams)
{
	// Call PreInit callback
	if (InitParams.PreInitialize)
	{
		InitParams.PreInitialize(this);
	}

	check(PropertyOverrideDesc);
	check(GetBaseContainer());
	check(GetBaseContainer()->GetContainerPackage() == InitParams.PackageName);
	
	SetIsProxy();

	// Copy values from Container we are proxying
	ContainerPackageName = GetBaseContainer()->GetContainerPackage();
	ContentBundleGuid = GetBaseContainer()->GetContentBundleGuid();
	ExternalDataLayerAsset = GetBaseContainer()->GetExternalDataLayerAsset();

	TMap<FWorldPartitionActorDesc*, UActorDescContainer*> OverrideDescToBaseContainer;
	auto CloneActorDesc = [this, &OverrideDescToBaseContainer](FWorldPartitionActorDesc* InActorDesc, const FActorContainerPath& InContainerPath)
	{
		if (!GetOverrideActorDesc(InActorDesc->GetGuid(), InContainerPath))
		{
			// Clone actordesc
			UClass* ActorNativeClass = InActorDesc->GetActorNativeClass();
			if (ensure(ActorNativeClass))
			{
				TUniquePtr<FWorldPartitionActorDesc> NewActorDesc(AActor::StaticCreateClassActorDesc(ActorNativeClass));
				FWorldPartitionActorDescInitData ActorDescInitData = FWorldPartitionActorDescInitData()
					.SetNativeClass(ActorNativeClass)
					.SetPackageName(InActorDesc->GetActorPackage())
					.SetActorPath(InActorDesc->GetActorSoftPath());
				InActorDesc->SerializeTo(ActorDescInitData.GetSerializedData());
				NewActorDesc->Init(ActorDescInitData);
				check(NewActorDesc->Equals(InActorDesc));

				// Set Container
				check(!NewActorDesc->GetContainer());
				NewActorDesc->SetContainer(this);

				OverrideDescToBaseContainer.Emplace(NewActorDesc.Get(), InActorDesc->GetContainer());
				TMap<FGuid, TUniquePtr<FWorldPartitionActorDesc>>& TransientOverridesMap = PerContainerTransientOverrideActorDescs.FindOrAdd(InContainerPath);
				TransientOverridesMap.Emplace(NewActorDesc->GetGuid(), MoveTemp(NewActorDesc));
			}
		}
	};

	// Paths that are no longer valid, because for example an overridden sub-LI was removed from an overridden LI while the main level wasn't loaded.
	TArray<FActorContainerPath> StalePaths;

	TMap<FActorContainerPath, UActorDescContainer*> BaseContainers;
	for (const auto& ActorDescsPerContainerElement : PropertyOverrideDesc->ActorDescsPerContainer)
	{
		const FActorContainerPath& ContainerPath = ActorDescsPerContainerElement.Key;
		UActorDescContainer* BaseContainer = const_cast<UActorDescContainer *>(PropertyOverrideDesc->GetBaseContainer(GetBaseContainer(), ContainerPath));
		if (BaseContainer)
		{
			if (ActorDescsPerContainerElement.Value.IsEmpty())
			{
				// There were actors at this path before, but they were all removed
				StalePaths.Add(ContainerPath);
				continue;
			}

			check(BaseContainer->IsInitialized());
			BaseContainers.Emplace(ContainerPath, BaseContainer);

			for (const auto& ActorDescsMapElement : ActorDescsPerContainerElement.Value)
			{
				const TSharedPtr<FWorldPartitionActorDesc>& OverrideActorDesc = ActorDescsMapElement.Value;
				OverrideDescToBaseContainer.Emplace(OverrideActorDesc.Get(), BaseContainer);

				FWorldPartitionActorDesc* BaseActorDesc = BaseContainer->GetActorDesc(OverrideActorDesc->GetGuid());
				if (ensure(BaseActorDesc))
				{
					// Check if override actordesc actor transform is different
					if (!OverrideActorDesc->GetActorTransformRelative().Equals(BaseActorDesc->GetActorTransformRelative()))
					{
						// Create transient override actordescs for all children of this override actordesc (if they are not already part of the override actordesc).
						// This will allow mutate their world editor and runtime bounds using UpdateActorToWorld.
						// Note that GetOverrideActorDesc now also searches in the transient overrides.
						BaseContainer->ForEachChildActorDesc(OverrideActorDesc->GetGuid(), [this, &CloneActorDesc, &ContainerPath, BaseContainer](const FGuid& ChildActorGuid)
						{
							if (FWorldPartitionActorDesc* ChildActorDesc = BaseContainer->GetActorDesc(ChildActorGuid))
							{
								CloneActorDesc(ChildActorDesc, ContainerPath);
							}
						});
					}
				}
			}
		}
		else
		{
			// Saved container path no longer resolves against the live Level Instance hierarchy. This is a legitimate
			// runtime occurrence (the referenced nested Level Instance was deleted, replaced, was switched away from Partitioned, etc.
			// Skip the entry: bounds for actors under this stale path will not be updated, but the override remains queryable until it is re-saved.
			StalePaths.Add(ContainerPath);
		}
	}

	// Heal the in-memory desc so this session's WP runtime view doesn't carry stale entries. Also fixes the container so the next asset save
	// can detect discrepancies without having to rewalk the container.
	if (StalePaths.Num() > 0)
	{
		for (const FActorContainerPath& StalePath : StalePaths)
		{
			PropertyOverrideDesc->ActorDescsPerContainer.Remove(StalePath);
		}
		UE_LOG(LogWorldPartition, Verbose, TEXT("[%s] Healed %d stale path(s)."), *PropertyOverrideDesc->GetContainerName(), StalePaths.Num());
	}

	auto UpdateActorToWorld = [this, &OverrideDescToBaseContainer](const FGuid& InActorGuid, const FActorContainerPath& InContainerPath)
	{
		if (FWorldPartitionActorDesc* MutableActorDesc = GetOverrideActorDesc(InActorGuid, InContainerPath))
		{
			FTransform ParentTransform = FTransform::Identity;
			if (MutableActorDesc->GetParentActor().IsValid())
			{
				const FWorldPartitionActorDesc* ParentActorDesc = GetOverrideActorDesc(MutableActorDesc->GetParentActor(), InContainerPath);
				if (!ParentActorDesc)
				{
					// Parent actor is not part of overrides, find it in MutableActorDesc's associated container
					UActorDescContainer* MutableActorDescContainer = OverrideDescToBaseContainer.FindChecked(MutableActorDesc);
					ParentActorDesc = MutableActorDescContainer->GetActorDesc(MutableActorDesc->GetParentActor());
				}
				if (ensure(ParentActorDesc))
				{
					ParentTransform = ParentActorDesc->GetActorTransform();
				}
			}

			// Always pass parent transform (even when Identity) because we don't want UpdateActorToWorld to resolve
			// the parent by itself, as it misses the context (InContainerPath) to properly resolve the actordesc.
			UpdateActorDesc(MutableActorDesc,
				[&ParentTransform](FWorldPartitionActorDesc* ActorDesc)
				{
					ActorDesc->UpdateActorToWorld(&ParentTransform);
				}
			);
		}
	};

	for (const auto& BaseContainerElement : BaseContainers)
	{
		const FActorContainerPath& ContainerPath = BaseContainerElement.Key;
		UActorDescContainer* BaseContainer = BaseContainerElement.Value;

		// Update the world bounds of all overrides and transient overrides starting from the root actordescs.
		for (FActorDescList::TIterator<> ActorDescIterator(BaseContainer); ActorDescIterator; ++ActorDescIterator)
		{
			FWorldPartitionActorDesc* ActorDesc = *ActorDescIterator;
			if (!ActorDesc->GetParentActor().IsValid())
			{
				UpdateActorToWorld(ActorDesc->GetGuid(), ContainerPath);
				BaseContainer->ForEachChildActorDesc(ActorDesc->GetGuid(), [this, &UpdateActorToWorld, &ContainerPath](const FGuid& ChildActorGuid)
				{
					UpdateActorToWorld(ChildActorGuid, ContainerPath);
				});
			}
		}
	}
	bContainerInitialized = true;
}

void ULevelInstancePropertyOverrideContainer::Uninitialize()
{
	// UActorDescContainer::BeginDestroy calls Uninitialize()
	if (bContainerInitialized)
	{
		// nothing to do except unregister delegates this class is a proxy to the PropertyOverrideDesc Base Container + Override Descs
		UnregisterBaseContainerDelegates();
		PropertyOverrideDesc = nullptr;
		PerContainerTransientOverrideActorDescs.Reset();
		bContainerInitialized = false;
	}
}

FActorDescList::FGuidActorDescMap& ULevelInstancePropertyOverrideContainer::GetProxyActorsByGuid() const
{
	return GetBaseContainer()->GetActorsByGuid();
}

UActorDescContainer* ULevelInstancePropertyOverrideContainer::GetBaseContainer() const
{
	check(PropertyOverrideDesc);
	return PropertyOverrideDesc->GetBaseContainer();
}

void ULevelInstancePropertyOverrideContainer::SetPropertyOverrideDesc(const TSharedPtr<FLevelInstancePropertyOverrideDesc>& InPropertyOverrideDesc)
{
	if (PropertyOverrideDesc == InPropertyOverrideDesc)
	{
		return;
	}
	check(InPropertyOverrideDesc);

	if (PropertyOverrideDesc)
	{
		UnregisterBaseContainerDelegates();
	}

	check(!PropertyOverrideDesc || (GetContainerName() == InPropertyOverrideDesc->GetContainerName()));
			
	PropertyOverrideDesc = InPropertyOverrideDesc;
	PropertyOverrideDesc->SetContainerForActorDescs(this);

	RegisterBaseContainerDelegates();
}

void ULevelInstancePropertyOverrideContainer::UnregisterBaseContainerDelegates()
{
	check(PropertyOverrideDesc);
	UActorDescContainer* BaseContainer = GetBaseContainer();
	check(BaseContainer);
	BaseContainer->OnActorDescRemovedEvent.RemoveAll(this);
	BaseContainer->OnActorDescPreUpdateEvent.RemoveAll(this);
	BaseContainer->OnActorDescUpdatingEvent.RemoveAll(this);
	BaseContainer->OnActorDescUpdatedEvent.RemoveAll(this);
}

void ULevelInstancePropertyOverrideContainer::RegisterBaseContainerDelegates()
{
	check(PropertyOverrideDesc)
	UActorDescContainer* BaseContainer = PropertyOverrideDesc->GetBaseContainer();
	check(BaseContainer);
	BaseContainer->OnActorDescRemovedEvent.AddUObject(this, &ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescRemoved);
	BaseContainer->OnActorDescPreUpdateEvent.AddUObject(this, &ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescPreUpdate);
	BaseContainer->OnActorDescUpdatingEvent.AddUObject(this, &ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescUpdating);
	BaseContainer->OnActorDescUpdatedEvent.AddUObject(this, &ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescUpdated);
}

void ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescRemoved(FWorldPartitionActorDesc* InActorDesc)
{
	OnActorDescRemoved(InActorDesc);
}

void ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescPreUpdate(FWorldPartitionActorDesc* InActorDesc)
{
	OnActorDescPreUpdate(InActorDesc);
}

void ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescUpdating(FWorldPartitionActorDesc* InActorDesc)
{
	OnActorDescUpdating(InActorDesc);
}

void ULevelInstancePropertyOverrideContainer::OnBaseContainerActorDescUpdated(FWorldPartitionActorDesc* InActorDesc)
{
	OnActorDescUpdated(InActorDesc);
}

FString ULevelInstancePropertyOverrideContainer::GetContainerName() const
{
	check(PropertyOverrideDesc);
	return PropertyOverrideDesc->GetContainerName();
}

FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetActorDesc(const FGuid& InActorGuid)
{
	// Same pattern used for all Get*(ActorGuid) methods overriden, find the ActorDesc in the base container
	// If we find it then check if we have an override for it passing in an empty path as we are looking for an override on this top level container and not a child container
	if (FWorldPartitionActorDesc* BaseActorDesc = GetBaseContainer()->GetActorDesc(InActorGuid))
	{
		if (FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(InActorGuid))
		{
			return OverrideActorDesc;
		}

		return BaseActorDesc;
	}

	return nullptr;
}

const FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetActorDesc(const FGuid& InActorGuid) const
{
	return const_cast<ULevelInstancePropertyOverrideContainer*>(this)->GetActorDesc(InActorGuid);
}

FWorldPartitionActorDesc& ULevelInstancePropertyOverrideContainer::GetActorDescChecked(const FGuid& InActorGuid)
{
	if (FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(InActorGuid))
	{
		return *OverrideActorDesc;
	}

	return GetBaseContainer()->GetActorDescChecked(InActorGuid);
}

const FWorldPartitionActorDesc& ULevelInstancePropertyOverrideContainer::GetActorDescChecked(const FGuid& InActorGuid) const
{
	return const_cast<ULevelInstancePropertyOverrideContainer*>(this)->GetActorDescChecked(InActorGuid);
}

const FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetActorDescByPath(const FString& InActorPath) const
{
	if (const FWorldPartitionActorDesc* BaseActorDesc = GetBaseContainer()->GetActorDescByPath(InActorPath))
	{
		if (const FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(BaseActorDesc->GetGuid()))
		{
			return OverrideActorDesc;
		}

		return BaseActorDesc;
	}

	return nullptr;
}

const FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetActorDescByPath(const FSoftObjectPath& InActorPath) const
{
	return GetActorDescByPath(InActorPath.ToString());
}

const FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetActorDescByName(FName InActorName) const
{
	if (const FWorldPartitionActorDesc* BaseActorDesc = GetBaseContainer()->GetActorDescByName(InActorName))
	{
		if (const FWorldPartitionActorDesc* OverrideActorDesc = GetOverrideActorDesc(BaseActorDesc->GetGuid()))
		{
			return OverrideActorDesc;
		}

		return BaseActorDesc;
	}

	return nullptr;
}

FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath) const
{
	check(PropertyOverrideDesc);
	FWorldPartitionActorDesc* Desc = PropertyOverrideDesc->GetOverrideActorDesc(InActorGuid, InContainerPath);
	if (!Desc)
	{
		Desc = GetTransientOverrideActorDesc(InActorGuid, InContainerPath);
	}
	return Desc;
}

FWorldPartitionActorDesc* ULevelInstancePropertyOverrideContainer::GetTransientOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath) const
{
	if (const TMap<FGuid, TUniquePtr<FWorldPartitionActorDesc>>* ActorDescs = PerContainerTransientOverrideActorDescs.Find(InContainerPath))
	{
		if (const TUniquePtr<FWorldPartitionActorDesc>* ActorDesc = ActorDescs->Find(InActorGuid))
		{
			return ActorDesc->Get();
		}
	}
	return nullptr;
}

#endif

