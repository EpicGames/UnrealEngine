// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Misc/Optional.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "LevelInstancePropertyOverrideContainer.generated.h"

class FLevelInstancePropertyOverrideDesc;
class FWorldPartitionActorDesc;
struct FActorContainerPath;

/**
 * LevelInstancePropertyOverrideContainer is a proxy to its ContainerPackage ActorDescContainer
 * plus some potential ActorDesc overrides for its proxy Container or any other child Container in 
 * its Container child hierarchy.
 */
UCLASS(MinimalAPI)
class ULevelInstancePropertyOverrideContainer : public UActorDescContainer
{
	GENERATED_BODY()

	ULevelInstancePropertyOverrideContainer() {}
	virtual ~ULevelInstancePropertyOverrideContainer() = default;

#if WITH_EDITOR
public:
	//~ Begin UActorDescContainer Interface
	virtual void Initialize(const FInitializeParams& InitParams) override;
	virtual void Uninitialize() override;
		
	virtual FString GetContainerName() const override;

	virtual FWorldPartitionActorDesc* GetActorDesc(const FGuid& InActorGuid) override;
	virtual const FWorldPartitionActorDesc* GetActorDesc(const FGuid& InActorGuid) const override;

	virtual FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& InActorGuid) override;
	virtual const FWorldPartitionActorDesc& GetActorDescChecked(const FGuid& InActorGuid) const override;

	virtual const FWorldPartitionActorDesc* GetActorDescByPath(const FString& InActorPath) const override;
	virtual const FWorldPartitionActorDesc* GetActorDescByPath(const FSoftObjectPath& InActorPath) const override;
	virtual const FWorldPartitionActorDesc* GetActorDescByName(FName InActorName) const override;
	//~ End UActorDescContainer Interface
				
private:
	friend class FLevelInstanceActorDesc;
	friend class ULevelInstanceContainerInstance;

	FWorldPartitionActorDesc* GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath = FActorContainerPath()) const;
	FWorldPartitionActorDesc* GetTransientOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath = FActorContainerPath()) const;

	void SetPropertyOverrideDesc(const TSharedPtr<FLevelInstancePropertyOverrideDesc>& InPropertyOverrideDesc);
	const FLevelInstancePropertyOverrideDesc* GetPropertyOverrideDesc() const { return PropertyOverrideDesc.Get(); }

	//~ Begin UActorDescContainer Interface
	virtual FGuidActorDescMap& GetProxyActorsByGuid() const override;
	virtual bool ShouldRegisterDelegates() const override { return false; }
	//~ End UActorDescContainer Interface

	void RegisterBaseContainerDelegates();
	void UnregisterBaseContainerDelegates();

	void OnBaseContainerActorDescRemoved(FWorldPartitionActorDesc* InActorDesc);
	void OnBaseContainerActorDescPreUpdate(FWorldPartitionActorDesc* InActorDesc);
	void OnBaseContainerActorDescUpdating(FWorldPartitionActorDesc* InActorDesc);
	void OnBaseContainerActorDescUpdated(FWorldPartitionActorDesc* InActorDesc);

	UActorDescContainer* GetBaseContainer() const;

	TSharedPtr<FLevelInstancePropertyOverrideDesc> PropertyOverrideDesc;

	TMap<FActorContainerPath, TMap<FGuid, TUniquePtr<FWorldPartitionActorDesc>>> PerContainerTransientOverrideActorDescs;
#endif
};
