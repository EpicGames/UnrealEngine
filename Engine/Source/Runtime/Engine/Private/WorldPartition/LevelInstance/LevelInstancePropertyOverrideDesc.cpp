// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideDesc.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "LevelUtils.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

class FPropertyOverrideUtils
{
public:
	// When serializing a desc created from an actor of an instanced level
	static void CopyFrom(const FWorldPartitionActorDesc* InSource, FWorldPartitionActorDesc* InDestination)
	{
		check(InSource->GetGuid() == InDestination->GetGuid());
		check(InSource->GetActorNativeClass() == InDestination->GetActorNativeClass());
		InDestination->ActorPackage = InSource->GetActorPackage();
		InDestination->ActorPath = InSource->GetActorSoftPath();
	}

	static void Serialize(FArchive& Ar, FWorldPartitionActorDesc* InActorDesc, const FWorldPartitionActorDesc* InBaseDesc)
	{
		check(InActorDesc);
		check(InBaseDesc);

		FActorDescArchive ActorDescAr(Ar, InActorDesc, InBaseDesc);
		ActorDescAr.Init();

		InActorDesc->Serialize(ActorDescAr);
	}
};

FLevelInstancePropertyOverrideDesc::~FLevelInstancePropertyOverrideDesc()
{
	ActorDescsPerContainer.Empty();

	if (BaseContainer && !IsEngineExitRequested())
	{
		UActorDescContainerSubsystem::GetChecked().UnregisterContainer(BaseContainer);
	}
	BaseContainer = nullptr;
}

void FLevelInstancePropertyOverrideDesc::Init(const ULevelInstancePropertyOverrideAsset* InPropertyOverride)
{
	check(!BaseContainer);
	AssetPath = FSoftObjectPath(InPropertyOverride);
	PackageName = InPropertyOverride->GetPackage()->GetFName();
	WorldAsset = InPropertyOverride->GetWorldAsset().ToSoftObjectPath();
				
	UActorDescContainer* BaseContainerPtr = UActorDescContainerSubsystem::GetChecked().RegisterContainer({ *WorldAsset.GetLongPackageName() });
	BaseContainer = BaseContainerPtr;

	// Only Create new ActorDescs if we are saving a PropertyOverride Edit
	// If we aren't then we will transfer from the previous desc in the next TransferFrom call
	if (InPropertyOverride->bSavingOverrideEdit)
	{
		for (auto const& [ContainerPath, ContainerOverride] : InPropertyOverride->GetPropertyOverridesPerContainer())
		{
			TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>> SavedActorDescs;

			for (auto const& [ActorGuid, ActorOverride] : ContainerOverride.ActorOverrides)
			{
				// We only serialize ActorDescs for Actors that we've just finished overriding
				// Those will have a valid Actor pointer
				// Other Container overrides ActorDescs will be transfered from the previous FLevelInstancePropertyOverrideDesc
				if (AActor* Actor = ActorOverride.Actor.Get(); IsValid(Actor) && Actor->IsPackageExternal())
				{
					ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Actor->GetLevel());
					check(LevelStreaming);

					// Make sure to remove LevelStreaming transform before creating New Actor desc
					check(Actor->GetLevel()->bAlreadyMovedActors);
					FLevelUtils::RemoveEditorTransform(LevelStreaming, false, Actor);

					TSharedPtr<FWorldPartitionActorDesc> NewActorDesc = MakeShareable(Actor->CreateActorDesc().Release());

					const FWorldPartitionActorDesc* BaseDesc = GetBaseDescByGuid(ContainerPath, ActorGuid);
					check(BaseDesc);

					// Make sure to use non instanced Package/ActorPath so copy it from Base desc
					FPropertyOverrideUtils::CopyFrom(BaseDesc, NewActorDesc.Get());

					// Reapply LevelStreaming transform
					FLevelUtils::ApplyEditorTransform(LevelStreaming, false, Actor);

					SavedActorDescs.Add(NewActorDesc->GetGuid(), NewActorDesc);
				}
			}

			if (SavedActorDescs.Num() > 0)
			{
				ActorDescsPerContainer.Add(ContainerPath, MoveTemp(SavedActorDescs));
			}
		}
	}
}

void FLevelInstancePropertyOverrideDesc::TransferNonEditedContainers(const FLevelInstancePropertyOverrideDesc* InExistingOverrideDesc)
{
	check(InExistingOverrideDesc->GetWorldPackage() == GetWorldPackage());
	for (auto const& [ContainerPath, ContainerOverride] : InExistingOverrideDesc->ActorDescsPerContainer)
	{
		// Only transfer exiting overrides for containers that this instance doesn't have yet
		if (!ActorDescsPerContainer.Contains(ContainerPath))
		{
			TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>& ActorDescs = ActorDescsPerContainer.Add(ContainerPath);
			for (auto const& [ActorGuid, ActorDesc] : ContainerOverride)
			{
				if (!ActorDescs.Contains(ActorGuid))
				{
					ActorDescs.Add(ActorGuid, ActorDesc);
					// Reset Container (because ownership of ActorDesc changes)
					ActorDesc->SetContainer(nullptr);
				}
			}
		}
	}
}

void FLevelInstancePropertyOverrideDesc::SetContainerForActorDescs(UActorDescContainer* InContainer)
{
	for (auto& [ContainerPath, ActorDescs] : ActorDescsPerContainer)
	{
		for (auto& [ActorGuid, ActorDesc] : ActorDescs)
		{
			const UActorDescContainer* CurrentContainer = ActorDesc->GetContainer();
			// When updating the Override Desc we might transfer ActorDescs from previous Desc to updated Desc
			check(!CurrentContainer || CurrentContainer == InContainer);
			if (!CurrentContainer)
			{
				ActorDesc->SetContainer(InContainer);
			}
		}
	}
}

FString FLevelInstancePropertyOverrideDesc::GetContainerName() const
{
	return GetContainerNameFromAssetPath(AssetPath);
}

FString FLevelInstancePropertyOverrideDesc::GetContainerNameFromAssetPath(const FSoftObjectPath& InAssetPath)
{
	return InAssetPath.ToString();
}

FString FLevelInstancePropertyOverrideDesc::GetContainerNameFromAsset(ULevelInstancePropertyOverrideAsset* InAsset)
{
	return GetContainerNameFromAssetPath(FSoftObjectPath(InAsset));
}

FWorldPartitionActorDesc* FLevelInstancePropertyOverrideDesc::GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath)
{
	if (const TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>* ActorDescs = ActorDescsPerContainer.Find(InContainerPath))
	{
		if (const TSharedPtr<FWorldPartitionActorDesc>* ActorDesc = ActorDescs->Find(InActorGuid))
		{
			return ActorDesc->Get();
		}
	}

	return nullptr;
}

const FWorldPartitionActorDesc* FLevelInstancePropertyOverrideDesc::GetOverrideActorDesc(const FGuid& InActorGuid, const FActorContainerPath& InContainerPath) const
{
	return const_cast<FLevelInstancePropertyOverrideDesc*>(this)->GetOverrideActorDesc(InActorGuid, InContainerPath);
}

const UActorDescContainer* FLevelInstancePropertyOverrideDesc::GetBaseContainer(const UActorDescContainer* InContainer, const FActorContainerPath& InContainerPath) const
{
	check(InContainer);

	const UActorDescContainer* Container = InContainer;
	for (const FGuid& ContainerGuid : InContainerPath.ContainerGuids)
	{
		const FWorldPartitionActorDesc* ContainerDesc = Container->GetActorDesc(ContainerGuid);
		if (!ContainerDesc)
		{
			return nullptr;
		}

		Container = ContainerDesc->GetChildContainer();
		if (!Container)
		{
			return nullptr;
		}
	}

	return Container;
}

const FWorldPartitionActorDesc* FLevelInstancePropertyOverrideDesc::GetBaseDescByGuid(const FActorContainerPath& InContainerPath, const FGuid& InActorGuid) const
{
	check(BaseContainer);
	
	if (const UActorDescContainer* Container = GetBaseContainer(BaseContainer, InContainerPath))
	{
		return Container->GetActorDesc(InActorGuid);
	}

	return nullptr;
}

void FLevelInstancePropertyOverrideDesc::SerializeTo(TArray<uint8>& OutPayload)
{
	check(BaseContainer);

	TArray<uint8> UnversionedPayloadData;
	FMemoryWriter MemoryWriter(UnversionedPayloadData, true);
		
	FString WorldAssetStr = WorldAsset.ToString();
	MemoryWriter << WorldAssetStr;

	int32 ContainerCount = ActorDescsPerContainer.Num();
	MemoryWriter << ContainerCount;

	// Serialize ActorDescs for overriden actors
	for (const auto& [ContainerPath, ActorDescs] : ActorDescsPerContainer)
	{
		// Serialize Actor Editor Path
		MemoryWriter << const_cast<FActorContainerPath&>(ContainerPath);

		int32 ActorDescCount = ActorDescs.Num();
		MemoryWriter << ActorDescCount;

		for (const auto& [ActorGuid, ActorDesc] : ActorDescs)
		{
			FGuid CopyGuid = ActorGuid;
			MemoryWriter << CopyGuid;

			// Serialize class so that we can deserialize data even if actor no longer exists
			FString NativeClass = ActorDesc->GetNativeClass().ToString();
			MemoryWriter << NativeClass;
						
			const FWorldPartitionActorDesc* BaseDesc = GetBaseDescByGuid(ContainerPath, ActorGuid);
			FPropertyOverrideUtils::Serialize(MemoryWriter, ActorDesc.Get(), BaseDesc);
		}
	}
		
	// Serialize custom versions
	TArray<uint8> HeaderData;
	FMemoryWriter HeaderAr(HeaderData);
	FCustomVersionContainer CustomVersions = MemoryWriter.GetCustomVersions();
	CustomVersions.Serialize(HeaderAr);

	// Append data
	OutPayload = MoveTemp(HeaderData);
	OutPayload.Append(UnversionedPayloadData);
}

void FLevelInstancePropertyOverrideDesc::SerializeFrom(const TArray<uint8>& InPayload)
{
	// Serialize actor metadata
	FMemoryReader MemoryReader(InPayload, true);

	// Serialize metadata custom versions
	FCustomVersionContainer CustomVersions;
	CustomVersions.Serialize(MemoryReader);
	MemoryReader.SetCustomVersions(CustomVersions);
		
	FString WorldAssetStr;
	MemoryReader << WorldAssetStr;
	WorldAsset = FSoftObjectPath(WorldAssetStr);
	UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldAsset);
		
	check(!BaseContainer);
	BaseContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer({ *WorldAsset.GetLongPackageName() });
	
	int32 ContainerCount = 0;
	MemoryReader << ContainerCount;

	for (int32 i = 0; i < ContainerCount; ++i)
	{
		FActorContainerPath ContainerPath;
		MemoryReader << ContainerPath;

		TMap<FGuid, TSharedPtr<FWorldPartitionActorDesc>>& ActorDescs = ActorDescsPerContainer.Add(ContainerPath);

		int32 ActorDescCount = 0;
		MemoryReader << ActorDescCount;

		for (int32 j = 0; j < ActorDescCount; ++j)
		{
			FGuid ActorGuid;
			MemoryReader << ActorGuid;

			FString NativeClass;
			MemoryReader << NativeClass;
						
			UClass* NativeClassPtr = FWorldPartitionActorDescUtils::GetActorNativeClassFromString(NativeClass);
			
			const FWorldPartitionActorDesc* BaseDesc = GetBaseDescByGuid(ContainerPath, ActorGuid);
			
			const bool bValidOverride = BaseDesc && BaseDesc->GetActorNativeClass() == NativeClassPtr;
									
			TSharedPtr<FWorldPartitionActorDesc> NewActorDesc = MakeShareable(AActor::StaticCreateClassActorDesc(NativeClassPtr ? NativeClassPtr : AActor::StaticClass()).Release());
			FActorDescArchive ActorDescArchive(MemoryReader, NewActorDesc.Get(), bValidOverride ? BaseDesc : nullptr);
			ActorDescArchive.Init();
			
			FWorldPartitionActorDescInitData ActorDescInitData = FWorldPartitionActorDescInitData(&ActorDescArchive)
				.SetNativeClass(NativeClassPtr);
			if (bValidOverride)
			{
				ActorDescInitData.SetPackageName(BaseDesc->GetActorPackage()).SetActorPath(BaseDesc->GetActorSoftPath());
			}
			NewActorDesc->Init(ActorDescInitData);

			if (bValidOverride)
			{
				ActorDescs.Add(NewActorDesc->GetGuid(), NewActorDesc);
			}
		}
	}
}

FArchive& operator<<(FArchive& Ar, FLevelInstancePropertyOverrideDesc& InPropertyOverrideDesc)
{
	int32 PayloadSize = 0;
	TArray<uint8> Payload;

	if (Ar.IsSaving())
	{
		InPropertyOverrideDesc.SerializeTo(Payload);
		PayloadSize = Payload.Num();
	}

	Ar << PayloadSize;
	
	if (Ar.IsLoading())
	{
		Payload.SetNumUninitialized(PayloadSize);
	}
	
	Ar.Serialize(Payload.GetData(), PayloadSize);
	
	if (Ar.IsLoading())
	{
		InPropertyOverrideDesc.SerializeFrom(Payload);
	}
			
	return Ar;
}

#endif