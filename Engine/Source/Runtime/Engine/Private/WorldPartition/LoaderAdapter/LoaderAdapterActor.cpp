// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterActor.h"
#include "GameFramework/Actor.h"

#if WITH_EDITOR
FLoaderAdapterActor::FLoaderAdapterActor(AActor* InActor)
	: ILoaderAdapterSpatial(InActor->GetWorld())
	, Actor(InActor)
{
	SetUserCreated(true);
}

TOptional<FBox> FLoaderAdapterActor::GetBoundingBox() const
{
	return Actor->GetComponentsBoundingBox(/*bNonColliding*/true);
}

TOptional<FString> FLoaderAdapterActor::GetLabel() const
{
	return Actor->GetActorNameOrLabel();
}

bool FLoaderAdapterActor::Intersect(const FBox& Box) const
{
	// we'll always intersect until we start handling custom brushes
	return true;
}
#endif
