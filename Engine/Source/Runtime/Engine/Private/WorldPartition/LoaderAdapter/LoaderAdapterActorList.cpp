// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterActorList.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#if WITH_EDITOR
FLoaderAdapterActorList::FLoaderAdapterActorList(UWorld* InWorld)
	: FLoaderAdapterList(InWorld)
{
	Load();
}

void FLoaderAdapterActorList::AddActors(const TArray<FGuid>& ActorGuids)
{
	if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
	{
		TArray<FWorldPartitionHandle> ActorHandles;
		ActorHandles.Reserve(ActorGuids.Num());

		for (const FGuid& ActorGuid : ActorGuids)
		{
			FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid);
			if (ActorHandle.IsValid())
			{
				ActorHandles.Add(ActorHandle);
			}
		}

		AddActors(ActorHandles);
	}
}

void FLoaderAdapterActorList::AddActors(const TArray<FWorldPartitionHandle>& ActorHandles)
{
	Actors.Append(ActorHandles);
	RefreshLoadedState();
}

void FLoaderAdapterActorList::RemoveActors(const TArray<FGuid>& ActorGuids)
{
	if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
	{
		TArray<FWorldPartitionHandle> ActorHandles;
		ActorHandles.Reserve(ActorGuids.Num());

		for (const FGuid& ActorGuid : ActorGuids)
		{
			FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid);
			if (ActorHandle.IsValid())
			{
				ActorHandles.Add(ActorHandle);
			}
		}

		RemoveActors(ActorHandles);
	}
}

void FLoaderAdapterActorList::RemoveActors(const TArray<FWorldPartitionHandle>& ActorHandles)
{
	ActorsToRemove = TSet<FWorldPartitionHandle>(ActorHandles);

	for (const FWorldPartitionHandle& ActorHandle : ActorHandles)
	{
		if (ActorHandle->IsChildContainerInstance())
		{
			if (UWorldPartition* ContainerWorldPartition = GetLoadedChildWorldPartition(ActorHandle); ContainerWorldPartition && ContainerWorldPartition->IsStreamingEnabledInEditor())
			{
				for (FActorDescContainerInstanceCollection::TIterator<> Iterator(ContainerWorldPartition); Iterator; ++Iterator)
				{
					FWorldPartitionHandle SubActorHandle(ContainerWorldPartition, Iterator->GetGuid());
					ActorsToRemove.Add(SubActorHandle);
				}
			}
		}
	}

	RefreshLoadedState();
	Actors = Actors.Difference(ActorsToRemove);
	ActorsToRemove.Empty();
}

bool FLoaderAdapterActorList::ContainsActor(const FGuid& ActorGuid) const
{
	if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
	{
		FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid);
		if(ActorHandle.IsValid())
		{
			return ContainsActor(ActorHandle);
		}
	}
	return false;
}

bool FLoaderAdapterActorList::ContainsActor(const FWorldPartitionHandle& ActorHandle) const
{
	return Actors.Contains(ActorHandle);
}

bool FLoaderAdapterActorList::PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const
{
	return ActorHandle.IsValid() && !ActorsToRemove.Contains(ActorHandle) && FLoaderAdapterList::PassActorDescFilter(ActorHandle);
}
#endif
