// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterList.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Engine/Level.h"

#if WITH_EDITOR
FLoaderAdapterList::FLoaderAdapterList(UWorld* InWorld)
	: ILoaderAdapter(InWorld)
{}

void FLoaderAdapterList::ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const
{
	for (const FWorldPartitionHandle& Actor : Actors)
	{
		if (Actor.IsValid())
		{
			InOperation(Actor);
			HandleActorContainer(Actor, InOperation);
		}
	}
}

void FLoaderAdapterList::HandleActorContainer(const FWorldPartitionHandle& InActor, TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const
{
	if (InActor->IsChildContainerInstance())
	{
		if (UWorldPartition* ContainerWorldPartition = GetLoadedChildWorldPartition(InActor); ContainerWorldPartition && ContainerWorldPartition->IsStreamingEnabledInEditor())
		{
			for (FActorDescContainerInstanceCollection::TIterator<> Iterator(ContainerWorldPartition); Iterator; ++Iterator)
			{
				FWorldPartitionHandle ActorHandle(ContainerWorldPartition, Iterator->GetGuid());
				InOperation(ActorHandle);
				HandleActorContainer(ActorHandle, InOperation);
			}
		}
	}
}
#endif
