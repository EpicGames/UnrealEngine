// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterPinnedActors.h"

#if WITH_EDITOR

#include "Engine/World.h"
#include "Engine/Level.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#define LOCTEXT_NAMESPACE "FLoaderAdapterPinnedActors"

namespace LoaderAdapterPinnedActorsUtils
{
	static bool SupportsPinning(FWorldPartitionActorDescInstance* InActorDescInstance, bool bCheckIsMainWorldPartition)
	{
		if (!InActorDescInstance)
		{
			return false;
		}

		// Only Spatially loaded actors can be pinned with the exception of non spatially loaded runtime only actors (ex: HLODs)
		if (!InActorDescInstance->GetIsSpatiallyLoaded() && !InActorDescInstance->GetActorIsRuntimeOnly())
		{
			return false;
		}

		// This allows skipping actors that can never be loaded in current context: ex: bActorShouldSkipLevelInstance
		if (!InActorDescInstance->IsEditorRelevant() && !InActorDescInstance->GetActorIsRuntimeOnly())
		{
			return false;
		}

		if (UActorDescContainerInstance* ContainerInstance = InActorDescInstance->GetContainerInstance())
		{
			const UWorldPartition* ContainerWorldPartition = ContainerInstance->GetOuterWorldPartition();
			return ContainerWorldPartition && (ContainerWorldPartition->IsMainWorldPartition() || !bCheckIsMainWorldPartition);
		}

		return false;
	}
}

bool FLoaderAdapterPinnedActors::PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const
{
	// We want to be able to pin any type of actors (HLODs, etc).
	// Allow recursive pinning by setting bCheckIsMainWorldPartition = false
	const bool bCheckIsMainWorldPartition = false;
	return ActorHandle.IsValid() && !ActorsToRemove.Contains(ActorHandle) && LoaderAdapterPinnedActorsUtils::SupportsPinning(*ActorHandle, bCheckIsMainWorldPartition);
}

bool FLoaderAdapterPinnedActors::SupportsPinning(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	// Public api doesn't allow pinning of non main world partition actors
	const bool bCheckIsMainWorldPartition = true;
	return LoaderAdapterPinnedActorsUtils::SupportsPinning(InActorDescInstance, bCheckIsMainWorldPartition);
}

bool FLoaderAdapterPinnedActors::SupportsPinning(AActor* InActor)
{
	if (InActor)
	{
		// Pinning of Actors is only supported on the main world partition
		const ULevel* Level = InActor->GetLevel();
		const UWorld* World = Level->GetWorld();

		// Only Spatially loaded actors can be pinned with the exception of non spatially loaded, runtime only actors (ex: HLODs)
		return World && !World->IsGameWorld() && !!World->GetWorldPartition() && Level->IsPersistentLevel() && InActor->IsPackageExternal() && (InActor->GetIsSpatiallyLoaded() || InActor->IsRuntimeOnly());
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

#endif
