// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"

#if WITH_EDITOR
FLoaderAdapterShape::FLoaderAdapterShape(UWorld* InWorld, const FBox& InBoundingBox, const FString& InLabel)
	: ILoaderAdapterSpatial(InWorld)
	, BoundingBox(InBoundingBox)
	, Label(InLabel)
{
	ensureMsgf(BoundingBox.IsValid, TEXT("Invalid FLoaderAdapterShape: %s"), *InLabel);
}

TOptional<FBox> FLoaderAdapterShape::GetBoundingBox() const
{
	return BoundingBox.IsValid? BoundingBox : TOptional<FBox>();
}

TOptional<FString> FLoaderAdapterShape::GetLabel() const
{
	return Label;
}

bool FLoaderAdapterShape::Intersect(const FBox& Box) const
{
	return BoundingBox.Intersect(Box);
}
#endif
