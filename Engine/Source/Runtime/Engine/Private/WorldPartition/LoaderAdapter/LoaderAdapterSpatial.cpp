// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/LoaderAdapter/LoaderAdapterSpatial.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#if WITH_EDITOR
ILoaderAdapterSpatial::ILoaderAdapterSpatial(UWorld* InWorld)
	: ILoaderAdapter(InWorld)
	, bIncludeSpatiallyLoadedActors(true)
	, bIncludeNonSpatiallyLoadedActors(false)
{}

void ILoaderAdapterSpatial::ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const
{
	if (GetBoundingBox().IsSet())
	{
		if (UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition())
		{
			HandleIntersectingContainer(WorldPartition, *GetBoundingBox(), InOperation);
		}
	}
}

void ILoaderAdapterSpatial::HandleIntersectingContainer(UWorldPartition* InWorldPartition, const FBox& InBoundingBox, TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const
{
	UWorldPartitionEditorHash::FForEachIntersectingActorParams ForEachIntersectingActorParams = UWorldPartitionEditorHash::FForEachIntersectingActorParams()
		.SetIncludeSpatiallyLoadedActors(bIncludeSpatiallyLoadedActors)
		.SetIncludeNonSpatiallyLoadedActors(bIncludeNonSpatiallyLoadedActors);

	const FTransform InstanceTransform = InWorldPartition->GetInstanceTransform();
	const FBox LocalBoundingBox = InBoundingBox.InverseTransformBy(InstanceTransform);
	InWorldPartition->EditorHash->ForEachIntersectingActor(LocalBoundingBox, [this, InWorldPartition, &InstanceTransform, &InOperation](FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		const FBox WorldActorEditorBounds = ActorDescInstance->GetLocalEditorBounds().TransformBy(InstanceTransform);
		if (Intersect(WorldActorEditorBounds))
		{
			FWorldPartitionHandle ActorHandle(InWorldPartition, ActorDescInstance->GetGuid());
			InOperation(ActorHandle);

			if (ActorDescInstance->GetIsSpatiallyLoaded() && ActorDescInstance->IsChildContainerInstance())
			{
				if (UWorldPartition* ContainerWorldPartition = GetLoadedChildWorldPartition(ActorHandle))
				{
					const FBox InnerBoundingBox = ContainerWorldPartition->IsStreamingEnabledInEditor() ? *GetBoundingBox() : FBox(FVector(-HALF_WORLD_MAX), FVector(HALF_WORLD_MAX));
					HandleIntersectingContainer(ContainerWorldPartition, InnerBoundingBox, InOperation);
				}
			}
		}
	}, ForEachIntersectingActorParams);
}

#endif
