// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/NavigationData/NavigationDataChunkActor.h"
#include "AI/NavigationSystemBase.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationDataChunkActor)

#if WITH_EDITOR
#include "Editor.h"
#endif

ANavigationDataChunkActor::ANavigationDataChunkActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);
	SetActorEnableCollision(false);
}

#if WITH_EDITOR
void ANavigationDataChunkActor::PostLoad()
{
	Super::PostLoad();

	if (UWorld* World = GetWorld())
	{
		if (UNavigationSystemBase* NavSys = World->GetNavigationSystem())
		{
			if (NavSys->IsWorldInitDone())
			{
				AddNavigationDataChunkInEditor(*NavSys);
			}
			else
			{
				UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().AddUObject(this, &ANavigationDataChunkActor::AddNavigationDataChunkInEditor);
			}
		}
	}
}

void ANavigationDataChunkActor::BeginDestroy()
{
	UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().RemoveAll(this);
	
	if (GEditor)
	{
		const bool bIsInPIE = (GEditor->PlayWorld != nullptr) && (!GEditor->bIsSimulatingInEditor);
		if (!bIsInPIE)
		{
			Log(ANSI_TO_TCHAR(__FUNCTION__));
			RemoveNavigationDataChunkFromWorld();
		}
	}

	Super::BeginDestroy();
}

uint32 ANavigationDataChunkActor::GetDefaultGridSize(UWorld* InWorld) const
{
	return InWorld->GetWorldSettings()->NavigationDataChunkGridSize;
}

void ANavigationDataChunkActor::AddNavigationDataChunkInEditor(const UNavigationSystemBase& NavSys)
{
	if (GEditor)
	{
		const bool bIsInPIE = (GEditor->PlayWorld != nullptr) && (!GEditor->bIsSimulatingInEditor);
		if (!bIsInPIE)
		{
			if (NavSys.GetWorld() == GetWorld())
			{
				Log(ANSI_TO_TCHAR(__FUNCTION__));
				UE_LOGF(LogNavigation, Verbose, "   pos: %ls ext: %ls", *DataChunkActorBounds.GetCenter().ToCompactString(), *DataChunkActorBounds.GetExtent().ToCompactString());
				AddNavigationDataChunkToWorld();
			}
		}
	}
}
#endif // WITH_EDITOR

void ANavigationDataChunkActor::CollectNavData(const FBox& QueryBounds, FBox& OutTilesBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ANavigationDataChunkActor::CollectNavData);
	Log(ANSI_TO_TCHAR(__FUNCTION__));

	if (UWorld* World = GetWorld())
	{
		if (UNavigationSystemBase* NavSys = World->GetNavigationSystem())
		{
			NavSys->FillNavigationDataChunkActor(QueryBounds, *this, OutTilesBounds);
		}
	}
}

#if WITH_EDITOR
void ANavigationDataChunkActor::SetDataChunkActorBounds(const FBox& InBounds)
{
	DataChunkActorBounds = InBounds;
}
#endif //WITH_EDITOR

void ANavigationDataChunkActor::BeginPlay()
{
	Log(ANSI_TO_TCHAR(__FUNCTION__));
	Super::BeginPlay();
	AddNavigationDataChunkToWorld();
}

void ANavigationDataChunkActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Log(ANSI_TO_TCHAR(__FUNCTION__));
	RemoveNavigationDataChunkFromWorld();
	Super::EndPlay(EndPlayReason);
}

void ANavigationDataChunkActor::AddNavigationDataChunkToWorld()
{
	if (UWorld* World = GetWorld())
	{
		if (UNavigationSystemBase* NavSys = World->GetNavigationSystem())
		{
			NavSys->AddNavigationDataChunk(*this);
		}
	}
}

void ANavigationDataChunkActor::RemoveNavigationDataChunkFromWorld()
{
	if (UWorld* World = GetWorld())
	{
		if (UNavigationSystemBase* NavSys = World->GetNavigationSystem())
		{
			NavSys->RemoveNavigationDataChunk(*this);
		}
	}
}

void ANavigationDataChunkActor::Log(const TCHAR* FunctionName) const
{
	UE_LOGF(LogNavigation, Verbose, "[%ls] %ls", *GetName(), FunctionName);
}

void ANavigationDataChunkActor::GetActorBounds(bool bOnlyCollidingComponents, FVector& OutOrigin, FVector& OutBoxExtent, bool bIncludeFromChildActors) const
{
	DataChunkActorBounds.GetCenterAndExtents(OutOrigin, OutBoxExtent);
}

#if WITH_EDITOR
void ANavigationDataChunkActor::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	OutRuntimeBounds = OutEditorBounds = DataChunkActorBounds;
}
#endif // WITH_EDITOR

