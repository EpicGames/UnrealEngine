// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/PackedLevelActor/PackedLevelActorDesc.h"

#if WITH_EDITOR
#include "PackedLevelActor/PackedLevelActor.h"
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

void FPackedLevelActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const APackedLevelActor* PackedLevelActor = CastChecked<APackedLevelActor>(InActor);
	WorldAsset = PackedLevelActor->GetWorldAsset().ToSoftObjectPath();
	ContainerFilter = PackedLevelActor->GetFilter();
}

void FPackedLevelActorDesc::Init(const FWorldPartitionActorDescInitData& DescData)
{
	FWorldPartitionActorDesc::Init(DescData);
}

bool FPackedLevelActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FPackedLevelActorDesc* PackedLevelActorDesc = (FPackedLevelActorDesc*)Other;

		return WorldAsset == PackedLevelActorDesc->WorldAsset &&
			ContainerFilter == PackedLevelActorDesc->ContainerFilter;
	}

	return false;
}

void FPackedLevelActorDesc::Serialize(FArchive& Ar)
{
	FWorldPartitionActorDesc::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::PackedLevelActorDesc)
	{
		Ar << TDeltaSerialize<FSoftObjectPath>(WorldAsset);
		Ar << TDeltaSerialize<FWorldPartitionActorFilter>(ContainerFilter);
	}
}

#endif
