// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimePartition)

URuntimePartition::URuntimePartition(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	SetDefaultValues();
	
	// Keep the original BoundsMethod by default to maintain backward compatibility
	BoundsMethod = ERuntimePartitionCellBoundsMethod::UseMinContentCellBounds;
#endif
}

#if WITH_EDITOR
void URuntimePartition::SetDefaultValues()
{
	bBlockOnSlowStreaming = false;
	bClientOnlyVisible = false;
	Priority = 0;
	BoundsMethod = ERuntimePartitionCellBoundsMethod::UseContent;
	LoadingRange = 25600;
	DebugColor = FLinearColor::MakeRandomSeededColor(GetTypeHash(GetName()));
	HLODIndex = INDEX_NONE;
}

void URuntimePartition::InitHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition, int32 InHLODIndex)
{
	LoadingRange = InRuntimePartition->LoadingRange * 2;
	HLODIndex = InHLODIndex;
}

void URuntimePartition::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(URuntimePartition, LoadingRange))
	{
		LoadingRange = FMath::Max(LoadingRange, 0);
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

URuntimePartition* URuntimePartition::CreateHLODRuntimePartition(int32 InHLODIndex) const
{
	URuntimePartition* HLODRuntimePartition = DuplicateObject<URuntimePartition>(this, GetOuter());
	HLODRuntimePartition->InitHLODRuntimePartitionFrom(this, InHLODIndex);
	return HLODRuntimePartition;
}

URuntimePartition::FCellDesc URuntimePartition::CreateCellDesc(const FString& InName, bool bInIsSpatiallyLoaded, int32 InLevel, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& InActorSetInstances)
{
	FCellDesc CellDesc;

	// Construct a unique name like this: PartitionName_CellName
	TStringBuilder<512> StringBuilder;
	StringBuilder += Name.ToString();
	StringBuilder += TEXT("_");
	StringBuilder += InName;
	CellDesc.Name = *StringBuilder;

	// Copy values coming from this partition
	CellDesc.bBlockOnSlowStreaming = bBlockOnSlowStreaming;
	CellDesc.bClientOnlyVisible = bClientOnlyVisible;
	CellDesc.Priority = Priority;

	// Set provided input values
	CellDesc.bIsSpatiallyLoaded = bInIsSpatiallyLoaded;
	CellDesc.Level = InLevel;

	// Add actor set instances
	CellDesc.ActorSetInstances = InActorSetInstances;

	return CellDesc;
}

URuntimePartition::FCellDescInstance::FCellDescInstance(const FCellDesc& InCellDesc, URuntimePartition* InSourcePartition, const TArray<const UDataLayerInstance*>& InDataLayerInstances, const FGuid& InContentBundleID)
	: FCellDesc(InCellDesc)
	, SourcePartition(InSourcePartition)
	, DataLayerInstances(InDataLayerInstances)
	, ContentBundleID(InContentBundleID)
{
	bClientOnlyVisible = InCellDesc.bClientOnlyVisible || FDataLayerUtils::IsClientOnlyFromDataLayers(DataLayerInstances);
}

#endif
