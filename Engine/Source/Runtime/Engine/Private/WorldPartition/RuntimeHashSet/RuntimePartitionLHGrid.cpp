// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/RuntimePartitionLHGrid.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "Misc/HashBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimePartitionLHGrid)

#if WITH_EDITOR
struct FCellCoord
{
	FCellCoord(int64 InX, int64 InY, int64 InZ, int32 InLevel)
		: X(InX)
		, Y(InY)
		, Z(InZ)
		, Level(InLevel)
	{}

	static FCellCoord Invalid;

	inline FString ToString2D() const
	{
		check(!Z);
		return FString::Printf(TEXT("L%d_X%" INT64_FMT "_Y%" INT64_FMT), Level, X, Y);
	}

	inline FString ToString3D() const
	{
		return FString::Printf(TEXT("L%d_X%" INT64_FMT "_Y%" INT64_FMT "_Z%" INT64_FMT), Level, X, Y, Z);
	}

	inline bool operator==(const FCellCoord& Other) const
	{
		return (X == Other.X) && (Y == Other.Y) && (Z == Other.Z) && (Level == Other.Level);
	}

	static inline int32 GetLevelForBox(const FBox& InBox, int32 InCellSize, const FVector& InOrigin)
	{
		const FVector Extent = InBox.GetSize();
		const FVector::FReal MaxLength = Extent.GetMax();
		const int32 Level = FMath::CeilToInt32(FMath::Max<FVector::FReal>(FMath::Log2(MaxLength / InCellSize), 0));

		if (Level)
		{
			const FCellCoord CellCoord = GetCellCoords(InBox.GetCenter(), InCellSize, Level - 1, InOrigin);
			const FBox CellBounds = GetCellBounds(CellCoord, InCellSize, InOrigin);
			const FVector MaxUnderLap = FVector::Max3(CellBounds.Min - InBox.Min, InBox.Max - CellBounds.Max, FVector::Zero());
			const FVector::FReal MaxUnderLapLength = MaxUnderLap.GetMax();

			// Allow objects that slightly exceed the cell size to be placed in the lower levels. We don't want large objects to
			// grow cells to their maximum extent, which is half the cell size on each axis, so we allow a quarter on each axis.
			if (MaxUnderLapLength < InCellSize / 4)
			{
				return Level - 1;
			}
		}

		return Level;
	}

	static inline FCellCoord GetCellCoords(const FVector& InPos, int32 InCellSize, int32 InLevel, const FVector& InOrigin)
	{
		check(InLevel >= 0);
		const int64 CellSizeForLevel = (int64)InCellSize * (1LL << InLevel);
		return FCellCoord(
			FMath::FloorToInt((InPos.X - InOrigin.X) / CellSizeForLevel),
			FMath::FloorToInt((InPos.Y - InOrigin.Y) / CellSizeForLevel),
			FMath::FloorToInt((InPos.Z - InOrigin.Z) / CellSizeForLevel),
			InLevel
		);
	}

	static inline FBox GetCellBounds(const FCellCoord& InCellCoord, int32 InCellSize, const FVector& InOrigin)
	{
		check(InCellCoord.Level >= 0);
		const int64 CellSizeForLevel = (int64)InCellSize * (1LL << InCellCoord.Level);
		const FVector Min = InOrigin + FVector(
			static_cast<FVector::FReal>(InCellCoord.X * CellSizeForLevel), 
			static_cast<FVector::FReal>(InCellCoord.Y * CellSizeForLevel), 
			static_cast<FVector::FReal>(InCellCoord.Z * CellSizeForLevel)
		);
		const FVector Max = Min + FVector(static_cast<double>(CellSizeForLevel));
		return FBox(Min, Max);
	}

	friend uint32 GetTypeHash(const FCellCoord& CellCoord)
	{
		FHashBuilder HashBuilder;
		HashBuilder << CellCoord.X << CellCoord.Y << CellCoord.Z << CellCoord.Level;
		return HashBuilder.GetHash();
	}

	int64 X;
	int64 Y;
	int64 Z;
	int32 Level;
};

FCellCoord FCellCoord::Invalid(0, 0, 0, -1);

bool URuntimePartitionLHGrid::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FString PropertyName = InProperty->GetName();
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(URuntimePartitionLHGrid, bIs2D))
		{
			return HLODIndex == INDEX_NONE;
		}
	}

	return Super::CanEditChange(InProperty);
}

static bool GPackageWasDirty = false;
void URuntimePartitionLHGrid::PreEditChange(FProperty* InPropertyAboutToChange)
{
	GPackageWasDirty = GetPackage()->IsDirty();
	Super::PreEditChange(InPropertyAboutToChange);
}

void URuntimePartitionLHGrid::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(URuntimePartitionLHGrid, CellSize))
	{
		CellSize = FMath::Max<int32>(CellSize, 1600);
	}
	else if ((PropertyName == GET_MEMBER_NAME_CHECKED(URuntimePartitionLHGrid, bShowGridPreview)) || 
			 (PropertyName == GET_MEMBER_NAME_CHECKED(URuntimePartitionLHGrid, bIs2D)))
	{
		WorldGridPreviewer.Reset();

		if (bShowGridPreview)
		{
			WorldGridPreviewer = MakeUnique<FWorldGridPreviewer>(GetTypedOuter<UWorld>(), bIs2D);
		}

		if ((PropertyName == GET_MEMBER_NAME_CHECKED(URuntimePartitionLHGrid, bShowGridPreview)) && !GPackageWasDirty)
		{
			GetPackage()->ClearDirtyFlag();
		}
	}

	if (WorldGridPreviewer)
	{
		WorldGridPreviewer->CellSize = CellSize;
		WorldGridPreviewer->GridColor = DebugColor;
		WorldGridPreviewer->GridOffset = Origin;
		WorldGridPreviewer->LoadingRange = LoadingRange;
		WorldGridPreviewer->Update();
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

void URuntimePartitionLHGrid::InitHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition, int32 InHLODIndex)
{
	Super::InitHLODRuntimePartitionFrom(InRuntimePartition, InHLODIndex);
	const URuntimePartitionLHGrid* RuntimePartitionLHGrid = CastChecked<const URuntimePartitionLHGrid>(InRuntimePartition);
	CellSize = RuntimePartitionLHGrid->CellSize * 2;
	bIs2D = RuntimePartitionLHGrid->bIs2D;
	Origin = RuntimePartitionLHGrid->Origin;
}

void URuntimePartitionLHGrid::UpdateHLODRuntimePartitionFrom(const URuntimePartition* InRuntimePartition)
{
	Super::UpdateHLODRuntimePartitionFrom(InRuntimePartition);
	const URuntimePartitionLHGrid* RuntimePartitionLHGrid = CastChecked<const URuntimePartitionLHGrid>(InRuntimePartition);
	bIs2D = RuntimePartitionLHGrid->bIs2D;
	Origin = RuntimePartitionLHGrid->Origin;
}

void URuntimePartitionLHGrid::SetDefaultValues()
{
	Super::SetDefaultValues();
	CellSize = LoadingRange / 2;
	bIs2D = true;
}
#endif

bool URuntimePartitionLHGrid::IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const
{
	return InPartitionTokens.Num() == 1;
}

#if WITH_EDITOR
bool URuntimePartitionLHGrid::GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult)
{
	UWorldPartition* WorldPartition = GetTypedOuter<UWorldPartition>();
	UWorld* World = WorldPartition->GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const bool bIsMainWorldPartition = (World == OuterWorld);

	TMap<FCellCoord, TArray<const IStreamingGenerationContext::FActorSetInstance*>> CellsActorSetInstances;
	for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : *InParams.ActorSetInstances)
	{
		if (ActorSetInstance->bIsSpatiallyLoaded)
		{
			FBox ActorSetInstanceBounds = ActorSetInstance->Bounds;

			// Ensure cached bounds are in 2D so as to not affect CellCoords.
			if (bIs2D)
			{
				ActorSetInstanceBounds.Min.Z = 0;
				ActorSetInstanceBounds.Max.Z = 0;
			}

			const int32 GridLevel = FCellCoord::GetLevelForBox(ActorSetInstanceBounds, CellSize, FVector(Origin.X, Origin.Y, bIs2D ? 0 : Origin.Z));
			const FCellCoord CellCoord = FCellCoord::GetCellCoords(ActorSetInstanceBounds.GetCenter(), CellSize, GridLevel, FVector(Origin.X, Origin.Y, bIs2D ? 0 : Origin.Z));
			CellsActorSetInstances.FindOrAdd(CellCoord).Add(ActorSetInstance);
		}
		else
		{
			CellsActorSetInstances.FindOrAdd(FCellCoord::Invalid).Add(ActorSetInstance);
		}
	}

	for (auto& [CellCoord, CellActorSetInstances] : CellsActorSetInstances)
	{
		const bool bIsSpatiallyLoaded = CellCoord != FCellCoord::Invalid;

		URuntimePartition::FCellDesc& CellDesc = OutResult.RuntimeCellDescs.Emplace_GetRef(CreateCellDesc(bIs2D ? CellCoord.ToString2D() : CellCoord.ToString3D(), bIsSpatiallyLoaded, CellCoord.Level, CellActorSetInstances));

		if (bIsSpatiallyLoaded)
		{
			CellDesc.CellBounds = FCellCoord::GetCellBounds(CellCoord, CellSize, FVector(Origin.X, Origin.Y, bIs2D ? 0 : Origin.Z));

			// Ensure cell bounds are extended to max in Z so as to include all content regardless of Z.
			if (bIs2D)
			{
				CellDesc.CellBounds.GetValue().Min.Z = -HALF_WORLD_MAX;
				CellDesc.CellBounds.GetValue().Max.Z = HALF_WORLD_MAX;
			}
		}

		CellDesc.bIs2D = bIsSpatiallyLoaded && bIs2D;
	}

	return true;
}

FArchive& URuntimePartitionLHGrid::AppendCellGuid(FArchive& InAr)
{
	return Super::AppendCellGuid(InAr) << CellSize;
}

TOptional<FFixedGridInfo> URuntimePartitionLHGrid::GetFixedGridInfo() const
{
	FFixedGridInfo Info;
	Info.CellSize = CellSize;
	Info.Origin = Origin;
	Info.bIs2D = bIs2D;

	return Info;
}
#endif
