// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/RuntimePartitionLevelStreaming.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimePartitionLevelStreaming)

bool URuntimePartitionLevelStreaming::IsValidPartitionTokens(const TArray<FName>& InPartitionTokens) const
{
	return InPartitionTokens.Num() && (InPartitionTokens.Num() <= 2);
}

#if WITH_EDITOR
bool URuntimePartitionLevelStreaming::GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult)
{
	UWorldPartition* WorldPartition = GetTypedOuter<UWorldPartition>();
	UWorld* World = WorldPartition->GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const bool bIsMainWorldPartition = (World == OuterWorld);

	TMap<FName, TArray<const IStreamingGenerationContext::FActorSetInstance*>> CellsActorSetInstances;
	for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : *InParams.ActorSetInstances)
	{
		FName LevelName = NAME_Default;
		if (!ActorSetInstance->RuntimeGrid.IsNone())
		{
			TArray<FName> MainPartitionTokens;
			TArray<FName> HLODPartitionTokens;
			if (UWorldPartitionRuntimeHashSet::ParseGridName(ActorSetInstance->RuntimeGrid, MainPartitionTokens, HLODPartitionTokens))
			{
				if (MainPartitionTokens.Num() == 2)
				{
					LevelName = MainPartitionTokens[1];
				}
			}
		}

		FName CellName;
		if (ActorSetInstance->bIsSpatiallyLoaded)
		{
			TStringBuilder<512> StringBuilder;
			StringBuilder += Name.ToString();
			StringBuilder += TEXT("_");
			StringBuilder += LevelName.ToString();
			CellName = *StringBuilder;
		}
		else
		{
			CellName = NAME_PersistentLevel;
		}

		CellsActorSetInstances.FindOrAdd(CellName).Add(ActorSetInstance);
	}

	for (auto& [CellName, CellActorSetInstances] : CellsActorSetInstances)
	{
		OutResult.RuntimeCellDescs.Emplace(CreateCellDesc(CellName.ToString(), CellName != NAME_PersistentLevel, 0, CellActorSetInstances));
	}

	return true;
}
#endif
