// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RuntimePartitionPersistent)

#if WITH_EDITOR
bool URuntimePartitionPersistent::GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingResult& OutResult)
{
	const FString CellName("Persistent");
	OutResult.RuntimeCellDescs.Emplace(CreateCellDesc(CellName, false, 0, *InParams.ActorSetInstances));
	return true;
}
#endif
