// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeCellDataHashSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellDataHashSet)

#if WITH_EDITOR
void UWorldPartitionRuntimeCellDataHashSet::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Super::DumpStateLog(Ar);

	Ar.Printf(TEXT("Is 2D: %s"), bIs2D ? TEXT("True") : TEXT("False"));
}
#endif
