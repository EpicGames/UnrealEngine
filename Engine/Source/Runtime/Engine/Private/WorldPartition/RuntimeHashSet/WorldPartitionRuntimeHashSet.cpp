// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeCellDataHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionLHGrid.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/HashBuilder.h"
#include "UObject/UObjectGlobalsInternal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeHashSet)

static bool GRuntimeHashSetWorldAssetStreamingIncludeHLODFlagInCellHash = true;
static FAutoConsoleVariableRef CVarRuntimeHashSetWorldAssetStreamingIncludeHLODFlagInCellHash(
	TEXT("wp.Runtime.HashSet.WorldAssetStreaming.IncludeHLODFlagInCellHash"),
	GRuntimeHashSetWorldAssetStreamingIncludeHLODFlagInCellHash,
	TEXT("Used to choose whether HLOD flag should be included in cell hash generated during world asset streaming generation."));

#if WITH_EDITOR
TObjectPtr<URuntimePartition> FRuntimePartitionDesc::GetFirstSpatiallyLoadedHLODPartitionAncestor(int32 HLODSetupsIndex)
{
	check(HLODSetupsIndex < HLODSetups.Num());

	for (int32 Index = HLODSetupsIndex - 1; Index >= 0; Index--)
	{
		if (HLODSetups[Index].bIsSpatiallyLoaded)
		{
			return HLODSetups[Index].PartitionLayer;
		}
	}
	return MainLayer;
}
#endif

void FRuntimePartitionStreamingData::CreatePartitionsSpatialIndex() const
{
	if (!SpatialIndex)
	{
		uint32 StaticIndexAllocatedSize = 0;

		TArray<UWorldPartitionRuntimeCell*> SpatiallyLoadedCells3D;
		TArray<UWorldPartitionRuntimeCell*> SpatiallyLoadedCells2D;
		
		SpatiallyLoadedCells3D.Reserve(SpatiallyLoadedCells.Num());		
		SpatiallyLoadedCells2D.Reserve(SpatiallyLoadedCells.Num());

		for (UWorldPartitionRuntimeCell* Cell : SpatiallyLoadedCells)
		{
			if (CastChecked<UWorldPartitionRuntimeCellDataHashSet>(Cell->RuntimeCellData)->bIs2D)
			{
				SpatiallyLoadedCells2D.Add(Cell);
			}
			else
			{
				SpatiallyLoadedCells3D.Add(Cell);
			}
		}

		if (SpatiallyLoadedCells3D.Num())
		{
			SpatialIndex = MakeUnique<FStaticSpatialIndexType>();
			{
				TArray<TPair<FBox, TObjectPtr<UWorldPartitionRuntimeCell>>> PartitionsElements;
				Algo::Transform(SpatiallyLoadedCells3D, PartitionsElements, [](UWorldPartitionRuntimeCell* Cell)
				{
					return TPair<FBox, TObjectPtr<UWorldPartitionRuntimeCell>>(Cell->GetStreamingBounds(), Cell);
				});
				SpatialIndex->Init(MoveTemp(PartitionsElements));
				StaticIndexAllocatedSize += SpatialIndex->GetAllocatedSize();
			}
		
			SpatialIndexForce2D = MakeUnique<FStaticSpatialIndexType2D>();
			{
				TArray<TPair<FBox2D, TObjectPtr<UWorldPartitionRuntimeCell>>> PartitionsElements;
				Algo::Transform(SpatiallyLoadedCells3D, PartitionsElements, [](UWorldPartitionRuntimeCell* Cell)
				{
					const FBox CellBounds = Cell->GetStreamingBounds();
					const FBox2D CellBounds2D = FBox2D(FVector2D(CellBounds.Min), FVector2D(CellBounds.Max));
					return TPair<FBox2D, TObjectPtr<UWorldPartitionRuntimeCell>>(CellBounds2D, Cell);
				});
				SpatialIndexForce2D->Init(MoveTemp(PartitionsElements));
				StaticIndexAllocatedSize += SpatialIndexForce2D->GetAllocatedSize();
			}
		}

		if (SpatiallyLoadedCells2D.Num())
		{
			SpatialIndex2D = MakeUnique<FStaticSpatialIndexType2D>();
			{
				TArray<TPair<FBox2D, TObjectPtr<UWorldPartitionRuntimeCell>>> PartitionsElements;
				Algo::Transform(SpatiallyLoadedCells2D, PartitionsElements, [](UWorldPartitionRuntimeCell* Cell)
				{
					const FBox CellBounds = Cell->GetStreamingBounds();
					const FBox2D CellBounds2D = FBox2D(FVector2D(CellBounds.Min), FVector2D(CellBounds.Max));
					return TPair<FBox2D, TObjectPtr<UWorldPartitionRuntimeCell>>(CellBounds2D, Cell);
				});
				SpatialIndex2D->Init(MoveTemp(PartitionsElements));
				StaticIndexAllocatedSize += SpatialIndex2D->GetAllocatedSize();
			}
		}

#if WITH_EDITOR
		UE_LOGF(LogWorldPartition, Verbose, "CreatePartitionsSpatialIndex: %ls used %ls", *DebugName, *FGenericPlatformMemory::PrettyMemory(StaticIndexAllocatedSize));
#endif
	}
}

void FRuntimePartitionStreamingData::DestroyPartitionsSpatialIndex() const
{
	SpatialIndex.Reset();
	SpatialIndexForce2D.Reset();
	SpatialIndex2D.Reset();
}

int32 FRuntimePartitionStreamingData::GetLoadingRange() const
{
#if !UE_BUILD_SHIPPING
	int32 OverriddenLoadingRange;
	if (UWorldPartitionSubsystem::GetOverrideLoadingRange(Name, OverriddenLoadingRange))
	{
		return OverriddenLoadingRange;
	}
#endif
	return LoadingRange;
}

void URuntimeHashSetExternalStreamingObject::CreatePartitionsSpatialIndex() const
{
	for (const FRuntimePartitionStreamingData& StreamingData : RuntimeStreamingData)
	{
		StreamingData.CreatePartitionsSpatialIndex();
	}
}

void URuntimeHashSetExternalStreamingObject::DestroyPartitionsSpatialIndex() const
{
	for (const FRuntimePartitionStreamingData& StreamingData : RuntimeStreamingData)
	{
		StreamingData.DestroyPartitionsSpatialIndex();
	}
}

#if WITH_EDITOR
void URuntimeHashSetExternalStreamingObject::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	Super::DumpStateLog(Ar);

	TArray<const UWorldPartitionRuntimeCell*> StreamingCells;
	for (const FRuntimePartitionStreamingData& StreamingData : RuntimeStreamingData)
	{
		auto HandleStreamingCell = [&StreamingCells](UWorldPartitionRuntimeCell* StreamingCell)
		{
			if (!StreamingCell->IsAlwaysLoaded() || !IsRunningCookCommandlet())
			{
				StreamingCells.Add(StreamingCell);
			}
		};

		for (UWorldPartitionRuntimeCell* StreamingCell : StreamingData.SpatiallyLoadedCells)
		{
			HandleStreamingCell(StreamingCell);
		}

		for (UWorldPartitionRuntimeCell* StreamingCell : StreamingData.NonSpatiallyLoadedCells)
		{
			HandleStreamingCell(StreamingCell);
		}
	}
				
	StreamingCells.Sort([this](const UWorldPartitionRuntimeCell& A, const UWorldPartitionRuntimeCell& B)
	{
		if (A.IsAlwaysLoaded() ==  B.IsAlwaysLoaded())
		{
			return A.GetFName().LexicalLess(B.GetFName());
		}

		return A.IsAlwaysLoaded() > B.IsAlwaysLoaded();
	});

	for (const UWorldPartitionRuntimeCell* StreamingCell : StreamingCells)
	{
		FHierarchicalLogArchive::FIndentScope CellIndentScope = Ar.PrintfIndent(TEXT("Content of Cell %s (%s)"), *StreamingCell->GetDebugName(), *StreamingCell->GetName());
		StreamingCell->DumpStateLog(Ar);
	}

	Ar.Printf(TEXT(""));
}
#endif

void URuntimeHashSetExternalStreamingObject::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
#if WITH_EDITOR
	URuntimeHashSetExternalStreamingObject* This = CastChecked<URuntimeHashSetExternalStreamingObject>(InThis);
	for (const FRuntimePartitionStreamingData& StreamingData : This->RuntimeStreamingData)
	{
		if (StreamingData.SpatialIndex.IsValid())
		{
			StreamingData.SpatialIndex->AddReferencedObjects(Collector);
		}
		
		if (StreamingData.SpatialIndexForce2D.IsValid())
		{
			StreamingData.SpatialIndexForce2D->AddReferencedObjects(Collector);
		}

		if (StreamingData.SpatialIndex2D.IsValid())
		{
			StreamingData.SpatialIndex2D->AddReferencedObjects(Collector);
		}
	}
#endif

	Super::AddReferencedObjects(InThis, Collector);
}

UWorldPartitionRuntimeHashSet::UWorldPartitionRuntimeHashSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (UClass* RuntimeSpatialHashClass = FindObject<UClass>(nullptr, TEXT("/Script/Engine.WorldPartitionRuntimeSpatialHash")))
		{
			RegisterWorldPartitionRuntimeHashConverter(RuntimeSpatialHashClass, GetClass(), [](const UWorldPartitionRuntimeHash* SrcHash) -> UWorldPartitionRuntimeHash*
			{
				return CreateFrom(SrcHash);
			});
		}
	}
#endif
}

void UWorldPartitionRuntimeHashSet::OnBeginPlay()
{
	Super::OnBeginPlay();

	if (GetTypedOuter<UWorld>()->IsGameWorld())
	{
		ForEachStreamingData([this](const FRuntimePartitionStreamingData& StreamingData)
		{
#if !WITH_EDITOR
			FRuntimePartitionStreamingData& NonConstStreamingData = (const_cast<FRuntimePartitionStreamingData&>(StreamingData));		
			RemoveIrrelevantCells(NonConstStreamingData);
#endif
			StreamingData.CreatePartitionsSpatialIndex();
			return true;
		});
	}

	UpdateRuntimeDataGridMap();
}

#if WITH_EDITOR
void UWorldPartitionRuntimeHashSet::SetDefaultValues()
{
	check(RuntimePartitions.IsEmpty());

	FRuntimePartitionDesc& RuntimePartitionDesc = RuntimePartitions.AddDefaulted_GetRef();
	RuntimePartitionDesc.Name = TEXT("MainPartition");

	RuntimePartitionDesc.MainLayer = NewObject<URuntimePartitionLHGrid>(this, NAME_None);
	RuntimePartitionDesc.MainLayer->Name = RuntimePartitionDesc.Name;
	RuntimePartitionDesc.MainLayer->SetDefaultValues();

	UWorldPartition* WorldPartition = GetTypedOuter<UWorldPartition>();
	check(WorldPartition);

	if (const UHLODLayer* HLODLayer = WorldPartition->GetDefaultHLODLayer())
	{
		uint32 HLODIndex = 0;
		while (HLODLayer)
		{
			FRuntimePartitionHLODSetup& HLODSetup = RuntimePartitionDesc.HLODSetups.AddDefaulted_GetRef();

			HLODSetup.Name = HLODLayer->GetFName();
			HLODSetup.HLODLayers = { HLODLayer };

			URuntimePartitionLHGrid* HLODLHGrid = NewObject<URuntimePartitionLHGrid>(this, NAME_None);
			HLODLHGrid->CellSize = CastChecked<URuntimePartitionLHGrid>(RuntimePartitionDesc.MainLayer)->CellSize * (2 << HLODIndex);
			HLODLHGrid->LoadingRange = RuntimePartitionDesc.MainLayer->LoadingRange * (2 << HLODIndex);

			HLODSetup.PartitionLayer = HLODLHGrid;
			HLODSetup.PartitionLayer->Name = HLODSetup.Name;
			HLODSetup.PartitionLayer->bBlockOnSlowStreaming = false;
			HLODSetup.PartitionLayer->bClientOnlyVisible = true;
			HLODSetup.PartitionLayer->Priority = 0;
			HLODSetup.PartitionLayer->HLODIndex = HLODIndex;

			HLODLayer = HLODLayer->GetParentLayer();
			HLODIndex++;
		}
	}
}

void UWorldPartitionRuntimeHashSet::FlushStreamingContent()
{
	Super::FlushStreamingContent();
	RuntimeStreamingData.Empty();
	UpdateRuntimeDataGridMap();
}

FName UWorldPartitionRuntimeHashSet::GetDefaultGrid() const
{
	return RuntimePartitions.Num() ? RuntimePartitions[0].Name : NAME_None;
}

bool UWorldPartitionRuntimeHashSet::IsValidGrid(FName GridName, const UClass* ActorClass) const
{
	return GridName.IsNone() || !!ResolveRuntimePartition(GridName);
}
#endif

const URuntimePartition* UWorldPartitionRuntimeHashSet::ResolveRuntimePartition(FName GridName, bool bMainPartitionLayer) const
{
	TArray<FName> MainPartitionTokens;
	TArray<FName> HLODPartitionTokens;

	// Parse the potentially dot separated grid name to identiy the associated runtime partition
	if (ParseGridName(GridName, MainPartitionTokens, HLODPartitionTokens))
	{
		// The None grid name will always map to the first runtime partition in the list
		if (MainPartitionTokens[0].IsNone())
		{
			if (RuntimePartitions.IsValidIndex(0))
			{
				return RuntimePartitions[0].MainLayer;
			}
			else
			{
				return nullptr;
			}
		}

		// Make sure the runtime partition is valid
		const FRuntimePartitionDesc* FoundRuntimePartitionDesc = RuntimePartitions.FindByPredicate([&MainPartitionTokens](const FRuntimePartitionDesc& RuntimePartitionDesc)
		{
			return RuntimePartitionDesc.Name == MainPartitionTokens[0];
		});
		
		if (!FoundRuntimePartitionDesc || !FoundRuntimePartitionDesc->MainLayer || !FoundRuntimePartitionDesc->MainLayer->IsValidPartitionTokens(MainPartitionTokens))
		{
			return nullptr;
		}
	
		// If an HLOD partition token was specified ("MainPartition:HLODPartition"), make sure it's valid too
		if (!bMainPartitionLayer && HLODPartitionTokens.IsValidIndex(0) && !HLODPartitionTokens[0].IsNone())
		{
			const FRuntimePartitionHLODSetup* FoundRuntimePartitionHLODSetup = FoundRuntimePartitionDesc->HLODSetups.FindByPredicate([&HLODPartitionTokens](const FRuntimePartitionHLODSetup& RuntimePartitionHLODSetup)
			{
				return RuntimePartitionHLODSetup.Name == HLODPartitionTokens[0];
			});

			if (!FoundRuntimePartitionHLODSetup || !FoundRuntimePartitionHLODSetup->PartitionLayer || !FoundRuntimePartitionHLODSetup->PartitionLayer->IsValidPartitionTokens(HLODPartitionTokens))
			{
				return nullptr;
			}

			return FoundRuntimePartitionHLODSetup->PartitionLayer;
		}

		return FoundRuntimePartitionDesc->MainLayer;
	}			

	return nullptr;
}

#if WITH_EDITOR
bool UWorldPartitionRuntimeHashSet::IsValidHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const
{
	return ResolveRuntimePartitionForHLODLayer(GridName, HLODLayerPath) != nullptr;
}

TOptional<FFixedGridInfo> UWorldPartitionRuntimeHashSet::GetFixedGridInfo(FName InGridName) const
{
	const URuntimePartition* Partition = ResolveRuntimePartition(InGridName, /*bMainPartitionLayer*/ true);
	return Partition ? Partition->GetFixedGridInfo() : TOptional<FFixedGridInfo>{};
}

const URuntimePartition* UWorldPartitionRuntimeHashSet::ResolveRuntimePartitionForHLODLayer(FName GridName, const FSoftObjectPath& HLODLayerPath) const
{
	const URuntimePartition* RuntimePartition = nullptr;

	if (!RuntimePartitions.IsEmpty())
	{
	    if (const UHLODLayer* HLODLayer = Cast<UHLODLayer>(HLODLayerPath.ResolveObject()))
	    {
		    // The None grid name will always map to the first runtime partition in the list
		    int32 RuntimePartitionIndex = GridName.IsNone() ? 0 : INDEX_NONE;
    
		    if (RuntimePartitionIndex == INDEX_NONE)
		    {
			    TArray<FName> PartitionTokens;
			    TArray<FName> HLODPartitionTokens;
    
			    // Parse the potentially dot separated grid name to identiy the associated runtime partition
			    if (ParseGridName(GridName, PartitionTokens, HLODPartitionTokens))
			    {
				    int32 RuntimePartitionIndexLookup = 0;
				    for (const FRuntimePartitionDesc& RuntimePartitionDesc : RuntimePartitions)
				    {
					    if (RuntimePartitionDesc.Name == PartitionTokens[0])
					    {
						    RuntimePartitionIndex = RuntimePartitionIndexLookup;
						    break;
					    }
    
					    RuntimePartitionIndexLookup++;
				    }
			    }
		    }
    
		    if (RuntimePartitionIndex != INDEX_NONE)
		    {
		        const FRuntimePartitionHLODSetup* FoundRuntimePartitionHLODSetup = RuntimePartitions[RuntimePartitionIndex].HLODSetups.FindByPredicate([&HLODLayer](const FRuntimePartitionHLODSetup& RuntimePartitionHLODSetup)
		        {
			        return RuntimePartitionHLODSetup.HLODLayers.Contains(HLODLayer);
		        });

				RuntimePartition = FoundRuntimePartitionHLODSetup ? FoundRuntimePartitionHLODSetup->PartitionLayer.Get() : nullptr;
    		}
	    }
	}

	return RuntimePartition;
}
#endif

bool UWorldPartitionRuntimeHashSet::ParseGridName(FName GridName, TArray<FName>& MainPartitionTokens, TArray<FName>& HLODPartitionTokens)
{
	// If the grid name is none, it directly maps to the main partition
	if (GridName.IsNone())
	{
		MainPartitionTokens.Add(NAME_None);
		return true;
	}

	// Split grid name into its partition and HLOD parts
	TArray<FString> GridNameTokens;
	if (!GridName.ToString().ParseIntoArray(GridNameTokens, TEXT(":")))
	{
		GridNameTokens.Add(GridName.ToString());
	}

	// Parsed grid names token should be either "RuntimeHash" or "RuntimeHash:HLODLayer"
	if (GridNameTokens.Num() > 2)
	{
		return false;
	}

	// Parse the target main partition
	TArray<FString> MainPartitionTokensStr;
	if (GridNameTokens[0].ParseIntoArray(MainPartitionTokensStr, TEXT(".")))
	{
		Algo::Transform(MainPartitionTokensStr, MainPartitionTokens, [](const FString& GridName) { return *GridName; });
	}

	// Parse the target HLOD partition
	if (GridNameTokens.IsValidIndex(1))
	{
		HLODPartitionTokens.Add(*GridNameTokens[1]);
	}

	return true;
}

#if WITH_EDITOR
bool UWorldPartitionRuntimeHashSet::HasStreamingContent() const
{
	return !RuntimeStreamingData.IsEmpty();
}

void UWorldPartitionRuntimeHashSet::StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject)
{
	check(!RuntimeStreamingData.IsEmpty());

	Super::StoreStreamingContentToExternalStreamingObject(OutExternalStreamingObject);

	URuntimeHashSetExternalStreamingObject* StreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(OutExternalStreamingObject);
	StreamingObject->RuntimeStreamingData = MoveTemp(RuntimeStreamingData);

	for (FRuntimePartitionStreamingData& StreamingData : StreamingObject->RuntimeStreamingData)
	{
		for (UWorldPartitionRuntimeCell* Cell : StreamingData.SpatiallyLoadedCells)
		{
			Cell->Rename(nullptr, StreamingObject,  REN_DoNotDirty);
		}

		for (UWorldPartitionRuntimeCell* Cell : StreamingData.NonSpatiallyLoadedCells)
		{
			Cell->Rename(nullptr, StreamingObject,  REN_DoNotDirty);
		}
	}
}
#endif

bool UWorldPartitionRuntimeHashSet::InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	if (Super::InjectExternalStreamingObject(ExternalStreamingObject))
	{
		URuntimeHashSetExternalStreamingObject* HashSetExternalStreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(ExternalStreamingObject);

#if !WITH_EDITOR
		for (FRuntimePartitionStreamingData& StreamingData : HashSetExternalStreamingObject->RuntimeStreamingData)
		{
			RemoveIrrelevantCells(StreamingData);
		}
#endif

		HashSetExternalStreamingObject->CreatePartitionsSpatialIndex();
		UpdateRuntimeDataGridMap();
		return true;
	}

	return false;
}

bool UWorldPartitionRuntimeHashSet::RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	if (Super::RemoveExternalStreamingObject(ExternalStreamingObject))
	{
		URuntimeHashSetExternalStreamingObject* HashSetExternalStreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(ExternalStreamingObject);
		HashSetExternalStreamingObject->DestroyPartitionsSpatialIndex();
		UpdateRuntimeDataGridMap();
		return true;
	}

	return false;
}

// Streaming interface
void UWorldPartitionRuntimeHashSet::ForEachStreamingCells(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const
{
	auto ForEachCells = [this, &Func](const TArray<TObjectPtr<UWorldPartitionRuntimeCell>>& InCells)
	{
		for (UWorldPartitionRuntimeCell* Cell : InCells)
		{
			if (!Func(Cell))
			{
				return false;
			}
		}
		return true;
	};

	ForEachStreamingData([&ForEachCells](const FRuntimePartitionStreamingData& StreamingData)
	{
		return ForEachCells(StreamingData.SpatiallyLoadedCells) && ForEachCells(StreamingData.NonSpatiallyLoadedCells);
	});
}

void UWorldPartitionRuntimeHashSet::ForEachStreamingCellsQuery(const FWorldPartitionStreamingQuerySource& QuerySource, TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func, FWorldPartitionQueryCache* QueryCache) const
{
	auto ShouldAddCell = [this](const UWorldPartitionRuntimeCell* Cell, const FWorldPartitionStreamingQuerySource& QuerySource)
	{
		if (IsCellRelevantFor(Cell->GetClientOnlyVisible()))
		{
			if (Cell->HasDataLayers())
			{
				if (QuerySource.bIncludeAnyDataLayer)
				{
					return true;
				}
				
				if (!QuerySource.DataLayers.IsEmpty() &&
					Cell->GetDataLayers().FindByPredicate([&](const FName& DataLayerName) { return QuerySource.DataLayers.Contains(DataLayerName); }))
				{
					return true;
				}
			}
			else if (!QuerySource.bDataLayersOnly)
			{
				return true;
			}
		}

		return false;
	};

	auto ForEachSpatiallyLoadedCells = [&ShouldAddCell, QueryCache, &QuerySource, &Func]<typename SpatialIndexType>(SpatialIndexType* InSpatialIndex, int32 InLoadingRange)
	{
		if (InSpatialIndex)
		{
			QuerySource.ForEachShape(InLoadingRange, false, [InSpatialIndex, QueryCache, &ShouldAddCell, &QuerySource, &Func](const FSphericalSector& Shape)
			{
				auto ForEachIntersectingElement = [InSpatialIndex, QueryCache, &ShouldAddCell, &Shape, &QuerySource, &Func]<typename ShapeType>(const ShapeType& InShape)
				{
					InSpatialIndex->ForEachIntersectingElement(InShape, [QueryCache, &ShouldAddCell, &Shape, &QuerySource, &Func](UWorldPartitionRuntimeCell* RuntimeCell)
					{
						if (QueryCache)
						{
							QueryCache->AddCellInfo(RuntimeCell, Shape);
						}

						return !ShouldAddCell(RuntimeCell, QuerySource) || Func(RuntimeCell);
					});
				};

				if (Shape.IsSphere())
				{
					const FStaticSpatialIndex::FSphere Sphere(Shape.GetCenter(), Shape.GetRadius());
					ForEachIntersectingElement(Sphere);
				}
				else
				{
					const FStaticSpatialIndex::FCone Cone(Shape.GetCenter(), Shape.GetAxis(), Shape.GetRadius(), Shape.GetAngle());
					ForEachIntersectingElement(Cone);
				}
			});
		}

		return true;
	};

	auto ForEachNonSpatiallyLoadedCells = [this, &ShouldAddCell, &QuerySource, &Func](const TArray<TObjectPtr<UWorldPartitionRuntimeCell>>& InNonSpatiallyLoadedCells)
	{
		if (!bQueryNonSpatialCells)
		{
			return true;
		}
		for (const UWorldPartitionRuntimeCell* Cell : InNonSpatiallyLoadedCells)
		{
			if (ShouldAddCell(Cell, QuerySource))
			{
				if (!Func(Cell))
				{
					return false;
				}
			}
		}
		return true;
	};

	ForEachStreamingData([&QuerySource, &ForEachSpatiallyLoadedCells, &ForEachNonSpatiallyLoadedCells](const FRuntimePartitionStreamingData& StreamingData)
	{
        if (FStreamingSourceShapeHelper::IsSourceAffectingGrid(QuerySource.TargetGrids, QuerySource.TargetBehavior, StreamingData.Name))
        {
				return ForEachSpatiallyLoadedCells(StreamingData.SpatialIndex.Get(), StreamingData.GetLoadingRange()) &&
				   ForEachSpatiallyLoadedCells(StreamingData.SpatialIndex2D.Get(), StreamingData.GetLoadingRange()) &&
				   ForEachNonSpatiallyLoadedCells(StreamingData.NonSpatiallyLoadedCells);
		}
		return true;
	});
}

void UWorldPartitionRuntimeHashSet::ForEachStreamingCellsSources(const TArray<FWorldPartitionStreamingSource>& Sources, TFunctionRef<bool(const UWorldPartitionRuntimeCell*, EStreamingSourceTargetState)> Func, const FWorldPartitionStreamingContext& InContext) const
{
	// Build a context when none is provided (for backward compatibility)
	const FWorldPartitionStreamingContext StackContext = !InContext.IsValid() ? FWorldPartitionStreamingContext::Create(GetTypedOuter<UWorld>()) : FWorldPartitionStreamingContext();
	const FWorldPartitionStreamingContext& Context = InContext.IsValid() ? InContext : StackContext;
	check(Context.IsValid());

	FDynamicLoadingRangeScales LoadingRangeScales
	{
#if WITH_EDITOR
		Context.GetDynamicLoadingRangeScales()
#endif // WITH_EDITOR
	};

	// Non-spatially loaded cells
	for (const FRuntimePartitionStreamingData* StreamingData : RuntimeNonSpatiallyLoadedDataGridList)
	{
		for (UWorldPartitionRuntimeCell* Cell : StreamingData->NonSpatiallyLoadedCells)
		{
#if WITH_EDITOR
			if (!IsCellRelevantFor(Cell->GetClientOnlyVisible()))
			{
				continue;
			}
#else
			check(IsCellRelevantFor(Cell->GetClientOnlyVisible()));
#endif
			const EDataLayerRuntimeState CellEffectiveWantedState = Cell->GetCellEffectiveWantedState(Context);
			if (CellEffectiveWantedState != EDataLayerRuntimeState::Unloaded)
			{
				Func(Cell, (CellEffectiveWantedState == EDataLayerRuntimeState::Loaded) ? EStreamingSourceTargetState::Loaded : EStreamingSourceTargetState::Activated);
			}
		}
	}

	// Spatially loaded cells
	for (const FWorldPartitionStreamingSource& Source : Sources)
	{
		// Build the source target grids based on target behavior
		TArray<FName, TInlineAllocator<8>> TargetGrids;
		if (Source.TargetBehavior == EStreamingSourceTargetBehavior::Include)
		{
			if (Source.TargetGrids.Num())
			{
				TargetGrids = Source.TargetGrids.Array();
			}
			else
			{
				RuntimeSpatiallyLoadedDataGridMap.GenerateKeyArray(TargetGrids);
			}
		}
		else if (Source.TargetBehavior == EStreamingSourceTargetBehavior::Exclude)
		{
			for (auto& [GridName, StreamingDataList] : RuntimeSpatiallyLoadedDataGridMap)
			{
				if (!Source.TargetGrids.Contains(GridName))
				{
					TargetGrids.Add(GridName);
				}
			}
		}

		for (FName GridName : TargetGrids)
		{
			if (const TArray<const FRuntimePartitionStreamingData*>* StreamingDataList = RuntimeSpatiallyLoadedDataGridMap.Find(GridName))
			{
				check(FStreamingSourceShapeHelper::IsSourceAffectingGrid(Source.TargetGrids, Source.TargetBehavior, GridName));

				for (const FRuntimePartitionStreamingData* StreamingData : *StreamingDataList)
				{
					Source.ForEachShape(StreamingData->GetLoadingRange(), false, [this, &Source, StreamingData, &Context, &Func, &LoadingRangeScales](const FSphericalSector& Shape)
					{
						auto ForEachIntersectingElementFunc = [this, &Source, &Shape, &Context, &Func](UWorldPartitionRuntimeCell* Cell)
						{
#if WITH_EDITOR
							if (!IsCellRelevantFor(Cell->GetClientOnlyVisible()))
							{
								return;
							}
#else
							check(IsCellRelevantFor(Cell->GetClientOnlyVisible()));
#endif
							const EDataLayerRuntimeState CellEffectiveWantedState = Cell->GetCellEffectiveWantedState(Context);
							if (CellEffectiveWantedState != EDataLayerRuntimeState::Unloaded)
							{
								Cell->AppendStreamingSourceInfo(Source, Shape, Context);
								Func(Cell, ((CellEffectiveWantedState == EDataLayerRuntimeState::Loaded) || (Source.TargetState == EStreamingSourceTargetState::Loaded)) ? EStreamingSourceTargetState::Loaded : EStreamingSourceTargetState::Activated);
							}
						};

						auto ForEachIntersectingElement = [&StreamingData, &Source, &ForEachIntersectingElementFunc, &LoadingRangeScales]<typename ShapeType>(ShapeType& Shape)
						{
							float ShapeRadius = Shape.Radius;
							if (StreamingData->SpatialIndex.IsValid())
							{
								if (Source.bForce2D)
								{
									Shape.Radius = ShapeRadius * LoadingRangeScales.Scale2D;
									StreamingData->SpatialIndexForce2D->ForEachIntersectingElement(Shape, ForEachIntersectingElementFunc);
								}
								else
								{
									Shape.Radius = ShapeRadius * LoadingRangeScales.Scale3D;
									StreamingData->SpatialIndex->ForEachIntersectingElement(Shape, ForEachIntersectingElementFunc);
								}
							}

							if (StreamingData->SpatialIndex2D.IsValid())
							{
								Shape.Radius = ShapeRadius * LoadingRangeScales.Scale2D;
								StreamingData->SpatialIndex2D->ForEachIntersectingElement(Shape, ForEachIntersectingElementFunc);
							}
						};

						if (Shape.IsSphere())
						{
							FStaticSpatialIndex::FSphere ShapeSphere(Shape.GetCenter(), Shape.GetRadius());
							ForEachIntersectingElement(ShapeSphere);
						}
						else
						{
							FStaticSpatialIndex::FCone ShapeCone(Shape.GetCenter(), Shape.GetAxis(), Shape.GetRadius(), Shape.GetAngle());
							ForEachIntersectingElement(ShapeCone);
						}
					});
				}
			}
		}
	}
}

bool UWorldPartitionRuntimeHashSet::SupportsWorldAssetStreaming(const FName& InTargetGrid)
{
	return ResolveRuntimePartition(InTargetGrid) != nullptr;
}

FGuid UWorldPartitionRuntimeHashSet::RegisterWorldAssetStreaming(const UWorldPartition::FRegisterWorldAssetStreamingParams& InParams)
{
	if (!InParams.IsValid())
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Invalid parameters provided.");
		return FGuid();
	}

	if (WorldAssetStreamingObjects.Contains(InParams.Guid))
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: World asset guid '%ls' was already registered.", *InParams.Guid.ToString());
		return FGuid();
	}

	FName RuntimePartitionName;
	const URuntimePartition* RuntimePartition = ResolveRuntimePartition(InParams.WorldAssetDesc.TargetGrid);
	if (!RuntimePartition)
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Unable to resolve TargetGrid '%ls'.", *InParams.WorldAssetDesc.TargetGrid.ToString());
		return FGuid();
	}

	TArray<TPair<const URuntimePartition*, const TSoftObjectPtr<UWorld>>> HLODRuntimePartitions;
	for (const UWorldPartition::FRegisterWorldAssetStreamingParams::FWorldAssetDesc& HLODWorldAssetDesc : InParams.HLODWorldAssetDescs)
	{
		if (!HLODWorldAssetDesc.WorldAsset.IsNull() && !HLODWorldAssetDesc.TargetGrid.IsNone())
		{
			const URuntimePartition* HLODRuntimePartition = ResolveRuntimePartition(HLODWorldAssetDesc.TargetGrid);
			if (!HLODRuntimePartition)
			{
				UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Unable to resolve TargetGridHLOD '%ls'.", *HLODWorldAssetDesc.TargetGrid.ToString());
				return FGuid();
			}
			else
			{
				HLODRuntimePartitions.Add({ HLODRuntimePartition, HLODWorldAssetDesc.WorldAsset });
			}
		}
	}

	URuntimeHashSetExternalStreamingObject* StreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(CreateExternalStreamingObject(URuntimeHashSetExternalStreamingObject::StaticClass(), this, GetTypedOuter<UWorld>()));
	if (!StreamingObject)
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Couldn't create ExternalStreamingObject.");
		return FGuid();
	}		

	FGuid SourceCellGuid;
		
	auto CreateStreamingCell = [&InParams, &SourceCellGuid, StreamingObject, this](const URuntimePartition* TargetPartition, const TSoftObjectPtr<UWorld>& WorldAsset, bool bIsHLODPass) -> bool
	{
		check(TargetPartition);
		
		bool bClientOnlyVisible = TargetPartition->bClientOnlyVisible;
		bool bBlockOnSlowStreaming = TargetPartition->bBlockOnSlowStreaming;

		FRuntimePartitionStreamingData StreamingData;
		StreamingData.Name = TargetPartition->Name;
		StreamingData.LoadingRange = TargetPartition->LoadingRange;

		// Create Cell
		FName TargetPartitionName = TargetPartition->Name;
		FGuid InstanceGuid = InParams.Guid;

		FArchiveMD5 ArMD5;
		ArMD5 << TargetPartitionName << InstanceGuid; 
		if (GRuntimeHashSetWorldAssetStreamingIncludeHLODFlagInCellHash)
		{
			ArMD5 << bIsHLODPass;
		}

		const FGuid CellGuid = ArMD5.GetGuidFromHash();
		check(CellGuid.IsValid());
		if (!bIsHLODPass)
		{
			SourceCellGuid = CellGuid;
		}

		const FString CellName = FString::Printf(TEXT("InjectedCell_%s"), *CellGuid.ToString());

		if (UWorldPartitionRuntimeLevelStreamingCell* RuntimeCell = Cast<UWorldPartitionRuntimeLevelStreamingCell>(
			CreateRuntimeCell(UWorldPartitionRuntimeLevelStreamingCell::StaticClass(), UWorldPartitionRuntimeCellDataHashSet::StaticClass(), CellName, InParams.CellInstanceSuffix, StreamingObject)))
		{
			RuntimeCell->SetClientOnlyVisible(bClientOnlyVisible);
			RuntimeCell->SetBlockOnSlowLoading(bBlockOnSlowStreaming);
			RuntimeCell->SetIsHLOD(bIsHLODPass);
			RuntimeCell->SetGuid(CellGuid);
			RuntimeCell->SetCellDebugColor(FLinearColor::MakeRandomSeededColor(GetTypeHash(CellName)));

			if (bIsHLODPass)
			{
				RuntimeCell->SetSourceCellGuid(SourceCellGuid);
			}

			UWorldPartitionRuntimeCellDataHashSet* RuntimeCellData = CastChecked<UWorldPartitionRuntimeCellDataHashSet>(RuntimeCell->RuntimeCellData);
			RuntimeCellData->DebugName = CellName + InParams.CellInstanceSuffix;
			RuntimeCellData->CellBounds = InParams.Bounds;
			RuntimeCellData->ContentBounds = InParams.Bounds;
			RuntimeCellData->HierarchicalLevel = MAX_int32;
			RuntimeCellData->Priority = InParams.Priority;
			RuntimeCellData->GridName = TargetPartition->Name;
			RuntimeCellData->bIs2D = false;

			if (RuntimeCell->CreateAndSetLevelStreaming(WorldAsset, InParams.Transform))
			{
				StreamingData.SpatiallyLoadedCells.Add(RuntimeCell);
				StreamingObject->RuntimeStreamingData.Emplace(MoveTemp(StreamingData));
			}
			else
			{
				UE_LOGF(LogWorldPartition, Error, "Error creating streaming cell %ls for world asset %ls at %ls", *RuntimeCell->GetName(), *WorldAsset.ToString(), *InParams.Transform.ToString());
				return false;
			}
		}		
		else
		{
			UE_LOGF(LogWorldPartition, Error, "Error creating streaming cell %ls for world asset %ls at %ls", *CellName, *WorldAsset.ToString(), *InParams.Transform.ToString());
			return false;
		}

		return true;
	};

	if (CreateStreamingCell(RuntimePartition, InParams.WorldAssetDesc.WorldAsset, false /* bIsHLODPass */) == false)
	{
		return FGuid();
	}
	for (const TPair<const URuntimePartition*, const TSoftObjectPtr<UWorld>>& HLODRuntimePartition : HLODRuntimePartitions)
	{
		if (CreateStreamingCell(HLODRuntimePartition.Key, HLODRuntimePartition.Value, true /* bIsHLODPass */) == false)
		{
			return FGuid();
		}
	}

	GetOuterUWorldPartition()->InjectExternalStreamingObject(StreamingObject);
	WorldAssetStreamingObjects.Add(InParams.Guid, StreamingObject);

	return InParams.Guid;
}

bool UWorldPartitionRuntimeHashSet::UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid)
{
	if (TObjectPtr<URuntimeHashSetExternalStreamingObject>* StreamingObject = WorldAssetStreamingObjects.Find(InWorldAssetStreamingGuid))
	{
		// External streaming objects are created with a provided name which helps to detect invalid runtime states of injected content.
		// Before releasing these objects, trash their name to make sure they won't be recycled if the tile re-injects the objects before a GC was triggered first.
		// Apply the same logic on the LevelStreaming object of each injected cell as it is named using the injected cell name and is outered to the owning world.
		auto TrashExternalStreamingData = [](URuntimeHashSetExternalStreamingObject* InStreamingObject)
		{
			InStreamingObject->ForEachStreamingCells([](UWorldPartitionRuntimeCell& Cell)
			{
				UWorldPartitionRuntimeLevelStreamingCell* InjectedCell = Cast<UWorldPartitionRuntimeLevelStreamingCell>(&Cell);
				if (UWorldPartitionLevelStreamingDynamic* LevelStreaming = InjectedCell ? InjectedCell->GetLevelStreaming() : nullptr)
				{
					TrashObject(LevelStreaming);
					// Make sure to flag this streaming level to be unloaded and removed as we don't want any future RequestLevel 
					// of a newly created streaming level of the same WorldAsset to fail. 
					LevelStreaming->SetIsRequestingUnloadAndRemoval(true);
				}
			});

			TrashObject(InStreamingObject);
		};

		if (IsValid(*StreamingObject))
		{
			if (IsValid(GetOuterUWorldPartition()))
			{
				GetOuterUWorldPartition()->RemoveExternalStreamingObject(*StreamingObject);
			}
			TrashExternalStreamingData(*StreamingObject);
		}
		WorldAssetStreamingObjects.Remove(InWorldAssetStreamingGuid);
		return true;
	}

	return false;
}

TArray<UWorldPartitionRuntimeCell*> UWorldPartitionRuntimeHashSet::GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid)
{
	TArray<UWorldPartitionRuntimeCell*> Result;
	if (TObjectPtr<URuntimeHashSetExternalStreamingObject>* StreamingObject = WorldAssetStreamingObjects.Find(InWorldAssetStreamingGuid))
	{
		(*StreamingObject)->ForEachStreamingCells([&Result](UWorldPartitionRuntimeCell& Cell) { Result.Add(&Cell); });
	}
	return Result;
}

#if WITH_EDITOR
void UWorldPartitionRuntimeHashSet::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	static FName NAME_RuntimePartitions("RuntimePartitions");
	static FName NAME_HLODSetups("HLODSetups");
	static FName NAME_HLODLayers("HLODLayers");

	FName PropertyName = PropertyChangedEvent.GetPropertyName();
	FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	FName ActiveMemberNodeName = NAME_None;
	FName ActiveNodeName = NAME_None;
	EPropertyChangeType::Type ChangeType = PropertyChangedEvent.ChangeType;

	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		ActiveMemberNodeName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();
	}

	if (PropertyChangedEvent.PropertyChain.GetActiveNode())
	{
		ActiveNodeName = PropertyChangedEvent.PropertyChain.GetActiveNode()->GetValue()->GetFName();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FRuntimePartitionDesc, Name))
	{
		int32 RuntimePartitionIndex = PropertyChangedEvent.GetArrayIndex(NAME_RuntimePartitions.ToString());
		check(RuntimePartitions.IsValidIndex(RuntimePartitionIndex));

		FRuntimePartitionDesc& RuntimePartitionDesc = RuntimePartitions[RuntimePartitionIndex];

		int32 HLODSetupsIndex = PropertyChangedEvent.GetArrayIndex(NAME_HLODSetups.ToString());
		if (RuntimePartitionDesc.HLODSetups.IsValidIndex(HLODSetupsIndex))
		{
			FRuntimePartitionHLODSetup& RuntimePartitionHLODSetup = RuntimePartitionDesc.HLODSetups[HLODSetupsIndex];

			// Rename HLOD if it clashes with another one
			auto FindUniqueHLODName = [this, &RuntimePartitionDesc](FName TestedName, int32 SkipIndex)->FName
			{
				bool FoundDuplicate;
				do
				{
					FoundDuplicate = false;
					for (int32 CurHLODSetupsIndex = 0; CurHLODSetupsIndex < RuntimePartitionDesc.HLODSetups.Num(); CurHLODSetupsIndex++)
					{
						if (CurHLODSetupsIndex != SkipIndex)
						{
							if (TestedName == RuntimePartitionDesc.HLODSetups[CurHLODSetupsIndex].Name)
							{
								FoundDuplicate = true;
								TestedName.SetNumber(TestedName.GetNumber() + 1);
								break;
							}
						}
					}
				} while (FoundDuplicate);
				return TestedName;
			};

			RuntimePartitionHLODSetup.Name = FindUniqueHLODName(RuntimePartitionHLODSetup.Name, HLODSetupsIndex);
			RuntimePartitionHLODSetup.PartitionLayer->Name = RuntimePartitionHLODSetup.Name;
			RuntimePartitionHLODSetup.RowDisplayName = RuntimePartitionHLODSetup.PartitionLayer->Name;
		}
		else
		{
			// Rename layer if it clashes with another one
			auto FindUniquePartitionName = [this](FName TestedName, int32 SkipIndex)->FName
			{
				bool FoundDuplicate;
				do
				{
					FoundDuplicate = false;
					for (int32 CurRuntimePartitionIndex = 0; CurRuntimePartitionIndex < RuntimePartitions.Num(); CurRuntimePartitionIndex++)
					{
						if (CurRuntimePartitionIndex != SkipIndex)
						{
							if (TestedName == RuntimePartitions[CurRuntimePartitionIndex].Name)
							{
								FoundDuplicate = true;
								TestedName.SetNumber(TestedName.GetNumber() + 1);
								break;
							}
						}
					}
				} while (FoundDuplicate);
				return TestedName;
			};

			RuntimePartitionDesc.Name = FindUniquePartitionName(RuntimePartitionDesc.Name, RuntimePartitionIndex);
			RuntimePartitionDesc.MainLayer->Name = RuntimePartitionDesc.Name;
		}
	}
	else if (PropertyName == NAME_HLODSetups)
	{
		int32 RuntimePartitionIndex = PropertyChangedEvent.GetArrayIndex(NAME_RuntimePartitions.ToString());
		if (RuntimePartitions.IsValidIndex(RuntimePartitionIndex))
		{
			FRuntimePartitionDesc& RuntimePartitionDesc = RuntimePartitions[RuntimePartitionIndex];

			int32 HLODSetupsIndex = PropertyChangedEvent.GetArrayIndex(NAME_HLODSetups.ToString());
			if (RuntimePartitionDesc.HLODSetups.IsValidIndex(HLODSetupsIndex))
			{
				FRuntimePartitionHLODSetup& RuntimePartitionHLODSetup = RuntimePartitionDesc.HLODSetups[HLODSetupsIndex];
				URuntimePartition* ParentRuntimePartition = RuntimePartitionDesc.GetFirstSpatiallyLoadedHLODPartitionAncestor(HLODSetupsIndex);
				RuntimePartitionHLODSetup.Name = *FString::Printf(TEXT("HLOD_%d"), HLODSetupsIndex);
				RuntimePartitionHLODSetup.PartitionLayer = ParentRuntimePartition->CreateHLODRuntimePartition(HLODSetupsIndex);
				RuntimePartitionHLODSetup.PartitionLayer->Name = RuntimePartitionHLODSetup.Name;
			}

			FixupHLODSetup(RuntimePartitionDesc);
		}
	}
	else if (PropertyName == NAME_HLODLayers)
	{
		int32 RuntimePartitionIndex = PropertyChangedEvent.GetArrayIndex(NAME_RuntimePartitions.ToString());
		if (RuntimePartitions.IsValidIndex(RuntimePartitionIndex))
		{
			FRuntimePartitionDesc& RuntimePartitionDesc = RuntimePartitions[RuntimePartitionIndex];

			int32 HLODSetupsIndex = PropertyChangedEvent.GetArrayIndex(NAME_HLODSetups.ToString());
			if (RuntimePartitionDesc.HLODSetups.IsValidIndex(HLODSetupsIndex))
			{
				FRuntimePartitionHLODSetup& RuntimePartitionHLODSetup = RuntimePartitionDesc.HLODSetups[HLODSetupsIndex];

				int32 HLODLayersIndex = PropertyChangedEvent.GetArrayIndex(NAME_HLODLayers.ToString());
				if (RuntimePartitionHLODSetup.HLODLayers.IsValidIndex(HLODLayersIndex))
				{
					const UHLODLayer* HLODLayer = RuntimePartitionHLODSetup.HLODLayers[HLODLayersIndex];

					// Remove duplicated entries
					for (int32 CurrentHLODSetupsIndex = 0; CurrentHLODSetupsIndex < RuntimePartitionDesc.HLODSetups.Num(); CurrentHLODSetupsIndex++)
					{
						FRuntimePartitionHLODSetup& CurrentRuntimePartitionHLODSetup = RuntimePartitionDesc.HLODSetups[CurrentHLODSetupsIndex];
						for (int32 CurrentHLODLayerIndex = 0; CurrentHLODLayerIndex < CurrentRuntimePartitionHLODSetup.HLODLayers.Num(); CurrentHLODLayerIndex++)
						{
							const UHLODLayer* CurrentHLODLayer = CurrentRuntimePartitionHLODSetup.HLODLayers[CurrentHLODLayerIndex];
							if (((CurrentHLODSetupsIndex != HLODSetupsIndex) || (CurrentHLODLayerIndex != HLODLayersIndex)) && (CurrentHLODLayer == HLODLayer))
							{
								CurrentRuntimePartitionHLODSetup.HLODLayers.RemoveAt(CurrentHLODLayerIndex--);
								break;
							}
						}
					}
				}
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FRuntimePartitionDesc, MainLayer))
	{
		int32 RuntimePartitionIndex = PropertyChangedEvent.GetArrayIndex(NAME_RuntimePartitions.ToString());
		check(RuntimePartitions.IsValidIndex(RuntimePartitionIndex));

		FRuntimePartitionDesc& RuntimePartitionDesc = RuntimePartitions[RuntimePartitionIndex];

		FixupHLODSetup(RuntimePartitionDesc);

		for (const FRuntimePartitionHLODSetup& HLODSetup : RuntimePartitionDesc.HLODSetups)
		{
			if (HLODSetup.PartitionLayer)
			{
				HLODSetup.PartitionLayer->UpdateHLODRuntimePartitionFrom(RuntimePartitionDesc.MainLayer);
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldPartitionRuntimeHashSet, RuntimePartitions))
	{
		if (RuntimePartitions.IsEmpty())
		{
			// Reapply the default values as we need at least a main partition
			SetDefaultValues();
		}
	}
}

void UWorldPartitionRuntimeHashSet::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);
	
	// In PIE, create streaming datas spatial indexes and runtime grid data maps right after world duplication to allow making spatial queries from calls 
	// to BlockTillLevelStreamingCompleted before the world has issued OnBeginPlay (see UGameInstance::StartPlayInEditorGameInstance).
	if (DuplicateMode == EDuplicateMode::PIE)
	{
		ForEachStreamingData([this](const FRuntimePartitionStreamingData& StreamingData) { StreamingData.CreatePartitionsSpatialIndex(); return true; });
		UpdateRuntimeDataGridMap();
	}
}

void UWorldPartitionRuntimeHashSet::PostLoad()
{
	Super::PostLoad();

	// Perform fixup for potentially wrong HLOD setups
	for (FRuntimePartitionDesc& RuntimePartition : RuntimePartitions)
	{
		FixupHLODSetup(RuntimePartition);
	}

	// Clear streaming data that could have inadvertently been serialized while saving in PIE when loading from disk (need to keep it when coming from PIE duplication).
	if (GetLinker())
	{
		FlushStreamingContent();
	}
}

void UWorldPartitionRuntimeHashSet::FixupHLODSetup(FRuntimePartitionDesc& RuntimePartition)
{
	if (RuntimePartition.MainLayer)
	{
		// By default, HLOD partitions are only loaded on the clients.
		// However, some HLOD layer configurations might requires them to be loaded on the server (for example, HLOD destruction
		// requires HLOD actors data to be replicated).
		auto HLODPartitionMustBeLoadedOnServer = [](const FRuntimePartitionHLODSetup& HLODSetup) -> bool
		{
			return Algo::AnyOf(HLODSetup.HLODLayers, [](const UHLODLayer* HLODLayer)
			{
				return HLODLayer && HLODLayer->MustBeLoadedOnServer();
			});			
		};

		TArray<TObjectPtr<const UHLODLayer>> PersistentHLODLayers;

		for (int32 HLODSetupIndex = 0; HLODSetupIndex < RuntimePartition.HLODSetups.Num(); )
		{
			FRuntimePartitionHLODSetup& HLODSetup = RuntimePartition.HLODSetups[HLODSetupIndex];
			
			if (!HLODSetup.bIsSpatiallyLoaded)
			{
				PersistentHLODLayers.Append(HLODSetup.HLODLayers);
				RuntimePartition.HLODSetups.RemoveAt(HLODSetupIndex);
				continue;
			}

			if (HLODSetup.PartitionLayer)
			{
				// If needed, recreate the HLOD partition to use the same class as the main partition
				if (HLODSetup.PartitionLayer->GetClass() != RuntimePartition.MainLayer->GetClass())
				{
					URuntimePartition* ParentRuntimePartition = RuntimePartition.GetFirstSpatiallyLoadedHLODPartitionAncestor(HLODSetupIndex);
					HLODSetup.PartitionLayer = ParentRuntimePartition->CreateHLODRuntimePartition(HLODSetupIndex);
					HLODSetup.PartitionLayer->Name = HLODSetup.Name;
				}

				// Row display name (used as title when showing as an array item) must match the partition name
				HLODSetup.RowDisplayName = HLODSetup.Name;

				// Make sure that HLOD partitions settings are updated to match the main layer settings
				HLODSetup.PartitionLayer->UpdateHLODRuntimePartitionFrom(RuntimePartition.MainLayer);

				// Only load HLOD on clients unless the HLOD setup requires them on the server too.
				HLODSetup.PartitionLayer->bClientOnlyVisible = !HLODPartitionMustBeLoadedOnServer(HLODSetup);
			}

			HLODSetupIndex++;
		}

		if (IWorldPartitionEditorModule::Get().GetRequireExplicitHLODLayerPartitionAssignation())
		{
			FRuntimePartitionHLODSetup& PersistentHLODSetup = RuntimePartition.HLODSetups.Emplace_GetRef();
			PersistentHLODSetup.PartitionLayer = NewObject<URuntimePartitionPersistent>(this, NAME_None);
			PersistentHLODSetup.PartitionLayer->LoadingRange = 0;
			PersistentHLODSetup.PartitionLayer->Name = NAME_None;
			PersistentHLODSetup.Name = NAME_None;
			PersistentHLODSetup.RowDisplayName = "Non-Spatially Loaded HLODs";
			PersistentHLODSetup.HLODLayers = MoveTemp(PersistentHLODLayers);
			PersistentHLODSetup.bIsSpatiallyLoaded = false;
			PersistentHLODSetup.PartitionLayer->bClientOnlyVisible = !HLODPartitionMustBeLoadedOnServer(PersistentHLODSetup);
		}
	}
}

void UWorldPartitionRuntimeHashSet::ForEachHLODLayer(TFunctionRef<bool(FName, FName, int32)> Func) const
{
	for (const FRuntimePartitionDesc& RuntimePartition : RuntimePartitions)
	{
		if (RuntimePartition.MainLayer)
		{
			for (int32 HLODSetupIndex = 0; HLODSetupIndex < RuntimePartition.HLODSetups.Num(); HLODSetupIndex++)
			{
				if (!Func(RuntimePartition.Name, RuntimePartition.HLODSetups[HLODSetupIndex].Name, HLODSetupIndex))
				{
					return;
				}
			}
		}
	}
}

int32 UWorldPartitionRuntimeHashSet::ComputeHLODHierarchyDepth() const
{
	int32 MaxHLODDepth = 0;
	for (const FRuntimePartitionDesc& RuntimePartition : RuntimePartitions)
	{
		if (RuntimePartition.MainLayer)
		{
			for (const FRuntimePartitionHLODSetup& HLODSetup : RuntimePartition.HLODSetups)
			{
				for (const TObjectPtr<const UHLODLayer>& HLODLayer : HLODSetup.HLODLayers)
				{
					const UHLODLayer* CurrentHLODLayer = HLODLayer;
					int32 CurrentHLODDepth = 0;
					while (CurrentHLODLayer)
					{
						CurrentHLODDepth++;
						CurrentHLODLayer = CurrentHLODLayer->GetParentLayer();
					}
					MaxHLODDepth = FMath::Max(MaxHLODDepth, CurrentHLODDepth);
				}
			}
		}
	}
	
	return MaxHLODDepth;
}

FName UWorldPartitionRuntimeHashSet::ResolveRuntimeGridNameForHLODLayer(FName RuntimeGrid, const FSoftObjectPath& HLODLayerPath) const
{
	if (const URuntimePartition* RuntimePartition = ResolveRuntimePartitionForHLODLayer(RuntimeGrid, HLODLayerPath))
	{
		if (!RuntimePartition->SupportsHLODs())
		{
			return NAME_None;
		}

		return FName(RuntimeGrid.ToString() + TEXT(":") + RuntimePartition->Name.ToString());
	}
	return NAME_None;
}

bool UWorldPartitionRuntimeHashSet::CopyGridSetupFrom(const UWorldPartitionRuntimeHash* Other)
{
	if (const UWorldPartitionRuntimeHashSet* OtherHashSet = Cast<UWorldPartitionRuntimeHashSet>(Other))
	{
		RuntimePartitions.Empty();

		for (const FRuntimePartitionDesc& SourcePartitionDesc : OtherHashSet->RuntimePartitions)
		{
			FRuntimePartitionDesc& PartitionDesc = RuntimePartitions.AddDefaulted_GetRef();
			PartitionDesc.Name = SourcePartitionDesc.Name;
			PartitionDesc.MainLayer = DuplicateObject<URuntimePartition>(SourcePartitionDesc.MainLayer, this);

			for (const FRuntimePartitionHLODSetup& SourceHLODSetup : SourcePartitionDesc.HLODSetups)
			{
				FRuntimePartitionHLODSetup& HLODSetup = PartitionDesc.HLODSetups.AddDefaulted_GetRef();
				HLODSetup.Name = SourceHLODSetup.Name;
				HLODSetup.HLODLayers = SourceHLODSetup.HLODLayers;
				HLODSetup.PartitionLayer = DuplicateObject<URuntimePartition>(SourceHLODSetup.PartitionLayer, this);
			}
		}
		return true;
	}
	return false;
}

UWorldPartitionRuntimeHashSet::FCellUniqueId UWorldPartitionRuntimeHashSet::GetCellUniqueId(const URuntimePartition::FCellDescInstance& InCellDescInstance) const
{
	FCellUniqueId CellUniqueId;
	FName CellNameID = InCellDescInstance.Name;
	FDataLayersID DataLayersID(InCellDescInstance.DataLayerInstances);
	FGuid ContentBundleID(InCellDescInstance.ContentBundleID);

	// Build cell unique name
	{
		UWorld* OuterWorld = GetTypedOuter<UWorld>();
		check(OuterWorld);

		FString WorldName = FPackageName::GetShortName(OuterWorld->GetPackage());

		if (!IsRunningCookCommandlet() && OuterWorld->IsGameWorld())
		{
			FString SourceWorldPath;
			FString InstancedWorldPath;
			if (OuterWorld->GetSoftObjectPathMapping(SourceWorldPath, InstancedWorldPath))
			{
				const FTopLevelAssetPath SourceAssetPath(SourceWorldPath);
				WorldName = FPackageName::GetShortName(SourceAssetPath.GetPackageName());
						
				InstancedWorldPath = UWorld::RemovePIEPrefix(InstancedWorldPath);

				const FString SourcePackageName = SourceAssetPath.GetPackageName().ToString();
				const FTopLevelAssetPath InstanceAssetPath(InstancedWorldPath);
				const FString InstancePackageName = InstanceAssetPath.GetPackageName().ToString();

				if (int32 Index = InstancePackageName.Find(SourcePackageName); Index != INDEX_NONE)
				{
					CellUniqueId.InstanceSuffix = InstancePackageName.Mid(Index + SourcePackageName.Len());
				}
			}
		}

		TStringBuilder<128> CellNameBuilder;
		CellNameBuilder.Appendf(TEXT("%s_%s"), *WorldName, *CellNameID.ToString());

		if (DataLayersID.GetHash())
		{
			CellNameBuilder.Appendf(TEXT("_d%X"), DataLayersID.GetHash());
		}

		if (ContentBundleID.IsValid())
		{
			CellNameBuilder.Appendf(TEXT("_c%s"), *UContentBundleDescriptor::GetContentBundleCompactString(ContentBundleID));
		}

		CellUniqueId.Name = CellNameBuilder.ToString();
	}

	// Build cell guid
	{
		FArchiveMD5 ArMD5;
		ArMD5 << CellNameID << DataLayersID << ContentBundleID;
		InCellDescInstance.SourcePartition->AppendCellGuid(ArMD5);
		CellUniqueId.Guid = ArMD5.GetGuidFromHash();
		check(CellUniqueId.Guid.IsValid());
	}

	return CellUniqueId;
}
#endif

void UWorldPartitionRuntimeHashSet::ForEachStreamingData(TFunctionRef<bool(const FRuntimePartitionStreamingData&)> Func) const
{
	for (const FRuntimePartitionStreamingData& StreamingData : RuntimeStreamingData)
	{
		if (!Func(StreamingData))
		{
			return;
		}
	}

	for (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>& InjectedExternalStreamingObject : InjectedExternalStreamingObjects)
	{
		if (InjectedExternalStreamingObject.IsValid())
		{
			URuntimeHashSetExternalStreamingObject* ExternalStreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(InjectedExternalStreamingObject.Get());
			
			for (const FRuntimePartitionStreamingData& StreamingData : ExternalStreamingObject->RuntimeStreamingData)
			{
				if (!Func(StreamingData))
				{
					return;
				}
			}
		}
	}
}

void UWorldPartitionRuntimeHashSet::RemoveIrrelevantCells(FRuntimePartitionStreamingData& StreamingData)
{
	const int32 PreviousSpatialCellCount = StreamingData.SpatiallyLoadedCells.Num();

	StreamingData.SpatiallyLoadedCells.SetNum(Algo::RemoveIf(StreamingData.SpatiallyLoadedCells, [this](UWorldPartitionRuntimeCell* Cell) { return !IsCellRelevantFor(Cell->GetClientOnlyVisible()); }));
	StreamingData.NonSpatiallyLoadedCells.SetNum(Algo::RemoveIf(StreamingData.NonSpatiallyLoadedCells, [this](UWorldPartitionRuntimeCell* Cell) { return !IsCellRelevantFor(Cell->GetClientOnlyVisible()); }));

	// See URuntimeHashSetExternalStreamingObject::AddReferencedObjects - cells in spatial indices are not tracked via ARO in non-editor builds, will lead to deallocated mem access
	checkf(StreamingData.SpatiallyLoadedCells.Num() == PreviousSpatialCellCount || !StreamingData.HasSpatialIndex(),
		TEXT("Spatial cells removed after SpatialIndex creation."));
}

void UWorldPartitionRuntimeHashSet::UpdateRuntimeDataGridMap()
{
	RuntimeSpatiallyLoadedDataGridMap.Reset();
	RuntimeNonSpatiallyLoadedDataGridList.Reset();

	ForEachStreamingData([this](const FRuntimePartitionStreamingData& StreamingData)
	{
		if (StreamingData.SpatiallyLoadedCells.Num())
		{
			TArray<const FRuntimePartitionStreamingData*>& StreamingDataList = RuntimeSpatiallyLoadedDataGridMap.FindOrAdd(StreamingData.Name);
			StreamingDataList.Add(&StreamingData);
		}

		if (StreamingData.NonSpatiallyLoadedCells.Num())
		{
			RuntimeNonSpatiallyLoadedDataGridList.Add(&StreamingData);
		}
		return true;
	});
	
	if (RuntimeSpatiallyLoadedDataGridMap.IsEmpty())
	{
		RuntimeSpatiallyLoadedDataGridMap.Empty();
	}
	if (RuntimeNonSpatiallyLoadedDataGridList.IsEmpty())
	{
		RuntimeNonSpatiallyLoadedDataGridList.Empty();
	}
}

const FGuid* UWorldPartitionRuntimeHashSet::GetStandaloneHLODActorSourceCellOverride(const FGuid& InActorGuid) const
{
	return StandaloneHLODActorToSourceCellsMap.Find(InActorGuid);
}

const FGuid* UWorldPartitionRuntimeHashSet::GetCustomHLODActorSourceCellOverride(const FGuid& InActorGuid) const
{
	return CustomHLODActorToSourceCellsMap.Find(InActorGuid);
}

