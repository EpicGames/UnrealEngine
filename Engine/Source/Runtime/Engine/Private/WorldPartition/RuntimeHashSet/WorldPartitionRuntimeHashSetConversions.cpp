// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionLHGrid.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationNullErrorHandler.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#if WITH_EDITOR
UWorldPartitionRuntimeHashSet* UWorldPartitionRuntimeHashSet::CreateFrom(const UWorldPartitionRuntimeHash* SrcHash)
{
	const UWorldPartitionRuntimeSpatialHash* SpatialHash = CastChecked<UWorldPartitionRuntimeSpatialHash>(SrcHash);
	UWorldPartitionRuntimeHashSet* HashSet = NewObject<UWorldPartitionRuntimeHashSet>(SrcHash->GetOuter(), NAME_None, RF_Transactional);

	UWorldPartition* WorldPartition = SrcHash->GetTypedOuter<UWorldPartition>();
	check(WorldPartition);

	FStreamingGenerationNullErrorHandler NullErrorHandler;
	FActorDescContainerInstanceCollection Collection({ TObjectPtr<UActorDescContainerInstance>(WorldPartition->GetActorDescContainerInstance()) });
	UWorldPartition::FGenerateStreamingParams Params = UWorldPartition::FGenerateStreamingParams()
		.SetContainerInstanceCollection(Collection, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs)
		.SetErrorHandler(&NullErrorHandler);

	UWorldPartition::FGenerateStreamingContext Context;
	TUniquePtr<IStreamingGenerationContext> StreamingGenerationContext = WorldPartition->GenerateStreamingGenerationContext(Params, Context);
	if (StreamingGenerationContext.IsValid())
	{
		// Gather all HLOD layers into their corresponding grids
		TMap<FName, TMap<FName, TSet<const UHLODLayer*>>> GridHLODLayersMap;
		const UHLODLayer* DefaultHLODLayer = WorldPartition->GetDefaultHLODLayer();
		const bool bUseAlignedGridLevels = SpatialHash->GetUseAlignedGridLevels();

		for (const FSpatialHashRuntimeGrid& Grid : SpatialHash->Grids)
		{
			GridHLODLayersMap.Add(Grid.GridName);
		}

		StreamingGenerationContext->ForEachActorSetInstance([&GridHLODLayersMap, SpatialHash, DefaultHLODLayer](const IStreamingGenerationContext::FActorSetInstance& ActorSetInstance)
		{
			ActorSetInstance.ForEachActor([&GridHLODLayersMap, SpatialHash, DefaultHLODLayer, &ActorSetInstance](const FGuid& ActorGuid)
			{
				const IWorldPartitionActorDescInstanceView& ActorDescView = ActorSetInstance.ActorSetContainerInstance->ActorDescViewMap->FindByGuidChecked(ActorGuid);

				if (const UHLODLayer* HLODLayer = ActorDescView.GetHLODLayer().IsValid() ? Cast<UHLODLayer>(ActorDescView.GetHLODLayer().TryLoad()) : DefaultHLODLayer)
				{
					uint32 HLODIndex = 0;
					while (HLODLayer)
					{
						const FName GridName = ActorDescView.GetRuntimeGrid().IsNone() ? SpatialHash->Grids[0].GridName : ActorDescView.GetRuntimeGrid();

						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						const FName HLODGridName = HLODLayer->GetRuntimeGrid(HLODIndex);
						PRAGMA_ENABLE_DEPRECATION_WARNINGS

						if (!HLODGridName.IsNone())
						{
							GridHLODLayersMap.FindOrAdd(GridName).FindOrAdd(HLODGridName).Add(HLODLayer);
							HLODLayer = HLODLayer->GetParentLayer();
							HLODIndex++;
						}
						else
						{
							HLODLayer = nullptr;
						}
					}
				}
			});
		});

		for (const FSpatialHashRuntimeGrid& Grid : SpatialHash->Grids)
		{
			if (!Grid.HLODLayer)
			{
				FRuntimePartitionDesc& RuntimePartitionDesc = HashSet->RuntimePartitions.AddDefaulted_GetRef();
				RuntimePartitionDesc.Name = Grid.GridName;

				URuntimePartitionLHGrid* LHGrid = NewObject<URuntimePartitionLHGrid>(HashSet, NAME_None);
				LHGrid->Name = RuntimePartitionDesc.Name;
				LHGrid->CellSize = Grid.CellSize;
				LHGrid->Origin = bUseAlignedGridLevels ? FVector::ZeroVector : FVector(LHGrid->CellSize * -0.5f);
				LHGrid->bIs2D = true;
				LHGrid->bBlockOnSlowStreaming = Grid.bBlockOnSlowStreaming;
				LHGrid->bClientOnlyVisible = Grid.bClientOnlyVisible;
				LHGrid->Priority = Grid.Priority;
				LHGrid->LoadingRange = Grid.LoadingRange;
				LHGrid->HLODIndex = INDEX_NONE;

				RuntimePartitionDesc.MainLayer = LHGrid;

				for (const auto& [HLODGridName, HLODLayers] : GridHLODLayersMap.FindChecked(Grid.GridName))
				{
					const int32 HLODIndex = RuntimePartitionDesc.HLODSetups.Num();
					FRuntimePartitionHLODSetup& HLODSetup = RuntimePartitionDesc.HLODSetups.AddDefaulted_GetRef();

					HLODSetup.Name = HLODGridName;
					HLODSetup.HLODLayers = HLODLayers.Array();

					URuntimePartitionLHGrid* HLODLHGrid = NewObject<URuntimePartitionLHGrid>(HashSet, NAME_None);

					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					HLODLHGrid->CellSize = HLODSetup.HLODLayers[0]->GetCellSize();
					HLODLHGrid->LoadingRange = HLODSetup.HLODLayers[0]->GetLoadingRange();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					HLODLHGrid->Origin = bUseAlignedGridLevels ? FVector::ZeroVector : FVector(HLODLHGrid->CellSize * -0.5f);
					HLODLHGrid->bIs2D = true;
					HLODSetup.PartitionLayer = HLODLHGrid;

					HLODSetup.PartitionLayer->Name = HLODSetup.Name;
					HLODSetup.PartitionLayer->bBlockOnSlowStreaming = false;
					HLODSetup.PartitionLayer->bClientOnlyVisible = true;
					HLODSetup.PartitionLayer->Priority = 0;
					HLODSetup.PartitionLayer->HLODIndex = HLODIndex;
				}
			}
		}

		WorldPartition->FlushStreaming();
	}

	return HashSet;
}
#endif