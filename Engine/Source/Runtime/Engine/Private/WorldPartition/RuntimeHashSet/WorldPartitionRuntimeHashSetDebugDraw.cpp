// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "UObject/Package.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/WorldPartitionDraw2DContext.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "DrawDebugHelpers.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/Paths.h"

static int32 GShowRuntimeHashSetDebugDisplayLevel = 0;
static FAutoConsoleVariableRef CVarShowRuntimeHashSetDebugDisplayLevel(
	TEXT("wp.Runtime.HashSet.ShowDebugDisplayLevel"),
	GShowRuntimeHashSetDebugDisplayLevel,
	TEXT("Used to choose which level to display when showing runtime partitions."));

static int32 GShowRuntimeHashSetDebugDisplayLevelCount = 1;
static FAutoConsoleVariableRef CVarShowRuntimeHashSetDebugDisplayLevelCount(
	TEXT("wp.Runtime.HashSet.ShowDebugDisplayLevelCount"),
	GShowRuntimeHashSetDebugDisplayLevelCount,
	TEXT("Used to choose how many levels to display when showing runtime partitions."));

static int32 GShowRuntimeHashSetDebugDisplayMode = 0;
static FAutoConsoleVariableRef CVarShowRuntimeHashSetDebugDisplayMode(
	TEXT("wp.Runtime.HashSet.ShowDebugDisplayMode"),
	GShowRuntimeHashSetDebugDisplayMode,
	TEXT("Used to choose what mode to display when showing runtime partitions (0=Level Streaming State, 1=Data Layers, 2=Content Bundles, 3=Level Streaming State Without Data Layers/Content Bundles)."));

static EWorldPartitionRuntimeCellVisualizeMode GetStreamingCellVisualizeMode()
{
	const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = FWorldPartitionDebugHelper::IsRuntimeSpatialHashCellStreamingPriorityShown() ? EWorldPartitionRuntimeCellVisualizeMode::StreamingPriority : EWorldPartitionRuntimeCellVisualizeMode::StreamingStatus;
	return VisualizeMode;
}

static TMap<FName, FColor> GetDataLayerDebugColors(const UWorldPartition* InWorldPartition)
{
	TMap<FName, FColor> DebugColors;
	if (const UDataLayerManager* DataLayerManager = InWorldPartition->GetDataLayerManager())
	{
		DataLayerManager->ForEachDataLayerInstance([&DebugColors](UDataLayerInstance* DataLayerInstance)
		{
			DebugColors.Add(DataLayerInstance->GetFName(), DataLayerInstance->GetDebugColor());
			return true;
		});
	}
	return DebugColors;
}

static bool IsDebugCellShown(UWorldPartitionRuntimeCell* Cell)
{
	UWorldPartitionRuntimeCellData* RuntimeCellData = Cell->RuntimeCellData;
	return ((RuntimeCellData->HierarchicalLevel >= GShowRuntimeHashSetDebugDisplayLevel) &&
		(RuntimeCellData->HierarchicalLevel < (GShowRuntimeHashSetDebugDisplayLevel + GShowRuntimeHashSetDebugDisplayLevelCount)) &&
		(GShowRuntimeHashSetDebugDisplayMode != 3 || (!Cell->GetDataLayers().Num() && !Cell->GetContentBundleID().IsValid())) &&
		Cell->IsDebugShown());
}

bool UWorldPartitionRuntimeHashSet::Draw2D(FWorldPartitionDraw2DContext& DrawContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeHashSet::Draw2D);

	const UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	const TArray<FWorldPartitionStreamingSource>& Sources = WorldPartition->GetStreamingSources();
	
	if (!Sources.Num())
	{
		return false;
	}

	TMap<FName, TArray<const FRuntimePartitionStreamingData*>> FilteredStreamingObjects;
	ForEachStreamingData([&FilteredStreamingObjects](const FRuntimePartitionStreamingData& StreamingData)
	{
		if (StreamingData.SpatiallyLoadedCells.Num())
		{
			if (FWorldPartitionDebugHelper::IsDebugRuntimeHashGridShown(StreamingData.Name))
			{
				FilteredStreamingObjects.FindOrAdd(StreamingData.Name).Add(&StreamingData);
			}
		}
		return true;
	});

	if (!FilteredStreamingObjects.Num())
	{
		return false;
	}

	const FTransform WorldPartitionTransform = WorldPartition->GetInstanceTransform();
	const FTransform2D LocalWPToGlobal(FQuat2D(FMath::DegreesToRadians(WorldPartitionTransform.Rotator().Yaw)), FVector2D(WorldPartitionTransform.GetLocation()));
	const FTransform2D GlobalToLocalWP = LocalWPToGlobal.Inverse();
	const FBox2D& WorldRegion = DrawContext.GetWorldRegion();
	const FVector2D WorldRegionSize = WorldRegion.GetSize();
	TArray<FVector2D> Points;
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + FVector2D(WorldRegionSize.X, 0)));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + WorldRegionSize));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + FVector2D(0, WorldRegionSize.Y)));
	const FBox2D LocalRegion(Points);

	const FBox2D& CanvasRegion = DrawContext.GetCanvasRegion();
	const int32 GridScreenWidthDivider = DrawContext.IsDetailedMode() ? FilteredStreamingObjects.Num() : 1;
	const float GridScreenWidthShrinkSize = GridScreenWidthDivider > 1 ? 20.f : 0.f;
	const float CanvasMaxScreenWidth = CanvasRegion.GetSize().X;
	const float GridMaxScreenWidth = CanvasMaxScreenWidth / GridScreenWidthDivider;
	const float GridEffectiveScreenWidth = FMath::Min(GridMaxScreenWidth, CanvasRegion.GetSize().Y) - GridScreenWidthShrinkSize;
	const FVector2D PartitionCanvasSize = FVector2D(CanvasRegion.GetSize().GetMin());
	const FVector2D GridScreenExtent = FVector2D(GridEffectiveScreenWidth, GridEffectiveScreenWidth);
	const FVector2D GridScreenHalfExtent = 0.5f * GridScreenExtent;
	const FVector2D GridScreenInitialOffset = CanvasRegion.Min;

	float DynamicLoadingRangeScale2D = 1.f;
	float DynamicLoadingRangeScale3D = 1.f;
#if WITH_EDITOR
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld()))
	{
		const FDynamicLoadingRangeScales& Scales = WorldPartitionSubsystem->GetDynamicLoadingRangeScales();
		DynamicLoadingRangeScale2D = Scales.Scale2D;
		DynamicLoadingRangeScale3D = Scales.Scale3D;
	}
#endif // WITH_EDITOR

	auto DrawStreamingData = [this, &WorldPartitionTransform, DynamicLoadingRangeScale2D, DynamicLoadingRangeScale3D](const FRuntimePartitionStreamingData* StreamingData, const FBox2D& Region2D, const FBox2D& GridScreenBounds, TFunctionRef<FVector2D(const FVector2D&, bool)> InWorldToScreen, FWorldPartitionDraw2DContext& DrawContext)
	{
		const UWorldPartition* WorldPartition = GetOuterUWorldPartition();
		const TArray<FWorldPartitionStreamingSource>& Sources = WorldPartition->GetStreamingSources();

		const FBox Region(FVector(Region2D.Min.X, Region2D.Min.Y, 0), FVector(Region2D.Max.X, Region2D.Max.Y, 0));
		const UWorld* OwningWorld = WorldPartition->GetWorld();
		const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = GetStreamingCellVisualizeMode();
		const UContentBundleManager* ContentBundleManager = OwningWorld->ContentBundleManager;
		TMap<FName, FColor> DataLayerDebugColors = GetDataLayerDebugColors(WorldPartition);

		auto WorldToScreen = [&](const FVector2D& Pos, bool bIsLocal = true)
		{
			return InWorldToScreen(Pos, bIsLocal);
		};

		TArray<const UWorldPartitionRuntimeCell*> FilteredCells;
		const FBox Region3D(FVector(Region.Min.X, Region.Min.Y, -HALF_WORLD_MAX), FVector(Region.Max.X, Region.Max.Y, HALF_WORLD_MAX));

		auto FilterCell = [&FilteredCells](UWorldPartitionRuntimeCell* Cell)
		{
			if (IsDebugCellShown(Cell))
			{
				FilteredCells.Add(Cell);
			}
		};
		
		if (StreamingData->SpatialIndex.IsValid())
		{
			StreamingData->SpatialIndex->ForEachIntersectingElement(Region3D, FilterCell);
		}

		if (StreamingData->SpatialIndex2D.IsValid())
		{
			StreamingData->SpatialIndex2D->ForEachIntersectingElement(Region2D, FilterCell);
		}

		if (FilteredCells.Num())
		{
			for (const UWorldPartitionRuntimeCell* Cell : FilteredCells)
			{
				// Draw fixed cell bounds
				const FVector2D CellBoundsSize = FVector2D(Cell->GetCellBounds().GetSize());
				const FVector2D CellBoundsMin = FVector2D(Cell->GetCellBounds().Min);
				DrawContext.LocalDrawTile(GridScreenBounds, CellBoundsMin, CellBoundsSize, FLinearColor::Gray.CopyWithNewOpacity(0.05f), WorldToScreen);
				DrawContext.LocalDrawBox(GridScreenBounds, CellBoundsMin, CellBoundsSize, FLinearColor::Black.CopyWithNewOpacity(0.05f), 1, WorldToScreen);

				// Draw streaming cell bounds
				const FVector2D CellStreamingBoundsSize = FVector2D(Cell->GetStreamingBounds().GetSize());
				const FVector2D CellStreamingBoundsMin = FVector2D(Cell->GetStreamingBounds().Min);

				float CellOpacity = 0.0f;
				TArray<FLinearColor> CellColors;

				switch (GShowRuntimeHashSetDebugDisplayMode)
				{
				case 0:
				case 3:
					CellColors.Add(Cell->GetDebugColor(VisualizeMode));
					CellOpacity = (VisualizeMode == EWorldPartitionRuntimeCellVisualizeMode::StreamingPriority) ? 0.75f : 0.25f / FMath::Max<float>(GShowRuntimeHashSetDebugDisplayLevelCount, 1);
					break;
				case 1:
					if (DataLayerDebugColors.Num() && Cell->GetDataLayers().Num())
					{
						for (const FName& DataLayer : Cell->GetDataLayers())
						{
							CellColors.Add(DataLayerDebugColors[DataLayer]);
						}
						CellOpacity = 0.67f;
					}
					break;
				case 2:
					if (ContentBundleManager && Cell->GetContentBundleID().IsValid())
					{
						if (const FContentBundleBase* ContentBundle = ContentBundleManager->GetContentBundle(OwningWorld, Cell->GetContentBundleID()))
						{
							check(ContentBundle->GetDescriptor());
							CellColors.Add(ContentBundle->GetDescriptor()->GetDebugColor());
							CellOpacity = 0.67f;
						}
					}
					break;
				}

				if (CellColors.IsEmpty())
				{
					CellColors.Add(FLinearColor::White);
					CellOpacity = 0.1f;
				}

				FVector2D::FReal BoundsOffsetX = 0;
				const FVector2D::FReal BoundsOffsetStepX = CellStreamingBoundsSize.X / CellColors.Num();
				for (const FLinearColor& CellColor : CellColors)
				{
					const FVector2D EffectiveCellStreamingBoundsMin(CellStreamingBoundsMin.X + BoundsOffsetX, CellStreamingBoundsMin.Y);
					const FVector2D EffectiveCellStreamingBoundsSize(BoundsOffsetStepX, CellStreamingBoundsSize.Y);
					DrawContext.LocalDrawTile(GridScreenBounds, EffectiveCellStreamingBoundsMin, EffectiveCellStreamingBoundsSize, CellColor.CopyWithNewOpacity(CellOpacity), WorldToScreen);
					BoundsOffsetX += BoundsOffsetStepX;
				}
				
				DrawContext.LocalDrawBox(GridScreenBounds, CellStreamingBoundsMin, CellStreamingBoundsSize, FLinearColor::Black, 1, WorldToScreen);
			}
		}

		// Draw X/Y Axis
		if (DrawContext.GetDrawGridAxis())
		{
			DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(FVector2D(-1638400.f, 0.f), false), WorldToScreen(FVector2D(1638400.f, 0.f), false), FLinearColor::Red, 3);
			DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(FVector2D(0.f, -1638400.f), false), WorldToScreen(FVector2D(0.f, 1638400.f), false), FLinearColor::Green, 3);
		}

		// Draw Grid Bounds
		if (DrawContext.GetDrawGridBounds())
		{
			const FBox2D Bounds = GridScreenBounds.ExpandBy(FVector2D(10));
			const FVector2D Size = GridScreenBounds.GetSize();
			DrawContext.PushDrawBox(Bounds, GridScreenBounds.Min, GridScreenBounds.Min + FVector2D(Size.X, 0), GridScreenBounds.Max, GridScreenBounds.Min + FVector2D(0, Size.Y), FLinearColor::White, 1);
		}

		// Draw Streaming Sources
		for (const FWorldPartitionStreamingSource& Source : Sources)
		{
			const FColor Color = Source.GetDebugColor();
			Source.ForEachShape(StreamingData->GetLoadingRange(), StreamingData->Name, true, [&Color, &WorldToScreen, &GridScreenBounds, &DrawContext, &Source, DynamicLoadingRangeScale2D, DynamicLoadingRangeScale3D, this](const FSphericalSector& Shape)
			{
				check(!Shape.IsNearlyZero())

				float DynamicLoadingRangeScale = (Source.bForce2D) ? DynamicLoadingRangeScale2D : DynamicLoadingRangeScale3D;

				// Spherical Sector
				const FVector2D Center2D(FVector2D(Shape.GetCenter()));
				const FSphericalSector::FReal Angle = Shape.GetAngle();
				const int32 MaxSegments = FMath::Max(4, FMath::CeilToInt(64 * Angle / 360.f));
				const float AngleIncrement = Angle / MaxSegments;
				const FVector2D Axis = FVector2D(Shape.GetAxis());
				const FVector Startup = FRotator(0, -0.5f * Angle, 0).RotateVector(Shape.GetScaledAxis() * DynamicLoadingRangeScale);
				FVector2D LineStart = FVector2D(Startup);

				if (!Shape.IsSphere())
				{
					// Draw sector start axis
					DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + LineStart), Color, 2);
				}

				// Draw sector Arc
				for (int32 i = 1; i <= MaxSegments; i++)
				{
					FVector2D LineEnd = FVector2D(FRotator(0, AngleIncrement * i, 0).RotateVector(Startup));
					DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D + LineStart), WorldToScreen(Center2D + LineEnd), Color, 2);
					LineStart = LineEnd;
				}

				// If sphere, close circle, else draw sector end axis
				DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D + LineStart), WorldToScreen(Center2D + (Shape.IsSphere() ? FVector2D(Startup) : FVector2D::ZeroVector)), Color, 2);

				// Draw direction vector
				DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + Axis * Shape.GetRadius() * DynamicLoadingRangeScale), Color, 4);
			});

			// Draw velocity vector			
			const FVector2D Velocity2D = FVector2D(Source.Velocity);

			if (!Velocity2D.IsNearlyZero())
			{
				const FVector2D Center2D = FVector2D(Source.Location);
				DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + Velocity2D.GetSafeNormal() * StreamingData->GetLoadingRange() * 0.5f), Color, 1);
			}
		}
	};

	FBox GridsShapeBounds(ForceInit);
	FBox2D GridsBounds(ForceInit);
	int32 GridIndex = 0;
	for (auto& [Name, StreamingDataList] : FilteredStreamingObjects)
	{
		for (const FWorldPartitionStreamingSource& Source : Sources)
		{
			Source.ForEachShape(StreamingDataList[0]->GetLoadingRange(), Name, true, [&GridsShapeBounds](const FSphericalSector& Shape) { GridsShapeBounds += Shape.CalcBounds(); });
		}

		FVector2D GridReferenceWorldPos;
		FVector2D WorldRegionExtent;

		if (DrawContext.IsDetailedMode())
		{
			GridReferenceWorldPos = FVector2D(GridsShapeBounds.GetCenter());
			WorldRegionExtent = FVector2D(GridsShapeBounds.ExpandBy(GridsShapeBounds.GetExtent() * 0.1f).GetExtent().GetMax());
		}
		else
		{
			GridReferenceWorldPos = FVector2D(WorldRegion.GetCenter());
			WorldRegionExtent = FVector2D(WorldRegion.GetExtent().GetMax());
		}

		const FVector2D GridScreenOffset = GridScreenInitialOffset + ((float)GridIndex * FVector2D(GridMaxScreenWidth, 0.f)) + GridScreenHalfExtent + FVector2D(GridScreenWidthShrinkSize * 0.5f);
		const FVector2D WorldToScreenScale = GridScreenHalfExtent / WorldRegionExtent;
		const FBox2D GridScreenBounds(GridScreenOffset - GridScreenHalfExtent, GridScreenOffset + GridScreenHalfExtent);

		auto WorldToScreen = [&](const FVector2D& LocalWorldPos, bool bIsLocalWorldPos = true)
		{
			FVector2D GlocalPos = bIsLocalWorldPos ? LocalWPToGlobal.TransformPoint(LocalWorldPos) : LocalWorldPos;
			return (WorldToScreenScale * (GlocalPos - GridReferenceWorldPos)) + GridScreenOffset;
		};

		for (const FRuntimePartitionStreamingData* StreamingData : StreamingDataList)
		{
			DrawStreamingData(StreamingData, LocalRegion, GridScreenBounds, WorldToScreen, DrawContext);
		}

		GridsBounds += GridScreenBounds;

		if (DrawContext.IsDetailedMode())
		{
			FVector2D GridInfoPos = GridScreenOffset - GridScreenHalfExtent;
			FWorldPartitionCanvasMultiLineText MultiLineText;
			MultiLineText.Emplace(UWorld::RemovePIEPrefix(FPaths::GetBaseFilename(WorldPartition->GetPackage()->GetName())), FLinearColor::White);
			FString GridInfoText = FString::Printf(TEXT("%s | %d m"), *Name.ToString(), int32(StreamingDataList[0]->GetLoadingRange() * 0.01f));
			MultiLineText.Emplace(GridInfoText, FLinearColor::Yellow);
			FWorldPartitionCanvasMultiLineTextItem Item(GridInfoPos, MultiLineText);
			DrawContext.PushDrawText(Item);
		}

		++GridIndex;
	}

	FBox2D DesiredWorldBounds(ForceInit);
	if (GridsShapeBounds.IsValid)
	{
		// Convert to 2D
		FBox2D GridsShapeBounds2D = FBox2D(FVector2D(GridsShapeBounds.Min.X, GridsShapeBounds.Min.Y), FVector2D(GridsShapeBounds.Max.X, GridsShapeBounds.Max.Y));
		FVector2D CenterGlobalPos = LocalWPToGlobal.TransformPoint(GridsShapeBounds2D.GetCenter());
		// Use max extent of X/Y
		FVector2D Extent = FVector2D(GridsShapeBounds2D.GetExtent().GetMax());
		// Transform to global space
		GridsShapeBounds2D = FBox2D(CenterGlobalPos - Extent, CenterGlobalPos + Extent);
		// Expand by 10% computed bounds
		DesiredWorldBounds = GridsShapeBounds2D.ExpandBy(GridsShapeBounds2D.GetExtent() * 0.1f);
	}
	DrawContext.SetDesiredWorldBounds(DesiredWorldBounds);
	DrawContext.SetUsedCanvasBounds(GridsBounds);

	return true;
}

void UWorldPartitionRuntimeHashSet::Draw3D(const TArray<FWorldPartitionStreamingSource>& Sources) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeHashSet::Draw3D);

	const UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	const FTransform WorldPartitionTransform = WorldPartition->GetInstanceTransform();
	const UWorld* OwningWorld = WorldPartition->GetWorld();
	const UContentBundleManager* ContentBundleManager = OwningWorld->ContentBundleManager;
	const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = GetStreamingCellVisualizeMode();
	const TMap<FName, FColor> DataLayerDebugColors = GetDataLayerDebugColors(WorldPartition);

	float DynamicLoadingRangeScale2D = 1.f;
	float DynamicLoadingRangeScale3D = 1.f;
#if WITH_EDITOR
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld()))
	{
		const FDynamicLoadingRangeScales& Scales = WorldPartitionSubsystem->GetDynamicLoadingRangeScales();
		DynamicLoadingRangeScale2D = Scales.Scale2D;
		DynamicLoadingRangeScale3D = Scales.Scale3D;
	}
#endif // WITH_EDITOR

	auto DebugDrawCell3D = [WorldPartitionTransform, OwningWorld, ContentBundleManager, VisualizeMode, &DataLayerDebugColors](UWorldPartitionRuntimeCell* Cell)
	{
		if (!IsDebugCellShown(Cell))
		{
			return;
		}

		const FVector2D CellStreamingBoundsSize = FVector2D(Cell->GetStreamingBounds().GetSize());
		const FVector2D CellStreamingBoundsMin = FVector2D(Cell->GetStreamingBounds().Min);

		float CellOpacity = 0.0f;
		TArray<FLinearColor> CellColors;

		switch (GShowRuntimeHashSetDebugDisplayMode)
		{
		case 0:
		case 3:
			CellColors.Add(Cell->GetDebugColor(VisualizeMode));
			CellOpacity = 0.25f / FMath::Max<float>(GShowRuntimeHashSetDebugDisplayLevelCount, 1);
			break;
		case 1:
			if (DataLayerDebugColors.Num() && Cell->GetDataLayers().Num())
			{
				for (const FName& DataLayer : Cell->GetDataLayers())
				{
					CellColors.Add(DataLayerDebugColors[DataLayer]);
				}
				CellOpacity = 0.67f;
			}
			break;
		case 2:
			if (ContentBundleManager && Cell->GetContentBundleID().IsValid())
			{
				if (const FContentBundleBase* ContentBundle = ContentBundleManager->GetContentBundle(OwningWorld, Cell->GetContentBundleID()))
				{
					check(ContentBundle->GetDescriptor());
					CellColors.Add(ContentBundle->GetDescriptor()->GetDebugColor());
					CellOpacity = 0.67f;
				}
			}
			break;
		}

		if (CellColors.IsEmpty())
		{
			CellColors.Add(FLinearColor::White);
			CellOpacity = 0.1f;
		}

		// Draw Cell using its debug color
		const FBox Box(Cell->GetStreamingBounds());
		const FVector BoxCenter(Box.GetCenter());
		const FColor BoxColor(CellColors[0].CopyWithNewOpacity(CellOpacity).ToFColor(true));
		const FVector CellPos = WorldPartitionTransform.TransformPosition(BoxCenter);
		DrawDebugBox(OwningWorld, CellPos, Box.GetExtent(), WorldPartitionTransform.GetRotation(), BoxColor.WithAlpha(255), false, -1.f, 0, 20.f);
	};

	ForEachStreamingData([this, &Sources, VisualizeMode, &DataLayerDebugColors, ContentBundleManager, OwningWorld, &WorldPartitionTransform, &DebugDrawCell3D, DynamicLoadingRangeScale2D, DynamicLoadingRangeScale3D](const FRuntimePartitionStreamingData& StreamingData)
	{
		if (StreamingData.SpatialIndex)
		{
			for (const FWorldPartitionStreamingSource& Source : Sources)
			{
				Source.ForEachShape(StreamingData.GetLoadingRange(), StreamingData.Name, false, [this, &StreamingData, VisualizeMode, &DataLayerDebugColors, ContentBundleManager, OwningWorld, &WorldPartitionTransform, &DebugDrawCell3D, &Source, DynamicLoadingRangeScale2D, DynamicLoadingRangeScale3D](const FSphericalSector& Shape)
				{
					auto ForEachIntersectingElement = [&]<typename ShapeType>(const ShapeType& InShape)
					{
						StreamingData.SpatialIndex.Get()->ForEachIntersectingElement(InShape, [this, VisualizeMode, &DataLayerDebugColors, ContentBundleManager, OwningWorld, &WorldPartitionTransform, &DebugDrawCell3D](UWorldPartitionRuntimeCell* Cell)
						{
							DebugDrawCell3D(Cell);
						});
					};

					float DynamicLoadingRangeScale = (Source.bForce2D) ? DynamicLoadingRangeScale2D : DynamicLoadingRangeScale3D;

					if (Shape.IsSphere())
					{
						const FStaticSpatialIndex::FSphere Sphere(Shape.GetCenter(), Shape.GetRadius() * DynamicLoadingRangeScale);
						ForEachIntersectingElement(Sphere);
					}
					else
					{
						const FStaticSpatialIndex::FCone Cone(Shape.GetCenter(), Shape.GetAxis(), Shape.GetRadius() * DynamicLoadingRangeScale, Shape.GetAngle());
						ForEachIntersectingElement(Cone);
					}
				});
			}
		}

		if (StreamingData.SpatialIndex2D.IsValid())
		{
			for (const FWorldPartitionStreamingSource& Source : Sources)
			{
				const bool bCalcBoundsIn2D = true;
				const FBox SourceBounds = Source.CalcBounds(StreamingData.GetLoadingRange() * DynamicLoadingRangeScale2D, StreamingData.Name, bCalcBoundsIn2D);
				const FBox2D SourceBounds2D(FVector2D(SourceBounds.Min.X, SourceBounds.Min.Y), FVector2D(SourceBounds.Max.X, SourceBounds.Max.Y));
				StreamingData.SpatialIndex2D->ForEachIntersectingElement(SourceBounds2D, [&DebugDrawCell3D](UWorldPartitionRuntimeCell* Cell)
				{
					DebugDrawCell3D(Cell);
				});
			}
		}
		return true;
	});
}

bool UWorldPartitionRuntimeHashSet::ContainsRuntimeHash(const FString& Name) const
{
	const FRuntimePartitionDesc* FoundRuntimePartitionDesc = RuntimePartitions.FindByPredicate([&Name](const FRuntimePartitionDesc& RuntimePartitionDesc)
		{
			return RuntimePartitionDesc.Name.ToString().Equals(Name, ESearchCase::IgnoreCase);
		});

	return FoundRuntimePartitionDesc != nullptr;
}
