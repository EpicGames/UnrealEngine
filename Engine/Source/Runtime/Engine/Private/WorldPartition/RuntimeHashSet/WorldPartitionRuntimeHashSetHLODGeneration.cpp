// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/HLOD/HLODModifier.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"
#include "WorldPartition/HLOD/StandaloneHLODSubsystem.h"

#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"

#if WITH_EDITOR
#include "EditorLevelUtils.h"
#include "EditorWorldUtils.h"
#include "EngineUtils.h"
#endif

#include "UObject/GCObjectScopeGuard.h"
#include "UObject/SavePackage.h"
#include "HAL/PlatformFileManager.h"

#define LOCTEXT_NAMESPACE "WorldPartitionRuntimeHashSetHLODGeneration"

#if WITH_EDITOR
class FHLODStreamingGenerationContext : public IStreamingGenerationContext
{
	using FActorSetInstanceList = TArray<IStreamingGenerationContext::FActorSetInstance>;

public:
	FHLODStreamingGenerationContext()
		: WorldBounds(ForceInit)
	{}

	virtual FBox GetWorldBounds() const override
	{
		return WorldBounds;
	}

	virtual const FActorSetContainerInstance* GetActorSetContainerForContextBaseContainerInstance() const override
	{
		return &ActorSetContainerInstance;
	}

	virtual void ForEachActorSetInstance(TFunctionRef<void(const FActorSetInstance&)> Func) const override
	{
		for (const FActorSetInstance& ActorSetInstance : ActorSetInstanceList)
		{
			Func(ActorSetInstance);
		}
	}

	virtual void ForEachActorSetContainerInstance(TFunctionRef<void(const FActorSetContainerInstance&)> Func) const override
	{
		Func(ActorSetContainerInstance);
	}

	FBox WorldBounds;
	FActorSetContainerInstance ActorSetContainerInstance;
	FStreamingGenerationActorDescViewMap ActorDescViewMap;
	FActorSetInstanceList ActorSetInstanceList;
};

namespace PrivateUtils
{
	static void GameTick(UWorld* InWorld)
	{
		static int32 TickRendering = 0;
		static const int32 FlushRenderingFrequency = 256;

		// Perform a GC when memory usage exceeds a given threshold
		if (FWorldPartitionHelpers::ShouldCollectGarbage())
		{
			FWorldPartitionHelpers::DoCollectGarbage();
		}

		// When running with -AllowCommandletRendering we want to flush
		if (((++TickRendering % FlushRenderingFrequency) == 0) && IsAllowCommandletRendering())
		{
			FWorldPartitionHelpers::FakeEngineTick(InWorld);
		}
	}
	static void SavePackage(UPackage* Package, ISourceControlHelper* SourceControlHelper)
	{
		if (SourceControlHelper)
		{
			SourceControlHelper->Save(Package);
		}
		else
		{
			Package->MarkAsFullyLoaded();

			FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(Package->GetName());
			const FString PackageFileName = PackagePath.GetLocalFullPath();
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
			{
				UE_LOGF(LogWorldPartition, Error, "Error saving package %ls.", *Package->GetName());
				check(0);
			}
		}
	}

	static void SavePackages(const TArray<UPackage*>& Packages, ISourceControlHelper* SourceControlHelper)
	{
		if (SourceControlHelper)
		{
			SourceControlHelper->Save(Packages);
		}
		else
		{
			for (UPackage* Package : Packages)
			{
				SavePackage(Package, SourceControlHelper);
			}
		}
	}

	static void DeletePackage(const FString& PackageName, ISourceControlHelper* SourceControlHelper)
	{
		FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(PackageName);
		const FString PackageFileName = PackagePath.GetLocalFullPath();

		if (SourceControlHelper)
		{
			SourceControlHelper->Delete(PackageFileName);
		}
		else
		{
			FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PackageFileName);
		}
	}

	static void DeletePackages(const TArray<FString>& PackageNames, ISourceControlHelper* SourceControlHelper)
	{
		TArray<FString> PackagesFullPaths;

		Algo::Transform(PackageNames, PackagesFullPaths, [](const FString& PackageName)
		{
			FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(PackageName);
			return PackagePath.GetLocalFullPath();
		});

		if (SourceControlHelper)
		{
			SourceControlHelper->Delete(PackagesFullPaths);
		}
		else
		{
			for (const FString& PackageFullPath : PackagesFullPaths)
			{
				FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PackageFullPath);
			}
		}
	}

	static void DeletePackage(UPackage* Package, ISourceControlHelper* SourceControlHelper)
	{
		if (SourceControlHelper)
		{
			SourceControlHelper->Delete(Package);
		}
		else
		{
			DeletePackage(Package->GetName(), SourceControlHelper);
		}
	}

	static void DeletePackages(const TArray<UPackage*>& Packages, ISourceControlHelper* SourceControlHelper)
	{
		if (SourceControlHelper)
		{
			SourceControlHelper->Delete(Packages);
		}
		else
		{
			for (UPackage* Package : Packages)
			{
				DeletePackage(Package, SourceControlHelper);
			}
		}
	}

	static void DeletePackage(UWorldPartition* WorldPartition, const FWorldPartitionHandle& Handle, ISourceControlHelper* SourceControlHelper)
	{
		if (Handle.IsLoaded())
		{
			DeletePackage(Handle.GetActor()->GetPackage(), SourceControlHelper);
			WorldPartition->OnPackageDeleted(Handle.GetActor()->GetPackage());
		}
		else
		{
			DeletePackage(Handle->GetActorPackage().ToString(), SourceControlHelper);
			WorldPartition->RemoveActor(Handle->GetGuid());
		}
	}

	static void DeletePackages(UWorldPartition* WorldPartition, const TArray<FWorldPartitionHandle>& Handles, ISourceControlHelper* SourceControlHelper)
	{
		TArray<UPackage*> LoadedPackagesToDelete;
		TArray<FString> UnloadedPackagesToDelete;
		TArray<FGuid> UnloadedActorsToRemove;

		for (const FWorldPartitionHandle& Handle : Handles)
		{
			if (Handle.IsLoaded())
			{
				LoadedPackagesToDelete.Add(Handle.GetActor()->GetPackage());
			}
			else
			{
				UnloadedPackagesToDelete.Add(Handle->GetActorPackage().ToString());
				UnloadedActorsToRemove.Add(Handle->GetGuid());
			}
		}
		check(UnloadedPackagesToDelete.Num() == UnloadedActorsToRemove.Num());

		if (LoadedPackagesToDelete.Num())
		{
			DeletePackages(LoadedPackagesToDelete, SourceControlHelper);
			for (UPackage* Package : LoadedPackagesToDelete)
			{
				WorldPartition->OnPackageDeleted(Package);
			}
		}

		if (UnloadedPackagesToDelete.Num())
		{
			DeletePackages(UnloadedPackagesToDelete, SourceControlHelper);
			for (const FGuid& ActorGuid : UnloadedActorsToRemove)
			{
				WorldPartition->RemoveActor(ActorGuid);
			}
		}
	}
}

bool UWorldPartitionRuntimeHashSet::SupportsHLODs() const
{
	for (const FRuntimePartitionDesc& RuntimePartitionDesc : RuntimePartitions)
	{
		if (RuntimePartitionDesc.MainLayer)
		{
			if (RuntimePartitionDesc.MainLayer->SupportsHLODs())
			{
				return true;
			}
		}
	}

	return false;
}

bool UWorldPartitionRuntimeHashSet::SetupHLODActors(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params) const
{
	IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
	IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr;
	if (WPHLODUtilities == nullptr)
	{
		UE_LOGF(LogWorldPartition, Error, "%s requires plugin 'World Partition HLOD Utilities'.", __FUNCTION__);
		return false;
	}

	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
	IStreamingGenerationContext::FActorSetContainerInstance* BaseActorSetContainerInstance = const_cast<IStreamingGenerationContext::FActorSetContainerInstance*>(StreamingGenerationContext->GetActorSetContainerForContextBaseContainerInstance());
	const FStreamingGenerationContainerInstanceCollection* BaseContainerInstanceCollection = BaseActorSetContainerInstance->ContainerInstanceCollection;

	// Create the HLOD creation context
	FHLODCreationContext HLODCreationContext;
	TSet<FGuid> ContentBundleGuids;
	BaseContainerInstanceCollection->ForEachActorDescContainerInstance([&HLODCreationContext, &ContentBundleGuids, WorldPartition](const UActorDescContainerInstance* ActorDescContainerInstance)
	{
		ContentBundleGuids.Add(ActorDescContainerInstance->GetContentBundleGuid());
		for (UActorDescContainerInstance::TConstIterator<AWorldPartitionHLOD> HLODIterator(ActorDescContainerInstance); HLODIterator; ++HLODIterator)
		{
			FWorldPartitionHandle HLODActorHandle(WorldPartition, HLODIterator->GetGuid());
			HLODCreationContext.HLODActorDescs.Emplace(HLODIterator->GetActorName(), MoveTemp(HLODActorHandle));
		}
	});

	TArray<TUniquePtr<FWorldPartitionActorDesc> > UnsavedHLODActorDescs;
	TArray<TUniquePtr<FWorldPartitionActorDescInstance> > UnsavedHLODActorDescInstances;
	
	if (Params.bConsiderUnsavedHLODActors)
	{
		for (TActorIterator<AWorldPartitionHLOD> It(WorldPartition->GetWorld()); It; ++It)
		{
			if (!HLODCreationContext.HLODActorDescs.Contains(It->GetFName()))
			{
				HLODCreationContext.UnsavedHLODActors.Emplace(It->GetFName(), *It);
			}
		}
	}

	TUniquePtr<IStreamingGenerationContext> CurrentHLODStreamingGenerationContext = MakeUnique<FStreamingGenerationContextProxy>(StreamingGenerationContext);

	TArray<UPackage*> PackagesToSave;
	TArray<FWorldPartitionReference> ActorsToSaveReferences;

	int32 HLODLevel = 0;
	while (CurrentHLODStreamingGenerationContext)
	{
		UWorld* StandaloneHLODWorld = nullptr;
		if (WorldPartition->HasStandaloneHLOD())
		{
			UWorld* SourceWorld = WorldPartition->GetWorld();

			FString FolderPath, PackagePrefix;
			UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(SourceWorld->GetPackage()->GetName(), FolderPath, PackagePrefix);
			FString LODLevelPackageName = FString::Printf(TEXT("%s/%s%d"), *FolderPath, *PackagePrefix, HLODLevel);

			UPackage* LODPackage = LoadWorldPackageForEditor(LODLevelPackageName, EWorldType::Editor, LOAD_NoWarn);
			if (!LODPackage)
			{
				LODPackage = CreatePackage(*LODLevelPackageName);
			}
			LODPackage->FullyLoad();
			LODPackage->Modify();

			StandaloneHLODWorld = UWorld::FindWorldInPackage(LODPackage);
			if (!StandaloneHLODWorld)
			{
				// Create Standalone HLOD World if not found
				UWorld::InitializationValues IVS;
				IVS.RequiresHitProxies(false);
				IVS.ShouldSimulatePhysics(false);
				IVS.EnableTraceCollision(false);
				IVS.CreateNavigation(false);
				IVS.CreateAISystem(false);
				IVS.AllowAudioPlayback(false);
				IVS.CreatePhysicsScene(true);
				IVS.CreateWorldPartition(true);

				StandaloneHLODWorld = UWorld::CreateWorld(EWorldType::Editor, false, FPackageName::GetShortFName(LODPackage->GetFName()), LODPackage, false, /*InFeatureLevel=*/ERHIFeatureLevel::Num, &IVS, /*bInSkipInitWorld=*/ true);
				StandaloneHLODWorld->SetFlags(RF_Public | RF_Standalone);

				UWorldPartition* StandaloneHLODWorldPartition = StandaloneHLODWorld->GetWorldPartition();
				StandaloneHLODWorldPartition->SetIsStandaloneHLODWorld(true);
				StandaloneHLODWorldPartition->SetDefaultHLODLayer(nullptr);

				// Create RuntimePartitions setup in the Standalone HLOD world based on the source world
				if (!StandaloneHLODWorldPartition->RuntimeHash->CopyGridSetupFrom(WorldPartition->RuntimeHash))
				{
					UE_LOGF(LogWorldPartition, Warning, "Standalone HLOD World Init - failed to copy grid setup from the source world");
				}

				// Save World Data Layers Package
				UPackage* WorldDataLayersPackage = StandaloneHLODWorld->PersistentLevel->GetWorldDataLayers()->GetPackage();
				PrivateUtils::SavePackage(WorldDataLayersPackage, Params.SourceControlHelper);

				PrivateUtils::SavePackage(LODPackage, Params.SourceControlHelper);
			}

			UWorldPartition* StandaloneHLODWorldPartition = StandaloneHLODWorld->GetWorldPartition();

			if (!StandaloneHLODWorldPartition->IsInitialized())
			{
				StandaloneHLODWorldPartition->Initialize(StandaloneHLODWorld, FTransform::Identity);
			}

			// Fixup actor folders
			if (StandaloneHLODWorld->PersistentLevel->IsUsingActorFolders())
			{
				if (!StandaloneHLODWorld->PersistentLevel->LoadedExternalActorFolders.IsEmpty())
				{
					StandaloneHLODWorld->PersistentLevel->bFixupActorFoldersAtLoad = false;
					StandaloneHLODWorld->PersistentLevel->FixupActorFolders();
				}
			}

			// If necessary, update Standalone HLOD world data layers based on source world
			bool bDataLayersChanged = false;
			SourceWorld->GetWorldDataLayers()->ForEachDataLayerInstance([StandaloneHLODWorld, &bDataLayersChanged](UDataLayerInstance* DataLayer)
			{
				if (DataLayer->IsA<UDataLayerInstanceWithAsset>() && StandaloneHLODWorld->GetWorldDataLayers()->GetDataLayerInstance(DataLayer->GetAsset()) == nullptr)
				{
					StandaloneHLODWorld->GetWorldDataLayers()->CreateDataLayer<UDataLayerInstanceWithAsset>(DataLayer->GetAsset());
					bDataLayersChanged = true;
				}
				return true;
			});
			if (bDataLayersChanged)
			{
				UPackage* WorldDataLayersPackage = StandaloneHLODWorld->PersistentLevel->GetWorldDataLayers()->GetPackage();
				PrivateUtils::SavePackage(WorldDataLayersPackage, Params.SourceControlHelper);
			}

			// Add Standalone HLOD world to be processed by World Partition builder
			Params.OutAdditionalWorldPartitionsForStandaloneHLOD.AddUnique(StandaloneHLODWorldPartition);

			StandaloneHLODWorldPartition->ForEachActorDescContainerInstance([&HLODCreationContext, &ContentBundleGuids, StandaloneHLODWorldPartition](const UActorDescContainerInstance* ActorDescContainerInstance)
			{
				if (ContentBundleGuids.Contains(ActorDescContainerInstance->GetContentBundleGuid()))
				{
					for (FActorDescContainerInstanceCollection::TIterator<AWorldPartitionHLOD> HLODIterator(StandaloneHLODWorldPartition); HLODIterator; ++HLODIterator)
					{
						FWorldPartitionHandle HLODActorHandle(StandaloneHLODWorldPartition, HLODIterator->GetGuid());
						HLODCreationContext.HLODActorDescs.Emplace(HLODIterator->GetActorName(), MoveTemp(HLODActorHandle));
					}
				}
			});
		}

		// Prevent StandaloneHLODWorld from being garbage collected during GameTick calls
		FGCObjectScopeGuard StandaloneHLODWorldGuard(StandaloneHLODWorld);

		TMap<URuntimePartition*, TArray<URuntimePartition::FCellDescInstance>> RuntimePartitionsStreamingDescs;
		GenerateRuntimePartitionsStreamingDescs(CurrentHLODStreamingGenerationContext.Get(), RuntimePartitionsStreamingDescs);

		int32 NumNextLayerHLODActors = 0;
		TArray<FGuid> HLODActorGuids;
		for (auto& [RuntimePartition, CellDescInstances] : RuntimePartitionsStreamingDescs)
		{
			int32 CellDescInstanceIndex = 0;
			for (URuntimePartition::FCellDescInstance& CellDescInstance : CellDescInstances)
			{
				// Skip non-spatially loaded cells as they require no HLOD representation
				if (!CellDescInstance.bIsSpatiallyLoaded)
				{
					continue;
				}

				const FCellUniqueId CellUniqueId = GetCellUniqueId(CellDescInstance);

				UE_LOGF(LogWorldPartition, Display, "[%d / %d] %ls %ls...", ++CellDescInstanceIndex, CellDescInstances.Num(), *LOCTEXT("ProcessingCell", "Processing cell").ToString(), *CellUniqueId.Name);

				TArray<IStreamingGenerationContext::FActorInstance> ActorInstances;
				for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : CellDescInstance.ActorSetInstances)
				{
					ActorSetInstance->ForEachActor([this, ActorSetInstance, &ActorInstances](const FGuid& ActorGuid)
					{
						ActorInstances.Emplace(ActorGuid, ActorSetInstance);
					});
				}

				// Fake tick
				PrivateUtils::GameTick(WorldPartition->GetWorld());

				// Resolve main partition
				FName ActorsRuntimeGrid = ActorInstances[0].ActorSetInstance->RuntimeGrid;
				const URuntimePartition* MainRuntimePartition = ResolveRuntimePartition(ActorsRuntimeGrid, /*bMainPartitionLayer*/ true);
				check(MainRuntimePartition);
				
				// Retrieve the runtime grid to use for this HLOD
				auto GetHLODRuntimeGrid = [this, MainRuntimePartition](const UHLODLayer* InHLODLayer) -> FName
				{					
					const URuntimePartition* HLODRuntimePartition = ResolveRuntimePartitionForHLODLayer(MainRuntimePartition->Name, InHLODLayer);
					if (!HLODRuntimePartition)
					{
						return NAME_None;
					}

					if (HLODRuntimePartition->IsA<URuntimePartitionPersistent>())
					{
						return NAME_None;
					}

					return FName(*FString::Printf(TEXT("%s:%s"), *MainRuntimePartition->Name.ToString(), *HLODRuntimePartition->Name.ToString()));
				};				

				FHLODCreationParams HLODCreationParams;
				HLODCreationParams.WorldPartition = WorldPartition;
				HLODCreationParams.TargetWorld = WorldPartition->GetWorld();
				HLODCreationParams.CellName = CellUniqueId.Name;
				HLODCreationParams.CellGuid = CellUniqueId.Guid;
				HLODCreationParams.GetRuntimeGrid = GetHLODRuntimeGrid;
				HLODCreationParams.HLODLevel = HLODLevel;
				HLODCreationParams.MinVisibleDistance = RuntimePartition->LoadingRange;
				HLODCreationParams.ContentBundleGuid = CellDescInstance.ContentBundleID;
				HLODCreationParams.DataLayerInstances = CellDescInstance.DataLayerInstances;
				HLODCreationParams.bIsStandalone = false;
				HLODCreationParams.Filters = Params.Filters;

				if (WorldPartition->HasStandaloneHLOD())
				{
					HLODCreationParams.TargetWorld = StandaloneHLODWorld;
					HLODCreationParams.bIsStandalone = true;

					// Map data layers from the source world to data layers from Standalone HLOD world
					HLODCreationParams.DataLayerInstances.Empty();
					for (const UDataLayerInstance* DataLayerInstance : CellDescInstance.DataLayerInstances)
					{
						if (const UDataLayerInstanceWithAsset* DataLayerInstanceWithAsset = Cast<UDataLayerInstanceWithAsset>(DataLayerInstance))
						{
							if (const UDataLayerAsset* DataLayerAsset = DataLayerInstanceWithAsset->GetAsset())
							{
								if (const UDataLayerInstance* StandaloneHLODWorldDataLayerInstance = StandaloneHLODWorld->GetWorldDataLayers()->GetDataLayerInstance(DataLayerAsset))
								{
									HLODCreationParams.DataLayerInstances.Add(StandaloneHLODWorldDataLayerInstance);
								}
								else
								{
									UE_LOGF(LogWorldPartition, Log, "Couldn't find data layer %ls in Standalone HLOD world", *DataLayerInstanceWithAsset->GetDataLayerFullName());
								}
							}
						}
					}

					UWorld* SourceWorld = WorldPartition->GetWorld();
					
					if (WorldPartition->ShouldExternalizeHLODAssets())
					{
						FString FolderPath, PackagePrefix;
						UWorldPartitionStandaloneHLODSubsystem::GetStandaloneHLODFolderPathAndPackagePrefix(SourceWorld->GetPackage()->GetName(), FolderPath, PackagePrefix);
						HLODCreationParams.HLODResourcesFolder = FName(FString::Printf(TEXT("%s/Resources/%s%d/"), *FolderPath, *PackagePrefix, HLODLevel));
					}
				}

				TArray<AWorldPartitionHLOD*> CellHLODActors = WPHLODUtilities->CreateHLODActors(HLODCreationContext, HLODCreationParams, ActorInstances);

				if (!CellHLODActors.IsEmpty())
				{
					for (AWorldPartitionHLOD* CellHLODActor : CellHLODActors)
					{
						FGuid ActorGuid = CellHLODActor->GetActorGuid();

						UPackage* CellHLODActorPackage = CellHLODActor->GetPackage();
						if (CellHLODActorPackage->HasAnyPackageFlags(PKG_NewlyCreated))
						{
							// Get a reference to newly create actors so they get unloaded when we release the references
							HLODCreationContext.ActorReferences.Emplace(WorldPartition, CellHLODActor->GetActorGuid());
						}

						HLODActorGuids.Add(ActorGuid);
						NumNextLayerHLODActors += CellHLODActor->GetHLODLayer() ? 1 : 0;
					}

					if (!Params.bReportOnly && Params.bSaveActors)
					{
						for (AWorldPartitionHLOD* CellHLODActor : CellHLODActors)
						{
							if (CellHLODActor->GetPackage()->IsDirty())
							{
								PackagesToSave.Add(CellHLODActor->GetPackage());
								ActorsToSaveReferences.Emplace(WorldPartition, CellHLODActor->GetActorGuid());
							}
						}
					}
				}

				// Unload actors
				HLODCreationContext.ActorReferences.Empty();
			}
		}

		if (PackagesToSave.Num() > 0)
		{
			PrivateUtils::SavePackages(PackagesToSave, Params.SourceControlHelper);
			PackagesToSave.Empty();
		}
		ActorsToSaveReferences.Empty();

		CurrentHLODStreamingGenerationContext.Reset();

		// Build the next HLOD generation context
		if (NumNextLayerHLODActors)
		{
			TMap<FGuid, TWeakObjectPtr<AWorldPartitionHLOD> > UnsavedHLODActors;
			if (Params.bConsiderUnsavedHLODActors)
			{
				for (TActorIterator<AWorldPartitionHLOD> It(WorldPartition->GetWorld()); It; ++It)
				{	
					if (It->GetPackage() && It->GetPackage()->IsDirty())
					{
						UnsavedHLODActors.Add(It->GetActorGuid(), *It);
					}
				}
			}

			CurrentHLODStreamingGenerationContext = MakeUnique<FHLODStreamingGenerationContext>();
			FHLODStreamingGenerationContext* HLODStreamingGenerationContext = (FHLODStreamingGenerationContext*)CurrentHLODStreamingGenerationContext.Get();

			HLODStreamingGenerationContext->ActorSetInstanceList.Reserve(HLODActorGuids.Num());
			HLODStreamingGenerationContext->ActorSetContainerInstance.ActorDescViewMap = &HLODStreamingGenerationContext->ActorDescViewMap;

			UWorldPartition* CurrentWorldPartition = nullptr;
			if (WorldPartition->HasStandaloneHLOD())
			{
				CurrentWorldPartition = StandaloneHLODWorld->GetWorldPartition();
				DataLayerManager = CurrentWorldPartition->GetDataLayerManager();
			}
			else
			{
				CurrentWorldPartition = WorldPartition;
			}

			UE_LOGF(LogWorldPartition, Log, "%ls", *LOCTEXT("CreatingHLODContext:", "Creating HLOD context:").ToString());
			for (const FGuid& HLODActorGuid : HLODActorGuids)
			{
				FWorldPartitionActorDescInstance* HLODActorDescInstance = nullptr;
				if (Params.bConsiderUnsavedHLODActors)
				{
					if (TWeakObjectPtr<AWorldPartitionHLOD>* UnsavedHLODActorPtr = UnsavedHLODActors.Find(HLODActorGuid))
					{
						if (AWorldPartitionHLOD* UnsavedHLODActor = UnsavedHLODActorPtr->Get())
						{
							if (UActorDescContainerInstance* HandlingContainer = const_cast<UActorDescContainerInstance*>(BaseContainerInstanceCollection->FindHandlingContainerInstance(UnsavedHLODActor).Get()))
							{
								TUniquePtr<FWorldPartitionActorDesc>& UnsavedHLODActorDesc = UnsavedHLODActorDescs.Add_GetRef(UnsavedHLODActor->CreateActorDesc());
								TUniquePtr<FWorldPartitionActorDescInstance>& UnsavedHLODActorDescInstance = UnsavedHLODActorDescInstances.Add_GetRef(MakeUnique<FWorldPartitionActorDescInstance>(HandlingContainer, UnsavedHLODActorDesc.Get()));
								DataLayerManager->ResolveActorDescInstanceDataLayers(UnsavedHLODActorDescInstance.Get());

								HLODActorDescInstance = UnsavedHLODActorDescInstance.Get();
							}
						}
					}
				}

				if (!HLODActorDescInstance)
				{
					HLODActorDescInstance = CurrentWorldPartition->GetActorDescInstance(HLODActorGuid);
				}
				check(HLODActorDescInstance);

				FStreamingGenerationActorDescView* HLODActorDescView = HLODStreamingGenerationContext->ActorDescViewMap.Emplace(HLODActorDescInstance);
				HLODStreamingGenerationContext->WorldBounds += HLODActorDescView->GetRuntimeBounds();
			
				// Create actor set instances
				IStreamingGenerationContext::FActorSet* ActorSet = HLODStreamingGenerationContext->ActorSetContainerInstance.ActorSets.Emplace_GetRef(MakeUnique<IStreamingGenerationContext::FActorSet>()).Get();
				ActorSet->Actors.Add(HLODActorDescView->GetGuid());

				IStreamingGenerationContext::FActorSetInstance& ActorSetInstance = HLODStreamingGenerationContext->ActorSetInstanceList.Emplace_GetRef();

				ActorSetInstance.Bounds = HLODActorDescView->GetRuntimeBounds();
				ActorSetInstance.RuntimeGrid = HLODActorDescView->GetRuntimeGrid();
				ActorSetInstance.bIsSpatiallyLoaded = HLODActorDescView->GetIsSpatiallyLoaded();
				ActorSetInstance.ContentBundleID = BaseContainerInstanceCollection->GetContentBundleGuid();
				ActorSetInstance.ActorSetContainerInstance = &HLODStreamingGenerationContext->ActorSetContainerInstance;
				ActorSetInstance.ActorSet = ActorSet;

				FDataLayerInstanceNames RuntimeDataLayerInstanceNames;
				if (FDataLayerUtils::ResolveRuntimeDataLayerInstanceNames(DataLayerManager, *HLODActorDescView, *BaseActorSetContainerInstance->DataLayerResolvers, RuntimeDataLayerInstanceNames))
				{
					HLODActorDescView->SetRuntimeDataLayerInstanceNames(RuntimeDataLayerInstanceNames);
					ActorSetInstance.DataLayers = DataLayerManager->GetRuntimeDataLayerInstances(RuntimeDataLayerInstanceNames.ToArray());
				}

				UE_LOGF(LogWorldPartition, Log, "\t- %ls", *HLODActorDescInstance->ToString());
			}

			HLODLevel++;
		}
	}

	// Destroy all unreferenced HLOD actors
	if (!Params.bReportOnly)
	{
		TArray<UPackage*> PackagesToDelete;
		TArray<FWorldPartitionHandle> ActorHandlesToDelete;

		for (const auto& HLODActorPair : HLODCreationContext.HLODActorDescs)
		{
			check(HLODActorPair.Value.IsValid());

			FHLODCreationFilterContext FilterContext;
			FilterContext.Bounds = HLODActorPair.Value->GetRuntimeBounds();
			const bool bPassesAllFilters = UE::HLOD::CreationFilter::PassesFilters(Params.Filters, FilterContext);

			if (bPassesAllFilters)
			{
				if (Params.bSaveActors)
				{
					if (WorldPartition->ShouldExternalizeHLODAssets())
					{
						FWorldPartitionReference HLODActorRef = HLODActorPair.Value.ToReference();
						if (AWorldPartitionHLOD* HLODActor = CastChecked<AWorldPartitionHLOD>(HLODActorRef.GetActor()))
						{
							if (UPackage* HLODResourcesPackage = HLODActor->GetHLODResourcesPackage())
							{	
								if (HLODResourcesPackage != HLODActor->GetPackage())
								{
									PackagesToDelete.Add(HLODResourcesPackage);
								}
							}
						}
					}
					
					ActorHandlesToDelete.Add(HLODActorPair.Value);
				}
				else
				{
					FWorldPartitionReference HLODActorRef = HLODActorPair.Value.ToReference();
					AWorldPartitionHLOD* HLODActor = CastChecked<AWorldPartitionHLOD>(HLODActorRef.GetActor());
					
					ResetLoaders(HLODActor);
					WorldPartition->GetWorld()->EditorDestroyActor(HLODActor, /*bShouldModifyLevel=*/false);
					WorldPartition->OnPackageDeleted(HLODActor->GetPackage());
					HLODActor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}
			}
		}

		if (PackagesToDelete.Num())
		{
			PrivateUtils::DeletePackages(PackagesToDelete, Params.SourceControlHelper);
		}

		if (ActorHandlesToDelete.Num())
		{
			PrivateUtils::DeletePackages(WorldPartition, ActorHandlesToDelete, Params.SourceControlHelper);
		}
	}

	return false;
}
#endif

#undef LOCTEXT_NAMESPACE