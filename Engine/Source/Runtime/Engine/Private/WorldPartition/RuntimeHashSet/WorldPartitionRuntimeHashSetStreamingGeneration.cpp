// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeCellDataHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartition.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActorDesc.h"
#include "WorldPartition/HLOD/StandaloneHLODSubsystem.h"

#if WITH_EDITOR
bool UWorldPartitionRuntimeHashSet::GenerateRuntimePartitionsStreamingDescs(const IStreamingGenerationContext* StreamingGenerationContext, TMap<URuntimePartition*, TArray<URuntimePartition::FCellDescInstance>>& OutRuntimeCellDescs) const
{
	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	UWorld* World = WorldPartition->GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const bool bIsMainWorldPartition = (World == OuterWorld);

	if (RuntimePartitions.IsEmpty())
	{
		return false;
	}

	//
	// Split actor sets into their corresponding runtime partition implementation
	//
	TMap<FName, URuntimePartition*> NameToRuntimePartitionMap;
		
	TMap<URuntimePartition*, TArray<const IStreamingGenerationContext::FActorSetInstance*>> RuntimePartitionsToActorSetMap;
	StreamingGenerationContext->ForEachActorSetInstance([this, &NameToRuntimePartitionMap, &RuntimePartitionsToActorSetMap](const IStreamingGenerationContext::FActorSetInstance& ActorSetInstance)
	{
		URuntimePartition* RuntimePartition = nullptr;
		URuntimePartition** RuntimePartitionPtr = NameToRuntimePartitionMap.Find(ActorSetInstance.RuntimeGrid);

		if (RuntimePartitionPtr)
		{
			RuntimePartition = *RuntimePartitionPtr;
		}
		else
		{
			RuntimePartition = const_cast<URuntimePartition*>(ResolveRuntimePartition(ActorSetInstance.RuntimeGrid));// @todo-ow: GenerateStreaming() requires a non-const URuntimePartition object
			NameToRuntimePartitionMap.Emplace(ActorSetInstance.RuntimeGrid, RuntimePartition);
		}

		if (RuntimePartition)
		{
			RuntimePartitionsToActorSetMap.FindOrAdd(RuntimePartition).Add(&ActorSetInstance);
		}
	});

	//
	// Generate runtime partitions streaming data
	//
	TMap<URuntimePartition*, TArray<URuntimePartition::FCellDesc>> RuntimePartitionsStreamingDescs;
	for (auto& [RuntimePartition, ActorSetInstances] : RuntimePartitionsToActorSetMap)
	{
		// Gather runtime partition cell descs
		URuntimePartition::FGenerateStreamingParams GenerateStreamingParams;
		GenerateStreamingParams.ActorSetInstances = &ActorSetInstances;

		URuntimePartition::FGenerateStreamingResult GenerateStreamingResult;

		if (!RuntimePartition->GenerateStreaming(GenerateStreamingParams, GenerateStreamingResult))
		{
			return false;
		}

		RuntimePartitionsStreamingDescs.Add(RuntimePartition, GenerateStreamingResult.RuntimeCellDescs);
	}

	TSet<FName> CellDescsNames;
	for (auto& [RuntimePartition, RuntimeCellDescs] : RuntimePartitionsStreamingDescs)
	{
		for (const URuntimePartition::FCellDesc& RuntimeCellDesc : RuntimeCellDescs)
		{
			TMap<FDataLayersID, URuntimePartition::FCellDescInstance> RuntimeCellDescsInstancesSet;

			bool bCellNameExists;
			CellDescsNames.Add(RuntimeCellDesc.Name, &bCellNameExists);
			check(!bCellNameExists);

			// Split cell descs into data layers
			for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : RuntimeCellDesc.ActorSetInstances)
			{
				const FDataLayersID DataLayersID(ActorSetInstance->DataLayers);
				URuntimePartition::FCellDescInstance* CellDescInstance = RuntimeCellDescsInstancesSet.Find(DataLayersID);

				if (!CellDescInstance)
				{
					CellDescInstance = &RuntimeCellDescsInstancesSet.Emplace(DataLayersID, URuntimePartition::FCellDescInstance(RuntimeCellDesc, RuntimePartition, ActorSetInstance->DataLayers, ActorSetInstance->ContentBundleID));
					CellDescInstance->ActorSetInstances.Empty();
				}

				CellDescInstance->ActorSetInstances.Add(ActorSetInstance);
			}

			for (auto& [DataLayersID, CellDescInstance] : RuntimeCellDescsInstancesSet)
			{
				OutRuntimeCellDescs.FindOrAdd(RuntimePartition).Add(CellDescInstance);
			}
		}
	}

	return true;
}

bool UWorldPartitionRuntimeHashSet::GenerateStreaming(UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate)
{
	verify(Super::GenerateStreaming(StreamingPolicy, StreamingGenerationContext, OutPackagesToGenerate));

	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	UWorld* World = WorldPartition->GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const bool bIsMainWorldPartition = (World == OuterWorld);

	// Get container name
	const FString ContainerPackageName = StreamingGenerationContext->GetActorSetContainerForContextBaseContainerInstance()->ContainerInstanceCollection->GetBaseContainerInstancePackageName().ToString();
	FString ContainerShortName = FPackageName::GetShortName(ContainerPackageName);
	if (!ContainerPackageName.StartsWith(TEXT("/Game/")))
	{
		TArray<FString> SplitContainerPath;
		if (ContainerPackageName.ParseIntoArray(SplitContainerPath, TEXT("/")))
		{
			ContainerShortName += TEXT(".");
			ContainerShortName += SplitContainerPath[0];
		}
	}

	//
	// Generate runtime partitions streaming cell descriptors
	//
	TMap<URuntimePartition*, TArray<URuntimePartition::FCellDescInstance>> RuntimePartitionsStreamingDescs;
	GenerateRuntimePartitionsStreamingDescs(StreamingGenerationContext, RuntimePartitionsStreamingDescs);

	//
	// Create and populate streaming object
	//

	// Generate runtime cells
	auto CreateRuntimeCellFromCellDesc = [this](const URuntimePartition::FCellDescInstance& CellDescInstance, TSubclassOf<UWorldPartitionRuntimeCell> CellClass, TSubclassOf<UWorldPartitionRuntimeCellDataHashSet> CellDataClass)
	{
		const FCellUniqueId CellUniqueId = GetCellUniqueId(CellDescInstance);

		UWorldPartitionRuntimeCell* RuntimeCell = CreateRuntimeCell(CellClass, CellDataClass, CellUniqueId.Name, CellUniqueId.InstanceSuffix);

		RuntimeCell->SetDataLayers(CellDescInstance.DataLayerInstances);
		RuntimeCell->SetContentBundleUID(CellDescInstance.ContentBundleID);
		RuntimeCell->SetClientOnlyVisible(CellDescInstance.bClientOnlyVisible);
		const bool bIsHLOD = CellDescInstance.SourcePartition->HLODIndex != INDEX_NONE;
		const bool bBlockOnSlowStreaming = ResolveBlockOnSlowStreamingForCell(CellDescInstance.bBlockOnSlowStreaming, bIsHLOD, CellDescInstance.DataLayerInstances);
		RuntimeCell->SetBlockOnSlowLoading(bBlockOnSlowStreaming);
		RuntimeCell->SetIsHLOD(bIsHLOD);
		RuntimeCell->SetGuid(CellUniqueId.Guid);
		RuntimeCell->SetCellDebugColor(CellDescInstance.SourcePartition->DebugColor);

		UWorldPartitionRuntimeCellDataHashSet* RuntimeCellData = CastChecked<UWorldPartitionRuntimeCellDataHashSet>(RuntimeCell->RuntimeCellData);
		RuntimeCellData->DebugName = CellUniqueId.Name;
		RuntimeCellData->CellBounds = CellDescInstance.CellBounds;
		RuntimeCellData->HierarchicalLevel = CellDescInstance.bIsSpatiallyLoaded ? CellDescInstance.Level : MAX_int32;
		const int32 DataLayersStreamingPriority = GetDataLayersStreamingPriority(CellDescInstance.DataLayerInstances);
		RuntimeCellData->Priority = CellDescInstance.Priority + DataLayersStreamingPriority;
		RuntimeCellData->GridName = CellDescInstance.SourcePartition->Name;
		RuntimeCellData->bIs2D = CellDescInstance.bIs2D;

		return RuntimeCell;
	};

	CustomHLODActorToSourceCellsMap.Empty();
	StandaloneHLODActorToSourceCellsMap.Empty();

	TMap<FGuid, FGuid> StandaloneHLODActorToCell;
	TArray<UWorldPartitionRuntimeCell*> RuntimeCells;
	TMap<URuntimePartition*, FRuntimePartitionStreamingData> RuntimePartitionsStreamingData;
	for (auto& [RuntimePartition, CellDescInstances] : RuntimePartitionsStreamingDescs)
	{
		for (URuntimePartition::FCellDescInstance& CellDescInstance : CellDescInstances)
		{
			const bool bIsCellAlwaysLoaded = !CellDescInstance.bIsSpatiallyLoaded && !CellDescInstance.DataLayerInstances.Num() && !CellDescInstance.ContentBundleID.IsValid();

			TArray<IStreamingGenerationContext::FActorInstance> CellActorInstances;
			if (PopulateCellActorInstances(CellDescInstance.ActorSetInstances, bIsMainWorldPartition, bIsCellAlwaysLoaded, CellActorInstances))
			{
				UWorldPartitionRuntimeCell* RuntimeCell = RuntimeCells.Emplace_GetRef(CreateRuntimeCellFromCellDesc(CellDescInstance, StreamingPolicy->GetRuntimeCellClass(), UWorldPartitionRuntimeCellDataHashSet::StaticClass()));
				RuntimeCell->SetIsAlwaysLoaded(bIsCellAlwaysLoaded);
				RuntimeCell->SetIsSpatiallyLoaded(CellDescInstance.bIsSpatiallyLoaded);

				// Handle HLOD source cell placeholder actors.
				// Save information about mapping from HLOD actor GUID to source cell and remove placeholder actor from the list of actors to be placed in this cell.
				// For more information see FWorldPartitionStreamingGenerator::CreateActorDescViewMap
				bool bRemovedAnyHLODPlaceholderActors = false;
				for (TArray<IStreamingGenerationContext::FActorInstance>::TIterator ActorInstanceIterator = CellActorInstances.CreateIterator(); ActorInstanceIterator; ++ActorInstanceIterator)
				{
					if (ActorInstanceIterator->GetActorDescView().GetActorNativeClass()->IsChildOf<AWorldPartitionHLODSourceCellPlaceholder>())
					{
						const FHLODSourceCellPlaceholderActorDesc* PlaceholderDesc = static_cast<const FHLODSourceCellPlaceholderActorDesc*>(ActorInstanceIterator->GetActorDescView().GetActorDesc());
						const EHLODSourceCellPlaceholderType PlaceholderType = PlaceholderDesc->GetPlaceholderType();
						check(PlaceholderType != EHLODSourceCellPlaceholderType::None);

						if (PlaceholderType == EHLODSourceCellPlaceholderType::StandaloneHLOD)
						{
							StandaloneHLODActorToSourceCellsMap.Add(PlaceholderDesc->GetHLODActorGuid(), RuntimeCell->GetGuid());
						}
						else if (PlaceholderType == EHLODSourceCellPlaceholderType::CustomHLOD)
						{
							CustomHLODActorToSourceCellsMap.Add(PlaceholderDesc->GetHLODActorGuid(), RuntimeCell->GetGuid());
						}

						ActorInstanceIterator.RemoveCurrent();
						bRemovedAnyHLODPlaceholderActors = true;
					}
				}
				// If, after removing all placeholder actors, there are no actors left in this cell, mark the cell as HLOD Placeholder cell, so that we can still use it at runtime to control HLOD actor's visibility.
				RuntimeCell->SetIsCustomHLODPlaceholderCell(bRemovedAnyHLODPlaceholderActors && CellActorInstances.IsEmpty());

				if (WorldPartition->ShouldStripRuntimeCellByTransformerStack(RuntimeCell))
				{
					RuntimeCells.Pop(EAllowShrinking::No);
					continue;
				}

				PopulateRuntimeCell(RuntimeCell, CellActorInstances, OutPackagesToGenerate);

				WorldPartition->ApplyRuntimeCellsTransformerStack(RuntimeCell);

				// Save HLOD actor instance GUID -> Cell GUID mapping to use it for Standalone HLOD source cell override
				for (const IStreamingGenerationContext::FActorInstance& ActorInstance : CellActorInstances)
				{
					const FStreamingGenerationActorDescView& ActorDescView = ActorInstance.GetActorDescView();
					if (ActorDescView.GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>())
					{
						FGuid ActorInstanceGuid = ActorDescView.GetContainerInstance()->GetContainerID().GetActorGuid(ActorDescView.GetGuid());
						StandaloneHLODActorToCell.Add(ActorInstanceGuid, RuntimeCell->GetGuid());
					}
				}

				UWorldPartitionRuntimeCellDataHashSet* RuntimeCellData = CastChecked<UWorldPartitionRuntimeCellDataHashSet>(RuntimeCell->RuntimeCellData);

				if (CellDescInstance.CellBounds.IsSet())
				{
					if (!RuntimeCell->RuntimeCellData->HierarchicalLevel)
					{
						switch (CellDescInstance.SourcePartition->BoundsMethod)
						{
						case ERuntimePartitionCellBoundsMethod::UseCellBounds:
							RuntimeCell->RuntimeCellData->ContentBounds = CellDescInstance.CellBounds.GetValue();
							break;
						case ERuntimePartitionCellBoundsMethod::UseMinContentCellBounds:
							if (RuntimeCell->RuntimeCellData->ContentBounds.IsValid)
							{
								RuntimeCell->RuntimeCellData->ContentBounds = RuntimeCell->RuntimeCellData->ContentBounds.Overlap(CellDescInstance.CellBounds.GetValue());
								check(CellDescInstance.CellBounds.GetValue().IsValid);
							}
							break;
						}
					}
				}

				// Create partition streaming data
				FRuntimePartitionStreamingData& StreamingData = RuntimePartitionsStreamingData.FindOrAdd(CellDescInstance.SourcePartition);

				StreamingData.Name = CellDescInstance.SourcePartition->Name;
				StreamingData.LoadingRange = CellDescInstance.SourcePartition->LoadingRange;

				StreamingData.DebugName = ContainerShortName + TEXT(".") + CellDescInstance.SourcePartition->Name.ToString();

				if (CellDescInstance.DataLayerInstances.Num())
				{
					const FDataLayersID DataLayerdID(CellDescInstance.DataLayerInstances);
					StreamingData.DebugName += FString::Printf(TEXT("_d%x"), DataLayerdID.GetHash());
				}

				if (CellDescInstance.ContentBundleID.IsValid())
				{
					StreamingData.DebugName += FString::Printf(TEXT("_c%s"), *UContentBundleDescriptor::GetContentBundleCompactString(CellDescInstance.ContentBundleID));
				}

				if (CellDescInstance.bIsSpatiallyLoaded)
				{
					StreamingData.SpatiallyLoadedCells.Add(RuntimeCell);
				}
				else
				{
					StreamingData.NonSpatiallyLoadedCells.Add(RuntimeCell);
				}
			}
		}
	}

	// Standalone HLOD actors are embedded in the main world streaming cells, so for Standalone HLOD actors that are using other Standalone HLOD actors as source actors,
	// source cell GUID won't be correct. Save the actual source cell GUID, so that we can use that override at runtime.
	StreamingGenerationContext->ForEachActorSetContainerInstance([this, &StandaloneHLODActorToCell, World](const IStreamingGenerationContext::FActorSetContainerInstance& ActorSetContainerInstance)
	{
		const FStreamingGenerationActorDescViewMap* ActorDescViewMap = ActorSetContainerInstance.ActorDescViewMap;
		ActorDescViewMap->ForEachActorDescView([this, &StandaloneHLODActorToCell, World](const FStreamingGenerationActorDescView& ActorDescView)
		{
			if (ActorDescView.GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>())
			{
				const FHLODActorDesc* ActorDesc = static_cast<const FHLODActorDesc*>(ActorDescView.GetActorDesc());
				for (const FGuid& ExternalChildHLODActorGuid : ActorDesc->GetExternalChildHLODActors())
				{
					// Find External Child HLOD Actor Instance Guid, as there can be multiple instances of the same level used as SubWP
					FGuid ExternalChildHLODActorInstanceGuid = ExternalChildHLODActorGuid;
					if (World)
					{
						if (UWorldPartitionStandaloneHLODSubsystem* StandaloneHLODSubsystem = UWorld::GetSubsystem<UWorldPartitionStandaloneHLODSubsystem>(World))
						{
							FActorContainerID ExternalChildHLODActorContainerID;
							if (StandaloneHLODSubsystem->GetChildHLODActorsContainer(ActorDescView.GetContainerInstance()->GetContainerID(), ExternalChildHLODActorContainerID))
							{
								ExternalChildHLODActorInstanceGuid = ExternalChildHLODActorContainerID.GetActorGuid(ExternalChildHLODActorGuid);
							}
						}
					}

					if (FGuid* CellGuidPtr = StandaloneHLODActorToCell.Find(ExternalChildHLODActorInstanceGuid))
					{
						FGuid CellGuid = *CellGuidPtr;
						FGuid ActorInstanceGuid = ActorDescView.GetContainerInstance()->GetContainerID().GetActorGuid(ActorDescView.GetGuid());

						if (FGuid* SourceCellGuidPtr = StandaloneHLODActorToSourceCellsMap.Find(ActorInstanceGuid))
						{
							if (CellGuid != *SourceCellGuidPtr)
							{
								UE_LOGF(LogWorldPartition, Log, "External child HLOD actors of %ls are not in the same cell. Fix clustering settings!", *ActorDesc->GetActorNameString())
							}
						}
						else
						{
							StandaloneHLODActorToSourceCellsMap.Add(ActorInstanceGuid, CellGuid);
						}
					}
				}
			}
		});
	});

	//
	// Finalize streaming object
	//
	check(RuntimeStreamingData.IsEmpty());
	for (auto& [Partition, StreamingData] : RuntimePartitionsStreamingData)
	{
		StreamingData.CreatePartitionsSpatialIndex();
		RuntimeStreamingData.Emplace(MoveTemp(StreamingData));
	}

	UpdateRuntimeDataGridMap();
	return true;
}

void UWorldPartitionRuntimeHashSet::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Super::DumpStateLog(Ar);

	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	Ar.Printf(TEXT("%s - Runtime Hash Set"), *GetWorld()->GetName());
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));

	TArray<const UWorldPartitionRuntimeCell*> StreamingCells;
	ForEachStreamingCells([&StreamingCells](const UWorldPartitionRuntimeCell* StreamingCell)
	{
		if (!StreamingCell->IsAlwaysLoaded() || !IsRunningCookCommandlet())
		{
			StreamingCells.Add(StreamingCell);
		}
		return true;
	});
				
	StreamingCells.Sort([this](const UWorldPartitionRuntimeCell& A, const UWorldPartitionRuntimeCell& B)
	{
		if (A.IsAlwaysLoaded() ==  B.IsAlwaysLoaded())
		{
			return A.GetFName().LexicalLess(B.GetFName());
		}

		return A.IsAlwaysLoaded() > B.IsAlwaysLoaded();
	});

	for (const UWorldPartitionRuntimeCell* StreamingCell : StreamingCells)
	{
		FHierarchicalLogArchive::FIndentScope CellIndentScope = Ar.PrintfIndent(TEXT("Content of Cell %s (%s)"), *StreamingCell->GetDebugName(), *StreamingCell->GetName());
		StreamingCell->DumpStateLog(Ar);
	}

	Ar.Printf(TEXT(""));
}
#endif
