// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/RuntimeSpatialHash/RuntimeSpatialHashGridHelper.h"
#include "ActorPartition/PartitionActor.h"
#include "HAL/IConsoleManager.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"

#if WITH_EDITOR

bool GRuntimeSpatialHashUseAlignedGridLevels = true;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashUseAlignedGridLevels(
	TEXT("wp.Runtime.RuntimeSpatialHashUseAlignedGridLevels"),
	GRuntimeSpatialHashUseAlignedGridLevels,
	TEXT("Set RuntimeSpatialHashUseAlignedGridLevels to false to help break the pattern caused by world partition promotion of actors to upper grid levels that are always aligned on child levels."));

bool GRuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels = true;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels(
	TEXT("wp.Runtime.RuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels"),
	GRuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels,
	TEXT("Set RuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels to false to avoid snapping higher levels cells to child cells. Only used when GRuntimeSpatialHashUseAlignedGridLevels is false."));

bool GRuntimeSpatialHashPlaceSmallActorsUsingLocation = false;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashPlaceSmallActorsUsingLocation(
	TEXT("wp.Runtime.RuntimeSpatialHashPlaceSmallActorsUsingLocation"),
	GRuntimeSpatialHashPlaceSmallActorsUsingLocation,
	TEXT("Set RuntimeSpatialHashPlaceSmallActorsUsingLocation to true to place actors smaller than a cell size into their corresponding cell using their location instead of their bounding box."));

bool GRuntimeSpatialHashPlacePartitionActorsUsingLocation = true;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashPlacePartitionActorsUsingLocation(
	TEXT("wp.Runtime.RuntimeSpatialHashPlacePartitionActorsUsingLocation"),
	GRuntimeSpatialHashPlacePartitionActorsUsingLocation,
	TEXT("Set RuntimeSpatialHashPlacePartitionActorsUsingLocation to true to place partitioned actors into their corresponding cell using their location instead of their bounding box."));

#endif

FSquare2DGridHelper::FSquare2DGridHelper(const FBox& InWorldBounds, const FVector& InOrigin, int64 InCellSize, bool bUseAlignedGridLevels)
	: WorldBounds(InWorldBounds)
	, Origin(InOrigin)
	, CellSize(InCellSize)
{
	// Compute Grid's size and level count based on World bounds
	int64 GridSize = 1;
	int32 GridLevelCount = 1;

	if (WorldBounds.IsValid && CellSize)
	{
		const FVector2D DistMin = FVector2D(WorldBounds.Min - Origin).GetAbs();
		const FVector2D DistMax = FVector2D(WorldBounds.Max - Origin).GetAbs();
		const double WorldBoundsMaxExtent = FMath::Max(DistMin.GetMax(), DistMax.GetMax());

		if (WorldBoundsMaxExtent > 0)
		{
			GridSize = 2 * FMath::CeilToDouble(WorldBoundsMaxExtent / CellSize); 
			if (!FMath::IsPowerOfTwo(GridSize))
			{
				GridSize = FMath::Pow(2, FMath::CeilToDouble(FMath::Log2(static_cast<double>(GridSize))));
			}
			GridSize = FMath::Max(GridSize, 1);
			GridLevelCount = FMath::FloorLog2_64(GridSize) + 1;
		}
	}

	if (!GridSize || !FMath::IsPowerOfTwo(GridSize))
	{
		GridSize = 1;
		GridLevelCount = 1;
	}

	Levels.Reserve(GridLevelCount);
	int64 CurrentCellSize = CellSize;
	int64 CurrentGridSize = GridSize;
	for (int32 Level = 0; Level < GridLevelCount; ++Level)
	{
		int64 LevelGridSize = CurrentGridSize;

		if (!bUseAlignedGridLevels)
		{
			// Except for top level, adding 1 to CurrentGridSize (which is always a power of 2) breaks the pattern of perfectly aligned cell edges between grid level cells.
			// This will prevent weird artefact during actor promotion when an actor is placed using its bounds and which overlaps multiple cells.
			// In this situation, the algorithm will try to find a cell that encapsulates completely the actor's bounds by searching in the upper levels, until it finds one.
			// Also note that, the default origin of each level will always be centered at the middle of the bounds of (level's cellsize * level's grid size).
			LevelGridSize = (Level == GridLevelCount - 1) ? CurrentGridSize : CurrentGridSize + 1;
		}

		Levels.Emplace(FVector2D(InOrigin), CurrentCellSize, LevelGridSize, Level);

		CurrentCellSize <<= 1;
		CurrentGridSize >>= 1;
	}

	// Make sure the always loaded cell exists
	GetAlwaysLoadedCell();
}

void FSquare2DGridHelper::ForEachCells(TFunctionRef<void(const FSquare2DGridHelper::FGridLevel::FGridCell&)> InOperation) const
{
	for (int32 Level = 0; Level < Levels.Num(); Level++)
	{
		for (const FSquare2DGridHelper::FGridLevel::FGridCell& ThisCell : Levels[Level].Cells)
		{
			InOperation(ThisCell);
		}
	}
}

int32 FSquare2DGridHelper::ForEachIntersectingCells(const FBox& InBox, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel) const
{
	int32 NumCells = 0;

	for (int32 Level = InStartLevel; Level < Levels.Num(); Level++)
	{
		NumCells += Levels[Level].ForEachIntersectingCells(InBox, [InBox, Level, InOperation](const FGridCellCoord2& Coord) { InOperation(FGridCellCoord(Coord.X, Coord.Y, Level)); });
	}

	return NumCells;
}

int32 FSquare2DGridHelper::ForEachIntersectingCells(const FSphere& InSphere, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel) const
{
	int32 NumCells = 0;

	for (int32 Level = InStartLevel; Level < Levels.Num(); Level++)
	{
		NumCells += Levels[Level].ForEachIntersectingCells(InSphere, [InSphere, Level, InOperation](const FGridCellCoord2& Coord) { InOperation(FGridCellCoord(Coord.X, Coord.Y, Level)); });
	}

	return NumCells;
}

int32 FSquare2DGridHelper::ForEachIntersectingCells(const FSphericalSector& InShape, TFunctionRef<void(const FGridCellCoord&)> InOperation, int32 InStartLevel) const
{
	int32 NumCells = 0;

	for (int32 Level = InStartLevel; Level < Levels.Num(); Level++)
	{
		NumCells += Levels[Level].ForEachIntersectingCells(InShape, [Level, InOperation](const FGridCellCoord2& Coord) { InOperation(FGridCellCoord(Coord.X, Coord.Y, Level)); });
	}

	return NumCells;
}

#if WITH_EDITOR

FSquare2DGridHelper GetGridHelper(const FBox& WorldBounds, const FVector& GridOrigin, int64 GridCellSize, bool bUseAlignedGridLevels)
{
	// Default grid to a minimum of 1 level and 1 cell, for always loaded actors
	return FSquare2DGridHelper(WorldBounds, GridOrigin, GridCellSize, bUseAlignedGridLevels);
}

FSquare2DGridHelper GetPartitionedActors(const FBox& WorldBounds, const FSpatialHashRuntimeGrid& Grid, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances, const FSpatialHashSettings& Settings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetPartitionedActors);

	//
	// Create the hierarchical grids for the game
	//	
	FSquare2DGridHelper PartitionedActors = GetGridHelper(WorldBounds, FVector(Grid.Origin, 0), Grid.CellSize, Settings.bUseAlignedGridLevels);
	const bool bWorldBoundsValid = WorldBounds.IsValid && WorldBounds.GetExtent().Size2D() > 0;
	if (ensure(PartitionedActors.Levels.Num()) && bWorldBoundsValid)
	{
		int32 IntersectingCellCount = 0;
		FSquare2DGridHelper::FGridLevel& LastGridLevel = PartitionedActors.Levels.Last();
		LastGridLevel.ForEachIntersectingCells(WorldBounds, [&IntersectingCellCount](const FGridCellCoord2& Coords) { ++IntersectingCellCount; });
		if (!ensure(IntersectingCellCount == 1))
		{
			UE_LOGF(LogWorldPartition, Warning, "Can't find grid cell that encompasses world bounds.");
		}
	}

	const int32 GridLevelCount = PartitionedActors.Levels.Num();
	const int64 CellSize = PartitionedActors.GetLowestLevel().CellSize;
	const float CellArea = CellSize * CellSize;

	auto ShouldActorUseLocationPlacement = [CellArea, CellSize, GridLevelCount, &Settings](const IStreamingGenerationContext::FActorSetInstance* InActorSetInstance, const FBox2D& InActorSetInstanceBounds, int32& OutGridLevel)
	{
		OutGridLevel = 0;
		if (Settings.bPlaceSmallActorsUsingLocation && (InActorSetInstanceBounds.GetArea() <= CellArea))
		{
			return true;
		}

		if (Settings.bPlacePartitionActorsUsingLocation)
		{
			bool bUseLocation = true;
			InActorSetInstance->ForEachActor([InActorSetInstance, &bUseLocation](const FGuid& ActorGuid)
			{
				const IWorldPartitionActorDescInstanceView& ActorDescView = InActorSetInstance->ActorSetContainerInstance->ActorDescViewMap->FindByGuidChecked(ActorGuid);

				if (!ActorDescView.GetActorNativeClass()->IsChildOf<APartitionActor>())
				{
					bUseLocation = false;
					return false;
				}

				return true;
			});
			if (bUseLocation)
			{
				// Find grid level that best matches actor set bounds
				const float MaxLength = InActorSetInstanceBounds.GetSize().GetMax();
				OutGridLevel = FMath::Min(FMath::CeilToInt(FMath::Max<float>(FMath::Log2(MaxLength / CellSize), 0)), GridLevelCount - 1);
			}
			return bUseLocation;
		}

		return false;
	};

	for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : ActorSetInstances)
	{
		check(ActorSetInstance->ActorSet->Actors.Num() > 0);

		FSquare2DGridHelper::FGridLevel::FGridCell* GridCell = nullptr;

		if (ActorSetInstance->bIsSpatiallyLoaded)
		{
			const FBox2D ActorSetInstanceBounds(FVector2D(ActorSetInstance->Bounds.Min), FVector2D(ActorSetInstance->Bounds.Max));
			int32 LocationPlacementGridLevel = 0;
			if (ShouldActorUseLocationPlacement(ActorSetInstance, ActorSetInstanceBounds, LocationPlacementGridLevel))
			{
				// Find grid level cell that contains the actor cluster pivot and put actors in it.
				FGridCellCoord2 CellCoords;
				if (PartitionedActors.Levels[LocationPlacementGridLevel].GetCellCoords(ActorSetInstanceBounds.GetCenter(), CellCoords))
				{
					GridCell = &PartitionedActors.Levels[LocationPlacementGridLevel].GetCell(CellCoords);
				}
			}
			else
			{
				// Find grid level cell that encompasses the actor cluster bounding box and put actors in it.
				const FVector2D ClusterSize = ActorSetInstanceBounds.GetSize();
				const double MinRequiredCellExtent = FMath::Max(ClusterSize.X, ClusterSize.Y);
				const int32 FirstPotentialGridLevel = FMath::Max(FMath::CeilToDouble(FMath::Log2(MinRequiredCellExtent / (double)PartitionedActors.CellSize)), 0);

				for (int32 GridLevelIndex = FirstPotentialGridLevel; GridLevelIndex < PartitionedActors.Levels.Num(); GridLevelIndex++)
				{
					FSquare2DGridHelper::FGridLevel& GridLevel = PartitionedActors.Levels[GridLevelIndex];

					if (GridLevel.GetNumIntersectingCells(ActorSetInstance->Bounds) == 1)
					{
						GridLevel.ForEachIntersectingCells(ActorSetInstance->Bounds, [&GridLevel, &GridCell](const FGridCellCoord2& Coords)
						{
							check(!GridCell);
							GridCell = &GridLevel.GetCell(Coords);
						});

						break;
					}
				}
			}
		}
		
		if (!GridCell)
		{
			GridCell = &PartitionedActors.GetAlwaysLoadedCell();
		}

		GridCell->AddActorSetInstance(ActorSetInstance);
	}

	return PartitionedActors;
}

#endif // #if WITH_EDITOR

bool FSquare2DGridHelper::FGrid2D::DoesCircleSectorIntersectsCell(const FGridCellCoord2& Coords, const FSphericalSector& InShape) const
{
	FBox2D CellBounds;
	GetCellBounds(Coords, CellBounds);

	return InShape.IntersectsBox(CellBounds);
}

int32 FSquare2DGridHelper::FGrid2D::ForEachIntersectingCells(const FSphericalSector& InShape, TFunctionRef<void(const FGridCellCoord2&)> InOperation) const
{
	check(!InShape.IsNearlyZero());
	const bool bIsSphere = InShape.IsSphere();

	int32 NumCells = 0;
	const FSphere Sphere(InShape.GetCenter(), InShape.GetRadius());
	ForEachIntersectingCells(Sphere, [this, bIsSphere, InShape, InOperation, &NumCells](const FGridCellCoord2& Coords)
	{
		// If sector, filter coords that intersect sector
		if (bIsSphere || DoesCircleSectorIntersectsCell(Coords, InShape))
		{
			InOperation(Coords);
			NumCells++;
		}
	});

	return NumCells;
}

