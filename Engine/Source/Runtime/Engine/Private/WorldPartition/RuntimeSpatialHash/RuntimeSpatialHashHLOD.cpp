// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"

#if WITH_EDITOR
#include "HAL/PlatformFile.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/RuntimeSpatialHash/RuntimeSpatialHashGridHelper.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/HLOD/HLODModifier.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilities.h"
#include "WorldPartition/HLOD/IWorldPartitionHLODUtilitiesModule.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/ContentBundle/ContentBundleActivationScope.h"

#include "UObject/GCObjectScopeGuard.h"
#include "UObject/SavePackage.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "RuntimeSpatialHashHLOD"

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionRuntimeSpatialHashHLOD, Log, All);

static void SavePackage(UPackage* Package, ISourceControlHelper* SourceControlHelper)
{
	if (SourceControlHelper)
	{
		SourceControlHelper->Save(Package);
	}
	else
	{
		Package->MarkAsFullyLoaded();

		FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(Package->GetName());
		const FString PackageFileName = PackagePath.GetLocalFullPath();
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
		{
			UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Error, "Error saving package %ls.", *Package->GetName());
			check(0);
		}
	}
}

static void DeletePackage(const FString& PackageName, ISourceControlHelper* SourceControlHelper)
{
	FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(PackageName);
	const FString PackageFileName = PackagePath.GetLocalFullPath();

	if (SourceControlHelper)
	{
		SourceControlHelper->Delete(PackageFileName);
	}
	else
	{
		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PackageFileName);
	}
}

static void DeletePackage(UPackage* Package, ISourceControlHelper* SourceControlHelper)
{
	if (SourceControlHelper)
	{
		SourceControlHelper->Delete(Package);
	}
	else
	{
		DeletePackage(Package->GetName(), SourceControlHelper);
	}
}

static void DeletePackage(UWorldPartition* WorldPartition, FWorldPartitionActorDescInstance* ActorDescInstance, ISourceControlHelper* SourceControlHelper)
{
	if (ActorDescInstance->IsLoaded())
	{
		DeletePackage(ActorDescInstance->GetActor()->GetPackage(), SourceControlHelper);
		WorldPartition->OnPackageDeleted(ActorDescInstance->GetActor()->GetPackage());
	}
	else
	{
		DeletePackage(ActorDescInstance->GetActorPackage().ToString(), SourceControlHelper);
		WorldPartition->RemoveActor(ActorDescInstance->GetGuid());
	}
}

static void GameTick(UWorld* InWorld)
{
	static int32 TickRendering = 0;
	static const int32 FlushRenderingFrequency = 256;

	// Perform a GC when memory usage exceeds a given threshold
	if (FWorldPartitionHelpers::ShouldCollectGarbage())
	{
		FWorldPartitionHelpers::DoCollectGarbage();
	}

	// When running with -AllowCommandletRendering we want to flush
	if (((++TickRendering % FlushRenderingFrequency) == 0) && IsAllowCommandletRendering())
	{
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
	}
}

static TArray<FGuid> GenerateHLODActorsForGrid(UWorldPartition* WorldPartition, const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params, const FSpatialHashRuntimeGrid& RuntimeGrid, uint32 HLODLevel, FHLODCreationContext& Context, ISourceControlHelper* SourceControlHelper, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances, TArray<FName>& NewActors, TArray<FName>& DirtyActors, const FSpatialHashSettings& Settings)
{
	IWorldPartitionHLODUtilitiesModule* WPHLODUtilitiesModule = FModuleManager::Get().LoadModulePtr<IWorldPartitionHLODUtilitiesModule>("WorldPartitionHLODUtilities");
	IWorldPartitionHLODUtilities* WPHLODUtilities = WPHLODUtilitiesModule != nullptr ? WPHLODUtilitiesModule->GetUtilities() : nullptr;
	if (WPHLODUtilities == nullptr)
	{
		UE_LOGF(LogWorldPartition, Error, "%s requires plugin 'World Partition HLOD Utilities'.", __FUNCTION__);
		return {};
	}

	const FBox WorldBounds = StreamingGenerationContext->GetWorldBounds();
	const FSquare2DGridHelper PartitionedActors = GetPartitionedActors(WorldBounds, RuntimeGrid, ActorSetInstances, Settings);
	const FSquare2DGridHelper::FGridLevel::FGridCell& AlwaysLoadedCell = PartitionedActors.GetAlwaysLoadedCell();

	auto ShouldGenerateHLODActors = [&AlwaysLoadedCell](const FSquare2DGridHelper::FGridLevel::FGridCell& GridCell, const FSquare2DGridHelper::FGridLevel::FGridCellDataChunk& GridCellDataChunk)
	{
		const bool bIsCellAlwaysLoaded = &GridCell == &AlwaysLoadedCell;
		const bool bChunkHasActors = !GridCellDataChunk.GetActorSetInstances().IsEmpty();

		const bool bShouldGenerateHLODActors = !bIsCellAlwaysLoaded && bChunkHasActors;
		return bShouldGenerateHLODActors;
	};

	UWorld* OuterWorld = WorldPartition->GetTypedOuter<UWorld>();

	// Quick pass to compute the number of cells we'll have to process, to provide a meaningful progress display
	int32 NbCellsToProcess = 0;
	PartitionedActors.ForEachCells([&](const FSquare2DGridHelper::FGridLevel::FGridCell& GridCell)
	{
		for (const FSquare2DGridHelper::FGridLevel::FGridCellDataChunk& GridCellDataChunk : GridCell.GetDataChunks())
		{
			const bool bShouldGenerateHLODActors = ShouldGenerateHLODActors(GridCell, GridCellDataChunk);
			if (bShouldGenerateHLODActors)
			{
				NbCellsToProcess++;
			}
		}
	});

	UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Verbose, "%ls %ls...", *LOCTEXT("BuildingHLODsForGrid", "Building HLODs for grid").ToString(), *RuntimeGrid.GridName.ToString());

	FScopedSlowTask SlowTask(NbCellsToProcess, FText::FromString(FString::Printf(TEXT("%s %s..."), *LOCTEXT("BuildingHLODsForGrid", "Building HLODs for grid").ToString(), *RuntimeGrid.GridName.ToString())));
	SlowTask.MakeDialog();

	TArray<FGuid> GridHLODActors;
	PartitionedActors.ForEachCells([&](const FSquare2DGridHelper::FGridLevel::FGridCell& GridCell)
	{
		const FGridCellCoord CellCoord = GridCell.GetCoords();

		FBox2D CellBounds2D;
		PartitionedActors.GetCellBounds(CellCoord, CellBounds2D);

		FGridCellCoord CellGlobalCoord;
		verify(PartitionedActors.GetCellGlobalCoords(CellCoord, CellGlobalCoord));

		for (const FSquare2DGridHelper::FGridLevel::FGridCellDataChunk& GridCellDataChunk : GridCell.GetDataChunks())
		{
			// Fake tick
			GameTick(WorldPartition->GetWorld());

			const bool bShouldGenerateHLODActors = ShouldGenerateHLODActors(GridCell, GridCellDataChunk);
			if (bShouldGenerateHLODActors)
			{
				SlowTask.EnterProgressFrame(1);

				const FGuid CellGuid = UWorldPartitionRuntimeSpatialHash::GetCellGuid(RuntimeGrid.GridName, RuntimeGrid.CellSize, CellGlobalCoord, GridCellDataChunk.GetDataLayersID(), GridCellDataChunk.GetContentBundleID());
				const FString CellName = UWorldPartitionRuntimeSpatialHash::GetCellNameString(OuterWorld, RuntimeGrid.GridName, CellGlobalCoord, GridCellDataChunk.GetDataLayersID(), GridCellDataChunk.GetContentBundleID());

				UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Verbose, "%ls %ls -> %ls...", *CellName, *LOCTEXT("CreatingHLODCell", "Creating HLOD for cell").ToString() , *CellCoord.ToString());

				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*CellCoord.ToString());

				UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Display, "[%d / %d] %ls %ls...", (int32)SlowTask.CompletedWork + 1, (int32)SlowTask.TotalAmountOfWork, *LOCTEXT("ProcessingCell", "Processing cell").ToString(), *CellName);

				// We know the cell's 2D bound but must figure out the bounds in Z
				FBox CellBounds = FBox(FVector(CellBounds2D.Min, MAX_dbl), FVector(CellBounds2D.Max, -MAX_dbl));

				TArray<IStreamingGenerationContext::FActorInstance> ActorInstances;
				for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : GridCellDataChunk.GetActorSetInstances())
				{
					ActorSetInstance->ForEachActor([&ActorInstances, ActorSetInstance, &CellBounds](const FGuid& ActorGuid)
					{
						const IStreamingGenerationContext::FActorInstance& ActorInstance = ActorInstances.Emplace_GetRef(ActorGuid, ActorSetInstance);
						const FBox RuntimeBounds = ActorInstance.GetActorDescView().GetRuntimeBounds();

						if (RuntimeBounds.IsValid)
						{
							CellBounds.Min.Z = FMath::Min(CellBounds.Min.Z, RuntimeBounds.Min.Z);
							CellBounds.Max.Z = FMath::Max(CellBounds.Max.Z, RuntimeBounds.Max.Z);
						}
					});
				}

				// Ensure the Z bounds are valid
				check(CellBounds.Min.Z <= CellBounds.Max.Z);

				FHLODCreationParams CreationParams;
				CreationParams.WorldPartition = WorldPartition;
				CreationParams.TargetWorld = WorldPartition->GetWorld();
				CreationParams.CellGuid = CellGuid;
				CreationParams.CellName = CellName;
				CreationParams.GetRuntimeGrid = [HLODLevel](const UHLODLayer* HLODLayer) 
				{ 
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					bool bIsSpatiallyLoaded = HLODLayer->IsSpatiallyLoaded();
					int32 CellSize = HLODLayer->GetCellSize();
					double LoadingRange = HLODLayer->GetLoadingRange();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					return !bIsSpatiallyLoaded ? NAME_None : FName(UWorldPartitionRuntimeSpatialHash::GetHLODGridName(HLODLevel, CellSize, LoadingRange));
				};
				CreationParams.HLODLevel = HLODLevel;
				CreationParams.MinVisibleDistance = RuntimeGrid.LoadingRange;
				CreationParams.ContentBundleGuid = GridCellDataChunk.GetContentBundleID();
				CreationParams.DataLayerInstances = GridCellDataChunk.GetDataLayers();
				CreationParams.bIsStandalone = false;
				CreationParams.Filters = Params.Filters;

				TArray<AWorldPartitionHLOD*> CellHLODActors = WPHLODUtilities->CreateHLODActors(Context, CreationParams, ActorInstances);
				if (!CellHLODActors.IsEmpty())
				{
					TArray<AWorldPartitionHLOD*> NewCellHLODActors;

					for (AWorldPartitionHLOD* CellHLODActor : CellHLODActors)
					{
						FGuid ActorGuid = CellHLODActor->GetActorGuid();
						GridHLODActors.Add(ActorGuid);

						UPackage* CellHLODActorPackage = CellHLODActor->GetPackage();
						if (CellHLODActorPackage->HasAnyPackageFlags(PKG_NewlyCreated))
						{
							NewCellHLODActors.Add(CellHLODActor);
							NewActors.Add(CellHLODActor->GetFName());
						}
						else if (CellHLODActorPackage->IsDirty())
						{
							DirtyActors.Add(CellHLODActor->GetFName());
						}
					}

					if (!Params.bReportOnly)
					{
						for (AWorldPartitionHLOD* CellHLODActor : CellHLODActors)
						{
							if (CellHLODActor->GetPackage()->IsDirty())
							{
								SavePackage(CellHLODActor->GetPackage(), SourceControlHelper);
							}
						}
					}

					// Update newly created HLOD actors
					for (AWorldPartitionHLOD* CellHLODActor : NewCellHLODActors)
					{
						Context.ActorReferences.Emplace(WorldPartition, CellHLODActor->GetActorGuid());
					}
				}

				// Unload actors
				Context.ActorReferences.Empty();
			}
		}
	});

	// Ensure all async packages writes are completed before we start processing another HLOD level or grid
	UPackage::WaitForAsyncFileWrites();

	// Need to collect garbage here since some HLOD actors have been marked pending kill when destroying them
	// and they may be loaded when generating the next HLOD layer.
	FWorldPartitionHelpers::DoCollectGarbage();

	return GridHLODActors;
}

// Find all referenced HLODLayer assets
static TMap<UHLODLayer*, int32> GatherHLODLayers(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition* WorldPartition)
{
	// Gather up all HLODLayers referenced by the actors, along with the HLOD level at which it was used
	TMap<UHLODLayer*, int32> HLODLayersLevel;

	StreamingGenerationContext->ForEachActorSetContainerInstance([&HLODLayersLevel, WorldPartition](const IStreamingGenerationContext::FActorSetContainerInstance& ActorSetContainerInstance)
	{
		const FStreamingGenerationActorDescViewMap* ActorDescViewMap = ActorSetContainerInstance.ActorDescViewMap;
		ActorDescViewMap->ForEachActorDescView([&HLODLayersLevel, WorldPartition](const FStreamingGenerationActorDescView& ActorDescView)
		{
			if (!ActorDescView.GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>())
			{
				if (ActorDescView.GetActorIsHLODRelevant())
				{
					UHLODLayer* HLODLayer = Cast<UHLODLayer>(ActorDescView.GetHLODLayer().TryLoad());
		
					// If layer was already encountered, no need to process it again
					if (!HLODLayersLevel.Contains(HLODLayer))
					{
						// Walk up the parent HLOD layers, keep track of HLOD level
						int32 CurrentHLODLevel = 0;
						while (HLODLayer != nullptr)
						{
							int32& HLODLevel = HLODLayersLevel.FindOrAdd(HLODLayer);
							HLODLevel = FMath::Max(HLODLevel, CurrentHLODLevel);
		
							HLODLayer = HLODLayer->GetParentLayer();
							CurrentHLODLevel++;
						}
					}
				}
			}
		});
	});

	return HLODLayersLevel;
}

static TMap<FName, FSpatialHashRuntimeGrid> CreateHLODGrids(TMap<UHLODLayer*, int32>& InHLODLayersLevel)
{
	TMap<FName, FSpatialHashRuntimeGrid> HLODGrids;

	// Create HLOD runtime grids
	for (const TPair<UHLODLayer*, int32>& HLODLayerLevel : InHLODLayersLevel)
	{
		UHLODLayer* HLODLayer = HLODLayerLevel.Key;
		const int32 HLODLevel = HLODLayerLevel.Value;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bIsSpatiallyLoaded = HLODLayer->IsSpatiallyLoaded();
		const FName GridName = HLODLayer->GetRuntimeGrid(HLODLayerLevel.Value);
		const int32 CellSize = HLODLayer->GetCellSize();
		const double LoadingRange = HLODLayer->GetLoadingRange();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// No need to create a runtime grid if the HLOD layer is set to be non spatially loaded
		if (bIsSpatiallyLoaded)
		{
			FSpatialHashRuntimeGrid& HLODGrid = HLODGrids.FindOrAdd(GridName);
			if (HLODGrid.GridName.IsNone())
			{
				HLODGrid.CellSize = CellSize;
				HLODGrid.LoadingRange = LoadingRange;
				HLODGrid.GridName = GridName;
				HLODGrid.HLODLayer = HLODLayer;
				HLODGrid.bClientOnlyVisible = true;

				// HLOD grid have a lower priority than standard grids
				HLODGrid.Priority = 100 + HLODLevel;

				// Setup grid debug color to match HLODColorationColors
				const int32 LastLODColorationColorIdx = GEngine->HLODColorationColors.Num() - 1;
				const int32 HLODColorIdx = FMath::Clamp(HLODLevel + 2, 0U, (int32)LastLODColorationColorIdx);
				HLODGrid.DebugColor = GEngine->HLODColorationColors[HLODColorIdx];
			}

			// Temp solution to have replication working for hlod layers that have a modifier
			// Modifiers are used for example for destructible HLODs, that need to have a server side component
			// The real solution would probably be to check if we have any actors that are
			// returning true for NeedsLoadForServer()
			HLODGrid.bClientOnlyVisible &= HLODLayer->GetHLODModifierClass() == nullptr;
		}
	}

	HLODGrids.KeySort(FNameLexicalLess());

	return HLODGrids;
}

// Actor tag used to mark ASpatialHashRuntimeGridInfo actors as being valid for the current grid setup.
static const FName HLODGridActorInUseTag = "HLODGridActorInUse";

// Create/destroy HLOD grid actors
static void UpdateHLODGridsActors(UWorld* World, const UActorDescContainerInstance* ContainerInstance, const TMap<FName, FSpatialHashRuntimeGrid>& HLODGrids, ISourceControlHelper* SourceControlHelper)
{
	// Gather all existing HLOD grid actors
	TMap<FName, ASpatialHashRuntimeGridInfo*> ExistingGridActors;
	for (UActorDescContainerInstance::TConstIterator<ASpatialHashRuntimeGridInfo> Iterator(ContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->IsLoaded())
		{
		    if (ASpatialHashRuntimeGridInfo* GridActor = CastChecked<ASpatialHashRuntimeGridInfo>(Iterator->GetActor()))
		    {
			    const FSpatialHashRuntimeGrid* HLODGrid = HLODGrids.Find(GridActor->GridSettings.GridName);
			    if (HLODGrid && GridActor->GridSettings.Priority && GridActor->GridSettings.HLODLayer)
			    {
				    check(GridActor->GetContentBundleGuid() == ContainerInstance->GetContentBundleGuid());
				    ExistingGridActors.Emplace(GridActor->GridSettings.GridName, GridActor);
			    }
		    }
		}
	}

	// Create missing HLOD grid actors or update existing actors
	for (const auto& HLODGridEntry : HLODGrids)
	{
		FName GridName = HLODGridEntry.Key;
		const FSpatialHashRuntimeGrid& GridSettings = HLODGridEntry.Value;

		bool bDirty = false;

		ASpatialHashRuntimeGridInfo* GridActor = ExistingGridActors.FindRef(GridName);
		if (!GridActor)
		{
			FContentBundleActivationScope ContentBndleScope(ContainerInstance->GetContentBundleGuid());
			FScopedOverrideSpawningLevelMountPointObject EDLScope(ContainerInstance->GetExternalDataLayerAsset());

			FActorSpawnParameters SpawnParams;
			SpawnParams.bCreateActorPackage = true;
			GridActor = World->SpawnActor<ASpatialHashRuntimeGridInfo>(SpawnParams);
			bDirty = true;

			check(GridActor->GetContentBundleGuid() == ContainerInstance->GetContentBundleGuid());
			check(GridActor->GetExternalDataLayerAsset() == ContainerInstance->GetExternalDataLayerAsset());
		}

		// Flag this ASpatialHashRuntimeGridInfo actor as being in-use
		GridActor->Tags.Add(HLODGridActorInUseTag);

		const FString ActorLabel = GridSettings.GridName.ToString();
		if (GridActor->GetActorLabel() != ActorLabel)
		{
			GridActor->SetActorLabel(ActorLabel);
			bDirty = true;
		}

		if (GridActor->GridSettings != GridSettings)
		{
			GridActor->GridSettings = GridSettings;
			bDirty = true;
		}			

		if (bDirty)
		{
			SavePackage(GridActor->GetPackage(), SourceControlHelper);
		}
	}
}

bool UWorldPartitionRuntimeSpatialHash::SetupHLODActors(const IStreamingGenerationContext* StreamingGenerationContext, const UWorldPartition::FSetupHLODActorsParams& Params) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeSpatialHash::SetupHLODActors);

	ISourceControlHelper* SourceControlHelper = Params.SourceControlHelper;

	TArray<FName> NewActors;
	TArray<FName> DirtyActors;
	TArray<FName> DeletedActors;

	if (Params.bReportOnly)
	{
		LogWorldPartitionRuntimeSpatialHashHLOD.SetVerbosity(ELogVerbosity::Verbose);
	}

	if (!Grids.Num())
	{
		UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Error, "Invalid partition grids setup");
		return false;
	}

	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	IStreamingGenerationContext::FActorSetContainerInstance* BaseActorSetContainerInstance = const_cast<IStreamingGenerationContext::FActorSetContainerInstance*>(StreamingGenerationContext->GetActorSetContainerForContextBaseContainerInstance());
	const FStreamingGenerationContainerInstanceCollection* BaseContainerInstanceCollection = BaseActorSetContainerInstance->ContainerInstanceCollection;

	// Find all used HLOD layers
	TMap<UHLODLayer*, int32> HLODLayersLevels = GatherHLODLayers(StreamingGenerationContext, WorldPartition);
	TArray<UHLODLayer*> HLODLayers;
	HLODLayersLevels.GetKeys(HLODLayers);

	// Keep references to HLODLayers or they may be GC'd
	TGCObjectsScopeGuard<UHLODLayer> KeepHLODLayersAlive(HLODLayers);

	TMap<FName, FSpatialHashRuntimeGrid> HLODGrids = CreateHLODGrids(HLODLayersLevels);
	
	TMap<FName, int32> GridsMapping;
	GridsMapping.Add(NAME_None, 0);
	for (int32 i = 0; i < Grids.Num(); i++)
	{
		const FSpatialHashRuntimeGrid& Grid = Grids[i];
		check(!GridsMapping.Contains(Grid.GridName));
		GridsMapping.Add(Grid.GridName, i);
	}

	// HLOD creation context
	FHLODCreationContext Context;
	BaseContainerInstanceCollection->ForEachActorDescContainerInstance([&Context, WorldPartition](const UActorDescContainerInstance* ActorDescContainerInstance)
	{
		for (UActorDescContainerInstance::TConstIterator<AWorldPartitionHLOD> HLODIterator(ActorDescContainerInstance); HLODIterator; ++HLODIterator)
		{
			FWorldPartitionHandle HLODActorHandle(WorldPartition, HLODIterator->GetGuid());
			Context.HLODActorDescs.Emplace(HLODIterator->GetActorName(), MoveTemp(HLODActorHandle));
		}
	});

	TArray<TArray<const IStreamingGenerationContext::FActorSetInstance*>> GridActorSetInstances;
	GridActorSetInstances.InsertDefaulted(0, Grids.Num());

	StreamingGenerationContext->ForEachActorSetInstance([&GridsMapping, &GridActorSetInstances](const IStreamingGenerationContext::FActorSetInstance& ActorSetInstance)
	{
		int32* FoundIndex = GridsMapping.Find(ActorSetInstance.RuntimeGrid);
		if (!FoundIndex)
		{
			//@todo_ow should this be done upstream?
			UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Log, "Invalid partition grid '%ls' referenced by actor cluster", *ActorSetInstance.RuntimeGrid.ToString());
		}

		int32 GridIndex = FoundIndex ? *FoundIndex : 0;
		GridActorSetInstances[GridIndex].Add(&ActorSetInstance);
	});

	// Keep track of all valid HLOD actors, along with which runtime grid they live in
	TMap<FName, TArray<FGuid>> GridsHLODActors;

	const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();

	// Make sure hash settings are up to date
	FSpatialHashSettings HashSettings = Settings;	
	HashSettings.UpdateSettings(*this);

	auto GenerateHLODActors = [&GridsHLODActors, &BaseActorSetContainerInstance, StreamingGenerationContext, &Params, WorldPartition, DataLayerManager, &Context, SourceControlHelper, &NewActors, &DirtyActors, &HashSettings, this]
	(const FSpatialHashRuntimeGrid& RuntimeGrid, uint32 HLODLevel, const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances)
	{
		// Generate HLODs for this grid
		TArray<FGuid> HLODActors = GenerateHLODActorsForGrid(WorldPartition, StreamingGenerationContext, Params, RuntimeGrid, HLODLevel, Context, SourceControlHelper, ActorSetInstances, NewActors, DirtyActors, HashSettings);

		for (const FGuid& HLODActorGuid : HLODActors)
		{
			if (!Params.bReportOnly)
			{
				FWorldPartitionActorDescInstance* HLODActorDescInstance = WorldPartition->GetActorDescInstance(HLODActorGuid);
				check(HLODActorDescInstance);
				FStreamingGenerationActorDescViewMap* NonConstActorDescViewMap = const_cast<FStreamingGenerationActorDescViewMap*>(BaseActorSetContainerInstance->ActorDescViewMap);
				FStreamingGenerationActorDescView* ActorDescView = NonConstActorDescViewMap->Emplace(HLODActorDescInstance);
				
				FDataLayerInstanceNames RuntimeDataLayerInstanceNames;
				if (FDataLayerUtils::ResolveRuntimeDataLayerInstanceNames(DataLayerManager, *ActorDescView, *BaseActorSetContainerInstance->DataLayerResolvers, RuntimeDataLayerInstanceNames))
				{
					ActorDescView->SetRuntimeDataLayerInstanceNames(RuntimeDataLayerInstanceNames);
				}

				GridsHLODActors.FindOrAdd(ActorDescView->GetRuntimeGrid()).Add(HLODActorGuid);
			}
		}
	};

	// Generate HLODs for the standard runtime grids (HLOD 0)
	for (int32 GridIndex = 0; GridIndex < Grids.Num(); GridIndex++)
	{
		// Generate HLODs for this grid - retrieve actors from the world partition
		GenerateHLODActors(Grids[GridIndex], 0, GridActorSetInstances[GridIndex]);
	}

	// Now, go on and create HLOD actors from HLOD grids (HLOD 1-N)
	for (auto It = HLODGrids.CreateIterator(); It; ++It)
	{
		const FName HLODGridName = It.Key();

		// No need to process empty grids
		if (!GridsHLODActors.Contains(HLODGridName))
		{
			It.RemoveCurrent(); // No need to keep this grid around, we have no actors in it
			continue;
		}

		// Create one actor cluster per HLOD actor
		TArray<IStreamingGenerationContext::FActorSetInstance> HLODActorSetInstances;
		TArray<const IStreamingGenerationContext::FActorSetInstance*> HLODActorSetInstancePtrs;
		HLODActorSetInstances.Reserve(GridsHLODActors[HLODGridName].Num());
		HLODActorSetInstancePtrs.Reserve(GridsHLODActors[HLODGridName].Num());
		BaseActorSetContainerInstance->ActorSets.Reserve(BaseActorSetContainerInstance->ActorSets.Num() + GridsHLODActors[HLODGridName].Num());

		for (const FGuid& HLODActorGuid : GridsHLODActors[HLODGridName])
		{
			const FStreamingGenerationActorDescView& HLODActorDescView = BaseActorSetContainerInstance->ActorDescViewMap->FindByGuidChecked(HLODActorGuid);
			
			IStreamingGenerationContext::FActorSetInstance& NewHLODActorSetInstance = HLODActorSetInstances.Emplace_GetRef();
			
			NewHLODActorSetInstance.Bounds = HLODActorDescView.GetRuntimeBounds();
			NewHLODActorSetInstance.RuntimeGrid = HLODActorDescView.GetRuntimeGrid();
			NewHLODActorSetInstance.bIsSpatiallyLoaded = HLODActorDescView.GetIsSpatiallyLoaded();
			if (DataLayerManager)
			{
				NewHLODActorSetInstance.DataLayers = DataLayerManager->GetRuntimeDataLayerInstances(HLODActorDescView.GetRuntimeDataLayerInstanceNames().ToArray());
			}
			NewHLODActorSetInstance.ContentBundleID = BaseContainerInstanceCollection->GetContentBundleGuid();
			NewHLODActorSetInstance.ActorSetContainerInstance = BaseActorSetContainerInstance;
			NewHLODActorSetInstance.ActorSet = BaseActorSetContainerInstance->ActorSets.Add_GetRef(MakeUnique<IStreamingGenerationContext::FActorSet>()).Get();
			const_cast<IStreamingGenerationContext::FActorSet*>(NewHLODActorSetInstance.ActorSet)->Actors.Add(HLODActorDescView.GetGuid());

			HLODActorSetInstancePtrs.Add(&NewHLODActorSetInstance);
		};

		// Generate HLODs for this grid - retrieve actors from our ValidHLODActors map
		// We can't rely on actor descs for newly created HLOD actors
		GenerateHLODActors(HLODGrids[HLODGridName], HLODLayersLevels[HLODGrids[HLODGridName].HLODLayer] + 1, HLODActorSetInstancePtrs);
	}

	auto DeleteHLODActor = [&SourceControlHelper, WorldPartition](FWorldPartitionHandle ActorHandle)
	{
		FWorldPartitionActorDescInstance* ActorDescInstance = *ActorHandle;
		check(ActorDescInstance);

		DeletePackage(WorldPartition, ActorDescInstance, SourceControlHelper);
	};

	// Destroy all unreferenced HLOD actors
	for (const auto& HLODActorPair : Context.HLODActorDescs)
	{
		FHLODCreationFilterContext FilterContext;
		FilterContext.Bounds = HLODActorPair.Value->GetRuntimeBounds();
		const bool bPassesAllFilters = UE::HLOD::CreationFilter::PassesFilters(Params.Filters, FilterContext);

		if (bPassesAllFilters)
		{
			DeletedActors.Add(HLODActorPair.Key);

			if (!Params.bReportOnly)
			{
				if (Params.bSaveActors)
				{
					DeleteHLODActor(HLODActorPair.Value);
				}
				else
				{
					FWorldPartitionReference HLODActorRef = HLODActorPair.Value.ToReference();
					AWorldPartitionHLOD* HLODActor = CastChecked<AWorldPartitionHLOD>(HLODActorRef.GetActor());

					ResetLoaders(HLODActor);
					WorldPartition->GetWorld()->EditorDestroyActor(HLODActor, /*bShouldModifyLevel=*/false);
					WorldPartition->OnPackageDeleted(HLODActor->GetPackage());
					HLODActor->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
				}
			}
		}
	}

	if (!Params.bReportOnly)
	{
		BaseContainerInstanceCollection->ForEachActorDescContainerInstance([this, &HLODGrids, &SourceControlHelper](const UActorDescContainerInstance* ActorDescContainerInstance)
		{
			// Create/destroy HLOD grid actors
			UpdateHLODGridsActors(GetWorld(), ActorDescContainerInstance, HLODGrids, SourceControlHelper);
		});
	}

	auto DumpActorsStats = [](const TCHAR* StatType, const TArray<FName>& Actors)
	{
		UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Display, "\t%ls actors (%d):", StatType, Actors.Num());
		for (FName ActorName : Actors)
		{
			UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Display, "\t\t%ls:", *ActorName.ToString());
		}
	};

	UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Display, "SetupHLODActors stats:");

	DumpActorsStats(TEXT("New"), NewActors);
	DumpActorsStats(TEXT("Dirty"), DirtyActors);
	DumpActorsStats(TEXT("Deleted"), DeletedActors);

	return true;
}

void UWorldPartitionRuntimeSpatialHash::PreSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const
{
	if (!InParams.bReportOnly)
	{
		// Clear "in-use" tag on all ASpatialHashRuntimeGridInfo actors
		for (FActorDescContainerInstanceCollection::TConstIterator<ASpatialHashRuntimeGridInfo> Iterator(InWorldPartition); Iterator; ++Iterator)
		{
			if (ASpatialHashRuntimeGridInfo* SpatialHashRuntimeGridInfo = Cast<ASpatialHashRuntimeGridInfo>(Iterator->GetActor()))
			{
				SpatialHashRuntimeGridInfo->Tags.Remove(HLODGridActorInUseTag);
			}
		}
	}
}

void UWorldPartitionRuntimeSpatialHash::PostSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams) const
{
	if (!InParams.bReportOnly)
	{
		// Delete all ASpatialHashRuntimeGridInfo actors that don't have the "in-use" tag
		for (FActorDescContainerInstanceCollection::TConstIterator<ASpatialHashRuntimeGridInfo> Iterator(InWorldPartition); Iterator; ++Iterator)
		{
			if (ASpatialHashRuntimeGridInfo* SpatialHashRuntimeGridInfo = Cast<ASpatialHashRuntimeGridInfo>(Iterator->GetActor()))
			{
				if (!SpatialHashRuntimeGridInfo->ActorHasTag(HLODGridActorInUseTag))
				{
					FString PackageName = Iterator->GetActorPackage().ToString();
					bool bDeleted = InParams.SourceControlHelper->Delete(SpatialHashRuntimeGridInfo->GetPackage());
					if (bDeleted)
					{
						UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Display, "Deleting %ls...", *PackageName);
					}
					else
					{
						UE_LOGF(LogWorldPartitionRuntimeSpatialHashHLOD, Error, "Failed to delete %ls", *PackageName);
					}
				}
			}
		}
	}
}

#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
