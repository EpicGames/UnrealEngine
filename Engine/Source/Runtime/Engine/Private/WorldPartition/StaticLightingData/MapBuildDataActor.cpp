// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/StaticLightingData/MapBuildDataActor.h"

#include "Engine/MapBuildDataRegistry.h"
#include "WorldPartition/WorldPartition.h"
#include "SceneInterface.h"
#include "Engine/World.h"
#include "Engine/Level.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MapBuildDataActor)

AMapBuildDataActor::AMapBuildDataActor(class FObjectInitializer const &ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAddedToWorld = false;
}

void AMapBuildDataActor::PostLoad()
{
	Super::PostLoad();

	if (BuildData)
	{
		BuildData->SetupLightmapResourceClusters();
	}

	if (UWorld* World = GetWorld())
	{ 
		//AddToWorldMapBuildData();	// in PIE/Runtime the actor is loaded in a Level that isn't in the world so can't do this here
		InitializeRenderingResources();
	}
}

void AMapBuildDataActor::PostUnregisterAllComponents()
{
	RemoveFromWorldMapBuildData();
	ReleaseRenderingResources();

	Super::PostUnregisterAllComponents();
}

void AMapBuildDataActor::BeginDestroy()
{
	check(!bAddedToWorld); // too late to do it once you're here
	ReleaseRenderingResources();
	
	Super::BeginDestroy();
}

#if WITH_EDITOR
void AMapBuildDataActor::PreDuplicateFromRoot(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicateFromRoot(DupParams);

	if (DupParams.DuplicateMode == EDuplicateMode::PIE)
	{
		if (BuildData)
		{
			// Prevent MapBuildData duplication when PIEing, saves a little bit of memory. 
			DupParams.DuplicationSeed.Emplace(BuildData, BuildData);
		}
	}
}
#endif

void AMapBuildDataActor::PreRegisterAllComponents()
{ 
	Super::PreRegisterAllComponents();

	AddToWorldMapBuildData();
	InitializeRenderingResources();
}

void AMapBuildDataActor::AddToWorldMapBuildData()
{
	UWorld* World = GetWorld(); 
	if (BuildData && !bAddedToWorld)
	{
		check(World);
		if  (UMapBuildDataRegistry* Registry = World->PersistentLevel->MapBuildData)
		{
#if WITH_EDITOR
			//@todo_ow: If we need this sooner (in postload without the world) we could make the redirect TMap static
			Registry->RedirectToRegistry(ActorInstances, BuildData);
#endif
			bAddedToWorld = true;
		}
	}
}

void AMapBuildDataActor::RemoveFromWorldMapBuildData()
{
	if (BuildData && bAddedToWorld && GetWorld())
	{
		if (UMapBuildDataRegistry* Registry = GetWorld()->PersistentLevel->MapBuildData)
		{
#if WITH_EDITOR
			Registry->RemoveRedirect(ActorInstances, BuildData);
#else
			Registry->RemoveRegistry(BuildData);
#endif
			bAddedToWorld = false;
		}
	}
}

void AMapBuildDataActor::GetActorBounds(bool bOnlyCollidingComponents, FVector& OutOrigin, FVector& OutBoxExtent, bool bIncludeFromChildActors) const
{
	Super::GetActorBounds(bOnlyCollidingComponents, OutOrigin, OutBoxExtent, bIncludeFromChildActors);

	if (!bOnlyCollidingComponents)
	{
		ActorBounds.GetCenterAndExtents(OutOrigin, OutBoxExtent);
	}
}

void AMapBuildDataActor::LinkToActor(AActor* Actor)
{
	ForceLinkToActor = Actor;
}

UMapBuildDataRegistry* AMapBuildDataActor::GetBuildData(bool bCreateIfNotFound)
{
	if (bCreateIfNotFound && !BuildData)
	{
		BuildData = NewObject<UMapBuildDataRegistry>(this, FName(FString::Printf(TEXT("MapBuildData_%s"), *CellPackage.ToString())));
		MarkPackageDirty();
	}

	return BuildData;
}

void AMapBuildDataActor::SetBuildData(UMapBuildDataRegistry* MapBuildData)
{
	check(!BuildData || (BuildData == MapBuildData));

	if (MapBuildData->GetOuter() != this)
	{
		check(MapBuildData->GetOuter() == GetWorld()->PersistentLevel->MapBuildData);	// make sure we're only reoutering MapBuildData created in FStaticLightingDescriptors::GetOrCreateRegistryForActor
		MapBuildData->Rename(nullptr, this, REN_DontCreateRedirectors);
		MapBuildData->ClearFlags(RF_Standalone); // If created before the MapBuildData actor the BuilData will be marked RF_StandAlone, not the case when outered to a AMapBuildDataActor
	}
	
	BuildData = MapBuildData;
}

void AMapBuildDataActor::InitializeRenderingResources()
{
	if (GetWorld() && FApp::CanEverRender())
	{
		if (GetWorld()->Scene && BuildData)
		{
			BuildData->SetupLightmapResourceClusters(); // Done in MapBuildData PostLoad but order is not guaranteed
			BuildData->InitializeClusterRenderingResources(GetWorld()->Scene->GetFeatureLevel());
		}
	}
}

void AMapBuildDataActor::ReleaseRenderingResources()
{	
	// Calls to MapBuildData::RelaseResources happens in UMapBuildDataRegistry::BeginDestroy() we don't need to do anything here
	if (GetWorld() && FApp::CanEverRender())
	{
		if (GetWorld()->Scene && BuildData)
		{
		}
	}
}

#if WITH_EDITOR
TUniquePtr<class FWorldPartitionActorDesc> AMapBuildDataActor::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FMapBuildDataActorDesc());
}

void AMapBuildDataActor::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	FBox Bounds;
	check(ActorBounds.IsValid);

	OutRuntimeBounds = OutEditorBounds = ActorBounds;
}

void AMapBuildDataActor::SetBounds(FBox& Bounds)
{
	check(Bounds.IsValid);
	ActorBounds = Bounds;
}

FMapBuildDataActorDesc::FMapBuildDataActorDesc()
{
}

void FMapBuildDataActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	const AMapBuildDataActor* MapBuildDataActor = CastChecked<const AMapBuildDataActor>(InActor);

	CellPackage = MapBuildDataActor->CellPackage;
}

bool FMapBuildDataActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FMapBuildDataActorDesc& MapBuildDataActorDesc = *(FMapBuildDataActorDesc*)Other;		
		return CellPackage == MapBuildDataActorDesc.CellPackage;
	}
	return false;
}

void FMapBuildDataActorDesc::Serialize(FArchive& Ar)
{
	FWorldPartitionActorDesc::Serialize(Ar);

	if (!bIsDefaultActorDesc)
	{
		Ar << CellPackage;
	}
}

bool FMapBuildDataActorDesc::IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	return true;
}

#endif // WITH_EDITOR
