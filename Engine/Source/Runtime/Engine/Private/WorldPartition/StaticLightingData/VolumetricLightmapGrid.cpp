// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/StaticLightingData/VolumetricLightmapGrid.h"

#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "PrecomputedVolumetricLightmap.h"

#include "Serialization/VersionedArchive.h"
#include "Misc/FileHelper.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/BulkDataReader.h"
#include "Serialization/BulkDataWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumetricLightmapGrid)

using FVersionedBulkDataReader = TVersionedReader<FBulkDataReader>;
using FVersionedBulkDataWriter = TVersionedWriter<FBulkDataWriter>;

#if WITH_EDITOR
void FVolumetricLightMapGridDesc::Initialize(UWorld* InWorld, const FBox& InBounds)
{
	// Initialize to a new guid to unlink all data	
	Guid = FGuid::NewGuid();

	FVector::FReal VLMCellSize = InWorld->GetWorldSettings()->LightmassSettings.VolumetricLightmapLoadingCellSize;
	FVector::FReal VLMDetailCellSize = InWorld->GetWorldSettings()->LightmassSettings.VolumetricLightmapDetailCellSize;

	int VLMBrickSize = 4;
	int VLMMaxRefinementLevels = 3;
	
	GConfig->GetInt(TEXT("DevOptions.VolumetricLightmaps"), TEXT("BrickSize"), VLMBrickSize, GLightmassIni);
	GConfig->GetInt(TEXT("DevOptions.VolumetricLightmaps"), TEXT("MaxRefinementLevels"), VLMMaxRefinementLevels, GLightmassIni);

	VLMMaxRefinementLevels = FMath::Clamp(VLMMaxRefinementLevels, 1, 6);

	const int32 BrickSizeLog2 = FMath::FloorLog2(VLMBrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (VLMMaxRefinementLevels * BrickSizeLog2);

	// Adjust VLMCellSize to fit our requirements
	// VLMCellSize must be an integer multiple of a brick in world units so that we can properly align bricks with cell transitions	
	VLMCellSize = FMath::CeilToDouble(VLMCellSize / (VLMDetailCellSize*VLMBrickSize)) * (VLMDetailCellSize*VLMBrickSize);

	FVector VLMCellMinExtent = FVector(0, 0, InBounds.Min.Z);
	FVector VLMCellMaxExtent = FVector(VLMCellSize, VLMCellSize, InBounds.Max.Z);	

	FIntVector2 VLMGridMin = FIntVector2(FMath::FloorToInt(InBounds.Min.X / VLMCellSize), FMath::FloorToInt(InBounds.Min.Y / VLMCellSize));
	FIntVector2 VLMGridMax = FIntVector2(FMath::CeilToInt(InBounds.Max.X / VLMCellSize), FMath::CeilToInt(InBounds.Max.Y / VLMCellSize));
		
	// Keep the values used to generate the grid for proper usage of the data
	DetailCellSize = VLMDetailCellSize;
	BrickSize = VLMDetailCellSize * VLMBrickSize;
	CellSize = VLMCellSize;	

	GridBounds = FBox(FVector(VLMGridMin.X * VLMCellSize, VLMGridMin.Y * VLMCellSize, InBounds.Min.Z) ,
					  FVector(VLMGridMax.X * VLMCellSize, VLMGridMax.Y * VLMCellSize, InBounds.Max.Z));

	
	// Initialize the cells
	Cells.Reset();
	Cells.SetNum((VLMGridMax.X - VLMGridMin.X) * (VLMGridMax.Y - VLMGridMin.Y));

	uint32 CellID = 0;

	for (int i = VLMGridMin.X; i < VLMGridMax.X; i++)
	{
		for (int j = VLMGridMin.Y; j < VLMGridMax.Y; j++)
		{			
			FVector CellPosition (i * VLMCellSize, j * VLMCellSize, 0);

			Cells[CellID].Bounds = FBox(CellPosition + VLMCellMinExtent, CellPosition + VLMCellMaxExtent);
			Cells[CellID].CellID = CellID;
			CellID++;
		}
	}
}
#endif

FVolumetricLightMapGridCell* FVolumetricLightMapGridDesc::GetCell(FGuid CellGuid) const
{
	check(IsValid());
	
	for (const FVolumetricLightMapGridCell& Cell : Cells)
	{
		if (GetCellGuid(Cell.CellID) == CellGuid)
		{
			return const_cast<FVolumetricLightMapGridCell*>(&Cell);
		}
	}	

	return nullptr;
}

FVolumetricLightMapGridCell* FVolumetricLightMapGridDesc::GetCell(const FVector& InPosition) const
{
	check(IsValid());
	
	for (const FVolumetricLightMapGridCell& Cell : Cells)
	{
		if (Cell.Bounds.IsInsideOrOn(InPosition))		
		{
			return const_cast<FVolumetricLightMapGridCell*>(&Cell);
		}
	}	

	return nullptr;
}

FGuid FVolumetricLightMapGridDesc::GetCellGuid(uint32 CellID) const
{
	return FGuid::Combine(Guid, FGuid(CellID, CellID, CellID, CellID));
}

FPrecomputedVolumetricLightmapData* FVolumetricLightMapGridDesc::GetPrecomputedVolumetricLightmapBuildData(FGuid LevelId) const
{
#if WITH_EDITOR	
	if (FVolumetricLightMapGridCell* Cell = GetCell(LevelId))
	{
		return Cell->EditorData;
	}
#endif

	return nullptr;
}


FPrecomputedVolumetricLightmapData* FVolumetricLightMapGridDesc::GetOrCreatePrecomputedVolumetricLightmapBuildData(FGuid LevelId)
{
#if WITH_EDITOR	
	FVolumetricLightMapGridCell* Cell = GetCell(LevelId);
	check(Cell); 

	if (!Cell->EditorData)
	{
		Cell->EditorData  = new FPrecomputedVolumetricLightmapData();
	}

	return Cell->EditorData;
#else
	return nullptr;
#endif
}


TArray<FVolumetricLightMapGridCell*> FVolumetricLightMapGridDesc::GetIntersectingCells(const FBox& InBounds, bool bInWithData)
{
	TArray<FVolumetricLightMapGridCell*> IntersectingCells;

	for (FVolumetricLightMapGridCell& Cell : Cells)
	{		
#if WITH_EDITOR
		bool bCellHasData = Cell.BulkData.GetElementCount() || Cell.EditorData;
#else
		bool bCellHasData = Cell.BulkData.GetElementCount() > 0;
#endif

		if (Cell.Bounds.Intersect(InBounds) && (bCellHasData || !bInWithData) )
		{
			IntersectingCells.Add(&Cell);
		}
	}

	return IntersectingCells;
}

void FVolumetricLightMapGridDesc::SerializeBulkData(FArchive& Ar, UObject* Owner)
{	
	for (FVolumetricLightMapGridCell& Cell : Cells)
	{
		Cell.BulkData.Serialize(Ar, Owner);
	}
}

#if WITH_EDITOR

void FVolumetricLightMapGridDesc::InitializeBulkData()
{
	for (FVolumetricLightMapGridCell& Cell : Cells)
	{
		Cell.BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);

		if (!Cell.EditorData)
		{
			Cell.BulkData.RemoveBulkData();
			continue;
		}
			
		FVersionedBulkDataWriter Ar(Cell.BulkData, true);
		FPrecomputedVolumetricLightmapData* LightmapData = Cell.EditorData;
		Ar << LightmapData;
	}
}
#endif

void FVolumetricLightMapGridDesc::LoadVolumetricLightMapCell(FVolumetricLightMapGridCell& Cell, FPrecomputedVolumetricLightmapData*& OutData)
{
	if (Cell.BulkData.GetElementCount())
	{
		FVersionedBulkDataReader Ar(Cell.BulkData, true);
		Ar << OutData;
	}
}

FString	FVolumetricLightMapGridDesc::GetCellDesc(FGuid CellGuid) const
{
	if (FVolumetricLightMapGridCell* Cell = GetCell(CellGuid))
	{
		return FString::Printf(TEXT("Cell: %i (%s -> %s)"), Cell->CellID, *Cell->Bounds.Min.ToString(), *Cell->Bounds.Max.ToString());
	}

	return FString();		
}
