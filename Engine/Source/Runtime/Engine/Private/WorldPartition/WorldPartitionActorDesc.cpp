// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

#include "Misc/Paths.h"
#include "Misc/ArchiveMD5.h"
#include "UObject/MetaData.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstanceViewInterface.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"
#include "ActorReferencesUtils.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionActorDesc"

TMap<TSubclassOf<AActor>, FWorldPartitionActorDesc::FActorDescDeprecator> FWorldPartitionActorDesc::Deprecators;

static FGuid GetDefaultActorDescGuid(const FWorldPartitionActorDesc* ActorDesc)
{
	FArchiveMD5 ArMD5;
	FString ClassPath = ActorDesc->GetBaseClass().IsValid() ? ActorDesc->GetBaseClass().ToString() : ActorDesc->GetNativeClass().ToString();	
	ArMD5 << ClassPath;
	return ArMD5.GetGuidFromHash();
}

FWorldPartitionComponentDesc::FWorldPartitionComponentDesc()
{}

void FWorldPartitionComponentDesc::Init(const UActorComponent* InComponent)
{
	check(IsValid(InComponent));

	UClass* ComponentClass = InComponent->GetClass();

	// Get the first native class in the hierarchy
	ComponentNativeClass = GetParentNativeClass(ComponentClass);
	NativeClass = *ComponentNativeClass->GetPathName();
	
	// For native class, don't set this
	if (!ComponentClass->IsNative())
	{
		BaseClass = *ComponentClass->GetPathName();
	}

	FString CleanComponentName = InComponent->GetName();
	CleanComponentName.RemoveFromEnd(UActorComponent::ComponentTemplateNameSuffix);
	ComponentName = *CleanComponentName;
}

void FWorldPartitionComponentDesc::Serialize(FArchive& Ar)
{
	Ar << BaseClass;
}

FWorldPartitionActorDesc::FWorldPartitionActorDesc()
	: ActorTransformRelative(FTransform::Identity)
	, RuntimeBoundsRelative(ForceInit)
	, EditorBoundsRelative(ForceInit)
	, bIsSpatiallyLoaded(false)
	, bActorIsEditorOnly(false)
	, bActorIsRuntimeOnly(false)
	, bActorIsMainWorldOnly(false)
	, bActorIsHLODRelevant(false)
	, bActorIsListedInSceneOutliner(true)
	, bIsUsingDataLayerAsset(false)
	, ActorNativeClass(nullptr)
	, Container(nullptr)
	, ActorTransform(FTransform::Identity)
	, RuntimeBounds(ForceInit)
	, EditorBounds(ForceInit)
	, bIsDefaultActorDesc(false)
	, bHasValidRelativeBounds(false)
{}

void FWorldPartitionActorDesc::Init(const AActor* InActor)
{	
	check(IsValid(InActor));

	UClass* ActorClass = InActor->GetClass();

	// Get the first native class in the hierarchy
	ActorNativeClass = GetParentNativeClass(ActorClass);
	NativeClass = *ActorNativeClass->GetPathName();
	
	// For native class, don't set this
	if (!ActorClass->IsNative())
	{
		BaseClass = *ActorClass->GetPathName();
	}

	if (InActor->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		check(!InActor->IsPackageExternal());
		check(!InActor->GetActorGuid().IsValid());
		Guid = GetDefaultActorDescGuid(this);
		bIsDefaultActorDesc = true;
	}
	else
	{
		check(InActor->GetActorGuid().IsValid());
		Guid = InActor->GetActorGuid();
	}

	check(Guid.IsValid());

	RuntimeBounds.Init();
	EditorBounds.Init();
	
	if (!bIsDefaultActorDesc)
	{
		ActorTransform = InActor->GetActorTransform();
		ActorTransformRelative = InActor->GetRootComponent() ? InActor->GetRootComponent()->GetRelativeTransform() : ActorTransform;

		InActor->GetStreamingBounds(RuntimeBounds, EditorBounds);
		FixupStreamingBounds();

		RuntimeBoundsRelative = FWorldPartitionRelativeBounds(RuntimeBounds).InverseTransformBy(ActorTransform);
		EditorBoundsRelative = FWorldPartitionRelativeBounds(EditorBounds).InverseTransformBy(ActorTransform);

		bHasValidRelativeBounds = true;
	}

	RuntimeGrid = InActor->GetRuntimeGrid();
	bIsSpatiallyLoaded = InActor->GetIsSpatiallyLoaded();
	bActorIsEditorOnly = InActor->IsEditorOnly();
	bActorIsRuntimeOnly = InActor->IsRuntimeOnly();
	bActorIsHLODRelevant = InActor->IsHLODRelevant();
	bActorIsListedInSceneOutliner = InActor->IsListedInSceneOutliner();
	bActorIsMainWorldOnly = InActor->bIsMainWorldOnly;
	HLODLayer = InActor->GetHLODLayer() ? FSoftObjectPath(InActor->GetHLODLayer()->GetPathName()) : FSoftObjectPath();
	
	// DataLayers
	{
		DataLayers.Empty();

		// Possible there is no World Partition for regular OFPA levels that haven't been converted to support Data Layers
		if (UWorldPartition* ActorWorldPartition = FWorldPartitionHelpers::GetWorldPartition(InActor))
		{
			// Force load Data Layer Assets (SoftObjectPtrs) to make sure desc remains in sync with the actor
			InActor->ForceLoadDataLayerAssets();
			const bool bIncludeExternalDataLayerAsset = false;
			TArray<const UDataLayerAsset*> DataLayerAssets = InActor->GetDataLayerAssets(bIncludeExternalDataLayerAsset);
			DataLayers.Reserve(DataLayerAssets.Num());
			// Init DataLayers persistent info
			for (const UDataLayerAsset* DataLayerAsset : DataLayerAssets)
			{
				if (DataLayerAsset)
				{
					DataLayers.Add(*DataLayerAsset->GetPathName());
				}
			}
		}

		// Initialize ExternalDataLayerAsset
		ExternalDataLayerAsset = InActor->GetExternalDataLayerAsset() ? FSoftObjectPath(InActor->GetExternalDataLayerAsset()->GetPathName()) : FSoftObjectPath();
	}

	Tags = InActor->Tags;

	check(Properties.IsEmpty());
	InActor->GetActorDescProperties(Properties);

	ActorPackage = InActor->GetPackage()->GetFName();
	ActorPath = bIsDefaultActorDesc ? *ActorClass->GetPathName() : *InActor->GetPathName();
	ActorName = InActor->GetFName();
	ActorNameString = ActorName.ToString();

	ContentBundleGuid = InActor->GetContentBundleGuid();

	if (!bIsDefaultActorDesc)
	{
		FolderPath = InActor->GetFolderPath();
		FolderGuid = InActor->GetFolderGuid();

		const AActor* AttachParentActor = InActor->GetAttachParentActor();
		if (AttachParentActor)
		{
			while (AttachParentActor->GetParentActor())
			{
				AttachParentActor = AttachParentActor->GetParentActor();
			}
			ParentActor = AttachParentActor->GetActorGuid();
		}

		const ActorsReferencesUtils::FGetActorReferencesParams Params = ActorsReferencesUtils::FGetActorReferencesParams(const_cast<AActor*>(InActor));
		TArray<ActorsReferencesUtils::FActorReference> ActorReferences = ActorsReferencesUtils::GetActorReferences(Params);

		if (ActorReferences.Num())
		{
			References.Reserve(ActorReferences.Num());
			for (const ActorsReferencesUtils::FActorReference& ActorReference : ActorReferences)
			{
				const FGuid ActorReferenceGuid = ActorReference.Actor->GetActorGuid();

				References.Add(ActorReferenceGuid);

				if (ActorReference.bIsEditorOnly)
				{
					EditorOnlyReferences.Add(ActorReferenceGuid);
				}
			}
		}

		ActorLabel = *InActor->GetActorLabel(false);
	}

	ActorLabelString = ActorLabel.ToString();
	ActorDisplayClassNameString = GetDisplayClassName().ToString();

	if (!bIsDefaultActorDesc)
	{
		InActor->ForEachComponent(false, [this](const UActorComponent* ActorComponent)
		{
			if (TUniquePtr<FWorldPartitionComponentDesc> ComponentDesc = ActorComponent->CreateComponentDesc())
			{
				ComponentDescs.Add(MoveTemp(ComponentDesc));
			}
		});
	}
	else
	{
		AActor::ForEachComponentOfActorClassDefault<UActorComponent>(ActorClass, [this](const UActorComponent* ActorComponent)
		{
			if (TUniquePtr<FWorldPartitionComponentDesc> ComponentDesc = ActorComponent->CreateComponentDesc())
			{
				ComponentDescs.Add(MoveTemp(ComponentDesc));
			}
			return true;
		});
	}

	Container = nullptr;
}

void FWorldPartitionActorDesc::InitTransientProperties(const FWorldPartitionActorDescInitData& DescData)
{
	ActorPackage = DescData.PackageName;
	ActorPath = DescData.ActorPath;
	ActorNativeClass = DescData.NativeClass ? DescData.NativeClass : AActor::StaticClass();
	NativeClass = *ActorNativeClass->GetPathName();
	ActorName = *FPaths::GetExtension(ActorPath.ToString());
	ActorNameString = ActorName.ToString();
}

void FWorldPartitionActorDesc::Init(const FWorldPartitionActorDescInitData& DescData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionActorDesc::Init);

	InitTransientProperties(DescData);

	auto DeprecateClass = [this](FArchive& Archive)
	{
		// Call registered deprecator
		TSubclassOf<AActor> DeprecatedClass = ActorNativeClass;
		while (DeprecatedClass)
		{
			if (FActorDescDeprecator* Deprecator = Deprecators.Find(DeprecatedClass))
			{
				(*Deprecator)(Archive, this);
				break;
			}
			DeprecatedClass = DeprecatedClass->GetSuperClass();
		}
	};

	// Serialize actor metadata
	if (!DescData.IsUsingArchive())
	{
		FMemoryReader MetadataAr(DescData.GetSerializedData(), true);

		// Serialize metadata custom versions
		FCustomVersionContainer CustomVersions;
		CustomVersions.Serialize(MetadataAr);
		MetadataAr.SetCustomVersions(CustomVersions);

		TArray<FCustomVersionDifference> Diffs = FCurrentCustomVersions::Compare(CustomVersions.GetAllVersions(), *DescData.PackageName.ToString());
		for (FCustomVersionDifference Diff : Diffs)
		{
			if (Diff.Type == ECustomVersionDifference::Missing)
			{
				UE_LOGF(LogWorldPartition, Fatal, "Missing custom version for actor descriptor '%ls'", *DescData.PackageName.ToString());
			}
			else if (Diff.Type == ECustomVersionDifference::Invalid)
			{
				UE_LOGF(LogWorldPartition, Fatal, "Invalid custom version for actor descriptor '%ls'", *DescData.PackageName.ToString());
			}
			else if (Diff.Type == ECustomVersionDifference::Newer)
			{
				int32 PackageVersion = -1;
				int32 HeadCodeVersion = -1;
				if (const FCustomVersion* PackagePtr = CustomVersions.GetVersion(Diff.Version->Key))
				{
					PackageVersion = PackagePtr->Version;
				}
				if (TOptional<FCustomVersion> CurrentPtr = FCurrentCustomVersions::Get(Diff.Version->Key))
				{
					HeadCodeVersion = CurrentPtr->Version;
				}
				UE_LOGF(LogWorldPartition, Fatal, "Newer custom version for actor descriptor '%ls' (file: %d, head: %d)", *DescData.PackageName.ToString(), PackageVersion, HeadCodeVersion);
			}
		}
	
		// Serialize metadata payload
		FActorDescArchive ActorDescAr(MetadataAr, this);
		ActorDescAr.Init();

		Serialize(ActorDescAr);
		DeprecateClass(MetadataAr);
	}
	else
	{
		Serialize(*DescData.GetArchive());
		DeprecateClass(*DescData.GetArchive());
	}

	ActorLabelString = ActorLabel.ToString();
	ActorDisplayClassNameString = GetDisplayClassName().ToString();

	Container = nullptr;

	FixupStreamingBounds();
}

void FWorldPartitionActorDesc::Patch(const FWorldPartitionActorDescInitData& DescData, TArray<uint8>& OutData, FWorldPartitionAssetDataPatcher* InAssetDataPatcher)
{
	// Serialize actor metadata
	FMemoryReader MetadataAr(DescData.GetSerializedData(), true);

	// Serialize metadata custom versions
	FCustomVersionContainer CustomVersions;
	CustomVersions.Serialize(MetadataAr);
	MetadataAr.SetCustomVersions(CustomVersions);
	
	// Patch metadata payload
	TArray<uint8> PatchedPayloadData;
	FMemoryWriter PatchedPayloadAr(PatchedPayloadData, true);

	UClass* NativeClass = DescData.NativeClass ? DescData.NativeClass : AActor::StaticClass();
	TUniquePtr<FWorldPartitionActorDesc> ActorDesc(AActor::StaticCreateClassActorDesc(NativeClass));
	ActorDesc->InitTransientProperties(DescData);
	FActorDescArchivePatcher ActorDescAr(MetadataAr, ActorDesc.Get(), PatchedPayloadAr, InAssetDataPatcher);
	FTopLevelAssetPath ActorClassPath(NativeClass->GetPathName());
	ActorDescAr.Init(ActorClassPath);

	ActorDesc->Serialize(ActorDescAr);

	// Serialize custom versions
	TArray<uint8> HeaderData;
	FMemoryWriter HeaderAr(HeaderData);
	CustomVersions.Serialize(HeaderAr); 

	// Append data
	OutData = MoveTemp(HeaderData);
	OutData.Append(PatchedPayloadData);
}

bool FWorldPartitionActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	return
		Guid == Other->Guid &&
		BaseClass == Other->BaseClass &&
		NativeClass == Other->NativeClass &&
		ActorPackage == Other->ActorPackage &&
		ActorPath == Other->ActorPath &&
		ActorLabel == Other->ActorLabel &&
		ActorTransformRelative.Equals(Other->ActorTransformRelative, 0.1f) &&
		RuntimeBoundsRelative.Equals(Other->RuntimeBoundsRelative, 0.1f) &&
		EditorBoundsRelative.Equals(Other->EditorBoundsRelative, 0.1f) &&
		RuntimeGrid == Other->RuntimeGrid &&
		bIsSpatiallyLoaded == Other->bIsSpatiallyLoaded &&
		bActorIsEditorOnly == Other->bActorIsEditorOnly &&
		bActorIsRuntimeOnly == Other->bActorIsRuntimeOnly &&
		bActorIsMainWorldOnly == Other->bActorIsMainWorldOnly &&
		bActorIsHLODRelevant == Other->bActorIsHLODRelevant &&
		bActorIsListedInSceneOutliner == Other->bActorIsListedInSceneOutliner &&
		HLODLayer == Other->HLODLayer &&
		FolderPath == Other->FolderPath &&
		FolderGuid == Other->FolderGuid &&
		ParentActor == Other->ParentActor &&
		ContentBundleGuid == Other->ContentBundleGuid &&
		ComponentDescs == Other->ComponentDescs &&
		CompareUnsortedArrays(DataLayers, Other->DataLayers) &&
		CompareUnsortedArrays(References, Other->References) &&
		CompareUnsortedArrays(EditorOnlyReferences, Other->EditorOnlyReferences) &&
		CompareUnsortedArrays(Tags, Other->Tags) &&
		Properties == Other->Properties &&
		ExternalDataLayerAsset == Other->ExternalDataLayerAsset;
}

bool FWorldPartitionActorDesc::ShouldResave(const FWorldPartitionActorDesc* Other) const
{
	check(Guid == Other->Guid);
	check(ActorPackage == Other->ActorPackage);
	check(ActorPath == Other->ActorPath);

	if (RuntimeGrid != Other->RuntimeGrid ||
		bIsSpatiallyLoaded != Other->bIsSpatiallyLoaded ||
		bActorIsEditorOnly != Other->bActorIsEditorOnly ||
		bActorIsRuntimeOnly != Other->bActorIsRuntimeOnly ||
		bActorIsMainWorldOnly != Other->bActorIsMainWorldOnly ||
		RuntimeBoundsRelative.IsValid() != Other->RuntimeBoundsRelative.IsValid() ||
		EditorBoundsRelative.IsValid() != Other->EditorBoundsRelative.IsValid() ||
		HLODLayer != Other->HLODLayer ||
		ParentActor != Other->ParentActor ||
		ContentBundleGuid != Other->ContentBundleGuid ||
		!CompareUnsortedArrays(DataLayers, Other->DataLayers) ||
		!CompareUnsortedArrays(References, Other->References) ||
		!CompareUnsortedArrays(EditorOnlyReferences, Other->EditorOnlyReferences) ||
		Properties != Other->Properties ||
		ExternalDataLayerAsset != Other->ExternalDataLayerAsset)
	{
		return true;
	}

	// Tolerate up to 5% for bounds change
	if (RuntimeBoundsRelative.IsValid())
	{
		const FBox ThisBounds = RuntimeBoundsRelative.ToAABB();
		const FBox OtherBounds = Other->RuntimeBoundsRelative.ToAABB();
		const FVector BoundsChangeTolerance = ThisBounds.GetSize() * 0.05f;
		const FVector MinDiff = FVector(OtherBounds.Min - ThisBounds.Min).GetAbs();
		const FVector MaxDiff = FVector(OtherBounds.Max - ThisBounds.Max).GetAbs();
		if ((MinDiff.X > BoundsChangeTolerance.X) || (MaxDiff.X > BoundsChangeTolerance.X) ||
			(MinDiff.Y > BoundsChangeTolerance.Y) || (MaxDiff.Y > BoundsChangeTolerance.Y) ||
			(MinDiff.Z > BoundsChangeTolerance.Z) || (MaxDiff.Z > BoundsChangeTolerance.Z))
		{
			return true;
		}
	}

	// If the actor descriptor says the actor is HLOD relevant but in reality it's not, this will incur a loading time penalty during HLOD generation
	// but will not affect the final result, as the value from the loaded actor will be used instead, so don't consider this as affecting streaming generation.
	return !bActorIsHLODRelevant && Other->bActorIsHLODRelevant;
}

void FWorldPartitionActorDesc::SerializeTo(TArray<uint8>& OutData, FWorldPartitionActorDesc* BaseDesc) const
{
	FWorldPartitionActorDesc* MutableThis = const_cast<FWorldPartitionActorDesc*>(this);

	// Serialize to archive and gather custom versions
	TArray<uint8> PayloadData;
	FMemoryWriter PayloadAr(PayloadData, true);
	FActorDescArchive ActorDescAr(PayloadAr, MutableThis, BaseDesc);
	ActorDescAr.Init();

	MutableThis->Serialize(ActorDescAr);

	// Serialize custom versions
	TArray<uint8> HeaderData;
	FMemoryWriter HeaderAr(HeaderData);
	FCustomVersionContainer CustomVersions = ActorDescAr.GetCustomVersions();
	CustomVersions.Serialize(HeaderAr); 

	// Append data
	OutData = MoveTemp(HeaderData);
	OutData.Append(PayloadData);
}

bool FWorldPartitionActorDesc::GetActorIsEditorOnlyLoadedInPIE() const
{
	return ActorNativeClass->GetDefaultObject<AActor>()->IsEditorOnlyLoadedInPIE();
}

TArray<FName> FWorldPartitionActorDesc::GetDataLayers(bool bIncludeExternalDataLayer) const
{
	const FName ExternalDataLayer = GetExternalDataLayer();
	if (!bIncludeExternalDataLayer || ExternalDataLayer.IsNone())
	{
		return DataLayers;
	}

	TArray<FName> AllDataLayers;
	AllDataLayers.Reserve(DataLayers.Num() + 1);
	AllDataLayers.Add(ExternalDataLayer);
	AllDataLayers.Append(DataLayers);
	return AllDataLayers;
}

FName FWorldPartitionActorDesc::GetExternalDataLayer() const
{
	const bool bHasExternalDataLayerAsset = ExternalDataLayerAsset.IsValid();
	return bHasExternalDataLayerAsset ? FName(ExternalDataLayerAsset.GetAssetPath().ToString()) : NAME_None;
}

void FWorldPartitionActorDesc::TransferFrom(const FWorldPartitionActorDesc* From)
{
	Container = From->Container;
}

void FWorldPartitionActorDesc::RegisterActorDescDeprecator(TSubclassOf<AActor> ActorClass, const FActorDescDeprecator& Deprecator)
{
	check(!Deprecators.Contains(ActorClass));
	Deprecators.Add(ActorClass, Deprecator);
}

FString FWorldPartitionActorDesc::ToString(EToStringMode Mode) const
{
	FWorldPartitionActorDescStringBuilder Result(Mode);
	Result.AppendToString(TEXT("Guid"), Guid, false);

	if (Mode >= EToStringMode::Compact)
	{
		if (BaseClass.IsValid())
		{
			Result.AppendToString(TEXT("BaseClass"), BaseClass);
		}

		Result.AppendToString(TEXT("NativeClass"), NativeClass);
		Result.AppendFromString(TEXT("Name"), GetActorNameString());

		if (Mode >= EToStringMode::Verbose)
		{
			Result.AppendToString(TEXT("ActorPackage"), ActorPackage);
			Result.AppendToString(TEXT("ActorPath"), ActorPath);
		}

		if (ComponentDescs.Num())
		{
			Result.AppendFromString(TEXT("Components"), FString::JoinBy(ComponentDescs, TEXT(","),
				[&](const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc)
				{
					return ComponentDesc->ToString();
				}));
		}

		Result.AppendToString(TEXT("Label"), GetActorLabel());
		Result.AppendFromBool(TEXT("SpatiallyLoaded"), bIsSpatiallyLoaded);
		Result.AppendToString(TEXT("EditorBounds"), EditorBounds);
		Result.AppendToString(TEXT("EditorBoundsRelative"), EditorBoundsRelative);
		Result.AppendToString(TEXT("RuntimeBounds"), RuntimeBounds);
		Result.AppendToString(TEXT("RuntimeBoundsRelative"), RuntimeBoundsRelative);
		Result.AppendToString(TEXT("RuntimeGrid"), RuntimeGrid);
		Result.AppendFromBool(TEXT("EditorOnly"), bActorIsEditorOnly);
		Result.AppendFromBool(TEXT("RuntimeOnly"), bActorIsRuntimeOnly);
		Result.AppendFromBool(TEXT("HLODRelevant"), bActorIsHLODRelevant);
		Result.AppendFromBool(TEXT("ListedInSceneOutliner"), bActorIsListedInSceneOutliner);
		Result.AppendFromBool(TEXT("IsMainWorldOnly"), IsMainWorldOnly());

		if (ParentActor.IsValid())
		{
			Result.AppendToString(TEXT("Parent"), ParentActor);
		}

		if (HLODLayer.IsValid())
		{
			Result.AppendToString(TEXT("HLODLayer"), HLODLayer);
		}

		if (!FolderPath.IsNone())
		{
			Result.AppendToString(TEXT("FolderPath"), FolderPath);
		}

		if (FolderGuid.IsValid())
		{
			Result.AppendToString(TEXT("FolderGuid"), FolderGuid);
		}

		if (Mode >= EToStringMode::Full)
		{
			if (References.Num())
			{
				Result.AppendFromString(TEXT("References"), FString::JoinBy(References, TEXT(","),
					[&](const FGuid& ReferenceGuid)
					{
						return ReferenceGuid.ToString();
					}));
			}

			if (EditorOnlyReferences.Num())
			{
				Result.AppendFromString(TEXT("EditorOnlyReferences"), FString::JoinBy(EditorOnlyReferences, TEXT(","),
					[&](const FGuid& ReferenceGuid)
					{
						return ReferenceGuid.ToString();
					}));
			}

			if (Tags.Num())
			{
				Result.AppendFromString(TEXT("Tags"), FString::JoinBy(Tags, TEXT(","),
					[&](const FName& TagName)
					{
						return TagName.ToString();
					}));
			}

			if (Properties.Num())
			{
				Result.AppendToString(TEXT("Properties"), Properties);
			}

			if (DataLayers.Num())
			{
				Result.AppendFromString(TEXT("DataLayers"), FString::JoinBy(DataLayers, TEXT(","),
					[&](const FName& DataLayerName)
					{
						return DataLayerName.ToString();
					}));
			}

			if (ExternalDataLayerAsset.IsValid())
			{
				Result.AppendToString(TEXT("ExternalDataLayerAsset"), ExternalDataLayerAsset);
			}
		}
	}

	return Result.ToString();
}

void FWorldPartitionActorDesc::Serialize(FArchive& Ar)
{
	check(Ar.IsPersistent());

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (bIsDefaultActorDesc)
	{
		if (Ar.IsLoading())
		{
			Guid = GetDefaultActorDescGuid(this);

			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionClasDescGuidTransient)
			{
				FGuid ClassDescGuid;
				Ar << ClassDescGuid;
			}
		}
	}
	else
	{
		Ar << Guid;

		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescActorTransformSerialization)
		{
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeRelativeTransform)
			{
				Ar << ActorTransform;
			}
			else
			{
				Ar << ActorTransformRelative;
			}
		}
	}

	if(Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::LargeWorldCoordinates)
	{
		FVector3f BoundsLocationFlt, BoundsExtentFlt;
		Ar << BoundsLocationFlt << BoundsExtentFlt;
		RuntimeBounds = FBox(FVector(BoundsLocationFlt - BoundsExtentFlt), FVector(BoundsLocationFlt + BoundsExtentFlt));
		EditorBounds = RuntimeBounds;
	}
	else if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeEditorBounds)
		{
			bool bIsBoundsValid = true;
			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeInvalidBounds)
			{
				Ar << bIsBoundsValid;
			}

			if (bIsBoundsValid)
			{
				FVector BoundsLocation;
				FVector BoundsExtent;

				Ar << BoundsLocation << BoundsExtent;

				RuntimeBounds = FBox(BoundsLocation - BoundsExtent, BoundsLocation + BoundsExtent);
				EditorBounds = RuntimeBounds;
			}
			else
			{
				RuntimeBounds.Init();
				EditorBounds.Init();
			}
		}
		else if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeRelativeTransform)
		{
			Ar << RuntimeBounds << EditorBounds;
		}
		else
		{
			Ar << RuntimeBoundsRelative << EditorBoundsRelative;
			bHasValidRelativeBounds = true;
		}
	}
	else
	{
		RuntimeBounds.Init();
		EditorBounds.Init();
	}

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ConvertedActorGridPlacementToSpatiallyLoadedFlag)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EActorGridPlacement GridPlacement;
		Ar << (__underlying_type(EActorGridPlacement)&)GridPlacement;
		bIsSpatiallyLoaded = GridPlacement != EActorGridPlacement::AlwaysLoaded;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		Ar << TDeltaSerialize<bool>(bIsSpatiallyLoaded);
	}

	Ar << TDeltaSerialize<FName>(RuntimeGrid);
	Ar << TDeltaSerialize<bool>(bActorIsEditorOnly);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeActorIsRuntimeOnly)
	{
		Ar << TDeltaSerialize<bool>(bActorIsRuntimeOnly);
	}
	else
	{
		bActorIsRuntimeOnly = false;
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescRemoveBoundsRelevantSerialization)
	{
		bool bLevelBoundsRelevant;
		Ar << bLevelBoundsRelevant;
	}
	
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescSerializeDataLayers)
	{
		TArray<FName> Deprecated_Layers;
		Ar << Deprecated_Layers;
	}

	Ar << References;

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeEditorOnlyReferences)
	{
		Ar << EditorOnlyReferences;
	}

	if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescTagsSerialization)
	{
		Ar << TDeltaSerialize<TArray<FName>>(Tags);
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescSerializeArchivePersistent)
	{
		Ar << ActorPackage;
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescActorAndClassPaths)
		{ 
			FName ActorPathName;
			Ar << ActorPathName;
			ActorPath = FSoftObjectPath(ActorPathName.ToString());
		}
		else
		{
			Ar << ActorPath;
		}
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::WorldPartitionActorDescSerializeDataLayers)
	{
		Ar << TDeltaSerialize<TArray<FName>>(DataLayers);
	}

	if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescSerializeDataLayerAssets)
	{
		// Keep loading this value even though it is not used anymore, in case we ever need it for conversions
		Ar << TDeltaSerialize<bool>(bIsUsingDataLayerAsset);
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::WorldPartitionActorDescSerializeActorLabel)
	{
		Ar << TDeltaSerialize<FName>(ActorLabel);
	}

	if ((Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::WorldPartitionActorDescSerializeHLODInfo) ||
		(Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionActorDescSerializeHLODInfo))
	{
		Ar << TDeltaSerialize<bool>(bActorIsHLODRelevant);

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeSoftObjectPathSupport)
		{
			Ar << TDeltaSerialize<FSoftObjectPath, FName>(HLODLayer, [](FSoftObjectPath& Value, const FName& DeprecatedValue)
			{
				Value = FSoftObjectPath(*DeprecatedValue.ToString());
			});
		}
		else
		{
			Ar << TDeltaSerialize<FSoftObjectPath>(HLODLayer);
		}
	}
	else
	{
		bActorIsHLODRelevant = true;
		HLODLayer = FSoftObjectPath();
	}

	if (!bIsDefaultActorDesc)
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionActorDescSerializeActorFolderPath)
		{
			Ar << FolderPath;
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionActorDescSerializeAttachParent)
		{
			Ar << ParentActor;
		}

		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::AddLevelActorFolders)
		{
			Ar << FolderGuid;
		}

		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescPropertyMapSerialization)
		{
			Ar << Properties;
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionActorDescSerializeContentBundleGuid)
	{
		ContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(ActorPackage.ToString());
	}
	else
	{
		Ar << TDeltaSerialize<FGuid>(ContentBundleGuid);

		// @todo_ow: remove once we find why some actors end up with invalid ContentBundleGuids
		if (Ar.IsLoading())
		{
			FGuid FixupContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(ActorPackage.ToString());
			if (ContentBundleGuid != FixupContentBundleGuid)
			{
				UE_LOGF(LogWorldPartition, Log, "ActorDesc ContentBundleGuid was fixed up: %ls", *GetActorNameString());
				ContentBundleGuid = FixupContentBundleGuid;
			}
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorDescIsMainWorldOnly)
	{
		Ar << TDeltaSerialize<bool>(bActorIsMainWorldOnly);
	}

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionActorDescSerializeActorIsListedInSceneOutliner)
	{
		Ar << TDeltaSerialize<bool>(bActorIsListedInSceneOutliner);
	}

    if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::WorldPartitionExternalDataLayers)
	{
		Ar << TDeltaSerialize<FSoftObjectPath>(ExternalDataLayerAsset);
	}

	// Fixup redirected data layer asset paths
	if (Ar.IsLoading())
	{
		for (FName& DataLayer : DataLayers)
		{
			UAssetRegistryHelpers::FixupRedirectedAssetPath(DataLayer);
		}
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorComponentDesc)
	{
		TArray<FWorldPartitionComponentDescInitData> ComponentDescsInitData;

		if (Ar.IsSaving())
		{
			ComponentDescsInitData.Reserve(ComponentDescs.Num());
			for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ComponentDescs)
			{
				ComponentDescsInitData.Add({ ComponentDesc->NativeClass, ComponentDesc->ComponentName });
			}
		}

		Ar << ComponentDescsInitData;

		if (Ar.IsLoading())
		{
			for (const FWorldPartitionComponentDescInitData& ComponentDescInitData : ComponentDescsInitData)
			{
				UClass* ComponentNativeClass = FWorldPartitionActorDescUtils::GetNativeClassFromClassName(*ComponentDescInitData.Type.ToString());
				check(ComponentNativeClass);

				TUniquePtr<class FWorldPartitionComponentDesc> ComponentDesc = UActorComponent::StaticCreateClassComponentDesc(ComponentNativeClass);
				ComponentDesc->NativeClass = *ComponentNativeClass->GetPathName();
				ComponentDesc->ComponentNativeClass = ComponentNativeClass;
				ComponentDesc->ComponentName = ComponentDescInitData.Name;

				ComponentDescs.Add(MoveTemp(ComponentDesc));
			}
		}

		for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ComponentDescs)
		{
			FActorDescArchive& ActorDescAr = (FActorDescArchive&)Ar;
			ActorDescAr.SetComponentDesc(ComponentDesc.Get());
			
			ComponentDesc->Serialize(Ar);

			ActorDescAr.SetComponentDesc(nullptr);
		}
	}
}

FBox FWorldPartitionActorDesc::GetEditorBounds() const
{
	return EditorBounds.IsValid ? EditorBounds : RuntimeBounds;
}

FBox FWorldPartitionActorDesc::GetRuntimeBounds() const
{
	return RuntimeBounds;
}

void FWorldPartitionActorDesc::SetEditorBounds(const FBox& InEditorBounds)
{
	EditorBounds = InEditorBounds;
	EditorBoundsRelative = FWorldPartitionRelativeBounds(EditorBounds).InverseTransformBy(ActorTransform);
}

void FWorldPartitionActorDesc::SetRuntimeBounds(const FBox& InRuntimeBounds)
{
	RuntimeBounds = InRuntimeBounds;
	RuntimeBoundsRelative = FWorldPartitionRelativeBounds(RuntimeBounds).InverseTransformBy(ActorTransform);
}

FName FWorldPartitionActorDesc::GetActorName() const
{
	return ActorName;
}

const FString& FWorldPartitionActorDesc::GetActorNameString() const
{
	return ActorNameString;
}

const FString& FWorldPartitionActorDesc::GetActorLabelString() const
{
	return ActorLabelString;
}

const FString& FWorldPartitionActorDesc::GetDisplayClassNameString() const
{
	return ActorDisplayClassNameString;
}

FName FWorldPartitionActorDesc::GetActorLabelOrName() const
{
	return GetActorLabel().IsNone() ? GetActorName() : GetActorLabel();
}

FName FWorldPartitionActorDesc::GetDisplayClassName() const
{
	auto GetCleanClassName = [](FTopLevelAssetPath ClassPath) -> FName
	{
		// Should this just return ClassPath.GetAssetName() with _C removed if necessary?
		int32 Index;
		FString ClassNameStr(ClassPath.ToString());
		if (ClassNameStr.FindLastChar(TCHAR('.'), Index))
		{
			FString CleanClassName = ClassNameStr.Mid(Index + 1);
			CleanClassName.RemoveFromEnd(TEXT("_C"));
			return *CleanClassName;
		}
		return *ClassPath.ToString(); 
	};

	return BaseClass.IsNull() ? GetCleanClassName(NativeClass) : GetCleanClassName(BaseClass);
}

FGuid FWorldPartitionActorDesc::GetContentBundleGuid() const
{
	return ContentBundleGuid;
}

void FWorldPartitionActorDesc::CheckForErrors(const IWorldPartitionActorDescInstanceView* InActorDescView, IStreamingGenerationErrorHandler* ErrorHandler) const
{
	if (IsResaveNeeded())
	{
		ErrorHandler->OnActorNeedsResave(*InActorDescView);
	}
}

bool FWorldPartitionActorDesc::IsMainWorldOnly() const
{
	return bActorIsMainWorldOnly || CastChecked<AActor>(ActorNativeClass->GetDefaultObject())->IsMainWorldOnly();
}

bool FWorldPartitionActorDesc::IsListedInSceneOutliner() const
{
	return bActorIsListedInSceneOutliner;
}

bool FWorldPartitionActorDesc::IsEditorRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	if (GetActorIsRuntimeOnly())
	{
		return false;
	}

	if (IsMainWorldOnly())
	{
		return InActorDescInstance->GetContainerInstance()->GetContainerID().IsMainContainer();
	}

	return true;
}

bool FWorldPartitionActorDesc::IsRuntimeRelevant(const FWorldPartitionActorDescInstance* InActorDescInstance) const
{
	return !IsMainWorldOnly() || InActorDescInstance->GetContainerInstance()->GetContainerID().IsMainContainer();
}

void FWorldPartitionActorDesc::UpdateActorToWorld(FTransform* InParentTransform)
{
	// If set, use provided parent transform, else get it from ParentActor
	FTransform ParentTransform = InParentTransform ? *InParentTransform : FTransform::Identity;
	if (ParentActor.IsValid() && !InParentTransform)
	{
		if (FWorldPartitionActorDesc* ParentActorDesc = Container->GetActorDesc(ParentActor))
		{
			ParentTransform = ParentActorDesc->GetActorTransform();
		}
		else
		{
			// This can happen if the parent actor is set, but its actor descriptor has not been registered yet.
			// In that case we can skip this update, since we don't have the parent transform. This actor will be updated
			// during PropagateActorToWorldUpdate, after the parent actor descriptor is registered.
			return;
		}
	}

	if (bHasValidRelativeBounds == false)
	{
		ActorTransformRelative = ActorTransform * ParentTransform.Inverse();

		RuntimeBoundsRelative = FWorldPartitionRelativeBounds(RuntimeBounds).InverseTransformBy(ActorTransform);
		EditorBoundsRelative = FWorldPartitionRelativeBounds(EditorBounds).InverseTransformBy(ActorTransform);

		bHasValidRelativeBounds = true;
	}
	else
	{
		ActorTransform = ActorTransformRelative * ParentTransform;

		RuntimeBounds = RuntimeBoundsRelative.TransformBy(ActorTransform).ToAABB();
		EditorBounds = EditorBoundsRelative.TransformBy(ActorTransform).ToAABB();
	}
}

void FWorldPartitionActorDesc::FixupStreamingBounds()
{
	const bool bRuntimeBoundsContainsNaN = RuntimeBounds.IsValid && RuntimeBounds.ContainsNaN();
	const bool bEditorBoundsContainsNaN = EditorBounds.IsValid && EditorBounds.ContainsNaN();

	if (bRuntimeBoundsContainsNaN != bEditorBoundsContainsNaN)
	{
		if (bRuntimeBoundsContainsNaN)
		{
			UE_LOGF(LogWorldPartition, Log, "Invalid runtime bounds for actor descriptor '%ls', overwriting with editor bounds.", *ToString(EToStringMode::Compact));
			RuntimeBounds = EditorBounds;			
		}
		else
		{
			UE_LOGF(LogWorldPartition, Log, "Invalid editor bounds for actor descriptor '%ls', overwriting with runtime bounds.", *ToString(EToStringMode::Compact));
			EditorBounds = RuntimeBounds;			
		}
	}
	else if (bRuntimeBoundsContainsNaN && bEditorBoundsContainsNaN)
	{
		UE_LOGF(LogWorldPartition, Log, "Invalid streaming bounds for actor descriptor '%ls'", *ToString(EToStringMode::Compact));
		RuntimeBounds.Init();		
		EditorBounds.Init();
	}
}

FWorldPartitionActorDescStringBuilder::FWorldPartitionActorDescStringBuilder(FWorldPartitionActorDesc::EToStringMode InMode)
	: Mode(InMode)
	, LineStart((Mode == FWorldPartitionActorDesc::EToStringMode::ForDiff) ? TEXT('\t') : TEXT(' '))
	, LineEnd((Mode == FWorldPartitionActorDesc::EToStringMode::ForDiff) ? LINE_TERMINATOR : nullptr)
{
}

void FWorldPartitionActorDescStringBuilder::AppendFromString(const TCHAR* Name, const FString& Value, bool bUseLineStart)
{
	AppendPair(Name, *Value, bUseLineStart);
}

void FWorldPartitionActorDescStringBuilder::AppendFromBool(const TCHAR* Name, bool bValue, bool bUseLineStart)
{
	AppendPair(Name, *LexToString(bValue), bUseLineStart);
}

void FWorldPartitionActorDescStringBuilder::AppendPair(const TCHAR* Name, const TCHAR* Value, bool bUseLineStart)
{
	if (bUseLineStart)
	{
		AppendChar(LineStart);
	}
	Append(Name);
	AppendChar(TEXT(':'));
	Append(Value);

	if (LineEnd)
	{
		Append(LineEnd);
	}
}

#undef LOCTEXT_NAMESPACE

#endif
