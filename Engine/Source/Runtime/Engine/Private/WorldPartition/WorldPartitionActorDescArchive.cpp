// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDescArchive.h"
#include "WorldPartition/WorldPartitionClassDescRegistry.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "Misc/RedirectCollector.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

FActorDescArchive::FActorDescArchive(FArchive& InArchive, FWorldPartitionActorDesc* InActorDesc, const FWorldPartitionActorDesc* InBaseActorDesc)
	: FArchiveProxy(InArchive)
	, ActorDesc(InActorDesc)
	, ComponentDesc(nullptr)
	, BaseDesc(InBaseActorDesc)
	, BaseComponentDesc(nullptr)
	, BaseDescSizeof(0)
	, BaseComponentDescSizeof(0)
	, bIsMissingBaseDesc(false)
{
	check(InArchive.IsPersistent());

	SetIsPersistent(true);
	SetIsLoading(InArchive.IsLoading());
}

void FActorDescArchive::SetComponentDesc(FWorldPartitionComponentDesc* InComponentDesc)
{
	if (InComponentDesc)
	{
		check(!ComponentDesc);
		check(!BaseComponentDesc);
		check(!BaseComponentDescSizeof);

		ComponentDesc = InComponentDesc;
		BaseComponentDesc = nullptr;

		for (const TUniquePtr<FWorldPartitionComponentDesc>& TargetBaseComponentDesc : BaseDesc->ComponentDescs)
		{
			if (TargetBaseComponentDesc->ComponentName == ComponentDesc->ComponentName)
			{
				BaseComponentDesc = TargetBaseComponentDesc.Get();
				BaseComponentDescSizeof = BaseComponentDesc->GetSizeOf();
				break;
			}
		}
	}
	else
	{
		check(ComponentDesc);

		ComponentDesc = nullptr;
		BaseComponentDesc = nullptr;
		BaseComponentDescSizeof = 0;
	}
}

void FActorDescArchive::Init(const FTopLevelAssetPath InClassPath)
{
	UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::WorldPartitionActorClassDescSerialize)
	{
		*this << ActorDesc->bIsDefaultActorDesc;
	}

	if (CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::WorldPartitionActorDescNativeBaseClassSerialization)
	{
		if (CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescActorAndClassPaths)
		{
			FName BaseClassPathName;
			*this << BaseClassPathName;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ActorDesc->BaseClass = FAssetData::TryConvertShortClassNameToPathName(BaseClassPathName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			*this << ActorDesc->BaseClass;
		}
	}

	if (CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::WorldPartitionActorDescActorAndClassPaths)
	{
		FName NativeClassPathName;
		*this << NativeClassPathName;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActorDesc->NativeClass = FAssetData::TryConvertShortClassNameToPathName(NativeClassPathName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		*this << ActorDesc->NativeClass;
	}

	if (IsLoading())
	{
		auto TryRedirectClass = [](FTopLevelAssetPath& InOutClassPath, bool bNativeClass)
		{
			if (InOutClassPath.IsValid())
			{
				FCoreRedirectObjectName ClassRedirect(InOutClassPath);
				FCoreRedirectObjectName RedirectedClassRedirect = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, ClassRedirect);

				if (ClassRedirect != RedirectedClassRedirect)
				{
					InOutClassPath = FTopLevelAssetPath(RedirectedClassRedirect.ToString());
				}

				if (!bNativeClass)
				{
					FSoftObjectPath RedirectedClassPath(InOutClassPath);
					UAssetRegistryHelpers::FixupRedirectedAssetPath(RedirectedClassPath);
					InOutClassPath = RedirectedClassPath.GetAssetPath();
				}
			}
		};

		TryRedirectClass(ActorDesc->NativeClass, true);
		TryRedirectClass(ActorDesc->BaseClass, false);
	}

	// Get the class descriptor to do delta serialization if no base desc was provided
	if (!BaseDesc)
	{
		FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();
		const FTopLevelAssetPath ClassPath = InClassPath.IsValid() ? InClassPath : (ActorDesc->BaseClass.IsValid() ? ActorDesc->BaseClass : ActorDesc->NativeClass);
		BaseDesc = (ActorDesc->bIsDefaultActorDesc && !InClassPath.IsValid()) ? ClassDescRegistry.GetClassDescDefaultForClass(ClassPath) : ClassDescRegistry.GetClassDescDefaultForActor(ClassPath);

		if (!BaseDesc)
		{
			if (IsLoading())
			{
				bIsMissingBaseDesc = true;

				BaseDesc = ClassDescRegistry.GetClassDescDefault(FTopLevelAssetPath(TEXT("/Script/Engine.Actor")));
				check(BaseDesc);

				UE_LOGF(LogWorldPartition, Log, "Can't find class descriptor '%ls' for loading '%ls', using '%ls'", *ClassPath.ToString(), *ActorDesc->GetActorSoftPath().ToString(), *BaseDesc->GetActorSoftPath().ToString());
			}
			else
			{
				UE_LOGF(LogWorldPartition, Log, "Can't find class descriptor '%ls' for saving '%ls'", *ClassPath.ToString(), *ActorDesc->GetActorSoftPath().ToString());
			}
		}
	}

	BaseDescSizeof = BaseDesc ? BaseDesc->GetSizeOf() : 0;
}

FArchive& FActorDescArchive::operator<<(FTopLevelAssetPath& Value)
{
	((FArchive&)*this) << Value;

	return *this;
};

FArchive& FActorDescArchive::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePathWithoutFixup(*this);

	if (IsLoading())
	{
		UAssetRegistryHelpers::FixupRedirectedAssetPath(Value);
	}

	return *this;
}

FArchive& FActorDescArchivePatcher::operator<<(FName& Value)
{
	{
		TGuardValue<bool> GuardIsPatching(bIsPatching, true);
		FActorDescArchive::operator<<(Value);
		AssetDataPatcher->DoPatch(Value);
	}

    // Only write out values if we aren't already patching since this function can be called
	// from other patching functions which will perform the final write of the patched values
	if (!bIsPatching)
	{
		OutAr << Value;
	}
	return *this;
}

FArchive& FActorDescArchivePatcher::operator<<(FSoftObjectPath& Value)
{
	{
		TGuardValue<bool> GuardIsPatching(bIsPatching, true);
		FActorDescArchive::operator<<(Value);
		AssetDataPatcher->DoPatch(Value);
	}

    // Only write out values if we aren't already patching since this function can be called
	// from other patching functions which will perform the final write of the patched values
	if (!bIsPatching)
	{
		Value.SerializePathWithoutFixup(OutAr);
	}
	return *this;
}

FArchive& FActorDescArchivePatcher::operator<<(FTopLevelAssetPath& Value)
{
	{
		TGuardValue<bool> GuardIsPatching(bIsPatching, true);
		FActorDescArchive::operator<<(Value);
		AssetDataPatcher->DoPatch(Value);
	}

	// Only write out values if we aren't already patching since this function can be called
	// from other patching functions which will perform the final write of the patched values
	if (!bIsPatching)
	{
		OutAr << Value;
	}
	return *this;
}

void FActorDescArchivePatcher::Serialize(void* V, int64 Length)
{
	FActorDescArchive::Serialize(V, Length);

	if (!bIsPatching)
	{
		OutAr.Serialize(V, Length);
	}
}
#endif
