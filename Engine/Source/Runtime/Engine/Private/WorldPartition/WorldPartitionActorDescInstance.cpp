// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionActorDescInstance.h"

#if WITH_EDITOR

#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/LevelInstance/LevelInstancePropertyOverrideContainer.h"
#include "UObject/MetaData.h"

FWorldPartitionActorDescInstance::FActorDescUpdatedEvent FWorldPartitionActorDescInstance::OnActorDescUpdated;

#define LOCTEXT_NAMESPACE "FWorldPartitionActorDescInstance"

FWorldPartitionActorDescInstance::FWorldPartitionActorDescInstance()
	: ContainerInstance(nullptr)
	, SoftRefCount(0)
	, HardRefCount(0)
	, bIsForcedNonSpatiallyLoaded(false)
	, bIsRegisteringOrUnregistering(false)
	, UnloadedReason(nullptr)
	, AsyncLoadID(INDEX_NONE)
	, ActorDesc(nullptr)
	, ChildContainerInstance(nullptr)
{}

FWorldPartitionActorDescInstance::FWorldPartitionActorDescInstance(UActorDescContainerInstance* InContainerInstance, FWorldPartitionActorDesc* InActorDesc)
	: FWorldPartitionActorDescInstance()
{
	check(InContainerInstance);
	ContainerInstance = InContainerInstance;

	check(InActorDesc);
	ActorDesc = InActorDesc;
}

void FWorldPartitionActorDescInstance::UpdateActorDesc(FWorldPartitionActorDesc* InActorDesc)
{
	ActorDesc = InActorDesc;
	
	OnActorDescUpdated.Broadcast(this);
}

bool FWorldPartitionActorDescInstance::IsLoaded(bool bEvenIfPendingKill) const
{
	if (AsyncLoadID != INDEX_NONE)
	{
		return false;
	}

	if (ActorPtr.IsExplicitlyNull() || ActorPtr.IsStale())
	{
		ActorPtr = FindObject<AActor>(nullptr, *GetActorSoftPath().ToString());
	}

	return ActorPtr.IsValid(bEvenIfPendingKill);
}

AActor* FWorldPartitionActorDescInstance::GetActor(bool bEvenIfPendingKill, bool bEvenIfUnreachable) const
{
	FlushAsyncLoad();

	if (ActorPtr.IsExplicitlyNull() || ActorPtr.IsStale())
	{
		ActorPtr = FindObject<AActor>(nullptr, *GetActorSoftPath().ToString());
	}

	return bEvenIfUnreachable ? ActorPtr.GetEvenIfUnreachable() : ActorPtr.Get(bEvenIfPendingKill);
}

TWeakObjectPtr<AActor>* FWorldPartitionActorDescInstance::GetActorPtr(bool bEvenIfPendingKill, bool bEvenIfUnreachable) const
{
	return GetActor(bEvenIfPendingKill, bEvenIfUnreachable) ? &ActorPtr : nullptr;
}

FSoftObjectPath FWorldPartitionActorDescInstance::GetActorSoftPath() const
{
	return ActorPath.IsSet() ? ActorPath.GetValue() : ActorDesc->GetActorSoftPath();
}

FName FWorldPartitionActorDescInstance::GetActorName() const
{
	return ActorDesc->GetActorName();
}

bool FWorldPartitionActorDescInstance::IsValid() const
{
	return !!ActorDesc;
}

bool FWorldPartitionActorDescInstance::IsEditorRelevant() const
{
	return ActorDesc->IsEditorRelevant(this);
}

bool FWorldPartitionActorDescInstance::IsRuntimeRelevant() const
{
	return ActorDesc->IsRuntimeRelevant(this);
}

FBox FWorldPartitionActorDescInstance::GetEditorBounds() const
{
	return GetLocalEditorBounds().TransformBy(GetContainerInstance()->GetTransform());
}

FBox FWorldPartitionActorDescInstance::GetLocalEditorBounds(bool bInForceComputeOnChildContainerInstance) const
{ 
	auto BuildActorContainerPath = [](const UActorDescContainerInstance* ContainerInstance)
	{
		FActorContainerPath ContainerPath;
		while (ContainerInstance)
		{
			// Don't include top container
			if (ContainerInstance->GetContainerActorGuid().IsValid())
			{
				ContainerPath.ContainerGuids.Add(ContainerInstance->GetContainerActorGuid());
			}
			ContainerInstance = ContainerInstance->GetParentContainerInstance();
		}
		Algo::Reverse(ContainerPath.ContainerGuids);
		return ContainerPath;
	};

	auto GetTopMostContainerInstance = [](const UActorDescContainerInstance* ContainerInstance)
	{
		const UActorDescContainerInstance* CurrentContainerInstance = ContainerInstance;
		while (CurrentContainerInstance && CurrentContainerInstance->GetParentContainerInstance())
		{
			CurrentContainerInstance = CurrentContainerInstance->GetParentContainerInstance();
		}
		return CurrentContainerInstance;
	};

	auto FindChildContainerInstance = [](const UActorDescContainerInstance* ContainerInstance, const FActorContainerPath ContainerPath) -> const UActorDescContainerInstance*
	{
		const UActorDescContainerInstance* CurrentContainerInstance = ContainerInstance;
		for (const FGuid& Guid : ContainerPath.ContainerGuids)
		{
			if (const TObjectPtr<UActorDescContainerInstance>* FoundContainerInstance = CurrentContainerInstance->GetChildContainerInstances().Find(Guid))
			{
				CurrentContainerInstance = *FoundContainerInstance;
			}
			else
			{
				return nullptr;
			}
		}
		return CurrentContainerInstance;
	};

	if (IsChildContainerInstance())
	{
		// Use cached value if valid
		if (ChildContainerLocalEditorBoundsCache.IsSet())
		{
			return ChildContainerLocalEditorBoundsCache.GetValue();
		}

		// When container uses property overrides, compute bounds dynamically and cache the result.
		const UActorDescContainerInstance* ChildContainer = GetChildContainerInstance();
		const bool bForceComputeOnChildContainerInstance = bInForceComputeOnChildContainerInstance || (ChildContainer && ChildContainer->GetContainer()->IsA<ULevelInstancePropertyOverrideContainer>());
		if (ChildContainer && bForceComputeOnChildContainerInstance)
		{
			FBox Bounds(ForceInitToZero);
			for (UActorDescContainerInstance::TConstIterator<AActor> ActorDescIt(ChildContainer); ActorDescIt; ++ActorDescIt)
			{
				if (!ActorDescIt->IsMainWorldOnly())
				{
					Bounds += ActorDescIt->GetLocalEditorBounds(bForceComputeOnChildContainerInstance);
				}
			}
			Bounds = Bounds.TransformBy(ActorDesc->GetActorTransformRelative());
			ChildContainerLocalEditorBoundsCache = Bounds;
			return Bounds;
		}

		// ChildContainer can be null if we loaded an instance of a world partition (most likely a level instance).
		// In which case, the hierarchy of container instances is not initialized (to avoid massive duplication of container instances).
		if (!ChildContainer)
		{
			// We can instead find the corresponding container instance in the main world partition using the container path.
			// We are able to find the top most container instance only because this container instance has its parent 
			// container instance set (which is part of the main world partition hierarchy).
			const FActorContainerPath ContainerPath = BuildActorContainerPath(GetContainerInstance());
			const UActorDescContainerInstance* TopMostContainerInstance = GetTopMostContainerInstance(GetContainerInstance());
			const UActorDescContainerInstance* MatchingContainerInstance = FindChildContainerInstance(TopMostContainerInstance, ContainerPath);
			// It's possible that we can't find a matching container instance when a newly created (unsaved) Level Instance is added to the world
			if (MatchingContainerInstance && (MatchingContainerInstance != GetContainerInstance()))
			{
				FWorldPartitionActorDescInstance* MatchingActorDescInstance = MatchingContainerInstance->GetActorDescInstance(GetGuid());
				if (ensure(MatchingActorDescInstance))
				{
					return MatchingActorDescInstance->GetLocalEditorBounds();
				}
			}
		}
	}

	// Use ActorDesc editor bounds
	return ActorDesc->GetEditorBounds();
}

void FWorldPartitionActorDescInstance::OnPreUpdate()
{
	// Invalidate cached bounds
	if (IsChildContainerInstance())
	{
		UActorDescContainerInstance* CurrentContainerInstance = GetContainerInstance();
		while (CurrentContainerInstance)
		{
			for (UActorDescContainerInstance::TIterator<> It(CurrentContainerInstance); It; ++It)
			{
				It->InvalidateCachedBounds();
			}
			// Recurse to parent as parent bounds are affected by children bounds
			CurrentContainerInstance = const_cast<UActorDescContainerInstance*>(CurrentContainerInstance->GetParentContainerInstance());
		}
	}
}

void FWorldPartitionActorDescInstance::InvalidateCachedBounds()
{
	ChildContainerLocalEditorBoundsCache.Reset();
}

FBox FWorldPartitionActorDescInstance::GetRuntimeBounds() const
{
	// Override is not supported for streamed level instances.
	// No need to compute dynamically runtime bounds per instance.
	return ActorDesc->GetRuntimeBounds().TransformBy(GetContainerInstance()->GetTransform());
}

bool FWorldPartitionActorDescInstance::StartAsyncLoad()
{
	UnloadedReason = nullptr;

	if (ActorPtr.IsExplicitlyNull() || ActorPtr.IsStale())
	{
		// First, try to find the existing actor which could have been loaded by another actor (through standard serialization)
		ActorPtr = FindObject<AActor>(nullptr, * GetActorSoftPath().ToString());
	}

	// Then, if the actor isn't loaded, load it
	if (ActorPtr.IsExplicitlyNull())
	{
		const FName ActorPackage = GetActorPackage();
		const FLinkerInstancingContext* InstancingContext = GetContainerInstance()->GetInstancingContext();		
		const FName PackageName = InstancingContext ? InstancingContext->RemapPackage(ActorPackage) : ActorPackage;
		const FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(ActorPackage);

		AsyncLoadID = LoadPackageAsync(PackagePath, PackageName, FLoadPackageAsyncDelegate::CreateLambda([this, ActorPackage](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
		{
			if (AsyncLoadID != INDEX_NONE)
			{
				AsyncLoadFinished(ActorPackage, PackageName, Package, Result == EAsyncLoadingResult::Succeeded);
				AsyncLoadID = INDEX_NONE;
			}
		})
		, PKG_None, INDEX_NONE, 0, InstancingContext);
	}

	return (AsyncLoadID != INDEX_NONE) || ActorPtr.IsValid(false);
}

void FWorldPartitionActorDescInstance::FlushAsyncLoad() const
{
	if (AsyncLoadID != INDEX_NONE)
	{
		// Instead relying on AsyncLoading to call the callback during a flush, we'll do it ourselves here explicitly
		// This is because during a callstack where we are already async loading, we may not get completion callbacks called before returning from this flush.
		// Setting AsyncLoadID to INDEX_NONE before flushing so that we do not end up calling AsyncLoadFinished twice in cases where the completion callback is called during the flush
		int32 IDToFlush = AsyncLoadID;
		AsyncLoadID = INDEX_NONE;
		FlushAsyncLoading(IDToFlush);

		// Now call AsyncLoadFinished to assign ActorPtr
		const FName ActorPackage = GetActorPackage();
		const FLinkerInstancingContext* InstancingContext = GetContainerInstance()->GetInstancingContext();
		const FName PackageName = InstancingContext ? InstancingContext->RemapPackage(ActorPackage) : ActorPackage;
		UPackage* Package = FindPackage(nullptr, *PackageName.ToString());
		AsyncLoadFinished(ActorPackage, PackageName, Package, Package != nullptr);
	}
}

void FWorldPartitionActorDescInstance::AsyncLoadFinished(const FName& ActorPackage, const FName& PackageName, UPackage* Package, bool bSuccessful) const
{
	static FText FailedToLoad(LOCTEXT("FailedToLoadReason", "Failed to load"));

	if (!bSuccessful || !Package)
	{
		UE_LOGF(LogWorldPartition, Warning, "Can't load actor guid `%ls` ('%ls') from package '%ls'", *GetGuid().ToString(), *GetActorNameString(), *ActorPackage.ToString());
		UnloadedReason = &FailedToLoad;
		return;
	}

	ActorPtr = FindObject<AActor>(nullptr, *GetActorSoftPath().ToString());

	if (!ActorPtr.IsValid())
	{
		UE_LOGF(LogWorldPartition, Warning, "Can't find actor guid `%ls` ('%ls') in package '%ls'", *GetGuid().ToString(), *GetActorNameString(), *ActorPackage.ToString());
		UnloadedReason = &FailedToLoad;
		return;
	}

	check(ActorPtr->GetPackage() == Package);
}

void FWorldPartitionActorDescInstance::MarkUnload()
{
	if (AActor* Actor = GetActor())
	{
		// At this point, it can happen that an actor isn't in an external package:
		//
		// PIE travel: 
		//		in this case, actors referenced by the world package (an example is the level script) will be duplicated as part of the PIE world duplication and will end up
		//		not being using an external package, which is fine because in that case they are considered as always loaded.
		//
		// FWorldPartitionCookPackageSplitter:
		//		should mark each FWorldPartitionActorDesc as moved, and the splitter should take responsbility for calling ClearFlags on every object in 
		//		the package when it does the move.

		ActorPtr = nullptr;
	}
}

void FWorldPartitionActorDescInstance::Invalidate()
{
	check(!ChildContainerInstance);
	ContainerInstance = nullptr;
}

const FDataLayerInstanceNames& FWorldPartitionActorDescInstance::GetDataLayerInstanceNames() const
{
	static FDataLayerInstanceNames EmptyDataLayers;
	if (ensure(HasResolvedDataLayerInstanceNames()))
	{
		return ResolvedDataLayerInstanceNames.GetValue();
	}
	return EmptyDataLayers;
}

const FText& FWorldPartitionActorDescInstance::GetUnloadedReason() const
{
	static FText Unloaded(LOCTEXT("UnloadedReason", "Unloaded"));
	return UnloadedReason ? *UnloadedReason : Unloaded;
}

const FString& FWorldPartitionActorDescInstance::GetActorNameString() const
{
	return ActorDesc->GetActorNameString();
}

const FString& FWorldPartitionActorDescInstance::GetActorLabelString() const
{
	return ActorDesc->GetActorLabelString();
}

const FString& FWorldPartitionActorDescInstance::GetDisplayClassNameString() const
{
	return ActorDesc->GetDisplayClassNameString();
}

FString FWorldPartitionActorDescInstance::ToString(FWorldPartitionActorDesc::EToStringMode Mode) const
{
	return ActorDesc->ToString(Mode);
}

void FWorldPartitionActorDescInstance::RegisterChildContainerInstance()
{
	check(IsChildContainerInstance());
	check(!ChildContainerInstance);

	ChildContainerInstance = ActorDesc->CreateChildContainerInstance(this);
	if (ChildContainerInstance)
	{
		ContainerInstance->OnRegisterChildContainerInstance(GetGuid(), ChildContainerInstance);
	}
}

void FWorldPartitionActorDescInstance::UnregisterChildContainerInstance()
{
	if (ChildContainerInstance)
	{
		ContainerInstance->OnUnregisterChildContainerInstance(GetGuid());
		ChildContainerInstance->Uninitialize();
		ChildContainerInstance = nullptr;
	}
}

void FWorldPartitionActorDescInstance::UpdateChildContainerInstance()
{
	// Create before unregistering so that we benefit from shared containers (use GetActorDesc->IsChildContainerInstance as we want to know if our updated desc should be a Container instance or not)
	// ChildContainerInstance member might be non null and we don't want IsChildContainerInstance() to return true in this case if the ActorDesc isn't a Container anymore
	UActorDescContainerInstance* NewChildContainerInstance = ActorDesc->IsChildContainerInstance() ? ActorDesc->CreateChildContainerInstance(this) : nullptr;
	
	// Unregister previous
	if (ChildContainerInstance)
	{
		ContainerInstance->OnUnregisterChildContainerInstance(GetGuid());
		ChildContainerInstance->Uninitialize();
		ChildContainerInstance = nullptr;
	}
	
	// Register new if it is valid
	if (NewChildContainerInstance)
	{
		ChildContainerInstance = NewChildContainerInstance;
		ContainerInstance->OnRegisterChildContainerInstance(GetGuid(), ChildContainerInstance);
	}
}

#undef LOCTEXT_NAMESPACE

#endif