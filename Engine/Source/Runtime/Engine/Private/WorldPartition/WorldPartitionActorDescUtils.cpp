// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/CoreRedirects.h"
#include "Misc/Base64.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

static FName NAME_ActorMetaDataClass("ActorMetaDataClass");
static FName NAME_ActorMetaData("ActorMetaData");

namespace FWorldPartitionActorDescUtilsPrivate
{
	static FString ResolveClassRedirector(const FString& InClassName)
	{
		FString ClassName;
		FString ClassPackageName;
		if (!InClassName.Split(TEXT("."), &ClassPackageName, &ClassName))
		{
			ClassName = *InClassName;
		}

		// Look for a class redirectors
		const FCoreRedirectObjectName OldClassName = FCoreRedirectObjectName(*ClassName, NAME_None, *ClassPackageName);
		const FCoreRedirectObjectName NewClassName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldClassName);

		return NewClassName.ToString();
	}
}

FWorldPartitionActorDescUtils::FActorDescInitParams::FActorDescInitParams(const AActor* InActor)
{
	// Avoid an assert when calling StaticFindObject during save to retrieve the actor's class.
	// Since we are only looking for a native class, the call to StaticFindObject is legit.
	TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

	TUniquePtr<FWorldPartitionActorDesc> ActorDesc(InActor->CreateActorDesc());

	PathName = *InActor->GetPathName();
	BaseClassName = !InActor->GetClass()->IsNative() ? FName(*InActor->GetClass()->GetPathName()) : NAME_None;
	NativeClassName = *(GetParentNativeClass(InActor->GetClass()))->GetPathName();
	AssetData = FWorldPartitionActorDescUtils::GetAssetDataFromActorDescriptor(ActorDesc);
}

FWorldPartitionActorDescUtils::FActorDescInitParams::FActorDescInitParams(const FString& InString)
{
	TArray<FString> ParsedAssetData;
	if (InString.ParseIntoArray(ParsedAssetData, TEXT(",")))
	{
		PathName = *ParsedAssetData[0];
		BaseClassName = ParsedAssetData.IsValidIndex(3) ? FName(*ParsedAssetData[3]) : NAME_None;
		NativeClassName = *ParsedAssetData[1];
		AssetData = *ParsedAssetData[2];
	}
}

FString FWorldPartitionActorDescUtils::FActorDescInitParams::ToString() const
{
	return FString::Printf(TEXT("%s,%s,%s,%s"), *PathName.ToString(), *NativeClassName.ToString(), *AssetData, *BaseClassName.ToString());
}

FName FWorldPartitionActorDescUtils::ActorMetaDataClassTagName()
{
	return NAME_ActorMetaDataClass;
}

FName FWorldPartitionActorDescUtils::ActorMetaDataTagName()
{
	return NAME_ActorMetaData;
}

bool FWorldPartitionActorDescUtils::IsValidActorDescriptorFromAssetData(const FAssetData& InAssetData)
{
	return InAssetData.FindTag(NAME_ActorMetaDataClass) && InAssetData.FindTag(NAME_ActorMetaData);
}

UClass* FWorldPartitionActorDescUtils::GetActorNativeClassFromAssetData(const FAssetData& InAssetData)
{
	FString ActorMetaDataClass;
	if (InAssetData.GetTagValue(NAME_ActorMetaDataClass, ActorMetaDataClass))
	{
		return GetActorNativeClassFromString(ActorMetaDataClass);
	}
	return nullptr;
}

UClass* FWorldPartitionActorDescUtils::GetActorNativeClassFromString(const FString& InClassPath)
{
	// Avoid an assert when calling StaticFindObject during save to retrieve the actor's class.
	// Since we are only looking for a native class, the call to StaticFindObject is legit.
	TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

	// Look for a class redirectors
	const FString ActorNativeClassName = FWorldPartitionActorDescUtilsPrivate::ResolveClassRedirector(InClassPath);

	// Handle deprecated short class names
	const FTopLevelAssetPath ClassPath = FAssetData::TryConvertShortClassNameToPathName(*ActorNativeClassName, ELogVerbosity::Log);

	// Lookup the native class
	return UClass::TryFindTypeSlow<UClass>(ClassPath.ToString(), EFindFirstObjectOptions::ExactClass);
}

TUniquePtr<FWorldPartitionActorDesc> FWorldPartitionActorDescUtils::GetActorDescriptorFromInitParams(const FActorDescInitParams& InActorDescInitParams, FName InPackageName)
{
	FWorldPartitionActorDescInitData ActorDescInitData = FWorldPartitionActorDescInitData()
		.SetNativeClass(GetNativeClassFromClassName(InActorDescInitParams.NativeClassName))
		.SetPackageName(InPackageName)
		.SetActorPath(InActorDescInitParams.PathName.ToString());

	verify(FBase64::Decode(InActorDescInitParams.AssetData, ActorDescInitData.GetSerializedData()));

	TUniquePtr<FWorldPartitionActorDesc> NewActorDesc(AActor::StaticCreateClassActorDesc(ActorDescInitData.NativeClass ? ActorDescInitData.NativeClass : AActor::StaticClass()));

	NewActorDesc->Init(ActorDescInitData);
			
	if (!ActorDescInitData.NativeClass)
	{
		UE_LOGF(LogWorldPartition, Warning, "Invalid class for actor guid `%ls` ('%ls') from package '%ls'", *NewActorDesc->GetGuid().ToString(), *NewActorDesc->GetActorName().ToString(), *NewActorDesc->GetActorPackage().ToString());
		NewActorDesc->NativeClass.Reset();
	}

	return NewActorDesc;
}

TUniquePtr<FWorldPartitionActorDesc> FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(const FAssetData& InAssetData)
{
	if (IsValidActorDescriptorFromAssetData(InAssetData))
	{
		FString ActorMetaDataStr;
		if (InAssetData.GetTagValue(NAME_ActorMetaData, ActorMetaDataStr))
		{
			// Fixing here for now until following jira tasks are closed to unblock users that are renaming plugins:
			// @todo_ow: remove once https://jira.it.epicgames.com/browse/UE-168245 & https://jira.it.epicgames.com/browse/UE-144431 are closed
			FSoftObjectPath ActorPath = InAssetData.GetSoftObjectPath();
			ActorPath.FixupCoreRedirects();

			FActorDescInitParams ActorDescInitParams;
			ActorDescInitParams.PathName = *ActorPath.ToString();
			ActorDescInitParams.NativeClassName = *GetActorNativeClassFromAssetData(InAssetData)->GetPathName();
			ActorDescInitParams.AssetData = ActorMetaDataStr;
			return GetActorDescriptorFromInitParams(ActorDescInitParams, InAssetData.PackageName);
		}
	}

	return nullptr;
}

void FWorldPartitionActorDescUtils::AppendAssetDataTagsFromActor(const AActor* InActor, FAssetRegistryTagsContext Context)
{
	// Avoid an assert when calling StaticFindObject during save to retrieve the actor's class.
	// Since we are only looking for a native class, the call to StaticFindObject is legit.
	TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

	TUniquePtr<FWorldPartitionActorDesc> ActorDesc(InActor->CreateActorDesc());

	if (!InActor->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		// If the actor is not added to a world, we can't retrieve its bounding volume, so try to get the existing one
		if (!InActor->HasActorRegisteredAllComponents())
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.PackageNames.Add(InActor->GetPackage()->GetLoadedPath().GetPackageFName());

			TArray<FAssetData> Assets;
			AssetRegistry.GetAssets(Filter, Assets);

			if (Assets.Num() == 1)
			{
				if (TUniquePtr<FWorldPartitionActorDesc> NewActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(Assets[0]))
				{
					ActorDesc->TransferWorldData(NewActorDesc.Get());
				}
			}
		}
	}

	const FString ActorMetaDataClass = GetParentNativeClass(InActor->GetClass())->GetPathName();
	Context.AddTag(UObject::FAssetRegistryTag(NAME_ActorMetaDataClass, ActorMetaDataClass, UObject::FAssetRegistryTag::TT_Hidden));

	const FString ActorMetaData = GetAssetDataFromActorDescriptor(ActorDesc);
	Context.AddTag(UObject::FAssetRegistryTag(NAME_ActorMetaData, ActorMetaData, UObject::FAssetRegistryTag::TT_Hidden));
}

FString FWorldPartitionActorDescUtils::GetAssetDataFromActorDescriptor(TUniquePtr<FWorldPartitionActorDesc>& InActorDesc)
{
	TArray<uint8> SerializedData;
	InActorDesc->SerializeTo(SerializedData);
	return FBase64::Encode(SerializedData);
}

bool FWorldPartitionActorDescUtils::GetPatchedAssetDataFromAssetData(const FAssetData& InAssetData, FString& OutAssetData, FWorldPartitionAssetDataPatcher* InAssetDataPatcher)
{
	if (IsValidActorDescriptorFromAssetData(InAssetData))
	{
		FWorldPartitionActorDescInitData ActorDescInitData = FWorldPartitionActorDescInitData()
			.SetNativeClass(GetActorNativeClassFromAssetData(InAssetData))
			.SetPackageName(InAssetData.PackageName)
			.SetActorPath(InAssetData.GetSoftObjectPath());

		FString ActorMetaDataStr;
		verify(InAssetData.GetTagValue(NAME_ActorMetaData, ActorMetaDataStr));
		verify(FBase64::Decode(ActorMetaDataStr, ActorDescInitData.GetSerializedData()));

		TArray<uint8> PatchedData;
		FWorldPartitionActorDesc::Patch(ActorDescInitData, PatchedData, InAssetDataPatcher);
		OutAssetData = FBase64::Encode(PatchedData);

		return OutAssetData != ActorMetaDataStr;
	}

	return false;
}

void FWorldPartitionActorDescUtils::UpdateActorDescriptorFromActor(const AActor* InActor, TUniquePtr<FWorldPartitionActorDesc>& OutActorDesc)
{
	TUniquePtr<FWorldPartitionActorDesc> NewActorDesc(InActor->CreateActorDesc());
	UpdateActorDescriptorFromActorDescriptor(NewActorDesc, OutActorDesc);
}

void FWorldPartitionActorDescUtils::UpdateActorDescriptorFromActorDescriptor(TUniquePtr<FWorldPartitionActorDesc>& InActorDesc, TUniquePtr<FWorldPartitionActorDesc>& OutActorDesc)
{
	InActorDesc->TransferFrom(OutActorDesc.Get());
	InActorDesc->UpdateActorToWorld();
	OutActorDesc = MoveTemp(InActorDesc);
}

void FWorldPartitionActorDescUtils::ReplaceActorDescriptorPointerFromActor(const AActor* InOldActor, AActor* InNewActor, FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (InNewActor)
	{
		checkf(InOldActor->GetActorGuid() == InNewActor->GetActorGuid(), TEXT("Mismatching new actor GUID: old=%s new=%s"), *InOldActor->GetActorGuid().ToString(), *InNewActor->GetActorGuid().ToString());
		checkf(InNewActor->GetActorGuid() == InActorDescInstance->GetGuid(), TEXT("Mismatching desc actor GUID: desc=%s new=%s"), *InActorDescInstance->GetGuid().ToString(), *InNewActor->GetActorGuid().ToString());
	}

	if (InActorDescInstance->ActorPtr.IsValid())
	{
		checkf(InActorDescInstance->ActorPtr == InOldActor, TEXT("Mismatching old desc actor: desc=%s old=%s"), *InActorDescInstance->ActorPtr->GetActorNameOrLabel(), *InOldActor->GetActorNameOrLabel());
	}

	InActorDescInstance->ActorPtr = InNewActor;
}

UClass* FWorldPartitionActorDescUtils::GetNativeClassFromClassName(FName InClassName)
{
	// Avoid an assert when calling StaticFindObject during save to retrieve the class.
	// Since we are only looking for a native class, the call to StaticFindObject is legit.
	TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

	// Look for a class redirectors
	const FString NativeClassName = FWorldPartitionActorDescUtilsPrivate::ResolveClassRedirector(InClassName.ToString());
		
	// Handle deprecated short class names
	const FTopLevelAssetPath ClassPath = FAssetData::TryConvertShortClassNameToPathName(*NativeClassName, ELogVerbosity::Log);

	// Lookup the native class
	return UClass::TryFindTypeSlow<UClass>(ClassPath.ToString(), EFindFirstObjectOptions::ExactClass);
}

#endif