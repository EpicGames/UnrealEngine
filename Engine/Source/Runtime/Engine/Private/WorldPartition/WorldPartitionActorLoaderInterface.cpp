// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionActorLoaderInterface)

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Level.h"
#include "Containers/Ticker.h"
#include "Misc/ScopedSlowTask.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#endif

#define LOCTEXT_NAMESPACE "WorldPartition"

UWorldPartitionActorLoaderInterface::UWorldPartitionActorLoaderInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR
TArray<TSharedRef<IWorldPartitionActorLoaderInterface::FActorDescFilter>> IWorldPartitionActorLoaderInterface::ActorDescFilters;
IWorldPartitionActorLoaderInterface::FOnActorLoaderInterfaceRefreshState IWorldPartitionActorLoaderInterface::ActorLoaderInterfaceRefreshState;

void IWorldPartitionActorLoaderInterface::RegisterActorDescFilter(const TSharedRef<FActorDescFilter>& InActorDescFilter)
{
	ActorDescFilters.Add(InActorDescFilter);
	ActorDescFilters.Sort([](const TSharedRef<FActorDescFilter>& FilterA, const TSharedRef<FActorDescFilter>& FilterB) { return FilterA.Get().GetFilterPriority() > FilterB.Get().GetFilterPriority(); });
}

IWorldPartitionActorLoaderInterface::ILoaderAdapter::ILoaderAdapter(UWorld* InWorld)
	: World(InWorld)
	, bLoaded(false)
	, bUserCreated(false)
{
	check(!World->IsGameWorld());
}

IWorldPartitionActorLoaderInterface::ILoaderAdapter::~ILoaderAdapter()
{
	UnregisterDelegates();

	FWorldPartitionLoadingContext::FDeferred LoadingContext;
	ContainerActorReferences.Empty();
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::Load()
{
	if (!bLoaded)
	{
		bLoaded = true;
		RefreshLoadedState();
		RegisterDelegates();

		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			WorldPartition->OnLoaderAdapterStateChanged(this);
		}
	}
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::Unload()
{
	if (bLoaded && !IsEngineExitRequested())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			UnregisterDelegates();

			FScopedSlowTask SlowTask(1, LOCTEXT("UpdatingLoading", "Updating loading..."));
			SlowTask.MakeDialogDelayed(1.0f);

			int32 NumUnloads = 0;
			{
				FWorldPartitionLoadingContext::FDeferred LoadingContext;
				ContainerActorReferences.Empty();
				NumUnloads = LoadingContext.GetNumUnregistrations();
			}

			SlowTask.EnterProgressFrame(1);

			bLoaded = false;

			WorldPartition->OnLoaderAdapterStateChanged(this);

			if (NumUnloads)
			{
				PostLoadedStateChanged(0, NumUnloads, true);
			}
		}
	}
}

bool IWorldPartitionActorLoaderInterface::ILoaderAdapter::IsLoaded() const
{
	return bLoaded;
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::RegisterDelegates()
{
	ActorLoaderInterfaceRefreshState.AddRaw(this, &IWorldPartitionActorLoaderInterface::ILoaderAdapter::OnRefreshLoadedState);

	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		WorldPartition->OnActorDescContainerInstanceRegistered.AddRaw(this, &IWorldPartitionActorLoaderInterface::ILoaderAdapter::ILoaderAdapter::OnActorDescContainerInstanceInitialize);
		WorldPartition->OnActorDescContainerInstanceUnregistered.AddRaw(this, &IWorldPartitionActorLoaderInterface::ILoaderAdapter::ILoaderAdapter::OnActorDescContainerInstanceUninitialize);
	}
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::UnregisterDelegates()
{
	ActorLoaderInterfaceRefreshState.RemoveAll(this);

	if (UWorldPartition* WorldPartition = World->GetWorldPartition())
	{
		WorldPartition->OnActorDescContainerInstanceRegistered.RemoveAll(this);
		WorldPartition->OnActorDescContainerInstanceUnregistered.RemoveAll(this);
	}
}

bool IWorldPartitionActorLoaderInterface::ILoaderAdapter::PassActorDescFilter(const FWorldPartitionHandle& Actor) const
{
	return Actor->IsEditorRelevant();
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::RefreshLoadedState()
{
	if (bLoaded)
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			TArray<FWorldPartitionHandle> ActorsToLoad;
			TArray<FWorldPartitionHandle> ActorsToUnload;			
			ForEachActor([this, &ActorsToLoad, &ActorsToUnload](const FWorldPartitionHandle& ActorHandle)
			{
				const FActorReferenceMap* ActorReferences = GetContainerReferencesConst(ActorHandle->GetContainerInstance());

				if (ShouldActorBeLoaded(ActorHandle))
				{
					if (!ActorReferences || !ActorReferences->Contains(ActorHandle->GetGuid()))
					{
						ActorsToLoad.Add(ActorHandle);
					}
				}
				else if (ActorReferences && ActorReferences->Contains(ActorHandle->GetGuid()))
				{
					ActorsToUnload.Add(ActorHandle);
				}
			});

			if (ActorsToLoad.Num() || ActorsToUnload.Num())
			{
				int32 NumLoads = 0;
				int32 NumUnloads = 0;
				bool bClearTransactions = false;
				{
					FWorldPartitionLoadingContext::FDeferred LoadingContext;

					if (ActorsToLoad.Num())
					{
						FScopedSlowTask SlowTask(ActorsToLoad.Num(), LOCTEXT("ActorsLoading", "Loading actors..."));
						SlowTask.MakeDialogDelayed(1.0f);

						for (FWorldPartitionHandle& ActorToLoad : ActorsToLoad)
						{
							AddReferenceToActor(ActorToLoad);
							SlowTask.EnterProgressFrame(1);
						}
					}

					if (ActorsToUnload.Num())
					{
						FScopedSlowTask SlowTask(ActorsToUnload.Num(), LOCTEXT("ActorsUnoading", "Unloading actors..."));
						SlowTask.MakeDialogDelayed(1.0f);

						for (FWorldPartitionHandle& ActorToUnload : ActorsToUnload)
						{
							RemoveReferenceToActor(ActorToUnload);
							SlowTask.EnterProgressFrame(1);
						}
					}

					NumLoads = LoadingContext.GetNumRegistrations();
					NumUnloads = LoadingContext.GetNumUnregistrations();
					bClearTransactions = LoadingContext.GetNeedsClearTransactions();
				}

				if (NumLoads || NumUnloads)
				{
					PostLoadedStateChanged(NumLoads, NumUnloads, bClearTransactions);
				}
			}
		}
	}

	// Remove invalid containers (happens when reloading level instances, etc)
	for (FContainerReferenceMap::TIterator It(ContainerActorReferences); It; ++It)
	{
		if (!It->Key.IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::OnActorDescContainerInstanceInitialize(UActorDescContainerInstance* ContainerInstance)
{
	RefreshLoadedState();
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::OnActorDescContainerInstanceUninitialize(UActorDescContainerInstance* ContainerInstance)
{
	// @todo_ow :
	// If the container is removed before broadcasting its uninitialize event we could simply call RefreshLoadedState(), but it would break call symetry
	TArray<FWorldPartitionHandle> ActorsToUnload;
	for(UActorDescContainerInstance::TConstIterator<> It(ContainerInstance); It; ++It)
	{
		const FActorReferenceMap* ActorReferences = GetContainerReferencesConst(It->GetContainerInstance());

		if (ActorReferences && ActorReferences->Find(It->GetGuid()))
		{
			ActorsToUnload.Emplace(ContainerInstance, It->GetGuid());
		}
	}
	
	if (ActorsToUnload.Num())
	{
		FWorldPartitionLoadingContext::FDeferred LoadingContext;
	
		FScopedSlowTask SlowTask(ActorsToUnload.Num(), LOCTEXT("ActorsUnoading", "Unloading actors..."));
		SlowTask.MakeDialogDelayed(1.0f);
	
		for (FWorldPartitionHandle& ActorToUnload : ActorsToUnload)
		{
			RemoveReferenceToActor(ActorToUnload);
			SlowTask.EnterProgressFrame(1);
		}
	
		PostLoadedStateChanged(0, ActorsToUnload.Num(), true);
	}
}

bool IWorldPartitionActorLoaderInterface::ILoaderAdapter::ShouldActorBeLoaded(const FWorldPartitionHandle& Actor) const
{
	check(Actor.IsValid());

	Actor->SetUnloadedReason(nullptr);

	if (!PassActorDescFilter(Actor))
	{
		return false;
	}

	for (auto& ActorDescFilter : ActorDescFilters)
	{
		if (!ActorDescFilter.Get().PassFilter(World, Actor))
		{
			Actor->SetUnloadedReason(ActorDescFilter.Get().GetFilterReason());
			return false;
		}
	}

	return true;
};

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::PostLoadedStateChanged(int32 NumLoads, int32 NumUnloads, bool bClearTransactions)
{
	check(NumLoads || NumUnloads);

	if (!IsRunningCommandlet())
	{
		GEngine->BroadcastLevelActorListChanged();

		if (NumUnloads)
		{
			if (!GIsTransacting && bClearTransactions)
			{
				GEditor->ResetTransaction(LOCTEXT("UnloadingEditorActorResetTrans", "Editor Actors Unloaded"));
			}

			GEngine->ForceGarbageCollection(true);
		}
	}
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::AddReferenceToActor(FWorldPartitionHandle& ActorHandle)
{
	TFunction<void(const FWorldPartitionHandle&, FReferenceMap&)> AddReferences = [this, &AddReferences](const FWorldPartitionHandle& Handle, FReferenceMap& ReferenceMap)
	{
		const FWorldPartitionActorDescInstance* ActorDescInstance = *Handle;
		if (!ReferenceMap.Contains(ActorDescInstance->GetGuid()))
		{
			ReferenceMap.Emplace(ActorDescInstance->GetGuid(), Handle.ToReference());
			
			for (const FGuid& ReferencedActorGuid : ActorDescInstance->GetReferences())
			{
				FWorldPartitionHandle ReferenceActorHandle(Handle->GetContainerInstance(), ReferencedActorGuid);

				if (ReferenceActorHandle.IsValid())
				{
					AddReferences(ReferenceActorHandle, ReferenceMap);
				}
			}
		}
	};

	FActorReferenceMap& ActorReferences = GetContainerReferences(ActorHandle->GetContainerInstance());
	AddReferences(ActorHandle, ActorReferences.Emplace(ActorHandle->GetGuid()));
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::RemoveReferenceToActor(FWorldPartitionHandle& ActorHandle)
{
	FActorReferenceMap& ActorReferences = GetContainerReferences(ActorHandle->GetContainerInstance());
	ActorReferences.Remove(ActorHandle->GetGuid());
}

void IWorldPartitionActorLoaderInterface::ILoaderAdapter::OnRefreshLoadedState(bool bFromUserOperation)
{
	RefreshLoadedState();
}

void IWorldPartitionActorLoaderInterface::RefreshLoadedState(bool bIsFromUserChange)
{
	if (!GUndo && !GIsTransacting)
	{
		ActorLoaderInterfaceRefreshState.Broadcast(bIsFromUserChange);
	}
	else
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([=](float DeltaTime)
		{
			if (!GUndo && !GIsTransacting)
			{
				ActorLoaderInterfaceRefreshState.Broadcast(bIsFromUserChange);
				return false;
			}
			return true;
		}));
	}
}

IWorldPartitionActorLoaderInterface::FActorReferenceMap& IWorldPartitionActorLoaderInterface::ILoaderAdapter::GetContainerReferences(UActorDescContainerInstance* InContainerInstance)
{
	return ContainerActorReferences.FindOrAdd(InContainerInstance);
}

const IWorldPartitionActorLoaderInterface::FActorReferenceMap* IWorldPartitionActorLoaderInterface::ILoaderAdapter::GetContainerReferencesConst(UActorDescContainerInstance* InContainerInstance) const
{
	return ContainerActorReferences.Find(InContainerInstance);
}

UWorldPartition* IWorldPartitionActorLoaderInterface::ILoaderAdapter::GetLoadedChildWorldPartition(const FWorldPartitionHandle& Handle) const
{
	return IWorldPartitionActorLoaderInterface::GetLoadedChildWorldPartition(Handle);
}

UWorldPartition* IWorldPartitionActorLoaderInterface::GetLoadedChildWorldPartition(const FWorldPartitionHandle& Handle)
{
	if (FWorldPartitionActorDescInstance* Instance = *Handle)
	{
		return Instance->GetLoadedChildWorldPartition();
	}

	return nullptr;
}
#endif

#undef LOCTEXT_NAMESPACE

