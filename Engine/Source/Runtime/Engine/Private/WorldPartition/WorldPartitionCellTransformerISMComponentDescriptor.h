// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Components/InstancedStaticMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Serialization/ArchiveCrc32.h"

#include "WorldPartitionCellTransformerISMComponentDescriptor.generated.h"

/** Bucket key for UWorldPartitionRuntimeCellTransformerISM. Adds UPrimitiveComponent fields the merge would otherwise drop. */
USTRUCT()
struct FWorldPartitionCellTransformerISMComponentDescriptor : public FISMComponentDescriptor
{
	GENERATED_BODY()

	// Defaults match the UPrimitiveComponent ctor so off-mode keys carry the same value the
	// merged ISM would end up with anyway.
	UPROPERTY()
	TEnumAsByte<ECanBeCharacterBase> CanCharacterStepUpOn = ECB_Yes;

	bool bStrictBucketing = false;

	virtual void InitFrom(const UStaticMeshComponent* Template, bool bInitBodyInstance = true) override
	{
		FISMComponentDescriptor::InitFrom(Template, bInitBodyInstance);
		if (bStrictBucketing && IsCanCharacterStepUpOnRelevant())
		{
			CanCharacterStepUpOn = Template->CanCharacterStepUpOn;
		}
	}

	virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const override
	{
		FISMComponentDescriptor::InitComponent(ISMComponent);
		if (bStrictBucketing && IsCanCharacterStepUpOnRelevant())
		{
			ISMComponent->CanCharacterStepUpOn = CanCharacterStepUpOn;
		}
	}

	// CanCharacterStepUpOn only matters when the component participates in collision queries which
	// is the path characters use to bump into geometry.
	bool IsCanCharacterStepUpOnRelevant() const
	{
		return CollisionEnabledHasQuery(BodyInstance.GetCollisionEnabled());
	}

	virtual uint32 ComputeHash() const override
	{
		FISMComponentDescriptor::ComputeHash();

		FArchiveCrc32 CrcArchive(Hash);
		uint8 StepUp = static_cast<uint8>(CanCharacterStepUpOn.GetValue());
		CrcArchive << StepUp;
		Hash = CrcArchive.GetCrc();
		return Hash;
	}

	bool operator==(const FWorldPartitionCellTransformerISMComponentDescriptor& Other) const
	{
		return FISMComponentDescriptor::operator==(Other) && CanCharacterStepUpOn == Other.CanCharacterStepUpOn;
	}

	bool operator!=(const FWorldPartitionCellTransformerISMComponentDescriptor& Other) const
	{
		return !(*this == Other);
	}
};
