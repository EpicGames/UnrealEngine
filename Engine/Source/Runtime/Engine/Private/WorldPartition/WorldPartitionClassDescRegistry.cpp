// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionClassDescRegistry.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "Cooker/CookAssetRegistryAccessTracker_IgnoreScope.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "UObject/CoreRedirects.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IPluginManager.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

/*
 * Class descriptors are specific properties that we keep available for unloaded actor classes. We want to keep in sync class descriptors and their
 * corresponding Blueprint generated class, to have the exact same behavior as delta serialization of Actors from their class CDOs. The different
 * scenarios for creating and updating class descriptors from the class CDO are as follow:
 *
 *	[Creating a new Blueprint]
 *	  - Don't create a class descriptor in that case, since we can't save an actor instance until we save the Blueprint.
 *	  - The Blueprint generated class is not even created at this point anyways.
 *	
 *	[Saving a new Blueprint]
 *	  - Create the class descriptor from the newly generated Blueprint class.
 *	
 *	[Modifying an existing Blueprint]
 *	  - Don't update the class descriptor, as actor instances of this class will use the existing (outdated) generated Blueprint class for delta
 *		serialization.
 *
 *	[Compiling an existing Blueprint]
 *	  - Create or update the class descriptor, but only if it already exists on disk.
 *
 *	[Saving a existing Blueprint]
 *	  - Create or update the class descriptor.
 * 
 *	[Saving a Blueprint Actor]
 *	  - Prefetch the class descriptor for the Blueprint Actor Class
 * 
 */

static FAutoConsoleCommand DumpClassDescs(
	TEXT("wp.Editor.DumpClassDescs"),
	TEXT("Dump the list of class descriptors in a CSV file."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() > 0)
		{
			const FString& Path = Args[0];
			if (FArchive* LogFile = IFileManager::Get().CreateFileWriter(*Path))
			{
				TArray<const FWorldPartitionActorDesc*> ClassDescs;
				const FWorldPartitionClassDescRegistry& ClassDescRegistry = FWorldPartitionClassDescRegistry::Get();

				for (FWorldPartitionClassDescRegistry::TConstIterator<> ClassDescIt(&ClassDescRegistry); ClassDescIt; ++ClassDescIt)
				{
					ClassDescs.Add(*ClassDescIt);
				}
				ClassDescs.Sort([](const FWorldPartitionActorDesc& A, const FWorldPartitionActorDesc& B)
				{
					return A.GetGuid() < B.GetGuid();
				});
				for (const FWorldPartitionActorDesc* ClassDescsIterator : ClassDescs)
				{
					FString LineEntry = ClassDescsIterator->ToString(FWorldPartitionActorDesc::EToStringMode::Full) + LINE_TERMINATOR;
					LogFile->Serialize(TCHAR_TO_ANSI(*LineEntry), LineEntry.Len());
				}

				LogFile->Close();
				delete LogFile;
			}
		}
	})
);

/*
 * We have to deal with some different realities between non-cooked and cooked editor builds: for non-cooked editor builds, UBlueprint are considered assets
 * and will be the only thing visible from the asset registry standpoint. On the other hand, for cooked editor builds, UBlueprint are not considered assets
 * while their corresponding UBlueprintGeneratedClass will be. Also, we expect redirectors only in non-cooked editor builds.
 */	
static FTopLevelAssetPath GetAssetDataClassNameForBlueprint(const FString& InAssetDataClassName)
{
	return FTopLevelAssetPath(InAssetDataClassName + TEXT("_C"));
}

static FTopLevelAssetPath GetAssetDataClassNameForBlueprintGeneratedClass(const FString& InAssetDataClassName)
{
	return FTopLevelAssetPath(InAssetDataClassName);
}

static FTopLevelAssetPath GetAssetDataClassName(const FAssetData& InAssetData)
{
	const FString AssetClassName = InAssetData.ToSoftObjectPath().ToString();

	if (InAssetData.GetClass()->IsChildOf<UBlueprint>())
	{
		return GetAssetDataClassNameForBlueprint(AssetClassName);
	}
	
	check(InAssetData.GetClass()->IsChildOf<UBlueprintGeneratedClass>());
	return GetAssetDataClassNameForBlueprintGeneratedClass(AssetClassName);
}

FWorldPartitionClassDescRegistry& FWorldPartitionClassDescRegistry::Get()
{
	return TLazySingleton<FWorldPartitionClassDescRegistry>::Get();
}

void FWorldPartitionClassDescRegistry::TearDown()
{
	TLazySingleton<FWorldPartitionClassDescRegistry>::TearDown();
}

void FWorldPartitionClassDescRegistry::Initialize()
{
	UE_SCOPED_TIMER(TEXT("FWorldPartitionClassDescRegistry::Initialize"), LogWorldPartition, Display);
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionClassDescRegistry::Initialize);

	check(!IsInitialized());

	RegisterClasses();

	FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FWorldPartitionClassDescRegistry::OnObjectPreSave);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FWorldPartitionClassDescRegistry::OnObjectPropertyChanged);

	IPluginManager& PluginManager = IPluginManager::Get();
	ELoadingPhase::Type LoadingPhase = PluginManager.GetLastCompletedLoadingPhase();
	if (LoadingPhase == ELoadingPhase::None || LoadingPhase < ELoadingPhase::PostEngineInit)
	{
		PluginManager.OnLoadingPhaseComplete().AddRaw(this, &FWorldPartitionClassDescRegistry::OnPluginLoadingPhaseComplete);
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.OnAssetRemoved().AddRaw(this, &FWorldPartitionClassDescRegistry::OnAssetRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FWorldPartitionClassDescRegistry::OnAssetRenamed);

	check(IsInitialized());
}

void FWorldPartitionClassDescRegistry::Uninitialize()
{
	check(IsInitialized());
	FActorDescList::Empty();

	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	
	if (!IsEngineExitRequested())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
		AssetRegistry.OnAssetRenamed().RemoveAll(this);
	}

	ClassByPath.Empty();
	ParentClassMap.Empty();

	check(!IsInitialized());
}

bool FWorldPartitionClassDescRegistry::IsInitialized() const
{
	return !FActorDescList::IsEmpty();
}

void FWorldPartitionClassDescRegistry::PrefetchClassDescs(const TArray<FTopLevelAssetPath>& InClassPaths)
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);
	PrefetchClassDescsInternal(InClassPaths);
}

void FWorldPartitionClassDescRegistry::PrefetchClassDescsInternal(const TArray<FTopLevelAssetPath>& InClassPaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionClassDescRegistry::PrefetchClassDescs);

	// This function makes asset registry request during the cook. The tracker will register those requests as dependencies. The issue
	// is the requests are not always made by the same package so I end up with non deterministic dependencies.
	// Those dependencies are useless. This function makes a list of BP and native classes used by the actor descriptors. Whatever dependencies
	// map/generated packages and actors need should already be tracked by the actors/map themselves.
	UE::CookAssetRegistryAccessTracker::FIgnoreScope IgnoreAssetRegistryTracker;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	// First, remap classes and remove already known ones.
	TArray<FTopLevelAssetPath> ClassPaths;
	ClassPaths.Reserve(InClassPaths.Num());
	Algo::Transform(InClassPaths, ClassPaths, [this](const FTopLevelAssetPath& ClassPath) { return RedirectClassPathInternal(ClassPath); });
	ClassPaths.SetNum(Algo::RemoveIf(ClassPaths, [this](const FTopLevelAssetPath& ClassPath)
	{
		const FString ClasPathString = ClassPath.ToString();
		return ClassByPath.Contains(ClassPath) || FPackageName::IsScriptPackage(ClasPathString) || FPackageName::IsMemoryPackage(ClasPathString) || FPackageName::IsTempPackage(ClasPathString);
	}));

	if (ClassPaths.IsEmpty())
	{
		return;
	}

	// Make sure the asset registry has all paths scanned
	TArray<FString> FilePaths;
	FilePaths.Reserve(ClassPaths.Num());
	Algo::Transform(ClassPaths, FilePaths, [this](const FTopLevelAssetPath& ClassPath) { return ClassPath.GetPackageName().ToString(); });

	// In the editor, the asset registry will not return blueprint generated classes or their default object but will during cooks (see AssetRegistry::FFiltering::ShouldSkipAsset),
	// so we must filter our results by UBlueprint and UObjectRedirector. For redirectors, they will show up for the blueprint, generated class and default objects in both situations,
	// so we must take special care to filter out everything except blueprint objects. The current way to find the blueprint asset redirector is based on finding the redirector with
	// the shortest name.
	auto SortBlueprintAssetsByNameLength = [](const FAssetData& Lhs, const FAssetData& Rhs) { return Lhs.AssetName.GetStringLength() < Rhs.AssetName.GetStringLength(); };

	auto GetBlueprintAssets = [&AssetRegistry](const TArray<FString>& FilePaths, TArray<FAssetData>& Assets)
	{
		int32 AssetIndex = Assets.Num();

		FARFilter Filter;
		// Don't rely on the AR to filter out classes as it's going to gather all assets for the specified classes first, then filtering for the provided packages names afterward,
		// resulting in execution time being over 100 times slower than filtering for classes after package names.
		//Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		//Filter.ClassPaths.Add(UBlueprintGeneratedClass::StaticClass()->GetClassPathName());
		//Filter.ClassPaths.Add(UObjectRedirector::StaticClass()->GetClassPathName());
		//Filter.bRecursiveClasses = true;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackageNames.Reserve(FilePaths.Num());
		Algo::Transform(FilePaths, Filter.PackageNames, [](const FString& ClassPath) { return *ClassPath; });

		AssetRegistry.ScanSynchronous(TArray<FString>(), FilePaths);
		AssetRegistry.GetAssets(Filter, Assets);

		// Filter out unwanted classes, see above comment for details.
		for (; AssetIndex < Assets.Num(); AssetIndex++)
		{
			if (UClass* AssetClass = Assets[AssetIndex].GetClass(); !AssetClass || (!AssetClass->IsChildOf<UBlueprint>() && !AssetClass->IsChildOf<UBlueprintGeneratedClass>() && !AssetClass->IsChildOf<UObjectRedirector>()))
			{
				Assets.RemoveAtSwap(AssetIndex--);
			}
		}
	};

	TArray<FAssetData> Assets;
	GetBlueprintAssets(FilePaths, Assets);

	// We are only interested in redirectors for blueprint objects, not their generated class (ClassName_C) or their default object (Default__ClassName_C), as per previous comment.
	{
		TMap<FName, TArray<FAssetData*>> PackageRedirectors;
		Algo::ForEach(Assets, [&PackageRedirectors](FAssetData& AssetData) { PackageRedirectors.FindOrAdd(AssetData.PackageName).Add(&AssetData); });
		Algo::ForEach(PackageRedirectors, [&SortBlueprintAssetsByNameLength](TPair<FName, TArray<FAssetData*>>& Pair)
		{
			// Find the blueprint asset or the proper blueprint redirector
			Pair.Value.Sort(SortBlueprintAssetsByNameLength);
			for (int32 AssetIndex = 1; AssetIndex < Pair.Value.Num(); AssetIndex++)	{ *Pair.Value[AssetIndex] = FAssetData(); }
		});
		Assets.SetNum(Algo::RemoveIf(Assets, [](const FAssetData& AssetData) { return !AssetData.IsValid(); }));
	}

	TMap<FTopLevelAssetPath, FTopLevelAssetPath> ParentClassPathsMap;
	Algo::ForEach(Assets, [this, &AssetRegistry, &ParentClassPathsMap, &SortBlueprintAssetsByNameLength, &GetBlueprintAssets](FAssetData& AssetData)
	{
		check(AssetData.IsValid());

		TSet<FTopLevelAssetPath> ClassRedirects;
		while (AssetData.IsRedirector())
		{
			// Folow the redirector to the destination object
			FString DestinationObjectPath;
			if (!AssetData.GetTagValue(TEXT("DestinationObject"), DestinationObjectPath))
			{
				UE_LOGF(LogWorldPartition, Warning, "Failed to follow class redirector for '%ls'", *AssetData.ToSoftObjectPath().ToString());
				AssetData = FAssetData();
				break;
			}

			const FTopLevelAssetPath AssetClassPath = FTopLevelAssetPath(DestinationObjectPath);

			TArray<FAssetData> RedirectAssets;
			GetBlueprintAssets({ AssetClassPath.GetPackageName().ToString() }, RedirectAssets);

			if (RedirectAssets.IsEmpty())
			{
				UE_LOGF(LogWorldPartition, Warning, "Failed to find redirected assets for '%ls' from '%ls'", *AssetData.ToSoftObjectPath().ToString(), *AssetClassPath.ToString());
				AssetData = FAssetData();
				break;
			}

			bool bRedirectAlreadyRegistered;
			ClassRedirects.Add(FTopLevelAssetPath(AssetData.ToSoftObjectPath().ToString()), &bRedirectAlreadyRegistered);

			if (bRedirectAlreadyRegistered)
			{
				const FString ClassRedirectsLoop = FString::JoinBy(ClassRedirects, TEXT("\n"), [](const FTopLevelAssetPath& RedirectPath) { return FString::Printf(TEXT("  -> %s"), *RedirectPath.ToString()); })
					+ FString::Printf(TEXT("\n  -> %s"), *FTopLevelAssetPath(AssetData.ToSoftObjectPath().ToString()).ToString());
				UE_LOGF(LogWorldPartition, Warning, "Redirector loop detected for '%ls' from '%ls':\n%ls", *AssetData.ToSoftObjectPath().ToString(), *AssetClassPath.ToString(), *ClassRedirectsLoop);
				AssetData = FAssetData();
				break;
			}

			// Find the blueprint asset or the proper blueprint redirector
			RedirectAssets.Sort(SortBlueprintAssetsByNameLength);
			AssetData = MoveTemp(RedirectAssets[0]);
		}

		if (AssetData.IsValid())
		{
			// Register redirects
			for (const FTopLevelAssetPath& ClassRedirect : ClassRedirects)
			{
				const FTopLevelAssetPath SourceClassPath(GetAssetDataClassNameForBlueprint(ClassRedirect.ToString()));
				const FTopLevelAssetPath RedirectedClassPath(GetAssetDataClassName(AssetData));
				RedirectClassMap.Add(SourceClassPath, RedirectedClassPath);
			}

			FString ParentClassName;
			if (AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName))
			{
				const FTopLevelAssetPath AssetClassName = GetAssetDataClassName(AssetData);
				ParentClassPathsMap.Add(AssetClassName, FTopLevelAssetPath(ParentClassName));
			}
		}
	});

	// Register parent class descriptors
	if (!ParentClassPathsMap.IsEmpty())
	{
		TArray<FTopLevelAssetPath> ParentClassPaths;
		ParentClassPathsMap.GenerateValueArray(ParentClassPaths);
		PrefetchClassDescsInternal(ParentClassPaths);

		for (auto& [ClassPath, ParentClassPath] : ParentClassPathsMap)
		{
			ParentClassMap.Add(ClassPath, RedirectClassPathInternal(ParentClassPath));
		}
	}

	// Register current class descriptors
	Algo::ForEach(Assets, [this, &AssetRegistry](FAssetData& AssetData)
	{
		// Asset could have been invalidated if a redirector chain wasn't properly resolved, etc.
		if (AssetData.IsValid())
		{
			const FTopLevelAssetPath AssetDataClassName(GetAssetDataClassName(AssetData));
			AssetData.AssetName = *AssetDataClassName.GetAssetName().ToString();

			// Lookup for an already registered class
			if (!ClassByPath.Contains(FTopLevelAssetPath(AssetData.ToSoftObjectPath().ToString())))
			{
				bool bOldAsset = false;
				FAssetPackageData PackageData;
				if (AssetRegistry.TryGetAssetPackageData(AssetData.PackageName, PackageData) == UE::AssetRegistry::EExists::Exists)
				{
					for (const UE::AssetRegistry::FPackageCustomVersion& CustomVersion : PackageData.GetCustomVersions())
					{
						if(CustomVersion.Key == FFortniteMainBranchObjectVersion::GUID)
						{
							if (CustomVersion.Version < FFortniteMainBranchObjectVersion::WorldPartitionActorClassDescSerialize)
							{
								bOldAsset = true;
							}
							break;
						}
					}
				}
				else
				{
					bOldAsset = true;
				}

				if (TUniquePtr<FWorldPartitionActorDesc> ClassDesc = !bOldAsset ? FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(AssetData) : nullptr; ClassDesc.IsValid())
				{
					FWorldPartitionActorDesc* Result = ClassDesc.Release();
					RegisterClassDescriptor(Result);
				}
				else
				{
					// For missing class descriptors, register a null entry so we won't be trying to register them again for future objects
					ClassByPath.Add(FTopLevelAssetPath(FTopLevelAssetPath(AssetData.ToSoftObjectPath().ToString())), nullptr);
				}
			}
		}
	});

	ValidateInternalState();
}

bool FWorldPartitionClassDescRegistry::IsRegisteredClass(const FTopLevelAssetPath& InClassPath) const
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);
	const FTopLevelAssetPath ClassPath = RedirectClassPathInternal(InClassPath);
	return ClassByPath.Contains(ClassPath);
}

bool FWorldPartitionClassDescRegistry::IsDerivedFrom(const FWorldPartitionActorDesc* InClassDesc, const FWorldPartitionActorDesc* InParentClassDesc) const
{
	UE::TScopeLock Lock(ClassDescLock);

	const FTopLevelAssetPath ParentClassPath = FTopLevelAssetPath(InParentClassDesc->GetActorSoftPath().ToString());

	FTopLevelAssetPath ClassPath = FTopLevelAssetPath(InClassDesc->GetActorSoftPath().ToString());

	while (ClassPath.IsValid())
	{
		if (ClassPath == ParentClassPath)
		{
			return true;
		}

		ClassPath = ParentClassMap.FindRef(ClassPath);
	}

	return false;
}

const FWorldPartitionActorDesc* FWorldPartitionClassDescRegistry::GetClassDescDefaultInternal(const FTopLevelAssetPath& InClassPath) const
{
	const FWorldPartitionActorDesc* Result = nullptr;
	FTopLevelAssetPath ClassPath = InClassPath;

	while (ClassPath.IsValid())
	{
		// Lookup for an already registered class
		if (const TUniquePtr<FWorldPartitionActorDesc>* const* ClassDesc = ClassByPath.Find(ClassPath); ClassDesc && *ClassDesc)
		{
			Result = (*ClassDesc)->Get();
			break;
		}

		ClassPath = ParentClassMap.FindRef(ClassPath);
	}

	return Result;
}

const FWorldPartitionActorDesc* FWorldPartitionClassDescRegistry::GetClassDescDefault(const FTopLevelAssetPath& InClassPath) const
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);
	return GetClassDescDefaultInternal(InClassPath);
}

const FWorldPartitionActorDesc* FWorldPartitionClassDescRegistry::GetClassDescDefaultForActor(const FTopLevelAssetPath& InClassPath) const
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);
	return GetClassDescDefaultInternal(RedirectClassPathInternal(InClassPath));
}

const FWorldPartitionActorDesc* FWorldPartitionClassDescRegistry::GetClassDescDefaultForClass(const FTopLevelAssetPath& InClassPath) const
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);
	const FTopLevelAssetPath ParentClassPath = ParentClassMap.FindRef(RedirectClassPathInternal(InClassPath));
	return GetClassDescDefaultInternal(ParentClassPath);
}

void FWorldPartitionClassDescRegistry::PrefetchClassDesc(UClass* InClass)
{
	for (UClass* ParentClass = InClass; !ParentClass->IsNative(); ParentClass = ParentClass->GetSuperClass())
	{
		const FTopLevelAssetPath ParentClassPath(ParentClass->GetPathName());
						
		if (!ClassByPath.Contains(ParentClassPath))
		{
			// Only prefetch classes that exists on disk
			const UPackage* ParentClassPackage = ParentClass->GetPackage();
			if ((ParentClassPackage != GetTransientPackage()) && !ParentClassPackage->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				PrefetchClassDescsInternal({ ParentClassPath });
			}

			if (ClassByPath.Contains(ParentClassPath))
			{
				break;
			}

			ClassByPath.Add(ParentClassPath, nullptr);
			ParentClassMap.Add(FTopLevelAssetPath(ParentClass->GetPathName()), FTopLevelAssetPath(ParentClass->GetSuperClass()->GetPathName()));
		}
	}
}

void FWorldPartitionClassDescRegistry::RegisterClassDescriptor(FWorldPartitionActorDesc* InClassDesc)
{
	FActorDescList::AddActorDescriptor(InClassDesc);

	// Validate that the class isn't registered yet
	const FTopLevelAssetPath ClassPath(InClassDesc->GetActorSoftPath().ToString());
	if (TUniquePtr<FWorldPartitionActorDesc>** ExistingClassDesc = ClassByPath.Find(ClassPath))
	{
		check(!(*ExistingClassDesc));
	}

	ClassByPath.Add(ClassPath, ActorsByGuid.FindChecked(InClassDesc->GetGuid()));
}

void FWorldPartitionClassDescRegistry::UnregisterClassDescriptor(FWorldPartitionActorDesc* InClassDesc)
{
	verify(ClassByPath.Remove(FTopLevelAssetPath(InClassDesc->GetActorSoftPath().ToString())));
	FActorDescList::RemoveActorDescriptor(InClassDesc);
}

void FWorldPartitionClassDescRegistry::RegisterClassDescriptorFromAssetData(const FAssetData& InAssetData)
{
	const FTopLevelAssetPath ClassPath(GetAssetDataClassName(InAssetData));
	check(!ClassByPath.Contains(ClassPath));

	UBlueprint* Blueprint = CastChecked<UBlueprint>(InAssetData.GetAsset());
	RegisterClassDescriptorFromActorClass(Blueprint->GeneratedClass);
}

void FWorldPartitionClassDescRegistry::RegisterClassDescriptorFromActorClass(const UClass* InActorClass)
{
	if (UClass* ParentClass = InActorClass->GetSuperClass(); ParentClass && ParentClass->IsChildOf<AActor>())
	{
		if (!ClassByPath.Contains(FTopLevelAssetPath(ParentClass->GetPathName())))
		{
			RegisterClassDescriptorFromActorClass(ParentClass);
		}
	}

	ParentClassMap.Add(FTopLevelAssetPath(InActorClass->GetPathName()), FTopLevelAssetPath(InActorClass->GetSuperClass()->GetPathName()));

	const AActor* ActorCDO = CastChecked<AActor>(InActorClass->GetDefaultObject());
	FWorldPartitionActorDesc* NewActorDesc = ActorCDO->CreateActorDesc().Release();
	RegisterClassDescriptor(NewActorDesc);
}

void FWorldPartitionClassDescRegistry::OnObjectPreSave(UObject* InObject, FObjectPreSaveContext InSaveContext)
{
	const FWorldPartitionActorDesc* UpdatedClassDesc = nullptr;

	{
		UE::TScopeLock Lock(ClassDescLock);

		// Are we are saving a blueprint?
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
		{
			// Is this blueprint an asset (could be a level script)?
			if (Cast<UPackage>(Blueprint->GetOuter()))
			{
				// Is this blueprint generated class valid?
				if (UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass))
				{
					// Is it an actor derived blueprint?
					if (BlueprintGeneratedClass->IsChildOf<AActor>())
					{
						PrefetchClassDesc(BlueprintGeneratedClass);
						UpdatedClassDesc = UpdateClassDescriptor(Blueprint, false);
					}
				}
			}
		}

		if (InObject->IsA<AActor>())
		{
			if (UBlueprintGeneratedClass* BlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(InObject->GetClass()))
			{
				PrefetchClassDesc(BlueprintGeneratedClass);
			}
		}
	}

	if (UpdatedClassDesc)
	{
		ClassDescriptorUpdatedEvent.Broadcast(UpdatedClassDesc);
	}
}

void FWorldPartitionClassDescRegistry::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FWorldPartitionActorDesc* UpdatedClassDesc = nullptr;

	{
		UE::TScopeLock Lock(ClassDescLock);

		// We only want to handle BP class compiles, not property changes.
		if (!InPropertyChangedEvent.Property)
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(InObject))
			{
				// The generated class is invalid in some situations, like renaming a blueprint, etc.
				if (Blueprint->IsUpToDate() && Blueprint->GeneratedClass && Blueprint->GeneratedClass->IsChildOf<AActor>())
				{
					PrefetchClassDesc(Blueprint->GeneratedClass);
					UpdatedClassDesc = UpdateClassDescriptor(InObject, true);
				}
			}
		}
	}

	if (UpdatedClassDesc)
	{
		ClassDescriptorUpdatedEvent.Broadcast(UpdatedClassDesc);
	}
}

void FWorldPartitionClassDescRegistry::OnPluginLoadingPhaseComplete(ELoadingPhase::Type InLoadingPhase, bool bInPhaseSuccessful)
{
	if (InLoadingPhase == ELoadingPhase::PostEngineInit)
	{
		RegisterClasses();
		IPluginManager::Get().OnLoadingPhaseComplete().RemoveAll(this);
	}
}

void FWorldPartitionClassDescRegistry::OnAssetRemoved(const FAssetData& InAssetData)
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);

	if (InAssetData.GetClass() && InAssetData.GetClass()->IsChildOf(UBlueprint::StaticClass()))
	{
		const FTopLevelAssetPath ClassPath(GetAssetDataClassName(InAssetData));

		if (TUniquePtr<FWorldPartitionActorDesc>** ExistingClassDesc = ClassByPath.Find(ClassPath))
		{
			ValidateInternalState();
			if (*ExistingClassDesc)
			{
				UnregisterClassDescriptor((*ExistingClassDesc)->Get());
			}
			else
			{
				ClassByPath.Remove(ClassPath);
			}
			ParentClassMap.Remove(ClassPath);
			ValidateInternalState();
		}
	}
}

void FWorldPartitionClassDescRegistry::OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	check(IsInitialized());
	UE::TScopeLock Lock(ClassDescLock);

	if (InAssetData.GetClass() && InAssetData.GetClass()->IsChildOf(UBlueprint::StaticClass()))
	{
		ValidateInternalState();

		const FTopLevelAssetPath OldClassPath(GetAssetDataClassNameForBlueprint(InOldObjectPath));
		const FTopLevelAssetPath NewClassPath(GetAssetDataClassName(InAssetData));

		if (TUniquePtr<FWorldPartitionActorDesc>** ExistingClassDesc = ClassByPath.Find(OldClassPath))
		{
			if (*ExistingClassDesc)
			{
				UnregisterClassDescriptor((*ExistingClassDesc)->Get());
				RegisterClassDescriptorFromAssetData(InAssetData);
			}
			else
			{
				ClassByPath.Remove(OldClassPath);
				ClassByPath.Add(NewClassPath, nullptr);
			}
		
			// Update renamed class parent
			const FTopLevelAssetPath ParentClassPath = ParentClassMap.FindAndRemoveChecked(OldClassPath);
			ParentClassMap.Add(NewClassPath, ParentClassPath);

			// Update renamed class child classes
			TArray<FTopLevelAssetPath> ChildsToUpdate;
			for (auto& [ChildClassPath, ChildParentClassPath] : ParentClassMap)
			{
				if (ChildParentClassPath == OldClassPath)
				{
					ChildsToUpdate.Add(ChildClassPath);
				}
			}

			for (const FTopLevelAssetPath& ChildClassPath : ChildsToUpdate)
			{
				ParentClassMap.FindChecked(ChildClassPath) = NewClassPath;
			}
		}

		ValidateInternalState();
	}
}

void FWorldPartitionClassDescRegistry::RegisterClass(UClass* Class)
{
	if (!ClassByPath.Contains(FTopLevelAssetPath(Class->GetPathName())))
	{
		if (Class != AActor::StaticClass())
		{
			RegisterClass(Class->GetSuperClass());
		}

		RegisterClassDescriptorFromActorClass(Class);
	}
}

void FWorldPartitionClassDescRegistry::RegisterClasses()
{
	UE::TScopeLock Lock(ClassDescLock);
	ValidateInternalState();

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (Class->IsChildOf<AActor>() && (Class->GetPackage() != GetTransientPackage()))
		{
			RegisterClass(Class);
		}
	}

	ValidateInternalState();
}

const FWorldPartitionActorDesc* FWorldPartitionClassDescRegistry::UpdateClassDescriptor(UObject* InObject, bool bOnlyIfExists)
{
	check(IsInitialized());

	UBlueprint* Blueprint = CastChecked<UBlueprint>(InObject);
	check(Blueprint->GeneratedClass);

	AActor* ActorCDO = CastChecked<AActor>(Blueprint->GeneratedClass->GetDefaultObject());

	const FTopLevelAssetPath ClassPath(ActorCDO->GetClass()->GetPathName());
	const FTopLevelAssetPath CurrentParentClassPath = FTopLevelAssetPath(ActorCDO->GetClass()->GetSuperClass()->GetPathName());

	if (TUniquePtr<FWorldPartitionActorDesc>** ExistingClassDesc = ClassByPath.Find(ClassPath))
	{
		if (!*ExistingClassDesc)
		{
			if (!bOnlyIfExists)
			{
				ValidateInternalState();

				FWorldPartitionActorDesc* NewActorDesc = ActorCDO->CreateActorDesc().Release();
				RegisterClassDescriptor(NewActorDesc);

				ParentClassMap.Add(ClassPath, CurrentParentClassPath);

				ValidateInternalState();
			}
		}
		else
		{
			FWorldPartitionActorDescUtils::UpdateActorDescriptorFromActor(ActorCDO, **ExistingClassDesc);

			const FTopLevelAssetPath PreviousParentClassPath = ParentClassMap.FindChecked(ClassPath);
			if (PreviousParentClassPath != CurrentParentClassPath)
			{
				ValidateInternalState();

				// We are reparenting a blueprint, update our parent map
				ParentClassMap.Add(ClassPath, CurrentParentClassPath);

				ValidateInternalState();
			}

			// Return the updated descriptor so the caller can broadcast after releasing the lock
			return (*ExistingClassDesc)->Get();
		}
	}

	return nullptr;
}

void FWorldPartitionClassDescRegistry::ValidateInternalState()
{
#if DO_GUARD_SLOW
	check(ClassByPath.Num() == ParentClassMap.Num());
	static const FTopLevelAssetPath ActorClassPath(TEXT("/Script/Engine.Actor"));
	for (auto& [ClassPath, ParentClassPath] : ParentClassMap)
	{
		// Validate that the class parents chain is valid
		check(ClassByPath.Contains(ClassPath));
		check((ParentClassPath != ActorClassPath) || ParentClassMap.Contains(ParentClassPath));
	}
#endif
}

FTopLevelAssetPath FWorldPartitionClassDescRegistry::RedirectClassPathInternal(const FTopLevelAssetPath& InClassPath) const
{
	// Resolve native class redirectors
	const FCoreRedirectObjectName OldClassName(InClassPath.GetAssetName(), NAME_None, InClassPath.GetPackageName());
	const FCoreRedirectObjectName NewClassName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldClassName);
	const FTopLevelAssetPath ResolvedClassPath(NewClassName.PackageName, NewClassName.ObjectName);

	// Resolve blueprint object redirectors
	if (const FTopLevelAssetPath* RedirectedClass = RedirectClassMap.Find(ResolvedClassPath))
	{
		return *RedirectedClass;
	}

	return ResolvedClassPath;
}

FTopLevelAssetPath FWorldPartitionClassDescRegistry::RedirectClassPath(const FTopLevelAssetPath& InClassPath) const
{
	UE::TScopeLock Lock(ClassDescLock);
	return RedirectClassPathInternal(InClassPath);
}
#endif
