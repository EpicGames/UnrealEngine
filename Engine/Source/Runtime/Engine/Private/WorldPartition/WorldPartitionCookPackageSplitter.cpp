// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionCookPackageSplitter.h"

#if WITH_EDITOR

#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionSettings.h"

TArray<ICookPackageSplitter::FGeneratedPackage> FWorldPartitionCookPackageSplitter::GetGenerateList(const UPackage* OwnerPackage)
{
	World = UWorld::FindWorldInPackage(const_cast<UPackage*>(OwnerPackage));
	check(World);

	UE_LOGF(LogWorldPartition, Display, "[Cook] Gathering packages to cook from generators for owner object %ls.", *GetFullNameSafe(World));

	// Assume that the world is partitioned as per FWorldPartitionCookPackageSplitter::ShouldSplit
	UWorldPartition* WorldPartition = World->PersistentLevel->GetWorldPartition();
	check(WorldPartition);

	// We expect the WorldPartition has not yet been initialized
	ensure(!WorldPartition->IsInitialized());
	WorldPartition->Initialize(World, FTransform::Identity);
	WorldPartition->BeginCook(CookContext);

	bool bIsSuccess = CookContext.GatherPackagesToCook();
	UE_CLOGF(!bIsSuccess, LogWorldPartition, Warning, "[Cook] Errors while gathering packages to took from generators for owner object %ls.", *GetFullNameSafe(World));

	UE_LOGF(LogWorldPartition, Display, "[Cook] Gathered %u packages to generate from %u Generators.", CookContext.NumPackageToGenerate(), CookContext.NumGenerators());

	TArray<ICookPackageSplitter::FGeneratedPackage> PackagesToGenerate;
	BuildPackagesToGenerateList(PackagesToGenerate);

	UE_LOGF(LogWorldPartition, Display, "[Cook] Sending %u packages to be generated.", PackagesToGenerate.Num());
	UE_LOGF(LogWorldPartition, Display, "[Cook] Debug(GetGenerateList) : OwnerWorld=%ls", *GetFullNameSafe(World));

	return PackagesToGenerate;
}

TOptional<bool> FWorldPartitionCookPackageSplitter::RequiresEmptyPackageBeforePopulate(const UPackage* Package)
{
	if (CookContext.IsGeneratingPackage(Package))
	{
		return true;
	}
	return TOptional<bool>();
}

bool FWorldPartitionCookPackageSplitter::PopulateGeneratedPackage(ICookPackageSplitter::FPopulateContext& PopulateContext)
{
	const ICookPackageSplitter::FGeneratedPackageForPopulate& GeneratedPackage = *PopulateContext.GetTargetGeneratedPackage();
	TGuardValue<ICookPackageSplitter::FPopulateContext*> Guard(CookContext.PopulateContext, &PopulateContext);

	UE_LOGF(LogWorldPartition, Verbose, "[Cook][PopulateGeneratedPackage] Processing %ls", *FWorldPartitionCookPackage::MakeGeneratedFullPath(GeneratedPackage.GeneratedRootPath, GeneratedPackage.RelativePath));

	bool bIsSuccess = true;

	IWorldPartitionCookPackageGenerator* CookPackageGenerator = nullptr;
	FWorldPartitionCookPackage* CookPackage = nullptr;
	TArray<UPackage*> ModifiedPackages;
	if (CookContext.GetCookPackageGeneratorAndPackage(GeneratedPackage.GeneratedRootPath, GeneratedPackage.RelativePath, CookPackageGenerator, CookPackage))
	{
		bIsSuccess = CookPackageGenerator->PopulateGeneratedPackageForCook(CookContext, *CookPackage, ModifiedPackages);
	}
	else
	{
		UE_LOGF(LogWorldPartition, Error, "[Cook][PopulateGeneratedPackage] Could not find WorldPartitionCookPackage for %ls", *FWorldPartitionCookPackage::MakeGeneratedFullPath(GeneratedPackage.GeneratedRootPath, GeneratedPackage.RelativePath));
		bIsSuccess = false;
	}

	UE_LOGF(LogWorldPartition, Verbose, "[Cook][PopulateGeneratedPackage] Gathered %u modified packages for %ls", ModifiedPackages.Num() , *FWorldPartitionCookPackage::MakeGeneratedFullPath(GeneratedPackage.GeneratedRootPath, GeneratedPackage.RelativePath));
	PopulateContext.ReportKeepReferencedPackages(ModifiedPackages);

	return bIsSuccess;
}

bool FWorldPartitionCookPackageSplitter::PopulateGeneratorPackage(ICookPackageSplitter::FPopulateContext& PopulateContext)
{
	TConstArrayView<ICookPackageSplitter::FGeneratedPackageForPopulate> GeneratedPackages = PopulateContext.GetGeneratedPackages();
	TGuardValue<ICookPackageSplitter::FPopulateContext*> Guard(CookContext.PopulateContext, &PopulateContext);

	UE_LOGF(LogWorldPartition, Display, "[Cook][PopulateGeneratorPackage] Processing %u packages", GeneratedPackages.Num());

	bool bIsSuccess = true;

	TArray<UPackage*> ModifiedPackages;
	for (IWorldPartitionCookPackageGenerator* CookPackageGenerator : CookContext.GetCookPackageGenerators())
	{
		bIsSuccess &= CookPackageGenerator->PrepareGeneratorPackageForCook(CookContext, ModifiedPackages);
		if (const TArray<FWorldPartitionCookPackage*>* CookPackages = CookContext.GetCookPackages(CookPackageGenerator))
		{
			bIsSuccess &= CookPackageGenerator->PopulateGeneratorPackageForCook(CookContext, *CookPackages, ModifiedPackages);
		}
	}

	UE_LOGF(LogWorldPartition, Display, "[Cook][PopulateGeneratorPackage] Gathered %u modified packages", ModifiedPackages.Num());
	PopulateContext.ReportKeepReferencedPackages(ModifiedPackages);

	return bIsSuccess;
}

TOptional<bool> FWorldPartitionCookPackageSplitter::SkipSaveGeneratorPackage(const UPackage* OwnerPackage)
{
	// If we have EDL cook filters, we are trying to cook the EDLs in isolation. Do not save the map even though it was loaded.
	if (!UWorldPartitionSettings::ShouldCookEDLGenerator(OwnerPackage))
	{
		return true;
	}
	return TOptional<bool>();
}

void FWorldPartitionCookPackageSplitter::Teardown(ICookPackageSplitter::ETeardown Status)
{
	UE_LOGF(LogWorldPartition, Display, "[Cook] Debug(TearDown): OwnerWorld=%ls", *GetFullNameSafe(World));

	// We were destructed without GetGenerateList being called; nothing to teardown
	// Assume that the world is partitioned as per FWorldPartitionCookPackageSplitter::ShouldSplit
	UWorldPartition* WorldPartition = World->PersistentLevel->GetWorldPartition();
	check(WorldPartition);

	WorldPartition->EndCook(CookContext);
	WorldPartition->Uninitialize();
}

void FWorldPartitionCookPackageSplitter::BuildPackagesToGenerateList(TArray<ICookPackageSplitter::FGeneratedPackage>& PackagesToGenerate) const
{
	const bool bEDLCopyChunkAssignmentFromGenerator = UWorldPartitionSettings::Get()->ShouldEDLPackagesInheritWorldChunkAssignmentsDuringCook();

	for (const IWorldPartitionCookPackageGenerator* CookPackageGenerator : CookContext.GetCookPackageGenerators())
	{
		if (const TArray<FWorldPartitionCookPackage*>* CookPackages = CookContext.GetCookPackages(CookPackageGenerator))
		{
			PackagesToGenerate.Reserve(CookPackages->Num());

			for (const FWorldPartitionCookPackage* CookPackage : *CookPackages)
			{
				ICookPackageSplitter::FGeneratedPackage& GeneratedPackage = PackagesToGenerate.Emplace_GetRef();
				GeneratedPackage.GeneratedRootPath = CookPackage->Root;
				GeneratedPackage.RelativePath = CookPackage->RelativePath;
				GeneratedPackage.GenerationHash = CookPackage->GenerationHash;

				CookContext.RegisterGeneratedPackage(CookPackage->GetFullGeneratedPath());
				UE_LOGF(LogWorldPartition, Log, "Adding Cell %ls with GenerationHash: %ls to packages to generate", *CookPackage->GetFullGeneratedPath(), *LexToString(CookPackage->GenerationHash))

				GeneratedPackage.SetCreateAsMap(CookPackage->Type == FWorldPartitionCookPackage::EType::Level);

				// Fill generated package dependencies for iterative cooking
				if (UWorldPartitionRuntimeCell* Cell = CookPackageGenerator->GetCellForPackage(*CookPackage))
				{
					check(CookPackage->Type == FWorldPartitionCookPackage::EType::Level);

					// If this package belongs to an External Data Layer, do NOT inherit the generator's chunk assignment if this option is enabled for the project
					GeneratedPackage.bCopyChunkAssignmentFromGenerator = bEDLCopyChunkAssignmentFromGenerator || Cell->GetExternalDataLayer().IsNone();

					TSet<FName> ActorPackageNames = Cell->GetActorPackageNames();
					GeneratedPackage.PackageDependencies.Reset(ActorPackageNames.Num());
					for (FName ActorPackageName : ActorPackageNames)
					{
						//The logic here for the dependency property is:
						// EditorOnly (no Game flag) : so the dependency is not added as a runtime dependency.
						// Build : so the runtime dependencies of ActorPackageName are added as runtime dependencies.
						GeneratedPackage.PackageDependencies.Add(FAssetDependency::PackageDependency(ActorPackageName,
							UE::AssetRegistry::EDependencyProperty::Hard | UE::AssetRegistry::EDependencyProperty::Build));
					}
				}
				else
				{
					// Copy chunk assignment only for level packages.
					GeneratedPackage.bCopyChunkAssignmentFromGenerator = bEDLCopyChunkAssignmentFromGenerator || CookPackage->Type == FWorldPartitionCookPackage::EType::Level;
				}
			}
		}
	}
}

#endif