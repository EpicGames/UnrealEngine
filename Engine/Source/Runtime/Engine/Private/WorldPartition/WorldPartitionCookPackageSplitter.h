// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "WorldCookPackageSplitter.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContext.h"

class FWorldPartitionCookPackageSplitter : public FWorldCookPackageSplitter::ISubSplitter
{
public:
	//~Begin FWorldCookPackageSplitter::ISubSplitter interface
	virtual TArray<ICookPackageSplitter::FGeneratedPackage> GetGenerateList(const UPackage* OwnerPackage) override;
	virtual TOptional<bool> RequiresEmptyPackageBeforePopulate(const UPackage* Package) override;
	virtual bool PopulateGeneratedPackage(ICookPackageSplitter::FPopulateContext& PopulateContext) override;
	virtual bool PopulateGeneratorPackage(ICookPackageSplitter::FPopulateContext& PopulateContext) override;
	virtual TOptional<bool> SkipSaveGeneratorPackage(const UPackage* OwnerPackage) override;
	virtual void Teardown(ICookPackageSplitter::ETeardown Status) override;
	//~End FWorldCookPackageSplitter::ISubSplitter interface

private:
	void BuildPackagesToGenerateList(TArray<ICookPackageSplitter::FGeneratedPackage>& PackagesToGenerate) const;
	
	UWorld* World = nullptr;
	FWorldPartitionCookPackageContext CookContext;
};

#endif