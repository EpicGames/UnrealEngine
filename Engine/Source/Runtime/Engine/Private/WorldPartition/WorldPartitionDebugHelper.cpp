// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "Engine/Engine.h"
#include "GlobalRenderResources.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Engine/Canvas.h"
#include "Algo/AllOf.h"

bool FWorldPartitionDebugHelper::bCanDrawContentBundles = true;
FAutoConsoleVariableRef FWorldPartitionDebugHelper::DrawContentBundlesCommand(
	TEXT("wp.Runtime.DrawContentBundles"),
	FWorldPartitionDebugHelper::bCanDrawContentBundles,
	TEXT("Enable to draw debug display of content bundle."));

TSet<FName> FWorldPartitionDebugHelper::DebugRuntimeHashFilter;
FAutoConsoleCommand FWorldPartitionDebugHelper::DebugFilterByRuntimeHashGridNameCommand(
	TEXT("wp.Runtime.DebugFilterByRuntimeHashGridName"),
	TEXT("Filter debug diplay of world partition streaming by grid name. Args [grid names]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		DebugRuntimeHashFilter.Reset();
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (UWorldPartition* WorldPartition = World->GetWorldPartition())
				{
					for (const FString& Arg : InArgs)
					{
						if (WorldPartition->RuntimeHash && WorldPartition->RuntimeHash->ContainsRuntimeHash(Arg))
						{
							DebugRuntimeHashFilter.Add(FName(Arg));
						}
					}
				}
			}
		}
	})
);

bool FWorldPartitionDebugHelper::IsDebugRuntimeHashGridShown(FName Name)
{
	return !DebugRuntimeHashFilter.Num() || DebugRuntimeHashFilter.Contains(Name);
}

TSet<FName> FWorldPartitionDebugHelper::DebugDataLayerFilter;
FAutoConsoleCommand FWorldPartitionDebugHelper::DebugFilterByDataLayerCommand(
	TEXT("wp.Runtime.DebugFilterByDataLayer"),
	TEXT("Filter debug diplay of world partition streaming by data layer. Args [datalayer labels]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		DebugDataLayerFilter.Reset();

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{
					WorldPartitionSubsystem->ForEachWorldPartition([&InArgs](UWorldPartition* WorldPartition)
					{
						if (UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
						{
							TArray<UDataLayerInstance*> DataLayerInstances = DataLayerManager->ConvertArgsToDataLayers(InArgs);
							for (UDataLayerInstance* DataLayerInstance : DataLayerInstances)
							{
								DebugDataLayerFilter.Add(DataLayerInstance->GetFName());
							}
						}
						return true;
					});
				}
			}
		}

		if (Algo::AnyOf(InArgs, [](const FString& Arg) { return FName(Arg) == NAME_None; }))
		{
			DebugDataLayerFilter.Add(NAME_None);
		}
	})
);

bool FWorldPartitionDebugHelper::IsDebugDataLayerShown(FName DataLayerName)
{
	return !DebugDataLayerFilter.Num() || DebugDataLayerFilter.Contains(DataLayerName);
}

bool FWorldPartitionDebugHelper::AreDebugDataLayersShown(const TArray<FName>& DataLayerNames)
{
	if (DebugDataLayerFilter.Num())
	{
		bool bFilter = !DataLayerNames.IsEmpty() || !DebugDataLayerFilter.Contains(NAME_None);
		for (const FName& DataLayerName : DataLayerNames)
		{
			if (DebugDataLayerFilter.Contains(DataLayerName))
			{
				bFilter = false;
				break;
			}
		}
		if (bFilter)
		{
			return false;
		}
	}
	return true;
}

TSet<EStreamingStatus> FWorldPartitionDebugHelper::DebugStreamingStatusFilter;
FAutoConsoleCommand FWorldPartitionDebugHelper::DebugFilterByStreamingStatusCommand(
	TEXT("wp.Runtime.DebugFilterByStreamingStatus"),
	TEXT("Filter debug diplay of world partition streaming by streaming status. Args [streaming status]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		DebugStreamingStatusFilter.Reset();
		for (const FString& Arg : InArgs)
		{
			int32 Result = 0;
			TTypeFromString<int32>::FromString(Result, *Arg);
			if (Result >= 0 && Result < (int32)LEVEL_StreamingStatusCount)
			{
				DebugStreamingStatusFilter.Add((EStreamingStatus)Result);
			}
		}
	})
);

bool FWorldPartitionDebugHelper::IsDebugStreamingStatusShown(EStreamingStatus StreamingStatus)
{
	return !DebugStreamingStatusFilter.Num() || DebugStreamingStatusFilter.Contains(StreamingStatus);
}

TArray<FString> FWorldPartitionDebugHelper::DebugCellNameFilter;
FAutoConsoleCommand FWorldPartitionDebugHelper::DebugFilterByCellNameCommand(
	TEXT("wp.Runtime.DebugFilterByCellName"),
	TEXT("Filter debug diplay of world partition streaming by full or partial cell name. Args [cell name]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		TSet<FString> Filter;
		Filter.Append(InArgs);
		DebugCellNameFilter = Filter.Array();
	})
);

bool FWorldPartitionDebugHelper::IsDebugCellNameShown(const FString& CellName)
{
	return !DebugCellNameFilter.Num() || Algo::AllOf(DebugCellNameFilter, [&CellName](const FString& NameFilter) { return CellName.Contains(NameFilter); });
}

void FWorldPartitionDebugHelper::DrawText(UCanvas* Canvas, const FString& Text, const UFont* Font, const FColor& Color, FVector2D& Pos, float* MaxTextWidth)
{
	const float XScale = 1.f;
	const float YScale = 1.f;
	FFontRenderInfo RenderInfo;
	RenderInfo.bClipText = true;
	RenderInfo.bEnableShadow = true;
	Canvas->SetDrawColor(Color);
	Canvas->DrawText(Font, Text, Pos.X, Pos.Y, XScale, YScale, RenderInfo);
	float TextWidth, TextHeight;
	Canvas->StrLen(Font, Text, TextWidth, TextHeight);
	Pos.Y += TextHeight + 1;
	if (MaxTextWidth)
	{
		*MaxTextWidth = FMath::Max(*MaxTextWidth, TextWidth);
	}
}

void FWorldPartitionDebugHelper::DrawLegendItem(UCanvas* Canvas, const FString& Text, const UFont* Font, const FColor& Color, const FColor& TextColor, FVector2D& Pos, float* MaxItemWidth)
{
	static const FVector2D ItemSize(12, 12);
	
	float MaxTextWidth = 0.f;

	const FVector2D ShadowItemPos(Pos - FVector2D(1, 1));
	const FVector2D ShadowItemSize(ItemSize + FVector2D(2, 2));
	const FColor ShadowItemColor(0,0,0);
	FCanvasTileItem ShadowItem(ShadowItemPos, GWhiteTexture, ShadowItemSize, ShadowItemColor);
	Canvas->DrawItem(ShadowItem);

	FCanvasTileItem Item(Pos, GWhiteTexture, ItemSize, Color);
	Canvas->DrawItem(Item);

	FVector2D TextPos(Pos.X + ItemSize.X + 10, Pos.Y);
	float TextWidth = 0.f;
	FWorldPartitionDebugHelper::DrawText(Canvas, Text, Font, TextColor, TextPos, &TextWidth);
	if (MaxItemWidth)
	{
		*MaxItemWidth = FMath::Max(*MaxItemWidth, TextWidth + ItemSize.X + 10);
	}

	Pos.Y = TextPos.Y;
}

int32 FWorldPartitionDebugHelper::ShowRuntimeSpatialHashCellStreamingPriorityMode = 0;
FAutoConsoleVariableRef FWorldPartitionDebugHelper::ShowRuntimeSpatialHashCellStreamingPriorityModeCommand(
	TEXT("wp.Runtime.ShowRuntimeSpatialHashCellStreamingPriority"),
	FWorldPartitionDebugHelper::ShowRuntimeSpatialHashCellStreamingPriorityMode,
	TEXT("Enable to show a heatmap of the runtime spatial hash grid cells based on their priority (0=disabled, 1=heatmap, 2=grayscale."));