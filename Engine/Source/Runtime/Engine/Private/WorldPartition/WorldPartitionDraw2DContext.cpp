// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionDraw2DContext.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "Math/UnrealMathUtility.h"
#include "GlobalRenderResources.h"
#include "BatchedElements.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"

FWorldPartitionDraw2DCanvas::FWorldPartitionDraw2DCanvas(UCanvas* InCanvas)
: Canvas(InCanvas)
{
	check(Canvas);
	FCanvas* CanvasObject = Canvas->Canvas;
	CanvasLineBatchedElements = CanvasObject->GetBatchedElements(FCanvas::ET_Line);
	CanvasTriangleBatchedElements = CanvasObject->GetBatchedElements(FCanvas::ET_Triangle, nullptr, GWhiteTexture, SE_BLEND_Translucent);
}

void FWorldPartitionDraw2DCanvas::PrepareDraw(int32 InLineCount, int32 InBoxCount)
{
	check(InLineCount >= 0);
	check(InBoxCount >= 0);

	CanvasLineBatchedElements->AddReserveLines(InLineCount);
	CanvasLineBatchedElements->AddReserveLines(InLineCount, false, true);

	CanvasTriangleBatchedElements->ReserveVertices(4 * InBoxCount);
	CanvasTriangleBatchedElements->AddReserveTriangles(InBoxCount * 2, GWhiteTexture, SE_BLEND_Translucent);
}

void FWorldPartitionDraw2DCanvas::Draw(const FWorldPartitionCanvasItems& Items)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionDraw2DCanvas::Draw);

	const FHitProxyId CanvasHitProxyId = Canvas->Canvas->GetHitProxyId();

	for (const FWorldPartitionCanvasBoxItem& Box : Items.Boxes)
	{
		int32 V0 = CanvasTriangleBatchedElements->AddVertexf(FVector4f(Box.Points[0].X, Box.Points[0].Y, 0, 1), FVector2f::ZeroVector, Box.Color, CanvasHitProxyId);
		int32 V1 = CanvasTriangleBatchedElements->AddVertexf(FVector4f(Box.Points[1].X, Box.Points[1].Y, 0, 1), FVector2f::ZeroVector, Box.Color, CanvasHitProxyId);
		int32 V2 = CanvasTriangleBatchedElements->AddVertexf(FVector4f(Box.Points[2].X, Box.Points[2].Y, 0, 1), FVector2f::ZeroVector, Box.Color, CanvasHitProxyId);
		int32 V3 = CanvasTriangleBatchedElements->AddVertexf(FVector4f(Box.Points[3].X, Box.Points[3].Y, 0, 1), FVector2f::ZeroVector, Box.Color, CanvasHitProxyId);
		CanvasTriangleBatchedElements->AddTriangle(V0, V1, V2, GWhiteTexture, SE_BLEND_Translucent);
		CanvasTriangleBatchedElements->AddTriangle(V2, V3, V0, GWhiteTexture, SE_BLEND_Translucent);
	}

	for (const FWorldPartitionLineCanvasItem& Line : Items.Lines)
	{
		CanvasLineBatchedElements->AddLine(Line.Start, Line.End, Line.Color, CanvasHitProxyId, Line.Thickness);
	}

	for (const FWorldPartitionCanvasMultiLineTextItem& Item : Items.MultiLineTexts)
	{
		FVector2D Position = Item.Position;
		for (const FWorldPartitionCanvasText& Text : Item.MultiLineText)
		{
			FWorldPartitionDebugHelper::DrawText(Canvas, Text.Key, GEngine->GetSmallFont(), Text.Value.ToFColor(true), Position);
		}
	}
}

static bool SegmentIntersection2D(const FVector2D& InSegmentStartA, const FVector2D& InSegmentEndA, const FVector2D& InSegmentStartB, const FVector2D& InSegmentEndB, FVector2D& OutIntersection)
{
	FVector Intersection;
	if (FMath::SegmentIntersection2D(FVector(InSegmentStartA, 0), FVector(InSegmentEndA, 0), FVector(InSegmentStartB, 0), FVector(InSegmentEndB, 0), Intersection))
	{
		OutIntersection = FVector2D(Intersection.X, Intersection.Y);
		return true;
	}
	return false;
}

bool FWorldPartitionDraw2DContext::BoxSegmentIntersect(const FBox2D& InBox, const FVector2D& InSegmentStart, const FVector2D& InSegmentEnd, TArray<FVector2D>& OutIntersections)
{
	if (InBox.IsInside(InSegmentStart))
	{
		OutIntersections.Add(InSegmentStart);
	}
	if (InBox.IsInside(InSegmentEnd))
	{
		OutIntersections.Add(InSegmentEnd);
	}
	if (OutIntersections.Num() == 2)
	{
		return true;
	}

	FVector2D BoxSize = InBox.GetSize();
	FVector2D BoxA = InBox.Min;
	FVector2D BoxB = InBox.Min + FVector2D(BoxSize.X, 0);
	FVector2D BoxC = InBox.Min + BoxSize;
	FVector2D BoxD = InBox.Min + FVector2D(0, BoxSize.Y);

	FVector2D Intersection;
	if (SegmentIntersection2D(InSegmentStart, InSegmentEnd, BoxA, BoxB, Intersection))
	{
		OutIntersections.Add(Intersection);
		if (OutIntersections.Num() == 2)
		{
			return true;
		}
	}
	if (SegmentIntersection2D(InSegmentStart, InSegmentEnd, BoxB, BoxC, Intersection))
	{
		OutIntersections.Add(Intersection);
		if (OutIntersections.Num() == 2)
		{
			return true;
		}
	}
	if (SegmentIntersection2D(InSegmentStart, InSegmentEnd, BoxC, BoxD, Intersection))
	{
		OutIntersections.Add(Intersection);
		if (OutIntersections.Num() == 2)
		{
			return true;
		}
	}
	if (SegmentIntersection2D(InSegmentStart, InSegmentEnd, BoxA, BoxD, Intersection))
	{
		OutIntersections.Add(Intersection);
		if (OutIntersections.Num() == 2)
		{
			return true;
		}
	}
	return false;
}