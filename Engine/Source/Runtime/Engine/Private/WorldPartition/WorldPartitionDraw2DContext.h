// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class UCanvas;
class FBatchedElements;
class FWorldPartitionCanvasItems;

using FWorldPartitionCanvasText = TPair<FString, FLinearColor>;
using FWorldPartitionCanvasMultiLineText = TArray<FWorldPartitionCanvasText>;

class FWorldPartitionCanvasMultiLineTextItem
{
public:
	FWorldPartitionCanvasMultiLineTextItem(const FVector2D& InPosition, const FWorldPartitionCanvasMultiLineText& InMultiLineText)
		: Position(InPosition)
		, MultiLineText(InMultiLineText)
	{
	}

	FVector2D Position;
	FWorldPartitionCanvasMultiLineText MultiLineText;
};

class FWorldPartitionLineCanvasItem
{
public:
	FWorldPartitionLineCanvasItem(const FVector2D& InStart, const FVector2D& InEnd, const FLinearColor& InColor = FLinearColor::White, float InThickness = 1)
		: Color(InColor)
		, Start(InStart, 0)
		, End(InEnd, 0)
		, Thickness(InThickness)
	{
	}
	FLinearColor Color;
	FVector Start;
	FVector End;
	float Thickness;
};

class FWorldPartitionCanvasBoxItem
{
public:
	FWorldPartitionCanvasBoxItem(const FVector2D& InPointA, const FVector2D& InPointB, const FVector2D& InPointC, const FVector2D& InPointD, const FLinearColor& InColor = FLinearColor::White)
		: Color(InColor)
	{
		Points[0] = InPointA;
		Points[1] = InPointB;
		Points[2] = InPointC;
		Points[3] = InPointD;
	};

	FLinearColor Color;
	FVector2D Points[4];
};

class FWorldPartitionDraw2DCanvas
{
public:
	FWorldPartitionDraw2DCanvas(UCanvas* InCanvas);
	void PrepareDraw(int32 InLineCount, int32 InBoxCount);
	void Draw(const FWorldPartitionCanvasItems& Items);

private:
	UCanvas* Canvas;
	FBatchedElements* CanvasLineBatchedElements;
	FBatchedElements* CanvasTriangleBatchedElements;
};

class FWorldPartitionCanvasItems
{
public:
	FWorldPartitionCanvasItems()
	{
		Lines.Reserve(1024);
		Boxes.Reserve(1024);
		MultiLineTexts.Reserve(128);
	}

	void AddLine(FWorldPartitionLineCanvasItem& InLine)
	{
		Lines.Add(MoveTemp(InLine));
	}

	void AddBox(FWorldPartitionCanvasBoxItem& InBox)
	{
		Boxes.Add(MoveTemp(InBox));
	}

	void AddText(FWorldPartitionCanvasMultiLineTextItem& InMultiLineText)
	{
		MultiLineTexts.Add(MoveTemp(InMultiLineText));
	}

	void Reset()
	{
		Lines.Reset();
		Boxes.Reset();
		MultiLineTexts.Reset();
	}

	int32 GetLineCount() const { return Lines.Num(); }
	int32 GetBoxCount() const { return Boxes.Num(); }

	void Draw(FWorldPartitionDraw2DCanvas& InCanvas);

private:
	TArray<FWorldPartitionLineCanvasItem> Lines;
	TArray<FWorldPartitionCanvasBoxItem> Boxes;
	TArray<FWorldPartitionCanvasMultiLineTextItem> MultiLineTexts;

	friend class FWorldPartitionDraw2DCanvas;
};

class FWorldPartitionDraw2DContext
{
public:
	FWorldPartitionDraw2DContext()
	{
		Initialize(FBox2D(ForceInit), FBox2D(ForceInit));
	}

	void Initialize(const FBox2D& InCanvasRegion, const FBox2D& InWorldRegion)
	{
		CanvasRegion = InCanvasRegion;
		WorldRegion = InWorldRegion;
		bDrawGridBounds = false;
		bDrawGridAxis = false;
		bIsDetailedMode = false;
		UsedCanvasBounds = FBox2D(ForceInit);
		DesiredWorldBounds = FBox2D(ForceInit);
		CanvasItems.Reset();
	}

	void SetIsDetailedMode(bool bValue) { bIsDetailedMode = bValue; }
	bool IsDetailedMode() const { return bIsDetailedMode; }

	void SetDrawGridBounds(bool bValue) { bDrawGridBounds = bValue; }
	bool GetDrawGridBounds() const { return bDrawGridBounds; }

	void SetDrawGridAxis(bool bValue) { bDrawGridAxis = bValue; }
	bool GetDrawGridAxis() const { return bDrawGridAxis; }

	void SetUsedCanvasBounds(const FBox2D& InUsedCanvasBounds) { UsedCanvasBounds = InUsedCanvasBounds; }
	const FBox2D& GetUsedCanvasBounds() const { return UsedCanvasBounds; }

	void SetDesiredWorldBounds(const FBox2D& InDesiredWorldBounds) { DesiredWorldBounds = InDesiredWorldBounds; }
	const FBox2D& GetDesiredWorldBounds() const { return DesiredWorldBounds; }

	const FBox2D& GetCanvasRegion() const { check(CanvasRegion.bIsValid); return CanvasRegion; }
	const FBox2D& GetWorldRegion() const { check(WorldRegion.bIsValid); return WorldRegion; }
	int32 GetLineCount() const { return CanvasItems.GetLineCount(); }
	int32 GetBoxCount() const { return CanvasItems.GetBoxCount(); }

	bool PushDrawSegment(const FBox2D& InBounds, const FVector2D& A, const FVector2D& B, const FLinearColor& InColor, float InLineThickness)
	{
		TArray<FVector2D> Intersections;
		if (BoxSegmentIntersect(InBounds, A, B, Intersections))
		{
			FWorldPartitionLineCanvasItem Line(Intersections[0], Intersections[1], InColor, InLineThickness);
			CanvasItems.AddLine(Line);
			return true;
		}
		return false;
	};

	bool PushDrawBox(const FBox2D& InBounds, const FVector2D& A, const FVector2D& B, const FVector2D& C, const FVector2D& D, const FLinearColor& InColor, float InLineThickness)
	{
		PushDrawSegment(InBounds, A, B, InColor, InLineThickness);
		PushDrawSegment(InBounds, B, C, InColor, InLineThickness);
		PushDrawSegment(InBounds, C, D, InColor, InLineThickness);
		PushDrawSegment(InBounds, A, D, InColor, InLineThickness);
		return true;
	};

	bool PushDrawTile(const FBox2D& InBounds, const FVector2D& A, const FVector2D& B, const FVector2D& C, const FVector2D& D, const FLinearColor& InColor)
	{
		const FBox2D ClipBox = InBounds.Overlap(FBox2D({ A, B, C, D}));
		if (ClipBox.bIsValid)
		{
			FWorldPartitionCanvasBoxItem Box(ClipBox.Min, FVector2D(ClipBox.Max.X, ClipBox.Min.Y), ClipBox.Max, FVector2D(ClipBox.Min.X, ClipBox.Max.Y), InColor);
			CanvasItems.AddBox(Box);
			return true;
		}
		return false;
	};

	bool PushDrawText(FWorldPartitionCanvasMultiLineTextItem& InMultiLineText)
	{
		CanvasItems.AddText(InMultiLineText);
		return true;
	}

	template <class WorldToScreenFunc>
	void LocalDrawTile(const FBox2D& GridScreenBounds, const FVector2D& Min, const FVector2D& Size, const FLinearColor& Color, WorldToScreenFunc WorldToScreen)
	{
		FVector2D A = WorldToScreen(Min);
		FVector2D B = WorldToScreen(Min + FVector2D(Size.X, 0));
		FVector2D C = WorldToScreen(Min + Size);
		FVector2D D = WorldToScreen(Min + FVector2D(0, Size.Y));
		PushDrawTile(GridScreenBounds, A, B, C, D, Color);
	};

	template <class WorldToScreenFunc>
	void LocalDrawBox(const FBox2D& GridScreenBounds, const FVector2D& Min, const FVector2D& Size, const FLinearColor& Color, float LineThickness, WorldToScreenFunc WorldToScreen)
	{
		FVector2D A = WorldToScreen(Min);
		FVector2D B = WorldToScreen(Min + FVector2D(Size.X, 0));
		FVector2D C = WorldToScreen(Min + Size);
		FVector2D D = WorldToScreen(Min + FVector2D(0, Size.Y));
		PushDrawBox(GridScreenBounds, A, B, C, D, Color, LineThickness);
	};

	template <class WorldToScreenFunc>
	void LocalDrawSegment(const FBox2D& GridScreenBounds, const FVector2D& Start, const FVector2D& End, const FLinearColor& Color, float LineThickness, WorldToScreenFunc WorldToScreen)
	{
		FVector2D A = WorldToScreen(Start);
		FVector2D B = WorldToScreen(End);
		PushDrawSegment(GridScreenBounds, A, B, Color, LineThickness);
	};

	const FWorldPartitionCanvasItems& GetCanvasItems() const { return CanvasItems; }

private:

	bool BoxSegmentIntersect(const FBox2D& Box, const FVector2D& a1, const FVector2D& a2, TArray<FVector2D>& Intersections);

	bool bDrawGridBounds;
	bool bDrawGridAxis;
	bool bIsDetailedMode;
	FBox2D CanvasRegion;
	FBox2D WorldRegion;
	FBox2D UsedCanvasBounds;
	FBox2D DesiredWorldBounds;
	FWorldPartitionCanvasItems CanvasItems;
};