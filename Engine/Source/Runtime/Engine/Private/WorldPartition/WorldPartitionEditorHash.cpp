// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionEditorHash.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionEditorHash)

UWorldPartitionEditorHash::UWorldPartitionEditorHash(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR
UWorldPartitionEditorHash::FForEachIntersectingActorParams::FForEachIntersectingActorParams()
	: bIncludeSpatiallyLoadedActors(true)
	, bIncludeNonSpatiallyLoadedActors(true)
{}
#endif
