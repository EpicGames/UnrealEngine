// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "Algo/Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionEditorPerProjectUserSettings)

#if WITH_EDITOR

void UWorldPartitionEditorPerProjectUserSettings::SetWorldDataLayersNonDefaultEditorLoadStates(UWorld* InWorld, const TArray<const UDataLayerInstance*>& InDataLayersLoadedInEditor, const TArray<const UDataLayerInstance*>& InDataLayersNotLoadedInEditor)
{
	if (ShouldSaveSettings(InWorld))
	{
		FWorldPartitionPerWorldSettings& PerWorldSettings = PerWorldEditorSettings.FindOrAdd(TSoftObjectPtr<UWorld>(InWorld));
		PerWorldSettings.LoadedDataLayers.Reset(InDataLayersLoadedInEditor.Num());
		PerWorldSettings.NotLoadedDataLayers.Reset(InDataLayersNotLoadedInEditor.Num());
		auto IsValidDataLayerInstance = [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance && DataLayerInstance->GetAsset(); };
		auto GetDataLayerInstanceAsset = [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetAsset(); };
		Algo::TransformIf(InDataLayersLoadedInEditor, PerWorldSettings.LoadedDataLayers, IsValidDataLayerInstance, GetDataLayerInstanceAsset);
		Algo::TransformIf(InDataLayersNotLoadedInEditor, PerWorldSettings.NotLoadedDataLayers, IsValidDataLayerInstance, GetDataLayerInstanceAsset);
		SaveConfig();
	}
}

void UWorldPartitionEditorPerProjectUserSettings::SetEditorLoadedRegions(UWorld* InWorld, const TArray<FBox>& InEditorLoadedRegions)
{
	if (ShouldSaveSettings(InWorld))
	{
		FWorldPartitionPerWorldSettings& PerWorldSettings = PerWorldEditorSettings.FindOrAdd(TSoftObjectPtr<UWorld>(InWorld));
		PerWorldSettings.LoadedEditorRegions.Empty();
		Algo::TransformIf(InEditorLoadedRegions, PerWorldSettings.LoadedEditorRegions, [](const FBox& InBox) { return InBox.IsValid; }, [](const FBox& InBox) { return InBox; });		
		SaveConfig();
	}
}

TArray<FBox> UWorldPartitionEditorPerProjectUserSettings::GetEditorLoadedRegions(UWorld* InWorld) const
{
	if (const FWorldPartitionPerWorldSettings* PerWorldSettings = GetWorldPartitionPerWorldSettings(InWorld))
	{
		return PerWorldSettings->LoadedEditorRegions;
	}

	return TArray<FBox>();
}

void UWorldPartitionEditorPerProjectUserSettings::SetEditorLoadedLocationVolumes(UWorld* InWorld, const TArray<FName>& InEditorLoadedLocationVolumes)
{
	if (ShouldSaveSettings(InWorld))
	{
		FWorldPartitionPerWorldSettings& PerWorldSettings = PerWorldEditorSettings.FindOrAdd(TSoftObjectPtr<UWorld>(InWorld));
		PerWorldSettings.LoadedEditorLocationVolumes = InEditorLoadedLocationVolumes;
		
		SaveConfig();
	}
}

TArray<FName> UWorldPartitionEditorPerProjectUserSettings::GetEditorLoadedLocationVolumes(UWorld* InWorld) const
{
	if (const FWorldPartitionPerWorldSettings* PerWorldSettings = GetWorldPartitionPerWorldSettings(InWorld))
	{
		return PerWorldSettings->LoadedEditorLocationVolumes;
	}

	return TArray<FName>();
}

TArray<const UDataLayerAsset*> UWorldPartitionEditorPerProjectUserSettings::GetWorldDataLayerAssetsNotLoadedInEditor(UWorld* InWorld) const
{
	TArray<const UDataLayerAsset*> NotLoadedDataLayerAssets;

	if (const FWorldPartitionPerWorldSettings* PerWorldSettings = GetWorldPartitionPerWorldSettings(InWorld))
	{
		for (const TSoftObjectPtr<const UDataLayerAsset>& SoftDataLayerAsset : PerWorldSettings->NotLoadedDataLayers)
		{
			if (const UDataLayerAsset* DataLayerAsset = SoftDataLayerAsset.Get())
			{
				NotLoadedDataLayerAssets.Emplace(DataLayerAsset);
			}
		}
	}

	return NotLoadedDataLayerAssets;
}

TArray<UDataLayerAsset*> UWorldPartitionEditorPerProjectUserSettings::GetWorldDataLayerAssetsNotLoadedInEditor(UWorld* InWorld)
{
	TArray<const UDataLayerAsset*> NotLoadedDataLayerAssets = AsConst(*this).GetWorldDataLayerAssetsNotLoadedInEditor(InWorld);
	TArray<UDataLayerAsset*> NotLoadedDataLayerAssetsMutable;
	Algo::Transform(NotLoadedDataLayerAssets, NotLoadedDataLayerAssetsMutable, [](const UDataLayerAsset* DataLayerAsset) { return const_cast<UDataLayerAsset*>(DataLayerAsset); });
	return NotLoadedDataLayerAssetsMutable;
}

TArray<const UDataLayerAsset*> UWorldPartitionEditorPerProjectUserSettings::GetWorldDataLayerAssetsLoadedInEditor(UWorld* InWorld) const
{
	TArray<const UDataLayerAsset*> LoadedDataLayerAssets;

	if (const FWorldPartitionPerWorldSettings* PerWorldSettings = GetWorldPartitionPerWorldSettings(InWorld))
	{
		for (const TSoftObjectPtr<const UDataLayerAsset>& SoftDataLayerAsset : PerWorldSettings->LoadedDataLayers)
		{
			if (const UDataLayerAsset* DataLayerAsset = SoftDataLayerAsset.Get())
			{
				LoadedDataLayerAssets.Emplace(DataLayerAsset);
			}
		}
	}

	return LoadedDataLayerAssets;
}

TArray<UDataLayerAsset*> UWorldPartitionEditorPerProjectUserSettings::GetWorldDataLayerAssetsLoadedInEditor(UWorld* InWorld)
{
	TArray<const UDataLayerAsset*> LoadedDataLayerAssets = AsConst(*this).GetWorldDataLayerAssetsLoadedInEditor(InWorld);
	TArray<UDataLayerAsset*> LoadedDataLayerAssetsMutable;
	Algo::Transform(LoadedDataLayerAssets, LoadedDataLayerAssetsMutable, [](const UDataLayerAsset* DataLayerAsset) { return const_cast<UDataLayerAsset*>(DataLayerAsset); });
	return LoadedDataLayerAssetsMutable;
}

const FWorldPartitionPerWorldSettings* UWorldPartitionEditorPerProjectUserSettings::GetWorldPartitionPerWorldSettings(UWorld* InWorld) const
{
	if (!ShouldLoadSettings(InWorld))
	{
		return nullptr;
	}

	if (const FWorldPartitionPerWorldSettings* ExistingPerWorldSettings = PerWorldEditorSettings.Find(TSoftObjectPtr<UWorld>(InWorld)))
	{
		return ExistingPerWorldSettings;
	}
	else if (const FWorldPartitionPerWorldSettings* DefaultPerWorldSettings = InWorld->GetWorldSettings()->GetDefaultWorldPartitionSettings())
	{
		return DefaultPerWorldSettings;
	}

	return nullptr;
}

#if WITH_EDITORONLY_DATA
bool UWorldPartitionEditorPerProjectUserSettings::GetForceDisableDynamicLoadingRangeScaling(UWorld* InWorld) const
{
	if (const FWorldPartitionPerWorldSettings* PerWorldSettings = GetWorldPartitionPerWorldSettings(InWorld))
	{
		return PerWorldSettings->bForceDisableDynamicLoadingRangeScaling;
	}
	return false;
}

void UWorldPartitionEditorPerProjectUserSettings::SetForceDisableDynamicLoadingRangeScaling(UWorld* InWorld, bool bOverride)
{
	if (ShouldSaveSettings(InWorld))
	{
		FWorldPartitionPerWorldSettings& PerWorldSettings = PerWorldEditorSettings.FindOrAdd(TSoftObjectPtr<UWorld>(InWorld));
		PerWorldSettings.bForceDisableDynamicLoadingRangeScaling = bOverride;
		SaveConfig();
	}
}
#endif // WITH_EDITORONLY_DATA

bool UWorldPartitionEditorPerProjectUserSettings::ShouldSaveSettings(const UWorld* InWorld) const
{
	return InWorld && !InWorld->IsGameWorld() && (InWorld->WorldType != EWorldType::Inactive) && FPackageName::DoesPackageExist(InWorld->GetPackage()->GetName());
}

bool UWorldPartitionEditorPerProjectUserSettings::ShouldLoadSettings(const UWorld* InWorld) const
{
	return InWorld && (InWorld->WorldType != EWorldType::Inactive);
}

#endif

