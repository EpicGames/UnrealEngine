// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionEditorSpatialHash.h"
#include "HAL/IConsoleManager.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionEditorSpatialHash)

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

#if WITH_EDITOR
TAutoConsoleVariable<bool> CVarEnableSpatialHashValidation(TEXT("wp.Editor.EnableSpatialHashValidation"), false, TEXT("Whether to enable World Partition editor spatial hash validation"), ECVF_Default);
#endif

UWorldPartitionEditorSpatialHash::UWorldPartitionEditorSpatialHash(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, CellSize(12800)
	, EditorBounds(ForceInit)
	, RuntimeBounds(ForceInit)
	, NonSpatialBounds(ForceInit)
	, bBoundsDirty(false)
#endif
{}

#if WITH_EDITOR
void UWorldPartitionEditorSpatialHash::Initialize()
{
	check(!AlwaysLoadedCell);

	AlwaysLoadedCell = MakeUnique<FCell>();
	AlwaysLoadedCell->Bounds.Init();
}

void UWorldPartitionEditorSpatialHash::SetDefaultValues()
{}

FName UWorldPartitionEditorSpatialHash::GetWorldPartitionEditorName() const
{
	return TEXT("SpatialHash");
}

FBox UWorldPartitionEditorSpatialHash::GetEditorWorldBounds() const
{
	return EditorBounds;
}

FBox UWorldPartitionEditorSpatialHash::GetRuntimeWorldBounds() const
{
	return RuntimeBounds;
}

FBox UWorldPartitionEditorSpatialHash::GetNonSpatialBounds() const
{
	return NonSpatialBounds;
}

void UWorldPartitionEditorSpatialHash::Tick(float DeltaSeconds)
{
	if (bBoundsDirty)
	{
		FBox NewEditorBounds(ForceInit);

		RuntimeBounds.Init();
		for (FCell* Cell: Cells)
		{
			NewEditorBounds += Cell->Bounds;

			for (const FWorldPartitionHandle& ActorHandle : Cell->Actors)
			{
				RuntimeBounds += ActorHandle->GetLocalEditorBounds();
			}
		}

		NonSpatialBounds.Init();
		for (const FWorldPartitionHandle& ActorHandle : AlwaysLoadedCell->Actors)
		{
			NonSpatialBounds += ActorHandle->GetLocalEditorBounds();
		}

		const int32 OldLevel = GetLevelForBox(EditorBounds);
		check(!EditorBounds.IsValid || OldLevel == HashLevels.Num() - 1);

		const int32 NewLevel = GetLevelForBox(NewEditorBounds);
		check(NewLevel <= OldLevel);		

		if (NewLevel < OldLevel)
		{
			HashLevels.SetNum(NewLevel + 1);
		}

		EditorBounds = NewEditorBounds;
		
		bBoundsDirty = false;
	}

	if (CVarEnableSpatialHashValidation.GetValueOnAnyThread())
	{
		if (EditorBounds.IsValid)
		{
			const int32 CurrentLevel = GetLevelForBox(EditorBounds);
			check(CurrentLevel == HashLevels.Num() - 1);

			for (int32 HashLevel = 0; HashLevel < HashLevels.Num() - 1; HashLevel++)
			{
				for (auto& HashLevelPair : HashLevels[HashLevel])
				{
					const FCellCoord CellCoord = HashLevelPair.Key;
					check(CellCoord.Level == HashLevel);

					const uint32 ChildIndex = CellCoord.GetChildIndex();
					const FCellCoord ParentCellCoord = CellCoord.GetParentCellCoord();
					check(ParentCellCoord.Level == HashLevel + 1);

					const FCellNodeElement& ParentCellNodeElement = HashLevels[ParentCellCoord.Level].FindChecked(ParentCellCoord);
					const FCellNode& ParentCellNode = ParentCellNodeElement.Key;
					check(ParentCellNode.HasChildNode(ChildIndex));
				}
			}
		}
	}
}

void UWorldPartitionEditorSpatialHash::HashActor(FWorldPartitionHandle& InActorHandle)
{
	check(InActorHandle.IsValid());

	const FWorldPartitionActorDescInstance* ActorDescInstance = *InActorHandle;
	const bool bConsiderActorSpatiallyLoaded = ActorDescInstance->GetIsSpatiallyLoaded();
	const FBox ActorBounds = bConsiderActorSpatiallyLoaded ? ActorDescInstance->GetLocalEditorBounds() : FBox(ForceInit);

#if DO_CHECK
	check(!HashedActors.Contains(ActorDescInstance->GetGuid()));
	HashedActors.Add(ActorDescInstance->GetGuid(), ActorBounds);
#endif

	if (!bConsiderActorSpatiallyLoaded)
	{
		AlwaysLoadedCell->Actors.Add(InActorHandle);
		NonSpatialBounds += ActorDescInstance->GetLocalEditorBounds();
	}
	else
	{
		const int32 CurrentLevel = GetLevelForBox(EditorBounds);
		const int32 ActorLevel = GetLevelForBox(ActorBounds);

		if (HashLevels.Num() <= ActorLevel)
		{
			HashLevels.AddDefaulted(ActorLevel - HashLevels.Num() + 1);
		}

		ForEachIntersectingCells(ActorBounds, ActorLevel, [&](const FCellCoord& CellCoord)
		{
			FCellNodeElement& CellNodeElement = HashLevels[CellCoord.Level].FindOrAdd(CellCoord);

			TUniquePtr<FCell>& Cell = CellNodeElement.Value;

			if (!Cell)
			{
				Cell = MakeUnique<FCell>();
				Cell->Bounds = GetCellBounds(CellCoord);

				Cells.Add(Cell.Get());

				// Increment spatial structure bounds
				EditorBounds += Cell->Bounds;

				// Update parent nodes
				FCellCoord CurrentCellCoord = CellCoord;
				while (CurrentCellCoord.Level < CurrentLevel)
				{
					const uint32 ChildIndex = CurrentCellCoord.GetChildIndex();
					CurrentCellCoord = CurrentCellCoord.GetParentCellCoord();

					FCellNodeElement& ParentCellNodeElement = HashLevels[CurrentCellCoord.Level].FindOrAdd(CurrentCellCoord);
					FCellNode& ParentCellNode = ParentCellNodeElement.Key;

					if (ParentCellNode.HasChildNode(ChildIndex))
					{
						break;
					}

					ParentCellNode.AddChildNode(ChildIndex);
				}
			}

			check(Cell);
			Cell->Actors.Add(InActorHandle);
		});

		const int32 NewLevel = GetLevelForBox(EditorBounds);
		check(NewLevel >= CurrentLevel);

		if (NewLevel > CurrentLevel)
		{
			if (HashLevels.Num() <= NewLevel)
			{
				HashLevels.AddDefaulted(NewLevel - HashLevels.Num() + 1);
			}

			for (int32 Level = CurrentLevel; Level < NewLevel; Level++)
			{
				for (auto& HashLevelPair : HashLevels[Level])
				{
					FCellCoord LevelCellCoord = HashLevelPair.Key;
					while (LevelCellCoord.Level < NewLevel)
					{
						const uint32 ChildIndex = LevelCellCoord.GetChildIndex();

						LevelCellCoord = LevelCellCoord.GetParentCellCoord();

						FCellNodeElement& CellNodeElement = HashLevels[LevelCellCoord.Level].FindOrAdd(LevelCellCoord);
						FCellNode& CellNode = CellNodeElement.Key;

						// We can break updating when aggregated flags are already properly set for parent nodes
						const bool bShouldBreak = CellNode.HasChildNodes();

						// Propagate the child mask
						if (!CellNode.HasChildNode(ChildIndex))
						{
							CellNode.AddChildNode(ChildIndex);
						}				
				
						if (bShouldBreak)
						{
							break;
						}
					}
				}
			}
		}

		RuntimeBounds += ActorDescInstance->GetLocalEditorBounds();
	}
}

void UWorldPartitionEditorSpatialHash::UnhashActor(FWorldPartitionHandle& InActorHandle)
{
	check(InActorHandle.IsValid());

	const FWorldPartitionActorDescInstance* ActorDescInstance = *InActorHandle;
	const bool bConsiderActorSpatiallyLoaded = ActorDescInstance->GetIsSpatiallyLoaded();
	const FBox ActorBounds = bConsiderActorSpatiallyLoaded ? ActorDescInstance->GetLocalEditorBounds() : FBox(ForceInit);

#if DO_CHECK
	const FBox OldActorBounds = HashedActors.FindAndRemoveChecked(ActorDescInstance->GetGuid());
	check(ActorBounds == OldActorBounds);
#endif

	if (!bConsiderActorSpatiallyLoaded)
	{
		AlwaysLoadedCell->Actors.Remove(InActorHandle);
	}
	else
	{
		const int32 CurrentLevel = GetLevelForBox(EditorBounds);
		const int32 ActorLevel = GetLevelForBox(ActorBounds);

		ForEachIntersectingCells(ActorBounds, ActorLevel, [&](const FCellCoord& CellCoord)
		{
			FCellNodeElement& CellNodeElement = HashLevels[CellCoord.Level].FindChecked(CellCoord);
			TUniquePtr<FCell>& Cell = CellNodeElement.Value;
			check(Cell);

			Cell->Actors.Remove(InActorHandle);

			if (!Cell->Actors.Num())
			{
				verify(Cells.Remove(Cell.Get()));
				CellNodeElement.Value.Reset();

				if (!CellNodeElement.Key.HasChildNodes())
				{
					FCellCoord CurrentCellCoord = CellCoord;
					while (CurrentCellCoord.Level < CurrentLevel)
					{
						FCellCoord ParentCellCoord = CurrentCellCoord.GetParentCellCoord();
						FCellNodeElement& ParentCellNodeElement = HashLevels[ParentCellCoord.Level].FindChecked(ParentCellCoord);
						FCellNode& ParentCellNode = ParentCellNodeElement.Key;

						const uint32 ChildIndex = CurrentCellCoord.GetChildIndex();
						ParentCellNode.RemoveChildNode(ChildIndex);

						HashLevels[CurrentCellCoord.Level].Remove(CurrentCellCoord);

						if (ParentCellNodeElement.Value || ParentCellNodeElement.Key.HasChildNodes())
						{
							break;
						}

						CurrentCellCoord = ParentCellCoord;
					}
				}
			}
		});
	}

	bBoundsDirty = true;
}

int32 UWorldPartitionEditorSpatialHash::ForEachIntersectingActor(const FBox& Box, TFunctionRef<void(FWorldPartitionActorDescInstance*)> InOperation, const FForEachIntersectingActorParams& Params)
{
	TSet<FGuid> IntersectedActors;

	const int MininumumLevel = Params.MinimumBox.IsSet() ? GetLevelForBox(*Params.MinimumBox) : 0;

	if (Params.bIncludeSpatiallyLoadedActors)
	{
		ForEachIntersectingCell(Box, [&](FCell* Cell)
		{
			for(FWorldPartitionHandle& ActorHandle: Cell->Actors)
			{
				if (ActorHandle.IsValid())
				{
					bool bWasAlreadyInSet;
					IntersectedActors.Add(ActorHandle->GetGuid(), &bWasAlreadyInSet);

					if (!bWasAlreadyInSet)
					{
						if (Box.Intersect(ActorHandle->GetLocalEditorBounds()))
						{
							InOperation(*ActorHandle);
						}
					}
				}
			}
		}, MininumumLevel);
	}

	if (Params.bIncludeNonSpatiallyLoadedActors)
	{
		for(FWorldPartitionHandle& ActorHandle : AlwaysLoadedCell->Actors)
		{
			if (ActorHandle.IsValid())
			{
				if (Box.Intersect(ActorHandle->GetLocalEditorBounds()))
				{
					bool bWasAlreadyInSet;
					IntersectedActors.Add(ActorHandle->GetGuid(), &bWasAlreadyInSet);
				
					if (!bWasAlreadyInSet)
					{
						InOperation(*ActorHandle);
					}
				}
			}
		}
	}

	return IntersectedActors.Num();
}

int32 UWorldPartitionEditorSpatialHash::ForEachIntersectingCellInner(const FBox& Box, const FCellCoord& CellCoord, TFunctionRef<void(FCell*)> InOperation, int32 MinimumLevel)
{
	int32 NumIntersecting = 0;

	if (const FCellNodeElement* CellNodeElement = HashLevels[CellCoord.Level].Find(CellCoord))
	{
		if (CellNodeElement->Value)
		{
			InOperation(CellNodeElement->Value.Get());
			NumIntersecting++;
		}

		if (MinimumLevel <= CellCoord.Level)
		{
			CellNodeElement->Key.ForEachChild([&](uint32 ChildIndex)
			{
				const FCellCoord ChildCellCoord = CellCoord.GetChildCellCoord(ChildIndex);
				const FBox CellBounds = GetCellBounds(ChildCellCoord);

				if (Box.Intersect(CellBounds))
				{
					NumIntersecting += ForEachIntersectingCellInner(Box, ChildCellCoord, InOperation, MinimumLevel);
				}
			});
		}
	}

	return NumIntersecting;
}

int32 UWorldPartitionEditorSpatialHash::ForEachIntersectingCell(const FBox& Box, TFunctionRef<void(FCell*)> InOperation, int32 MinimumLevel)
{
	int32 NumIntersecting = 0;

	if (HashLevels.Num())
	{
		const FBox SearchBox = Box.Overlap(EditorBounds);

		if (SearchBox.IsValid)
		{
			ForEachIntersectingCells(SearchBox, HashLevels.Num() - 1, [&](const FCellCoord& CellCoord)
			{
				NumIntersecting += ForEachIntersectingCellInner(Box, CellCoord, InOperation, MinimumLevel);
			});
		}
	}

	return NumIntersecting;
}
#endif

#undef LOCTEXT_NAMESPACE

