// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionHandle.h"

#include "Engine/Level.h"
#include "Engine/World.h"

#if WITH_EDITOR

#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Templates/GuardValueAccessors.h"

/**
* FWorldPartitionImplBase
*/
UActorDescContainerInstance* FWorldPartitionImplBase::GetActorDescContainerInstance(TUniquePtr<FWorldPartitionActorDescInstance>* InActorDescInstance)
{
	return InActorDescInstance ? InActorDescInstance->Get()->GetContainerInstance() : nullptr;
}

UActorDescContainerInstance* FWorldPartitionImplBase::GetActorDescContainerInstance(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance ? InActorDescInstance->GetContainerInstance() : nullptr;
}

FGuid FWorldPartitionImplBase::GetActorDescInstanceGuid(const FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance ? InActorDescInstance->GetGuid() : FGuid();
}

TUniquePtr<FWorldPartitionActorDescInstance>* FWorldPartitionImplBase::GetActorDescInstance(UActorDescContainerInstance* InContainerInstance, const FGuid& InActorGuid)
{
	return InContainerInstance ? InContainerInstance->GetActorDescInstancePtr(InActorGuid) : nullptr;
}

FWorldPartitionActorDesc* FWorldPartitionImplBase::GetActorDesc(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance ? const_cast<FWorldPartitionActorDesc*>(InActorDescInstance->GetActorDesc()) : nullptr;
}

bool FWorldPartitionImplBase::IsLoaded(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance->IsLoaded();
}

AActor* FWorldPartitionImplBase::GetActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return InActorDescInstance->GetActor();
}

/**
* FWorldPartitionLoadingContext
*/
FWorldPartitionLoadingContext::FImmediate FWorldPartitionLoadingContext::DefaultContext;
FWorldPartitionLoadingContext::IContext* FWorldPartitionLoadingContext::ActiveContext = &DefaultContext;

bool FWorldPartitionLoadingContext::IsDeferringRawObjects()
{
	return ActiveContext->IsDeferringRawObjects();
}

void FWorldPartitionLoadingContext::CancelOperationIfAny(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (!InActorDescInstance->bIsRegisteringOrUnregistering)
	{
		return;
	}

	ActiveContext->CancelOperation(InActorDescInstance);
}

void FWorldPartitionLoadingContext::LoadAndRegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	check(!InActorDescInstance->bIsRegisteringOrUnregistering);
	InActorDescInstance->bIsRegisteringOrUnregistering = true;

	ActiveContext->RegisterActor(InActorDescInstance);
}

void FWorldPartitionLoadingContext::UnloadAndUnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	check(!InActorDescInstance->bIsRegisteringOrUnregistering);
	InActorDescInstance->bIsRegisteringOrUnregistering = true;

	ActiveContext->UnregisterActor(InActorDescInstance);
}

/**
* IContext
*/
FWorldPartitionLoadingContext::IContext::IContext()
{
	if (ActiveContext == &DefaultContext)
	{
		ActiveContext = this;
	}
}

FWorldPartitionLoadingContext::IContext::~IContext()
{
	if (ActiveContext == this)
	{
		ActiveContext = &DefaultContext;
	}
}

/**
* FImmediate
*/
void FWorldPartitionLoadingContext::FImmediate::RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	// Set GIsEditorLoadingPackage to avoid dirtying the Actor package if Modify() is called during the load sequence
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

	if (InActorDescInstance->StartAsyncLoad())
	{
		UActorDescContainerInstance* Container = InActorDescInstance->GetContainerInstance();
		check(Container);

		const FTransform& ContainerTransform = Container->GetTransform();
		const FTransform* ContainerTransformPtr = ContainerTransform.Equals(FTransform::Identity) ? nullptr : &ContainerTransform;

		if (AActor* Actor = InActorDescInstance->GetActor())
		{
			Actor->GetLevel()->AddLoadedActor(Actor, ContainerTransformPtr);
		}
	}

	check(InActorDescInstance->bIsRegisteringOrUnregistering);
	InActorDescInstance->bIsRegisteringOrUnregistering = false;
}

void FWorldPartitionLoadingContext::FImmediate::UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	// Set GIsEditorLoadingPackage to avoid dirtying the Actor package if Modify() is called during the unload sequence
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

	// When cleaning up worlds, actors are already marked as garbage at this point, so no need to remove them from the world
	if (AActor* Actor = InActorDescInstance->GetActor(); IsValid(Actor) && Actor->GetLevel())
	{
		UActorDescContainerInstance* Container = InActorDescInstance->GetContainerInstance();
		check(Container);

		const FTransform& ContainerTransform = Container->GetTransform();
		const FTransform* ContainerTransformPtr = ContainerTransform.Equals(FTransform::Identity) ? nullptr : &ContainerTransform;

		Actor->GetLevel()->RemoveLoadedActor(Actor, ContainerTransformPtr);

		InActorDescInstance->MarkUnload();
	}

	check(InActorDescInstance->bIsRegisteringOrUnregistering);
	InActorDescInstance->bIsRegisteringOrUnregistering = false;
}

/**
* FDeferred
*/
FWorldPartitionLoadingContext::FDeferred::~FDeferred()
{
	// There shouldn't be any cancellations when starting, as they should only get generated when ContainerInstanceOps does not contain the Op
	// because the container is moved on the stack here. In other words, ContainerInstanceCancellations can only grow from within this destructor.
	check(ContainerInstanceCancellations.IsEmpty()); 									 

	TMap<UActorDescContainerInstance*, FContainerInstanceOps> LocalContainerInstanceOps = MoveTemp(ContainerInstanceOps);

	while (LocalContainerInstanceOps.Num())
	{
		for (auto& [Container, ContainerOp] : LocalContainerInstanceOps)
		{
			const FTransform& ContainerTransform = Container->GetTransform();
			const FTransform* ContainerTransformPtr = ContainerTransform.Equals(FTransform::Identity) ? nullptr : &ContainerTransform;

			auto CreateActorList = [this](TArray<AActor*>& ActorList, TSet<FWorldPartitionActorDescInstance*>& SourceList) -> ULevel*
			{
				ActorList.Empty(SourceList.Num());

				TArray<int32> AllAsyncLoadIds;
				AllAsyncLoadIds.Reserve(SourceList.Num());
				for (FWorldPartitionActorDescInstance* ActorDescInstance : SourceList)
				{
					if (int32 AsyncLoadId = ActorDescInstance->GetAsyncLoadID(); AsyncLoadId != INDEX_NONE)
					{
						AllAsyncLoadIds.Add(AsyncLoadId);
					}
				}
				
				// Only flush if there are items in the array. An empty array will result in a complete flush which is not the intended result.
				if (AllAsyncLoadIds.Num() > 0)
				{
					FlushAsyncLoading(AllAsyncLoadIds);

					// After a flush, Actor Instances in AllAsyncLoadIds may have been reinstanced, which means their FWorldPartitionActorDescInstance pointer is now deleted and unsafe to dereference. Apply those.
					for (FWorldPartitionActorDescInstance* CancelledActorDescInst : ContainerInstanceCancellations)
					{
						SourceList.Remove(CancelledActorDescInst);
					}
					ContainerInstanceCancellations.Reset();
				}

				ULevel* Level = nullptr;
				for (FWorldPartitionActorDescInstance* ActorDescInstance : SourceList)
				{
					// For async loads, the actor loading might have failed for several reasons, this will be handled here.
					// Also, when cleaning up worlds, actors are already marked as garbage at this point, so no need to remove them from the world.
					if (AActor* Actor = ActorDescInstance->GetActor(); IsValid(Actor))
					{
						ActorList.Add(Actor);

						ULevel* ActorLevel = Actor->GetLevel();
						check(!Level || (Level == ActorLevel));
						Level = ActorLevel;
					}
				}

				check(ContainerInstanceCancellations.IsEmpty()); // Verify that cancellations haven't been generated outside the FlushAsyncLoading block

				return Level;
			};

			if (ContainerOp.Registrations.Num())
			{
				// Set GIsEditorLoadingPackage to avoid dirtying the Actor package if Modify() is called during the load sequence
				TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

				TArray<AActor*> ActorList;
				if (ULevel* Level = CreateActorList(ActorList, ContainerOp.Registrations))
				{
					Level->AddLoadedActors(ActorList, ContainerTransformPtr);
				}

				for (FWorldPartitionActorDescInstance* ActorDescInstance : ContainerOp.Registrations)
				{
					check(ActorDescInstance->bIsRegisteringOrUnregistering);
					ActorDescInstance->bIsRegisteringOrUnregistering = false;
				}
			}

			if (ContainerOp.Unregistrations.Num())
			{
				// Set GIsEditorLoadingPackage to avoid dirtying the Actor package if Modify() is called during the unload sequence
				TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

				TArray<AActor*> ActorList;
				if (ULevel* Level = CreateActorList(ActorList, ContainerOp.Unregistrations))
				{
					Level->RemoveLoadedActors(ActorList, ContainerTransformPtr);
				}

				for (FWorldPartitionActorDescInstance* ActorDescInstance : ContainerOp.Unregistrations)
				{
					ActorDescInstance->MarkUnload();
					check(ActorDescInstance->bIsRegisteringOrUnregistering);
					ActorDescInstance->bIsRegisteringOrUnregistering = false;
				}
			}
		}

		// Continue with potentially new registrations/unregistrations that may have happenned during previous cycle
		LocalContainerInstanceOps = MoveTemp(ContainerInstanceOps);
	}
}

void FWorldPartitionLoadingContext::FDeferred::RegisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

	check(InActorDescInstance);
	if (InActorDescInstance->StartAsyncLoad())
	{
		UActorDescContainerInstance* Container = InActorDescInstance->GetContainerInstance();
		check(Container);

		bool bIsAlreadyInSetPtr;
		ContainerInstanceOps.FindOrAdd(Container).Registrations.Add(InActorDescInstance, &bIsAlreadyInSetPtr);
		check(!bIsAlreadyInSetPtr);
		NumRegistrations++;

		check(!ContainerInstanceOps.FindChecked(Container).Unregistrations.Contains(InActorDescInstance));
	}
}

void FWorldPartitionLoadingContext::FDeferred::UnregisterActor(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	check(InActorDescInstance);
	if (AActor* Actor = InActorDescInstance->GetActor(); IsValid(Actor))
	{
		UActorDescContainerInstance* Container = InActorDescInstance->GetContainerInstance();
		check(Container);

		bool bIsAlreadyInSetPtr;
		ContainerInstanceOps.FindOrAdd(Container).Unregistrations.Add(InActorDescInstance, &bIsAlreadyInSetPtr);
		check(!bIsAlreadyInSetPtr);
		NumUnregistrations++;

		check(!ContainerInstanceOps.FindChecked(Container).Registrations.Contains(InActorDescInstance));

		bNeedsClearTransactions |= Actor->GetTypedOuter<UWorld>()->HasAnyFlags(RF_Transactional);
	}
	else
	{
		check(InActorDescInstance->bIsRegisteringOrUnregistering);
		InActorDescInstance->bIsRegisteringOrUnregistering = false;
	}
}

void FWorldPartitionLoadingContext::FDeferred::CancelOperation(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	check(InActorDescInstance);
	check(InActorDescInstance->bIsRegisteringOrUnregistering); // Callers should avoid CancelOperation when unnecessary

	// If we aren't inside a ~FDeferred, we can find and cancel the op directly
	bool bRemoved = false;
	UActorDescContainerInstance* Container = InActorDescInstance->GetContainerInstance();
	check(Container);
	FContainerInstanceOps* Ops = ContainerInstanceOps.Find(Container);
	if (Ops != nullptr)
	{
		bRemoved = Ops->Registrations.Remove(InActorDescInstance) > 0;
		if (bRemoved)
		{
			--NumRegistrations;
		}
		else
		{
			bRemoved = Ops->Unregistrations.Remove(InActorDescInstance) > 0;
			if (bRemoved)
			{
				--NumUnregistrations;
			}
		}
	}

	// If we have not found the registration or unregistration, we are inside ~FDeferred, where the ContainerInstanceOps are held locally. In this case push to a list of cancellations.
	// No need to update NumRegistrations/NumUnregistrations as we are destroying
	if (!bRemoved)
	{
		bool bIsAlreadyInSetPtr;
		ContainerInstanceCancellations.Add(InActorDescInstance, &bIsAlreadyInSetPtr);
		check(!bIsAlreadyInSetPtr);
	}

	InActorDescInstance->bIsRegisteringOrUnregistering = false;
}

/**
* FWorldPartitionHandleImpl
*/
void FWorldPartitionHandleImpl::IncRefCount(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	InActorDescInstance->IncSoftRefCount();
}

void FWorldPartitionHandleImpl::DecRefCount(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	InActorDescInstance->DecSoftRefCount();
}

TWorldPartitionHandle<FWorldPartitionReferenceImpl> FWorldPartitionHandleImpl::ToReference(const TWorldPartitionHandle<FWorldPartitionHandleImpl>& Source)
{
	TWorldPartitionHandle<FWorldPartitionReferenceImpl> Result;
	Result = Source;
	return Result;
}

/**
* FWorldPartitionReferenceImpl
*/
void FWorldPartitionReferenceImpl::IncRefCount(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (InActorDescInstance->IncHardRefCount() == 1)
	{
		FWorldPartitionLoadingContext::LoadAndRegisterActor(InActorDescInstance);
	}
}

void FWorldPartitionReferenceImpl::DecRefCount(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	const bool bIsContainerInitialized = InActorDescInstance->GetContainerInstance()->IsInitialized();
	const bool bIsActorDescInstanceUnregistering = InActorDescInstance->bIsRegisteringOrUnregistering;
	const bool bReleasingLastReference = !InActorDescInstance->DecHardRefCount();

	if ((bReleasingLastReference || !bIsContainerInitialized) && !bIsActorDescInstanceUnregistering)
	{
		FWorldPartitionLoadingContext::UnloadAndUnregisterActor(InActorDescInstance);
	}
}

TWorldPartitionHandle<FWorldPartitionHandleImpl> FWorldPartitionReferenceImpl::ToHandle(const TWorldPartitionHandle<FWorldPartitionReferenceImpl>& Source)
{
	TWorldPartitionHandle<FWorldPartitionHandleImpl> Result;
	Result = Source;
	return Result;
}
#endif
