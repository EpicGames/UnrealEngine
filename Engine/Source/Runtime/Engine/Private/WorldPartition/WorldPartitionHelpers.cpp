// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionHelpers.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Algo/AnyOf.h"

#include "Commandlets/Commandlet.h"
#include "WorldPartition/WorldPartitionLog.h"

#if WITH_EDITOR
#include "Misc/RedirectCollector.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "Modules/ModuleManager.h"
#endif

namespace FWorldPartitionHelpersPrivate
{
	UWorldPartition* GetWorldPartitionFromObject(const UObject* InObject)
	{
		for (const UObject* Object = InObject; IsValid(Object); Object = Object->GetOuter())
		{
			if (const AActor* Actor = Cast<const AActor>(Object))
			{
				return GetWorldPartition(Actor);
			}
			else if (const ULevel* Level = Cast<const ULevel>(Object))
			{
				return GetWorldPartition(Level);
			}
			else if (const UWorld* World = Cast<const UWorld>(Object))
			{
				return GetWorldPartition(World);
			}
			else if (const UWorldPartition* WorldPartition = Cast<const UWorldPartition>(Object))
			{
				return const_cast<UWorldPartition*>(WorldPartition);
			}
		}

		return nullptr;
	}
}

void FWorldPartitionHelpers::ServerExecConsoleCommand(UWorld* InWorld, const FString& InConsoleCommandName, const TArray<FString>& InArgs)
{
#if !UE_BUILD_SHIPPING && !WITH_EDITOR
	if (InWorld && InWorld->IsGameWorld() && InWorld->IsNetMode(NM_Client))
	{
		if (APlayerController* PC = GEngine->GetFirstLocalPlayerController(InWorld))
		{
			TArray<FString> CmdList;
			CmdList.Add(InConsoleCommandName);
			CmdList.Append(InArgs);
			FString Cmd = FString::Join(CmdList, TEXT(" "));
			// Use ServerExecRPC instead of ServerExec to avoid any truncation
			PC->ServerExecRPC(Cmd);
		}
	}
#endif
}

#if WITH_EDITOR

bool FWorldPartitionHelpers::IsActorDescClassCompatibleWith(const FWorldPartitionActorDesc* ActorDesc, const UClass* Class)
{
	check(Class);

	UClass* ActorNativeClass = ActorDesc->GetActorNativeClass();
	UClass* ActorBaseClass = ActorNativeClass;

	if (!Class->IsNative())
	{
		if (FTopLevelAssetPath ActorBasePath = ActorDesc->GetBaseClass(); !ActorBasePath.IsNull())
		{
			ActorBaseClass = LoadClass<AActor>(nullptr, *ActorBasePath.ToString(), nullptr, LOAD_None, nullptr);

			if (!ActorBaseClass)
			{
				UE_LOGF(LogWorldPartition, Warning, "Failed to find actor base class: %ls.", *ActorBasePath.ToString());
				ActorBaseClass = ActorNativeClass;
			}
		}
	}

	return ActorBaseClass->IsChildOf(Class);
}

void FWorldPartitionHelpers::ForEachIntersectingActorDescInstance(UWorldPartition* WorldPartition, const FBox& Box, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func)
{
	if (WorldPartition->EditorHash)
	{
		bool bProcessNextActors = true;
		WorldPartition->EditorHash->ForEachIntersectingActor(Box, [&ActorClass, Func, &bProcessNextActors](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			if (bProcessNextActors && IsActorDescClassCompatibleWith(ActorDescInstance->GetActorDesc(), ActorClass))
			{
				bProcessNextActors = Func(ActorDescInstance);
			}
		});
	}
	else
	{
		ForEachActorDescInstance(WorldPartition, ActorClass, [&Box, Func](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			return !ActorDescInstance->GetEditorBounds().Intersect(Box) || Func(ActorDescInstance);
		});
	}
}

void FWorldPartitionHelpers::ForEachActorDescInstance(UWorldPartition* WorldPartition, TSubclassOf<AActor> ActorClass, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func)
{
	for (FActorDescContainerInstanceCollection::TConstIterator<> Iterator(WorldPartition); Iterator; ++Iterator)
	{
		if (IsActorDescClassCompatibleWith(Iterator->GetActorDesc(), ActorClass))
		{
			if (!Func(*Iterator))
			{
				return;
			}
		}
	}
}

namespace WorldPartitionHelpers
{
	void LoadReferencesInternal(UWorldPartition* WorldPartition, const FGuid& ActorGuid, TMap<FGuid, FWorldPartitionReference>& InOutActorReferences)
	{
		if (InOutActorReferences.Contains(ActorGuid))
		{
			return;
		}

		if (const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
		{
			InOutActorReferences.Emplace(ActorGuid);

			for (FGuid ReferenceGuid : ActorDescInstance->GetReferences())
			{
				LoadReferencesInternal(WorldPartition, ReferenceGuid, InOutActorReferences);
			}

			InOutActorReferences[ActorGuid] = FWorldPartitionReference(WorldPartition, ActorGuid);
		}
	}

	void LoadReferences(UWorldPartition* WorldPartition, const FGuid& ActorGuid, TMap<FGuid, FWorldPartitionReference>& InOutActorReferences)
	{
		FWorldPartitionLoadingContext::FDeferred LoadingContext;
		LoadReferencesInternal(WorldPartition, ActorGuid, InOutActorReferences);
	}
}

FWorldPartitionHelpers::FForEachActorWithLoadingParams::FForEachActorWithLoadingParams()
	: bGCPerActor(false)
	, bKeepReferences(false)
	, ActorClasses({ AActor::StaticClass() })
{}

void FWorldPartitionHelpers::ForEachActorWithLoading(UWorldPartition* WorldPartition, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func, const FForEachActorWithLoadingParams& Params)
{
	FForEachActorWithLoadingResult Result;
	ForEachActorWithLoading(WorldPartition, Func, Params, Result);
}

void FWorldPartitionHelpers::ForEachActorWithLoading(UWorldPartition* WorldPartition, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Func, const FForEachActorWithLoadingParams& Params, FForEachActorWithLoadingResult& Result)
{
	check(Result.ActorReferences.IsEmpty());
	auto CallGarbageCollect = [&Params, &Result]()
	{
		check(!Params.bKeepReferences);
		if (Params.OnPreGarbageCollect)
		{
			Params.OnPreGarbageCollect();
		}
		Result.ActorReferences.Empty();
		DoCollectGarbage();
	};

	auto ForEachActorWithLoadingImpl = [&](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		if (Algo::AnyOf(Params.ActorClasses, [ActorDescInstance](UClass* ActorClass) { return IsActorDescClassCompatibleWith(ActorDescInstance->GetActorDesc(), ActorClass); }))
		{
			if (!Params.FilterActorDescInstance || Params.FilterActorDescInstance(ActorDescInstance))
			{
				if (!Params.FilterActorDesc || Params.FilterActorDesc(ActorDescInstance->GetActorDesc()))
				{
					WorldPartitionHelpers::LoadReferences(WorldPartition, ActorDescInstance->GetGuid(), Result.ActorReferences);

					FWorldPartitionReference ActorReference(WorldPartition, ActorDescInstance->GetGuid());
					if (!Func(*ActorReference))
					{
						return false;
					}

					if (!Params.bKeepReferences && (Params.bGCPerActor || FWorldPartitionHelpers::ShouldCollectGarbage()))
					{
						CallGarbageCollect();
					}
				}
			}
		}

		return true;
	};

	if (Params.ActorGuids.IsEmpty())
	{
		for (FActorDescContainerInstanceCollection::TConstIterator<> ActorDescInstanceIterator(WorldPartition); ActorDescInstanceIterator; ++ActorDescInstanceIterator)
		{
			if (const FWorldPartitionActorDescInstance* ActorDesc = *ActorDescInstanceIterator)
			{
				if (!ForEachActorWithLoadingImpl(ActorDesc))
				{
					break;
				}
			}
			
		}
	}
	else
	{
		for (const FGuid& ActorGuid : Params.ActorGuids)
		{
			if (const FWorldPartitionActorDescInstance* ActorDesc = WorldPartition->GetActorDescInstance(ActorGuid))
			{
				if (!ForEachActorWithLoadingImpl(ActorDesc))
				{
					break;
				}
			}
		}
	}

	if (!Params.bKeepReferences)
	{
		CallGarbageCollect();
	}
}

bool FWorldPartitionHelpers::HasExceededMaxMemory()
{
	const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();

	const uint64 MemoryMinFreePhysical = 1llu * 1024 * 1024 * 1024;
	const uint64 MemoryMaxUsedPhysical = FMath::Max(32llu * 1024 * 1024 * 1024, MemStats.TotalPhysical / 2);

	const bool bHasExceededMinFreePhysical = MemStats.AvailablePhysical < MemoryMinFreePhysical;
	const bool bHasExceededMaxUsedPhysical = MemStats.UsedPhysical >= MemoryMaxUsedPhysical;
	const bool bHasExceededMaxMemory = bHasExceededMinFreePhysical || bHasExceededMaxUsedPhysical;

	return bHasExceededMaxMemory;
}

bool FWorldPartitionHelpers::ShouldCollectGarbage()
{
	return HasExceededMaxMemory();
}

void FWorldPartitionHelpers::DoCollectGarbage()
{
	const FPlatformMemoryStats MemStatsBefore = FPlatformMemory::GetStats();
	CollectGarbage(IsRunningCommandlet() ? RF_NoFlags : GARBAGE_COLLECTION_KEEPFLAGS, true);
	const FPlatformMemoryStats MemStatsAfter = FPlatformMemory::GetStats();

	UE_LOGF(LogWorldPartition, Log, "GC Performed - Available Physical: %.2fGB, Available Virtual: %.2fGB",
		(int64)MemStatsAfter.AvailablePhysical / (1024.0 * 1024.0 * 1024.0),
		(int64)MemStatsAfter.AvailableVirtual / (1024.0 * 1024.0 * 1024.0)
	);
}

void FWorldPartitionHelpers::FakeEngineTick(UWorld* InWorld)
{
	check(InWorld);

	CommandletHelpers::TickEngine(InWorld);
}

bool FWorldPartitionHelpers::ConvertRuntimePathToEditorPath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath)
{
	//
	// Try to convert from /.../WorldName/_Generated_/MainGrid_L0_X0_Y0_DL0.WorldName:PersistentLevel.ActorName
	//                  to /.../WorldName.WorldName:PersistentLevel.ActorName
	//
	FString OutPathString = InPath.ToString();
	const FStringView InPathView(OutPathString);

	if (int32 GeneratedPos = InPathView.Find(TEXTVIEW("/_Generated_/"), 0); GeneratedPos != INDEX_NONE)
	{
		if (int32 NextDotPos = InPathView.Find(TEXTVIEW("."), GeneratedPos); NextDotPos != INDEX_NONE)
		{
			if (int32 NextColonPos = InPathView.Find(TEXTVIEW(":"), NextDotPos); NextColonPos != INDEX_NONE)
			{
				OutPathString.RemoveAt(GeneratedPos, NextDotPos - GeneratedPos);

				// In the editor, the _LevelInstance_ID is appended to the persistent level, while at runtime it is appended to each cell package, so we need to remap it there if present.
				FString WorldAssetPackageName = InPath.GetAssetPath().GetPackageName().ToString();
				const int32 LevelInstancePos = WorldAssetPackageName.Find(TEXT("_LevelInstance_"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (LevelInstancePos != INDEX_NONE)
				{
					FString LevelInstanceTag = WorldAssetPackageName.RightChop(LevelInstancePos);
					OutPathString.InsertAt(GeneratedPos, LevelInstanceTag);
				}

				OutPath = OutPathString;
				return true;
			}
		}
	}
	else
	{
		OutPath = UWorld::RemovePIEPrefix(InPath.ToString());
		return true;
	}

	return false;
}

TMap<FGuid, AActor*> FWorldPartitionHelpers::GetLoadedActorsForLevel(const ULevel* InLevel)
{
	TMap<FGuid, AActor*> Result;
	ForEachObjectWithOuter(InLevel, [&Result](UObject* Object)
	{
		if (AActor* Actor = Cast<AActor>(Object))
		{
			if (!Actor->IsTemplate() && Actor->GetActorGuid().IsValid())
			{
				Result.Add(Actor->GetActorGuid(), Actor);
			}
		}
	});
	return MoveTemp(Result);
}

TMap<FGuid, AActor*> FWorldPartitionHelpers::GetRegisteredActorsForLevel(const ULevel* InLevel)
{
	TMap<FGuid, AActor*> Result;
	Algo::TransformIf(InLevel->Actors, Result, 
		[](const AActor* Actor)
		{
			return IsValid(Actor);
		}, 
		[](AActor* Actor)
		{
			return TPair<FGuid, AActor*>(Actor->GetActorGuid(), Actor);
		});
	return MoveTemp(Result);
}
#endif // #if WITH_EDITOR

bool FWorldPartitionHelpers::ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath)
{
	//
	// Try to convert from /.../WorldName.WorldName:PersistentLevel.ActorName
	//                  to /.../WorldName/_Generated_/MainGrid_L0_X0_Y0_DL0.WorldName:PersistentLevel.ActorName
	//
	FSoftObjectPath Path(InPath);
	FSoftObjectPath WorldPath(Path.GetAssetPath(), FString());

	if (UWorld* World = Cast<UWorld>(WorldPath.ResolveObject()))
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			return WorldPartition->ConvertEditorPathToRuntimePath(InPath, OutPath);
		}
	}

	return false;
}