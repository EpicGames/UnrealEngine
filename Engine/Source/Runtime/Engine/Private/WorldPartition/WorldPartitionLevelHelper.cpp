// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "Misc/PackageName.h"
#include "WorldPartition/WorldPartitionPackageHelper.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "Containers/StringFwd.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/IWorldPartitionObjectResolver.h"

#if WITH_EDITOR
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "Misc/Paths.h"
#include "Model.h"
#include "UnrealEngine.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionPackageHelper.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"
#include "GameFramework/WorldSettings.h"
#include "LevelUtils.h"
#include "ActorFolder.h"
#include "UObject/UObjectGlobalsInternal.h"

FUObjectAnnotationSparse<FWorldPartitionLevelHelper::FActorPropertyOverridesAnnotation, true> FWorldPartitionLevelHelper::ActorPropertyOverridesAnnotation;
#endif

bool FWorldPartitionResolveData::ResolveObject(UWorld* InWorld, const FSoftObjectPath& InObjectPath, UObject*& OutObject) const
{
	OutObject = nullptr;
	if (InWorld)
	{
		if (IsValid() && SourceWorldAssetPath == InObjectPath.GetAssetPath())
		{
			const FString SubPathString = FWorldPartitionLevelHelper::AddActorContainerIDToSubPathString(ContainerID, InObjectPath.GetSubPathString());
			// We don't read the return value as we always want to return true when using the resolve data.
			InWorld->ResolveSubobject(*SubPathString, OutObject, /*bLoadIfExists*/false);
			return true;
		}
	}

	return false;
}

FString FWorldPartitionLevelHelper::AddActorContainerID(const FActorContainerID& InContainerID, const FString& InActorName)
{
	const FName ActorName(*InActorName);
	const FString ActorPlainName(ActorName.GetPlainNameString() + TEXT("_") + InContainerID.ToShortString());
	return FName(*ActorPlainName, ActorName.GetNumber()).ToString();}

FString FWorldPartitionLevelHelper::AddActorContainerIDToSubPathString(const FActorContainerID& InContainerID, const FString& InSubPathString)
{
	if (!InContainerID.IsMainContainer())
	{
		constexpr const TCHAR PersistenLevelName[] = TEXT("PersistentLevel.");
		constexpr const int32 DotPos = UE_ARRAY_COUNT(PersistenLevelName);
		if (InSubPathString.StartsWith(PersistenLevelName))
		{
			const int32 SubObjectPos = InSubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, DotPos);
			if (SubObjectPos == INDEX_NONE)
			{
				return AddActorContainerID(InContainerID, InSubPathString);
			}
			else
			{
				return AddActorContainerID(InContainerID, InSubPathString.Mid(0, SubObjectPos)) + InSubPathString.Mid(SubObjectPos);
			}
		}
	}

	return InSubPathString;
}

#if WITH_EDITOR
FWorldPartitionLevelHelper& FWorldPartitionLevelHelper::Get()
{
	static FWorldPartitionLevelHelper Instance;
	return Instance;
}

FWorldPartitionLevelHelper::FWorldPartitionLevelHelper()
{
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FWorldPartitionLevelHelper::PreGarbageCollect);
}

void FWorldPartitionLevelHelper::PreGarbageCollect()
{
	// Don't attempt to unload packages while AsyncLoading.
	//   
	// WorldPartitionlevelHelper releases the reference and adds packages to PreGCPackagesToUnload in FinalizeRuntimeLevel, right after the loading is done.
	// However, AsyncLoading2 also tracks package references using GlobalImportStore. Package references are removed from the GlobalImportStore when FAsyncPackage2 is deleted,
	// which can happen after PreGarbageCollect, due to usage of DeferredDeletePackages queue.
	// If we get another request, which involves a package that has already been trashed (via FWorldPartitionPackageHelper::UnloadPackage)
	// but not yet removed from the GlobalImportStore, AsyncLoading2 will attempt to reuse that package. Since it has already been trashed at that point, it'll lead to undesired behavior.
	// To prevent this from happening, don't attempt to unload packages while AsyncLoading - IsAsyncLoading returns true until all AsyncPackages have not been deleted - see PackagesWithRemainingWorkCounter and DeleteAsyncPackage
	if (!IsAsyncLoading())
	{
		for (TWeakObjectPtr<UPackage>& PackageToUnload : PreGCPackagesToUnload)
		{
			// Test if WeakObjectPtr is valid since clean up could have happened outside of this helper
			if (PackageToUnload.IsValid())
			{
				FWorldPartitionPackageHelper::UnloadPackage(PackageToUnload.Get());
			}
		}
		PreGCPackagesToUnload.Reset();
	}
}

void FWorldPartitionLevelHelper::ApplyConstructionScriptPropertyOverridesFromAnnotation(AActor* InActor)
{
	if (IsValid(InActor))
	{
		FWorldPartitionLevelHelper::FActorPropertyOverridesAnnotation Annotation = FWorldPartitionLevelHelper::ActorPropertyOverridesAnnotation.GetAndRemoveAnnotation(InActor);
		if (!Annotation.IsDefault())
		{
			if (InActor->GetRootComponent())
			{
				const FTransform InverseTransform = Annotation.ContainerTransform.Inverse();
				FLevelUtils::FApplyLevelTransformParams TransformParams(InActor->GetLevel(), InverseTransform);
				TransformParams.Actor = InActor;
				TransformParams.bDoPostEditMove = false;
				TransformParams.bSetRelativeTransformDirectly = true;

				FLevelUtils::ApplyLevelTransform(TransformParams);
			}

			for (const FActorPropertyOverride& ActorOverride : Annotation.ActorPropertyOverrides)
			{
				const bool bConstructionScriptProperties = true;
				UWorldPartitionPropertyOverride::ApplyPropertyOverrides(&ActorOverride, InActor, bConstructionScriptProperties);
			}

			if (InActor->GetRootComponent())
			{
				FLevelUtils::FApplyLevelTransformParams TransformParams(InActor->GetLevel(), Annotation.ContainerTransform);
				TransformParams.Actor = InActor;
				TransformParams.bDoPostEditMove = false;
				TransformParams.bSetRelativeTransformDirectly = true;

				FLevelUtils::ApplyLevelTransform(TransformParams);
				InActor->GetRootComponent()->UpdateComponentToWorld();
				InActor->MarkComponentsRenderStateDirty();
			}
		}
	}
}

void FWorldPartitionLevelHelper::AddReference(UPackage* InPackage, FPackageReferencer* InReferencer)
{
	check(InPackage);
	FPackageReference& RefInfo = PackageReferences.FindOrAdd(InPackage->GetFName());
	check(RefInfo.Package == nullptr || RefInfo.Package == InPackage);
	RefInfo.Package = InPackage;
	RefInfo.Referencers.Add(InReferencer);
	PreGCPackagesToUnload.Remove(InPackage);
}

void FWorldPartitionLevelHelper::RemoveReferences(FPackageReferencer* InReferencer)
{
	for (auto It = PackageReferences.CreateIterator(); It; ++It)
	{
		FPackageReference& RefInfo = It->Value;
		RefInfo.Referencers.Remove(InReferencer);
		if (RefInfo.Referencers.Num() == 0)
		{
			// Test if WeakObjectPtr is valid since clean up could have happened outside of this helper
			if (RefInfo.Package.IsValid())
			{
				PreGCPackagesToUnload.Add(RefInfo.Package);
			}
			It.RemoveCurrent();
		}
	}
}

void FWorldPartitionLevelHelper::FPackageReferencer::AddReference(UPackage* InPackage)
{
	FWorldPartitionLevelHelper::Get().AddReference(InPackage, this);
}

void FWorldPartitionLevelHelper::FPackageReferencer::RemoveReferences()
{
	FWorldPartitionLevelHelper::Get().RemoveReferences(this);
}


 /**
  * Defaults World's initialization values for World Partition StreamingLevels
  */
UWorld::InitializationValues FWorldPartitionLevelHelper::GetWorldInitializationValues()
{
	return UWorld::InitializationValues()
		.InitializeScenes(false)
		.AllowAudioPlayback(false)
		.RequiresHitProxies(false)
		.CreatePhysicsScene(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.SetTransactional(false)
		.CreateFXSystem(false);
}

/**
 * Moves external actors into the given level
 */
void FWorldPartitionLevelHelper::MoveExternalActorsToLevel(const TArray<FWorldPartitionRuntimeCellObjectMapping>& InChildPackages, ULevel* InLevel, TArray<UPackage*>& OutModifiedPackages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionLevelHelper::MoveExternalActorsToLevel);

	check(InLevel);
	UPackage* LevelPackage = InLevel->GetPackage();

	// Gather existing actors to validate only the one we expect are added to the level
	TSet<FName> LevelActors;
	for (AActor* Actor : InLevel->Actors)
	{
		if (Actor)
		{
			LevelActors.Add(Actor->GetFName());
		}
	}

	// Move all actors to Cell level
	TSet<AActor*> LoadedActors;
	TSet<UPackage*> ModifiedPackages;
	TSet<UObject*> MovedObjects;

	// Actors renamed into InLevel that will need a deferred RerunConstructionScript call, to preserve attachment; renaming/AddLoadedActors registers components
	// which is needed for RerunConstructionScript attachment-preserving mechanisms to work. RRCS has to wait on all registers being done.
	TArray<AActor*> ActorsNeedingRRCS;

	for (const FWorldPartitionRuntimeCellObjectMapping& PackageObjectMapping : InChildPackages)
	{
		// We assume actor failed to duplicate if LoadedPath equals NAME_None (warning already logged we can skip this mapping)
		if (PackageObjectMapping.LoadedPath == NAME_None && !PackageObjectMapping.ContainerID.IsMainContainer())
		{
			continue;
		}

		// Always load editor-only actors during cooking and move them in their corresponding streaming cell, to avoid referencing public objects from the level instance package for embedded actors.
		// In PIE, we continue to filter out editor-only actors and also null-out references to these objects using the instancing context. In cook, the references will be filtered out by the cooker 
		// archive will be filtering editor-only objects, and will allow references from other cells because they all share the same outer.
		if (PackageObjectMapping.bIsEditorOnly && !IsRunningCookCommandlet())
		{
			continue;
		}

		AActor* Actor = FindObject<AActor>(nullptr, *PackageObjectMapping.LoadedPath.ToString());
		if (Actor)
		{
			UPackage* ActorPackage = Actor->GetPackage();
			check(ActorPackage);

			const bool bIsActorPackageExternal = Actor->IsPackageExternal();
			const bool bSameOuter = (InLevel == Actor->GetOuter());

			Actor->SetPackageExternal(false, false);

			// Avoid calling Rename on the actor if it's already outered to InLevel as this will cause it's name to be changed. 
			// (UObject::Rename doesn't check if Rename is being called with existing outer and assigns new name)
			if (!bSameOuter)
			{
				Actor->Rename(nullptr, InLevel, REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

				// AActor::Rename will register components but doesn't call RerunConstructionScripts like AddLoadedActors does.
				// If bIsWorldInitialized is false, RerunConstructionScripts will get called as part of UEditorEngine::InitializePhysicsSceneForSaveIfNecessary during Cell package save
				// If bIsWorldInitialized is true, defer RerunConstructionScripts
				if (InLevel->GetWorld()->bIsWorldInitialized)
				{
					ActorsNeedingRRCS.Add(Actor);
				}
			}
			else if (!InLevel->Actors.Contains(Actor))
			{
				LoadedActors.Emplace(Actor);
			}
			check(Actor->GetPackage() == LevelPackage);

			// Process objects found in the source actor package
			if (bIsActorPackageExternal)
			{
				TArray<UObject*> Objects;

				// Skip Garbage objects as the initial Rename on an actor with an ChildActorComponent can destroy its child actors.
				// This happens when the component has bNeedsRecreate set to true (when it has a valid ChildActorTemplate).
				GetObjectsWithPackage(ActorPackage, Objects, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

				for (UObject* Object : Objects)
				{
					if (Object->GetFName() != NAME_PackageMetaData)
					{
						if (Object->GetOuter()->IsA<ULevel>())
						{
							// Move objects that are outered the level in the destination level
							AActor* NestedActor = Cast<AActor>(Object);
							if (InLevel != Object->GetOuter())
							{
								Object->Rename(nullptr, InLevel, REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
								MovedObjects.Add(Object);
							}
							else if (NestedActor && !InLevel->Actors.Contains(NestedActor))
							{
								LoadedActors.Emplace(NestedActor);
							}
							if (NestedActor)
							{
								LevelActors.Add(NestedActor->GetFName());
							}
						}
						else
						{
							// Move objects in the destination level package
							// We handle name clashes in a custom way by increasing a suffix on the already existing base name instead
							// of using the default behavior of having the class name be the base name, this is to maximally preserve
							// information present in the object name in case somebody is using the name as significant information 
							FName NewName = Object->GetFName();
							if (StaticFindObject(nullptr, LevelPackage, *NewName.ToString()))
							{
								NewName = MakeUniqueObjectName(LevelPackage, Object->GetClass(), *Object->GetName());
							}

							Object->Rename(*NewName.ToString(), LevelPackage, REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
							MovedObjects.Add(Object);
						}
					}
				}
			}

			ModifiedPackages.Add(ActorPackage);
			LevelActors.Add(Actor->GetFName());
		}
		else
		{
			UE_LOGF(LogWorldPartition, Warning, "Can't find actor %ls.", *PackageObjectMapping.Path.ToString());
		}
	}

	InLevel->AddLoadedActors(LoadedActors.Array());

	// Run deferred RRCS now that all actors have been renamed into the level or AddLoadedActors above, and have had their components registered.
	for (AActor* Actor : ActorsNeedingRRCS)
	{
		if (IsValid(Actor))
		{
			Actor->RerunConstructionScripts();
			ApplyConstructionScriptPropertyOverridesFromAnnotation(Actor);
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ValidateLoaded);

		for (AActor* Actor : InLevel->Actors)
		{
			if (IsValid(Actor) && Actor->HasAllFlags(RF_WasLoaded))
			{
				checkf(LevelActors.Contains(Actor->GetFName()), TEXT("Actor %s(%s) was unexpectedly loaded when moving actors to streaming cell"), *Actor->GetActorNameOrLabel(), *Actor->GetName());
			}
		}
	}

	// Trash the modified packages (except the destination level) to guarantee that any potential future load of this actor won't find the old empty package
	for (UPackage* ModifiedPackage : ModifiedPackages)
	{
		if (ModifiedPackage != LevelPackage)
		{
			TrashObject(ModifiedPackage);
		}
	}

	OutModifiedPackages.Append(ModifiedPackages.Array());
	
	// Perform validations and fixups to make sure the level is saveable
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ValidateMovedObjects);

		TArray<UObject*> ObjectStack { InLevel };
		TSet<UObject*> VisitedObjects { InLevel };

		for (AActor* Actor : InLevel->Actors)
		{
			if (IsValid(Actor))
			{
				ObjectStack.Add(Actor);

				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (IsValid(Component))
					{
						ObjectStack.Add(Component);
					}
				}
			}
		}

		while (ObjectStack.Num())
		{
			UObject* Object = ObjectStack.Pop();

			Object->GetClass()->Visit(Object, 
				[&ObjectStack, &VisitedObjects, &MovedObjects, InLevel](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
				{
					const FPropertyVisitorPath& Path = Context.Path;
					const FPropertyVisitorData& Data = Context.Data;
					const FProperty* Property = Path.Top().Property;

					if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
					{
						if (UObject* PropertyObject = ObjectProperty->GetObjectPropertyValue(Data.PropertyData))
						{
							bool bWasAlreadyInSet;
							VisitedObjects.Add(PropertyObject, &bWasAlreadyInSet);

							if (bWasAlreadyInSet)
							{
								return EPropertyVisitorControlFlow::StepOver;
							}

							if (!ObjectProperty->HasAnyPropertyFlags(CPF_Transient | CPF_EditorOnly))
							{
								if (!PropertyObject->HasAnyFlags(RF_Transient) && !PropertyObject->IsEditorOnly())
								{
									if (MovedObjects.Remove(PropertyObject))
									{
										return EPropertyVisitorControlFlow::StepInto;
									}
								}
							}

							if (!IsValidChecked(PropertyObject))
							{
								UE_LOGF(LogWorldPartition, Log, "Reachable invalid object '%ls' from '%ls' will be nulled out", *GetNameSafe(PropertyObject), *Path.ToString());
								ObjectProperty->SetObjectPropertyValue(Data.PropertyData, nullptr);
								return EPropertyVisitorControlFlow::StepOver;
							}

							if (!PropertyObject->IsIn(InLevel))
							{
								return EPropertyVisitorControlFlow::StepOver;
							}
						}
					}

					return EPropertyVisitorControlFlow::StepInto;
				}, FPropertyVisitorContext::EScope::ObjectRefs);
		}

		if (MovedObjects.Num())
		{
			UE_LOGF(LogWorldPartition, Log, "Unreferenced objects from '%ls'", *InLevel->GetPackage()->GetName());

			for (UObject* Object : MovedObjects)
			{
				UE_LOGF(LogWorldPartition, Log, "\t'%ls'", *Object->GetName());
			}
		}
	}
}

void FWorldPartitionLevelHelper::RemapLevelSoftObjectPaths(ULevel* InLevel, UWorldPartition* InWorldPartition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionLevelHelper::RemapLevelSoftObjectPaths);

	check(InLevel);
	check(InWorldPartition);

	FSoftObjectPathFixupArchive FixupSerializer([InWorldPartition](FSoftObjectPath& Value)
	{
		if(!Value.IsNull())
		{
			InWorldPartition->RemapSoftObjectPath(Value);
		}
	});
	FixupSerializer.Fixup(InLevel);
}

FSoftObjectPath FWorldPartitionLevelHelper::RemapActorPath(const FActorContainerID& InContainerID, const FString& InSourceWorldPath, const FSoftObjectPath& InActorPath)
{
	// If Path is in an instanced package it will now be remapped to its source package
	FSoftObjectPath OutActorPath(FTopLevelAssetPath(InSourceWorldPath), InActorPath.GetSubPathString());
	
	if(!InContainerID.IsMainContainer())
	{
		// This gets called by UWorldPartitionLevelStreamingPolicy::PrepareActorToCellRemapping and FWorldPartitionLevelHelper::LoadActors
		// 
		// At this point we are changing the top level asset and remapping the SubPathString to add a ContainerID suffix so
		// '/Game/SomePath/LevelInstance.LevelInstance:PersistentLevel.ActorA' becomes
		// '/Game/SomeOtherPath/SourceWorldName.SourceWorldName:PersistentLevel.ActorA_{ContainerID}'
		FString RemappedSubPathString = FWorldPartitionLevelHelper::AddActorContainerIDToSubPathString(InContainerID, InActorPath.GetSubPathString());
		OutActorPath.SetSubPathString(RemappedSubPathString);
	}
	
	return OutActorPath;
}

bool FWorldPartitionLevelHelper::RemapLevelCellPathInContentBundle(ULevel* Level, const class FContentBundleEditor* ContentBundleEditor, const UWorldPartitionRuntimeCell* Cell)
{
	FString CellPath = ContentBundleEditor->GetExternalStreamingObjectPackagePath();
	CellPath += TEXT(".");
	CellPath += URuntimeHashExternalStreamingObjectBase::GetCookedExternalStreamingObjectName();
	CellPath += TEXT(".");
	CellPath += Cell->GetName();
	FSetWorldPartitionRuntimeCell SetWorldPartitionRuntimeCell(Level, FSoftObjectPath(CellPath));
	return Level->IsWorldPartitionRuntimeCell();
}

/**
 * Creates an empty Level used in World Partition
 */
ULevel* FWorldPartitionLevelHelper::CreateEmptyLevelForRuntimeCell(const UWorldPartitionRuntimeCell* Cell, const UWorld* InWorld, const FString& InWorldAssetName, UPackage* InPackage)
{
	// Create or use given package
	UPackage* CellPackage = nullptr;
	if (InPackage)
	{
		check(FindObject<UPackage>(nullptr, *InPackage->GetName()));
		CellPackage = InPackage;
	}
	else
	{
		FString PackageName = FPackageName::ObjectPathToPackageName(InWorldAssetName);
		check(!FindObject<UPackage>(nullptr, *PackageName));
		CellPackage = CreatePackage(*PackageName);
		CellPackage->SetPackageFlags(PKG_NewlyCreated);
	}

	if (InWorld->IsPlayInEditor())
	{
		check(!InPackage);
		CellPackage->SetPackageFlags(PKG_PlayInEditor);
		CellPackage->SetPIEInstanceID(InWorld->GetPackage()->GetPIEInstanceID());
	}

	// Create World & Persistent Level
	UWorld::InitializationValues IVS = FWorldPartitionLevelHelper::GetWorldInitializationValues();
	const FName WorldName = FName(FPackageName::ObjectPathToObjectName(InWorldAssetName));
	check(!FindObject<UWorld>(CellPackage, *WorldName.ToString()));
	UWorld* NewWorld = UWorld::CreateWorld(InWorld->WorldType, /*bInformEngineOfWorld*/false, WorldName, CellPackage, /*bAddToRoot*/false, InWorld->GetFeatureLevel(), &IVS, /*bInSkipInitWorld*/true);
	check(NewWorld);
	NewWorld->SetFlags(RF_Public | RF_Standalone);
	check(NewWorld->GetWorldSettings());
	check(UWorld::FindWorldInPackage(CellPackage) == NewWorld);
	check(InPackage || (NewWorld->GetPathName() == InWorldAssetName));
	// Make sure to have the default brush in PIE to fix cases where the editor expect to have a default brush at index 1 in the level's actors array
	if (InWorld->IsPlayInEditor())
	{
		NewWorld->RepairDefaultBrush();
		check(NewWorld->GetDefaultBrush());
	}
	// We don't need the cell level's world setting to replicate
	FSetActorReplicates SetActorReplicates(NewWorld->GetWorldSettings(), false);
	
	// Setup of streaming cell Runtime Level
	ULevel* NewLevel = NewWorld->PersistentLevel;
	check(NewLevel);
	check(NewLevel->GetFName() == InWorld->PersistentLevel->GetFName());
	check(NewLevel->OwningWorld == NewWorld);
	check(NewLevel->Model);
	check(!NewLevel->bIsVisible);

	NewLevel->WorldPartitionRuntimeCell = const_cast<UWorldPartitionRuntimeCell*>(Cell);
	
	// Mark the level package as fully loaded
	CellPackage->MarkAsFullyLoaded();

	// Mark the level package as containing a map
	CellPackage->ThisContainsMap();

	// Set the guids on the constructed level to something based on the generator rather than allowing indeterminism by
	// constructing new Guids on every cook
	// @todo_ow: revisit for static lighting support. We need to base the LevelBuildDataId on the relevant information from the
	// actor's package.
	NewLevel->LevelBuildDataId = InWorld->PersistentLevel->LevelBuildDataId;
	check(InWorld->PersistentLevel->Model && NewLevel->Model);
	NewLevel->Model->LightingGuid = InWorld->PersistentLevel->Model->LightingGuid;

	return NewLevel;
}

bool FWorldPartitionLevelHelper::LoadActorsWithPropertyOverridesInternal(FLoadActorsParams&& InParams)
{
	TMap<FString, FName> PropertyOverridesToLoad;

	struct FLoadProgress
	{
		int32 NumPendingLoadRequests = 0;
		int32 NumFailedLoadedRequests = 0;
		
		TMap<FSoftObjectPath, TSet<FActorContainerID>> AssetToContainerIDs;

		FLoadActorsParams Params;
		FLoadedPropertyOverrides LoadedPropertyOverrides;
	};
	TSharedPtr<FLoadProgress> LoadProgress = MakeShared<FLoadProgress>();
	LoadProgress->Params = MoveTemp(InParams);

	// Build up list of Property Overrides to load and an assocation between Property Override Asset Path and the overrides Owner Container ID
	for (const FWorldPartitionRuntimeCellObjectMapping& PackageObjectMapping : LoadProgress->Params.ActorPackages)
	{
		for (const FWorldPartitionRuntimeCellPropertyOverride& PropertyOverride : PackageObjectMapping.PropertyOverrides)
		{
			FName PackageName = PropertyOverridesToLoad.FindOrAdd(PropertyOverride.AssetPath, PropertyOverride.PackageName);
			check(PackageName == PropertyOverride.PackageName);
			LoadProgress->AssetToContainerIDs.FindOrAdd(PropertyOverride.AssetPath).Add(PropertyOverride.OwnerContainerID);
		}
	}

	// Nothing to load, move on to load actors
	if (PropertyOverridesToLoad.IsEmpty())
	{
		return LoadActorsInternal(MoveTemp(LoadProgress->Params), MoveTemp(LoadProgress->LoadedPropertyOverrides));
	}

	LoadProgress->NumPendingLoadRequests = PropertyOverridesToLoad.Num();

	// Do Loading
	for (auto const&[AssetPath, PackageName] : PropertyOverridesToLoad)
	{
		FSoftObjectPath SoftAssetPath(AssetPath);
		
		FLinkerInstancingContext InstancingContext;
		InstancingContext.AddTag(ULevel::DontLoadExternalObjectsTag);

		FSoftObjectPath RemappedPath = SoftAssetPath;

		// Loading embedded asset
		if (!SoftAssetPath.GetSubPathString().IsEmpty())
		{
			FString WorldPackageName = SoftAssetPath.GetLongPackageName();
			FName RemappedContainerPackage = FName(*(WorldPackageName + TEXT("_LoadPropertyOverride")));
			InstancingContext.AddPackageMapping(*WorldPackageName, RemappedContainerPackage);

			const FName AssetPackageInstanceName = FName(*ULevel::GetExternalActorPackageInstanceName(RemappedContainerPackage.ToString(), PackageName.ToString()));

			InstancingContext.AddPackageMapping(PackageName, AssetPackageInstanceName);
			InstancingContext.FixupSoftObjectPath(RemappedPath);

			// If packages are already loaded, add a reference, to make sure they're not trashed before completion callback is called
			if (UPackage* WorldPackage = FindPackage(nullptr, *RemappedContainerPackage.ToString()))
			{
				LoadProgress->Params.PackageReferencer->AddReference(WorldPackage);
			}
			if (UPackage* ActorPackage = FindPackage(nullptr, *AssetPackageInstanceName.ToString()))
			{
				LoadProgress->Params.PackageReferencer->AddReference(ActorPackage);
			}
		}

		FName RemappedPackageName = InstancingContext.RemapPackage(PackageName);
		FName PackageToLoad = PackageName;

		FLoadPackageAsyncDelegate CompletionCallback = FLoadPackageAsyncDelegate::CreateLambda([LoadProgress, SoftAssetPath, RemappedPath](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
		{
			check(LoadProgress->NumPendingLoadRequests);
			LoadProgress->NumPendingLoadRequests--;
			
			if (UWorldPartitionPropertyOverride* LoadedOverride = Cast<UWorldPartitionPropertyOverride>(RemappedPath.ResolveObject()))
			{
				// Reference World Package and Actor Package
				LoadProgress->Params.PackageReferencer->AddReference(LoadedOverride->GetOutermostObject()->GetPackage());
				LoadProgress->Params.PackageReferencer->AddReference(LoadedOverride->GetPackage());

				TSet<FActorContainerID>& OwnerContainerIDs = LoadProgress->AssetToContainerIDs.FindChecked(SoftAssetPath);
				for (FActorContainerID OwnerContainerID : OwnerContainerIDs)
				{
					LoadProgress->LoadedPropertyOverrides.PropertyOverrides.Add(OwnerContainerID, LoadedOverride);
				}
			}

			if (!LoadProgress->NumPendingLoadRequests)
			{
				LoadActorsInternal(MoveTemp(LoadProgress->Params), MoveTemp(LoadProgress->LoadedPropertyOverrides));
			}
		});

		if (LoadProgress->Params.bLoadAsync)
		{
			FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(PackageToLoad);

			int32 RequestID = ::LoadPackageAsync(PackagePath, RemappedPackageName, CompletionCallback, PKG_None, -1, 0, &InstancingContext);
			if (LoadProgress->Params.AsyncRequestIDs)
			{
				LoadProgress->Params.AsyncRequestIDs->Add(RequestID);
			}
		}
		else
		{
			UPackage* InstancingPackage = nullptr;
			if (PackageName != PackageToLoad)
			{
				InstancingPackage = CreatePackage(*RemappedPackageName.ToString());
			}

			UPackage* Package = LoadPackage(InstancingPackage, *PackageToLoad.ToString(), LOAD_None, nullptr, &InstancingContext);
			CompletionCallback.Execute(PackageToLoad, Package, Package ? EAsyncLoadingResult::Succeeded : EAsyncLoadingResult::Failed);
		}
	}

	return LoadProgress->NumPendingLoadRequests == 0;
}

// deprecated
bool FWorldPartitionLevelHelper::LoadActors(const FLoadActorsParams& InParams)
{
	FLoadActorsParams ParamsCopy = FLoadActorsParams()
		.SetActorPackages(InParams.ActorPackages)
		.SetCompletionCallback(InParams.CompletionCallback)
		.SetDestLevel(InParams.DestLevel)
		.SetInstancingContext(MoveTemp(InParams.InstancingContext))
		.SetLoadAsync(InParams.bLoadAsync, InParams.AsyncRequestIDs)
		.SetSilenceLoadFailures(InParams.bSilenceLoadFailures)
		.SetOuterWorld(InParams.OuterWorld)
		.SetPackageReferencer(InParams.PackageReferencer);
	return LoadActors(MoveTemp(ParamsCopy));
}

bool FWorldPartitionLevelHelper::LoadActors(FLoadActorsParams&& InParams)
{	
	return FWorldPartitionLevelHelper::LoadActorsWithPropertyOverridesInternal(MoveTemp(InParams));
}

bool FWorldPartitionLevelHelper::LoadActorsInternal(FLoadActorsParams&& InParams, FLoadedPropertyOverrides&& InLoadedPropertyOverrides)
{
	TArray<FWorldPartitionRuntimeCellObjectMapping*> ActorPackagesToLoad;
	TMap<FActorContainerID, FLinkerInstancingContext> LinkerInstancingContexts;

	// Generate a unique name to load a level instance embedded actor if there are multiple instances of this level instance and possibly across 
	// multiple instances of the WP world:
	auto GetContainerPackage = [](const FActorContainerID& InContainerID, const FString& InPackageName, const UObject* InContextObject, bool bUniquePackage) -> FName
	{
		TStringBuilder<512> PackageNameBuilder;
		
		// Distinguish between instances of the same level instance	
		PackageNameBuilder.Appendf(TEXT("/Temp%s_%s"), *InPackageName, *InContainerID.ToShortString());
		
		// Distinguish between instances of the same top level WP world; only for PIE, in cook we always cook the source WP and not an instance 
		// and actor packages no longer exist at runtime)
		const FString ContextObjectPathName = GetPathNameSafe(InContextObject);
		const uint64 ContextObjectPathNameHash = CityHash64(TCHAR_TO_ANSI(*ContextObjectPathName), ContextObjectPathName.Len());
		PackageNameBuilder.Appendf(TEXT("_%llx"), ContextObjectPathNameHash);

		if (!IsRunningCommandlet() && bUniquePackage)
		{
			// Distinguish between loading the same package after a reload between GCs (only for PIE)
			static uint32 ContextObjectUniqueID = 0;
			PackageNameBuilder.Appendf(TEXT("_%llx"), ContextObjectUniqueID++);
		}

		return PackageNameBuilder.ToString();
	};


	if (!InParams.ActorPackages.IsEmpty())
	{
		ActorPackagesToLoad.Reserve(InParams.ActorPackages.Num());

		// Add main container context
		LinkerInstancingContexts.Add(FActorContainerID::GetMainContainerID(), MoveTemp(InParams.InstancingContext));

		for (FWorldPartitionRuntimeCellObjectMapping& PackageObjectMapping : InParams.ActorPackages)
		{
			FLinkerInstancingContext* Context = LinkerInstancingContexts.Find(PackageObjectMapping.ContainerID);
			if (!Context)
			{
				check(!PackageObjectMapping.ContainerID.IsMainContainer());
		
				FLinkerInstancingContext& NewContext = LinkerInstancingContexts.Add(PackageObjectMapping.ContainerID);

				// Make sure here we don't remap the SoftObjectPaths through the linker when loading the embedded actor packages. 
				// A remapping will happen in the packaged loaded callback later in this method.
				NewContext.SetSoftObjectPathRemappingEnabled(false); 
		
				// Don't load external objects as we are going to individually load them
				NewContext.AddTag(ULevel::DontLoadExternalObjectsTag);
				NewContext.AddTag(ULevel::DontLoadExternalFoldersTag);

				// We only want unique packages for non-OFPA actors, @todo_ow: remove this and duplicate actors from non-OFPA levels instead of renaming.
				const bool bUniquePackage = !PackageObjectMapping.Package.ToString().Contains(FPackagePath::GetExternalActorsFolderName());
				const FName ContainerPackageInstanceName(GetContainerPackage(PackageObjectMapping.ContainerID, PackageObjectMapping.ContainerPackage.ToString(), InParams.OuterWorld, bUniquePackage));
				NewContext.AddPackageMapping(PackageObjectMapping.ContainerPackage, ContainerPackageInstanceName);

				// Add dynamic mapping function to handle cross-cell external actor references within this sub-container.
				// During linker load, references can pull in actors from other cells of the same level instance (e.g., editor-only properties). 
				// This function generates the correct instanced package name on-demand for any external actor package under this
				// container's path, avoiding PopulateInstancingContext warnings about unmapped packages with instanced outers.
				const FString ExternalActorsPathStr = ULevel::GetExternalActorsPath(PackageObjectMapping.ContainerPackage.ToString());
				const FString InstancedLevelPackageNameStr = ContainerPackageInstanceName.ToString();
				
				NewContext.AddPackageMappingFunc([ExternalActorsPathStr, InstancedLevelPackageNameStr](FName Original)
				{
					const FString OriginalStr = Original.ToString();
					if (OriginalStr.StartsWith(ExternalActorsPathStr))
					{
						return FName(*ULevel::GetExternalActorPackageInstanceName(InstancedLevelPackageNameStr, OriginalStr));
					}
					return Original;
				});

				Context = &NewContext;
			}
		
			const FName ContainerPackageInstanceName = Context->RemapPackage(PackageObjectMapping.ContainerPackage);
			const bool bConsiderActorEditorOnly = PackageObjectMapping.bIsEditorOnly && !IsRunningCookCommandlet(); // See relevant comment in MoveExternalActorsToLevel

			if (bConsiderActorEditorOnly || PackageObjectMapping.ContainerPackage != ContainerPackageInstanceName)
			{
				const FName ActorPackageName = *FPackageName::ObjectPathToPackageName(PackageObjectMapping.Package.ToString());

				// Only add PackageMappings for ExternalPackage Actors, otherwise they're invalid mappings
				if (ActorPackageName != PackageObjectMapping.ContainerPackage)
				{
					const FName ActorPackageInstanceName = bConsiderActorEditorOnly ? NAME_None : FName(*ULevel::GetExternalActorPackageInstanceName(ContainerPackageInstanceName.ToString(), ActorPackageName.ToString()));
					Context->AddPackageMapping(ActorPackageName, ActorPackageInstanceName);
				}
			}

			if (!bConsiderActorEditorOnly)
			{
				ActorPackagesToLoad.Add(&PackageObjectMapping);
			}
		}
	}

	if (ActorPackagesToLoad.IsEmpty())
	{
		InParams.CompletionCallback(true);
		return true;
	}

	struct FLoadProgress
	{
		int32 NumPendingLoadRequests;
		int32 NumFailedLoadedRequests;

		struct FDeferredGameWorldOverride
		{
			TWeakObjectPtr<AActor> Actor;
			FActorPropertyOverride Override;
			FActorContainerID ContainerID;
			FString SourceWorldPath;
			FTransform ContainerTransform;
			FTransform EditorOnlyParentTransform;
			FGuid ActorInstanceGuid;
		};
		TArray<FDeferredGameWorldOverride> DeferredGameWorldOverrides;
	};

	TSharedPtr<FLoadProgress> LoadProgress = MakeShared<FLoadProgress>();
	LoadProgress->NumPendingLoadRequests = ActorPackagesToLoad.Num();
	LoadProgress->NumFailedLoadedRequests = 0;

	for (FWorldPartitionRuntimeCellObjectMapping* PackageObjectMapping : ActorPackagesToLoad)
	{
		const FName PackageToLoad(*FPackageName::ObjectPathToPackageName(PackageObjectMapping->Package.ToString()));
		const FLinkerInstancingContext& ContainerInstancingContext = LinkerInstancingContexts.FindChecked(PackageObjectMapping->ContainerID);
		const FName PackageName = ContainerInstancingContext.RemapPackage(PackageToLoad);

		if (!PackageObjectMapping->ContainerID.IsMainContainer())
		{
			const FName ContainerPackageName = ContainerInstancingContext.RemapPackage(PackageObjectMapping->ContainerPackage);
			if (UPackage* ContainerPackage = FindPackage(nullptr, *ContainerPackageName.ToString()))
			{
				// If container package is already loaded, add a reference, to make sure it's not trashed before completion callback is called
				InParams.PackageReferencer->AddReference(ContainerPackage);
			}
		}

		FLoadPackageAsyncDelegate CompletionCallback = FLoadPackageAsyncDelegate::CreateLambda([LoadProgress, PackageObjectMapping, LoadedOverrides = InLoadedPropertyOverrides, PackageReferencer = InParams.PackageReferencer, WeakOuterWorld = MakeWeakObjectPtr(InParams.OuterWorld), DestLevel = InParams.DestLevel, CompletionCallback = InParams.CompletionCallback, bSilenceLoadFailures = InParams.bSilenceLoadFailures](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
		{
			const FName ActorName = *FPaths::GetExtension(PackageObjectMapping->Path.ToString());
			check(LoadProgress->NumPendingLoadRequests);
			LoadProgress->NumPendingLoadRequests--;

			// OuterWorld may have been destroyed while the async load was in progress. Treat this as a failed load request.
			UWorld* OuterWorld = WeakOuterWorld.Get();
			if (!OuterWorld)
			{
				UE_LOGF(LogWorldPartition, Verbose, "LoadActors completion callback invoked when OuterWorld is no longer valid. Treating this request as failed. Actor: %ls.", *ActorName.ToString());

				LoadProgress->NumFailedLoadedRequests++;
				if (!LoadProgress->NumPendingLoadRequests)
				{
					CompletionCallback(!LoadProgress->NumFailedLoadedRequests);
				}
				return;
			}

			// Deferred or immediate post-ops: annotation, transform, instance guid, soft path fixup, and resolver.
			// In game world, called in the deferred block after all overrides are applied; otherwise called immediately.
			auto ApplyContainerActorPostOps = [OuterWorld](
				AActor* InActor,
				TArray<FActorPropertyOverride>&& InPropertyOverrides,
				const FActorContainerID& InContainerID,
				const FTransform& InContainerTransform,
				const FTransform& InEditorOnlyParentTransform,
				const FGuid& InActorInstanceGuid,
				const FString& InSourceWorldPath,
				const FString& InSourceOuterWorldPath)
			{
				// Store annotation for Post RerunConstructionScript apply
				if (InPropertyOverrides.Num() > 0)
				{
					ActorPropertyOverridesAnnotation.AddAnnotation(InActor, FWorldPartitionLevelHelper::FActorPropertyOverridesAnnotation(MoveTemp(InPropertyOverrides), InContainerTransform));
				}

				USceneComponent* RootComponent = InActor->GetRootComponent();
				const bool bAbsLoc   = RootComponent && RootComponent->IsUsingAbsoluteLocation();
				const bool bAbsRot   = RootComponent && RootComponent->IsUsingAbsoluteRotation();
				const bool bAbsScale = RootComponent && RootComponent->IsUsingAbsoluteScale();

				// Apply the editor-only parent transform inline, per axis, never through
				// ApplyLevelTransform: that sets bApplyingLevelTransform, which causes
				// EntityProxyActor to skip local-transform updates — wrong here since we are
				// baking out a parent-child relationship and local coordinates must change.
				// If all components flagged as absolute, nothing to do as the parent had no contribution to transform
				if (!InEditorOnlyParentTransform.Equals(FTransform::Identity) && RootComponent && !(bAbsLoc && bAbsRot && bAbsScale))
				{
					if (!bAbsLoc)
					{
						RootComponent->SetRelativeLocation_Direct(InEditorOnlyParentTransform.TransformPosition(RootComponent->GetRelativeLocation()));
					}
					if (!bAbsRot)
					{
						RootComponent->SetRelativeRotation_Direct(InEditorOnlyParentTransform.TransformRotation(RootComponent->GetRelativeRotation().Quaternion()).Rotator());
					}
					if (!bAbsScale)
					{
						RootComponent->SetRelativeScale3D_Direct(InEditorOnlyParentTransform.GetScale3D() * RootComponent->GetRelativeScale3D());
					}
					InActor->MarkNeedsRecomputeBoundsOnceForGame();
				}

				// Apply the container transform unconditionally, absolute flags are
				// irrelevant here, as the container offset should not be affected by absolute flags
				FLevelUtils::FApplyLevelTransformParams ContainerParams(nullptr, InContainerTransform);
				ContainerParams.Actor = InActor;
				ContainerParams.bDoPostEditMove = false;
				FLevelUtils::ApplyLevelTransform(ContainerParams);

				// Set the actor's instance guid
				FSetActorInstanceGuid SetActorInstanceGuid(InActor, InActorInstanceGuid);

				// Fixup any FSoftObjectPath from this Actor (and its SubObjects) in this container to another object in the same container with a ContainerID suffix that can be remapped to
				// a Cell package in the StreamingPolicy.
				//
				// At  this point we are remapping the SubPathString and adding a ContainerID suffix so
				// '/Game/SomePath/WorldName.WorldName:PersistentLevel.ActorA' becomes
				// '/Game/SomeOtherPath/OuterWorldName.OuterWorldName:PersistentLevel.ActorA_{ContainerID}'
				FSoftObjectPathFixupArchive FixupArchive([&](FSoftObjectPath& Value)
				{
					if (!Value.IsNull() && Value.GetAssetPathString().Equals(InSourceWorldPath, ESearchCase::IgnoreCase))
					{
						if (OuterWorld->GetWorldPartition())
						{
							OuterWorld->GetWorldPartition()->ConvertContainerPathToEditorPath(InContainerID, FSoftObjectPath(Value), Value);
						}
						else
						{
							// Remap container path to source world path + container id
							Value = RemapActorPath(InContainerID, InSourceOuterWorldPath, Value);
						}
					}
				});
				FixupArchive.Fixup(InActor);

				if (IWorldPartitionObjectResolver* ObjectResolver = Cast<IWorldPartitionObjectResolver>(InActor))
				{
					ObjectResolver->SetWorldPartitionResolveData(FWorldPartitionResolveData(InContainerID, FTopLevelAssetPath(InSourceWorldPath)));
				}
			};

			AActor* Actor = nullptr;

			if (LoadedPackage)
			{
				if (LoadedPackage->ContainsMap())
				{
					if (UWorld* LoadedWorld = UWorld::FindWorldInPackage(LoadedPackage))
					{
						Actor = FindObject<AActor>(LoadedWorld->PersistentLevel, *ActorName.ToString());
					}
				}
				else
				{
					Actor = FindObject<AActor>(LoadedPackage, *ActorName.ToString());
				}
			}

			if (Actor)
			{
				const UWorld* ContainerWorld = PackageObjectMapping->ContainerID.IsMainContainer() ? OuterWorld : Actor->GetTypedOuter<UWorld>();
				
				TOptional<FName> SrcActorFolderPath;

				// Make sure Source level actor folder fixup was called
				if (ContainerWorld->PersistentLevel->IsUsingActorFolders())
				{ 
					if (!ContainerWorld->PersistentLevel->LoadedExternalActorFolders.IsEmpty())
					{
						ContainerWorld->PersistentLevel->bFixupActorFoldersAtLoad = false;
						ContainerWorld->PersistentLevel->FixupActorFolders();
					}

					// Since actor's level doesn't necessarily uses actor folders, access Folder Guid directly
					const bool bDirectAccess = true;
					const FGuid ActorFolderGuid = Actor->GetFolderGuid(bDirectAccess);
					// Resolve folder guid from source container level and resolve/backup the folder path
					UActorFolder* SrcFolder = ContainerWorld->PersistentLevel->GetActorFolder(ActorFolderGuid);
					SrcActorFolderPath = SrcFolder ? SrcFolder->GetPath() : NAME_None;
				}

				if (!PackageObjectMapping->ContainerID.IsMainContainer())
				{					
					// Add Cache handle on world so it gets unloaded properly
					PackageReferencer->AddReference(ContainerWorld->GetPackage());
										
					// We only care about the source paths here
					FString SourceWorldPath, DummyUnusedPath;
					// Verify that it is indeed an instanced world
					verify(ContainerWorld->GetSoftObjectPathMapping(SourceWorldPath, DummyUnusedPath));
					FString SourceOuterWorldPath;
					OuterWorld->GetSoftObjectPathMapping(SourceOuterWorldPath, DummyUnusedPath);

					// Rename through UObject to avoid changing Actor's external packaging and folder properties
					Actor->UObject::Rename(*AddActorContainerID(PackageObjectMapping->ContainerID, Actor->GetName()), DestLevel, REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

					// Handle child actors
					Actor->ForEachComponent<UChildActorComponent>(true, [DestLevel = DestLevel, PackageObjectMapping](UChildActorComponent* ChildActorComponent)
					{
						if (AActor* ChildActor = ChildActorComponent->GetChildActor())
						{
							ChildActor->UObject::Rename(*AddActorContainerID(PackageObjectMapping->ContainerID, ChildActor->GetName()), DestLevel, REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
						}
					});

					// Apply Pre-ConstructionScript Properties
					TArray<FActorPropertyOverride> ActorPropertyOverrides;
					bool bHasDeferredOverrides = false;
					for (auto ActorOverrideMapping : PackageObjectMapping->PropertyOverrides)
					{
						if (const UWorldPartitionPropertyOverride*const* LoadedOverride = LoadedOverrides.PropertyOverrides.Find(ActorOverrideMapping.OwnerContainerID))
						{
							if (const FContainerPropertyOverride* ContainerOverride = (*LoadedOverride)->PropertyOverridesPerContainer.Find(ActorOverrideMapping.ContainerPath))
							{
								if (const FActorPropertyOverride* ActorOverride = ContainerOverride->ActorOverrides.Find(Actor->GetActorGuid()))
								{
									const bool bConstructionScriptProperties = false;

									// Without DestLevel, SoftObjectPathTable fixup won't work anyway, so no point in deferring
									if (OuterWorld->IsGameWorld() && DestLevel)
									{
										// In PIE/standalone, we need to fix SoftObjectPathTable for raw pointers. But we also need to defer ApplyPropertyOverrides until all actors have been loaded with
										// their ContainerID-suffixed names, otherwise remapping may fail - this is because overrides are serialized through ResolveObject() which needs the final path.
										// See FWorldPartitionPropertyOverrideArchive::operator<<(UObject*& Obj)
										bHasDeferredOverrides = true;
										FLoadProgress::FDeferredGameWorldOverride& Deferred = LoadProgress->DeferredGameWorldOverrides.AddDefaulted_GetRef();
										Deferred.Actor = Actor;
										Deferred.Override = *ActorOverride;
										Deferred.ContainerID = PackageObjectMapping->ContainerID;
										Deferred.SourceWorldPath = SourceWorldPath;
										Deferred.ContainerTransform = PackageObjectMapping->ContainerTransform;
										Deferred.EditorOnlyParentTransform = PackageObjectMapping->EditorOnlyParentTransform;
										Deferred.ActorInstanceGuid = PackageObjectMapping->ActorInstanceGuid;
									}
									else
									{
										UWorldPartitionPropertyOverride::ApplyPropertyOverrides(ActorOverride, Actor, bConstructionScriptProperties);

										// Store ActorOverride for Post Construction Script apply
										ActorPropertyOverrides.Add(*ActorOverride);
									}
								}
							}
						}
					}

					if (!bHasDeferredOverrides)
					{
						ApplyContainerActorPostOps(Actor, MoveTemp(ActorPropertyOverrides), PackageObjectMapping->ContainerID, PackageObjectMapping->ContainerTransform, PackageObjectMapping->EditorOnlyParentTransform, PackageObjectMapping->ActorInstanceGuid, SourceWorldPath, SourceOuterWorldPath);
					}
				}
				else if (!PackageObjectMapping->EditorOnlyParentTransform.Equals(FTransform::Identity))
				{
					const FTransform& EditorOnlyParentTransform = PackageObjectMapping->EditorOnlyParentTransform;

					USceneComponent* RootComponent = Actor->GetRootComponent();
					const bool bAbsLoc   = RootComponent && RootComponent->IsUsingAbsoluteLocation();
					const bool bAbsRot   = RootComponent && RootComponent->IsUsingAbsoluteRotation();
					const bool bAbsScale = RootComponent && RootComponent->IsUsingAbsoluteScale();

					// Apply the editor-only parent transform inline, per axis, never through
					// ApplyLevelTransform: that sets bApplyingLevelTransform, which causes
					// EntityProxyActor to skip local-transform updates - wrong here since we are
					// baking out a parent-child relationship and local coordinates must change.
					// If all components flagged as absolute, nothing to do as the parent had no contribution to transform
					if (RootComponent && !(bAbsLoc && bAbsRot && bAbsScale))
					{
						if (!bAbsLoc)
						{
							RootComponent->SetRelativeLocation_Direct(EditorOnlyParentTransform.TransformPosition(RootComponent->GetRelativeLocation()));
						}
						if (!bAbsRot)
						{
							RootComponent->SetRelativeRotation_Direct(EditorOnlyParentTransform.TransformRotation(RootComponent->GetRelativeRotation().Quaternion()).Rotator());
						}
						if (!bAbsScale)
						{
							RootComponent->SetRelativeScale3D_Direct(EditorOnlyParentTransform.GetScale3D() * RootComponent->GetRelativeScale3D());
						}

						// Invoke callbacks normally done by FLevelUtils::ApplyLevelTransform, since there was no LevelTransform to apply here, but there are still subsystems
						// depending on these transform updates
						Actor->MarkNeedsRecomputeBoundsOnceForGame();
						if (ULevel* ActorLevel = Actor->GetLevel())
						{
							ActorLevel->OnApplyLevelTransform.Broadcast(EditorOnlyParentTransform);
							FWorldDelegates::PostApplyLevelTransform.Broadcast(ActorLevel, EditorOnlyParentTransform);
						}
					}
				}

				// Path to use when searching for this actor in MoveExternalActorsToLevel
				PackageObjectMapping->LoadedPath = *Actor->GetPathName();

				if (DestLevel)
				{
					// Propagate resolved actor folder path
					check(!DestLevel->IsUsingActorFolders());
					if (SrcActorFolderPath.IsSet())
					{
						Actor->SetFolderPath(*SrcActorFolderPath);
					}

					DestLevel->Actors.Add(Actor);
					checkf(Actor->GetLevel() == DestLevel, TEXT("Levels mismatch, got : %s, expected: %s\nActor: %s\nActorFullName: %s\nActorPackage: %s"), *DestLevel->GetFullName(), *Actor->GetLevel()->GetFullName(), *Actor->GetActorNameOrLabel(), *Actor->GetFullName(), *Actor->GetPackage()->GetFullName());

					// Handle child actors
					Actor->ForEachComponent<UChildActorComponent>(true, [DestLevel = DestLevel](UChildActorComponent* ChildActorComponent)
					{
						if (AActor* ChildActor = ChildActorComponent->GetChildActor())
						{
							DestLevel->Actors.Add(ChildActor);
							check(ChildActor->GetLevel() == DestLevel);
						}
					});
				}

				UE_LOGF(LogWorldPartition, Verbose, " ==> Loaded %ls (remaining: %d)", *Actor->GetFullName(), LoadProgress->NumPendingLoadRequests);
			}
			else
			{
				if (!bSilenceLoadFailures)
				{
					if (LoadedPackage)
					{
						UE_LOGF(LogWorldPartition, Warning, "\tPackage Content for '%ls:", *LoadedPackage->GetName());
						ForEachObjectWithOuter(LoadedPackage, [](UObject* Object)
						{
							UE_LOGF(LogWorldPartition, Warning, "\t\tObject %ls, Flags 0x%llx", *Object->GetPathName(), static_cast<uint64>(Object->GetFlags()));
							return true;
						}, EGetObjectsFlags::IncludeNestedObjects);
					}

					ensureMsgf(false, TEXT("Failed to find actor '%s' in package '%s'."), *ActorName.ToString(), *LoadedPackageName.ToString());
				}

				LoadProgress->NumFailedLoadedRequests++;
			}

			if (LoadProgress->NumPendingLoadRequests == 0)
			{
				// All actor packages have been loaded and renamed into DestLevel. Now apply all property overrides that were deferred.
				// At this point FSoftObjectPath::ResolveObject() should succeed for all entries.
				if (!LoadProgress->DeferredGameWorldOverrides.IsEmpty() && OuterWorld && DestLevel)
				{
					// Group deferred overrides by actor because editing annotations created in a previous iteration is fiddly.
					TMap<AActor*, TArray<int32>> ActorToOverrideIndices;
					for (int32 OverrideIdx = 0; OverrideIdx < LoadProgress->DeferredGameWorldOverrides.Num(); ++OverrideIdx)
					{
						if (AActor* DeferredActor = LoadProgress->DeferredGameWorldOverrides[OverrideIdx].Actor.Get())
						{
							ActorToOverrideIndices.FindOrAdd(DeferredActor).Add(OverrideIdx);
						}
					}

					FString DeferredSourceOuterWorldPath;
					{
						FString DummyUnused;
						OuterWorld->GetSoftObjectPathMapping(DeferredSourceOuterWorldPath, DummyUnused);
					}

					for (const TPair<AActor*, TArray<int32>>& Override : ActorToOverrideIndices)
					{
						AActor* DeferredActor = Override.Key;
						const TArray<int32>& OverrideIndices = Override.Value;
						TArray<FActorPropertyOverride> FixedOverrides;
						for (int32 OverrideIdx : OverrideIndices)
						{
							FLoadProgress::FDeferredGameWorldOverride& Deferred = LoadProgress->DeferredGameWorldOverrides[OverrideIdx];
							FActorPropertyOverride FixedOverride = Deferred.Override;

							// Fix SoftObjectPathTable for raw pointers
							for (FSoftObjectPath& Path : FixedOverride.ReferenceTable.SoftObjectPathTable)
							{
								if (!Path.IsNull() && Path.GetAssetPathString().Equals(Deferred.SourceWorldPath, ESearchCase::IgnoreCase))
								{
									// Example conversion
									// From: "/Game/LI_MyLevelInstance.LI_MyLevelInstance:PersistentLevel.StaticMeshActor_UAID_387C7649309B16A202_2034137445"
									// To:   "/Memory/UEDPIE_0_LV_MainLevel_9BSKPGTV8STCZQCVAAPOCF3VX.LV_MainLevel:PersistentLevel.StaticMeshActor_UAID_387C7649309B16A202_a141e76736713457_2034137445"
									const FString SubPath = Path.GetSubPathString();
									const int32 FirstDot = SubPath.Find(TEXT("."));
									if (FirstDot != INDEX_NONE)
									{
										const int32 SecondDot = SubPath.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, FirstDot + 1);
										const FString ActorLocalName = (SecondDot != INDEX_NONE)
											? SubPath.Mid(FirstDot + 1, SecondDot - FirstDot - 1)
											: SubPath.Mid(FirstDot + 1);
										const FString SubObjectRemainder = (SecondDot != INDEX_NONE) ? SubPath.Mid(SecondDot) : TEXT("");
										const FString RenamedActorName = AddActorContainerID(Deferred.ContainerID, ActorLocalName);
										const FSoftObjectPath FixedPath(DestLevel->GetWorld()->GetPathName() + TEXT(":") + DestLevel->GetName() + TEXT(".") + RenamedActorName + SubObjectRemainder);
										if (FixedPath.ResolveObject())
										{
											Path = FixedPath;
										}
									}
								}
							}

							// And ApplyPropertyOverrides now that ResolveObject will find the object being pointed to, since they all actors were iterated and renamed
							const bool bConstructionScriptProperties = false;
							UWorldPartitionPropertyOverride::ApplyPropertyOverrides(&FixedOverride, DeferredActor, bConstructionScriptProperties);
							FixedOverrides.Add(MoveTemp(FixedOverride));
						}

						const FLoadProgress::FDeferredGameWorldOverride& FirstDeferred = LoadProgress->DeferredGameWorldOverrides[OverrideIndices[0]];
						ApplyContainerActorPostOps(DeferredActor, MoveTemp(FixedOverrides), FirstDeferred.ContainerID, FirstDeferred.ContainerTransform, FirstDeferred.EditorOnlyParentTransform, FirstDeferred.ActorInstanceGuid, FirstDeferred.SourceWorldPath, DeferredSourceOuterWorldPath);
					}
				}

				CompletionCallback(!LoadProgress->NumFailedLoadedRequests);
			}
		});

		// If the package already exists, we are loading actors from a non-OFPA level package, just fire the completion callback in this case as all actors are
		// already loaded in.
		if (UPackage* ExistingPackage = FindPackage(nullptr, *PackageName.ToString()))
		{
			CompletionCallback.Execute(PackageToLoad, ExistingPackage, EAsyncLoadingResult::Succeeded);
		}
		else
		{
			EPackageFlags PackageFlags = PKG_None;
			int32 PIEInstanceID = INDEX_NONE;

			// Compute PIE flags/instance only if a DestLevel is provided
			if (InParams.DestLevel)
			{
				const UPackage* DestPackage = InParams.DestLevel->GetPackage();
				PackageFlags = DestPackage->HasAnyPackageFlags(PKG_PlayInEditor) ? PKG_PlayInEditor : PKG_None;
				PIEInstanceID = DestPackage->GetPIEInstanceID();
			}

			// Enqueue async load
			const FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(PackageToLoad);
			const int32 RequestID = ::LoadPackageAsync(PackagePath, PackageName, CompletionCallback, PackageFlags, PIEInstanceID, 0, &ContainerInstancingContext);
			if (InParams.AsyncRequestIDs)
			{
				InParams.AsyncRequestIDs->Add(RequestID);
			}

			// If a sync load was requested, wait for it
			if (!InParams.bLoadAsync)
			{
				FlushAsyncLoading(RequestID);
			}
		}
	}

	return (LoadProgress->NumPendingLoadRequests == 0);
}

void FWorldPartitionLevelHelper::SetForcePackageTrashingAtCleanup(ULevel* Level, bool bForcePackageTrashingAtCleanup)
{
	Level->bForcePackageTrashingAtCleanup = bForcePackageTrashingAtCleanup;
}

#endif
