// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"

#include "Engine/LevelStreaming.h"
#include "WorldPartition/WorldPartition.h"
#include "UObject/PropertyPortFlags.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionLevelStreamingDynamic)

#if WITH_EDITOR
#include "Misc/PackageName.h"
#include "Engine/Level.h"
#include "Misc/PathViews.h"
#include "ContentStreaming.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "LevelUtils.h"
#include "Misc/PlayInEditorLoadingScope.h"
#endif

#define LOCTEXT_NAMESPACE "World"

/*-----------------------------------------------------------------------------
	UWorldPartitionLevelStreamingDynamic
-----------------------------------------------------------------------------*/

UWorldPartitionLevelStreamingDynamic::UWorldPartitionLevelStreamingDynamic(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, RuntimeLevel(nullptr)
	, bLoadRequestInProgress(false)
	, bLoadSucceeded(false)
#endif
	, bShouldBeAlwaysLoaded(false)
	, bHasSetLevelTransform(false)
#if WITH_EDITORONLY_DATA
	, bShouldPerformStandardLevelLoading(false)
#endif
{
#if WITH_EDITOR
	SetShouldBeVisibleInEditor(false);
#endif
}

/**
 * Initializes a UWorldPartitionLevelStreamingDynamic.
 */
void UWorldPartitionLevelStreamingDynamic::Initialize(const UWorldPartitionRuntimeLevelStreamingCell& InCell)
{
	StreamingCell = &InCell;
	UWorld* World = GetWorld();
	check(!ShouldBeLoaded());
	check((World->IsGameWorld() && !ShouldBeVisible()) || (!World->IsGameWorld() && !GetShouldBeVisibleFlag()));
	check(!WorldAsset.IsNull());

	bShouldBeAlwaysLoaded = InCell.IsAlwaysLoaded();
	StreamingPriority = 0;

	UWorld* CellOuterWorld = InCell.GetOuterWorld();
#if WITH_EDITOR
	check(ChildPackages.Num() == 0);

	UnsavedActorsContainer = InCell.UnsavedActorsContainer; 

	Initialize(CellOuterWorld, InCell.GetPackages());
#else
	OuterWorldPartition = CellOuterWorld->GetWorldPartition();
#endif

	LevelColor = InCell.GetCellDebugColor();

	UpdateShouldSkipMakingVisibilityTransactionRequest();
}

void UWorldPartitionLevelStreamingDynamic::SetLevelTransform(const FTransform& InLevelTransform)
{
	if (!bHasSetLevelTransform)
	{
		LevelTransform = InLevelTransform;
		bHasSetLevelTransform = true;
	}
}

#if WITH_EDITOR

UWorldPartitionLevelStreamingDynamic* UWorldPartitionLevelStreamingDynamic::LoadInEditor(UWorld* World, FName LevelStreamingName, const TArray<FWorldPartitionRuntimeCellObjectMapping>& InPackages)
{
	check(World->WorldType == EWorldType::Editor);
	UWorldPartitionLevelStreamingDynamic* LevelStreaming = NewObject<UWorldPartitionLevelStreamingDynamic>(World, LevelStreamingName, RF_Transient);
	
	const FString WorldPackageName = World->GetPackage()->GetPathName();
	const FStringView WorldMountPointName = FPathViews::GetMountPointNameFromPath(WorldPackageName);

	FString PackageName = FString::Printf(TEXT("Temp/%s"), *LevelStreamingName.ToString());
	TSoftObjectPtr<UWorld> WorldAsset(FSoftObjectPath(FString::Printf(TEXT("/%.*s/%s.%s"), WorldMountPointName.Len(), WorldMountPointName.GetData(), *PackageName, *World->GetName())));
	LevelStreaming->SetWorldAsset(WorldAsset);

	// Assign a dummy runtime cell to ensure that code that is testing for IsWorldPartitionRuntimeCell() behaves as expected
	UWorldPartitionRuntimeLevelStreamingCell* StreamingCell = NewObject<UWorldPartitionRuntimeLevelStreamingCell>(LevelStreaming, NAME_None, RF_Transient);
	StreamingCell->RuntimeCellData = NewObject<UWorldPartitionRuntimeCellData>(StreamingCell);
	LevelStreaming->StreamingCell = StreamingCell;
	
	LevelStreaming->LevelTransform = FTransform::Identity;
	LevelStreaming->Initialize(World, InPackages);
	LevelStreaming->SetShouldBeVisibleInEditor(true);
	World->AddStreamingLevel(LevelStreaming);
	World->FlushLevelStreaming();

	// Mark the level package as transient to prevent the editor from asking to save it.
	ULevel* Level = LevelStreaming->GetLoadedLevel();
	if (Level)
	{
		Level->GetPackage()->SetFlags(RF_Transient);
	}
	
	return LevelStreaming;
}

void UWorldPartitionLevelStreamingDynamic::UnloadFromEditor(UWorldPartitionLevelStreamingDynamic* InLevelStreaming)
{
	UWorld* World = InLevelStreaming->GetWorld();
	check(World->WorldType == EWorldType::Editor);

	InLevelStreaming->SetShouldBeVisibleInEditor(false);
	InLevelStreaming->SetIsRequestingUnloadAndRemoval(true);

	if (ULevel* Level = InLevelStreaming->GetLoadedLevel())
	{
		World->RemoveLevel(Level);
		World->FlushLevelStreaming();

		// Destroy the package world and remove it from root
		UPackage* Package = Level->GetPackage();
		UWorld* PackageWorld = UWorld::FindWorldInPackage(Package);
		PackageWorld->DestroyWorld(false);
	}
}

void UWorldPartitionLevelStreamingDynamic::Initialize(UWorld* OuterWorld, const TArray<FWorldPartitionRuntimeCellObjectMapping>& InPackages)
{
	ChildPackages = InPackages;

	OriginalLevelPackageName = OuterWorld->GetPackage()->GetLoadedPath().GetPackageFName();
	PackageNameToLoad = GetWorldAssetPackageFName();
	OuterWorldPartition = OuterWorld->GetWorldPartition();
}

/**
 Custom destroy (delegate removal)
 */
void UWorldPartitionLevelStreamingDynamic::BeginDestroy()
{
	if (IsValid(RuntimeLevel))
	{
		RuntimeLevel->OnCleanupLevel.Remove(OnCleanupLevelDelegateHandle);
		OnCleanupLevelDelegateHandle.Reset();
	}
	PackageReferencer.RemoveReferences();
	Super::BeginDestroy();
}

/**
 * Creates a runtime level that we will use to emulate Level streaming
 */
void UWorldPartitionLevelStreamingDynamic::CreateRuntimeLevel()
{
	check(PendingUnloadLevel == nullptr);
	check(RuntimeLevel == nullptr);
	const UWorld* World = GetWorld();
	check(World && (World->IsGameWorld() || GetShouldBeVisibleInEditor()));

	// Make sure we are creating a runtime level for a cell.
	check(StreamingCell != nullptr);

	// Create streaming cell Level package
	RuntimeLevel = FWorldPartitionLevelHelper::CreateEmptyLevelForRuntimeCell(StreamingCell.Get(), World, GetWorldAsset().ToString());
	check(RuntimeLevel);

	// Force world partition level/actor packages to be trashed at cleanup
	FWorldPartitionLevelHelper::SetForcePackageTrashingAtCleanup(RuntimeLevel, true);

	// Make sure Actor Folders is disabled on generated runtime levels to avoid any problems with duplicate folders that
	// can be caused by level instances injecting their actors, which can cause duplicate folders (which only happens during PIE).
	FLevelActorFoldersHelper::SetUseActorFolders(RuntimeLevel, false);

	UPackage* RuntimeLevelPackage = RuntimeLevel->GetPackage();
	check(RuntimeLevelPackage);

	// Set flag here as this level isn't async loaded
	RuntimeLevel->bClientOnlyVisible = bClientOnlyVisible;

	// Mark this package as a dynamic PIE package with pending external actors
	RuntimeLevelPackage->SetDynamicPIEPackagePending(true);

	// Attach ourself to Level cleanup to do our own cleanup
	OnCleanupLevelDelegateHandle = RuntimeLevel->OnCleanupLevel.AddUObject(this, &UWorldPartitionLevelStreamingDynamic::OnCleanupLevel);
}

/**
 * Overrides default StreamingLevel behavior and manually load actors and add them to the runtime Level
 */
bool UWorldPartitionLevelStreamingDynamic::RequestLevel(UWorld* InPersistentWorld, bool bInAllowLevelLoadRequests, EReqLevelBlock InBlockPolicy)
{
	if (bShouldPerformStandardLevelLoading)
	{
		return Super::RequestLevel(InPersistentWorld, bInAllowLevelLoadRequests, InBlockPolicy);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ULevelStreaming_RequestLevel);

	// Quit early in case load request already issued
	if (GetLevelStreamingState() == ELevelStreamingState::Loading)
	{
		return true;
	}

	// Previous attempts have failed, no reason to try again
	if (GetLevelStreamingState() == ELevelStreamingState::FailedToLoad)
	{
		return false;
	}

	// Can not load new level now, there is still level pending unload
	if (PendingUnloadLevel)
	{
		return false;
	}

	const FName WorldAssetPackageFName = GetWorldAssetPackageFName();
	const FString WorldAssetPackageName = GetWorldAssetPackageName();

	// Can not load new level now either, we're still processing visibility for this one
	if (LoadedLevel && (InPersistentWorld->IsLevelMakingVisible(LoadedLevel) || InPersistentWorld->IsLevelMakingInvisible(LoadedLevel)))
	{
		UE_LOGF(LogLevelStreaming, Verbose, "Delaying load of new level %ls, because still processing visibility request.", *WorldAssetPackageName);
		return false;
	}

	// Validate that our new streaming level is unique, check for clash with currently loaded streaming levels
	if (!ValidateUniqueWorldAsset(InPersistentWorld))
	{
		return false;
	}

	FScopeCycleCounterUObject Context(InPersistentWorld);

	// Try to find the package to load
	UPackage* LevelPackage = (UPackage*)StaticFindObjectFast(UPackage::StaticClass(), nullptr, WorldAssetPackageFName, EFindObjectFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);
	UWorld* FoundWorld = LevelPackage ? UWorld::FindWorldInPackage(LevelPackage) : nullptr;
	check(!FoundWorld || IsValidChecked(FoundWorld));
	check(!FoundWorld || FoundWorld->PersistentLevel);
	if (FoundWorld && FoundWorld->PersistentLevel != RuntimeLevel)
	{
		check(ULevelStreaming::ShouldReuseUnloadedButStillAroundLevels(FoundWorld->PersistentLevel));
		check(RuntimeLevel == nullptr);
		check(LoadedLevel == nullptr);
		RuntimeLevel = FoundWorld->PersistentLevel;
	}

	if (RuntimeLevel && !IsValid(RuntimeLevel))
	{
		// We're trying to reload a level that has very recently been marked for garbage collection, it might not have been cleaned up yet
		// So continue attempting to reload the package if possible
		UE_LOGF(LogLevelStreaming, Verbose, "RequestLevel: Runtime level is marked as garbage %ls", *WorldAssetPackageFName.ToString());
		return false;
	}

	// Check if currently loaded level is what we want right now
	if (LoadedLevel)
	{
		check(GetLoadedLevelPackageName() == GetWorldAssetPackageFName());
		return true;
	}

	if (RuntimeLevel)
	{
		check(ULevelStreaming::ShouldReuseUnloadedButStillAroundLevels(RuntimeLevel));
		// Reuse existing Level
		UPackage* CellLevelPackage = RuntimeLevel->GetPackage();
		check(CellLevelPackage);
		UWorld* CellWorld = UWorld::FindWorldInPackage(CellLevelPackage);
		check(CellWorld);
		check(CellWorld == FoundWorld);
		check(IsValidChecked(CellWorld));
		check(CellWorld->PersistentLevel == RuntimeLevel);
		check(CellWorld->PersistentLevel != LoadedLevel);

		// Level already exists but may have the wrong type due to being inactive before, so copy data over
		check(InPersistentWorld->IsGameWorld() || GetShouldBeVisibleInEditor());
		CellWorld->WorldType = InPersistentWorld->WorldType;
		CellWorld->PersistentLevel->OwningWorld = InPersistentWorld;

		SetLoadedLevel(RuntimeLevel);

		// Broadcast level loaded event to blueprints
		OnLevelLoaded.Broadcast();
	}
	else if (bInAllowLevelLoadRequests)
	{
		// LODPackages not supported in this mode
		check(LODPackageNames.Num() == 0);
		if (RuntimeLevel == nullptr)
		{
			check(GetLevelStreamingState() == ELevelStreamingState::Unloaded);

			check(!RuntimeLevel);
			CreateRuntimeLevel();
			check(RuntimeLevel);

			UPackage* CellLevelPackage = RuntimeLevel->GetPackage();
			check(CellLevelPackage);
			check(UWorld::FindWorldInPackage(CellLevelPackage));
			check(RuntimeLevel->OwningWorld);

			EWorldType::Type WorldType = RuntimeLevel->OwningWorld->WorldType;
			check(WorldType == EWorldType::PIE ||
				((IsRunningGame() || IsRunningDedicatedServer()) && WorldType == EWorldType::Game) ||
				((WorldType == EWorldType::Editor || WorldType == EWorldType::EditorPreview) && GetShouldBeVisibleInEditor()));

			if (IssueLoadRequests())
			{
				UWorld* OuterWorld = GetStreamingWorld();
				bool bForceSkipBlockingLoad = false;// (OuterWorld && OuterWorld->IsGameWorld() && OuterWorld->IsInstanced() && ShouldBeAlwaysLoaded());
				// Editor immediately blocks on load and we also block if background level streaming is disabled.
				if (!bForceSkipBlockingLoad && ((InBlockPolicy == AlwaysBlock) || (ShouldBeAlwaysLoaded() && InBlockPolicy != NeverBlock)))
				{
					if (IsAsyncLoading())
					{
						UE_LOGF(LogStreaming, Display, "UWorldPartitionLevelStreamingDynamic::RequestLevel(%ls) is flushing async loading", *WorldAssetPackageName);
					}

					// Finish all async loading
					// Note that FlushAsyncLoading will invoke completion callbacks, which may themselves trigger new async loading 
					// requests (FWorldPartitionLevelHelper::LoadActorsWithPropertyOverridesInternal is a good example).
					// Loop until no new requests are found in AsyncRequestIDs.
					TSet<int32> PrevRequestIDs;
					TSet<int32> CurrRequestIDs;
					TSet<int32> DiffRequestIDs;
					do
					{
						CurrRequestIDs = TSet<int32>(AsyncRequestIDs);
						DiffRequestIDs = CurrRequestIDs.Difference(PrevRequestIDs);
						if (DiffRequestIDs.Num() > 0)
						{
							FlushAsyncLoading(DiffRequestIDs.Array());
							PrevRequestIDs = MoveTemp(CurrRequestIDs);
						}
					} while (DiffRequestIDs.Num() > 0);
				}
				else
				{
					SetCurrentState(ELevelStreamingState::Loading);
				}
			}
		}
	}

	return true;
}

/**
 * Loads all objects of a runtime Level
 */
bool UWorldPartitionLevelStreamingDynamic::IssueLoadRequests()
{
	check(ShouldBeLoaded() || GetShouldBeVisibleInEditor());
	check(!HasLoadedLevel());
	check(RuntimeLevel);
	check(!bLoadRequestInProgress);
	bLoadSucceeded = false;
	bLoadRequestInProgress = true;

	auto BuildInstancingContext = [this](FLinkerInstancingContext& OutLinkInstancingContext, const TArray<FName>& InChildPackagesToDuplicate)
	{
		// Don't do SoftObjectPath remapping for PersistentLevel actors because references can end up in different cells
		OutLinkInstancingContext.SetSoftObjectPathRemappingEnabled(false);

		UPackage* RuntimePackage = RuntimeLevel->GetPackage();
		OutLinkInstancingContext.AddPackageMapping(OriginalLevelPackageName, RuntimePackage->GetFName());

		for (const FWorldPartitionRuntimeCellObjectMapping& CellObjectMapping : ChildPackages)
		{
			if (ContentBundlePaths::IsAContentBundlePath(CellObjectMapping.ContainerPackage.ToString()) ||
				FExternalDataLayerHelper::IsExternalDataLayerPath(CellObjectMapping.ContainerPackage.ToString()))
			{
				check(CellObjectMapping.ContainerPackage != CellObjectMapping.WorldPackage);
				bool bIsContainerPackageAlreadyRemapped = OutLinkInstancingContext.RemapPackage(CellObjectMapping.ContainerPackage) != CellObjectMapping.ContainerPackage;
				if (!bIsContainerPackageAlreadyRemapped)
				{
					OutLinkInstancingContext.AddPackageMapping(CellObjectMapping.ContainerPackage, RuntimePackage->GetFName());
				}
			}
		}

		for (FName ChildPackageToDuplicate : InChildPackagesToDuplicate)
		{
			// Add mapping in case we have a ChildPackageToLoad that references this package (Spatial actor references Non-spatial actor in a ContentBundle Alwaysloaded Cell)
			OutLinkInstancingContext.AddPackageMapping(ChildPackageToDuplicate, RuntimePackage->GetFName());
		}
	};
	
	ChildPackagesToLoad.Reset(ChildPackages.Num());
	TArray<FName> ChildPackagesToDuplicate;

	UWorld* World = GetWorld();
	for (FWorldPartitionRuntimeCellObjectMapping& ChildPackage : ChildPackages)
	{
		bool bNeedDup = false;
		if (ChildPackage.ContainerID.IsMainContainer())
		{
			if (UnsavedActorsContainer)
			{
				FString SubObjectName;
				FString SubObjectContext;
				if (ChildPackage.LoadedPath.ToString().Split(TEXT("."), &SubObjectContext, &SubObjectName, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					if (AActor* ActorModifiedForPIE = UnsavedActorsContainer->Actors.FindRef(*SubObjectName))
					{
						ChildPackagesToDuplicate.Add(ChildPackage.Package);
						bNeedDup = true;
					}
				}
			}
		}

		if (!bNeedDup)
		{
			ChildPackagesToLoad.Add(ChildPackage);
		}
	}
		
	FLinkerInstancingContext InstancingContext;
	BuildInstancingContext(InstancingContext, ChildPackagesToDuplicate);

	// Duplicate unsaved actors
	if (UnsavedActorsContainer)
	{
		UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(RuntimeLevel->GetPackage()->GetPIEInstanceID());

		// Set RuntimeLevel on container temporarily (for duplication) so that external code can find the Level in PostLoad code 
		TGuardValue<TObjectPtr<ULevel>> ScopeRuntimeLevel(UnsavedActorsContainer->RuntimeLevel, RuntimeLevel);
		FObjectDuplicationParameters Parameters(UnsavedActorsContainer, RuntimeLevel->GetPackage());
		Parameters.DestClass = UnsavedActorsContainer->GetClass();
		Parameters.FlagMask = RF_AllFlags & ~(RF_MarkAsRootSet | RF_MarkAsNative | RF_HasExternalPackage);
		Parameters.InternalFlagMask = EInternalObjectFlags_AllFlags;
		Parameters.DuplicateMode = EDuplicateMode::PIE;
		Parameters.PortFlags = PPF_DuplicateForPIE;
		Parameters.DuplicationSeed.Add(World->PersistentLevel, RuntimeLevel);

		UActorContainer* ActorContainerDup = (UActorContainer*)StaticDuplicateObjectEx(Parameters);

		// Add the duplicated actors to the corresponding cell level
		for (auto& ActorPair : ActorContainerDup->Actors)
		{
			ActorPair.Value->Rename(nullptr, RuntimeLevel);
		}

		ActorContainerDup->MarkAsGarbage();
	}

	OnLoadingStarted();
	auto FinalizeLoading = [this](bool bSucceeded)
	{
		OnLoadingFinished();
		check(bLoadRequestInProgress);
		bLoadRequestInProgress = false;
		bLoadSucceeded = bSucceeded;
		if (!bSucceeded)
		{
			UE_LOGF(LogLevelStreaming, Warning, "UWorldPartitionLevelStreamingDynamic::IssueLoadRequests failed %ls", *GetWorldAssetPackageName());
		}

		FinalizeRuntimeLevel();
	};

	// Load saved actors
	if (ChildPackagesToLoad.Num())
	{
		FWorldPartitionLevelHelper::FLoadActorsParams Params = FWorldPartitionLevelHelper::FLoadActorsParams()
			.SetOuterWorld(GetStreamingWorld())
			.SetDestLevel(RuntimeLevel)
			.SetActorPackages(ChildPackagesToLoad)
			.SetPackageReferencer(&PackageReferencer)
			.SetCompletionCallback(FinalizeLoading)
			.SetLoadAsync(World->IsGameWorld(), &AsyncRequestIDs)
			.SetInstancingContext(MoveTemp(InstancingContext));

		FWorldPartitionLevelHelper::LoadActors(MoveTemp(Params));
	}
	else
	{
		FinalizeLoading(true);
	}

	return bLoadRequestInProgress;
}

void UWorldPartitionLevelStreamingDynamic::OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState)
{
	Super::OnCurrentStateChanged(InPrevState, InNewState);

	if (GetWorld()->IsPlayInEditor())
	{
		if (InNewState == ELevelStreamingState::LoadedVisible)
		{
			for (AActor* Actor : GetLoadedLevel()->Actors)
			{
				FWorldPartitionLevelHelper::ApplyConstructionScriptPropertyOverridesFromAnnotation(Actor);
			}
		}
	}
}

void UWorldPartitionLevelStreamingDynamic::FinalizeRuntimeLevel()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionLevelStreamingDynamic::FinalizeRuntimeLevel);

	check(!HasLoadedLevel());
	check(RuntimeLevel);
	check(!bLoadRequestInProgress);

	if (IsEngineExitRequested())
	{
		PackageReferencer.RemoveReferences();
		return;
	}

	// For RuntimeLevel's world NetGUID to be valid, make sure to flag bIsNameStableForNetworking so that IsNameStableForNetworking() returns true. (see FNetGUIDCache::SupportsObject)
	UWorld* OuterWorld = RuntimeLevel->GetTypedOuter<UWorld>();
	OuterWorld->bIsNameStableForNetworking = true;

	// Remap soft object paths
	if (StreamingCell.IsValid() && !StreamingCell->GetIsHLOD())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FixupIDs);

		// Here we remap SoftObjectPaths so that they are mapped from the PersistentLevel Package to the Cell Packages using the mapping built by the policy
		if (OuterWorld->IsPlayInEditor())
		{
			int32 PIEInstanceID = GetPackage()->GetPIEInstanceID();
			check(PIEInstanceID != INDEX_NONE);

			RuntimeLevel->FixupForPIE(PIEInstanceID, [&](int32 InPIEInstanceID, FSoftObjectPath& ObjectPath)
			{
				// Remap Runtime Level's SoftObjectPath before each PIE Fixup to avoid doing 2 passes of serialization
				OuterWorldPartition->RemapSoftObjectPath(ObjectPath);
			});
		}
		else if (OuterWorld->IsGameWorld())
		{
			check(IsRunningGame() || IsRunningDedicatedServer());
			// Remap Runtime Level's SoftObjectPaths
			FWorldPartitionLevelHelper::RemapLevelSoftObjectPaths(RuntimeLevel, OuterWorldPartition.Get());
		}
	}

	// Execute the cells transformers on the cell's level
	if (StreamingCell.IsValid() && OuterWorld->IsGameWorld())
	{
		OuterWorldPartition->ApplyRuntimeCellsTransformerStack(RuntimeLevel);
	}

	SetLoadedLevel(RuntimeLevel);

	// Broadcast level loaded event to blueprints
	OnLevelLoaded.Broadcast();

	RuntimeLevel->HandleLegacyMapBuildData();

	// Notify the streamer to start building incrementally the level streaming data.
	IStreamingManager::Get().AddLevel(RuntimeLevel);

	// Make sure this level will start to render only when it will be fully added to the world
	check(ShouldRequireFullVisibilityToRender());
	RuntimeLevel->bRequireFullVisibilityToRender = true;

	// Mark this package as fully loaded with regards to external objects
	RuntimeLevel->GetPackage()->SetDynamicPIEPackagePending(false);

	// Update runtime level package load time : take into account actor packages load time
	float LoadTime = RuntimeLevel->GetPackage()->GetLoadTime();
	for (AActor* Actor : RuntimeLevel->Actors)
	{
		if (UPackage* ActorPackage = Actor ? Actor->GetExternalPackage() : nullptr)
		{
			LoadTime += ActorPackage->GetLoadTime();
		}
	}
	RuntimeLevel->GetPackage()->SetLoadTime(LoadTime);

	PackageReferencer.RemoveReferences();
}

/**
 * Called by ULevel::CleanupLevel (which is callbed by FLevelStreamingGCHelper::PrepareStreamedOutLevelsForGC for this class)
 */
void UWorldPartitionLevelStreamingDynamic::OnCleanupLevel()
{
	PackageReferencer.RemoveReferences();

	if (RuntimeLevel)
	{
		check(OnCleanupLevelDelegateHandle.IsValid());
		RuntimeLevel->OnCleanupLevel.Remove(OnCleanupLevelDelegateHandle);
		OnCleanupLevelDelegateHandle.Reset();
		RuntimeLevel = nullptr;
	}
	else
	{
		check(!OnCleanupLevelDelegateHandle.IsValid());
	}
}

// Overriding base class to make sure the world outliner doesn't show runtime cell levels as root object.
// This could become an option in the world outliner when running PIE.
TOptional<FFolder::FRootObject> UWorldPartitionLevelStreamingDynamic::GetFolderRootObject() const
{
	if (UWorld* World = GetWorld())
	{
		return FFolder::GetWorldRootFolder(World).GetRootObject();
	}
	return FFolder::GetInvalidRootObject();
}

#endif

/*
 * Load StreamingLevel without adding it to world 
 */
void UWorldPartitionLevelStreamingDynamic::Load()
{
	UE_LOGF(LogLevelStreaming, Verbose, "UWorldPartitionLevelStreamingDynamic::Loading %ls", *GetWorldAssetPackageName());

	check(!ShouldBeLoaded());
	
	SetShouldBeLoaded(true);
	SetShouldBeVisible(false);
	SetIsRequestingUnloadAndRemoval(false);

	UWorld* PlayWorld = GetWorld();
	check(PlayWorld && PlayWorld->IsGameWorld());
	PlayWorld->AddUniqueStreamingLevel(this);
}

/*
 * Unload StreamingLevel
 */
void UWorldPartitionLevelStreamingDynamic::Unload()
{
	UE_LOGF(LogLevelStreaming, Verbose, "UWorldPartitionLevelStreamingDynamic::Unloading %ls", *GetWorldAssetPackageName());

	check(ShouldBeLoaded() || (GetLevelStreamingState() == ELevelStreamingState::FailedToLoad));

	SetShouldBeLoaded(false);
	SetShouldBeVisible(false);
	SetIsRequestingUnloadAndRemoval(true);
}

/**
  * Activates StreamingLevel by making sure it's in the World's StreamingLevels and that it should be loaded & visible
  */
void UWorldPartitionLevelStreamingDynamic::Activate()
{
	UE_LOGF(LogLevelStreaming, Verbose, "UWorldPartitionLevelStreamingDynamic::Activating %ls", *GetWorldAssetPackageName());

	check(!ShouldBeVisible());

	// Make sure we are in the correct state
	SetShouldBeLoaded(true);
	SetShouldBeVisible(true);
	SetIsRequestingUnloadAndRemoval(false);

	// Add ourself to the list of Streaming Level of the world
	UWorld* PlayWorld = GetWorld();
	check(PlayWorld && PlayWorld->IsGameWorld());
	PlayWorld->AddUniqueStreamingLevel(this);
}

/**
 * Deactivates StreamingLevel (Remove from world, keep loaded)
 */
void UWorldPartitionLevelStreamingDynamic::Deactivate()
{
	UE_LOGF(LogLevelStreaming, Verbose, "UWorldPartitionLevelStreamingDynamic::Deactivating %ls", *GetWorldAssetPackageName());

	check(ShouldBeLoaded() || (GetLevelStreamingState() == ELevelStreamingState::FailedToLoad));
	check(ShouldBeVisible() || (GetLevelStreamingState() == ELevelStreamingState::FailedToLoad));

	SetShouldBeVisible(false);
}

UWorld* UWorldPartitionLevelStreamingDynamic::GetStreamingWorld() const
{
	// For UWorldPartitionLevelStreamingDynamic the StreamingWorld is the world to which the OuterWorldPartition is outered.
	// This World can be used to resolved SoftObjectPaths between cells.
	return OuterWorldPartition.IsValid() ? OuterWorldPartition->GetTypedOuter<UWorld>() : nullptr;
}

bool UWorldPartitionLevelStreamingDynamic::CanChangeVisibility(bool bMakeVisible) const
{
	const ENetMode NetMode = GetWorld()->GetNetMode();
	if (NetMode != NM_DedicatedServer)
	{
		if (const UWorldPartitionRuntimeLevelStreamingCell* RuntimeLevelStreamingCell = StreamingCell.Get())
		{
			// Source cells that aren't HLOD can always be made visible
			bool bAlwaysAllowVisibilityChange = bMakeVisible && !RuntimeLevelStreamingCell->GetIsHLOD();

			if (!bAlwaysAllowVisibilityChange)
			{
				if (const UWorld* OuterWorld = RuntimeLevelStreamingCell->GetOuterWorld())
				{
					if (const UWorldPartition* WorldPartition = OuterWorld->GetWorldPartition())
					{
						if (UWorldPartitionHLODRuntimeSubsystem* HLODSubsystem = GetWorld()->GetSubsystem<UWorldPartitionHLODRuntimeSubsystem>())
						{
							if (bMakeVisible)
							{
								return HLODSubsystem->CanMakeVisible(RuntimeLevelStreamingCell);
							}
							else
							{
								return HLODSubsystem->CanMakeInvisible(RuntimeLevelStreamingCell);
							}
						}
					}
				}
			}
		}
	}

	return true;
}

bool UWorldPartitionLevelStreamingDynamic::RequestVisibilityChange(bool bVisible)
{
	const bool bSuperAllowsVisibilityChange = Super::RequestVisibilityChange(bVisible);
	const bool bAllowVisibilityChange = CanChangeVisibility(bVisible);
	return bSuperAllowsVisibilityChange && bAllowVisibilityChange;
}

bool UWorldPartitionLevelStreamingDynamic::ShouldBlockOnUnload() const
{
	if (Super::ShouldBlockOnUnload())
	{
		return true;
	}

	// When world partition cannot stream (anymore), return true so that RemoveFromWorld of this level is not incremental. 
	// This guarantees that unloaded instanced wp levels fully unload their cell levels.
	if (OuterWorldPartition.IsValid() && !OuterWorldPartition->CanStream())
	{
		return true;
	}

	return false;
}

void UWorldPartitionLevelStreamingDynamic::UpdateShouldSkipMakingVisibilityTransactionRequest()
{
	const UWorldPartitionRuntimeCell* Cell = GetWorldPartitionRuntimeCell();
	if (ensure(Cell && OuterWorldPartition.IsValid()))
	{
		// It is safe to skip client MakingVisibility transaction requests for cells without data layers when world partition server streaming is disabled
		bSkipClientUseMakingVisibleTransactionRequest = !(OuterWorldPartition->IsServerStreamingEnabled() || Cell->HasDataLayers());
		// We always need the MakeInvisibleTransactionRequest for levels that might have replicated actors associated with them including dynamically spawned actors.
		bSkipClientUseMakingInvisibleTransactionRequest = bSkipClientUseMakingVisibleTransactionRequest && Cell->GetClientOnlyVisible();
	}
}

#if !WITH_EDITOR
void UWorldPartitionLevelStreamingDynamic::PostLoad()
{
	Super::PostLoad();

	// UWorldPartitionLevelStreamingDynamic::Initialize is not called at runtime (except for content bundles)
	UpdateShouldSkipMakingVisibilityTransactionRequest();
}
#endif

#undef LOCTEXT_NAMESPACE

