// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * WorldPartitionLevelStreamingPolicy implementation
 */

#include "WorldPartition/WorldPartitionLevelStreamingPolicy.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "Engine/Level.h"
#include "Engine/Canvas.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "Misc/PackageName.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionLevelStreamingPolicy)

void UWorldPartitionLevelStreamingPolicy::ForEachActiveRuntimeCell(TFunctionRef<void(const UWorldPartitionRuntimeCell*)> Func) const
{
	UWorld* World = WorldPartition->GetWorld();
	for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
	{
		if (const UWorldPartitionRuntimeCell* Cell = Cast<const UWorldPartitionRuntimeCell>(LevelStreaming->GetWorldPartitionCell()))
		{
			Func(Cell);
		}
	}
}

bool UWorldPartitionLevelStreamingPolicy::IsStreamingCompleted(const TArray<FWorldPartitionStreamingSource>* InStreamingSources) const
{
	const UWorld* World = GetWorld();
	check(World);
	check(World->IsGameWorld());

	if (!Super::IsStreamingCompleted(InStreamingSources))
	{
		return false;
	}

	if (!InStreamingSources)
	{
		// Also verify that there's no remaining activity (mainly for unloading) 
		// waiting to be processed on level streaming of world partition runtime cells
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			ULevel* Level = StreamingLevel ? StreamingLevel->GetLoadedLevel() : nullptr;
			if (Level && Level->IsWorldPartitionRuntimeCell() && StreamingLevel->IsStreamingStatePending())
			{
				return false;
			}
		}
	}
	return true;
}

#if WITH_EDITOR

FString UWorldPartitionLevelStreamingPolicy::GetCellPackagePath(const FName& InCellName, const UWorld* InWorld)
{
	if (InWorld->IsGameWorld())
	{
		if (InWorld->IsPlayInEditor() || InWorld->IsPlayInPreview())
		{
			// Set as memory package to avoid wasting time in FPackageName::DoesPackageExist
			return FString::Printf(TEXT("/Memory/%s_%s"), *FPackageName::GetShortName(InWorld->GetPackage()), *InCellName.ToString());
		}
		
		// This is for non-PIE mode in editor, such as with -game and -server. It kind of mimics OFPAs being
		// flattened into the level list upon cooking.
		return FString::Printf(TEXT("%s/%s"), *InWorld->GetPackage()->GetName(), *InCellName.ToString());
	}

	return FString::Printf(TEXT("/%s"), *InCellName.ToString());
}

TSubclassOf<UWorldPartitionRuntimeCell> UWorldPartitionLevelStreamingPolicy::GetRuntimeCellClass() const
{
	return UWorldPartitionRuntimeLevelStreamingCell::StaticClass();
}

void UWorldPartitionLevelStreamingPolicy::PrepareActorToCellRemapping()
{
	FString SourceWorldPath, DummyUnusedPath;
	WorldPartition->GetTypedOuter<UWorld>()->GetSoftObjectPathMapping(SourceWorldPath, DummyUnusedPath);
	SourceWorldAssetPath = FTopLevelAssetPath(SourceWorldPath);
	
	// Build Actor-to-Cell remapping
	// The use cases for remapping are the following:
	//	- Spatially loaded or Datalayer Actors from the main World Partition map that get moved into
	//		a Streaming Cell. In this case, an actor path like '/Game/SomePath/WorldName.WorldName:PersistentLevel.ActorA'
	//		would be mapped to a cell name like 'WorldName_MainGrid_L0_X5_Y-4'.
	//	- Always loaded Actors from the main World:
	//		- In PIE they get remapped to the top level Cell 'WorldName_MainGrid_L{MAX}_X0_Y0'.
	//		- In Cook they don't need remapping as the top level Cell is
	//			the PersistentLevel (Cell->NeedsActorToCellRemapping() returns false).
	WorldPartition->RuntimeHash->ForEachStreamingCells([this, &SourceWorldPath](const UWorldPartitionRuntimeCell* Cell)
	{
		const UWorldPartitionRuntimeLevelStreamingCell* StreamingCell = Cast<const UWorldPartitionRuntimeLevelStreamingCell>(Cell);
		check(StreamingCell);

		if (Cell->NeedsActorToCellRemapping())
		{
			const FName CellName = StreamingCell->GetFName();

			for (const FWorldPartitionRuntimeCellObjectMapping& CellObjectMap : StreamingCell->GetPackages())
			{
				// Avoid remapping from any editor only references of actors that the cells may have. 
				if (!CellObjectMap.bIsEditorOnly)
				{
					const FSoftObjectPath CellActorPath = FWorldPartitionLevelHelper::RemapActorPath(CellObjectMap.ContainerID, SourceWorldPath, FSoftObjectPath(CellObjectMap.Path.ToString()));
					const FString ActorPath = CellActorPath.ToString();
					const int32 LastDotPos = ActorPath.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					check(LastDotPos != INDEX_NONE);
					SubObjectsToCellRemapping.Add(FName(*ActorPath.Mid(LastDotPos + 1)), CellName);
				}
			}
		}

		return true;
	});
}

void UWorldPartitionLevelStreamingPolicy::RemapSoftObjectPath(FSoftObjectPath& ObjectPath) const
{
	const FSoftObjectPath SrcPath(ObjectPath);
	ConvertEditorPathToRuntimePath(SrcPath, ObjectPath);
}

bool UWorldPartitionLevelStreamingPolicy::StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase& OutExternalStreamingObject)
{
	if (Super::StoreStreamingContentToExternalStreamingObject(OutExternalStreamingObject))
	{
		OutExternalStreamingObject.SubObjectsToCellRemapping = MoveTemp(SubObjectsToCellRemapping);
		OutExternalStreamingObject.ContainerResolver = MoveTemp(ContainerResolver);
		return true;
	}

	return false;
}

bool UWorldPartitionLevelStreamingPolicy::ConvertContainerPathToEditorPath(const FActorContainerID& InContainerID, const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const
{
	if (ContainerResolver.IsValid())
	{
		FString SubObjectString;
		FString SubObjectContext;
		if (InPath.GetSubPathString().Split(TEXT("."), &SubObjectContext, &SubObjectString))
		{
			if (SubObjectContext == TEXT("PersistentLevel"))
			{
				const FString* FoundEditorPath = ContainerResolver.FindContainerEditorPath(InContainerID);
				if (!FoundEditorPath)
				{
					for (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>& ExternalStreamingObject : ExternalStreamingObjects)
					{
						if (URuntimeHashExternalStreamingObjectBase* ExternalStreamingObjectPtr = ExternalStreamingObject.Get())
						{
							FoundEditorPath = ExternalStreamingObjectPtr->ContainerResolver.FindContainerEditorPath(InContainerID);
							if (FoundEditorPath)
							{
								break;
							}
						}
					}
				}

				if (FoundEditorPath)
				{
					FString SubPathString = TEXT("PersistentLevel.") + *FoundEditorPath;
					if (!SubObjectString.IsEmpty())
					{
						SubPathString += TEXT(".") + SubObjectString;
					}
					OutPath = FSoftObjectPath(SourceWorldAssetPath, SubPathString);
					return true;
				}
			}
		}

		return false;
	}
	
	// Previous behavior (remap container path to source world path + container id)
	OutPath = FWorldPartitionLevelHelper::RemapActorPath(InContainerID, SourceWorldAssetPath.ToString(), InPath);
	return true;
}

#endif

bool UWorldPartitionLevelStreamingPolicy::ConvertEditorPathToRuntimePath(const FSoftObjectPath& InPath, FSoftObjectPath& OutPath) const
{
	// Make sure to work on non-PIE path (can happen for modified actors in PIE)
	const UWorld* OuterWorld = WorldPartition->GetTypedOuter<UWorld>();
	const UPackage* OuterWorldPackage = OuterWorld->GetPackage();
	FTopLevelAssetPath WorldAssetPath(OuterWorld);

#if WITH_EDITOR
	const int32 PIEInstanceID = OuterWorldPackage->GetPIEInstanceID();
	check(PIEInstanceID == INDEX_NONE || OuterWorld->IsPlayInEditor());
	int32 PathPIEInstanceID = INDEX_NONE;
	WorldAssetPath = UWorld::RemovePIEPrefix(WorldAssetPath.ToString(), &PathPIEInstanceID);
	check(PathPIEInstanceID == INDEX_NONE || OuterWorldPackage->HasAnyPackageFlags(PKG_PlayInEditor));
	check(PathPIEInstanceID == PIEInstanceID);

	FString SrcPath = UWorld::RemovePIEPrefix(InPath.ToString(), &PathPIEInstanceID);
	checkf(PathPIEInstanceID == INDEX_NONE || PathPIEInstanceID == PIEInstanceID, TEXT("Unexpected PIEInstanceID %d while converting editor to runtime path %s for world %s with PIEInstanceID %d "), PathPIEInstanceID, *InPath.ToString(), *OuterWorld->GetFullName(), PIEInstanceID);
	const FSoftObjectPath SrcObjectPath(SrcPath);
#else
	const FSoftObjectPath SrcObjectPath(InPath);
#endif

	// Allow remapping of instanced source path or non-instanced source path
	if (SrcObjectPath.GetAssetPath() != SourceWorldAssetPath && 
		SrcObjectPath.GetAssetPath() != WorldAssetPath)
	{
		return false;
	}

	FString SubObjectString;
	FString SubObjectContext;
	if (SrcObjectPath.GetSubPathString().Split(TEXT("."), &SubObjectContext, &SubObjectString))
	{
		if (SubObjectContext == TEXT("PersistentLevel"))
		{
			FString OutSubObjectString;
			const UObject* OutLevelMountPointContext = nullptr;
			UObject* OutCellOuter = nullptr;
			const FName* CellName = FindCellNameForSubObject(SubObjectString, true, OutSubObjectString, OutLevelMountPointContext, OutCellOuter);
			FString SubPathString = SubObjectContext + TEXT(".") + OutSubObjectString;

			if (!CellName)
			{
				OutPath = FSoftObjectPath(WorldAssetPath, SubPathString);
			}
#if WITH_EDITOR
			else if (OuterWorld->IsGameWorld())
			{
				const FString PackagePath = UWorldPartitionLevelStreamingPolicy::GetCellPackagePath(*CellName, OuterWorld);
				OutPath = FString::Printf(TEXT("%s.%s:%s"), *PackagePath, *OuterWorld->GetName(), *SubPathString);
			}
#endif
			else
			{
				// In the editor, the _LevelInstance_ID is appended to the persistent level, while at runtime it is appended to each cell package, so we need to remap it there if present.
				// Also handle prefixes like "/Temp"
				FString LevelInstanceSuffix;
				FString LevelInstancePrefix;
				const FString WorldAssetPackageName = WorldAssetPath.GetPackageName().ToString();
#if WITH_EDITOR
				// Transform if necessary the world package name using the level mount point context object
				const FString SourceWorldAssetPackageName = ULevel::ResolveRootPath(SourceWorldAssetPath.GetPackageName().ToString(), OutLevelMountPointContext);
#else
				//@todo_ow: Verify if we need to make ULevel::ResolveRootPath available at runtime
				const FString SourceWorldAssetPackageName = SourceWorldAssetPath.GetPackageName().ToString();
#endif

				if (WorldAssetPackageName.Len() > SourceWorldAssetPackageName.Len())
				{
					if (const int32 Index = WorldAssetPackageName.Find(SourceWorldAssetPackageName); Index != INDEX_NONE)
					{
						LevelInstancePrefix = WorldAssetPackageName.Mid(0, Index);
						LevelInstanceSuffix = WorldAssetPackageName.Mid(Index + SourceWorldAssetPackageName.Len());
					}
				}

				OutPath = FString::Printf(TEXT("%s%s/_Generated_/%s%s.%s:%s"), *LevelInstancePrefix, *SourceWorldAssetPackageName, *(CellName->ToString()), *LevelInstanceSuffix, *WorldAssetPath.GetAssetName().ToString(), *SubPathString);
			}

#if WITH_EDITOR
			OutPath.FixupForPIE(PIEInstanceID);
#endif
			return true;
		}
	}

	return false;
}

UObject* UWorldPartitionLevelStreamingPolicy::GetSubObject(const TCHAR* SubObjectPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionLevelStreamingPolicy::GetSubObject);

	const FString SrcPath = UWorld::RemovePIEPrefix(SubObjectPath);

	FString OutSubObjectString;
	const UObject* OutLevelMountPointContext = nullptr;
	UObject* OutCellOuter = nullptr;
	if (const FName* CellName = FindCellNameForSubObject(SrcPath, /*bInResolveContainers=*/true, OutSubObjectString, OutLevelMountPointContext, OutCellOuter))
	{
		if (const UWorldPartitionRuntimeLevelStreamingCell* Cell = (UWorldPartitionRuntimeLevelStreamingCell*)StaticFindObject(UWorldPartitionRuntimeLevelStreamingCell::StaticClass(), OutCellOuter, *(CellName->ToString())))
		{
			if (UWorldPartitionLevelStreamingDynamic* LevelStreaming = Cell->GetLevelStreaming())
			{
				if (LevelStreaming->GetLoadedLevel())
				{
					return StaticFindObject(UObject::StaticClass(), LevelStreaming->GetLoadedLevel(), *OutSubObjectString);
				}
			}
		}
	}

	return nullptr;
}

bool UWorldPartitionLevelStreamingPolicy::InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	if (Super::InjectExternalStreamingObject(ExternalStreamingObject))
	{
		ExternalStreamingObjects.Add(ExternalStreamingObject);

		for (const TPair<FName, FName> Pair : ExternalStreamingObject->SubObjectsToCellRemapping)
		{
			SubObjectsToExternalStreamingObjectsRemapping.Add(Pair.Key, ExternalStreamingObject);
		}

		return true;
	}

	return false;
}

bool UWorldPartitionLevelStreamingPolicy::RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject)
{
	bool bSuccess = Super::RemoveExternalStreamingObject(ExternalStreamingObject);

	for (const TPair<FName, FName> Pair : ExternalStreamingObject->SubObjectsToCellRemapping)
	{
		if (TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>* RemovedExternalStreamingObjectPtr = SubObjectsToExternalStreamingObjectsRemapping.Find(Pair.Key))
		{
			TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase> RemovedExternalStreamingObject = *RemovedExternalStreamingObjectPtr;
			UE_CLOGF(RemovedExternalStreamingObject != ExternalStreamingObject, LogWorldPartition, Log, "Mismatch in SubObjectsToExternalStreamingObjectsRemapping map. Found: %ls. Expected: %ls",
				(RemovedExternalStreamingObject.IsValid() && RemovedExternalStreamingObject->GetPackage()) ? *RemovedExternalStreamingObject->GetPackage()->GetName() : TEXT("<Invalid>"),
				ExternalStreamingObject->GetPackage() ? *ExternalStreamingObject->GetPackage()->GetName() : TEXT("<Invalid>"));
			SubObjectsToExternalStreamingObjectsRemapping.Remove(Pair.Key);
		}
		else
		{
			UE_LOGF(LogWorldPartition, Log, "Mismatch in SubObjectsToExternalStreamingObjectsRemapping map. Subobject %ls not found", *Pair.Key.ToString());
		}
	}

	ExternalStreamingObjects.RemoveSwap(ExternalStreamingObject);

	return bSuccess;
}

void UWorldPartitionLevelStreamingPolicy::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	SIZE_T AllocatedSize = SubObjectsToCellRemapping.GetAllocatedSize();
	AllocatedSize += ContainerResolver.GetAllocatedSize();
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(AllocatedSize);
}

void UWorldPartitionLevelStreamingPolicy::DrawRuntimeCellsDetails(UCanvas* Canvas, FVector2D& Offset)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionLevelStreamingPolicy::DrawRuntimeCellsDetails);

	UWorld* World = WorldPartition->GetWorld();
	struct FCellsPerStreamingStatus
	{
		TArray<const UWorldPartitionRuntimeCell*> Cells;
	};
	FCellsPerStreamingStatus CellsPerStreamingStatus[(int32)LEVEL_StreamingStatusCount];
	ForEachActiveRuntimeCell([&CellsPerStreamingStatus](const UWorldPartitionRuntimeCell* Cell)
	{
		if (Cell->IsDebugShown())
		{
			CellsPerStreamingStatus[(int32)Cell->GetStreamingStatus()].Cells.Add(Cell);
		}
	});

	FVector2D Pos = Offset;
	const float BaseY = Offset.Y;

	float CurrentColumnWidth = 0.f;
	float MaxPosY = Pos.Y;

	auto DrawCellDetails = [&](const FString& Text, const UFont* Font, const FColor& Color)
	{
		FWorldPartitionDebugHelper::DrawText(Canvas, Text, Font, Color, Pos, &CurrentColumnWidth);
		MaxPosY = FMath::Max(MaxPosY, Pos.Y);
		if ((Pos.Y + 30) > Canvas->ClipY)
		{
			Pos.Y = BaseY;
			Pos.X += CurrentColumnWidth + 5;
			CurrentColumnWidth = 0.f;
		}
	};

	for (int32 i = 0; i < (int32)LEVEL_StreamingStatusCount; ++i)
	{
		const EStreamingStatus StreamingStatus = (EStreamingStatus)i;
		const TArray<const UWorldPartitionRuntimeCell*>& Cells = CellsPerStreamingStatus[i].Cells;
		if (Cells.Num() > 0)
		{
			const FString StatusDisplayName = *FString::Printf(TEXT("%s (%d)"), ULevelStreaming::GetLevelStreamingStatusDisplayName(StreamingStatus), Cells.Num());
			DrawCellDetails(StatusDisplayName, GEngine->GetSmallFont(), FColor::Yellow);

			const FColor Color = ULevelStreaming::GetLevelStreamingStatusColor(StreamingStatus);
			for (const UWorldPartitionRuntimeCell* Cell : Cells)
			{
				if ((StreamingStatus == EStreamingStatus::LEVEL_Loaded) ||
					(StreamingStatus == EStreamingStatus::LEVEL_MakingVisible) ||
					(StreamingStatus == EStreamingStatus::LEVEL_Visible) ||
					(StreamingStatus == EStreamingStatus::LEVEL_MakingInvisible))
				{
					float LoadTime = Cell->GetLevel() ? Cell->GetLevel()->GetPackage()->GetLoadTime() : 0.f;
					DrawCellDetails(FString::Printf(TEXT("%s (%s)"), *Cell->GetDebugName(), *FPlatformTime::PrettyTime(LoadTime)), GEngine->GetTinyFont(), Color);
				}
				else
				{
					DrawCellDetails(Cell->GetDebugName(), GEngine->GetTinyFont(), Color);
				}
			}
		}
	}

	Offset.Y = MaxPosY;
}

const FName* UWorldPartitionLevelStreamingPolicy::FindCellNameForSubObject(const FString& InSubObjectString, bool bInResolveContainers, FString& OutSubPathString, const UObject*& OutLevelMountPointContext, UObject*& OutCellOuter) const
{
	OutLevelMountPointContext = nullptr;
	OutCellOuter = nullptr;

	FString SubObjectString;
	FString SubObjectContext(InSubObjectString);
	InSubObjectString.Split(TEXT("."), &SubObjectContext, &SubObjectString);
	
	// Initialize to received value
	OutSubPathString = InSubObjectString;

	if (const FName* CellName = SubObjectsToCellRemapping.Find(*SubObjectContext))
	{
		OutCellOuter = GetOuterUWorldPartition()->RuntimeHash;
		return CellName;
	}

	if (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>* ExternalStreamingObject = SubObjectsToExternalStreamingObjectsRemapping.Find(*SubObjectContext); ExternalStreamingObject && ExternalStreamingObject->IsValid())
	{
		const URuntimeHashExternalStreamingObjectBase* ExternalStreamingObjectPtr = ExternalStreamingObject->Get();
		if (const FName* CellName = ExternalStreamingObjectPtr->SubObjectsToCellRemapping.Find(*SubObjectContext))
		{
			OutLevelMountPointContext = ExternalStreamingObjectPtr->GetLevelMountPointContextObject();
			OutCellOuter = ExternalStreamingObject->Get();
			return CellName;
		}
	}
	
	FString OutResolvedSubPathString;
	if (bInResolveContainers)
	{
		if (ContainerResolver.ResolveContainerPath(InSubObjectString, OutResolvedSubPathString))
		{
			return FindCellNameForSubObject(OutResolvedSubPathString, false, OutSubPathString, OutLevelMountPointContext, OutCellOuter);
		}

		for (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>& ExternalStreamingObject : ExternalStreamingObjects)
		{
			if (ExternalStreamingObject.IsValid())
			{
				if (ExternalStreamingObject.Get()->ContainerResolver.ResolveContainerPath(InSubObjectString, OutResolvedSubPathString))
				{
					return FindCellNameForSubObject(OutResolvedSubPathString, false, OutSubPathString, OutLevelMountPointContext, OutCellOuter);
				}
			}
		}
	}
			
	return nullptr;
}