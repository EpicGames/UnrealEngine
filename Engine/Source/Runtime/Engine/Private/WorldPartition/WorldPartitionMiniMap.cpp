// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionMiniMap.h"
#include "WorldPartition/WorldPartitionMiniMapVolume.h"
#include "WorldPartition/WorldPartitionMiniMapHelper.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Engine/Texture2D.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "EngineUtils.h"
#include "RenderUtils.h"
#include "RHI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionMiniMap)

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

AWorldPartitionMiniMap::AWorldPartitionMiniMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
	, MiniMapWorldBounds(ForceInit)
	, MiniMapTexture(nullptr)
	, WorldUnitsPerPixel(50)
	, BuilderCellSize(102400)
	, CaptureSource(ESceneCaptureSource::SCS_SceneColorHDR)
	, CaptureWarmupFrames(5)
{
}

void AWorldPartitionMiniMap::PostLoad()
{
	Super::PostLoad();

	if (MiniMapTileSize_DEPRECATED != 0)
	{
		const int32 MinimapBuilderIterativeCellSize = 102400;
		WorldUnitsPerPixel = MinimapBuilderIterativeCellSize / MiniMapTileSize_DEPRECATED;
		WorldUnitsPerPixel = FMath::Clamp(WorldUnitsPerPixel, 10, 100000);
		MiniMapTileSize_DEPRECATED = 0;
	}

	if (MiniMapTexture)
	{
		MiniMapTexture->SetFlags(RF_TextExportTransient);
	}
}

#if WITH_EDITOR
void AWorldPartitionMiniMap::CheckForErrors()
{
	Super::CheckForErrors();

	// We skip actors not part of main world (level instances)
	if (FWorldPartitionMiniMapHelper::GetWorldPartitionMiniMap(GetWorld()) != this)
	{
		return;
	}

	if (MiniMapTexture != nullptr)
	{
		const int32 MaxTextureDimension = GetMax2DTextureDimension();
		const bool bExceedMaxDimension = MiniMapTexture->GetImportedSize().GetMax() > MaxTextureDimension;
		if (!UseVirtualTexturing(GMaxRHIShaderPlatform) && bExceedMaxDimension)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ActorName"), FText::FromString(GetName()));
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_MinimapTextureSize", "{ActorName} : Texture size is too big, minimap won't be rendered. Reduce the MiniMapTileSize property or enable Virtual Texture Support for your project."), Arguments)))
				->AddToken(FMapErrorToken::Create("MinimapTextureSize"));
		}
	}

	{
		int32 EffectiveMinimapImageSizeX = 0;
		int32 EffectiveMinimapImageSizeY = 0;
		int32 EffectiveWorldUnitsPerPixel = 0;
		GetMiniMapResolution(EffectiveMinimapImageSizeX, EffectiveMinimapImageSizeY, EffectiveWorldUnitsPerPixel);

		if (EffectiveWorldUnitsPerPixel > IWorldPartitionEditorModule::Get().GetMinimapLowQualityWorldUnitsPerPixelThreshold())
		{
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_MinimapWorldUnitsPerPixel", "{0} : Expect a low quality minimap : effective world units per pixel is high ({1})."), FText::FromString(GetName()), EffectiveWorldUnitsPerPixel)))
				->AddToken(FMapErrorToken::Create("MinimapWorldUnitsPerPixel"));
		}
	}
}

FBox AWorldPartitionMiniMap::GetMiniMapWorldBounds() const
{
	FBox WorldBounds(ForceInit);

	UWorld* World = GetWorld();
	check(World);

	// Override the minimap bounds if world partition minimap volumes exists
	for (TActorIterator<AWorldPartitionMiniMapVolume> It(World); It; ++It)
	{
		if (AWorldPartitionMiniMapVolume* WorldPartitionMiniMapVolume = *It)
		{
			WorldBounds += WorldPartitionMiniMapVolume->GetBounds().GetBox();
		}
	}

	if (!WorldBounds.IsValid)
	{
		WorldBounds = World->GetWorldPartition()->GetRuntimeWorldBounds();
	}

	return WorldBounds;
}

void AWorldPartitionMiniMap::GetMiniMapResolution(int32& OutMinimapImageSizeX, int32& OutMinimapImageSizeY, int32& OutWorldUnitsPerPixel) const
{
	FBox WorldBounds = GetMiniMapWorldBounds();

	// Compute minimap image size
	OutMinimapImageSizeX = WorldBounds.GetSize().X / WorldUnitsPerPixel;
	OutMinimapImageSizeY = WorldBounds.GetSize().Y / WorldUnitsPerPixel;

	// For now, let's clamp to the maximum supported texture size
	OutMinimapImageSizeX = FMath::Min(OutMinimapImageSizeX, UTexture::GetMaximumDimensionOfNonVT());
	OutMinimapImageSizeY = FMath::Min(OutMinimapImageSizeY, UTexture::GetMaximumDimensionOfNonVT());
	OutWorldUnitsPerPixel = FMath::CeilToInt(FMath::Max(WorldBounds.GetSize().X / OutMinimapImageSizeX, WorldBounds.GetSize().Y / OutMinimapImageSizeY));
	OutMinimapImageSizeX = WorldBounds.GetSize().X / OutWorldUnitsPerPixel;
	OutMinimapImageSizeY = WorldBounds.GetSize().Y / OutWorldUnitsPerPixel;
}

#endif

#undef LOCTEXT_NAMESPACE

