// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * WorldPartitionMiniMapHelper implementation
 */

#include "WorldPartition/WorldPartitionMiniMapHelper.h"

#if WITH_EDITOR
#include "Camera/CameraTypes.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Engine/Scene.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Engine/SceneCapture2D.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "AssetCompilingManager.h"
#include "Math/OrthoMatrix.h"
#include "WorldPartition/WorldPartitionMiniMap.h"
#include "WorldPartition/WorldPartitionHelpers.h"

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionMiniMapHelper, All, All);

AWorldPartitionMiniMap* FWorldPartitionMiniMapHelper::GetWorldPartitionMiniMap(UWorld* World, bool bCreateNewMiniMap)
{
	if (!World->IsPartitionedWorld())
	{
		UE_LOGF(LogWorldPartitionMiniMapHelper, Error, "No WorldPartition Found. WorldPartition must exist to get WorldPartitionMiniMap");
		return nullptr;
	}

	ULevel* PersistentLevel = World->PersistentLevel;
	check(PersistentLevel);

	for (auto Actor : PersistentLevel->Actors)
	{
		if (Actor && Actor->IsA<AWorldPartitionMiniMap>())
		{
			return Cast<AWorldPartitionMiniMap>(Actor);
		}
	}

	if (bCreateNewMiniMap)
	{
		TSubclassOf<AWorldPartitionMiniMap> MiniMap(AWorldPartitionMiniMap::StaticClass());
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.OverrideLevel = PersistentLevel;
		return World->SpawnActor<AWorldPartitionMiniMap>(MiniMap, SpawnInfo);
	}
	
	return nullptr;
}

void FWorldPartitionMiniMapHelper::CaptureBoundsMiniMapToTexture(UWorld* InWorld, UObject* InOuterForTexture, uint32 InMiniMapSize, UTexture2D*& InOutMiniMapTexture, const FString& InTextureName, const FBox& InBounds)
{
	const uint32 DefaultNumWarmupFrames = 5;
	auto TickFunction = []() {};
	CaptureBoundsMiniMapToTexture(InWorld, InOuterForTexture, InMiniMapSize, InMiniMapSize, InOutMiniMapTexture, InTextureName, InBounds, SCS_BaseColor, DefaultNumWarmupFrames, TickFunction);
}

void FWorldPartitionMiniMapHelper::CaptureBoundsMiniMapToTexture(UWorld* InWorld, UObject* InOuterForTexture, uint32 InMiniMapSizeX, uint32 InMiniMapSizeY, UTexture2D*& InOutMiniMapTexture, const FString& InTextureName, const FBox& InBounds, ESceneCaptureSource InCaptureSource, uint32 InNumWarmupFrames,
	TFunctionRef<void()> WarmupTickCallback)
{
	// Before capturing the scene, make sure all assets are finished compiling
	FAssetCompilingManager::Get().FinishAllCompilation();

	//Calculate Projection matrix based on world bounds.
	FMatrix ProjectionMatrix;
	FWorldPartitionMiniMapHelper::CalTopViewOfWorld(ProjectionMatrix, InBounds, InMiniMapSizeX, InMiniMapSizeY);

	//Using SceneCapture Actor capture the scene to buffer
	UTextureRenderTarget2D* RenderTargetTexture = NewObject<UTextureRenderTarget2D>();
	RenderTargetTexture->ClearColor = FLinearColor::Transparent;
	RenderTargetTexture->TargetGamma = 2.2f;
	RenderTargetTexture->InitCustomFormat(InMiniMapSizeX, InMiniMapSizeY, PF_B8G8R8A8, false);
	RenderTargetTexture->UpdateResourceImmediate(true);

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.ObjectFlags |= RF_Transient;
	FVector CaptureActorLocation = FVector(InBounds.GetCenter().X, InBounds.GetCenter().Y, InBounds.GetCenter().Z + InBounds.GetExtent().Z);
	FRotator CaptureActorRotation = FRotator(-90.f, 0.f, -90.f);
	ASceneCapture2D* CaptureActor = InWorld->SpawnActor<ASceneCapture2D>(CaptureActorLocation, CaptureActorRotation, SpawnInfo);
	auto CaptureComponent = CaptureActor->GetCaptureComponent2D();
	CaptureComponent->TextureTarget = RenderTargetTexture;
	CaptureComponent->ProjectionType = ECameraProjectionMode::Orthographic;
	CaptureComponent->CaptureSource = InCaptureSource;
	CaptureComponent->OrthoWidth = InMiniMapSizeX;
	CaptureComponent->bUseCustomProjectionMatrix = true;
	CaptureComponent->CustomProjectionMatrix = ProjectionMatrix;
	CaptureComponent->bCaptureEveryFrame = false;
	CaptureComponent->bCaptureOnMovement = false;

	// We only want a basic color view of the world which isn't affected by any lighting or atmospheric effects which might obscure the minimap view:
	CaptureComponent->ShowFlags.Lighting = false;
	CaptureComponent->ShowFlags.Atmosphere = false;
	CaptureComponent->ShowFlags.PostProcessing = false;
	CaptureComponent->ShowFlags.AntiAliasing = false;
	CaptureComponent->ShowFlags.Fog = false;
	CaptureComponent->ShowFlags.VolumetricFog = false;
	CaptureComponent->ShowFlags.DynamicShadows = false;
	CaptureComponent->ShowFlags.SkyLighting = false;

	// Disable vignetting, otherwise we'll see it as a pattern between each captured tiles
	CaptureComponent->PostProcessSettings.bOverride_VignetteIntensity = true;
	CaptureComponent->PostProcessSettings.VignetteIntensity = 0.0f;

	for (uint32 i = 0; i < InNumWarmupFrames; i++)
	{
		WarmupTickCallback();
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
		CaptureComponent->CaptureScene();
	}

	CaptureComponent->CaptureScene();

	InWorld->DestroyActor(CaptureActor);
	CaptureActor = nullptr;	

	//Update the output texture
	if (!InOutMiniMapTexture)
	{
		InOutMiniMapTexture = RenderTargetTexture->ConstructTexture2D(InOuterForTexture, InTextureName, RF_NoFlags, CTF_Default, NULL);
	}
	else
	{
		FText ErrorMessage;
		if (RenderTargetTexture->UpdateTexture(InOutMiniMapTexture, CTF_Default, /*InAlphaOverride = */nullptr, /*InTextureChangingDelegate =*/ [](UTexture*) {}, &ErrorMessage))
		{
			check(InOutMiniMapTexture->Source.GetFormat() == TSF_BGRA8);
		}
		else
		{
			UE_LOGF(LogWorldPartitionMiniMapHelper, Error, "Couldn't copy render target to internal texture: %ls", *ErrorMessage.ToString());
		}
	}
}

void FWorldPartitionMiniMapHelper::CalTopViewOfWorld(FMatrix& OutProjectionMatrix, const FBox& WorldBox, uint32 ViewportWidth, uint32 ViewportHeight)
{
	const FVector Origin = WorldBox.GetCenter();

	FVector2D WorldSizeMin2D(WorldBox.Min.X, WorldBox.Min.Y);
	FVector2D WorldSizeMax2D(WorldBox.Max.X, WorldBox.Max.Y);

	FVector2D WorldSize2D = (WorldSizeMax2D - WorldSizeMin2D);
	WorldSize2D.X = FMath::Abs(WorldSize2D.X);
	WorldSize2D.Y = FMath::Abs(WorldSize2D.Y);
	const bool bUseXAxis = (WorldSize2D.X / WorldSize2D.Y) > 1.f;
	const float WorldAxisSize = bUseXAxis ? WorldSize2D.X : WorldSize2D.Y;
	const uint32 ViewportAxisSize = bUseXAxis ? ViewportWidth : ViewportHeight;
	const float OrthoZoom = WorldAxisSize / ViewportAxisSize / 2.f;
	const float OrthoWidth = FMath::Max(1.f, ViewportWidth * OrthoZoom);
	const float OrthoHeight = FMath::Max(1.f, ViewportHeight * OrthoZoom);

	const double ZOffset = UE_FLOAT_HUGE_DISTANCE / 2.0;
	OutProjectionMatrix = FReversedZOrthoMatrix(
		OrthoWidth,
		OrthoHeight,
		0.5f / ZOffset,
		ZOffset
	);

	ensureMsgf(!OutProjectionMatrix.ContainsNaN(), TEXT("Nans found on ProjectionMatrix"));
	if (OutProjectionMatrix.ContainsNaN())
	{
		OutProjectionMatrix.SetIdentity();
	}
}
#endif
