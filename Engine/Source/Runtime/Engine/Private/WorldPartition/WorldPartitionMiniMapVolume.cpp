// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionMiniMapVolume.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionMiniMapVolume)

AWorldPartitionMiniMapVolume::AWorldPartitionMiniMapVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
{
#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif

	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}
