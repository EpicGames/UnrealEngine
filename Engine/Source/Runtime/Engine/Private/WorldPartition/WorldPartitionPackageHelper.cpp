// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionPackageHelper.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Engine/Level.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "Engine/World.h"
#include "UObject/UObjectGlobalsInternal.h"

void FWorldPartitionPackageHelper::UnloadPackage(UPackage* InPackage)
{
	TrashObject(InPackage);

	// World specific
	if (UWorld* PackageWorld = UWorld::FindWorldInPackage(InPackage))
	{
		PackageWorld->ClearFlags(RF_Standalone);

		if (PackageWorld->PersistentLevel)
		{
			// Manual cleanup of level since world was not initialized
			PackageWorld->PersistentLevel->CleanupLevel(/*bCleanupResources*/ true, /*bUnloadFromEditor*/true);

			if (PackageWorld->PersistentLevel->IsUsingExternalObjects())
			{
				ForEachObjectWithOuter(PackageWorld->PersistentLevel, [](UObject* InObject)
				{
					if (UPackage* ExternalPackage = InObject->GetExternalPackage())
					{
						TrashObject(ExternalPackage);
					}
				}, EGetObjectsFlags::IncludeNestedObjects);
			}
		}
	}
}

#endif