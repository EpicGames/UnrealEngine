// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionPropertyOverride.h"
#include "Engine/Level.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionPropertyOverride)

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionPropertyOverrideSerialization.h"
#include "GameFramework/Actor.h"


bool UWorldPartitionPropertyOverride::ApplyPropertyOverrides(const FActorPropertyOverride* InPropertyOverride, AActor* InActor, bool bConstructionScriptProperties)
{
	bool bAppliedProperties = false;
	
	// Gather all Sub-Objects of the Actor that could have Property Overrides
	TArray<UObject*> Objects;
	TArray<UObject*> SubObjects;
	GetObjectsWithOuter(InActor, SubObjects, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
	Objects.Add(InActor);
	Objects.Append(MoveTemp(SubObjects));

	const FString ActorName = InActor->GetName();
	for (UObject* Object : Objects)
	{
		const FString ObjectSubPathString = FSoftObjectPath(Object).GetSubPathString();

		// Skip anything redundant string that wasn't saved as part of the SubObjectOverrides keys ex: PersistentLevel.ActorName
		const int32 IndexOfActorName = ObjectSubPathString.Find(ActorName);
		check(IndexOfActorName != INDEX_NONE);
		const FString SubObjectPathString = ObjectSubPathString.Mid(IndexOfActorName + ActorName.Len() + 1);
					
		if (const FSubObjectPropertyOverride* SubObjectOverride = InPropertyOverride->SubObjectOverrides.Find(SubObjectPathString))
		{
			// Check if we should apply or not 
			UActorComponent* ActorComponent = Cast<UActorComponent>(Object);
			if (!ActorComponent)
			{
				ActorComponent = Object->GetTypedOuter<UActorComponent>();
			}
			
			// If this object is a ActorComponent or outered to one then the properties need to be applied in the proper phase (depending on the type of component, construction script or not)
			if (ActorComponent && ActorComponent->IsCreatedByConstructionScript() != bConstructionScriptProperties)
			{
				continue;
			}

			// If this object is not a ActorComponent or outered to one then the properties are applied when bConstructionScriptProperties is false
			if (!ActorComponent && bConstructionScriptProperties)
			{
				continue;
			}

			Object->Modify(false);
			FWorldPartitionPropertyOverrideReader Reader(SubObjectOverride->SerializedTaggedProperties);
			FWorldPartitionPropertyOverrideArchive Archive(Reader, InPropertyOverride->ReferenceTable);

			// Remap archetype paths (e.g. /Game/MyLevel.MyLevel:...) to the
			// current streaming level's domain (e.g. /Temp/Game/MyLevel_PropertyOverride_123456789.MyLevel) 
			// so that object refs resolve in editor streaming. Not used for Game Worlds (PIE/standalone),
			// the WP level helper's FSoftObjectPathFixupArchive handles path remapping.
			if (!InActor->GetWorld()->IsGameWorld())
			{
				FString SourceWorldPath, RemappedWorldPath;
				UWorld* LevelOuterWorld = Cast<UWorld>(InActor->GetLevel()->GetOuter());
				const bool bRemapped = (LevelOuterWorld && LevelOuterWorld->GetSoftObjectPathMapping(SourceWorldPath, RemappedWorldPath)) || InActor->GetWorld()->GetSoftObjectPathMapping(SourceWorldPath, RemappedWorldPath);
				if (bRemapped)
				{
					Archive.SetInstancingContext(SourceWorldPath, RemappedWorldPath);
				}
			}

			UClass* ObjectClass = Object->GetClass();
			ObjectClass->SerializeTaggedProperties(Archive, (uint8*)Object, ObjectClass, nullptr);
			bAppliedProperties = true;

			if (ActorComponent && ActorComponent->IsRegistered())
			{
				// Notify object that it has changed if it is a ActorComponent or outered to one
				Object->PostEditChange();
			}
		}
	}

	return bAppliedProperties;
}

#endif
