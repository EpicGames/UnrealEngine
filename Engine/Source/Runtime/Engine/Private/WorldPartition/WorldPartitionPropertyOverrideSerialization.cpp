// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionPropertyOverrideSerialization.h"

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionSettings.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"

FWorldPartitionPropertyOverrideArchive::FWorldPartitionPropertyOverrideArchive(FArchive& InArchive, FPropertyOverrideReferenceTable& InReferenceTable, TSet<TObjectPtr<UObject>>* InOutObjectReferences)
	: FNameAsStringProxyArchive(InArchive)
	, ReferenceTable(InReferenceTable)
	, OutObjectReferences(InOutObjectReferences)
{
	check(InArchive.IsPersistent());
	check(!InArchive.IsFilterEditorOnly());
	check(InArchive.ShouldSkipBulkData());
	check(!InArchive.WantBinaryPropertySerialization());

	SetIsLoading(InArchive.IsLoading());
	SetIsSaving(InArchive.IsSaving());
	SetIsTextFormat(InArchive.IsTextFormat());
	SetWantBinaryPropertySerialization(InArchive.WantBinaryPropertySerialization());
	SetIsPersistent(true);
	FArchiveProxy::SetFilterEditorOnly(InArchive.IsFilterEditorOnly());
	ArShouldSkipBulkData = true;
	PropertyOverridePolicy = UWorldPartitionSettings::Get()->GetPropertyOverridePolicy();
}

bool FWorldPartitionPropertyOverrideArchive::ShouldSkipProperty(const FProperty* InProperty) const
{
	if (PropertyOverridePolicy)
	{
		return !PropertyOverridePolicy->CanOverrideProperty(InProperty);
	}

	return true;
}

static bool RemapInstancingPathInPlace(FSoftObjectPath& InOutPath, const FString& InSourcePrefix, const FString& InRemappedPrefix)
{
	const FString InPath = InOutPath.ToString();
	if (!InPath.StartsWith(InSourcePrefix))
	{
		return false;
	}
	// The match is only valid if InPath ends where InSourcePrefix ends, or if immediately followed by the sub-object delimiter (:)
	// i.e. don't match /Game/MyMap.MyMap with /Game/MyMap.MyMapAlt
	const int32 PrefixLen = InSourcePrefix.Len();
	if (InPath.Len() != PrefixLen && InPath[PrefixLen] != TEXT(':'))
	{
		return false;
	}
	InOutPath = InRemappedPrefix + InPath.RightChop(PrefixLen);
	return true;
}

FSoftObjectPath FWorldPartitionPropertyOverrideArchive::ReadSoftObjectPath()
{
	FSoftObjectPath Result;
	if (ReferenceTable.bIsValid)
	{
		int32 Index = INDEX_NONE;
		*this << Index;
		if (ReferenceTable.SoftObjectPathTable.IsValidIndex(Index))
		{
			Result = ReferenceTable.SoftObjectPathTable[Index];
		}
		else
		{
			// Only invalid Index we can expect is INDEX_NONE if the serialized SoftObjectPath was null
			ensureMsgf(Index == INDEX_NONE, TEXT("Invalid Index (%d) was read from the SoftObjectPathTable"), Index);
		}
	}
	else
	{
		// load the path name to the object
		FString LoadedString;
		InnerArchive << LoadedString;
		Result = FSoftObjectPath(LoadedString);
	}

	// Remap paths when a context is set.
	if (!InstancingOriginalWorldPath.IsEmpty() && !Result.IsNull())
	{
		RemapInstancingPathInPlace(Result, InstancingOriginalWorldPath, InstancingRemappedWorldPath);
	}

	return Result;
}

void FWorldPartitionPropertyOverrideArchive::WriteSoftObjectPath(FSoftObjectPath SoftObjectPath)
{
	ReferenceTable.bIsValid = true;
	int32 Index = INDEX_NONE;
	if (SoftObjectPath.IsValid())
	{
		// Remap paths when a context is set.
		if (!InstancingOriginalWorldPath.IsEmpty())
		{
			RemapInstancingPathInPlace(SoftObjectPath, InstancingOriginalWorldPath, InstancingRemappedWorldPath);
		}
		Index = ReferenceTable.SoftObjectPathTable.AddUnique(SoftObjectPath);
	}
	*this << Index;
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(FLazyObjectPtr& Value)
{ 
	return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); 
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(UObject*& Obj)
{
	if (IsLoading())
	{
		FSoftObjectPath Value = ReadSoftObjectPath();
		Obj = Value.ResolveObject();

		// Previous data didn't have hard references so make sure to load object if it isn't already
		if (!Obj && !ReferenceTable.bIsValid)
		{
			Obj = Value.TryLoad();
		}
		return *this;
	}
	else
	{
		if (OutObjectReferences)
		{
			OutObjectReferences->Add(Obj);
		}
		WriteSoftObjectPath(FSoftObjectPath(Obj));
	}

	return *this;
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(FWeakObjectPtr& Obj)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Obj);
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(FSoftObjectPtr& Value)
{
	if (IsLoading())
	{
		// Reset before serializing to clear the internal weak pointer. 
		Value.ResetWeakPtr();
		Value = ReadSoftObjectPath();
	}
	else
	{
		WriteSoftObjectPath(Value.GetUniqueID());
	}
	return *this;
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(FSoftObjectPath& Value)
{
	if (IsLoading())
	{
		Value = ReadSoftObjectPath();
	}
	else
	{
		WriteSoftObjectPath(Value);
	}
	return *this;
}

FArchive& FWorldPartitionPropertyOverrideArchive::operator<<(FObjectPtr& Obj)
{
	return FArchiveUObject::SerializeObjectPtr(*this, Obj);
}


FWorldPartitionPropertyOverrideWriter::FWorldPartitionPropertyOverrideWriter(TArray<uint8, TSizedDefaultAllocator<32>>& InBytes)
	: FMemoryWriter(InBytes, true)
{
	SetFilterEditorOnly(false);
	ArShouldSkipBulkData = true;
	SetIsTextFormat(false);
	SetWantBinaryPropertySerialization(false);
}

FWorldPartitionPropertyOverrideReader::FWorldPartitionPropertyOverrideReader(const TArray<uint8>& InBytes)
	: FMemoryReader(InBytes, true)
{
	SetFilterEditorOnly(false);
	ArShouldSkipBulkData = true;
	SetIsTextFormat(false);
	SetWantBinaryPropertySerialization(false);
}

#endif

