// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"

class UWorldPartitionPropertyOverridePolicy;

class FWorldPartitionPropertyOverrideArchive : public FNameAsStringProxyArchive
{
public:
	FWorldPartitionPropertyOverrideArchive(FArchive& InArchive, FPropertyOverrideReferenceTable& InReferenceTable, TSet<TObjectPtr<UObject>>* InOutObjectReferences = nullptr);

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override;
	virtual FArchive& operator<<(UObject*& Obj) override;
	virtual FArchive& operator<<(FWeakObjectPtr& Obj) override;
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(FObjectPtr& Obj) override;

	// Remapping applied during loading, to translate instanced world paths into streaming paths
	void SetInstancingContext(const FString& InOriginalWorldPath, const FString& InRemappedWorldPath)
	{
		InstancingOriginalWorldPath = InOriginalWorldPath;
		InstancingRemappedWorldPath = InRemappedWorldPath;
	}

private:
	UWorldPartitionPropertyOverridePolicy* PropertyOverridePolicy = nullptr;
	FPropertyOverrideReferenceTable& ReferenceTable;
	TSet<TObjectPtr<UObject>>* OutObjectReferences = nullptr;

	// Instancing context for source-domain → current-domain path substitution on load.
	FString InstancingOriginalWorldPath;
	FString InstancingRemappedWorldPath;

	FSoftObjectPath ReadSoftObjectPath();
	void WriteSoftObjectPath(FSoftObjectPath SoftObjectPath);
};

class FWorldPartitionPropertyOverrideWriter : public FMemoryWriter
{
public:
	FWorldPartitionPropertyOverrideWriter(TArray<uint8, TSizedDefaultAllocator<32>>& InBytes);
};

class FWorldPartitionPropertyOverrideReader : public FMemoryReader
{
public:
	FWorldPartitionPropertyOverrideReader(const TArray<uint8>& InBytes);
};

#endif