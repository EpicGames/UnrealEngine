// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionReplay.h"
#include "Engine/GameInstance.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartition.h"
#include "Engine/World.h"
#include "Net/UnrealNetwork.h"
#include "Engine/DemoNetDriver.h"
#include "ReplaySubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionReplay)

static int32 GUseReplayStreamingSources = 1;
static FAutoConsoleVariableRef CVarUseReplayStreamingSources(
	TEXT("wp.Runtime.UseReplayStreamingSources"),
	GUseReplayStreamingSources,
	TEXT("Set to 1 to use the recorded streaming sources when playing replay."));

static int32 GRecordReplayStreamingSources = 1;
static FAutoConsoleVariableRef CVarRecordReplayStreamingSources(
	TEXT("wp.Runtime.RecordReplayStreamingSources"),
	GRecordReplayStreamingSources,
	TEXT("Set to 1 to record streaming sources when recording replay."));

FArchive& operator<<(FArchive& Ar, FWorldPartitionReplayStreamingSource& StreamingSource)
{
	Ar << StreamingSource.Location;
	Ar << StreamingSource.Rotation;
	Ar << StreamingSource.TargetState;
	Ar << StreamingSource.bBlockOnSlowLoading;
	Ar << StreamingSource.Priority;
	Ar << StreamingSource.Velocity;
	return Ar;
}

FWorldPartitionReplaySample::FWorldPartitionReplaySample(AWorldPartitionReplay* InReplay)
	: Replay(InReplay)
{
	for (const FWorldPartitionStreamingSource& StreamingSource : Replay->GetRecordingStreamingSources())
	{
		StreamingSources.Add(FWorldPartitionReplayStreamingSource(StreamingSource));
		int32 Index = Replay->StreamingSourceNames.Find(StreamingSource.Name);
		if (Index != INDEX_NONE)
		{
			StreamingSourceNameIndices.Add(Index);
		}
		else
		{
			Replay->StreamingSourceNames.Add(StreamingSource.Name);
			StreamingSourceNameIndices.Add(Replay->StreamingSourceNames.Num() - 1);
		}
	}
}

FArchive& operator<<(FArchive& Ar, FWorldPartitionReplaySample& ReplaySample)
{
	Ar << ReplaySample.StreamingSourceNameIndices;
	Ar << ReplaySample.StreamingSources;

	if (Ar.IsLoading())
	{
		check(ReplaySample.StreamingSourceNameIndices.Num() == ReplaySample.StreamingSources.Num());
		for (int32 i = 0; i < ReplaySample.StreamingSources.Num(); ++i)
		{
			int32 NameIndex = ReplaySample.StreamingSourceNameIndices[i];
			// @todo_ow: It sometimes happens at start of replay that StreamingSourceNames haven't been replicated yet. need to investigate.
			if (ReplaySample.Replay->StreamingSourceNames.IsValidIndex(NameIndex))
			{
				ReplaySample.StreamingSources[i].Name = ReplaySample.Replay->StreamingSourceNames[NameIndex];
			}
			else
			{
				ReplaySample.StreamingSources[i].Name = FName(FString::Printf(TEXT("Source {%i}"), NameIndex));
			}
		}
	}
	return Ar;
}

AWorldPartitionReplay::AWorldPartitionReplay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAlwaysRelevant = true;
	bReplicates = true;
	NetDriverName = NAME_DemoNetDriver;
	bReplayRewindable = true;
#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
#endif
}

void AWorldPartitionReplay::RewindForReplay()
{
	Super::RewindForReplay();

	ReplaySamples.Empty();
}

void AWorldPartitionReplay::Initialize(UWorld* World)
{
	if (World->IsGameWorld())
	{
		UWorldPartition* WorldPartition = World->GetWorldPartition();
		check(WorldPartition);
		check(!WorldPartition->Replay);
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = AWorldPartitionReplay::StaticClass()->GetFName();
		WorldPartition->Replay = World->SpawnActor<AWorldPartitionReplay>(SpawnParams);
	}
}

void AWorldPartitionReplay::Uninitialize(UWorld* World)
{
	if (World->IsGameWorld())
	{
		UWorldPartition* WorldPartition = World->GetWorldPartition();
		if (ensure(WorldPartition && WorldPartition->Replay))
		{
			World->DestroyActor(WorldPartition->Replay);
			WorldPartition->Replay->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
			WorldPartition->Replay = nullptr;
		}
	}
}

bool AWorldPartitionReplay::IsPlaybackEnabled(const UWorld* World)
{
	if (GUseReplayStreamingSources && World->IsPlayingReplay())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			return !!WorldPartition->Replay;
		}
	}

	return false;
}

bool AWorldPartitionReplay::IsRecordingEnabled(const UWorld* World)
{
	if (GRecordReplayStreamingSources && World->IsRecordingReplay())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			return !!WorldPartition->Replay;
		}
	}

	return false;
}

void AWorldPartitionReplay::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME_CONDITION(AWorldPartitionReplay, StreamingSourceNames, COND_ReplayOnly);
}

void AWorldPartitionReplay::PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker)
{
	check(GetWorld()->IsRecordingReplay());
	Super::PreReplication(ChangedPropertyTracker);
	
	check(GetWorld()->IsPartitionedWorld());

	FWorldPartitionReplaySample Replay(this);
	if (Replay.StreamingSources.Num())
	{
		FBitWriter Writer(/*InMaxBits=*/1024, /*InAllowResize=*/true);
		Writer << Replay;
		if (Writer.GetNumBits())
		{
			if (UReplaySubsystem* ReplaySubsystem = UGameInstance::GetSubsystem<UReplaySubsystem>(GetGameInstance()))
			{
				ReplaySubsystem->SetExternalDataForObject(this, Writer.GetData(), Writer.GetNumBits());
			}			
		}
	}
}

bool AWorldPartitionReplay::GetReplayStreamingSources(TArray<FWorldPartitionStreamingSource>& OutStreamingSources)
{
	const UWorld* World = GetWorld();
	verify(IsPlaybackEnabled(World));

	UDemoNetDriver* DemoNetDriver = World->GetDemoNetDriver();
	if (DemoNetDriver && DemoNetDriver->IsPlaying())
	{
		const float CurrentTime = DemoNetDriver->GetDemoCurrentTime();
			
		// Erase samples older than 1 second
		while (ReplaySamples.Num() > 0)
		{
			if (ReplaySamples[0].TimeSeconds > CurrentTime - 1.0f)
			{
				break;
			}

			ReplaySamples.RemoveAt(0);
		}

		FReplayExternalDataArray* ExternalReplayData = DemoNetDriver->GetExternalDataArrayForObject(this);
		
		// Read new samples
		if (ExternalReplayData && ExternalReplayData->Num() > 0)
		{
			for (int i = 0; i < ExternalReplayData->Num(); i++)
			{
				FWorldPartitionReplaySample ReplaySample(this);
				(*ExternalReplayData)[i].Reader << ReplaySample;
				ReplaySample.TimeSeconds = (*ExternalReplayData)[i].TimeSeconds;
				ReplaySamples.Add(ReplaySample);
			}

			ExternalReplayData->Empty();
		}
				
		// Find current time samples to interpolate streaming sources
		for (int32 i = 0; i < ReplaySamples.Num() - 1; ++i)
		{
			if (CurrentTime >= ReplaySamples[i].TimeSeconds && CurrentTime <= ReplaySamples[i + 1].TimeSeconds)
			{
				const FWorldPartitionReplaySample& ReplaySample1 = ReplaySamples[i];
				const FWorldPartitionReplaySample& ReplaySample2 = ReplaySamples[i + 1];

				for (int32 SourceIndex = 0; SourceIndex < ReplaySample1.StreamingSources.Num(); ++SourceIndex)
				{
					int32 SourceIndex2 = ReplaySample2.StreamingSources.IndexOfByPredicate([&](const FWorldPartitionStreamingSource& StreamingSource) { return StreamingSource.Name == ReplaySample1.StreamingSources[SourceIndex].Name; });
					if (SourceIndex2 != INDEX_NONE)
					{
						const FWorldPartitionReplayStreamingSource& Source1 = ReplaySample1.StreamingSources[SourceIndex];
						const FWorldPartitionReplayStreamingSource& Source2 = ReplaySample2.StreamingSources[SourceIndex2];
						
						const float EPSILON = UE_SMALL_NUMBER;
						const float Delta = ReplaySample2.TimeSeconds - ReplaySample1.TimeSeconds;
						const float LerpPercent = Delta > EPSILON ? FMath::Clamp<float>((float)(CurrentTime - ReplaySample1.TimeSeconds) / Delta, 0.0f, 1.0f) : 1.0f;

						const FVector Location = FMath::Lerp(Source1.Location, Source2.Location, LerpPercent);
						const FQuat Rotation = FQuat::FastLerp(FQuat(Source1.Rotation), FQuat(Source2.Rotation), LerpPercent).GetNormalized();
						const FVector Velocity = FMath::Lerp(Source1.Velocity, Source2.Velocity, LerpPercent);
						OutStreamingSources.Add(FWorldPartitionReplayStreamingSource(Source1.Name, Location, Rotation.Rotator(), Source1.TargetState, Source1.bBlockOnSlowLoading, Source1.Priority, Velocity));
					}
					else
					{
						OutStreamingSources.Add(ReplaySample1.StreamingSources[SourceIndex]);
					}
				}
								
				break;
			}
		}

		return true;
	}

	return false;
}

TArray<FWorldPartitionStreamingSource> AWorldPartitionReplay::GetRecordingStreamingSources() const
{
	check(GetWorld());
	check(GetWorld()->GetWorldPartition());
	return GetWorld()->GetWorldPartition()->GetStreamingSources();
}

