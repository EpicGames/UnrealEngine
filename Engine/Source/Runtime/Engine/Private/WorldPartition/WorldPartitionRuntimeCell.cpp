// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "UObject/Package.h"
#include "Engine/Level.h"
#include "Algo/Transform.h"
#include "Algo/Count.h"
#include "Cooker/CookEvents.h"
#include "Misc/HierarchicalLogArchive.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayersID.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCell)

static_assert(std::is_same<std::underlying_type<EDataLayerRuntimeState>::type, uint8>::value);
static_assert(std::is_same<std::underlying_type<EWorldPartitionRuntimeCellState>::type, uint8>::value);

static_assert(static_cast<uint8>(EWorldPartitionRuntimeCellState::Unloaded) == static_cast<uint8>(EDataLayerRuntimeState::Unloaded));
static_assert(static_cast<uint8>(EWorldPartitionRuntimeCellState::Loaded) == static_cast<uint8>(EDataLayerRuntimeState::Loaded));
static_assert(static_cast<uint8>(EWorldPartitionRuntimeCellState::Activated) == static_cast<uint8>(EDataLayerRuntimeState::Activated));

UWorldPartitionRuntimeCell::UWorldPartitionRuntimeCell(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsAlwaysLoaded(false)
#if !UE_BUILD_SHIPPING
	, DebugStreamingPriority(-1.f)
#endif
	, RuntimeCellData(nullptr)
{
	EffectiveWantedState = EDataLayerRuntimeState::Unloaded;
	EffectiveWantedStateEpoch = MAX_int32;
}

UWorld* UWorldPartitionRuntimeCell::GetOwningWorld() const
{
	if (URuntimeHashExternalStreamingObjectBase* StreamingObjectOuter = GetTypedOuter<URuntimeHashExternalStreamingObjectBase>())
	{
		return StreamingObjectOuter->GetOwningWorld();
	}
	return UObject::GetTypedOuter<UWorldPartition>()->GetWorld();
}

UWorld* UWorldPartitionRuntimeCell::GetOuterWorld() const
{
	if (URuntimeHashExternalStreamingObjectBase* StreamingObjectOuter = GetTypedOuter<URuntimeHashExternalStreamingObjectBase>())
	{
		return StreamingObjectOuter->GetOuterWorld();
	}
	return UObject::GetTypedOuter<UWorld>();
}

#if WITH_EDITOR
void UWorldPartitionRuntimeCell::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (UnsavedActorsContainer)
	{
		// Make sure actor container isn't under PIE World so those template actors will never be considered part of the world.
		UnsavedActorsContainer->Rename(nullptr, GetPackage());

		for (auto& [ActorName, Actor] : UnsavedActorsContainer->Actors)
		{
			Actor->Rename(nullptr, UnsavedActorsContainer);
		}
	}
}

bool UWorldPartitionRuntimeCell::NeedsActorToCellRemapping() const
{
	// When cooking, always loaded cells content is moved to persistent level (see PopulateGeneratorPackageForCook)
	return !(IsAlwaysLoaded() && IsRunningCookCommandlet());
}

void UWorldPartitionRuntimeCell::SetDataLayers(const TArray<const UDataLayerInstance*>& InDataLayerInstances)
{
	// Validate that we have maximum 1 External Data Layer
	check(Algo::CountIf(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); }) <= 1);
	// Validate that all Data Layers are Runtime
	check(Algo::CountIf(InDataLayerInstances, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsRuntime(); }) == InDataLayerInstances.Num());

	// Sort Data Layers by FName except for External Data Layer that will always be the first in the list
	TArray<const UDataLayerInstance*> SortedDataLayerInstances(InDataLayerInstances);
	Algo::Sort(SortedDataLayerInstances, [](const UDataLayerInstance* A, const UDataLayerInstance* B)
	{
		if (A->IsA<UExternalDataLayerInstance>() && !B->IsA<UExternalDataLayerInstance>())
		{
			return true;
		}
		else if (!A->IsA<UExternalDataLayerInstance>() && B->IsA<UExternalDataLayerInstance>())
		{
			return false;
		}

		return A->GetFName().ToString() < B->GetFName().ToString();
	});

	TArray<FName> SortedDataLayerInstanceNames;
	bool bIsFirstDataLayerExternal = false;
	Algo::Transform(SortedDataLayerInstances, SortedDataLayerInstanceNames, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetFName(); });
	if (SortedDataLayerInstances.Num() > 0)
	{
		if (const UExternalDataLayerInstance* ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(SortedDataLayerInstances[0]))
		{
			bIsFirstDataLayerExternal = true;
			ExternalDataLayerAsset = ExternalDataLayerInstance->GetExternalDataLayerAsset();
		}
	}
	DataLayers = FDataLayerInstanceNames(SortedDataLayerInstanceNames, bIsFirstDataLayerExternal);
}

void UWorldPartitionRuntimeCell::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Ar.Printf(TEXT("Actor Count: %d"), GetActorCount());
	Ar.Printf(TEXT("Always Loaded: %s"), IsAlwaysLoaded() ? TEXT("True") : TEXT("False"));
	Ar.Printf(TEXT("Spatially Loaded: %s"), IsSpatiallyLoaded() ? TEXT("True") : TEXT("False"));

	if (TArray<const UDataLayerInstance*> DataLayerInstances = GetDataLayerInstances(); DataLayerInstances.Num() > 0)
	{
		UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Data Layers:")));
		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			Ar.Printf(TEXT("%s"), *DataLayerInstance->GetDataLayerFullName());
		}
	}

	RuntimeCellData->DumpStateLog(Ar);
}
#endif

int32 UWorldPartitionRuntimeCell::SortCompare(const UWorldPartitionRuntimeCell* Other) const
{
	return RuntimeCellData->SortCompare(Other->RuntimeCellData);
}

bool UWorldPartitionRuntimeCell::IsDebugShown() const
{
	return FWorldPartitionDebugHelper::IsDebugStreamingStatusShown(GetStreamingStatus()) &&
	       FWorldPartitionDebugHelper::AreDebugDataLayersShown(GetDataLayersInline()) &&
		   (FWorldPartitionDebugHelper::CanDrawContentBundles() || !ContentBundleID.IsValid()) &&
			RuntimeCellData->IsDebugShown();
}

UDataLayerManager* UWorldPartitionRuntimeCell::GetDataLayerManager() const
{
	return GetOuterWorld()->GetWorldPartition()->GetDataLayerManager();
}

EDataLayerRuntimeState UWorldPartitionRuntimeCell::GetCellEffectiveWantedState(const FWorldPartitionStreamingContext& Context) const
{
	if (!HasDataLayers())
	{
		UE::TWriteScopeLock WriteLock(EffectiveWantedStateLock);
		return EffectiveWantedState = EDataLayerRuntimeState::Activated;
	}

	const int32 ResolvingDataLayersRuntimeStateEpoch = Context.GetResolvingDataLayersRuntimeStateEpoch();
	{
		UE::TReadScopeLock ReadLock(EffectiveWantedStateLock);
		if (EffectiveWantedStateEpoch == ResolvingDataLayersRuntimeStateEpoch)
		{
			return EffectiveWantedState;
		}
	}

	// Epoch mismatch - resolve outside lock, then cache under write lock
	const EDataLayerRuntimeState ResolvedState = Context.ResolveDataLayerRuntimeState(DataLayers);
	UE::TWriteScopeLock WriteLock(EffectiveWantedStateLock);
	EffectiveWantedState = ResolvedState;
	EffectiveWantedStateEpoch = ResolvingDataLayersRuntimeStateEpoch;
	return EffectiveWantedState;
}

TArray<const UDataLayerInstance*> UWorldPartitionRuntimeCell::GetDataLayerInstances() const
{
	const UDataLayerManager* DataLayerManager = HasDataLayers() ? GetDataLayerManager() : nullptr;
	return DataLayerManager ? DataLayerManager->GetDataLayerInstances(GetDataLayersInline()) : TArray<const UDataLayerInstance*>();
}

const UExternalDataLayerInstance* UWorldPartitionRuntimeCell::GetExternalDataLayerInstance() const
{
	const UDataLayerManager* DataLayerManager = !GetExternalDataLayer().IsNone() ? GetDataLayerManager() : nullptr;
	return DataLayerManager ? Cast<UExternalDataLayerInstance>(DataLayerManager->GetDataLayerInstance(GetExternalDataLayer())) : nullptr;
}

bool UWorldPartitionRuntimeCell::ContainsDataLayer(const UDataLayerAsset* DataLayerAsset) const
{
	const UDataLayerManager* DataLayerManager = HasDataLayers() ? GetDataLayerManager() : nullptr;
	const UDataLayerInstance* DataLayerInstance = DataLayerManager ? DataLayerManager->GetDataLayerInstance(DataLayerAsset) : nullptr;
	return DataLayerInstance ? ContainsDataLayer(DataLayerInstance) : false;
}

bool UWorldPartitionRuntimeCell::HasContentBundle() const
{
	return GetContentBundleID().IsValid();
}

bool UWorldPartitionRuntimeCell::ContainsDataLayer(const UDataLayerInstance* DataLayerInstance) const
{
	return GetDataLayersInline().Contains(DataLayerInstance->GetFName());
}

FName UWorldPartitionRuntimeCell::GetLevelPackageName() const
{ 
#if WITH_EDITOR
	return LevelPackageName;
#else
	return NAME_None;
#endif
}

FString UWorldPartitionRuntimeCell::GetDebugName() const
{
	return RuntimeCellData->GetDebugName();
}

#if WITH_EDITOR

void FWorldPartitionRuntimeCellObjectMapping::UpdateHash(FWorldPartitionPackageHashBuilder& Builder) const
{	
	UE::Private::WorldPartition::UpdateHash(Builder, Package);
	UE::Private::WorldPartition::UpdateHash(Builder, Path);
	UE::Private::WorldPartition::UpdateHash(Builder, BaseClass);
	UE::Private::WorldPartition::UpdateHash(Builder, NativeClass);
	UE::Private::WorldPartition::UpdateHash(Builder, ContainerID);
	UE::Private::WorldPartition::UpdateHash(Builder, ContainerTransform);
	UE::Private::WorldPartition::UpdateHash(Builder, EditorOnlyParentTransform);
	UE::Private::WorldPartition::UpdateHash(Builder, ContainerPackage);
	UE::Private::WorldPartition::UpdateHash(Builder, WorldPackage);
	UE::Private::WorldPartition::UpdateHash(Builder, ActorInstanceGuid);
	UE::Private::WorldPartition::UpdateHash(Builder, LoadedPath);
	UE::Private::WorldPartition::UpdateHash(Builder, bIsEditorOnly);

	for (const FWorldPartitionRuntimeCellPropertyOverride& Override : PropertyOverrides)
	{
		Override.UpdateHash(Builder);
	}	
}

void FWorldPartitionRuntimeCellPropertyOverride::UpdateHash(FWorldPartitionPackageHashBuilder& Builder) const
{
	UE::Private::WorldPartition::UpdateHash(Builder, OwnerContainerID);
	UE::Private::WorldPartition::UpdateHash(Builder, AssetPath);
	UE::Private::WorldPartition::UpdateHash(Builder, PackageName);
	UE::Private::WorldPartition::UpdateHash(Builder, ContainerPath.ContainerGuids);

}
#endif

