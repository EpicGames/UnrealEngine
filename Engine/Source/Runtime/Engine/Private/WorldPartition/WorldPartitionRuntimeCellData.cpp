// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCellData.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "Misc/HierarchicalLogArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellData)

UWorldPartitionRuntimeCellData::UWorldPartitionRuntimeCellData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CachedMinSourcePriority(MAX_uint8)
	, bCachedWasRequestedByBlockingSource(false)
	, CachedMinSquareDistanceToBlockingSource(MAX_dbl)
	, CachedMinBlockOnSlowStreamingRatio(MAX_flt)
	, CachedMinSquareDistanceToSource(MAX_dbl)
	, CachedMinSlowStreamingRatio(MAX_flt)
	, CachedMinSpatialSortingPriority(MAX_dbl)
	, CachedSourceInfoEpoch(MIN_int32)
	, ContentBounds(ForceInit)
	, Priority(0)
	, HierarchicalLevel(0)
{}

void UWorldPartitionRuntimeCellData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << DebugName;
}

#if WITH_EDITOR
void UWorldPartitionRuntimeCellData::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Ar.Printf(TEXT("Content Bounds: %s"), *ContentBounds.ToString());

	if (CellBounds.IsSet())
	{
		Ar.Printf(TEXT("Cell Bounds: %s"), *CellBounds.GetValue().ToString());
	}
}
#endif

void UWorldPartitionRuntimeCellData::ResetStreamingSourceInfo(const FWorldPartitionStreamingContext& Context) const
{
	CachedMinSourcePriority = MAX_uint8;
	bCachedWasRequestedByBlockingSource = false;
	CachedMinSquareDistanceToBlockingSource = MAX_dbl;
	CachedMinBlockOnSlowStreamingRatio = MAX_flt;
	CachedMinSquareDistanceToSource = MAX_dbl;
	CachedMinSlowStreamingRatio = MAX_flt;
	CachedMinSpatialSortingPriority = MAX_dbl;
	CachedSourceInfoEpoch = Context.GetUpdateStreamingStateEpoch();
}

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("Append Streaming Source Info"), STAT_WorldPartitionAppendStreamingSourceInfo, STATGROUP_WorldPartition, EStatFlags::Verbose);

void UWorldPartitionRuntimeCellData::AppendStreamingSourceInfo(const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context, bool bBlockOnSlowLoading) const
{
	SCOPE_CYCLE_COUNTER(STAT_WorldPartitionAppendStreamingSourceInfo);

	if (CachedSourceInfoEpoch != Context.GetUpdateStreamingStateEpoch())
	{
		ResetStreamingSourceInfo(Context);
		check(CachedSourceInfoEpoch == Context.GetUpdateStreamingStateEpoch());
	}

	// Compute cosine angle from cell to source direction ratio
	const FVector CellToSource = SourceShape.GetCenter() - ContentBounds.GetClosestPointTo(SourceShape.GetCenter());
	const double CellToSourceSquareDistance = CellToSource.SizeSquared();
	const FVector CellToSourceNormal = FMath::IsNearlyZero(CellToSourceSquareDistance) ? FVector::ZeroVector : (CellToSource * FMath::InvSqrt(CellToSourceSquareDistance));
	const FVector SourceAxis = Source.bUseVelocityContributionToCellsSorting ? FVector(SourceShape.GetAxis() + Source.Velocity).GetSafeNormal() : SourceShape.GetAxis();
	const float SourceCosAngle = ContentBounds.IsInsideOrOn(SourceShape.GetCenter()) ? -1.0f : (SourceAxis | CellToSourceNormal);
	const float SourceCosAngleRatio = SourceCosAngle * 0.5f + 0.5f;

	CachedMinSourcePriority = FMath::Min((uint8)Source.Priority, CachedMinSourcePriority);

	if (bBlockOnSlowLoading && Source.bBlockOnSlowLoading)
	{
		bCachedWasRequestedByBlockingSource = true;

		CachedMinSquareDistanceToBlockingSource = FMath::Min(CellToSourceSquareDistance, CachedMinSquareDistanceToBlockingSource);

		const double BlockOnSlowStreamingRatio = FMath::SafeDivide(FMath::Sqrt(CachedMinSquareDistanceToBlockingSource), SourceShape.GetRadius());
		CachedMinBlockOnSlowStreamingRatio = FMath::Min(CachedMinBlockOnSlowStreamingRatio, BlockOnSlowStreamingRatio);
	}

	CachedMinSquareDistanceToSource = FMath::Min(CellToSourceSquareDistance, CachedMinSquareDistanceToSource);

	const double SlowStreamingRatio = FMath::SafeDivide(FMath::Sqrt(CachedMinSquareDistanceToSource), SourceShape.GetRadius());
	CachedMinSlowStreamingRatio = FMath::Min(CachedMinSlowStreamingRatio, SlowStreamingRatio);

	// Compute square distance from cell to source ratio
	const double SourceDistanceRatio = FMath::Clamp(FMath::SafeDivide(ContentBounds.ComputeSquaredDistanceToPoint(SourceShape.GetCenter()), FMath::Square(SourceShape.GetRadius())), 0.0f, 1.0f);

	// Compute final cell priority for this source
	const double SortingPriority = SourceDistanceRatio * SourceCosAngleRatio;

	// Update if lower
	CachedMinSpatialSortingPriority = FMath::Min(CachedMinSpatialSortingPriority, SortingPriority);
}

/**
 * Sorting criterias:
 *	- Highest priority affecting source (lowest to highest)
 *	- Cell hierarchical level (highest to lowest)
 -	- Cell custom priority (lowest to highest)
 *	- Cell distance and angle from source (lowest to highest)
 */
int32 UWorldPartitionRuntimeCellData::SortCompare(const UWorldPartitionRuntimeCellData* InOther) const
{
	int64 Result = (int32)CachedMinSourcePriority - (int32)InOther->CachedMinSourcePriority;

	if (!Result)
	{
		// Cell hierarchical level (highest to lowest)
		Result = InOther->HierarchicalLevel - HierarchicalLevel;

		if (!Result)
		{
			// Cell priority (lower value is higher prio)
			Result = Priority - InOther->Priority;		
			if (!Result)
			{
				double Diff = CachedMinSpatialSortingPriority - InOther->CachedMinSpatialSortingPriority;
				if (!FMath::IsNearlyZero(Diff))
				{
					Result = (CachedMinSpatialSortingPriority < InOther->CachedMinSpatialSortingPriority) ? -1 : 1;
				}
			}
		}
	}

	return (int32)FMath::Clamp(Result, -1, 1);
}

const FBox& UWorldPartitionRuntimeCellData::GetContentBounds() const
{
	return ContentBounds;
}

FBox UWorldPartitionRuntimeCellData::GetCellBounds() const
{
	return CellBounds.IsSet() ? *CellBounds : ContentBounds;
}

FBox UWorldPartitionRuntimeCellData::GetStreamingBounds() const
{
	return GetContentBounds();
}

FString UWorldPartitionRuntimeCellData::GetDebugName() const
{
	return DebugName.GetString();
}
