// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCellDataSpatialHash.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "Engine/Level.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellDataSpatialHash)

static float GRuntimeSpatialHashCellToSourceAngleContributionToCellImportance = 0.4f; // Value between [0, 1]
static FAutoConsoleVariableRef CVarRuntimeSpatialHashCellToSourceAngleContributionToCellImportance(
	TEXT("wp.Runtime.RuntimeSpatialHashCellToSourceAngleContributionToCellImportance"),
	GRuntimeSpatialHashCellToSourceAngleContributionToCellImportance,
	TEXT("Value between 0 and 1 that modulates the contribution of the angle between streaming source-to-cell vector and source-forward vector to the cell importance. The closest to 0, the less the angle will contribute to the cell importance."));

static bool GRuntimeSpatialHashSortUsingCellExtent = true;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashSortUsingCellExtent(
	TEXT("wp.Runtime.RuntimeSpatialHashSortUsingCellExtent"),
	GRuntimeSpatialHashSortUsingCellExtent,
	TEXT("Set to 1 to use cell extent instead of cell grid level when sorting cells by importance."));

static bool GRuntimeSpatialHashSortUsingCellPriority = true;
static FAutoConsoleVariableRef CVarRuntimeSpatialHashSortUsingCellPriority(
	TEXT("wp.Runtime.RuntimeSpatialHashSortUsingCellPriority"),
	GRuntimeSpatialHashSortUsingCellPriority,
	TEXT("Set to 1 to use cell priority before distance to/angle from source as part of the sorting criterias when sorting cells by importance."));

UWorldPartitionRuntimeCellDataSpatialHash::UWorldPartitionRuntimeCellDataSpatialHash(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Extent(0)
	, CachedMinBlockOnSlowStreamingRatio2D(MAX_dbl)
	, CachedMinSquareDistanceToBlockingSource2D(MAX_dbl)
	, CachedMinSlowStreamingRatio2D(MAX_dbl)
	, CachedMinSquareDistanceToSource2D(MAX_dbl)
{}

void UWorldPartitionRuntimeCellDataSpatialHash::ResetStreamingSourceInfo(const FWorldPartitionStreamingContext& Context) const
{
	Super::ResetStreamingSourceInfo(Context);

	CachedSourcePriorityWeights.Reset();
	CachedSourceSquaredDistances.Reset();
	CachedInstersectingShapes.Reset();
	CachedMinBlockOnSlowStreamingRatio2D = MAX_dbl;
	CachedMinSquareDistanceToBlockingSource2D = MAX_dbl;
	CachedMinSlowStreamingRatio2D = MAX_dbl;
	CachedMinSquareDistanceToSource2D = MAX_dbl;
}

float UWorldPartitionRuntimeCellDataSpatialHash::ComputeSourceToCellAngleFactor(const FSphericalSector& SourceShape) const
{
	float AngleContribution = FMath::Clamp(GRuntimeSpatialHashCellToSourceAngleContributionToCellImportance, 0.f, 1.f);
	float AngleFactor = 1.f;
	if (!FMath::IsNearlyZero(AngleContribution))
	{
		const FBox Box(FVector(Position.X - Extent, Position.Y - Extent, 0.f), FVector(Position.X + Extent, Position.Y + Extent, 0.f));
		const FVector2D SourcePos(SourceShape.GetCenter());
		const FVector StartVert(SourcePos, 0.f);
		const FVector EndVert(SourcePos + FVector2D(SourceShape.GetScaledAxis()), 0.f);

		float Angle = 0.f;
		if (!FMath::LineBoxIntersection(Box, StartVert, EndVert, EndVert - StartVert))
		{
			// Find smallest angle using 4 corners and center of cell bounds
			const FVector2D Position2D(Position);
			float MaxDot = 0.f;
			FVector2D SourceForward(SourceShape.GetAxis());
			SourceForward.Normalize();
			FVector2D CellPoints[5];
			CellPoints[0] = Position2D + FVector2D(-Extent, -Extent);
			CellPoints[1] = Position2D + FVector2D(-Extent, Extent);
			CellPoints[2] = Position2D + FVector2D(Extent, -Extent);
			CellPoints[3] = Position2D + FVector2D(Extent, Extent);
			CellPoints[4] = Position2D;
			for (const FVector2D& CellPoint : CellPoints)
			{
				const FVector2D SourceToCell(CellPoint - SourcePos);
				const float Dot = FVector2D::DotProduct(SourceForward, SourceToCell.GetSafeNormal());
				MaxDot = FMath::Max(MaxDot, Dot);
			}
			Angle = FMath::Abs(FMath::Acos(MaxDot) / UE_PI);
		}
		const float NormalizedAngle = FMath::Clamp(Angle, UE_PI / 180.f, 1.f);
		AngleFactor = FMath::Pow(NormalizedAngle, AngleContribution);
	}
	return AngleFactor;
}

void UWorldPartitionRuntimeCellDataSpatialHash::AppendStreamingSourceInfo(const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context, bool bBlockOnSlowLoading) const
{
	Super::AppendStreamingSourceInfo(Source, SourceShape, Context, bBlockOnSlowLoading);

	const double SquareDistance2D = FVector::DistSquared2D(SourceShape.GetCenter(), Position);

	// Update cached values based on the 2D distance
	if (bBlockOnSlowLoading && Source.bBlockOnSlowLoading)
	{
		CachedMinSquareDistanceToBlockingSource2D = FMath::Min(SquareDistance2D, CachedMinSquareDistanceToBlockingSource2D);
		CachedMinBlockOnSlowStreamingRatio2D = FMath::Min(CachedMinBlockOnSlowStreamingRatio2D, FMath::SafeDivide(FMath::Sqrt(CachedMinSquareDistanceToBlockingSource2D), SourceShape.GetRadius()));
	}
	CachedMinSquareDistanceToSource2D = FMath::Min(SquareDistance2D, CachedMinSquareDistanceToSource2D);
	CachedMinSlowStreamingRatio2D = FMath::Min(CachedMinSlowStreamingRatio2D, FMath::SafeDivide(FMath::Sqrt(CachedMinSquareDistanceToSource2D), SourceShape.GetRadius()));

	CachedSourceSquaredDistances.Add(SquareDistance2D);
	CachedSourcePriorityWeights.Add(1.0f - ((float)Source.Priority / (float)EStreamingSourcePriority::Lowest));
	CachedInstersectingShapes.Add(SourceShape);
}

void UWorldPartitionRuntimeCellDataSpatialHash::MergeStreamingSourceInfo() const
{
	Super::MergeStreamingSourceInfo();

	int32 Count = CachedSourceSquaredDistances.Num();
	check(Count == CachedSourcePriorityWeights.Num());
	check(Count == CachedInstersectingShapes.Num());

	CachedMinBlockOnSlowStreamingRatio = CachedMinBlockOnSlowStreamingRatio2D;
	CachedMinSquareDistanceToBlockingSource = CachedMinSquareDistanceToBlockingSource2D;
	CachedMinSquareDistanceToSource = CachedMinSquareDistanceToSource2D;
	CachedMinSlowStreamingRatio = CachedMinSlowStreamingRatio2D;

	if (Count)
	{
		double TotalSourcePriorityWeight = 0.f;
		for (int32 i = 0; i < Count; ++i)
		{
			TotalSourcePriorityWeight += CachedSourcePriorityWeights[i];
		}

		double HighestPrioWeight = 0;
		double HighestPrioMinModulatedDistance = 0;
		double WeightedModulatedDistance = 0;
		for (int32 i = 0; i < Count; ++i)
		{
			const float AngleFactor = ComputeSourceToCellAngleFactor(CachedInstersectingShapes[i]);
			const double CurrentModulatedDistance = CachedSourceSquaredDistances[i] * AngleFactor * AngleFactor;
			const double CurrentWeight = CachedSourcePriorityWeights[i];
			WeightedModulatedDistance += CurrentModulatedDistance * CurrentWeight / TotalSourcePriorityWeight;
			// Find highest priority source with the minimum modulated distance
			if ((i == 0) || ((CurrentModulatedDistance < HighestPrioMinModulatedDistance) && (CurrentWeight >= HighestPrioWeight)))
			{
				HighestPrioMinModulatedDistance = CurrentModulatedDistance;
				HighestPrioWeight = CurrentWeight;
			}
		}

		// Sorting distance is the minimum between these:
		// - the highest priority source with the minimum modulated distance 
		// - the weighted modulated distance
		CachedSourceSortingDistance = FMath::Min(HighestPrioMinModulatedDistance, WeightedModulatedDistance);
	}
}

int32 UWorldPartitionRuntimeCellDataSpatialHash::SortCompare(const UWorldPartitionRuntimeCellData* InOther) const
{
	if (const UWorldPartitionRuntimeCellDataSpatialHash* Other = Cast<UWorldPartitionRuntimeCellDataSpatialHash>(InOther))
	{
		int32 Result = (int32)CachedMinSourcePriority - (int32)Other->CachedMinSourcePriority;

		if (Result == 0)
		{
			// By default, now compare cell's extent instead of its grid level since we compare cells across multiple WPs/grids (higher value is higher prio)
			Result = GRuntimeSpatialHashSortUsingCellExtent ? int32(Other->Extent - Extent) : (Other->HierarchicalLevel - HierarchicalLevel);

			if (Result == 0)
			{
				if (GRuntimeSpatialHashSortUsingCellPriority)
				{
					// Cell priority (lower value is higher prio)
					Result = Priority - Other->Priority;
				}

				if (Result == 0)
				{
					// Closest distance (lower value is higher prio)
					const double Diff = CachedSourceSortingDistance - Other->CachedSourceSortingDistance;
					if (FMath::IsNearlyZero(Diff))
					{
						const double RawDistanceDiff = CachedMinSquareDistanceToSource2D - Other->CachedMinSquareDistanceToSource2D;
						Result = RawDistanceDiff < 0 ? -1 : (RawDistanceDiff > 0.f ? 1 : 0);
					}
					else
					{
						Result = Diff < 0.f ? -1 : (Diff > 0.f ? 1 : 0);
					}
				}
			}
		}

		if (!GRuntimeSpatialHashSortUsingCellPriority && Result == 0)
		{
			// Cell priority (lower value is higher prio)
			Result = Priority - Other->Priority;
		}

		return Result;
	}

	return Super::SortCompare(InOther);
}

FBox UWorldPartitionRuntimeCellDataSpatialHash::GetCellBounds() const
{
	FBox Box = FBox::BuildAABB(Position, FVector(Extent));
	// Use content bounds for the Z extent
	Box.Min.Z = GetContentBounds().Min.Z;
	Box.Max.Z = GetContentBounds().Max.Z;
	return Box;
}

FBox UWorldPartitionRuntimeCellDataSpatialHash::GetStreamingBounds() const
{
	return GetCellBounds();
}

bool UWorldPartitionRuntimeCellDataSpatialHash::IsDebugShown() const
{
	return Super::IsDebugShown() && 
		   FWorldPartitionDebugHelper::IsDebugRuntimeHashGridShown(GridName) &&
		   FWorldPartitionDebugHelper::IsDebugCellNameShown(GetName());
}