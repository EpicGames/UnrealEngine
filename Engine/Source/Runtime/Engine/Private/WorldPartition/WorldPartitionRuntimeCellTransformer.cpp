// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/Actor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellTransformer)

UWorldPartitionRuntimeCellTransformer::UWorldPartitionRuntimeCellTransformer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UWorldPartitionRuntimeCellTransformer::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Fix old objects that were created without RF_Transactional
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_Transactional))
	{
		SetFlags(RF_Transactional);
	}
#endif
}

#if WITH_EDITOR
bool UWorldPartitionRuntimeCellTransformer::IsLevelTransformable(ULevel* InLevel) const
{
	return true;
}

void UWorldPartitionRuntimeCellTransformer::PreTransform(ULevel* InLevel)
{
}

void UWorldPartitionRuntimeCellTransformer::Transform(ULevel* InLevel)
{
}

void UWorldPartitionRuntimeCellTransformer::PostTransform(ULevel* InLevel)
{
}

bool UWorldPartitionRuntimeCellTransformer::IsCellTransformable(const UWorldPartitionRuntimeCell* InCell) const
{
	return true;
}

bool UWorldPartitionRuntimeCellTransformer::ShouldStripRuntimeCell(const UWorldPartitionRuntimeCell* InCell) const
{
	return false;
}

void UWorldPartitionRuntimeCellTransformer::TransformRuntimeCell(UWorldPartitionRuntimeCell* InCell)
{
}

void UWorldPartitionRuntimeCellTransformer::ForEachIgnoredComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const
{
	for (const TSubclassOf<UActorComponent>& IgnoredComponentClass : GetDefault<UWorldPartitionRuntimeCellTransformerSettings>()->IgnoredComponentClasses)
	{
		if (!Func(IgnoredComponentClass))
		{
			return;
		}
	}
}

void UWorldPartitionRuntimeCellTransformer::ForEachIgnoredExactComponentClass(TFunctionRef<bool(const TSubclassOf<UActorComponent>&)> Func) const
{
	for (const TSubclassOf<UActorComponent>& IgnoredExactComponentClass : GetDefault<UWorldPartitionRuntimeCellTransformerSettings>()->IgnoredExactComponentClasses)
	{
		if (!Func(IgnoredExactComponentClass))
		{
			return;
		}
	}
}

bool UWorldPartitionRuntimeCellTransformer::CanIgnoreComponent(const UActorComponent* InComponent) const
{
	bool bCanIgnore = false;
	const UClass* ComponentClass = InComponent->GetClass();

	ForEachIgnoredComponentClass([&bCanIgnore, &ComponentClass](const TSubclassOf<UActorComponent>& IgnoredComponentClass)
	{
		if (ComponentClass->IsChildOf(IgnoredComponentClass))
		{
			bCanIgnore = true;
			return false;
		}
		return true;
	});

	if (bCanIgnore)
	{
		return true;
	}

	ForEachIgnoredExactComponentClass([&bCanIgnore, &ComponentClass](const TSubclassOf<UActorComponent>& IgnoredExactComponentClass)
	{
		if (ComponentClass == IgnoredExactComponentClass)
		{
			bCanIgnore = true;
			return false;
		}
		return true;
	});

	return bCanIgnore;
}

bool UWorldPartitionRuntimeCellTransformer::IsBlueprintActorWithLogic(AActor* InActor) const
{
	const FName FN_UserConstructionScript(TEXT("UserConstructionScript"));

	UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(InActor->GetClass());
	if (!Blueprint || !Blueprint->GeneratedClass)
	{
		return false;
	}

	check(Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(AActor::StaticClass()));

	UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass);
	if (!BPClass)
	{
		return false;
	}

	if (Blueprint->DelegateSignatureGraphs.Num() > 0)
	{
		return true;
	}

	if (Blueprint->ImplementedInterfaces.Num() > 0)
	{
		return true;
	}

	// Check if no extra functions, other than the user construction script (only AActor and subclasses of AActor have)
	if (Blueprint->FunctionGraphs.Num() > 1)
	{
		return true;
	}

	check(Blueprint->FunctionGraphs.Num() == 0 || Blueprint->FunctionGraphs[0]->GetFName() == FN_UserConstructionScript);

	// Check if the generated class has overridden any functions dynamically
	for (TFieldIterator<UFunction> It(BPClass, EFieldIteratorFlags::IncludeSuper); It; ++It)
	{
		UFunction* Function = *It;

		// Ignore functions from native C++ classes (i.e., inherited but not overridden in BP)
		if (Function->GetOwnerClass() == BPClass && Function->GetFName() != FN_UserConstructionScript)
		{
			return true; // Found an overridden function
		}
	}

	// If there is an enabled node in the event graph, the Blueprint is not data only
	for (UEdGraph* EventGraph : Blueprint->UbergraphPages)
	{
		for (UEdGraphNode* GraphNode : EventGraph->Nodes)
		{
			if (GraphNode && (GraphNode->GetDesiredEnabledState() != ENodeEnabledState::Disabled))
			{
				return true;
			}
		}
	}

	return false;
}
#endif

const FName UWorldPartitionRuntimeCellTransformer::NAME_CellTransformerIgnoreActor(TEXT("CellTransformer_IgnoreActor"));