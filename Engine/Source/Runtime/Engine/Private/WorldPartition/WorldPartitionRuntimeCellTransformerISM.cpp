// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCellTransformerISM.h"

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "ISMPartition/ISMComponentBatcher.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartitionCellTransformerISMComponentDescriptor.h"
#endif

#if WITH_EDITORONLY_DATA
#include "ActorPartition/PartitionActor.h"
#include "Engine/StaticMeshActor.h"
#endif

#include "GameFramework/ActorPrimitiveColorHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellTransformerISM)

#define LOCTEXT_NAMESPACE "WorldPartition"

UWorldPartitionRuntimeCellTransformerISM::UWorldPartitionRuntimeCellTransformerISM(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	AllowedClasses.Add(APartitionActor::StaticClass());
	AllowedClasses.Add(AStaticMeshActor::StaticClass());
	MinNumInstances = 2;
#endif

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_ImmutableDefaultObject) && ExactCast<UWorldPartitionRuntimeCellTransformerISM>(this))
	{
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(TEXT("CellTransformerISM"), LOCTEXT("CellTransformerISM", "Cell Transformer ISM"), false, [](const UPrimitiveComponent* InPrimitiveComponent)
		{
			if (AActor* Actor = InPrimitiveComponent ? InPrimitiveComponent->GetOwner() : nullptr)
			{
				if (Actor->IsA<AWorldPartitionAutoInstancedActor>())
				{
					return FLinearColor::MakeRandomSeededColor(GetTypeHash(InPrimitiveComponent->GetFName()));
				}
			}
			return FLinearColor::White;
		});
	}
#endif
}

#if WITH_EDITOR
void UWorldPartitionRuntimeCellTransformerISM::Transform(ULevel* InLevel)
{
	check(InLevel);

	struct FActorComponentBatcherDescriptor
	{
		TMap<TObjectPtr<AActor>*, TArray<UStaticMeshComponent*>> ActorComponents;
		FISMComponentBatcher ISMComponentBatcher;
	};

	TMap<FWorldPartitionCellTransformerISMComponentDescriptor, FActorComponentBatcherDescriptor> ISMComponentBatchers;

	for (TObjectPtr<AActor>& Actor : InLevel->Actors)
	{
		if (IsValid(Actor) && CanAutoInstanceActor(Actor))
		{
			bool bContainsNonTransformable = false;
			TArray<UStaticMeshComponent*> ActorTransformableComponents;
			// Gather potential components that can be merged
			Actor->ForEachComponent<UActorComponent>(true, [this, &ISMComponentBatchers, &ActorTransformableComponents, &bContainsNonTransformable, &Actor](UActorComponent* Component)
			{
				UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
				if (StaticMeshComponent && !StaticMeshComponent->IsEditorOnly() && StaticMeshComponent->IsVisible() && (StaticMeshComponent->Mobility == EComponentMobility::Static) && IsComponentTransformable(StaticMeshComponent))
				{
					ActorTransformableComponents.Add(StaticMeshComponent);
				}
				else if (!CanIgnoreComponent(Component))
				{
					bContainsNonTransformable = true;
				}
			});

			// Can't convert partially a Blueprint actor: Rerun CS is called in PIE when registering the component
			// and is also called when registering components during cook/save of the level.
			if (UBlueprint::GetBlueprintFromClass(Actor->GetClass()) && ActorTransformableComponents.Num() && bContainsNonTransformable)
			{
				continue;
			}

			for (UStaticMeshComponent* StaticMeshComponent : ActorTransformableComponents)
			{
				FWorldPartitionCellTransformerISMComponentDescriptor Descriptor;
				Descriptor.bStrictBucketing = bStrictBucketing;
				Descriptor.InitFrom(StaticMeshComponent);
				ISMComponentBatchers.FindOrAdd(Descriptor).ActorComponents.FindOrAdd(&Actor).Add(StaticMeshComponent);
			}
		}
	}

	// Keep a set of non-transformable actors
	TSet<AActor*> NonTransformableActors;
	for (auto It = ISMComponentBatchers.CreateIterator(); It; ++It)
	{
		const FActorComponentBatcherDescriptor& ActorComponentBatcherDescriptor = It->Value;

		if ((uint32)ActorComponentBatcherDescriptor.ActorComponents.Num() < MinNumInstances)
		{
			for (auto& [Actor, Components] : ActorComponentBatcherDescriptor.ActorComponents)
			{
				// For Blueprint actors, if any of their contributions fail MinNumInstances,
				// we treat the entire actor as non-transformable to avoid partial conversion.
				if (UBlueprint::GetBlueprintFromClass((*Actor)->GetClass()))
				{
					NonTransformableActors.Add(*Actor);
				}
			}

			// Remove elements that don't respect the minimum number of instances
			It.RemoveCurrent();
		}
	}

	int32 NumInstancedComponents = 0;
	for (auto& [ISMComponentDescriptor, ActorComponentBatcherDescriptor] : ISMComponentBatchers)
	{
		check((uint32)ActorComponentBatcherDescriptor.ActorComponents.Num() >= MinNumInstances);
		
		for (auto& [Actor, Components] : ActorComponentBatcherDescriptor.ActorComponents)
		{
			// Skip actors that were marked as non-transformable
			if (NonTransformableActors.Contains(*Actor))
			{
				continue;
			}

			for (UStaticMeshComponent* StaticMeshComponent : Components)
			{
				// Register the component into the batcher
				StaticMeshComponent->UpdateComponentToWorld();
				ActorComponentBatcherDescriptor.ISMComponentBatcher.Add(StaticMeshComponent);

				// Remove the component from the actor
				(*Actor)->RemoveOwnedComponent(StaticMeshComponent);
				StaticMeshComponent->MarkAsGarbage();

				NumInstancedComponents++;
			}

			if (CanRemoveActor(*Actor))
			{
				*Actor = nullptr;
			}
			else if (USceneComponent* OldRootComponent = (*Actor)->GetRootComponent(); OldRootComponent && !IsValid(OldRootComponent))
			{
				USceneComponent* NewRootComponent = NewObject<USceneComponent>(*Actor);
				NewRootComponent->SetRelativeTransform(OldRootComponent->GetRelativeTransform());
				NewRootComponent->SetMobility(OldRootComponent->GetMobility());
				(*Actor)->SetRootComponent(NewRootComponent);

				(*Actor)->ForEachComponent<USceneComponent>(false, [OldRootComponent, NewRootComponent](USceneComponent* Component)
				{
					if (Component->GetAttachParent() == OldRootComponent)
					{
						Component->SetupAttachment(NewRootComponent);
					}
				});
			}
		}
	}

	InLevel->Actors.Remove(nullptr);

	if (NumInstancedComponents)
	{
		AActor* PackedActor = NewObject<AWorldPartitionAutoInstancedActor>(InLevel);

		for (auto& [ISMComponentDescriptor, ActorComponentBatcherDescriptor] : ISMComponentBatchers)
		{
			if (ActorComponentBatcherDescriptor.ISMComponentBatcher.GetNumInstances())
			{
				UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(PackedActor);
				ISMComponentDescriptor.InitComponent(ISMComponent);
				ActorComponentBatcherDescriptor.ISMComponentBatcher.InitComponent(ISMComponent);

				if (!PackedActor->GetRootComponent())
				{
					PackedActor->SetRootComponent(ISMComponent);
				}

				ISMComponent->SetMobility(EComponentMobility::Static);
				ISMComponent->SetWorldTransform(FTransform::Identity);

				PackedActor->AddInstanceComponent(ISMComponent);
			}
		}

		InLevel->Actors.Add(PackedActor);
	}
}
bool UWorldPartitionRuntimeCellTransformerISM::CanAutoInstanceActor(AActor* InActor) const
{
	if (InActor->ActorHasTag(NAME_CellTransformerIgnoreActor))
	{
		return false;
	}
	
	if (InActor->GetIsReplicated())
	{
		return false;
	}

	if (!InActor->IsRootComponentStatic())
	{
		return false;
	}

	if (InActor->IsHidden())
	{
		return false;
	}

	if (InActor->IsEditorOnly())
	{
		return false;
	}

	if (InActor->Children.Num() || InActor->IsChildActor())
	{
		return false;
	}

	if (IsBlueprintActorWithLogic(InActor))
	{
		return false;
	}

	if (!IsActorTransformable(InActor))
	{
		return false;
	}

	UClass* ActorClass = InActor->GetClass();
	for (TSubclassOf<AActor> DisallowedClass : DisallowedClasses)
	{
		if (ActorClass == DisallowedClass)
		{
			return false;
		}
	}

	for (TSubclassOf<AActor> AllowedClass : AllowedClasses)
	{
		if (ActorClass->IsChildOf(AllowedClass))
		{
			return true;
		}
	}

	return false;
}

bool UWorldPartitionRuntimeCellTransformerISM::CanIgnoreComponent(const UActorComponent* InComponent) const
{
	return InComponent->IsEditorOnly() || Super::CanIgnoreComponent(InComponent);
}

bool UWorldPartitionRuntimeCellTransformerISM::CanRemoveActor(AActor* InActor) const
{
	for (UActorComponent* Component : InActor->GetComponents())
	{
		if (!CanIgnoreComponent(Component))
		{		
			return false;
		}
	}

	return true;
}
#endif

AWorldPartitionAutoInstancedActor::AWorldPartitionAutoInstancedActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#undef LOCTEXT_NAMESPACE
