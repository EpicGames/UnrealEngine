// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeCellTransformerLog.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "CoreMinimal.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Templates/Greater.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeCellTransformerLog)

UWorldPartitionRuntimeCellTransformerLog::UWorldPartitionRuntimeCellTransformerLog(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bOnlyLogDifferences = true;
#endif
}

#if WITH_EDITOR
void UWorldPartitionRuntimeCellTransformerLog::PreTransform(ULevel* InLevel)
{
	check(ClassNumInstancesBefore.IsEmpty());
	GatherLevelContentStats(InLevel, ClassNumInstancesBefore);

	if (!bOnlyLogDifferences)
	{
		UE_LOGF(LogWorldPartition, Log, "Level %ls PreTransform Content:", *InLevel->GetPackage()->GetName());
		DumpLevelContentStats(InLevel, ClassNumInstancesBefore);
	}
}

void UWorldPartitionRuntimeCellTransformerLog::PostTransform(ULevel* InLevel)
{
	TMap<UClass*, int32> ClassNumInstancesAfter;
	GatherLevelContentStats(InLevel, ClassNumInstancesAfter);

	if (bOnlyLogDifferences)
	{
		if (!ClassNumInstancesBefore.OrderIndependentCompareEqual(ClassNumInstancesAfter))
		{
			UE_LOGF(LogWorldPartition, Log, "Level %ls PostTransform Delta:", *InLevel->GetPackage()->GetName());

			TMap<UClass*, int32> ClassNumInstancesDelta;

			for (auto& [Class, NumInstances] : ClassNumInstancesBefore)
			{
				const int32* NumInstancesAfter = ClassNumInstancesAfter.Find(Class);
				ClassNumInstancesDelta.FindOrAdd(Class) += (NumInstancesAfter ? *NumInstancesAfter : 0) - NumInstances;
			}

			for (auto& [Class, NumInstances] : ClassNumInstancesAfter)
			{
				if (!ClassNumInstancesBefore.Contains(Class))
				{
					ClassNumInstancesDelta.FindOrAdd(Class) += NumInstances;
				}
			}

			DumpLevelContentStats(InLevel, ClassNumInstancesDelta);
		}
	}
	else
	{
		UE_LOGF(LogWorldPartition, Log, "Level %ls PostTransform Content:", *InLevel->GetPackage()->GetName());
		DumpLevelContentStats(InLevel, ClassNumInstancesAfter);
	}

	ClassNumInstancesBefore.Empty();
}

void UWorldPartitionRuntimeCellTransformerLog::GatherLevelContentStats(ULevel* InLevel, TMap<UClass*, int32>& OutClassNumInstances)
{
	for (AActor* Actor : InLevel->Actors)
	{
		if (IsValid(Actor))
		{
			OutClassNumInstances.FindOrAdd(Actor->GetClass())++;
		}
	}

	OutClassNumInstances.ValueSort(TGreater<>());
}

void UWorldPartitionRuntimeCellTransformerLog::DumpLevelContentStats(ULevel* InLevel, const TMap<UClass*, int32>& InClassNumInstances)
{
	if (InClassNumInstances.Num())
	{
		UE_LOGF(LogWorldPartition, Log, "\tNum Actors: %d", InLevel->Actors.Num());
		UE_LOGF(LogWorldPartition, Log, "\tActor Class Breakdown:");
		for (auto& [Class, NumInstances] : InClassNumInstances)
		{
			if (NumInstances)
			{
				UE_LOGF(LogWorldPartition, Log, "\t\t%ls: %d", *Class->GetName(), NumInstances);
			}
		}
	}
}
#endif
