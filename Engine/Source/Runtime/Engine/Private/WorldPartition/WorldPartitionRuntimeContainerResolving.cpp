// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeContainerResolving.h"
#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeContainerResolving)

bool FWorldPartitionRuntimeContainerResolver::ResolveContainerPath(const FString& InSubObjectString, FString& OutSubObjectString) const
{
	if (!IsValid())
	{
		return false;
	}

	FString SubObjectString;
	FString SubObjectContext(InSubObjectString);
	InSubObjectString.Split(TEXT("."), &SubObjectContext, &SubObjectString);
	
	const FWorldPartitionRuntimeContainer* Container = &Containers.FindChecked(MainContainerPackage);
	FActorContainerID ContainerID;
	check(ContainerID.IsMainContainer());
			
	while (FWorldPartitionRuntimeContainerInstance const* ContainerInstance = Container->ContainerInstances.Find(*SubObjectContext))
	{
		Container = &Containers.FindChecked(ContainerInstance->ContainerPackage);
		ContainerID = FActorContainerID(ContainerID, ContainerInstance->ActorGuid);
		
		SubObjectContext = SubObjectString;
		if (!SubObjectString.Split(TEXT("."), &SubObjectContext, &SubObjectString))
		{
			check(SubObjectContext == SubObjectString);
			SubObjectString.Empty();
			break;
		}
	}

	if (!ContainerID.IsMainContainer())
	{
		OutSubObjectString = FWorldPartitionLevelHelper::AddActorContainerID(ContainerID, SubObjectContext);
		if (!SubObjectString.IsEmpty())
		{
			OutSubObjectString += TEXT(".") + SubObjectString;
		}

		return true;
	}

	return false;
}

SIZE_T FWorldPartitionRuntimeContainer::GetAllocatedSize() const
{
	return ContainerInstances.GetAllocatedSize();
}

SIZE_T FWorldPartitionRuntimeContainerResolver::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = Containers.GetAllocatedSize();
	for (auto& [Name,Container] : Containers)
	{
		AllocatedSize += Container.GetAllocatedSize();
	}

	return AllocatedSize;
}

#if WITH_EDITOR

void FWorldPartitionRuntimeContainerResolver::BuildContainerIDToEditorPathMap()
{
	check(!MainContainerPackage.IsNone());
	check(ContainerIDToEditorPath.IsEmpty());

	TFunction<void(const FActorContainerID&, FName, const FString&)> BuildContainerIDToEditorMapRecursive =
		[this, &BuildContainerIDToEditorMapRecursive](const FActorContainerID& InParentContainerID, FName InContainerPackage, const FString& InPath)
	{
		const FWorldPartitionRuntimeContainer& Container = Containers.FindChecked(InContainerPackage);
		for (auto [Name, ContainerInstance] : Container.ContainerInstances)
		{
			FActorContainerID ContainerInstanceID(InParentContainerID, ContainerInstance.ActorGuid);
			FString ContainerInstanceEditorPath = InPath.IsEmpty() ? Name.ToString() : InPath + TEXT(".") + Name.ToString();
			ContainerIDToEditorPath.Add(ContainerInstanceID, ContainerInstanceEditorPath);

			BuildContainerIDToEditorMapRecursive(ContainerInstanceID, ContainerInstance.ContainerPackage, ContainerInstanceEditorPath);
		}
	};

	BuildContainerIDToEditorMapRecursive(FActorContainerID(), MainContainerPackage, FString());
}

#endif
