// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Misc/HierarchicalLogArchive.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "Misc/ArchiveMD5.h"
#include "Cooker/CookEvents.h"
#if WITH_EDITOR
#include "WorldPartition/Cook/WorldPartitionCookPackage.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContextInterface.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeHash)

#define LOCTEXT_NAMESPACE "WorldPartition"

ENGINE_API float GSlowStreamingRatio = 0.25f;
static FAutoConsoleVariableRef CVarSlowStreamingRatio(
	TEXT("wp.Runtime.SlowStreamingRatio"),
	GSlowStreamingRatio,
	TEXT("Ratio of DistanceToCell / LoadingRange to use to determine if World Partition streaming is considered to be slow"));

ENGINE_API float GSlowStreamingWarningFactor = 2.f;
static FAutoConsoleVariableRef CVarSlowStreamingWarningFactor(
	TEXT("wp.Runtime.SlowStreamingWarningFactor"),
	GSlowStreamingWarningFactor,
	TEXT("Factor of wp.Runtime.SlowStreamingRatio we want to start notifying the user"));

ENGINE_API float GBlockOnSlowStreamingRatio = 0.25f;
static FAutoConsoleVariableRef CVarBlockOnSlowStreamingRatio(
	TEXT("wp.Runtime.BlockOnSlowStreamingRatio"),
	GBlockOnSlowStreamingRatio,
	TEXT("Ratio of DistanceToCell / LoadingRange to use to determine if World Partition streaming needs to block"));

ENGINE_API float GBlockOnSlowStreamingWarningFactor = 2.f;
static FAutoConsoleVariableRef CVarBlockOnSlowStreamingWarningFactor(
	TEXT("wp.Runtime.BlockOnSlowStreamingWarningFactor"),
	GBlockOnSlowStreamingWarningFactor,
	TEXT("Factor of wp.Runtime.BlockOnSlowStreamingRatio we want to start notifying the user"));

const TCHAR* EnumToString(EWorldPartitionStreamingPerformance InStreamingPerformance)
{
	switch (InStreamingPerformance)
	{
	case EWorldPartitionStreamingPerformance::Immediate:
		return TEXT("Immediate");
	case EWorldPartitionStreamingPerformance::Critical:
		return TEXT("Critical");
	case EWorldPartitionStreamingPerformance::Slow:
		return TEXT("Slow");
	case EWorldPartitionStreamingPerformance::Good:
		return TEXT("Good");
	}
	ensure(false);
	return TEXT("Unknown");
}

#if DO_CHECK
void URuntimeHashExternalStreamingObjectBase::BeginDestroy()
{
	checkf(!TargetInjectedWorldPartition.Get(), TEXT("Destroying external streaming object that is still injected."));
	Super::BeginDestroy();
}
#endif

void URuntimeHashExternalStreamingObjectBase::ForEachStreamingCells(TFunctionRef<void(UWorldPartitionRuntimeCell&)> Func)
{
	TArray<UObject*> Objects;
	GetObjectsWithOuter(this, Objects);

	for (UObject* Object : Objects)
	{
		if (UWorldPartitionRuntimeCell* Cell = Cast<UWorldPartitionRuntimeCell>(Object))
		{
			Func(*Cell);
		}
	}
}

TSet<TObjectPtr<UDataLayerInstance>>& URuntimeHashExternalStreamingObjectBase::GetDataLayerInstances()
{
	return DataLayerInstances;
}

const UObject* URuntimeHashExternalStreamingObjectBase::GetLevelMountPointContextObject() const
{
	return GetRootExternalDataLayerAsset();
}

UWorld* URuntimeHashExternalStreamingObjectBase::GetOwningWorld() const
{
	// Once OnStreamingObjectLoaded is called and OwningWorld is set, use this cached value
	return OwningWorld.IsSet() ? OwningWorld.GetValue().Get() : GetOuterWorld()->GetWorldPartition()->GetWorld();
}

void URuntimeHashExternalStreamingObjectBase::OnStreamingObjectLoaded(UWorld* InjectedWorld)
{
#if !WITH_EDITOR
	if (!CellToStreamingData.IsEmpty())
	{
		// Cooked streaming object's Cells do not have LevelStreaming.
		ForEachStreamingCells([this](UWorldPartitionRuntimeCell& Cell)
		{
			UWorldPartitionRuntimeLevelStreamingCell* RuntimeCell = CastChecked<UWorldPartitionRuntimeLevelStreamingCell>(&Cell);

			const FWorldPartitionRuntimeCellStreamingData& CellStreamingData = CellToStreamingData.FindChecked(RuntimeCell->GetFName());
			RuntimeCell->CreateAndSetLevelStreaming(CellStreamingData.PackageName, CellStreamingData.WorldAsset);
		});
	}
#endif

	check(GetOuterWorld());
	check(GetOuterWorld()->GetWorldPartition());
	check(GetOuterWorld()->GetWorldPartition()->GetWorld());
	OwningWorld = GetOuterWorld()->GetWorldPartition()->GetWorld();
}

#if WITH_EDITOR
UWorldPartitionRuntimeCell* URuntimeHashExternalStreamingObjectBase::GetCellForCookPackage(const FString& InCookPackageName) const
{
	if (const TObjectPtr<UWorldPartitionRuntimeCell>* MatchingCell = PackagesToGenerateForCook.Find(InCookPackageName))
	{
		if (ensure(*MatchingCell))
		{
			return const_cast<UWorldPartitionRuntimeCell*>(ToRawPtr(*MatchingCell));
		}
	}
	return nullptr;
}

FString URuntimeHashExternalStreamingObjectBase::GetPackageNameToCreate() const
{
	// GetPackageNameToCreate should not be called for ExternalStreamingObjects belonging to content bundles
	if (ensure(ExternalDataLayerAsset))
	{
		return TEXT("/") + FExternalDataLayerHelper::GetExternalStreamingObjectPackageName(ExternalDataLayerAsset);
	}
	return FString();
}


void URuntimeHashExternalStreamingObjectBase::SetPackagePathToCreate(const  FString& InPath)
{
	PackagePathToCreate = InPath;
}

const FString& URuntimeHashExternalStreamingObjectBase::GetPackagePathToCreate() const
{
	return PackagePathToCreate;
}

bool URuntimeHashExternalStreamingObjectBase::PrepareForCook(const IWorldPartitionCookPackageContext& InCookContext)
{
	bool bResult = true;
	ForEachStreamingCells([this, &bResult, &InCookContext](UWorldPartitionRuntimeCell& Cell)
	{
		// Make cell is ready for cook
		if (Cell.PrepareCellForCook(InCookContext))
		{
			UWorldPartitionRuntimeLevelStreamingCell* RuntimeCell = CastChecked<UWorldPartitionRuntimeLevelStreamingCell>(&Cell);
			UWorldPartitionLevelStreamingDynamic* LevelStreamingDynamic = RuntimeCell->GetLevelStreaming();
			FWorldPartitionRuntimeCellStreamingData& CellStreamingData = CellToStreamingData.Add(RuntimeCell->GetFName());
			CellStreamingData.PackageName = LevelStreamingDynamic->GetWorldAsset().GetLongPackageName();
			// SoftObjectPath will be automatically remapped when ExternalStreamingObject will be instanced/loaded at runtime
			CellStreamingData.WorldAsset = LevelStreamingDynamic->GetWorldAsset().ToSoftObjectPath();

			// Level streaming are outered to the world and would not be saved within the ExternalStreamingObject.
			// Do not save them, instead they will be created once the external streaming object is loaded at runtime. 
			LevelStreamingDynamic->SetFlags(RF_Transient);
		}
		else
		{
			bResult = false;
		}
	});
	return bResult;
}

bool URuntimeHashExternalStreamingObjectBase::OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage)
{
	return PrepareForCook(InCookContext);
}

bool URuntimeHashExternalStreamingObjectBase::OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage, TArray<UPackage*>& OutModifiedPackages)
{
	if (PrepareForCook(InCookContext))
	{
		// We provide a new name for the URuntimeHashExternalStreamingObjectBase in the package so that we have a stable name (for cook determinism)
		return Rename(URuntimeHashExternalStreamingObjectBase::GetCookedExternalStreamingObjectName(), InGeneratedPackage, REN_DontCreateRedirectors);
	}
	return false;
}

void URuntimeHashExternalStreamingObjectBase::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	Ar.Printf(TEXT("%s%s"), *GetWorld()->GetName(), ExternalDataLayerAsset ? *FString::Printf(TEXT(" - External Data Layer - %s"), *ExternalDataLayerAsset->GetName()) : TEXT(""));
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
}

TMap<TPair<const UClass*, const UClass*>, UWorldPartitionRuntimeHash::FRuntimeHashConvertFunc> UWorldPartitionRuntimeHash::WorldPartitionRuntimeHashConverters;
#endif

UWorldPartitionRuntimeHash::UWorldPartitionRuntimeHash(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

UWorldPartitionRuntimeCell* UWorldPartitionRuntimeHash::CreateRuntimeCell(UClass* CellClass, UClass* CellDataClass, const FString& CellName, const FString& CellInstanceSuffix, UObject* InOuter)
{
	//@todo_ow: to reduce file paths on windows, we compute a MD5 hash from the unique cell name and use that hash as the cell filename. We
	//			should create the runtime cell using the name but make sure the package that gets associated with it when cooking gets the
	//			hash indtead.
	auto GetCellObjectName = [](FString CellName) -> FString
	{
		FArchiveMD5 ArMD5;
		ArMD5 << CellName;
		FGuid CellNameGuid = ArMD5.GetGuidFromHash();
		check(CellNameGuid.IsValid());

		return CellNameGuid.ToString(EGuidFormats::Base36Encoded);
	};

	// Cooking should have an empty CellInstanceSuffix
	check(!IsRunningCookCommandlet() || CellInstanceSuffix.IsEmpty());
	const FString CellObjectName = GetCellObjectName(CellName) + CellInstanceSuffix;
	// Use given outer if provided, else use hash as outer
	UObject* Outer = InOuter ? InOuter : this;
	UWorldPartitionRuntimeCell* RuntimeCell = NewObject<UWorldPartitionRuntimeCell>(Outer, CellClass, *CellObjectName);
	RuntimeCell->RuntimeCellData = NewObject<UWorldPartitionRuntimeCellData>(RuntimeCell, CellDataClass);
	return RuntimeCell;
}

EWorldPartitionStreamingPerformance UWorldPartitionRuntimeHash::GetStreamingPerformanceForCell(const UWorldPartitionRuntimeCell* Cell, bool& bOutShouldBlock) const
{
	EWorldPartitionStreamingPerformance BlockingPerformance = EWorldPartitionStreamingPerformance::Good;
	if (Cell->GetBlockOnSlowLoading() && Cell->RuntimeCellData->bCachedWasRequestedByBlockingSource)
	{
		if (Cell->RuntimeCellData->CachedMinBlockOnSlowStreamingRatio <= 0.0)
		{
			BlockingPerformance = EWorldPartitionStreamingPerformance::Immediate;
		}
		else if (Cell->RuntimeCellData->CachedMinBlockOnSlowStreamingRatio < GBlockOnSlowStreamingRatio)
		{
			BlockingPerformance = EWorldPartitionStreamingPerformance::Critical;
		}
		else if (Cell->RuntimeCellData->CachedMinBlockOnSlowStreamingRatio < (GBlockOnSlowStreamingRatio * GBlockOnSlowStreamingWarningFactor))
		{
			BlockingPerformance = EWorldPartitionStreamingPerformance::Slow;
		}
	}

	EWorldPartitionStreamingPerformance NonBlockingPerformance = EWorldPartitionStreamingPerformance::Good;
	if (Cell->RuntimeCellData->CachedMinSlowStreamingRatio <= 0.0)
	{
		NonBlockingPerformance = EWorldPartitionStreamingPerformance::Immediate;
	}
	else if (Cell->RuntimeCellData->CachedMinSlowStreamingRatio < GSlowStreamingRatio)
	{
		NonBlockingPerformance = EWorldPartitionStreamingPerformance::Critical;
	}
	else if (Cell->RuntimeCellData->CachedMinSlowStreamingRatio < (GSlowStreamingRatio * GSlowStreamingWarningFactor))
	{
		NonBlockingPerformance = EWorldPartitionStreamingPerformance::Slow;
	}

	if (BlockingPerformance >= NonBlockingPerformance)
	{
		bOutShouldBlock = BlockingPerformance >= EWorldPartitionStreamingPerformance::Critical;
		return BlockingPerformance;
	}
	else
	{
		bOutShouldBlock = false;
		return NonBlockingPerformance;
	}
}

URuntimeHashExternalStreamingObjectBase* UWorldPartitionRuntimeHash::CreateExternalStreamingObject(TSubclassOf<URuntimeHashExternalStreamingObjectBase> InClass, UObject* InOuter, UWorld* InOuterWorld)
{
	URuntimeHashExternalStreamingObjectBase* StreamingObject = NewObject<URuntimeHashExternalStreamingObjectBase>(InOuter, InClass, NAME_None, RF_Public);
	StreamingObject->OuterWorld = InOuterWorld;	
	return StreamingObject;
}

#if WITH_EDITOR
void UWorldPartitionRuntimeHash::PrepareEditorGameWorld()
{
	// Mark always loaded actors so that the Level will force reference to these actors for PIE.
	// These actor will then be duplicated for PIE during the PIE world duplication process
	ForceExternalActorLevelReference(/*bForceExternalActorLevelReference*/true);
}

void UWorldPartitionRuntimeHash::ShutdownEditorGameWorld()
{
	// Unmark always loaded actors
	ForceExternalActorLevelReference(/*bForceExternalActorLevelReference*/false);
}

bool UWorldPartitionRuntimeHash::GenerateStreaming(class UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate)
{
	return PackagesToGenerateForCook.IsEmpty();
}

void UWorldPartitionRuntimeHash::FlushStreamingContent()
{
	PackagesToGenerateForCook.Empty();

	// Release references (will unload actors that were not already loaded in the editor)
	{
		FWorldPartitionLoadingContext::FDeferred LoadingContext;
		EditorAlwaysLoadedActor.Empty();
	}
}

bool UWorldPartitionRuntimeHash::PopulateCellActorInstances(const TArray<const IStreamingGenerationContext::FActorSetInstance*>& ActorSetInstances, bool bIsMainWorldPartition, bool bIsCellAlwaysLoaded, TArray<IStreamingGenerationContext::FActorInstance>& OutCellActorInstances)
{
	// In PIE, Always loaded cell is not generated. Instead, always loaded actors will be added to AlwaysLoadedActorsForPIE.
	// This will trigger loading/registration of these actors in the PersistentLevel (if not already loaded).
	// Then, duplication of world for PIE will duplicate only these actors. 
	// When stopping PIE, WorldPartition will release these FWorldPartitionReferences which 
	// will unload actors that were not already loaded in the non PIE world.
	TArray<TPair<FWorldPartitionReference, const IWorldPartitionActorDescInstanceView*>> AlwaysLoadedReferences;
	{
		FWorldPartitionLoadingContext::FDeferred LoadingContext;

		const bool bForceLoadAlwaysLoadedReferences = bIsMainWorldPartition && bIsCellAlwaysLoaded && !IsRunningCookCommandlet();

		for (const IStreamingGenerationContext::FActorSetInstance* ActorSetInstance : ActorSetInstances)
		{
			ActorSetInstance->ForEachActor([this, ActorSetInstance, &AlwaysLoadedReferences, &OutCellActorInstances, bForceLoadAlwaysLoadedReferences](const FGuid& ActorGuid)
			{
				IStreamingGenerationContext::FActorInstance ActorInstance(ActorGuid, ActorSetInstance);
				const IWorldPartitionActorDescInstanceView& ActorDescView = ActorInstance.GetActorDescView();

				// Instanced world partition, ContainerID is the main container, but it's not the main world partition,
				// so the always loaded actors don't be part of the process of ForceExternalActorLevelReference/AlwaysLoadedActorsForPIE.
				// In PIE, always loaded actors of an instanced world partition will go in the always loaded cell.
				if (bForceLoadAlwaysLoadedReferences && ActorSetInstance->ContainerID.IsMainContainer())
				{
					// This will load the actor if it isn't already loaded, when the deferred context ends.
					AlwaysLoadedReferences.Emplace(FWorldPartitionReference(GetOuterUWorldPartition(), ActorDescView.GetGuid()), &ActorDescView);
				}
				else
				{
					// Actors that return true to ShouldLevelKeepRefIfExternal will always be part of the partitioned persistent level of a
					// world partition. In PIE, for an instanced world partition, we don't want this actor to be both in the persistent level
					// and also part of the always loaded cell level.
					//
					// @todo_ow: We need to implement PIE always loaded actors of instanced world partitions to be part
					//			 of the persistent level and get rid of the always loaded cell (to have the same behavior
					//			 as non-instanced world partition and as cooked world partition).
					if (!CastChecked<AActor>(ActorDescView.GetActorNativeClass()->GetDefaultObject())->ShouldLevelKeepRefIfExternal())
					{
						OutCellActorInstances.Emplace(ActorInstance);
					}
				}
			});
		}
	}

	// Here we need to use the actor descriptor view, as the always loaded reference object might not have a valid actor descriptor for newly added actors, etc.
	for (auto& [Reference, ActorDescView] : AlwaysLoadedReferences)
	{
		if (AActor* AlwaysLoadedActor = FindObject<AActor>(nullptr, *ActorDescView->GetActorSoftPath().ToString()))
		{
			EditorAlwaysLoadedActor.Emplace(Reference, AlwaysLoadedActor);
		}
	}

	return OutCellActorInstances.Num() > 0;
}

void UWorldPartitionRuntimeHash::PopulateRuntimeCell(UWorldPartitionRuntimeCell* RuntimeCell, const TArray<IStreamingGenerationContext::FActorInstance>& ActorInstances, TArray<FString>* OutPackagesToGenerate)
{
	for (const IStreamingGenerationContext::FActorInstance& ActorInstance : ActorInstances)
	{
		if (ActorInstance.GetContainerID().IsMainContainer())
		{
			const FStreamingGenerationActorDescView& ActorDescView = ActorInstance.GetActorDescView();
			if (AActor* Actor = FindObject<AActor>(nullptr, *ActorDescView.GetActorSoftPath().ToString()))
			{
				if (ActorDescView.IsUnsaved())
				{
					// Create an actor container to make sure duplicated actors will share an outer to properly remap inter-actors references
					RuntimeCell->UnsavedActorsContainer = NewObject<UActorContainer>(RuntimeCell);
					break;
				}
			}
		}
	}

	FBox CellContentBounds(ForceInit);
	for (const IStreamingGenerationContext::FActorInstance& ActorInstance : ActorInstances)
	{
		const FStreamingGenerationActorDescView& ActorDescView = ActorInstance.GetActorDescView();
		RuntimeCell->AddActorToCell(ActorDescView);

		CellContentBounds += ActorInstance.GetBounds();

		if (ActorInstance.GetContainerID().IsMainContainer() && RuntimeCell->UnsavedActorsContainer)
		{
			if (AActor* Actor = FindObject<AActor>(nullptr, *ActorDescView.GetActorSoftPath().ToString()))
			{
				RuntimeCell->UnsavedActorsContainer->Actors.Add(Actor->GetFName(), Actor);
			}
		}
	}

	RuntimeCell->RuntimeCellData->ContentBounds = CellContentBounds;
	RuntimeCell->Fixup();

	// Always loaded cell actors are transferred to World's Persistent Level (see UWorldPartitionRuntimeSpatialHash::PopulateGeneratorPackageForCook)
	if (OutPackagesToGenerate && RuntimeCell->GetActorCount() && !RuntimeCell->IsAlwaysLoaded())
	{
		const FString PackageRelativePath = RuntimeCell->GetPackageNameToCreate();
		check(!PackageRelativePath.IsEmpty());

		OutPackagesToGenerate->Add(PackageRelativePath);

		// Map relative package to StreamingCell for PopulateGeneratedPackageForCook/PopulateGeneratorPackageForCook/GetCellForPackage
		PackagesToGenerateForCook.Add(PackageRelativePath, RuntimeCell);
	}
}

UWorldPartitionRuntimeCell* UWorldPartitionRuntimeHash::GetCellForCookPackage(const FString& InCookPackageName) const
{
	if (const TObjectPtr<UWorldPartitionRuntimeCell>* MatchingCell = PackagesToGenerateForCook.Find(InCookPackageName))
	{
		if (ensure(*MatchingCell))
		{
			return const_cast<UWorldPartitionRuntimeCell*>(ToRawPtr(*MatchingCell));
		}
	}
	return nullptr;
}

URuntimeHashExternalStreamingObjectBase* UWorldPartitionRuntimeHash::StoreStreamingContentToExternalStreamingObject()
{
	URuntimeHashExternalStreamingObjectBase* NewExternalStreamingObject = CreateExternalStreamingObject(GetExternalStreamingObjectClass(), GetOuterUWorldPartition(), GetTypedOuter<UWorld>());
	StoreStreamingContentToExternalStreamingObject(NewExternalStreamingObject);
	return NewExternalStreamingObject;
}

void UWorldPartitionRuntimeHash::StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject)
{
	OutExternalStreamingObject->PackagesToGenerateForCook = MoveTemp(PackagesToGenerateForCook);
}

TArray<UWorldPartitionRuntimeCell*> UWorldPartitionRuntimeHash::GetAlwaysLoadedCells() const
{
	TArray<UWorldPartitionRuntimeCell*> Result;
	ForEachStreamingCells([&Result](const UWorldPartitionRuntimeCell* Cell)
	{
		if (Cell->IsAlwaysLoaded())
		{
			Result.Add(const_cast<UWorldPartitionRuntimeCell*>(Cell));
		}
		return true;
	});
	return Result;
}

void UWorldPartitionRuntimeHash::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	Ar.Printf(TEXT("%s - Persistent Level"), *GetWorld()->GetName());
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	{
		FHierarchicalLogArchive::FIndentScope CellIndentScope = Ar.PrintfIndent(TEXT("Content of %s Persistent Level"), *GetWorld()->GetName());

		TArray<TPair<FString, FString>> Actors;

		if (!IsRunningCookCommandlet())
		{
			for (const FEditorAlwaysLoadedActor& AlwaysLoadedActor : EditorAlwaysLoadedActor)
			{
				if (AlwaysLoadedActor.Actor.IsValid())
				{
					Actors.Add(
					{
						FString::Printf(TEXT("Actor Path: %s"), *AlwaysLoadedActor.Actor->GetPathName()), 
						FString::Printf(TEXT("Actor Package: %s"), *AlwaysLoadedActor.Actor->GetPackage()->GetName())
					});
				}
			}
		}
		else
		{
			for (UWorldPartitionRuntimeCell* Cell : GetAlwaysLoadedCells())
			{
				const UWorldPartitionRuntimeLevelStreamingCell* RuntimeCell = CastChecked<UWorldPartitionRuntimeLevelStreamingCell>(Cell);

				for (const FWorldPartitionRuntimeCellObjectMapping& Package : RuntimeCell->GetPackages())
				{
					Actors.Add(
					{
						FString::Printf(TEXT("Actor Path: %s"), *Package.Path.ToString()), 
						FString::Printf(TEXT("Actor Package: %s"), *Package.Package.ToString())
					});
				}
			}
		}

		Actors.Sort();

		Ar.Printf(TEXT("Always loaded Actor Count: %d "), Actors.Num());
		for (const TPair<FString, FString>& Actor : Actors)
		{
			Ar.Print(*Actor.Key);
			Ar.Print(*Actor.Value);
		}
	}
	Ar.Printf(TEXT(""));
}

void UWorldPartitionRuntimeHash::ForceExternalActorLevelReference(bool bForceExternalActorLevelReference)
{
	// Do this only on non game worlds prior to PIE so that always loaded actors get duplicated with the world
	if (!GetWorld()->IsGameWorld())
	{
		for (const FEditorAlwaysLoadedActor& AlwaysLoadedActor : EditorAlwaysLoadedActor)
		{
			if (AActor* Actor = AlwaysLoadedActor.Actor.Get())
			{
				Actor->SetForceExternalActorLevelReferenceForPIE(bForceExternalActorLevelReference);
			}
		}
	}
}

bool UWorldPartitionRuntimeHash::ResolveBlockOnSlowStreamingForCell(bool bInOwnerBlockOnSlowStreaming, bool bInIsHLODCell, const TArray<const UDataLayerInstance*>& InCellDataLayerInstances) const
{
	if (bInIsHLODCell)
	{
		return false;
	}

	TOptional<bool> DataLayersOverrideBlockOnSlowStreaming;
	for (const UDataLayerInstance* DataLayerInstance : InCellDataLayerInstances)
	{
		if (DataLayerInstance->GetOverrideBlockOnSlowStreaming() != EOverrideBlockOnSlowStreaming::NoOverride)
		{
			bool bIsBlocking = DataLayerInstance->GetOverrideBlockOnSlowStreaming() == EOverrideBlockOnSlowStreaming::Blocking;
			DataLayersOverrideBlockOnSlowStreaming = bIsBlocking;
			if (bIsBlocking)
			{
				break;
			}
		}
	}
	const bool bBlockOnSlowStreaming = DataLayersOverrideBlockOnSlowStreaming.IsSet() ? DataLayersOverrideBlockOnSlowStreaming.GetValue() : bInOwnerBlockOnSlowStreaming;
	return bBlockOnSlowStreaming;
}

int32 UWorldPartitionRuntimeHash::GetDataLayersStreamingPriority(const TArray<const UDataLayerInstance*>& InCellDataLayerInstances) const
{
	if (InCellDataLayerInstances.Num() == 0)
	{
		return 0;
	}

	bool bDetectedMismatch = false;
	int32 StreamingPriority = InCellDataLayerInstances[0]->GetStreamingPriority();
	for (const UDataLayerInstance* DataLayerInstance : InCellDataLayerInstances)
	{
		bDetectedMismatch |= (StreamingPriority != DataLayerInstance->GetStreamingPriority());
		StreamingPriority = FMath::Min(StreamingPriority, DataLayerInstance->GetStreamingPriority());
	}

#if !NO_LOGGING
	if (bDetectedMismatch && UE_LOG_ACTIVE(LogWorldPartition, Log))
	{
		TStringBuilder<0> HighestPriorityLayers;
		TStringBuilder<0> MismatchedLayers;
		for (const UDataLayerInstance* DataLayerInstance : InCellDataLayerInstances)
		{
			if (StreamingPriority == DataLayerInstance->GetStreamingPriority())
			{
				HighestPriorityLayers.Appendf(TEXT("%s "), *DataLayerInstance->GetDataLayerShortName());
			}
			else
			{
				MismatchedLayers.Appendf(TEXT("%s "), *DataLayerInstance->GetDataLayerShortName());
			}
		}
		UE_LOGF(LogWorldPartition, Log, "Found data layers streaming priority mismatch. The following data layers (%ls) have lower priority than (%ls) - using the highest found priority (%d) for this cell.", MismatchedLayers.ToString(), HighestPriorityLayers.ToString(), StreamingPriority);
	}
#endif

	return StreamingPriority;
}
#endif

bool UWorldPartitionRuntimeHash::IsCellRelevantFor(bool bClientOnlyVisible) const
{
	if (bClientOnlyVisible)
	{
		const UWorld* World = GetWorld();
		if (World->IsGameWorld())
		{
			// Dedicated server & listen server without server streaming won't consider client-only visible cells
			const ENetMode NetMode = World->GetNetMode();
			if ((NetMode == NM_DedicatedServer) || ((NetMode == NM_ListenServer) && !GetOuterUWorldPartition()->IsServerStreamingEnabled()))
			{
				return false;
			}
		}
	}
	return true;
}

EWorldPartitionStreamingPerformance UWorldPartitionRuntimeHash::GetStreamingPerformance(const TSet<const UWorldPartitionRuntimeCell*>& CellsToActivate, bool& bOutShouldBlock) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeHash::GetStreamingPerformance)
	EWorldPartitionStreamingPerformance StreamingPerformance = EWorldPartitionStreamingPerformance::Good;
	bOutShouldBlock = false;

	if (!CellsToActivate.IsEmpty())
	{
		for (const UWorldPartitionRuntimeCell* Cell : CellsToActivate)
		{
			if (!Cell->IsAlwaysLoaded() && !Cell->IsVisible())
			{
				bool bOutShouldBlockCell;
				EWorldPartitionStreamingPerformance CellPerformance = GetStreamingPerformanceForCell(Cell, bOutShouldBlockCell);
				bOutShouldBlock |= bOutShouldBlockCell;
				// Cell Performance is worst than previous cell performance
				if (CellPerformance > StreamingPerformance)
				{
					StreamingPerformance = CellPerformance;
					// Early out performance is critical
					if (bOutShouldBlockCell && StreamingPerformance >= EWorldPartitionStreamingPerformance::Critical)
					{
						return StreamingPerformance;
					}
				}
			}
		}
	}

	return StreamingPerformance;
}

bool UWorldPartitionRuntimeHash::IsExternalStreamingObjectInjected(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject) const
{
	return InjectedExternalStreamingObjects.Contains(InExternalStreamingObject);
}

bool UWorldPartitionRuntimeHash::InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject)
{
	if (!ensure(InExternalStreamingObject))
	{
		return false;
	}
	check(IsValid(InExternalStreamingObject));
	bool bAlreadyInSet = false;
	InjectedExternalStreamingObjects.Add(InExternalStreamingObject, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		UE_LOGF(LogWorldPartition, Warning, "External streaming object %ls already injected.", *InExternalStreamingObject->GetName());
		return false;
	}
	return true;
}

bool UWorldPartitionRuntimeHash::RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject)
{
	if (!ensure(InExternalStreamingObject))
	{
		return false;
	}
	check(IsValid(InExternalStreamingObject));
	if (!InjectedExternalStreamingObjects.Remove(InExternalStreamingObject))
	{
		UE_LOGF(LogWorldPartition, Warning, "External streaming object %ls was not injected.", *InExternalStreamingObject->GetName());
		return false;
	}
	return true;
}

#if WITH_EDITOR
void UWorldPartitionRuntimeHash::RegisterWorldPartitionRuntimeHashConverter(const UClass* InSrcClass, const UClass* InDstClass, FRuntimeHashConvertFunc&& InConverter)
{
	check(InSrcClass->IsChildOf<UWorldPartitionRuntimeHash>());
	check(InDstClass->IsChildOf<UWorldPartitionRuntimeHash>());
	WorldPartitionRuntimeHashConverters.Add({ InSrcClass, InDstClass }, MoveTemp(InConverter));
}

UWorldPartitionRuntimeHash* UWorldPartitionRuntimeHash::ConvertWorldPartitionHash(const UWorldPartitionRuntimeHash* InSrcHash, const UClass* InDstClass)
{
	check(InSrcHash);
	check(InDstClass);
	check(InDstClass->IsChildOf<UWorldPartitionRuntimeHash>());
	check(!InDstClass->HasAnyClassFlags(CLASS_Abstract));

	// Look for a registered converter
	const UClass* CurrentSrcClass = InSrcHash->GetClass();
	while(CurrentSrcClass != UWorldPartitionRuntimeHash::StaticClass())
	{
		if (FRuntimeHashConvertFunc* Converter = WorldPartitionRuntimeHashConverters.Find({ CurrentSrcClass, InDstClass }))
		{
			if (UWorldPartitionRuntimeHash* NewHash = (*Converter)(InSrcHash))
			{
				UE_LOGF(LogWorldPartition, Log, "Converted '%ls' runtime hash class from '%ls' to '%ls'.", *InSrcHash->GetPackage()->GetName(), *InSrcHash->GetClass()->GetName(), *InDstClass->GetName());
				return NewHash;
			}
			else
			{
				UE_LOGF(LogWorldPartition, Warning, "Failed to convert '%ls' runtime hash class from '%ls' to '%ls'.", *InSrcHash->GetPackage()->GetName(), *InSrcHash->GetClass()->GetName(), *InDstClass->GetName());
			}
		}
		CurrentSrcClass = CurrentSrcClass->GetSuperClass();
	}

	// No converter found, create a new hash of the target type with default values
	UE_LOGF(LogWorldPartition, Log, "No converter found to convert '%ls' runtime hash class from '%ls' to '%ls', creating new with default values.", *InSrcHash->GetPackage()->GetName(), *InSrcHash->GetClass()->GetName(), *InDstClass->GetName());
	UWorldPartitionRuntimeHash* NewHash = NewObject<UWorldPartitionRuntimeHash>(InSrcHash->GetOuter(), InDstClass, NAME_None, RF_Transactional);
	NewHash->SetDefaultValues();
	return NewHash;
}

void UWorldPartitionRuntimeHash::ExecutePreSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams)
{
	// Iterate over all hash types and call PreSetupHLODActors() on each of them
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UWorldPartitionRuntimeHash::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract))
		{
			CastChecked<UWorldPartitionRuntimeHash>(ClassIterator->GetDefaultObject())->PreSetupHLODActors(InWorldPartition, InParams);
		}
	}
}

void UWorldPartitionRuntimeHash::ExecutePostSetupHLODActors(const UWorldPartition* InWorldPartition, const UWorldPartition::FSetupHLODActorsParams& InParams)
{
	// Iterate over all hash types and call PostSetupHLODActors() on each of them
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		if (ClassIterator->IsChildOf(UWorldPartitionRuntimeHash::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract))
		{
			CastChecked<UWorldPartitionRuntimeHash>(ClassIterator->GetDefaultObject())->PostSetupHLODActors(InWorldPartition, InParams);
		}
	}
}

#endif

void UWorldPartitionRuntimeHash::FStreamingSourceCells::AddCell(const UWorldPartitionRuntimeCell* Cell, const FWorldPartitionStreamingSource& Source, const FSphericalSector& SourceShape, const FWorldPartitionStreamingContext& Context)
{
	Cell->AppendStreamingSourceInfo(Source, SourceShape, Context);
	Cells.Add(Cell);
}

void FWorldPartitionQueryCache::AddCellInfo(const UWorldPartitionRuntimeCell* Cell, const FSphericalSector& SourceShape)
{
	const double SquareDistance = FVector::DistSquared2D(SourceShape.GetCenter(), Cell->GetContentBounds().GetCenter());
	if (double* ExistingSquareDistance = CellToSourceMinSqrDistances.Find(Cell))
	{
		*ExistingSquareDistance = FMath::Min(*ExistingSquareDistance, SquareDistance);
	}
	else
	{
		CellToSourceMinSqrDistances.Add(Cell, SquareDistance);
	}
}

double FWorldPartitionQueryCache::GetCellMinSquareDist(const UWorldPartitionRuntimeCell* Cell) const
{
	const double* Dist = CellToSourceMinSqrDistances.Find(Cell);
	return Dist ? *Dist : MAX_dbl;
}

FWorldPartitionStreamingContext FWorldPartitionStreamingContext::Create(const UWorld* InWorld)
{
	if (InWorld && InWorld->GetWorldPartition() && InWorld->GetWorldDataLayers())
	{
		check(!InWorld->IsGameWorld() || IsInGameThread());
		return FWorldPartitionStreamingContext(InWorld);
	}
	return FWorldPartitionStreamingContext();
}

FWorldPartitionStreamingContext::FWorldPartitionStreamingContext()
	: bIsValid(false)
	, DataLayersLogicOperator(EWorldPartitionDataLayersLogicOperator::Or)
	, DataLayerEffectiveStates(nullptr)
	, UpdateStreamingStateEpoch(0)
{}

FWorldPartitionStreamingContext::FWorldPartitionStreamingContext(const UWorld* InWorld)
	: FWorldPartitionStreamingContext(InWorld->GetWorldPartition()->GetDataLayersLogicOperator(), FWorldDataLayersEffectiveStatesAccessor::Get(InWorld->GetWorldDataLayers()), InWorld->GetWorldPartition()->GetUpdateStreamingStateEpoch()
#if WITH_EDITOR
		, [InWorld]()
		{
			if (const UWorldPartitionSubsystem* Subsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(InWorld))
			{
				return Subsystem->GetDynamicLoadingRangeScales();
			}
			return FDynamicLoadingRangeScales{};
		}()
#endif // WITH_EDITOR
	)
{
}

FWorldPartitionStreamingContext::FWorldPartitionStreamingContext(EWorldPartitionDataLayersLogicOperator InDataLayersLogicOperator, const FWorldDataLayersEffectiveStates& InDataLayerEffectiveStates, int32 InUpdateStreamingStateEpoch
#if WITH_EDITOR
	, const FDynamicLoadingRangeScales& InDynamicLoadingRangeScales
#endif // WITH_EDITOR
)
	: bIsValid(true)
	, DataLayersLogicOperator(InDataLayersLogicOperator)
	, DataLayerEffectiveStates(&InDataLayerEffectiveStates)
	, UpdateStreamingStateEpoch(InUpdateStreamingStateEpoch)
#if WITH_EDITOR
	, DynamicLoadingRangeScales(InDynamicLoadingRangeScales)
#endif // WITH_EDITOR
{}

EDataLayerRuntimeState FWorldPartitionStreamingContext::GetDataLayerEffectiveRuntimeState(FName DataLayerName) const
{
	check(IsValid());
	check(DataLayerEffectiveStates);
	check(!DataLayerName.IsNone());
	return !DataLayerName.IsNone() ? DataLayerEffectiveStates->GetDataLayerEffectiveRuntimeStateByName(DataLayerName) : EDataLayerRuntimeState::Unloaded;
}

EDataLayerRuntimeState FWorldPartitionStreamingContext::ResolveDataLayerRuntimeState(const FDataLayerInstanceNames& InDataLayers) const
{
	if (InDataLayers.IsEmpty())
	{
		return EDataLayerRuntimeState::Activated;
	}

	check(IsValid());
	check(DataLayerEffectiveStates);
	EDataLayerRuntimeState Result = EDataLayerRuntimeState::Unloaded;

	// Determine the maximum runtime state the cell can have based on its External Data Layer. If none, maximum is Activated.
	FName ExternalDatalayerName = InDataLayers.GetExternalDataLayer();
	EDataLayerRuntimeState MaxEffectiveRuntimeState = !ExternalDatalayerName.IsNone() ? DataLayerEffectiveStates->GetDataLayerEffectiveRuntimeStateByName(ExternalDatalayerName) : EDataLayerRuntimeState::Activated;

	if (MaxEffectiveRuntimeState > EDataLayerRuntimeState::Unloaded)
	{
		TArrayView<const FName> NonExternalDataLayers = InDataLayers.GetNonExternalDataLayers();
		if (NonExternalDataLayers.IsEmpty())
		{
			Result = MaxEffectiveRuntimeState;
		}
		else
		{
			switch (DataLayersLogicOperator)
			{
			case EWorldPartitionDataLayersLogicOperator::Or:
				if (UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState(NonExternalDataLayers, EDataLayerRuntimeState::Activated, *DataLayerEffectiveStates))
				{
					Result = MaxEffectiveRuntimeState;
				}
				else if (UDataLayerManager::IsAnyDataLayerInEffectiveRuntimeState(NonExternalDataLayers, EDataLayerRuntimeState::Loaded, *DataLayerEffectiveStates))
				{
					Result = EDataLayerRuntimeState::Loaded;
				}
				break;
			case EWorldPartitionDataLayersLogicOperator::And:
				if (UDataLayerManager::IsAllDataLayerInEffectiveRuntimeState(NonExternalDataLayers, EDataLayerRuntimeState::Activated, *DataLayerEffectiveStates))
				{
					Result = MaxEffectiveRuntimeState;
				}
				else if (UDataLayerManager::IsAllDataLayerInEffectiveRuntimeState(NonExternalDataLayers, EDataLayerRuntimeState::Loaded, *DataLayerEffectiveStates))
				{
					Result = EDataLayerRuntimeState::Loaded;
				}
				break;
			default:
				checkNoEntry();
			}
		}
	}
	return Result;
}

#if WITH_EDITOR
FWorldPartitionPackageHash URuntimeHashExternalStreamingObjectBase::GetGenerationHash() const
{
	// Dependencies URuntimeHashExternalStreamingObjectBase are correctly handled the standard discovery through UObjects mechanisms
	return FWorldPartitionPackageHash();
}
#endif

#undef LOCTEXT_NAMESPACE
