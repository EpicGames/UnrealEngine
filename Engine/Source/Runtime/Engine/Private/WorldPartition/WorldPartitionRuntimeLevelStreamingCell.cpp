// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/WorldPartitionLevelStreamingPolicy.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/WorldPartitionUtils.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Level.h"
#include "Misc/HierarchicalLogArchive.h"
#include "Misc/Paths.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/Package.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Cooker/CookEvents.h"

#if WITH_EDITOR
#include "WorldPartition/Cook/WorldPartitionCookPackageContextInterface.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeLevelStreamingCell)

UWorldPartitionRuntimeLevelStreamingCell::UWorldPartitionRuntimeLevelStreamingCell(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LevelStreaming(nullptr)
{}

EWorldPartitionRuntimeCellState UWorldPartitionRuntimeLevelStreamingCell::GetCurrentState() const
{
	if (LevelStreaming)
	{
		ELevelStreamingState CurrentStreamingState = LevelStreaming->GetLevelStreamingState();
		if (CurrentStreamingState == ELevelStreamingState::LoadedVisible)
		{
			return EWorldPartitionRuntimeCellState::Activated;
		}
		else if (CurrentStreamingState >= ELevelStreamingState::LoadedNotVisible)
		{
			return EWorldPartitionRuntimeCellState::Loaded;
		}
	}
	
	//@todo_ow: Now that actors are moved to the persistent level, remove the AlwaysLoaded cell (it's always empty)
	return IsAlwaysLoaded() ? EWorldPartitionRuntimeCellState::Activated : EWorldPartitionRuntimeCellState::Unloaded;
}

UWorldPartitionLevelStreamingDynamic* UWorldPartitionRuntimeLevelStreamingCell::GetLevelStreaming() const
{
	return LevelStreaming;
}

bool UWorldPartitionRuntimeLevelStreamingCell::HasActors() const
{
#if WITH_EDITOR
	return GetActorCount() > 0;
#else
	return true;
#endif
}

#if WITH_EDITOR
FName UWorldPartitionRuntimeLevelStreamingCell::GetCookPackageName() const
{
	if (UWorld* World = GetOwningWorld())
	{
		const FString CookPackageName = FString::Printf(TEXT("%s/%s/%s"), *UWorld::RemovePIEPrefix(World->GetPackage()->GetName()), FPackageName::GetGeneratedPackageSubPath(), *GetName());
		return FName(CookPackageName);
	}
	return NAME_None;
}

TSet<FName> UWorldPartitionRuntimeLevelStreamingCell::GetActorPackageNames() const
{
	TSet<FName> Actors;
	Actors.Reserve(Packages.Num());
	for (const FWorldPartitionRuntimeCellObjectMapping& Package : Packages)
	{
		Actors.Add(Package.Package);
	}
	return Actors;
}
#endif

FName UWorldPartitionRuntimeLevelStreamingCell::GetLevelPackageName() const
{
#if WITH_EDITOR
	UWorld* World = GetOwningWorld();
	if (World && World->IsPlayInEditor())
	{
		return FName(UWorldPartitionLevelStreamingPolicy::GetCellPackagePath(GetFName(), World));
	}
#endif
	if (LevelStreaming)
	{
		return LevelStreaming->GetWorldAssetPackageFName();
	}
	return Super::GetLevelPackageName();
}

TArray<FName> UWorldPartitionRuntimeLevelStreamingCell::GetActors() const
{
	TArray<FName> Actors;

#if WITH_EDITOR
	Actors.Reserve(Packages.Num());

	for (const FWorldPartitionRuntimeCellObjectMapping& Package : Packages)
	{
		Actors.Add(*FPaths::GetExtension(Package.Path.ToString()));
	}
#endif

	return Actors;
}

void UWorldPartitionRuntimeLevelStreamingCell::CreateAndSetLevelStreaming(const FString& InPackageName, const FSoftObjectPath& InWorldAsset)
{
	LevelStreaming = CreateLevelStreaming(InPackageName, InWorldAsset);
}

bool UWorldPartitionRuntimeLevelStreamingCell::CreateAndSetLevelStreaming(const TSoftObjectPtr<UWorld>& InWorldAsset, const FTransform& InInstanceTransform) const
{
	UWorld* OwningWorld = GetOwningWorld();
	const FName LevelStreamingName = FName(*FString::Printf(TEXT("WorldPartitionLevelStreaming_%s"), *GetName()));
	if (FindObject<UWorldPartitionLevelStreamingDynamic>(OwningWorld, *LevelStreamingName.ToString()))
	{
		UE_LOGF(LogWorldPartition, Warning, "UWorldPartitionRuntimeLevelStreamingCell::CreateAndSetLevelStreaming can't create an already existing UWorldPartitionLevelStreamingDynamic object named %ls", *LevelStreamingName.ToString());
		return false;
	}
	LevelStreaming = NewObject<UWorldPartitionLevelStreamingDynamic>(OwningWorld, UWorldPartitionLevelStreamingDynamic::StaticClass(), LevelStreamingName, RF_NoFlags, NULL);

	// Generate unique Level Instance name assuming cell has a unique name
	const FString LongPackageName = InWorldAsset.GetLongPackageName();
	const FString PackagePath = FPackageName::GetLongPackagePath(LongPackageName);
	const FString ShortPackageName = FPackageName::GetShortName(LongPackageName);
	TStringBuilder<512> LevelPackageNameStrBuilder;
	LevelPackageNameStrBuilder.Append(PackagePath);
	LevelPackageNameStrBuilder.Append(TEXT("/"));
	LevelPackageNameStrBuilder.Append(ShortPackageName);
	LevelPackageNameStrBuilder.Append(TEXT("_LevelInstance_"));
	LevelPackageNameStrBuilder.Append(GetName());
	LevelPackageNameStrBuilder.Append(TEXT("."));
	LevelPackageNameStrBuilder.Append(ShortPackageName);
	LevelStreaming->SetWorldAsset(TSoftObjectPtr<UWorld>(FSoftObjectPath(LevelPackageNameStrBuilder.ToString())));

	// Include WorldPartition's transform to Level
	LevelStreaming->SetLevelTransform(InInstanceTransform * GetOuterWorld()->GetWorldPartition()->GetInstanceTransform());
	LevelStreaming->bClientOnlyVisible = GetClientOnlyVisible();
	LevelStreaming->Initialize(*this);
	LevelStreaming->PackageNameToLoad = FName(LongPackageName);

#if WITH_EDITOR
	LevelStreaming->SetShouldPerformStandardLevelLoading(true);

	if (OwningWorld->IsPlayInEditor() && OwningWorld->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) && OwningWorld->GetPackage()->GetPIEInstanceID() != INDEX_NONE)
	{
		// When renaming for PIE, make sure to keep World's name so that linker can properly remap with Package's instancing context
		LevelStreaming->RenameForPIE(OwningWorld->GetPackage()->GetPIEInstanceID(), /*bKeepWorldAssetName*/true);
	}
#endif

	return true;
}

UWorldPartitionLevelStreamingDynamic* UWorldPartitionRuntimeLevelStreamingCell::CreateLevelStreaming(const FString& InPackageName, const FSoftObjectPath& InWorldAsset) const
{
	auto GetUniqueLevelStreamingName = [this]()
	{
		const UWorld* OuterWorld = GetOuterWorld();
		TStringBuilder<128> LevelStreamingNameBuilder;
		LevelStreamingNameBuilder.Appendf(TEXT("WorldPartitionLevelStreaming_%s"), *GetName());
		if (OuterWorld->IsGameWorld())
		{
			FString OuterWorldPackageShortName = FPackageName::GetShortName(OuterWorld->GetPackage());
#if WITH_EDITOR
			OuterWorldPackageShortName = UWorld::RemovePIEPrefix(OuterWorldPackageShortName);
#endif
			// Include outer world package name to make sure we generate a unique name since we it is 
			LevelStreamingNameBuilder.Appendf(TEXT("_%s"), *OuterWorldPackageShortName);
		}
		return FName(*LevelStreamingNameBuilder);
	};

	if (HasActors())
	{
		UWorld* OuterWorld = GetOuterWorld();
		UWorld* OwningWorld = GetOwningWorld();
		
		// When called by Commandlet (PopulateGeneratedPackageForCook), LevelStreaming's outer is set to Cell/WorldPartition's outer to prevent warnings when saving Cell Levels (Warning: Obj in another map). 
		// At runtime, LevelStreaming's outer will be properly set to the main world (see UWorldPartitionRuntimeLevelStreamingCell::Activate).
		UWorld* LevelStreamingOuterWorld = IsRunningCommandlet() ? OuterWorld : OwningWorld;
		const FName LevelStreamingName = GetUniqueLevelStreamingName();
		UWorldPartitionLevelStreamingDynamic* NewLevelStreaming = NewObject<UWorldPartitionLevelStreamingDynamic>(LevelStreamingOuterWorld, UWorldPartitionLevelStreamingDynamic::StaticClass(), LevelStreamingName, RF_NoFlags, NULL);

		FName WorldName = OuterWorld->GetFName();
#if WITH_EDITOR
		// In PIE make sure that we are using the proper original world name so that actors resolve their outer property
		if (OwningWorld->IsPlayInEditor() && !OuterWorld->OriginalWorldName.IsNone())
		{
			WorldName = OuterWorld->OriginalWorldName;
		}

		FString PackageName = !InPackageName.IsEmpty() ? InPackageName : UWorldPartitionLevelStreamingPolicy::GetCellPackagePath(GetFName(), OuterWorld);
#else
		check(!InPackageName.IsEmpty());
		FString PackageName = InPackageName;
#endif

		// Set both PackageNameToLoad and WorldAsset (necessary to properly support instancing)
		NewLevelStreaming->PackageNameToLoad = *PackageName;
		if (InWorldAsset.IsValid())
		{
			NewLevelStreaming->SetWorldAsset(TSoftObjectPtr<UWorld>(InWorldAsset));
		}
		else
		{
			TSoftObjectPtr<UWorld> WorldAsset(FSoftObjectPath(FString::Printf(TEXT("%s.%s"), *PackageName, *WorldName.ToString())));
			NewLevelStreaming->SetWorldAsset(WorldAsset);
		}

		// Transfer WorldPartition's transform to Level
		const UWorldPartition* OuterWorldPartition = OuterWorld->GetWorldPartition();
		NewLevelStreaming->LevelTransform = OuterWorldPartition->GetInstanceTransform();
		NewLevelStreaming->bClientOnlyVisible = GetClientOnlyVisible();
		NewLevelStreaming->Initialize(*this);

#if WITH_EDITOR
		if (OwningWorld->IsPlayInEditor() && OwningWorld->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) && OwningWorld->GetPackage()->GetPIEInstanceID() != INDEX_NONE)
		{
			// When renaming for PIE, make sure to keep World's name so that linker can properly remap with Package's instancing context
			NewLevelStreaming->RenameForPIE(OwningWorld->GetPackage()->GetPIEInstanceID(), /*bKeepWorldAssetName*/true);
		}
#endif

		return NewLevelStreaming;
	}

	return nullptr;
}

EStreamingStatus UWorldPartitionRuntimeLevelStreamingCell::GetStreamingStatus() const
{
	if (LevelStreaming)
	{
		return LevelStreaming->GetLevelStreamingStatus();
	}
	return Super::GetStreamingStatus();
}

bool UWorldPartitionRuntimeLevelStreamingCell::IsVisible() const
{
	if (LevelStreaming)
	{
		// Use GetLevelStreamingState() which is protected by CurrentStateLock,
		// unlike GetLevelStreamingStatus() which reads LoadedLevel without synchronization.
		return LevelStreaming->GetLevelStreamingState() == ELevelStreamingState::LoadedVisible;
	}
	return false;
}

FLinearColor UWorldPartitionRuntimeLevelStreamingCell::GetDebugColor(EWorldPartitionRuntimeCellVisualizeMode VisualizeMode) const
{
#if !UE_BUILD_SHIPPING
	switch (VisualizeMode)
	{
		case EWorldPartitionRuntimeCellVisualizeMode::StreamingPriority:
		{
			if (DebugStreamingPriority >= 0.0f && DebugStreamingPriority <= 1.0f)
			{
				const float PriorityGradient = FMath::Cube(1.0f - DebugStreamingPriority);

				if (FWorldPartitionDebugHelper::GetRuntimeSpatialHashCellStreamingPriorityMode() == 2)
				{
					// Grayscale
					return FLinearColor(PriorityGradient, PriorityGradient, PriorityGradient, 1.0f);
				}

				// Heatmap
				static TArray<FLinearColor, TInlineAllocator<4>> Colors { FLinearColor::Blue, FLinearColor::Green, FLinearColor::Yellow, FLinearColor::Red };
				const float ColorGrad = FMath::Clamp(PriorityGradient, 0.0f, 1.0f) * (Colors.Num() - 1);
				return FLinearColor::LerpUsingHSV(
					Colors[FMath::Min<int32>(ColorGrad, Colors.Num() - 1)],
					Colors[FMath::Min<int32>(ColorGrad + 1, Colors.Num() - 1)],
					FMath::Frac(ColorGrad)
				);
			}
			return FLinearColor::Transparent;
		}
		case EWorldPartitionRuntimeCellVisualizeMode::StreamingStatus:
		{
			// Return streaming status color
			return LevelStreaming ? ULevelStreaming::GetLevelStreamingStatusColor(LevelStreaming->GetLevelStreamingStatus()) : FLinearColor::Black;
		}
	}
#endif

	return Super::GetDebugColor(VisualizeMode);
}

void UWorldPartitionRuntimeLevelStreamingCell::SetIsAlwaysLoaded(bool bInIsAlwaysLoaded)
{
	Super::SetIsAlwaysLoaded(bInIsAlwaysLoaded);
	if (LevelStreaming)
	{
		LevelStreaming->SetShouldBeAlwaysLoaded(true);
	}
}

#if WITH_EDITOR
void UWorldPartitionRuntimeLevelStreamingCell::AddActorToCell(const FStreamingGenerationActorDescView& ActorDescView)
{
	// Non-spatially loaded actors coming from level instances are not moved into the persistent level in PIE but rather placed into an always loaded cell,
	// because that would imply loading them through an instancing context.
	checkf(!ActorDescView.GetActorIsEditorOnly() || ActorDescView.GetActorIsEditorOnlyLoadedInPIE(), TEXT("Invalid editor-only actor descriptor:\n\t%s"), *ActorDescView.ToString(FWorldPartitionActorDesc::EToStringMode::ForDiff));

	const UActorDescContainerInstance* ContainerInstance = ActorDescView.GetContainerInstance();
	check(ContainerInstance);

	const FActorContainerID& ContainerID = ContainerInstance->GetContainerID();
	const FTransform ContainerTransform = ContainerInstance->GetTransform();
	const FName ContainerPackage = ContainerInstance->GetContainerPackage();

	// Add all parent LevelInstance actors to the dependency list for this cell (for incremental cooks)
	const UActorDescContainerInstance* CurrentContainerInstance = ContainerInstance;
	const UActorDescContainerInstance* ParentContainerInstance = ContainerInstance->GetParentContainerInstance();
	while (ParentContainerInstance)
	{
		FGuid ContainerActorGuid = CurrentContainerInstance->GetContainerActorGuid();
		if (FWorldPartitionActorDescInstance* ContainerActorDescInstance = ParentContainerInstance->GetActorDescInstance(ContainerActorGuid))
		{
			FName ContainerActorPackage = ContainerActorDescInstance->GetActorPackage();
			ActorContainerPackageDependencies.Add(ContainerActorPackage);
		}
		
		CurrentContainerInstance = ParentContainerInstance;
		ParentContainerInstance = CurrentContainerInstance->GetParentContainerInstance();
	}

	if (!IsRunningCookCommandlet())
	{
		for (const FGuid& EditorReferenceGuid : ActorDescView.GetEditorReferences())
		{
			FName ReferencePackage;
			FName ReferencePath;
			FTopLevelAssetPath ReferenceBaseClass;
			FTopLevelAssetPath ReferenceNativeClass;

			// Special case where the actor descriptor view has an invalid references, use invalid reference information as the actor guid isn't necessarily in the container instance.
			if (const FStreamingGenerationActorDescView::FInvalidReference* InvalidReference = ActorDescView.GetInvalidReference(EditorReferenceGuid))
			{
				ReferencePackage = InvalidReference->ActorPackage;
				ReferencePath = *InvalidReference->ActorSoftPath.ToString();
				ReferenceBaseClass = InvalidReference->BaseClass;
				ReferenceNativeClass = InvalidReference->NativeClass;
			}
			else
			{
				const FWorldPartitionActorDescInstance& ReferenceActorDesc = ContainerInstance->GetActorDescInstanceChecked(EditorReferenceGuid);
				ReferencePackage = ReferenceActorDesc.GetActorPackage();
				ReferencePath = *ReferenceActorDesc.GetActorSoftPath().ToString();
				ReferenceBaseClass = ReferenceActorDesc.GetBaseClass();
				ReferenceNativeClass = ReferenceActorDesc.GetNativeClass();
			}

			Packages.Emplace(
				ReferencePackage,
				ReferencePath,
				ReferenceBaseClass,
				ReferenceNativeClass,
				ContainerID,
				ContainerTransform,
				FTransform::Identity,
				ContainerPackage,
				GetWorld()->GetPackage()->GetFName(),
				ContainerID.GetActorGuid(EditorReferenceGuid),
				true
			);
		}
	}

	FWorldPartitionRuntimeCellObjectMapping ActorMapping(
		ActorDescView.GetActorPackage(), 
		*ActorDescView.GetActorSoftPath().ToString(), 
		ActorDescView.GetBaseClass(),
		ActorDescView.GetNativeClass(),
		ContainerID,
		ContainerTransform,
		ActorDescView.GetEditorOnlyParentTransform(),
		ContainerPackage, 
		GetWorld()->GetPackage()->GetFName(), 
		ContainerID.GetActorGuid(ActorDescView.GetGuid()),
		false
	);

	TArray<FWorldPartitionRuntimeCellPropertyOverride> PropertyOverrides;
	ContainerInstance->GetPropertyOverridesForActor(ContainerID, ActorDescView.GetGuid(), PropertyOverrides);
	if (PropertyOverrides.Num())
	{
		ActorMapping.PropertyOverrides = MoveTemp(PropertyOverrides);
	}

	Packages.Add(MoveTemp(ActorMapping));
}

void UWorldPartitionRuntimeLevelStreamingCell::Fixup()
{
	TMap<FGuid, FWorldPartitionRuntimeCellObjectMapping> UniquePackages;
	UniquePackages.Reserve(Packages.Num());

	for (FWorldPartitionRuntimeCellObjectMapping& Package : Packages)
	{
		FWorldPartitionRuntimeCellObjectMapping& UniquePackage = UniquePackages.FindOrAdd(Package.ActorInstanceGuid, Package);
		UniquePackage.bIsEditorOnly &= Package.bIsEditorOnly;
	};

	if (UniquePackages.Num() != Packages.Num())
	{
		UniquePackages.GenerateValueArray(Packages);
	}
}

FWorldPartitionPackageHash UWorldPartitionRuntimeLevelStreamingCell::GetGenerationHash() const
{
	FWorldPartitionPackageHashBuilder Builder;

	for (const FWorldPartitionRuntimeCellObjectMapping& ActorMapping : Packages)
	{
		ActorMapping.UpdateHash(Builder);
	}

	return Builder.Finalize();
}

FString UWorldPartitionRuntimeLevelStreamingCell::GetPackageNameToCreate() const
{
	return UWorldPartitionLevelStreamingPolicy::GetCellPackagePath(GetFName(), GetOuterWorld());
}

bool UWorldPartitionRuntimeLevelStreamingCell::OnPrepareGeneratorPackageForCook(TArray<UPackage*>& OutModifiedPackages)
{
	check(IsAlwaysLoaded());

	if (GetActorCount() > 0)
	{
		UWorld* OuterWorld = GetOuterWorld();
		UWorldPartition* WorldPartition = OuterWorld->GetWorldPartition();

		FWorldPartitionLevelHelper::FPackageReferencer PackageReferencer;
		TArray<int32> AsyncRequestIDs;
		FWorldPartitionLevelHelper::FLoadActorsParams Params = FWorldPartitionLevelHelper::FLoadActorsParams()
			.SetOuterWorld(OuterWorld)
			.SetDestLevel(nullptr)
			.SetActorPackages(Packages)
			.SetPackageReferencer(&PackageReferencer)
			.SetCompletionCallback([](bool) {})
			.SetLoadAsync(true, &AsyncRequestIDs)
			.SetInstancingContext(FLinkerInstancingContext(false)); // Don't do SoftObjectPath remapping for PersistentLevel actors because references can end up in different cells

		FWorldPartitionLevelHelper::LoadActors(MoveTemp(Params));

		// Finish all async loading
		// Note that FlushAsyncLoading will invoke completion callbacks, which may themselves trigger new async loading 
		// requests (FWorldPartitionLevelHelper::LoadActorsWithPropertyOverridesInternal is a good example).
		// Loop until no new requests are found in AsyncRequestIDs.
		while (!AsyncRequestIDs.IsEmpty())
		{
			TArray<int32> ToFlush = MoveTemp(AsyncRequestIDs);
			FlushAsyncLoading(ToFlush);
		}

		FWorldPartitionLevelHelper::MoveExternalActorsToLevel(Packages, OuterWorld->PersistentLevel, OutModifiedPackages);

		// Remap needed here for references to Actors that are inside a Container
		FWorldPartitionLevelHelper::RemapLevelSoftObjectPaths(OuterWorld->PersistentLevel, WorldPartition);

		// Make sure Asset Registry tags are updated here synchronously now that Package contains all its actors
		// ex: AFunctionalTest actors need to be part of the Worlds asset tags once they are not longer external so they can be discovered at runtime
		IAssetRegistry::Get()->AssetUpdateTags(OuterWorld, EAssetRegistryTagsCaller::Fast);

		// Preserve actor package list for dependencies, we'll need them later during OnCookEvent
		for (const FName& ActorName : GetActorPackageNames())
		{
			ActorContainerPackageDependencies.Add(ActorName);
		}

		// Empty cell's package list (this ensures that no one can rely on cell's content).
		Packages.Empty();
	}

	return true;
}

// Do all necessary work to prepare cell object for cook.
bool UWorldPartitionRuntimeLevelStreamingCell::PrepareCellForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage)
{
	// LevelStreaming could already be created
	if (!LevelStreaming && GetActorCount() > 0)
	{
		FString PackageName = InCookContext.GetGeneratedPackagePath(this);
		check(!InGeneratedPackage || (PackageName == InGeneratedPackage->GetName()));
		if (PackageName.IsEmpty())
		{
			return false;
		}
		// Validation
		check(PackageName.Contains(GetPackageNameToCreate()));
		LevelStreaming = CreateLevelStreaming(PackageName);
	}
	return true;
}

bool UWorldPartitionRuntimeLevelStreamingCell::OnPopulateGeneratorPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InGeneratedPackage)
{
	return PrepareCellForCook(InCookContext, InGeneratedPackage);
}

// Helper used by UWorldPartitionRuntimeLevelStreamingCell::OnPopulateGeneratedPackageForCook
class FScopedCookingExternalStreamingObject
{
private:
	FScopedCookingExternalStreamingObject(const URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject)
		: ExternalStreamingObject(const_cast<URuntimeHashExternalStreamingObjectBase*>(InExternalStreamingObject))
	{
		check(IsRunningCookCommandlet());
		if (ExternalStreamingObject)
		{
			UWorld* World = ExternalStreamingObject->GetOuterWorld();
			check(World);
			UWorldPartition* WorldPartition = World->GetWorldPartition();
			check(WorldPartition);
			check(WorldPartition->StreamingPolicy);
			verify(WorldPartition->StreamingPolicy->InjectExternalStreamingObject(const_cast<URuntimeHashExternalStreamingObjectBase*>(ExternalStreamingObject)));
		}
	}

	~FScopedCookingExternalStreamingObject()
	{
		if (ExternalStreamingObject)
		{
			UWorld* World = ExternalStreamingObject->GetOuterWorld();
			check(World);
			UWorldPartition* WorldPartition = World->GetWorldPartition();
			check(WorldPartition);
			check(WorldPartition->StreamingPolicy);
			verify(WorldPartition->StreamingPolicy->RemoveExternalStreamingObject(ExternalStreamingObject));
		}
	}

	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject;
	friend class UWorldPartitionRuntimeLevelStreamingCell;
};

void UWorldPartitionRuntimeLevelStreamingCell::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	// These dependencies will be added on the main world (WorldPartition outer world) package
	Super::OnCookEvent(CookEvent, CookContext);
	if (CookEvent == UE::Cook::ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		for (const FName& ActorName : GetActorPackageNames())
		{
			CookContext.AddSaveBuildDependency(UE::Cook::FCookDependency::Package(ActorName));
		}

		for (const FName& DependencyName : ActorContainerPackageDependencies)
		{
			CookContext.AddSaveBuildDependency(UE::Cook::FCookDependency::Package(DependencyName));
		}
	}
}

bool UWorldPartitionRuntimeLevelStreamingCell::OnPopulateGeneratedPackageForCook(const IWorldPartitionCookPackageContext& InCookContext, UPackage* InPackage, TArray<UPackage*>& OutModifiedPackages)
{
	check(!IsAlwaysLoaded());
	if (!InPackage)
	{
		return false;
	}

	if (GetActorCount() > 0)
	{
		// When cook splitter doesn't use deferred populate, cell needs to be prepared here.
		if (!PrepareCellForCook(InCookContext, InPackage))
		{
			return false;
		}

		// These dependencies will be added on the generated packages (the cells Level used in runtime)
		for (const FName& ActorName : GetActorPackageNames())
		{
			InCookContext.ReportSaveDependency(UE::Cook::FCookDependency::Package(ActorName));
		}

		for (const FName& DependencyName : ActorContainerPackageDependencies)
		{
			InCookContext.ReportSaveDependency(UE::Cook::FCookDependency::Package(DependencyName));
		}

		UWorld* OuterWorld = GetOuterWorld();

		// Until we also hash WorldPartition settings we add the world package (it contains all settings) to our dependencies
		InCookContext.ReportSaveDependency(UE::Cook::FCookDependency::Package(OuterWorld->GetPackage()->GetFName()));

		UWorldPartition* WorldPartition = OuterWorld->GetWorldPartition();

		// Load cell Actors
		FWorldPartitionLevelHelper::FPackageReferencer PackageReferencer;
		TArray<int32> AsyncRequestIDs;
		FWorldPartitionLevelHelper::FLoadActorsParams Params = FWorldPartitionLevelHelper::FLoadActorsParams()
			.SetOuterWorld(OuterWorld)
			.SetDestLevel(nullptr)
			.SetActorPackages(Packages)
			.SetPackageReferencer(&PackageReferencer)
			.SetCompletionCallback([](bool) {})
			.SetLoadAsync(true, &AsyncRequestIDs)
			.SetInstancingContext(FLinkerInstancingContext(false)); // Don't do SoftObjectPath remapping for PersistentLevel actors because references can end up in different cells

		FWorldPartitionLevelHelper::LoadActors(MoveTemp(Params));

		// Finish all async loading
		// Note that FlushAsyncLoading will invoke completion callbacks, which may themselves trigger new async loading 
		// requests (FWorldPartitionLevelHelper::LoadActorsWithPropertyOverridesInternal is a good example).
		// Loop until no new requests are found in AsyncRequestIDs.
		while (!AsyncRequestIDs.IsEmpty())
		{
			TArray<int32> ToFlush = MoveTemp(AsyncRequestIDs);
			FlushAsyncLoading(ToFlush);
		}

		// Create a level and move these actors in it
		ULevel* NewLevel = FWorldPartitionLevelHelper::CreateEmptyLevelForRuntimeCell(this, OuterWorld, LevelStreaming->GetWorldAsset().ToString(), InPackage);
		check(NewLevel->GetPackage() == InPackage);
		FWorldPartitionLevelHelper::MoveExternalActorsToLevel(Packages, NewLevel, OutModifiedPackages);

		WorldPartition->ApplyRuntimeCellsTransformerStack(NewLevel);

		// Push temporarily the cooking ExternalStreamingObject in the policy for RemapLevelSoftObjectPaths to use it to resolve softobjectpaths
		// Do this only if the ExternalStreamingObject has a valid root external data layer asset, as Content Bundle soft object remapping is not supported at cook time (there is no world package remapping)
		const URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = GetTypedOuter<URuntimeHashExternalStreamingObjectBase>();
		const URuntimeHashExternalStreamingObjectBase* CookingExternalStreamingObject = ExternalStreamingObject && ExternalStreamingObject->GetRootExternalDataLayerAsset() ? ExternalStreamingObject : nullptr;
		FScopedCookingExternalStreamingObject ScopeCookingExternalStreamingObject(CookingExternalStreamingObject);
		
		// Remap Level's SoftObjectPaths
		FWorldPartitionLevelHelper::RemapLevelSoftObjectPaths(NewLevel, WorldPartition);
	}
	return true;
}

int32 UWorldPartitionRuntimeLevelStreamingCell::GetActorCount() const
{
	return Packages.Num();
}

void UWorldPartitionRuntimeLevelStreamingCell::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Super::DumpStateLog(Ar);

	TArray<FWorldPartitionRuntimeCellObjectMapping> SortedPackages(Packages);

	SortedPackages.Sort([](const FWorldPartitionRuntimeCellObjectMapping& A, const FWorldPartitionRuntimeCellObjectMapping& B) { return A.ActorInstanceGuid < B.ActorInstanceGuid; });

	for (const FWorldPartitionRuntimeCellObjectMapping& Mapping : SortedPackages)
	{
		FHierarchicalLogArchive::FIndentScope ActorIndentScope = Ar.PrintfIndent(TEXT("%s"), *Mapping.Path.ToString());
		Ar.Printf(TEXT("        Package: %s"), *Mapping.Package.ToString());
		Ar.Printf(TEXT("    Editor Only: %d"), Mapping.bIsEditorOnly ? 1 : 0);
		Ar.Printf(TEXT("  Instance Guid: %s"), *Mapping.ActorInstanceGuid.ToString());

		FHierarchicalLogArchive::FIndentScope ContainerIndentScope = Ar.PrintfIndent(TEXT("Container:"));
		Ar.Printf(TEXT("       ID: %s"), *Mapping.ContainerID.ToString());
		Ar.Printf(TEXT("Transform: %s"), *Mapping.ContainerTransform.ToString());		
	}
}
#endif

UWorldPartitionLevelStreamingDynamic* UWorldPartitionRuntimeLevelStreamingCell::GetOrCreateLevelStreaming() const
{
#if WITH_EDITOR
	if (!LevelStreaming && GetActorCount())
	{
		LevelStreaming = CreateLevelStreaming();
		check(LevelStreaming);
	}
#else
	// In Runtime, always loaded cell level is handled by World directly
	check(LevelStreaming || IsAlwaysLoaded() || GetIsCustomHLODPlaceholderCell());
#endif

#if !WITH_EDITOR
	// In Runtime, prepare LevelStreaming for activation
	if (LevelStreaming)
	{
		const UWorldPartition* WorldPartition = GetOuterWorld()->GetWorldPartition();
		
		// Setup pre-created LevelStreaming's outer to the WorldPartition owning world.
		// This is needed because ULevelStreaming is within=World, and ULevelStreaming::GetWorld() assumes that the outer world is the main world.		
		UWorld* OwningWorld = GetOwningWorld();
		if (LevelStreaming->GetWorld() != OwningWorld)
		{
			LevelStreaming->Rename(nullptr, OwningWorld);
		}
		
		// Transfer WorldPartition's transform to LevelStreaming
		LevelStreaming->SetLevelTransform(WorldPartition->GetInstanceTransform());

		// Make sure we have a unique world assetif the world is instanced. Normally the world asset is remapped through the FLinkerInstancingContext soft object 
		// remapping, but loading an instance of a partitioned world through the LoadLevelInstance blueprint node will not.
		if (WorldPartition->GetTypedOuter<ULevel>()->IsInstancedLevel() && (LevelStreaming->PackageNameToLoad == LevelStreaming->GetWorldAssetPackageName()))
		{
			const TSoftObjectPtr<UWorld> NewWorldAsset(FSoftObjectPath(FString::Printf(TEXT("%s_LevelInstance_%08x"), *LevelStreaming->GetWorldAssetPackageFName().GetPlainNameString(), GetTypeHash(GetPackage()->GetName()))));
			LevelStreaming->SetWorldAsset(NewWorldAsset);
		}
	}
#endif

	if (LevelStreaming)
	{
		LevelStreaming->OnLevelShown.AddUniqueDynamic(const_cast<UWorldPartitionRuntimeLevelStreamingCell*>(this), &UWorldPartitionRuntimeLevelStreamingCell::OnLevelShown);
		LevelStreaming->OnLevelHidden.AddUniqueDynamic(const_cast<UWorldPartitionRuntimeLevelStreamingCell*>(this), &UWorldPartitionRuntimeLevelStreamingCell::OnLevelHidden);
	}

	return LevelStreaming;
}

void UWorldPartitionRuntimeLevelStreamingCell::Load() const
{
	if (UWorldPartitionLevelStreamingDynamic* LocalLevelStreaming = GetOrCreateLevelStreaming())
	{
		LocalLevelStreaming->Load();
	}
}

void UWorldPartitionRuntimeLevelStreamingCell::Activate() const
{
	if (UWorldPartitionLevelStreamingDynamic* LocalLevelStreaming = GetOrCreateLevelStreaming())
	{
		LocalLevelStreaming->Activate();
	}
	else if (GetIsCustomHLODPlaceholderCell())
	{
		OnCellShown();
	}
}

void UWorldPartitionRuntimeLevelStreamingCell::SetStreamingPriority(int32 InStreamingPriority) const
{
	if (LevelStreaming)
	{
		LevelStreaming->SetPriorityOverride(InStreamingPriority);
	}
}

ULevel* UWorldPartitionRuntimeLevelStreamingCell::GetLevel() const 
{
	return LevelStreaming ? LevelStreaming->GetLoadedLevel() : nullptr;
}

bool UWorldPartitionRuntimeLevelStreamingCell::CanUnload() const
{
	return true;
}

void UWorldPartitionRuntimeLevelStreamingCell::Unload() const
{
#if WITH_EDITOR
	check(LevelStreaming || GetActorCount() || GetIsCustomHLODPlaceholderCell());
#else
	// In Runtime, always loaded cell level is handled by World directly
	check(LevelStreaming || IsAlwaysLoaded() || GetIsCustomHLODPlaceholderCell());
#endif

	if (LevelStreaming)
	{
		LevelStreaming->Unload();
	}
	else if (GetIsCustomHLODPlaceholderCell())
	{
		OnCellHidden();
	}
}

void UWorldPartitionRuntimeLevelStreamingCell::Deactivate() const
{
#if WITH_EDITOR
	check(LevelStreaming || GetActorCount() || GetIsCustomHLODPlaceholderCell());
#else
	// In Runtime, always loaded cell level is handled by World directly
	check(LevelStreaming || IsAlwaysLoaded() || GetIsCustomHLODPlaceholderCell());
#endif

	if (LevelStreaming)
	{
		LevelStreaming->Deactivate();
	}
	else if (GetIsCustomHLODPlaceholderCell())
	{
		OnCellHidden();
	}
}

void UWorldPartitionRuntimeLevelStreamingCell::OnLevelShown()
{
	OnCellShown();
}

void UWorldPartitionRuntimeLevelStreamingCell::OnCellShown() const
{
	// Test if the outer world is valid to handle the rare case where a streaming level outlives its world
	// * Since those three objects are independant, they can possibly have different lifetime.
	// * The OnCellShown() call will be skipped if the level streaming is alive but its cell is not, as the delegate IsBound() test will reject it
	// * A crash would occurs if both the level streaming object and the runtime cell are alive, but the world is not
	if (UWorld* OuterWorld = GetOuterWorld())
	{
		UWorldPartition* OuterWorldPartition = OuterWorld->GetWorldPartition();
		if (OuterWorldPartition && OuterWorldPartition->IsInitialized())
		{
			OuterWorldPartition->OnCellShown(this);
		}
	}
}

void UWorldPartitionRuntimeLevelStreamingCell::OnLevelHidden()
{
	OnCellHidden();
}

void UWorldPartitionRuntimeLevelStreamingCell::OnCellHidden() const
{
	// Test if the outer world is valid to handle the rare case where a streaming level outlives its world
	// * Since those three objects are independant, they can possibly have different lifetime.
	// * The OnCellShown() call will be skipped if the level streaming is alive but its cell is not, as the delegate IsBound() test will reject it
	// * A crash would occurs if both the level streaming object and the runtime cell are alive, but the world is not
	if (UWorld* OuterWorld = GetOuterWorld())
	{
		UWorldPartition* OuterWorldPartition = OuterWorld->GetWorldPartition();
		if (OuterWorldPartition && OuterWorldPartition->IsInitialized())
		{
			OuterWorldPartition->OnCellHidden(this);
		}
	}
}
