// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldPartitionRuntimeSpatialHash.cpp: UWorldPartitionRuntimeSpatialHash implementation
=============================================================================*/
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "WorldPartition/WorldPartitionRuntimeCellDataSpatialHash.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHashGridPreviewer.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionDraw2DContext.h"
#include "WorldPartition/RuntimeSpatialHash/RuntimeSpatialHashGridHelper.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionLHGrid.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleBase.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "Components/BillboardComponent.h"
#include "Components/LineBatchComponent.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/Canvas.h"
#include "DrawDebugHelpers.h"
#include "GlobalRenderResources.h"
#include "Math/TranslationMatrix.h"
#include "Math/UnrealMathUtility.h"
#include "Math/TransformCalculus2D.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/HierarchicalLogArchive.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Misc/HashBuilder.h"
#include "Algo/AllOf.h"
#include "UObject/UObjectGlobalsInternal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeSpatialHash)

#if WITH_EDITOR
extern UNREALED_API class UEditorEngine* GEditor;
#else
#include "Misc/Paths.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "WorldPartition"

static int32 GShowRuntimeSpatialHashGridLevel = 0;
static FAutoConsoleVariableRef CVarShowRuntimeSpatialHashGridLevel(
	TEXT("wp.Runtime.ShowRuntimeSpatialHashGridLevel"),
	GShowRuntimeSpatialHashGridLevel,
	TEXT("Used to choose which grid level to display when showing world partition runtime hash."));

static int32 GShowRuntimeSpatialHashGridLevelCount = 1;
static FAutoConsoleVariableRef CVarShowRuntimeSpatialHashGridLevelCount(
	TEXT("wp.Runtime.ShowRuntimeSpatialHashGridLevelCount"),
	GShowRuntimeSpatialHashGridLevelCount,
	TEXT("Used to choose how many grid levels to display when showing world partition runtime hash."));

#if !UE_BUILD_SHIPPING
static int32 GFilterRuntimeSpatialHashGridLevel = INDEX_NONE;
static FAutoConsoleVariableRef CVarFilterRuntimeSpatialHashGridLevel(
	TEXT("wp.Runtime.FilterRuntimeSpatialHashGridLevel"),
	GFilterRuntimeSpatialHashGridLevel,
	TEXT("Used to choose filter a single world partition runtime hash grid level."));
#endif

static int32 GForceRuntimeSpatialHashZCulling = -1;
static FAutoConsoleVariableRef CVarForceRuntimeSpatialHashZCulling(
	TEXT("wp.Runtime.ForceRuntimeSpatialHashZCulling"),
	GForceRuntimeSpatialHashZCulling,
	TEXT("Used to force the behavior of the runtime hash cells Z culling. Set to 0 to force off, to 1 to force on and any other value to respect the runtime hash setting."));

static bool GetEffectiveEnableZCulling(bool bEnableZCulling)
{
	switch (GForceRuntimeSpatialHashZCulling)
	{
	case 0: return false;
	case 1: return true;
	}

	return bEnableZCulling;
}

// ------------------------------------------------------------------------------------------------
FSpatialHashStreamingGrid::FSpatialHashStreamingGrid()
	: Origin(ForceInitToZero)
	, CellSize(0)
	, LoadingRange(0.0f)
	, bBlockOnSlowStreaming(false)
	, DebugColor(ForceInitToZero)
	, WorldBounds(ForceInitToZero)
	, bClientOnlyVisible(false)
	, GridIndex(INDEX_NONE)
	, GridHelper(nullptr)
{
}

FSpatialHashStreamingGrid::~FSpatialHashStreamingGrid()
{
	if (GridHelper)
	{
		delete GridHelper;
	}
}

const FSquare2DGridHelper& FSpatialHashStreamingGrid::GetGridHelper() const
{
	if (!GridHelper)
	{
		GridHelper = new FSquare2DGridHelper(WorldBounds, Origin, CellSize, Settings.bUseAlignedGridLevels);
	}

	check(GridHelper->Levels.Num() == GridLevels.Num());
	check(GridHelper->Origin == Origin);
	check(GridHelper->CellSize == CellSize);
	check(GridHelper->WorldBounds == WorldBounds);

	return *GridHelper;
}

static const FString GOverrideRuntimeSpatialHashLoadingRangeName("wp.Runtime.OverrideRuntimeSpatialHashLoadingRange");
bool FSpatialHashStreamingGrid::bAddedWorldPartitionSubsystemDeinitializedCallback = false;
TMap<int32, float> FSpatialHashStreamingGrid::OverriddenLoadingRanges;
FAutoConsoleCommand FSpatialHashStreamingGrid::OverrideLoadingRangeCommand(
	*GOverrideRuntimeSpatialHashLoadingRangeName,
	TEXT("Sets runtime loading range. Args -grid=[index] -range=[override_loading_range]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		FString ArgString = FString::Join(InArgs, TEXT(" "));
		int32 OverrideGridIndex = 0;
		float OverrideLoadingRange = -1.f;
		FParse::Value(*ArgString, TEXT("grid="), OverrideGridIndex);
		FParse::Value(*ArgString, TEXT("range="), OverrideLoadingRange);

		if (!bAddedWorldPartitionSubsystemDeinitializedCallback)
		{
			bAddedWorldPartitionSubsystemDeinitializedCallback = true;
			UWorldPartitionSubsystem::OnWorldPartitionSubsystemDeinitialized.AddLambda([](UWorldPartitionSubsystem* InWorldPartitionSubsystem, UWorld* InWorld)
			{
				if (InWorld && InWorld->IsGameWorld())
				{
					FSpatialHashStreamingGrid::OverriddenLoadingRanges.Reset();
				}
			});
		}

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				FWorldPartitionHelpers::ServerExecConsoleCommand(World, GOverrideRuntimeSpatialHashLoadingRangeName, InArgs);
				if (UWorld::HasSubsystem<UWorldPartitionSubsystem>(World))
				{
					if (OverrideLoadingRange >= 0.f)
					{
						FSpatialHashStreamingGrid::OverriddenLoadingRanges.Add(OverrideGridIndex, OverrideLoadingRange);
					}
					else
					{
						FSpatialHashStreamingGrid::OverriddenLoadingRanges.Remove(OverrideGridIndex);
					}
					break;
				}
			}
		}
	})
);

float FSpatialHashStreamingGrid::GetLoadingRange() const
{
#if !UE_BUILD_SHIPPING
	int32 OverriddenLoadingRange;
	if (UWorldPartitionSubsystem::GetOverrideLoadingRange(GridName, OverriddenLoadingRange))
	{
		return OverriddenLoadingRange;
	}
#endif

	if (float* OverrideLoadingRange = FSpatialHashStreamingGrid::OverriddenLoadingRanges.Find(GridIndex))
	{
		check(*OverrideLoadingRange >= 0.f);
		return *OverrideLoadingRange;
	}

	return LoadingRange;
}

int64 FSpatialHashStreamingGrid::GetCellSize(int32 Level) const
{
	return GetGridHelper().Levels[Level].CellSize;
}

#if WITH_EDITOR
void FSpatialHashStreamingGrid::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	FHierarchicalLogArchive::FIndentScope GridIndentScope = Ar.PrintfIndent(TEXT("%s"), *GridName.ToString());
	Ar.Printf(TEXT("            Origin: %s"), *Origin.ToString());
	Ar.Printf(TEXT("         Cell Size: %d"), CellSize);
	Ar.Printf(TEXT("      World Bounds: %s"), *WorldBounds.ToString());
	Ar.Printf(TEXT("     Loading Range: %3.2f"), LoadingRange);
	Ar.Printf(TEXT("Block Slow Loading: %s"), bBlockOnSlowStreaming ? TEXT("Yes") : TEXT("No"));
	Ar.Printf(TEXT(" ClientOnlyVisible: %s"), bClientOnlyVisible ? TEXT("Yes") : TEXT("No"));
	Ar.Printf(TEXT(""));

	struct FGridLevelStats
	{
		int32 CellCount;
		int64 CellSize;
		int32 ActorCount;
	};

	TArray<FGridLevelStats> LevelsStats;
	int32 TotalActorCount = 0;

	{
		int32 Level = 0;
		for (const FSpatialHashStreamingGridLevel& GridLevel : GridLevels)
		{
			int32 LevelCellCount = 0;
			int32 LevelActorCount = 0;
			for (const FSpatialHashStreamingGridLayerCell& LayerCell : GridLevel.LayerCells)
			{
				LevelCellCount += LayerCell.GridCells.Num();
				for (const UWorldPartitionRuntimeCell* Cell : LayerCell.GridCells)
				{
					LevelActorCount += Cell->GetActorCount();
				}
			}
			LevelsStats.Add({ LevelCellCount, ((int64)CellSize << (int64)Level), LevelActorCount });
			TotalActorCount += LevelActorCount;
			++Level;
		}
		TotalActorCount = (TotalActorCount > 0) ? TotalActorCount : 1;
	}

	{
		FHierarchicalLogArchive::FIndentScope LevelIndentScope = Ar.PrintfIndent(TEXT("Grid Levels: %d"), GridLevels.Num());
		for (int Level = 0; Level < LevelsStats.Num(); ++Level)
		{
			Ar.Printf(TEXT("Level %2d: Cell Count %4d | Cell Size %7lld | Actor Count %4d (%3.1f%%)"), Level, LevelsStats[Level].CellCount, LevelsStats[Level].CellSize, LevelsStats[Level].ActorCount, (100.f * LevelsStats[Level].ActorCount) / TotalActorCount);
		}
	}

	{
		Ar.Printf(TEXT(""));
		int32 Level = 0;
		for (const FSpatialHashStreamingGridLevel& GridLevel : GridLevels)
		{
			FHierarchicalLogArchive::FIndentScope LevelIndentScope = Ar.PrintfIndent(TEXT("Content of Grid Level %d"), Level++);

			TArray<UWorldPartitionRuntimeCell*> Cells;
			for (const FSpatialHashStreamingGridLayerCell& LayerCell : GridLevel.LayerCells)
			{
				for (const TObjectPtr<UWorldPartitionRuntimeCell>& Cell : LayerCell.GridCells)
				{
					if (!Cell->IsAlwaysLoaded() || !IsRunningCookCommandlet())
					{
						Cells.Add(Cell);
					}
				}
			}

			Cells.Sort([this](const UWorldPartitionRuntimeCell& A, const UWorldPartitionRuntimeCell& B)
			{
				if (A.IsAlwaysLoaded() ==  B.IsAlwaysLoaded())
				{
					return A.GetFName().LexicalLess(B.GetFName());
				}

				return A.IsAlwaysLoaded() > B.IsAlwaysLoaded();
			});

			for (UWorldPartitionRuntimeCell* Cell : Cells)
			{
				FHierarchicalLogArchive::FIndentScope CellIndentScope = Ar.PrintfIndent(TEXT("Content of Cell %s (%s)"), *Cell->GetDebugName(), *Cell->GetName());
				Cell->DumpStateLog(Ar);
			}
		}
	}
	Ar.Printf(TEXT(""));
}
#endif

void FSpatialHashStreamingGrid::GetCells(const FWorldPartitionStreamingQuerySource& QuerySource, TSet<const UWorldPartitionRuntimeCell*>& OutCells, bool bEnableZCulling, FWorldPartitionQueryCache* QueryCache, bool bQueryNonSpatialCells) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSpatialHashStreamingGrid::GetCells_QuerySource);

	auto ShouldAddCell = [](const UWorldPartitionRuntimeCell* Cell, const FWorldPartitionStreamingQuerySource& QuerySource)
	{
		if (Cell->HasDataLayers())
		{
			if (QuerySource.bIncludeAnyDataLayer)
			{
				return true;
			}

			if (!QuerySource.DataLayers.IsEmpty() &&
				Cell->GetDataLayers().FindByPredicate([&](const FName& DataLayerName) { return QuerySource.DataLayers.Contains(DataLayerName); }))
			{
				return true;
			}
		}
		else if (!QuerySource.bDataLayersOnly)
		{
			return true;
		}

		return false;
	};

	const FSquare2DGridHelper& Helper = GetGridHelper();

	// Spatial Query
	if (QuerySource.bSpatialQuery)
	{
		QuerySource.ForEachShape(GetLoadingRange(), GridName, /*bProjectIn2D*/ true, [&](const FSphericalSector& Shape)
		{
			Helper.ForEachIntersectingCells(Shape, [&](const FGridCellCoord& Coords)
			{
				ForEachRuntimeCell(Coords, [&](const UWorldPartitionRuntimeCell* Cell)
				{
					bool bIncludeCell = true;

					if (bEnableZCulling)
					{
						const FVector2D CellMinMaxZ(Cell->GetContentBounds().Min.Z, Cell->GetContentBounds().Max.Z);
						bIncludeCell = TRange<double>::Inclusive(CellMinMaxZ.X, CellMinMaxZ.Y).Overlaps(TRange<double>::Inclusive(Shape.GetCenter().Z - Shape.GetRadius(), Shape.GetCenter().Z + Shape.GetRadius()));
					}
						
					if (bIncludeCell)
					{
						if (ShouldAddCell(Cell, QuerySource))
						{
							OutCells.Add(Cell);
							if (QueryCache)
							{
								QueryCache->AddCellInfo(Cell, Shape);
							}
						}
					}
				});
			});
		});
	}

	// Non Spatial (LayersCells is the array containing non-spatial cells)
	if (bQueryNonSpatialCells)
	{
		auto ForEachGridLevelCells = [&QuerySource, &OutCells, ShouldAddCell](const FSpatialHashStreamingGridLevel& InGridLevel)
		{
			for (const FSpatialHashStreamingGridLayerCell& LayerCell : InGridLevel.LayerCells)
			{
				for (const UWorldPartitionRuntimeCell* Cell : LayerCell.GridCells)
				{
					if (ShouldAddCell(Cell, QuerySource))
					{
						OutCells.Add(Cell);
					}
				}
			}
		};

		const int32 TopLevel = GridLevels.Num() - 1;
		ForEachGridLevelCells(GridLevels[TopLevel]);
		if (InjectedGridLevels.IsValidIndex(TopLevel))
		{
			ForEachGridLevelCells(InjectedGridLevels[TopLevel]);
		}
	}
}

void FSpatialHashStreamingGrid::GetCells(const TArray<FWorldPartitionStreamingSource>& Sources, UWorldPartitionRuntimeHash::FStreamingSourceCells& OutActivateCells, UWorldPartitionRuntimeHash::FStreamingSourceCells& OutLoadCells, bool bEnableZCulling, const FWorldPartitionStreamingContext& Context, float LoadingRangeScale) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSpatialHashStreamingGrid::GetCells);

	struct FStreamingSourceInfo
	{
		FStreamingSourceInfo(const FWorldPartitionStreamingSource& InSource, const FSphericalSector& InSourceShape)
			: Source(InSource)
			, SourceShape(InSourceShape)
		{}
		const FWorldPartitionStreamingSource& Source;
		const FSphericalSector SourceShape;
	};

	typedef TMap<FGridCellCoord, TArray<FStreamingSourceInfo>> FIntersectingCells;
	FIntersectingCells AllActivatedCells;

	const float GridLoadingRange = GetLoadingRange();
	const FSquare2DGridHelper& Helper = GetGridHelper();

	FSphericalSector AdjustedShape{EForceInit::ForceInit};

	for (const FWorldPartitionStreamingSource& Source : Sources)
	{
		Source.ForEachShape(GridLoadingRange, GridName, /*bProjectIn2D*/ true, [&](const FSphericalSector& Shape)
		{
			// Adjust radius for dynamic scales
			const FSphericalSector* EffectiveShape = &Shape;
			if (LoadingRangeScale < 1.f)
			{
				AdjustedShape = Shape;
				AdjustedShape.SetRadius(AdjustedShape.GetRadius() * LoadingRangeScale);
				EffectiveShape = &AdjustedShape;
			}

			FStreamingSourceInfo Info(Source, *EffectiveShape);

			Helper.ForEachIntersectingCells(*EffectiveShape, [&](const FGridCellCoord& Coords)
			{
				bool bAddedActivatedCell = false;

#if !UE_BUILD_SHIPPING
				if ((GFilterRuntimeSpatialHashGridLevel == INDEX_NONE) || (GFilterRuntimeSpatialHashGridLevel == Coords.Z))
#endif
				{
					ForEachRuntimeCell(Coords, [&](const UWorldPartitionRuntimeCell* Cell)
					{
						bool bIncludeCell = true;

						if (bEnableZCulling)
						{
							const FVector2D CellMinMaxZ(Cell->GetContentBounds().Min.Z, Cell->GetContentBounds().Max.Z);
							bIncludeCell = TRange<double>::Inclusive(CellMinMaxZ.X, CellMinMaxZ.Y).Overlaps(TRange<double>::Inclusive(EffectiveShape->GetCenter().Z - EffectiveShape->GetRadius(), EffectiveShape->GetCenter().Z + EffectiveShape->GetRadius()));
						}
						
						if (bIncludeCell)
						{
							switch (Cell->GetCellEffectiveWantedState(Context))
							{
							case EDataLayerRuntimeState::Loaded:
								OutLoadCells.AddCell(Cell, Source, *EffectiveShape, Context);
								break;
							case EDataLayerRuntimeState::Activated:
								switch (Source.TargetState)
								{
								case EStreamingSourceTargetState::Loaded:
									OutLoadCells.AddCell(Cell, Source, *EffectiveShape, Context);
									break;
								case EStreamingSourceTargetState::Activated:
									OutActivateCells.AddCell(Cell, Source, *EffectiveShape, Context);
									bAddedActivatedCell = !Settings.bUseAlignedGridLevels && Settings.bSnapNonAlignedGridLevelsToLowerLevels;
									break;
								default:
									checkNoEntry();
								}
								break;
							case EDataLayerRuntimeState::Unloaded:
								break;
							default:
								checkNoEntry();
							}
						}
					});
				}
				if (bAddedActivatedCell)
				{
					AllActivatedCells.FindOrAdd(Coords).Add(Info);
				}
			});
		});
	}

	GetNonSpatiallyLoadedCells(OutActivateCells.GetCells(), OutLoadCells.GetCells(), Context);

	if (!Settings.bUseAlignedGridLevels && Settings.bSnapNonAlignedGridLevelsToLowerLevels)
	{
		auto FindIntersectingParents = [&Helper, this](const FIntersectingCells& InAllCells, const FIntersectingCells& InTestCells, FIntersectingCells& OutIntersectingCells)
		{
			bool bFound = false;
			const int32 AlwaysLoadedLevel = Helper.Levels.Num() - 1;
			for (const auto& InTestCell : InTestCells)
			{
				const FGridCellCoord& TestCell = InTestCell.Key;
				int32 CurrentLevelIndex = TestCell.Z;
				int32 ParentLevelIndex = CurrentLevelIndex + 1;
				// Only test with Parent Level if it's below the AlwaysLoaded Level
				if (ParentLevelIndex < AlwaysLoadedLevel)
				{
					FBox2D CurrentLevelCellBounds;
					Helper.Levels[CurrentLevelIndex].GetCellBounds(FGridCellCoord2(TestCell.X, TestCell.Y), CurrentLevelCellBounds);
					FBox Box(FVector(CurrentLevelCellBounds.Min, 0), FVector(CurrentLevelCellBounds.Max, 0));

					Helper.ForEachIntersectingCells(Box, [&](const FGridCellCoord& IntersectingCoords)
					{
						check(IntersectingCoords.Z >= ParentLevelIndex);
						if (!InAllCells.Contains(IntersectingCoords))
						{
							if (!OutIntersectingCells.Contains(IntersectingCoords))
							{
								OutIntersectingCells.Add(IntersectingCoords, InTestCell.Value);
								bFound = true;
							}
						}
					}, ParentLevelIndex);
				}
			}
			return bFound;
		};
	
		FIntersectingCells AllParentCells;
		FIntersectingCells TestCells = AllActivatedCells;
		FIntersectingCells IntersectingCells;
		bool bFound = false;
		do
		{
			bFound = FindIntersectingParents(AllActivatedCells, TestCells, IntersectingCells);
			if (bFound)
			{
				AllActivatedCells.Append(IntersectingCells);
				AllParentCells.Append(IntersectingCells);
				TestCells = MoveTemp(IntersectingCells);
				check(IntersectingCells.IsEmpty());
			}
		} while (bFound);

		for (const auto& ParentCell : AllParentCells)
		{
			ForEachRuntimeCell(ParentCell.Key, [&](const UWorldPartitionRuntimeCell* Cell)
			{
				switch (Cell->GetCellEffectiveWantedState(Context))
				{
				case EDataLayerRuntimeState::Loaded:
					break;
				case EDataLayerRuntimeState::Activated:
					for (const auto& Info : ParentCell.Value)
					{
						OutActivateCells.AddCell(Cell, Info.Source, Info.SourceShape, Context);
					}
					break;
				case EDataLayerRuntimeState::Unloaded:
					break;
				default:
					checkNoEntry();
				}
			});
		}
	}
}

bool FSpatialHashStreamingGrid::InsertGridCell(UWorldPartitionRuntimeCell* InCell, const FGridCellCoord& InGridCellCoords)
{
	check(InCell);

	const int64 Level = InGridCellCoords.Z;
	const int64 CellCoordX = InGridCellCoords.X;
	const int64 CellCoordY = InGridCellCoords.Y;

	// Add Cell to grid
	if (GridLevels.IsValidIndex(Level))
	{
		uint64 CellIndex = 0;
		if (GetGridHelper().Levels[Level].GetCellIndex(FGridCellCoord2(CellCoordX, CellCoordY), CellIndex))
		{
			FSpatialHashStreamingGridLevel& GridLevel = GridLevels[Level];
			int32 LayerCellIndex;
			int32* LayerCellIndexPtr = GridLevel.LayerCellsMapping.Find(CellIndex);
			if (LayerCellIndexPtr)
			{
				LayerCellIndex = *LayerCellIndexPtr;
			}
			else
			{
				LayerCellIndex = GridLevel.LayerCells.AddDefaulted();
				GridLevel.LayerCellsMapping.Add(CellIndex, LayerCellIndex);
			}
			GridLevel.LayerCells[LayerCellIndex].GridCells.Add(InCell);
			return true;
		}
	}

	return false;
}

void FSpatialHashStreamingGrid::InjectExternalStreamingObjectGrid(const FSpatialHashStreamingGrid& InExternalObjectStreamingGrid) const
{
	if (InjectedGridLevels.IsEmpty())
	{
		InjectedGridLevels.SetNumZeroed(GridLevels.Num());
	}

	check(GetGridHelper().Levels.Num() == InjectedGridLevels.Num());

	for (int32 SourceGridLevel = 0; SourceGridLevel < InExternalObjectStreamingGrid.GridLevels.Num(); ++SourceGridLevel)
	{
		const FSpatialHashStreamingGridLevel& ExternalObjectGridLevel = InExternalObjectStreamingGrid.GridLevels[SourceGridLevel];
		const int32 TargetGridTopLevel = GridLevels.Num() - 1;
		const int32 SourceGridTopLevel = InExternalObjectStreamingGrid.GridLevels.Num() - 1;
		const bool bIsSourceTopLevel = (SourceGridLevel == SourceGridTopLevel);
		// Make sure to use target top level for source top level to make sure that the content won't be spatially loaded
		int32 TargetGridLevel = bIsSourceTopLevel ? TargetGridTopLevel : FMath::Min<int32>(SourceGridLevel, TargetGridTopLevel);
		if (ensure(InjectedGridLevels.IsValidIndex(TargetGridLevel)))
		{
			check(bIsSourceTopLevel || InExternalObjectStreamingGrid.GetGridHelper().Levels[TargetGridLevel].CellSize == GetGridHelper().Levels[TargetGridLevel].CellSize);

			for (const FSpatialHashStreamingGridLayerCell& ExternalObjectLayerCell : ExternalObjectGridLevel.LayerCells)
			{
				if (ensure(!ExternalObjectLayerCell.GridCells.IsEmpty()))
				{
					// Any cell position should be good to find the proper cell coord
					const FVector& CellPosition = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(ExternalObjectLayerCell.GridCells[0]->RuntimeCellData)->Position;
					FGridCellCoord2 CellCoords;
					const FSquare2DGridHelper::FGridLevel& InjectedGridHelperLevel = GetGridHelper().Levels[TargetGridLevel];
					if (!InjectedGridHelperLevel.GetCellCoords(FVector2D(CellPosition.X, CellPosition.Y), CellCoords))
					{
						// Because we support ExternalObjectStreamingGrid covering more than its source grid, we must handle this case.
						// Clamp cell position on level grid borders
						CellCoords.X = FMath::Clamp(CellCoords.X, 0, InjectedGridHelperLevel.GridSize);
						CellCoords.Y = FMath::Clamp(CellCoords.Y, 0, InjectedGridHelperLevel.GridSize);
					}
					const int64 CoordKey = CellCoords.Y * InjectedGridHelperLevel.GridSize + CellCoords.X;

					FSpatialHashStreamingGridLevel& TargetInjectedGridLevel = InjectedGridLevels[TargetGridLevel];
					int32 LayerCellIndex;
					int32* LayerCellIndexPtr = TargetInjectedGridLevel.LayerCellsMapping.Find(CoordKey);
					if (LayerCellIndexPtr)
					{
						LayerCellIndex = *LayerCellIndexPtr;
					}
					else
					{
						LayerCellIndex = TargetInjectedGridLevel.LayerCells.AddDefaulted();
						TargetInjectedGridLevel.LayerCellsMapping.Add(CoordKey, LayerCellIndex);
					}

					for (UWorldPartitionRuntimeCell* StreamingCell : ExternalObjectLayerCell.GridCells)
					{
						TargetInjectedGridLevel.LayerCells[LayerCellIndex].GridCells.Add(StreamingCell);
					}
				}
			}
		}
	}
}

void FSpatialHashStreamingGrid::RemoveExternalStreamingObjectGrid(const FSpatialHashStreamingGrid& InExternalObjectStreamingGrid) const
{
	for (int SourceGridLevel = 0; SourceGridLevel < InExternalObjectStreamingGrid.GridLevels.Num(); ++SourceGridLevel)
	{
		const FSpatialHashStreamingGridLevel& ExternalObjectGridLevel = InExternalObjectStreamingGrid.GridLevels[SourceGridLevel];
		const int32 TargetGridTopLevel = GridLevels.Num() - 1;
		const int32 SourceGridTopLevel = InExternalObjectStreamingGrid.GridLevels.Num() - 1;
		const bool bIsSourceTopLevel = (SourceGridLevel == SourceGridTopLevel);
		// Make sure to use target top level for source top level to make sure that the content won't be spatially loaded
		int32 TargetGridLevel = bIsSourceTopLevel ? TargetGridTopLevel : FMath::Min<int32>(SourceGridLevel, TargetGridTopLevel);
		if (ensure(InjectedGridLevels.IsValidIndex(TargetGridLevel)))
		{
			TArray<FSpatialHashStreamingGridLayerCell>& LayerCells = InjectedGridLevels[TargetGridLevel].LayerCells;
			TMap<int64, int32>& LayerCellsMapping = InjectedGridLevels[TargetGridLevel].LayerCellsMapping;
			// Prepare a reverse lookup LayerCellIndex -> CellKey
			TMap<int32, int64> InverseLayerCellsMapping;
			for (const auto& Pair : LayerCellsMapping)
			{
				InverseLayerCellsMapping.Add(Pair.Value, Pair.Key);
			}

			for (const FSpatialHashStreamingGridLayerCell& ExternalObjectLayerCell : ExternalObjectGridLevel.LayerCells)
			{
				for (UWorldPartitionRuntimeCell* StreamingCell : ExternalObjectLayerCell.GridCells)
				{
					bool bFound = false;
					for (int LayerCellIndex = 0; LayerCellIndex < LayerCells.Num(); ++ LayerCellIndex)
					{
						FSpatialHashStreamingGridLayerCell& LayerCell = LayerCells[LayerCellIndex];
						if (LayerCell.GridCells.RemoveSingleSwap(StreamingCell, EAllowShrinking::No))
						{
							bFound = true;
							// Cleanup LayerCell if empty
							if (LayerCell.GridCells.IsEmpty())
							{
								int32 RemovedIndex = LayerCellIndex;
								LayerCells.RemoveAtSwap(RemovedIndex, EAllowShrinking::No);
								int64 RemovedCellKey = InverseLayerCellsMapping[RemovedIndex];
								LayerCellsMapping.Remove(RemovedCellKey);
								InverseLayerCellsMapping.Remove(RemovedIndex);

								if (LayerCells.IsValidIndex(RemovedIndex))
								{
									int32 SwapedLayerCellIndex = LayerCells.Num();
									int64 SwapedCellKey = InverseLayerCellsMapping[SwapedLayerCellIndex];
									LayerCellsMapping.FindChecked(SwapedCellKey) = RemovedIndex;
									InverseLayerCellsMapping.Remove(SwapedLayerCellIndex);
									InverseLayerCellsMapping.Add(RemovedIndex, SwapedCellKey);
								}
							}
							break;
						}
					}
					LayerCells.Shrink();
					check(bFound);
				}
			}
		}
	}
}

void FSpatialHashStreamingGrid::ForEachLayerCell(const FGridCellCoord& Coords, TFunctionRef<void(const FSpatialHashStreamingGridLayerCell*)> Func) const
{
	check(GridLevels.IsValidIndex(Coords.Z));

	const int64 CoordKey = Coords.Y * GetGridHelper().Levels[Coords.Z].GridSize + Coords.X;

	if (const int32* LayerCellIndexPtr = GridLevels[Coords.Z].LayerCellsMapping.Find(CoordKey))
	{
		Func(&GridLevels[Coords.Z].LayerCells[*LayerCellIndexPtr]);
	}

	// Test in the injected grid levels (if any)
	if (InjectedGridLevels.IsValidIndex(Coords.Z))
	{
		if (const int32* LayerCellIndexPtr = InjectedGridLevels[Coords.Z].LayerCellsMapping.Find(CoordKey))
		{
			Func(&InjectedGridLevels[Coords.Z].LayerCells[*LayerCellIndexPtr]);
		}
	}
}

void FSpatialHashStreamingGrid::ForEachRuntimeCell(const FGridCellCoord& Coords, TFunctionRef<void(const UWorldPartitionRuntimeCell*)> Func) const
{
	ForEachLayerCell(Coords, [&](const FSpatialHashStreamingGridLayerCell* LayerCell)
	{
		for (const UWorldPartitionRuntimeCell* Cell : LayerCell->GridCells)
		{
			Func(Cell);
		}
	});
}

void FSpatialHashStreamingGrid::ForEachRuntimeCell(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const
{
	auto ForEachGridLevel = [Func](const TArray<FSpatialHashStreamingGridLevel>& InGridLevels)
	{
		for (const FSpatialHashStreamingGridLevel& GridLevel : InGridLevels)
		{
			for (const FSpatialHashStreamingGridLayerCell& LayerCell : GridLevel.LayerCells)
			{
				for (const UWorldPartitionRuntimeCell* Cell : LayerCell.GridCells)
				{
					if (!Func(Cell))
					{
						return;
					}
				}
			}
		}
	};

	ForEachGridLevel(GridLevels);
	ForEachGridLevel(InjectedGridLevels);
}

void FSpatialHashStreamingGrid::GetNonSpatiallyLoadedCells(TSet<const UWorldPartitionRuntimeCell*>& OutActivateCells, TSet<const UWorldPartitionRuntimeCell*>& OutLoadCells, const FWorldPartitionStreamingContext& Context) const
{
	if (GridLevels.Num() > 0)
	{
		auto ForEachGridLevelCells = [&OutActivateCells, &OutLoadCells, &Context](const FSpatialHashStreamingGridLevel& InGridLevel)
		{
			for (const FSpatialHashStreamingGridLayerCell& LayerCell : InGridLevel.LayerCells)
			{
				for (const UWorldPartitionRuntimeCell* Cell : LayerCell.GridCells)
				{
					switch (Cell->GetCellEffectiveWantedState(Context))
					{
					case EDataLayerRuntimeState::Loaded:
						check(Cell->HasDataLayers());
						OutLoadCells.Add(Cell);
						break;
					case EDataLayerRuntimeState::Activated:
						check(Cell->IsAlwaysLoaded() || Cell->HasDataLayers() || Cell->GetContentBundleID().IsValid());
						OutActivateCells.Add(Cell);
						break;
					case EDataLayerRuntimeState::Unloaded:
						break;
					default:
						checkNoEntry();
					}
				}
			}
		};

		const int32 TopLevel = GridLevels.Num() - 1;
		ForEachGridLevelCells(GridLevels[TopLevel]);
		if (InjectedGridLevels.IsValidIndex(TopLevel))
		{
			ForEachGridLevelCells(InjectedGridLevels[TopLevel]);
		}
	}
}

void FSpatialHashStreamingGrid::GetFilteredCellsForDebugDraw(const FSpatialHashStreamingGridLayerCell* LayerCell, TArray<const UWorldPartitionRuntimeCell*>& FilteredCells) const
{
	if (LayerCell)
	{
		Algo::CopyIf(LayerCell->GridCells, FilteredCells, [](const UWorldPartitionRuntimeCell* Cell)
		{
			if (Cell->IsDebugShown())
			{
				EStreamingStatus StreamingStatus = Cell->GetStreamingStatus();
				const TArray<FName>& DataLayers = Cell->GetDataLayers();

				switch (Cell->GetCellEffectiveWantedStateRaw())
				{
				case EDataLayerRuntimeState::Loaded:
				case EDataLayerRuntimeState::Activated:
					return true;
				case EDataLayerRuntimeState::Unloaded:
					break;
				default:
					checkNoEntry();
				}

				return (StreamingStatus != LEVEL_Unloaded) && (StreamingStatus != LEVEL_UnloadedButStillAround);
			}
			return false;
		});
	}
}

EWorldPartitionRuntimeCellVisualizeMode FSpatialHashStreamingGrid::GetStreamingCellVisualizeMode() const
{
	const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = FWorldPartitionDebugHelper::IsRuntimeSpatialHashCellStreamingPriorityShown() ? EWorldPartitionRuntimeCellVisualizeMode::StreamingPriority : EWorldPartitionRuntimeCellVisualizeMode::StreamingStatus;
	return VisualizeMode;
}

static TMap<FName, FColor> GetDataLayerDebugColors2(const UWorldPartition* InWorldPartition)
{
	TMap<FName, FColor> DebugColors;
	if (const UDataLayerManager* DataLayerManager = InWorldPartition->GetDataLayerManager())
	{
		DataLayerManager->ForEachDataLayerInstance([&DebugColors](UDataLayerInstance* DataLayerInstance)
		{
			DebugColors.Add(DataLayerInstance->GetFName(), DataLayerInstance->GetDebugColor());
			return true;
		});
	}
	return DebugColors;
}

void FSpatialHashStreamingGrid::Draw3D(const UWorldPartitionRuntimeSpatialHash* Owner, const TArray<FWorldPartitionStreamingSource>& Sources, const FTransform& Transform) const
{
	const UWorldPartition* WorldPartition = Owner->GetOuterUWorldPartition();
	UWorld* OwningWorld = WorldPartition->GetWorld();
	const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = GetStreamingCellVisualizeMode();
	const UContentBundleManager* ContentBundleManager = OwningWorld->ContentBundleManager;
	TMap<FName, FColor> DataLayerDebugColors = GetDataLayerDebugColors2(WorldPartition);

	float DynamicLoadingRangeScale = 1.f;
#if WITH_EDITOR
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(Owner->GetWorld()))
	{
		const FDynamicLoadingRangeScales& Scales = WorldPartitionSubsystem->GetDynamicLoadingRangeScales();
		DynamicLoadingRangeScale = Scales.Scale2D;
	}
#endif // WITH_EDITOR
	const FSquare2DGridHelper& Helper = GetGridHelper();
	int32 MinGridLevel = FMath::Clamp<int32>(GShowRuntimeSpatialHashGridLevel, 0, GridLevels.Num() - 1);
	int32 MaxGridLevel = FMath::Clamp<int32>(MinGridLevel + GShowRuntimeSpatialHashGridLevelCount - 1, 0, GridLevels.Num() - 1);
	const float GridViewMinimumSizeInCellCount = 5.f;
	const float GridViewLoadingRangeExtentRatio = 1.5f;
	const float GridLoadingRange = GetLoadingRange();
	const FVector MinExtent(CellSize * GridViewMinimumSizeInCellCount);
	TArray<const UWorldPartitionRuntimeCell*> FilteredCells;
	TSet<const UWorldPartitionRuntimeCell*> DrawnCells;

	for (const FWorldPartitionStreamingSource& Source : Sources)
	{
		FBox Region = Source.CalcBounds(GridLoadingRange * DynamicLoadingRangeScale, GridName);
		Region += FBox(Region.GetCenter() - MinExtent, Region.GetCenter() + MinExtent);

		for (int32 GridLevel = MinGridLevel; GridLevel <= MaxGridLevel; ++GridLevel)
		{
			Helper.Levels[GridLevel].ForEachIntersectingCells(Region, [&](const FGridCellCoord2& Coord2D)
			{
				FGridCellCoord Coords(Coord2D.X, Coord2D.Y, GridLevel);
				FilteredCells.Reset();
				ForEachLayerCell(Coords, [&](const FSpatialHashStreamingGridLayerCell* LayerCell)
				{
					GetFilteredCellsForDebugDraw(LayerCell, FilteredCells);
				});
				
				int32 FilteredCellIndex = 0;
				for (const UWorldPartitionRuntimeCell* Cell : FilteredCells)
				{
					++FilteredCellIndex;

					bool bIsAlreadyInSet = false;
					DrawnCells.Add(Cell, &bIsAlreadyInSet);
					if (bIsAlreadyInSet)
					{
						continue;
					}

					// Use cell MinMaxZ to compute effective cell bounds
					FBox CellBox = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(Cell->RuntimeCellData)->GetCellBounds();

					// Draw Cell using its debug color
					FColor CellColor = Cell->GetDebugColor(VisualizeMode).ToFColor(false).WithAlpha(16);
					DrawDebugSolidBox(OwningWorld, CellBox, CellColor, Transform, false, -1.f, 255);
					FVector CellPos = Transform.TransformPosition(CellBox.GetCenter());
					DrawDebugBox(OwningWorld, CellPos, CellBox.GetExtent(), Transform.GetRotation(), CellColor.WithAlpha(255), false, -1.f, 255, 10.f);

					// Draw Cell's DataLayers colored boxes
					if (DataLayerDebugColors.Num() && Cell->GetDataLayers().Num() > 0)
					{
						FBox DataLayerColoredBox = CellBox;
						double DataLayerSizeX = DataLayerColoredBox.GetSize().X / (5 * Cell->GetDataLayers().Num()); // Use 20% of cell's width
						DataLayerColoredBox.Max.X = DataLayerColoredBox.Min.X + DataLayerSizeX;
						double YFilteredCellSize = (CellBox.GetSize().Y / FilteredCells.Num());
						double YOffset = YFilteredCellSize * (FilteredCellIndex - 1);
						DataLayerColoredBox.Min.Y = CellBox.Min.Y + YOffset;
						DataLayerColoredBox.Max.Y = DataLayerColoredBox.Min.Y + YFilteredCellSize;
						FTranslationMatrix DataLayerOffsetMatrix(FVector(DataLayerSizeX, 0, 0));
						for (const FName& DataLayer : Cell->GetDataLayers())
						{
							const FColor& DataLayerColor = DataLayerDebugColors[DataLayer];
							DrawDebugSolidBox(OwningWorld, DataLayerColoredBox, DataLayerColor, Transform, false, -1.f, 255);
							DataLayerColoredBox = DataLayerColoredBox.TransformBy(DataLayerOffsetMatrix);
						}
					}

					// Draw Content Bundle colored boxes
					if (ContentBundleManager && FWorldPartitionDebugHelper::CanDrawContentBundles() && Cell->GetContentBundleID().IsValid())
					{
						if (const FContentBundleBase* ContentBundle = ContentBundleManager->GetContentBundle(OwningWorld, Cell->GetContentBundleID()))
						{
							check(ContentBundle->GetDescriptor());
							FBox ContentBundleColoredBox = CellBox;
							double ContentBundleSizeX = ContentBundleColoredBox.GetSize().X / 5; // Use 20% of cell's width
							ContentBundleColoredBox.Min.X = ContentBundleColoredBox.Max.X - ContentBundleSizeX;
							ContentBundleColoredBox.Max.X = ContentBundleColoredBox.Min.X + ContentBundleSizeX;
							FTranslationMatrix ContentBundleOffsetMatrix(FVector(ContentBundleColoredBox.GetSize().X - ContentBundleSizeX, 0, 0));
							ContentBundleColoredBox = ContentBundleColoredBox.TransformBy(ContentBundleOffsetMatrix);
							DrawDebugSolidBox(OwningWorld, ContentBundleColoredBox, ContentBundle->GetDescriptor()->GetDebugColor(), Transform, false, -1.f, 255);
						}
					}
				}
			});
		}

		// Draw Streaming Source
		double SourceLocationZ = Source.Location.Z;
		// In simulation, still use line trace to find best Z location for source debug draw as streaming source follows the camera
		if (UWorldPartition::IsSimulating())
		{
			FVector StartTrace = Source.Location + FVector(0.f, 0.f, 100.f);
			FVector EndTrace = StartTrace - FVector(0.f, 0.f, 1000000.f);
			FHitResult Hit;
			if (OwningWorld->LineTraceSingleByObjectType(Hit, StartTrace, EndTrace, FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionQueryParams(SCENE_QUERY_STAT(DebugWorldPartitionTrace), true)))
			{
				SourceLocationZ = Hit.ImpactPoint.Z;
			}
		}

		const FColor Color = Source.GetDebugColor();
		Source.ForEachShape(GridLoadingRange, GridName, /*bProjectIn2D*/ true, [&Color, &SourceLocationZ, &Transform, &OwningWorld, DynamicLoadingRangeScale, this](const FSphericalSector& Shape)
		{
			FSphericalSector ZOffsettedShape = Shape;
			// Adjust radius for dynamic scales
			ZOffsettedShape.SetRadius(ZOffsettedShape.GetRadius() * DynamicLoadingRangeScale);
			ZOffsettedShape.SetCenter(FVector(FVector2D(ZOffsettedShape.GetCenter()), SourceLocationZ));
			DrawStreamingSource3D(OwningWorld, ZOffsettedShape, Transform, Color);
		});
	}
}

void FSpatialHashStreamingGrid::DrawStreamingSource3D(UWorld* World, const FSphericalSector& InShape, const FTransform& InTransform, const FColor& InColor) const
{
	if (InShape.IsSphere())
	{
		FVector Location = InTransform.TransformPosition(InShape.GetCenter());
		DrawDebugSphere(World, Location, InShape.GetRadius(), 32, InColor, false, -1.f, 0, 20.f);
	}
	else
	{
		if (ULineBatchComponent* const LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World))
		{
			FSphericalSector Shape = InShape;
			Shape.SetAxis(InTransform.TransformVector(Shape.GetAxis()));
			Shape.SetCenter(InTransform.TransformPosition(Shape.GetCenter()));

			TArray<TPair<FVector, FVector>> Lines = Shape.BuildDebugMesh();
			TArray<FBatchedLine> BatchedLines;
			BatchedLines.Empty(Lines.Num());
			for (const auto& Line : Lines)
			{
				BatchedLines.Emplace(FBatchedLine(Line.Key, Line.Value, InColor, LineBatcher->DefaultLifeTime, 20.f, SDPG_World));
			};
			LineBatcher->DrawLines(BatchedLines);
		}
	}
}

void FSpatialHashStreamingGrid::Draw2D(const UWorldPartitionRuntimeSpatialHash* Owner, const FBox2D& Region2D, const FBox2D& GridScreenBounds, TFunctionRef<FVector2D(const FVector2D&, bool)> InWorldToScreen, FWorldPartitionDraw2DContext& DrawContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSpatialHashStreamingGrid::Draw2D);

	const UWorldPartition* WorldPartition = Owner->GetOuterUWorldPartition();
	const TArray<FWorldPartitionStreamingSource>& Sources = WorldPartition->GetStreamingSources();

	const FBox Region(FVector(Region2D.Min.X, Region2D.Min.Y, 0), FVector(Region2D.Max.X, Region2D.Max.Y, 0));
	const UWorld* OwningWorld = WorldPartition->GetWorld();
	const EWorldPartitionRuntimeCellVisualizeMode VisualizeMode = GetStreamingCellVisualizeMode();
	const UContentBundleManager* ContentBundleManager = OwningWorld->ContentBundleManager;
	TMap<FName, FColor> DataLayerDebugColors = GetDataLayerDebugColors2(WorldPartition);

	int32 MinGridLevel = FMath::Clamp<int32>(GShowRuntimeSpatialHashGridLevel, 0, GridLevels.Num() - 1);
	int32 MaxGridLevel = FMath::Clamp<int32>(MinGridLevel + GShowRuntimeSpatialHashGridLevelCount - 1, 0, GridLevels.Num() - 1);

	auto WorldToScreen = [&](const FVector2D& Pos, bool bIsLocal = true)
	{
		return InWorldToScreen(Pos, bIsLocal);
	};

	TArray<const UWorldPartitionRuntimeCell*> FilteredCells;
	for (int32 GridLevel = MinGridLevel; GridLevel <= MaxGridLevel; ++GridLevel)
	{
		// Draw Grid cells at desired grid level
		const FSquare2DGridHelper& Helper = GetGridHelper();
		Helper.Levels[GridLevel].ForEachIntersectingCells(Region, [&](const FGridCellCoord2& Coords2D)
		{
			FBox2D CellWorldBounds;
			FGridCellCoord Coords(Coords2D.X, Coords2D.Y, GridLevel);
			Helper.Levels[GridLevel].GetCellBounds(Coords2D, CellWorldBounds);
			
			FilteredCells.Reset();
			ForEachLayerCell(Coords, [&](const FSpatialHashStreamingGridLayerCell* LayerCell)
			{
				GetFilteredCellsForDebugDraw(LayerCell, FilteredCells);
			});

			if (FilteredCells.Num())
			{
				FVector2D CellBoundsSize = CellWorldBounds.GetSize();
				CellBoundsSize.Y /= FilteredCells.Num();
				FVector2D CellOffset(0.f, 0.f);
				for (const UWorldPartitionRuntimeCell* Cell : FilteredCells)
				{
					// Draw Cell using its debug color
					FVector2D StartPos = CellWorldBounds.Min + CellOffset;
					{
						DrawContext.LocalDrawTile(GridScreenBounds, StartPos, CellBoundsSize, Cell->GetDebugColor(VisualizeMode).CopyWithNewOpacity((VisualizeMode == EWorldPartitionRuntimeCellVisualizeMode::StreamingPriority) ? 0.75f : 0.25f), WorldToScreen);
					}

					CellOffset.Y += CellBoundsSize.Y;

					if (DrawContext.IsDetailedMode())
					{
						// Draw Cell's DataLayers colored boxes
						if (DataLayerDebugColors.Num() && Cell->GetDataLayers().Num() > 0)
						{
							FVector2D DataLayerOffset(0, 0);
							FVector2D DataLayerColoredBoxSize = CellBoundsSize;
							DataLayerColoredBoxSize.X /= (5 * Cell->GetDataLayers().Num()); // Use 20% of cell's width
							for (const FName& DataLayer : Cell->GetDataLayers())
							{
								const FColor& DataLayerColor = DataLayerDebugColors[DataLayer];
								DrawContext.LocalDrawTile(GridScreenBounds, StartPos + DataLayerOffset, DataLayerColoredBoxSize, DataLayerColor, WorldToScreen);
								DataLayerOffset.X += DataLayerColoredBoxSize.X;
							}
						}

						// Draw Content Bundle colored boxes
						if (ContentBundleManager && FWorldPartitionDebugHelper::CanDrawContentBundles() && Cell->GetContentBundleID().IsValid())
						{
							if (const FContentBundleBase* ContentBundle = ContentBundleManager->GetContentBundle(OwningWorld, Cell->GetContentBundleID()))
							{
								check(ContentBundle->GetDescriptor());
								FVector2D BoxSize = CellBoundsSize;
								BoxSize.X /= 5; // Use 20% of cell's width
								FVector2D ContentBundleOffset(CellBoundsSize.X - BoxSize.X, 0);
								DrawContext.LocalDrawTile(GridScreenBounds, StartPos + ContentBundleOffset, BoxSize, ContentBundle->GetDescriptor()->GetDebugColor(), WorldToScreen);
							}
						}
					}
				}

				// Draw cell bounds
				DrawContext.LocalDrawBox(GridScreenBounds, CellWorldBounds.Min, CellWorldBounds.GetSize(), FLinearColor::Black, 1, WorldToScreen);
			}
		});
	}

	// Draw X/Y Axis
	if (DrawContext.GetDrawGridAxis())
	{
		DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(FVector2D(-1638400.f, 0.f), false), WorldToScreen(FVector2D(1638400.f, 0.f), false), FLinearColor::Red, 3);
		DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(FVector2D(0.f, -1638400.f), false), WorldToScreen(FVector2D(0.f, 1638400.f), false), FLinearColor::Green, 3);
	}

	// Draw Grid Bounds
	if (DrawContext.GetDrawGridBounds())
	{
		FBox2D Bounds = GridScreenBounds.ExpandBy(FVector2D(10));
		FVector2D Size = GridScreenBounds.GetSize();
		DrawContext.PushDrawBox(Bounds, GridScreenBounds.Min, GridScreenBounds.Min + FVector2D(Size.X, 0), GridScreenBounds.Max, GridScreenBounds.Min + FVector2D(0, Size.Y), DebugColor, 1);
	}

	// Draw WorldBounds
	if (DrawContext.IsDetailedMode())
	{
		DrawContext.LocalDrawBox(GridScreenBounds, FVector2D(WorldBounds.Min), FVector2D(WorldBounds.GetSize()), FLinearColor::Yellow, 1, WorldToScreen);
	}

	// Draw Streaming Sources
	const float GridLoadingRange = GetLoadingRange();
	float DynamicLoadingRangeScale = 1.f;
#if WITH_EDITOR
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(Owner->GetWorld()))
	{
		const FDynamicLoadingRangeScales& Scales = WorldPartitionSubsystem->GetDynamicLoadingRangeScales();
		DynamicLoadingRangeScale = Scales.Scale2D;
	}
#endif // WITH_EDITOR

	for (const FWorldPartitionStreamingSource& Source : Sources)
	{
		const FColor Color = Source.GetDebugColor();
		Source.ForEachShape(GridLoadingRange, GridName, /*bProjectIn2D*/ true, [&Color, &WorldToScreen, &GridScreenBounds, &DrawContext, DynamicLoadingRangeScale, this](const FSphericalSector& Shape)
		{
			DrawStreamingSource2D(GridScreenBounds, Shape, WorldToScreen, Color, DrawContext, DynamicLoadingRangeScale);
		});

		// Draw velocity vector			
		const FVector2D Velocity2D = FVector2D(Source.Velocity);

		if (!Velocity2D.IsNearlyZero())
		{
			const FVector2D Center2D = FVector2D(Source.Location);
			DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + Velocity2D.GetSafeNormal() * GridLoadingRange * 0.5f), Color, 1);
		}
	}
}

void FSpatialHashStreamingGrid::DrawStreamingSource2D(const FBox2D& GridScreenBounds, const FSphericalSector& Shape, TFunctionRef<FVector2D(const FVector2D&)> WorldToScreen, const FColor& Color, FWorldPartitionDraw2DContext& DrawContext, float LoadingRangeScale) const
{
	check(!Shape.IsNearlyZero());

	// Spherical Sector
	const FVector2D Center2D(FVector2D(Shape.GetCenter()));
	const FSphericalSector::FReal Angle = Shape.GetAngle();
	const int32 MaxSegments = FMath::Max(4, FMath::CeilToInt(64 * Angle / 360.f));
	const float AngleIncrement = Angle / MaxSegments;
	const FVector2D Axis = FVector2D(Shape.GetAxis());
	const FVector Startup = FRotator(0, -0.5f * Angle, 0).RotateVector(Shape.GetScaledAxis() * LoadingRangeScale);

	FVector2D LineStart = FVector2D(Startup);
	if (!Shape.IsSphere())
	{
		// Draw sector start axis
		DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + LineStart), Color, 2);
	}
	// Draw sector Arc
	for (int32 i = 1; i <= MaxSegments; i++)
	{
		FVector2D LineEnd = FVector2D(FRotator(0, AngleIncrement * i, 0).RotateVector(Startup));
		DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D + LineStart), WorldToScreen(Center2D + LineEnd), Color, 2);
		LineStart = LineEnd;
	}
	// If sphere, close circle, else draw sector end axis
	DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D + LineStart), WorldToScreen(Center2D + (Shape.IsSphere() ? FVector2D(Startup) : FVector2D::ZeroVector)), Color, 2);

	// Draw direction vector
	DrawContext.PushDrawSegment(GridScreenBounds, WorldToScreen(Center2D), WorldToScreen(Center2D + Axis * Shape.GetRadius() * LoadingRangeScale), Color, 2);
}

// ------------------------------------------------------------------------------------------------

#if WITH_EDITORONLY_DATA

bool FSpatialHashRuntimeGrid::operator == (const FSpatialHashRuntimeGrid& Other) const
{
	return GridName == Other.GridName &&
		   CellSize == Other.CellSize &&
		   LoadingRange == Other.LoadingRange &&
		   bBlockOnSlowStreaming == Other.bBlockOnSlowStreaming &&
		   Origin == Other.Origin &&
		   Priority == Other.Priority &&
		   DebugColor == Other.DebugColor &&
		   bClientOnlyVisible == Other.bClientOnlyVisible &&
		   HLODLayer == Other.HLODLayer;
}

bool FSpatialHashRuntimeGrid::operator != (const FSpatialHashRuntimeGrid& Other) const
{
	return !(*this == Other);
}

#endif

// ------------------------------------------------------------------------------------------------

ASpatialHashRuntimeGridInfo::ASpatialHashRuntimeGridInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
{
#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
#endif
}

void ASpatialHashRuntimeGridInfo::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SpatialHashRuntimeGridInfoSpriteFixup)
	{
		// Fixes ASpatialHashRuntimeGridInfo that were saved prior to the constructor using ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite"))
		// Those Components are MarkForGarbage in their UActorComponent::PostInitProperties and so we need to clear the Component pointers
		FArchiveReplaceObjectRef<UObject>(this, { { GetSpriteComponent(), nullptr } });
	}
#endif
}

UWorldPartitionRuntimeSpatialHash::UWorldPartitionRuntimeSpatialHash(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bPreviewGrids(false)
	, PreviewGridLevel(0)
	, UseAlignedGridLevels(EWorldPartitionCVarProjectDefaultOverride::Disabled)
	, SnapNonAlignedGridLevelsToLowerLevels(EWorldPartitionCVarProjectDefaultOverride::Disabled)
	, PlaceSmallActorsUsingLocation(EWorldPartitionCVarProjectDefaultOverride::Enabled)
	, PlacePartitionActorsUsingLocation(EWorldPartitionCVarProjectDefaultOverride::Enabled)
#endif
	, bIsNameToGridMappingDirty(true)
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (UClass* RuntimeSpatialHashClass = FindObject<UClass>(nullptr, TEXT("/Script/Engine.WorldPartitionRuntimeHashSet")))
		{
			RegisterWorldPartitionRuntimeHashConverter(RuntimeSpatialHashClass, GetClass(), [](const UWorldPartitionRuntimeHash* SrcHash) -> UWorldPartitionRuntimeHash*
			{
				return CreateFrom(SrcHash);
			});
		}
	}
#endif
}

void UWorldPartitionRuntimeSpatialHash::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!IsRunningCookCommandlet())
	{
		// We don't want this to be persisted but we can't set the property Transient as it is NonPIEDuplicateTransient and those flags aren't compatible
		// If at some point GenerateStreaming is done after duplication we can remove this code.
		bIsNameToGridMappingDirty = true;
		StreamingGrids.Empty();
	}
}

FString UWorldPartitionRuntimeSpatialHash::GetCellCoordString(const FGridCellCoord& InCellGlobalCoord)
{
	return FString::Printf(TEXT("L%" INT64_FMT "_X%" INT64_FMT "_Y%" INT64_FMT), InCellGlobalCoord.Z, InCellGlobalCoord.X, InCellGlobalCoord.Y);
}

#if WITH_EDITOR
void UWorldPartitionRuntimeSpatialHash::DrawPreview() const
{
	// Use latest settings value for Preview
	GridPreviewer.Draw(GetWorld(), Grids, bPreviewGrids, PreviewGridLevel, GetUseAlignedGridLevels());
}

bool UWorldPartitionRuntimeSpatialHash::HasStreamingContent() const
{
	bool bHasStreamingContent = false;
	ForEachStreamingCells([&bHasStreamingContent](const UWorldPartitionRuntimeCell* Cell)
	{
		if (Cell)
		{
			bHasStreamingContent = true;
		}
		return !bHasStreamingContent;
	});
	return bHasStreamingContent;
}

void UWorldPartitionRuntimeSpatialHash::StoreStreamingContentToExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* OutExternalStreamingObject)
{
	Super::StoreStreamingContentToExternalStreamingObject(OutExternalStreamingObject);

	URuntimeSpatialHashExternalStreamingObject* StreamingObject = CastChecked<URuntimeSpatialHashExternalStreamingObject>(OutExternalStreamingObject);
	StreamingObject->StreamingGrids = MoveTemp(StreamingGrids);

	for (FSpatialHashStreamingGrid& StreamingGrid : StreamingObject->StreamingGrids)
	{
		for (FSpatialHashStreamingGridLevel& GridLevel : StreamingGrid.GridLevels)
		{
			// External streaming grids do not need to keep LayerCellsMapping as they will inject their cells into their associated source grid (see InjectExternalStreamingObject)
			GridLevel.LayerCellsMapping.Empty();

			for (FSpatialHashStreamingGridLayerCell& GridLayerCell : GridLevel.LayerCells)
			{
				for (UWorldPartitionRuntimeCell* Cell : GridLayerCell.GridCells)
				{
					// Do not dirty, otherwise it dirties the level previous outer (WorldPartition) which dirties the map. Occurs when entering PIE. 
					Cell->Rename(nullptr, StreamingObject,  REN_DoNotDirty);
				}
			}
		}
	}
}
#endif

void UWorldPartitionRuntimeSpatialHash::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldPartitionRuntimeSpatialHashCVarOverrides)
	{
		UseAlignedGridLevels = EWorldPartitionCVarProjectDefaultOverride::ProjectDefault;
		SnapNonAlignedGridLevelsToLowerLevels = EWorldPartitionCVarProjectDefaultOverride::ProjectDefault;
		PlaceSmallActorsUsingLocation = EWorldPartitionCVarProjectDefaultOverride::ProjectDefault;
		PlacePartitionActorsUsingLocation = EWorldPartitionCVarProjectDefaultOverride::ProjectDefault;
	}
#endif
}

void UWorldPartitionRuntimeSpatialHash::PostLoad()
{
	Super::PostLoad();

#if !WITH_EDITOR
	UE_LOGF(LogWorldPartition, Log, "UWorldPartitionRuntimeSpatialHash::PostLoad : UseAlignedGridLevels = %d, SnapNonAlignedGridLevelsToLowerLevels = %d",
		Settings.bUseAlignedGridLevels, Settings.bSnapNonAlignedGridLevelsToLowerLevels);
#endif
}

#if WITH_EDITOR
void UWorldPartitionRuntimeSpatialHash::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FSpatialHashStreamingGrid, CellSize))
	{
		for (FSpatialHashRuntimeGrid& Grid : Grids)
		{
			Grid.CellSize = FMath::Max(Grid.CellSize, 1600);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FSpatialHashStreamingGrid, GridName))
	{
		int GridIndex = 0;
		for (FSpatialHashRuntimeGrid& Grid : Grids)
		{
			if (Grid.GridName.IsNone())
			{
				Grid.GridName = *FString::Printf(TEXT("Grid%02d"), GridIndex++);
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FSpatialHashStreamingGrid, LoadingRange))
	{
		for (FSpatialHashRuntimeGrid& Grid : Grids)
		{
			Grid.LoadingRange = FMath::Max(Grid.LoadingRange, 0);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UWorldPartitionRuntimeSpatialHash::SetDefaultValues()
{
	check(!Grids.Num());

	FSpatialHashRuntimeGrid& MainGrid = Grids.AddDefaulted_GetRef();
	MainGrid.GridName = TEXT("MainGrid");
	MainGrid.DebugColor = FLinearColor::Gray;
}

void FSpatialHashSettings::UpdateSettings(const UWorldPartitionRuntimeSpatialHash& RuntimeSpatialHash)
{
#if WITH_EDITORONLY_DATA
	check(RuntimeSpatialHash.StreamingGrids.IsEmpty());
	bUseAlignedGridLevels = RuntimeSpatialHash.GetUseAlignedGridLevels();
	bSnapNonAlignedGridLevelsToLowerLevels = (RuntimeSpatialHash.SnapNonAlignedGridLevelsToLowerLevels == EWorldPartitionCVarProjectDefaultOverride::ProjectDefault) ? GRuntimeSpatialHashSnapNonAlignedGridLevelsToLowerLevels : (RuntimeSpatialHash.SnapNonAlignedGridLevelsToLowerLevels == EWorldPartitionCVarProjectDefaultOverride::Enabled);
	bPlaceSmallActorsUsingLocation = (RuntimeSpatialHash.PlaceSmallActorsUsingLocation == EWorldPartitionCVarProjectDefaultOverride::ProjectDefault) ? GRuntimeSpatialHashPlaceSmallActorsUsingLocation : (RuntimeSpatialHash.PlaceSmallActorsUsingLocation == EWorldPartitionCVarProjectDefaultOverride::Enabled);
	bPlacePartitionActorsUsingLocation = (RuntimeSpatialHash.PlacePartitionActorsUsingLocation == EWorldPartitionCVarProjectDefaultOverride::ProjectDefault) ? GRuntimeSpatialHashPlacePartitionActorsUsingLocation : (RuntimeSpatialHash.PlacePartitionActorsUsingLocation == EWorldPartitionCVarProjectDefaultOverride::Enabled);
	UE_LOGF(LogWorldPartition, Log, "FSpatialHashSettings::UpdateSettings : UseAlignedGridLevels = %d, SnapNonAlignedGridLevelsToLowerLevels = %d, PlaceSmallActorsUsingLocation = %d, PlacePartitionActorsUsingLocation = %d",
		bUseAlignedGridLevels, bSnapNonAlignedGridLevelsToLowerLevels, bPlaceSmallActorsUsingLocation, bPlacePartitionActorsUsingLocation);
#endif
}

bool UWorldPartitionRuntimeSpatialHash::GenerateStreaming(UWorldPartitionStreamingPolicy* StreamingPolicy, const IStreamingGenerationContext* StreamingGenerationContext, TArray<FString>* OutPackagesToGenerate)
{
	verify(Super::GenerateStreaming(StreamingPolicy, StreamingGenerationContext, OutPackagesToGenerate));

	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeSpatialHash::GenerateStreaming);
	UWorldPartition* WorldPartition = GetOuterUWorldPartition();

	if (!Grids.Num())
	{
		UE_LOGF(LogWorldPartition, Error, "Invalid partition grids setup");
		return false;
	}

	// Fix case where StreamingGrids might have been persisted.
	bIsNameToGridMappingDirty = true;
	StreamingGrids.Empty();
	
	// Apply Settings (should no longer change after streaming generation)
	Settings.UpdateSettings(*this);

	// Append grids from ASpatialHashRuntimeGridInfo actors to runtime spatial hash grids
	TArray<FSpatialHashRuntimeGrid> AllGrids;
	AllGrids.Append(Grids);

	StreamingGenerationContext->ForEachActorSetContainerInstance([&AllGrids, &WorldPartition](const IStreamingGenerationContext::FActorSetContainerInstance& ActorSetContainerInstance)
	{
		TArray<const FStreamingGenerationActorDescView*> SpatialHashRuntimeGridInfos = ActorSetContainerInstance.ActorDescViewMap->FindByExactNativeClass<ASpatialHashRuntimeGridInfo>();
		for (const FStreamingGenerationActorDescView* SpatialHashRuntimeGridInfo : SpatialHashRuntimeGridInfos)
		{
			FWorldPartitionReference Ref(WorldPartition, SpatialHashRuntimeGridInfo->GetGuid());
			ASpatialHashRuntimeGridInfo* RuntimeGridActor = CastChecked<ASpatialHashRuntimeGridInfo>(SpatialHashRuntimeGridInfo->GetActor());

			if (const FSpatialHashRuntimeGrid* ExistingRuntimeGrid = AllGrids.FindByPredicate([RuntimeGridActor](const FSpatialHashRuntimeGrid& RuntimeGrid) { return RuntimeGrid.GridName == RuntimeGridActor->GridSettings.GridName; }))
			{
				UE_LOGF(LogWorldPartition, Log, "Got duplicated runtime grid actor '%ls' for grid '%ls'", *SpatialHashRuntimeGridInfo->GetActorLabelOrName().ToString(), *RuntimeGridActor->GridSettings.GridName.ToString());
				check(*ExistingRuntimeGrid == RuntimeGridActor->GridSettings);
			}
			else
			{
				AllGrids.Add(RuntimeGridActor->GridSettings);
			}
		}
	});

	TMap<FName, int32> GridsMapping;
	GridsMapping.Add(NAME_None, 0);
	for (int32 i = 0; i < AllGrids.Num(); i++)
	{
		const FSpatialHashRuntimeGrid& Grid = AllGrids[i];
		check(!GridsMapping.Contains(Grid.GridName));
		GridsMapping.Add(Grid.GridName, i);
	}

	TArray<TArray<const IStreamingGenerationContext::FActorSetInstance*>> GridActorSetInstances;
	GridActorSetInstances.InsertDefaulted(0, AllGrids.Num());

	StreamingGenerationContext->ForEachActorSetInstance([&GridsMapping, &GridActorSetInstances](const IStreamingGenerationContext::FActorSetInstance& ActorSetInstance)
	{
		int32* FoundIndex = GridsMapping.Find(ActorSetInstance.RuntimeGrid);
		if (!FoundIndex)
		{
			//@todo_ow should this be done upstream?
			UE_LOGF(LogWorldPartition, Log, "Invalid partition grid '%ls' referenced by actor cluster", *ActorSetInstance.RuntimeGrid.ToString());
		}

		int32 GridIndex = FoundIndex ? *FoundIndex : 0;
		GridActorSetInstances[GridIndex].Add(&ActorSetInstance);
	});

	const FBox WorldBounds = StreamingGenerationContext->GetWorldBounds();
	for (int32 GridIndex = 0; GridIndex < AllGrids.Num(); GridIndex++)
	{
		const FSpatialHashRuntimeGrid& Grid = AllGrids[GridIndex];
		const FSquare2DGridHelper PartitionedActors = GetPartitionedActors(WorldBounds, Grid, GridActorSetInstances[GridIndex], Settings);
		if (!CreateStreamingGrid(Grid, PartitionedActors, StreamingPolicy, OutPackagesToGenerate))
		{
			return false;
		}
	}

	return true;
}

void UWorldPartitionRuntimeSpatialHash::DumpStateLog(FHierarchicalLogArchive& Ar) const
{
	Super::DumpStateLog(Ar);

	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	Ar.Printf(TEXT("%s - Runtime Spatial Hash"), *GetWorld()->GetName());
	Ar.Printf(TEXT("----------------------------------------------------------------------------------------------------------------"));
	FHierarchicalLogArchive::FIndentScope IndentScope = Ar.PrintfIndent(TEXT("StreamingGrids"));
	ForEachStreamingGrid([&Ar, this](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		StreamingGrid.DumpStateLog(Ar);
	});
}

UWorldPartitionRuntimeSpatialHash* UWorldPartitionRuntimeSpatialHash::CreateFrom(const UWorldPartitionRuntimeHash* SrcHash)
{
	const UWorldPartitionRuntimeHashSet* HashSet = CastChecked<UWorldPartitionRuntimeHashSet>(SrcHash);
	UWorldPartitionRuntimeSpatialHash* SpatialHash = NewObject<UWorldPartitionRuntimeSpatialHash>(SrcHash->GetOuter(), NAME_None, RF_Transactional);

	for (const FRuntimePartitionDesc& RuntimePartitionDesc : HashSet->RuntimePartitions)
	{
		FSpatialHashRuntimeGrid& Grid = SpatialHash->Grids.AddDefaulted_GetRef();
		Grid.GridName = RuntimePartitionDesc.Name;

		if (RuntimePartitionDesc.MainLayer)
		{
			Grid.DebugColor = RuntimePartitionDesc.MainLayer->DebugColor;
			Grid.bBlockOnSlowStreaming = RuntimePartitionDesc.MainLayer->bBlockOnSlowStreaming;
			Grid.bClientOnlyVisible = RuntimePartitionDesc.MainLayer->bClientOnlyVisible;
			Grid.Priority = RuntimePartitionDesc.MainLayer->Priority;

			if (URuntimePartitionLHGrid* LHGrid = Cast<URuntimePartitionLHGrid>(RuntimePartitionDesc.MainLayer))
			{
				Grid.CellSize = LHGrid->CellSize;
				Grid.Origin = FVector2D(LHGrid->Origin);
			}
		}	
	}

	return SpatialHash;
}

FString UWorldPartitionRuntimeSpatialHash::GetHLODGridName(uint32 InLODLevel, int32 InCellSize, double InLoadingRange)
{
	return FString::Format(TEXT("HLOD{0}_{1}m_{2}m"), { InLODLevel, int32(InCellSize * 0.01f), int32(InLoadingRange * 0.01f) });
}

FString UWorldPartitionRuntimeSpatialHash::GetCellNameString(UWorld* InOuterWorld, FName InGridName, const FGridCellCoord& InCellGlobalCoord, const FDataLayersID& InDataLayerID, const FGuid& InContentBundleID, FString* OutInstanceSuffix)
{
	FString WorldName = FPackageName::GetShortName(InOuterWorld->GetPackage());
	
	if (!IsRunningCookCommandlet() && InOuterWorld->IsGameWorld())
	{
		FString SourceWorldPath, InstancedWorldPath;
		if (InOuterWorld->GetSoftObjectPathMapping(SourceWorldPath, InstancedWorldPath))
		{
			const FTopLevelAssetPath SourceAssetPath(SourceWorldPath);
			WorldName = FPackageName::GetShortName(SourceAssetPath.GetPackageName());
						
			InstancedWorldPath = UWorld::RemovePIEPrefix(InstancedWorldPath);

			const FString SourcePackageName = SourceAssetPath.GetPackageName().ToString();
			const FTopLevelAssetPath InstanceAssetPath(InstancedWorldPath);
			const FString InstancePackageName = InstanceAssetPath.GetPackageName().ToString();

			if (int32 Index = InstancePackageName.Find(SourcePackageName); Index != INDEX_NONE)
			{
				*OutInstanceSuffix = InstancePackageName.Mid(Index + SourcePackageName.Len());
			}
		}
	}
			
	FString CellName = FString::Printf(TEXT("%s_%s_%s"), *WorldName, *InGridName.ToString(), *GetCellCoordString(InCellGlobalCoord));

	if (InDataLayerID.GetHash())
	{
		CellName += FString::Printf(TEXT("_DL%X"), InDataLayerID.GetHash());
	}

	if (InContentBundleID.IsValid())
	{
		CellName += FString::Printf(TEXT("_CB%s"), *UContentBundleDescriptor::GetContentBundleCompactString(InContentBundleID));
	}

	return CellName;
}

FGuid UWorldPartitionRuntimeSpatialHash::GetCellGuid(FName InGridName, int32 InCellSize, const FGridCellCoord& InCellGlobalCoord, const FDataLayersID& InDataLayerID, const FGuid& InContentBundleID)
{
	FString GridName = InGridName.ToString().ToLower();
	FGridCellCoord CellGlobalCoord = InCellGlobalCoord;
	uint32 DataLayerID = InDataLayerID.GetHash();
	uint32 ContentBundleID = InContentBundleID.IsValid() ? GetTypeHash(InContentBundleID) : 0;

	FGuid CellGuid;

	{
		FArchiveMD5 ArMD5;
		ArMD5 << GridName << CellGlobalCoord << DataLayerID << ContentBundleID;
		CellGuid = ArMD5.GetGuidFromHash();
		check(CellGuid.IsValid());
	}

	// FFortniteReleaseBranchCustomObjectVersion::WorldPartitionRuntimeCellGuidWithCellSize
	// CellGuid taking the cell size into account
	{
		FArchiveMD5 ArMD5;
		ArMD5 << CellGuid << InCellSize;
		CellGuid = ArMD5.GetGuidFromHash();
		check(CellGuid.IsValid());
	}

	return CellGuid;
}

bool UWorldPartitionRuntimeSpatialHash::GetPreviewGrids() const
{
	return bPreviewGrids;
}

void UWorldPartitionRuntimeSpatialHash::SetPreviewGrids(bool bInPreviewGrids)
{
	Modify(false);
	bPreviewGrids = bInPreviewGrids;
}

int32 UWorldPartitionRuntimeSpatialHash::GetPreviewGridLevel() const
{
	return PreviewGridLevel;
}

void UWorldPartitionRuntimeSpatialHash::SetPreviewGridLevel(int32 InPreviewGridLevel)
{
	Modify(false);
	PreviewGridLevel = FMath::Max(0, InPreviewGridLevel);
}

bool UWorldPartitionRuntimeSpatialHash::CreateStreamingGrid(const FSpatialHashRuntimeGrid& RuntimeGrid, const FSquare2DGridHelper& PartitionedActors, UWorldPartitionStreamingPolicy* StreamingPolicy, TArray<FString>* OutPackagesToGenerate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateStreamingGrid);

	if (IsRunningCookCommandlet() && !OutPackagesToGenerate)
	{
		UE_LOGF(LogWorldPartition, Error, "Error creating runtime streaming cells for cook, OutPackagesToGenerate is null.");
		return false;
	}

	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	UWorld* World = WorldPartition->GetWorld();
	UWorld* OuterWorld = GetTypedOuter<UWorld>();
	const bool bIsMainWorldPartition = (World == OuterWorld);

	FSpatialHashStreamingGrid& CurrentStreamingGrid = StreamingGrids.AddDefaulted_GetRef();
	CurrentStreamingGrid.Settings = Settings;
	CurrentStreamingGrid.GridName = RuntimeGrid.GridName;
	CurrentStreamingGrid.CellSize = PartitionedActors.CellSize;
	CurrentStreamingGrid.WorldBounds = PartitionedActors.WorldBounds;
	CurrentStreamingGrid.LoadingRange = RuntimeGrid.LoadingRange;
	CurrentStreamingGrid.bBlockOnSlowStreaming = RuntimeGrid.bBlockOnSlowStreaming;
	CurrentStreamingGrid.Origin = FVector(RuntimeGrid.Origin, 0);
	CurrentStreamingGrid.DebugColor = RuntimeGrid.DebugColor;
	CurrentStreamingGrid.bClientOnlyVisible = RuntimeGrid.bClientOnlyVisible;
	CurrentStreamingGrid.GridIndex = (StreamingGrids.Num() - 1);

	// Move actors into the final streaming grids
	CurrentStreamingGrid.GridLevels.Reserve(PartitionedActors.Levels.Num());

	int32 Level = INDEX_NONE;
	for (const FSquare2DGridHelper::FGridLevel& TempLevel : PartitionedActors.Levels)
	{
		Level++;

		FSpatialHashStreamingGridLevel& GridLevel = CurrentStreamingGrid.GridLevels.AddDefaulted_GetRef();

		for (const auto& TempCellMapping : TempLevel.CellsMapping)
		{
			const int64 CellIndex = TempCellMapping.Key;
			const int64 CellCoordX = CellIndex % TempLevel.GridSize;
			const int64 CellCoordY = CellIndex / TempLevel.GridSize;

			const FSquare2DGridHelper::FGridLevel::FGridCell& TempCell = TempLevel.Cells[TempCellMapping.Value];

			for (const FSquare2DGridHelper::FGridLevel::FGridCellDataChunk& GridCellDataChunk : TempCell.GetDataChunks())
			{
				// Cell cannot be treated as always loaded if it has data layers
				const bool bIsCellSpatiallyLoaded = &TempCell != &PartitionedActors.GetAlwaysLoadedCell();
				const bool bIsCellAlwaysLoaded = !bIsCellSpatiallyLoaded && !GridCellDataChunk.HasDataLayers() && !GridCellDataChunk.GetContentBundleID().IsValid();				
				
				TArray<IStreamingGenerationContext::FActorInstance> FilteredActors;
				if (PopulateCellActorInstances(GridCellDataChunk.GetActorSetInstances(), bIsMainWorldPartition, bIsCellAlwaysLoaded, FilteredActors))
				{
					FGridCellCoord CellGlobalCoords;
					FString WorldInstanceSuffix;
					verify(PartitionedActors.GetCellGlobalCoords(FGridCellCoord(CellCoordX, CellCoordY, Level), CellGlobalCoords));
					const FString CellName = GetCellNameString(OuterWorld, CurrentStreamingGrid.GridName, CellGlobalCoords, GridCellDataChunk.GetDataLayersID(), GridCellDataChunk.GetContentBundleID(), &WorldInstanceSuffix);
					const FGuid CellGuid = GetCellGuid(CurrentStreamingGrid.GridName, CurrentStreamingGrid.CellSize, CellGlobalCoords, GridCellDataChunk.GetDataLayersID(), GridCellDataChunk.GetContentBundleID());

					UWorldPartitionRuntimeCell* StreamingCell = CreateRuntimeCell(StreamingPolicy->GetRuntimeCellClass(), UWorldPartitionRuntimeCellDataSpatialHash::StaticClass(), CellName, WorldInstanceSuffix);
					UWorldPartitionRuntimeCellDataSpatialHash* CellDataSpatialHash = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(StreamingCell->RuntimeCellData);

					const TArray<const UDataLayerInstance*>& DataLayers = GridCellDataChunk.GetDataLayers();
					const bool bIsClientOnlyFromDataLayers = FDataLayerUtils::IsClientOnlyFromDataLayers(DataLayers);
					const bool bIsHLOD = RuntimeGrid.HLODLayer ? true : false;
					const bool bBlockOnSlowStreaming = ResolveBlockOnSlowStreamingForCell(CurrentStreamingGrid.bBlockOnSlowStreaming, bIsHLOD, DataLayers);
					StreamingCell->SetIsAlwaysLoaded(bIsCellAlwaysLoaded);
					StreamingCell->SetIsSpatiallyLoaded(bIsCellSpatiallyLoaded);
					StreamingCell->SetDataLayers(DataLayers);
					StreamingCell->SetContentBundleUID(GridCellDataChunk.GetContentBundleID());
					StreamingCell->SetClientOnlyVisible(CurrentStreamingGrid.bClientOnlyVisible || bIsClientOnlyFromDataLayers);
					StreamingCell->SetBlockOnSlowLoading(bBlockOnSlowStreaming);
					StreamingCell->SetIsHLOD(bIsHLOD);
					StreamingCell->SetGuid(CellGuid);

					FBox2D Bounds;
					verify(TempLevel.GetCellBounds(FGridCellCoord2(CellCoordX, CellCoordY), Bounds));				
					const double CellExtent = Bounds.GetExtent().X;
					check(CellExtent < MAX_flt);

					CellDataSpatialHash->HierarchicalLevel = Level;
					CellDataSpatialHash->Position = FVector(Bounds.GetCenter(), 0.f);
					CellDataSpatialHash->Extent = (float)CellExtent;
					CellDataSpatialHash->GridName = RuntimeGrid.GridName;
					CellDataSpatialHash->DebugName = CellName + WorldInstanceSuffix;
					const int32 DataLayersStreamingPriority = GetDataLayersStreamingPriority(DataLayers);
					CellDataSpatialHash->Priority = RuntimeGrid.Priority + DataLayersStreamingPriority;

					if (WorldPartition->ShouldStripRuntimeCellByTransformerStack(StreamingCell))
					{
						continue;
					}

					PopulateRuntimeCell(StreamingCell, FilteredActors, OutPackagesToGenerate);

					WorldPartition->ApplyRuntimeCellsTransformerStack(StreamingCell);

					int32 LayerCellIndex;
					int32* LayerCellIndexPtr = GridLevel.LayerCellsMapping.Find(CellIndex);
					if (LayerCellIndexPtr)
					{
						LayerCellIndex = *LayerCellIndexPtr;
					}
					else
					{
						LayerCellIndex = GridLevel.LayerCells.AddDefaulted();
						GridLevel.LayerCellsMapping.Add(CellIndex, LayerCellIndex);
					}

					GridLevel.LayerCells[LayerCellIndex].GridCells.Add(StreamingCell);
				}
			}
		}
	}

	return true;
}

bool UWorldPartitionRuntimeSpatialHash::GetUseAlignedGridLevels() const
{
	return UseAlignedGridLevels == EWorldPartitionCVarProjectDefaultOverride::ProjectDefault ? GRuntimeSpatialHashUseAlignedGridLevels : (UseAlignedGridLevels == EWorldPartitionCVarProjectDefaultOverride::Enabled);
}

void UWorldPartitionRuntimeSpatialHash::FlushStreamingContent()
{
	Super::FlushStreamingContent();
	bIsNameToGridMappingDirty = true;
	StreamingGrids.Empty();
}

FName UWorldPartitionRuntimeSpatialHash::GetDefaultGrid() const
{
	return Grids.Num() ? Grids[0].GridName : NAME_None;
}

bool UWorldPartitionRuntimeSpatialHash::IsValidGrid(FName GridName, const UClass* ActorClass) const
{
	// We always consider grids from auto-generated HLODs to be valid
	static const UClass* WorldPartitionHLODClass = FindObjectChecked<UClass>(nullptr, TEXT("/Script/Engine.WorldPartitionHLOD"));
	if (ActorClass->IsChildOf(WorldPartitionHLODClass))
	{
		return true;
	}

	// None always maps to the default grid
	if (GridName.IsNone())
	{
		return true;
	}

	for (const FSpatialHashRuntimeGrid& Grid : Grids)
	{
		if (Grid.GridName == GridName)
		{
			return true;
		}
	}

	return false;
}

TOptional<FFixedGridInfo> UWorldPartitionRuntimeSpatialHash::GetFixedGridInfo(FName InGridName) const
{
	// NAME_None maps to the first grid; mirrors the convention used elsewhere in WP runtime hashes.
	const FSpatialHashRuntimeGrid* Grid = nullptr;

	if (InGridName.IsNone())
	{
		Grid = Grids.IsEmpty() ? nullptr : &Grids[0];
	}
	else
	{
		for (const FSpatialHashRuntimeGrid& CandidateGrid : Grids)
		{
			if (CandidateGrid.GridName == InGridName)
			{
				Grid = &CandidateGrid;
				break;
			}
		}
	}

	if ((Grid == nullptr) || (Grid->CellSize <= 0))
	{
		return {};
	}

	FFixedGridInfo Info;
	Info.CellSize = static_cast<uint32>(Grid->CellSize);
	Info.Origin = FVector(Grid->Origin.X, Grid->Origin.Y, 0.0);
	// SpatialHash 2D-ness is hash-wide via bEnableZCulling; every grid in this hash shares the answer.
	Info.bIs2D = !IsStreaming3D();

	return Info;
}

#endif //WITH_EDITOR

// Streaming interface
void UWorldPartitionRuntimeSpatialHash::ForEachStreamingCells(TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func) const
{
	ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		if (IsCellRelevantFor(StreamingGrid.bClientOnlyVisible))
		{
			StreamingGrid.ForEachRuntimeCell(Func);
		}
	});
}

void UWorldPartitionRuntimeSpatialHash::ForEachStreamingCellsQuery(const FWorldPartitionStreamingQuerySource& QuerySource, TFunctionRef<bool(const UWorldPartitionRuntimeCell*)> Func, FWorldPartitionQueryCache* QueryCache) const
{
	ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		if (IsCellRelevantFor(StreamingGrid.bClientOnlyVisible))
		{
			TSet<const UWorldPartitionRuntimeCell*> Cells;
			StreamingGrid.GetCells(QuerySource, Cells, GetEffectiveEnableZCulling(bEnableZCulling), QueryCache, bQueryNonSpatialCells);

			for (const UWorldPartitionRuntimeCell* Cell : Cells)
			{
				//@todo_ow: Remove this test once always loaded cells (which are empty) are removed at cook time
				if (!Cell->IsAlwaysLoaded())
				{
					if (!Func(Cell))
					{
						break;
					}
				}
			}
		}
	});
}

uint32 UWorldPartitionRuntimeSpatialHash::ComputeUpdateStreamingHash() const
{
	FHashBuilder HashBuilder;
	ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		HashBuilder << StreamingGrid.GetLoadingRange();
	});
	return HashBuilder.GetHash();
}

void UWorldPartitionRuntimeSpatialHash::ForEachStreamingCellsSources(const TArray<FWorldPartitionStreamingSource>& Sources, TFunctionRef<bool(const UWorldPartitionRuntimeCell*, EStreamingSourceTargetState)> Func, const FWorldPartitionStreamingContext& InContext) const
{
	// Build a context when none is provided (for backward compatibility)
	const FWorldPartitionStreamingContext StackContext = !InContext.IsValid() ? FWorldPartitionStreamingContext::Create(GetTypedOuter<UWorld>()) : FWorldPartitionStreamingContext();
	const FWorldPartitionStreamingContext& Context = InContext.IsValid() ? InContext : StackContext;
	check(Context.IsValid());

	FStreamingSourceCells ActivateStreamingSourceCells;
	FStreamingSourceCells LoadStreamingSourceCells;
	float LoadingRangeScale = 1.f;
#if WITH_EDITOR
	LoadingRangeScale = Context.GetDynamicLoadingRangeScales().Scale2D;
#endif // WITH_EDITOR

	if (Sources.Num() == 0)
	{
		// Get always loaded cells
		ForEachStreamingGrid([&] (const FSpatialHashStreamingGrid& StreamingGrid)
		{
			if (IsCellRelevantFor(StreamingGrid.bClientOnlyVisible))
			{
				StreamingGrid.GetNonSpatiallyLoadedCells(ActivateStreamingSourceCells.GetCells(), LoadStreamingSourceCells.GetCells(), Context);
			}
		});
	}
	else
	{
		// Get cells based on streaming sources
		ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& StreamingGrid)
		{
			if (IsCellRelevantFor(StreamingGrid.bClientOnlyVisible))
			{
				StreamingGrid.GetCells(Sources, ActivateStreamingSourceCells, LoadStreamingSourceCells, GetEffectiveEnableZCulling(bEnableZCulling), Context, LoadingRangeScale);
			}
		});
	}

	for (const UWorldPartitionRuntimeCell* Cell : ActivateStreamingSourceCells.GetCells())
	{
		if (!Func(Cell, EStreamingSourceTargetState::Activated))
		{
			return;
		}
	}

	for (const UWorldPartitionRuntimeCell* Cell : LoadStreamingSourceCells.GetCells())
	{
		if (!Func(Cell, EStreamingSourceTargetState::Loaded))
		{
			return;
		}
	}
}

bool UWorldPartitionRuntimeSpatialHash::InjectExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject)
{
	if (!Super::InjectExternalStreamingObject(InExternalStreamingObject))
	{
		return false;
	}

	URuntimeSpatialHashExternalStreamingObject* SpatialHashExternalStreamingObject = CastChecked<URuntimeSpatialHashExternalStreamingObject>(InExternalStreamingObject);

	// Validate that there's a corresponding streaming grid for each streaming grid of this external streaming object
	for (const FSpatialHashStreamingGrid& ExternalStreamingGrid : SpatialHashExternalStreamingObject->StreamingGrids)
	{
		const FSpatialHashStreamingGrid* SourceGrid = GetStreamingGridByName(ExternalStreamingGrid.GridName);
		if (!SourceGrid)
		{
			UE_LOGF(LogWorldPartition, Error, "Failed to inject external streaming object %ls, can't find matching streaming grid %ls.",
				*InExternalStreamingObject->GetName(), *ExternalStreamingGrid.GridName.ToString());
			return false;
		}
		else if (SourceGrid->Settings != ExternalStreamingGrid.Settings)
		{
			UE_LOGF(LogWorldPartition, Error, "Failed to inject external streaming object %ls, miss matching settings.",
				*InExternalStreamingObject->GetName());
			return false;
		}
	}

	for (const FSpatialHashStreamingGrid& ExternalStreamingGrid : SpatialHashExternalStreamingObject->StreamingGrids)
	{
		const FSpatialHashStreamingGrid* SourceGrid = GetStreamingGridByName(ExternalStreamingGrid.GridName);
		SourceGrid->InjectExternalStreamingObjectGrid(ExternalStreamingGrid);
	}

	return true;
}

bool UWorldPartitionRuntimeSpatialHash::RemoveExternalStreamingObject(URuntimeHashExternalStreamingObjectBase* InExternalStreamingObject)
{
	if (!Super::RemoveExternalStreamingObject(InExternalStreamingObject))
	{
		return false;
	}

	URuntimeSpatialHashExternalStreamingObject* SpatialHashExternalStreamingObject = CastChecked<URuntimeSpatialHashExternalStreamingObject>(InExternalStreamingObject);

	for (const FSpatialHashStreamingGrid& ExternalStreamingGrid : SpatialHashExternalStreamingObject->StreamingGrids)
	{
		// In PIE, UWorldPartition::OnEndPlay() will call UWorldPartitionRuntimeSpatialHash::FlushStreamingContent() 
		// which will empty StreamingGrids before RemoveExternalStreamingObject is called.
		// In this case, we don't need to remove injected cells as the source grid was removed.
		if (const FSpatialHashStreamingGrid* SourceGrid = GetStreamingGridByName(ExternalStreamingGrid.GridName))
		{
			SourceGrid->RemoveExternalStreamingObjectGrid(ExternalStreamingGrid);
		}
	}

	return true;
}

uint32 UWorldPartitionRuntimeSpatialHash::GetNumGrids() const
{
	return StreamingGrids.Num();
}

void UWorldPartitionRuntimeSpatialHash::ForEachStreamingGrid(TFunctionRef<void(FSpatialHashStreamingGrid&)> Func)
{
	for (FSpatialHashStreamingGrid& StreamingGrid : StreamingGrids)
	{
		Func(StreamingGrid);
	}
}

void UWorldPartitionRuntimeSpatialHash::ForEachStreamingGridBreakable(TFunctionRef<bool(const FSpatialHashStreamingGrid&)> Func) const
{
	for (const FSpatialHashStreamingGrid& StreamingGrid : StreamingGrids)
	{
		if (!Func(StreamingGrid))
		{
			return;
		}
	}
}

void UWorldPartitionRuntimeSpatialHash::ForEachStreamingGrid(TFunctionRef<void(const FSpatialHashStreamingGrid&)> Func) const
{
	for (const FSpatialHashStreamingGrid& StreamingGrid : StreamingGrids)
	{
		Func(StreamingGrid);
	}
}

const TMap<FName, const FSpatialHashStreamingGrid*>& UWorldPartitionRuntimeSpatialHash::GetNameToGridMapping() const
{
	if (bIsNameToGridMappingDirty)
	{
		NameToGridMapping.Reset();
		ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& StreamingGrid)
		{
			ensureMsgf(!NameToGridMapping.Contains(StreamingGrid.GridName), TEXT("2 Streaming grids share the name: \"%s\". Some features will not work properly."), *StreamingGrid.GridName.ToString());
			NameToGridMapping.Add(StreamingGrid.GridName, &StreamingGrid);
		});
		bIsNameToGridMappingDirty = false;
	}

	return NameToGridMapping;
}

const FSpatialHashStreamingGrid* UWorldPartitionRuntimeSpatialHash::GetStreamingGridByName(FName InGridName) const
{
	if (const FSpatialHashStreamingGrid* const* StreamingGrid = GetNameToGridMapping().Find(InGridName))
	{
		return *StreamingGrid;
	}
	return nullptr;
}

ENGINE_API EWorldPartitionStreamingPerformance UWorldPartitionRuntimeSpatialHash::GetStreamingPerformanceForCell(const UWorldPartitionRuntimeCell* Cell, bool& bOutShouldBlock) const
{
	const UWorldPartitionRuntimeCellDataSpatialHash* CellDataSpatialHash = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(Cell->RuntimeCellData);
	const FSpatialHashStreamingGrid* StreamingGrid = GetStreamingGridByName(CellDataSpatialHash->GridName);

	if (ensure(StreamingGrid))
	{
		const float LoadingRange = StreamingGrid->LoadingRange;
		const double BlockOnSlowStreamingWarningRatio = GBlockOnSlowStreamingRatio * GBlockOnSlowStreamingWarningFactor;
		const double OnSlowStreamingWarningRatio = GSlowStreamingRatio * GSlowStreamingWarningFactor;
		EWorldPartitionStreamingPerformance BlockingPerformance = EWorldPartitionStreamingPerformance::Good;
		EWorldPartitionStreamingPerformance NonBlockingPerformance = EWorldPartitionStreamingPerformance::Good;
		if (Cell->GetBlockOnSlowLoading() && CellDataSpatialHash->bCachedWasRequestedByBlockingSource)
		{
			const double Distance = FMath::Sqrt(CellDataSpatialHash->CachedMinSquareDistanceToBlockingSource) - (double((int64)StreamingGrid->CellSize << (int64)CellDataSpatialHash->HierarchicalLevel) / 2);

			const double Ratio = Distance / LoadingRange;
			if (Ratio <= 0.0)
			{
				BlockingPerformance = EWorldPartitionStreamingPerformance::Immediate;
			}
			else if (Ratio < GBlockOnSlowStreamingRatio)
			{
				BlockingPerformance = EWorldPartitionStreamingPerformance::Critical;
			}
			else if (Ratio < BlockOnSlowStreamingWarningRatio)
			{
				BlockingPerformance = EWorldPartitionStreamingPerformance::Slow;
			}
		}

		const double Distance = FMath::Sqrt(CellDataSpatialHash->CachedMinSquareDistanceToSource) - (double((int64)StreamingGrid->CellSize << (int64)CellDataSpatialHash->HierarchicalLevel) / 2);

		const double Ratio = Distance / LoadingRange;
		if (Ratio <= 0.0)
		{
			NonBlockingPerformance = EWorldPartitionStreamingPerformance::Immediate;
		}
		else if (Ratio < GSlowStreamingRatio)
		{
			NonBlockingPerformance = EWorldPartitionStreamingPerformance::Critical;
		}
		else if (Ratio < OnSlowStreamingWarningRatio)
		{
			NonBlockingPerformance = EWorldPartitionStreamingPerformance::Slow;
		}

		if (BlockingPerformance >= NonBlockingPerformance)
		{
			bOutShouldBlock = BlockingPerformance >= EWorldPartitionStreamingPerformance::Critical;
			return BlockingPerformance;
		}
		else
		{
			bOutShouldBlock = false;
			return NonBlockingPerformance;
		}
	}

	bOutShouldBlock = false;
	return EWorldPartitionStreamingPerformance::Good;
}

bool UWorldPartitionRuntimeSpatialHash::SupportsWorldAssetStreaming(const FName& InTargetGrid)
{
	return GetNameToGridMapping().Contains(InTargetGrid);
}

FGuid UWorldPartitionRuntimeSpatialHash::RegisterWorldAssetStreaming(const UWorldPartition::FRegisterWorldAssetStreamingParams& InParams)
{
	if (!InParams.IsValid())
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Invalid parameters provided.");
		return FGuid();
	}

	if (WorldAssetStreamingObjects.Contains(InParams.Guid))
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: World asset guid '%ls' was already registered.", *InParams.Guid.ToString());
		return FGuid();
	}

	const FSpatialHashStreamingGrid* TargetGrid = GetStreamingGridByName(InParams.WorldAssetDesc.TargetGrid);
	if (!TargetGrid)
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Unable to resolve TargetGrid '%ls'.", *InParams.WorldAssetDesc.TargetGrid.ToString());
		return FGuid();
	}

	TArray<TPair<const FSpatialHashStreamingGrid*, const TSoftObjectPtr<UWorld>>> HLODTargetGrids;
	for (const UWorldPartition::FRegisterWorldAssetStreamingParams::FWorldAssetDesc& HLODWorldAssetDesc : InParams.HLODWorldAssetDescs)
	{
		if (!HLODWorldAssetDesc.WorldAsset.IsNull() && !HLODWorldAssetDesc.TargetGrid.IsNone())
		{
			const FSpatialHashStreamingGrid* HLODTargetGrid = GetStreamingGridByName(HLODWorldAssetDesc.TargetGrid);
			if (!HLODTargetGrid)
			{
				UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Unable to resolve TargetGridHLOD '%ls'.", *HLODWorldAssetDesc.TargetGrid.ToString());
			}
			else
			{
				HLODTargetGrids.Add({ HLODTargetGrid, HLODWorldAssetDesc.WorldAsset });
			}
		}
	}

	URuntimeSpatialHashExternalStreamingObject* StreamingObject = CastChecked<URuntimeSpatialHashExternalStreamingObject>(CreateExternalStreamingObject(URuntimeSpatialHashExternalStreamingObject::StaticClass(), this, GetTypedOuter<UWorld>()));
	if (!StreamingObject)
	{
		UE_LOGF(LogWorldPartition, Error, "RegisterWorldAssetStreaming: Couldn't create ExternalStreamingObject.");
		return FGuid();
	}		

	FGuid SourceCellGuid;
		
	auto CreateStreamingCell = [&InParams, &SourceCellGuid, StreamingObject, this](const FSpatialHashStreamingGrid* SourceGrid, const TSoftObjectPtr<UWorld>& WorldAsset, bool bIsHLODPass) -> bool
	{
		check(SourceGrid);

		FSquare2DGridHelper CurrentGridHelper(SourceGrid->WorldBounds, SourceGrid->Origin, SourceGrid->CellSize, SourceGrid->Settings.bUseAlignedGridLevels);
		FSpatialHashStreamingGrid& CurrentGrid = StreamingObject->StreamingGrids.AddDefaulted_GetRef();
		CurrentGrid.Settings = SourceGrid->Settings;
		CurrentGrid.GridName = SourceGrid->GridName;
		CurrentGrid.Origin = SourceGrid->Origin;
		CurrentGrid.CellSize = SourceGrid->CellSize;
		CurrentGrid.WorldBounds = SourceGrid->WorldBounds;
		CurrentGrid.LoadingRange = SourceGrid->LoadingRange;
		CurrentGrid.bBlockOnSlowStreaming = SourceGrid->bBlockOnSlowStreaming;
		CurrentGrid.DebugColor = FColor::MakeRandomColor();
		CurrentGrid.bClientOnlyVisible = SourceGrid->bClientOnlyVisible;
		CurrentGrid.GridLevels.SetNum(CurrentGridHelper.Levels.Num());

		FSquare2DGridHelper::FGridLevel::FGridCell* GridCell = nullptr;
		if (InParams.bBoundsPlacement)
		{
			// Find grid level cell that encompasses the actor cluster bounding box and put actors in it.
			const FVector ClusterSize = InParams.Bounds.GetSize();
			const double MinRequiredCellExtent = FMath::Max(ClusterSize.X, ClusterSize.Y);
			const int32 FirstPotentialGridLevel = FMath::Max(FMath::CeilToDouble(FMath::Log2(MinRequiredCellExtent / (double)CurrentGridHelper.CellSize)), 0);
			for (int32 GridLevelIndex = FirstPotentialGridLevel; GridLevelIndex < CurrentGridHelper.Levels.Num(); GridLevelIndex++)
			{
				FSquare2DGridHelper::FGridLevel& GridLevel = CurrentGridHelper.Levels[GridLevelIndex];
				if (GridLevel.GetNumIntersectingCells(InParams.Bounds) == 1)
				{
					GridLevel.ForEachIntersectingCells(InParams.Bounds, [&GridLevel, &GridCell](const FGridCellCoord2& Coords)
					{
						check(!GridCell);
						GridCell = &GridLevel.GetCell(Coords);
					});
					break;
				}
			}
		}
		else
		{
			const FBox2D Bounds2D(FVector2D(InParams.Bounds.Min), FVector2D(InParams.Bounds.Max));

			FGridCellCoord2 CellCoords;
			if (CurrentGridHelper.Levels[0].GetCellCoords(Bounds2D.GetCenter(), CellCoords))
			{
				GridCell = &CurrentGridHelper.Levels[0].GetCell(CellCoords);
			}
		}

		if (!GridCell)
		{
			// Fallback to always loaded
			GridCell = &CurrentGridHelper.GetAlwaysLoadedCell();
		}

		// Create Cell
		FName TargetGridName = SourceGrid->GridName;
		FGuid InstanceGuid = InParams.Guid;
		FArchiveMD5 ArMD5;
		ArMD5 << TargetGridName << InstanceGuid;
		const FGuid CellGuid = ArMD5.GetGuidFromHash();
		check(CellGuid.IsValid());
		if (!bIsHLODPass)
		{
			SourceCellGuid = CellGuid;
		}

		const FString CellName = FString::Printf(TEXT("InjectedCell_%s"), *CellGuid.ToString());

		if (UWorldPartitionRuntimeLevelStreamingCell* StreamingCell = Cast<UWorldPartitionRuntimeLevelStreamingCell>(
			CreateRuntimeCell(UWorldPartitionRuntimeLevelStreamingCell::StaticClass(), UWorldPartitionRuntimeCellDataSpatialHash::StaticClass(), CellName, InParams.CellInstanceSuffix, StreamingObject)))
		{
			UWorldPartitionRuntimeCellDataSpatialHash* CellDataSpatialHash = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(StreamingCell->RuntimeCellData);
			// Setup Cell's DataSpatialHash
			FBox2D CellBounds;
			FGridCellCoord Coord = GridCell->GetCoords();
			int64 Level = Coord.Z;
			int64 CellCoordX = Coord.X;
			int64 CellCoordY = Coord.Y;
			FSquare2DGridHelper::FGridLevel& TempLevel = CurrentGridHelper.Levels[Level];
			verify(TempLevel.GetCellBounds(FGridCellCoord2(CellCoordX, CellCoordY), CellBounds));
			const double CellExtent = CellBounds.GetExtent().X;
			check(CellExtent < MAX_flt);
			CellDataSpatialHash->HierarchicalLevel = Level;
			CellDataSpatialHash->Position = FVector(CellBounds.GetCenter(), 0.f);
			CellDataSpatialHash->Extent = (float)CellExtent;
			CellDataSpatialHash->GridName = CurrentGrid.GridName;
			CellDataSpatialHash->DebugName = CellName + InParams.CellInstanceSuffix;
			CellDataSpatialHash->ContentBounds = InParams.Bounds;
			CellDataSpatialHash->Priority = InParams.Priority;

			// Setup Cell
			const bool bIsCellAlwaysLoaded = (GridCell == &CurrentGridHelper.GetAlwaysLoadedCell());
			StreamingCell->SetIsAlwaysLoaded(bIsCellAlwaysLoaded);
			StreamingCell->SetClientOnlyVisible(CurrentGrid.bClientOnlyVisible);
			StreamingCell->SetBlockOnSlowLoading(CurrentGrid.bBlockOnSlowStreaming);
			StreamingCell->SetIsHLOD(bIsHLODPass);
			StreamingCell->SetGuid(CellGuid);

			if (bIsHLODPass)
			{
				StreamingCell->SetSourceCellGuid(SourceCellGuid);
			}

			if (StreamingCell->CreateAndSetLevelStreaming(WorldAsset, InParams.Transform))
			{
				// Insert cell in grid
				CurrentGrid.InsertGridCell(StreamingCell, GridCell->GetCoords());
			}
			else
			{
				UE_LOGF(LogWorldPartition, Error, "Error creating streaming cell %ls for world asset %ls at %ls", *StreamingCell->GetName(), *WorldAsset.ToString(), *InParams.Transform.ToString());
				return false;
			}
		}
		else
		{
			UE_LOGF(LogWorldPartition, Error, "Error creating streaming cell %ls for world asset %ls at %ls", *CellName, *WorldAsset.ToString(), *InParams.Transform.ToString());
			return false;
		}

		return true;
	};

	if (CreateStreamingCell(TargetGrid, InParams.WorldAssetDesc.WorldAsset, false /* bisHLODPass */) == false)
	{
		return FGuid();
	}
	for (const TPair<const FSpatialHashStreamingGrid*, const TSoftObjectPtr<UWorld>>& HLODTargetGrid : HLODTargetGrids)
	{
		if (CreateStreamingCell(HLODTargetGrid.Key, HLODTargetGrid.Value, true /* bIsHLODPass */) == false)
		{
			return FGuid();
		}
	}

	GetOuterUWorldPartition()->InjectExternalStreamingObject(StreamingObject);
	WorldAssetStreamingObjects.Add(InParams.Guid, StreamingObject);

	return InParams.Guid;
}

bool UWorldPartitionRuntimeSpatialHash::UnregisterWorldAssetStreaming(const FGuid& InWorldAssetStreamingGuid)
{
	if (TObjectPtr<URuntimeSpatialHashExternalStreamingObject>* StreamingObject = WorldAssetStreamingObjects.Find(InWorldAssetStreamingGuid))
	{
		// External streaming objects are created with a provided name which helps to detect invalid runtime states of injected content.
		// Before releasing these objects, trash their name to make sure they won't be recycled if the tile re-injects the objects before a GC was triggered first.
		// Apply the same logic on the LevelStreaming object of each injected cell as it is named using the injected cell name and is outered to the owning world.
		auto TrashExternalStreamingData = [](URuntimeSpatialHashExternalStreamingObject* InStreamingObject)
		{
			InStreamingObject->ForEachStreamingCells([](UWorldPartitionRuntimeCell& Cell)
			{
				UWorldPartitionRuntimeLevelStreamingCell* InjectedCell = Cast<UWorldPartitionRuntimeLevelStreamingCell>(&Cell);
				if (UWorldPartitionLevelStreamingDynamic* LevelStreaming = InjectedCell ? InjectedCell->GetLevelStreaming() : nullptr)
				{
					TrashObject(LevelStreaming);
					// Make sure to flag this streaming level to be unloaded and removed as we don't want any future RequestLevel 
					// of a newly created streaming level of the same WorldAsset to fail. 
					LevelStreaming->SetIsRequestingUnloadAndRemoval(true);
				}
			});

			TrashObject(InStreamingObject);
		};

		if (IsValid(*StreamingObject))
		{
			if (IsValid(GetOuterUWorldPartition()))
			{
				GetOuterUWorldPartition()->RemoveExternalStreamingObject(*StreamingObject);
			}
			TrashExternalStreamingData(*StreamingObject);
		}
		WorldAssetStreamingObjects.Remove(InWorldAssetStreamingGuid);
		return true;
	}

	return false;
}

TArray<UWorldPartitionRuntimeCell*> UWorldPartitionRuntimeSpatialHash::GetWorldAssetStreamingCells(const FGuid& InWorldAssetStreamingGuid)
{
	TArray<UWorldPartitionRuntimeCell*> Result;
	if (TObjectPtr<URuntimeSpatialHashExternalStreamingObject>* StreamingObject = WorldAssetStreamingObjects.Find(InWorldAssetStreamingGuid))
	{
		(*StreamingObject)->ForEachStreamingCells([&Result](UWorldPartitionRuntimeCell& Cell) { Result.Add(&Cell); });
	}
	return Result;
}

bool UWorldPartitionRuntimeSpatialHash::Draw2D(FWorldPartitionDraw2DContext& DrawContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionRuntimeSpatialHash::Draw2D);

	const UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	const TArray<FWorldPartitionStreamingSource>& Sources = WorldPartition->GetStreamingSources();

	TArray<const FSpatialHashStreamingGrid*> FilteredStreamingGrids = GetFilteredStreamingGrids();
	if (FilteredStreamingGrids.Num() == 0 || Sources.Num() == 0)
	{
		return false;
	}

	// Sort streaming grids to render them sorted by loading range
	FilteredStreamingGrids.Sort([](const FSpatialHashStreamingGrid& A, const FSpatialHashStreamingGrid& B) { return A.LoadingRange < B.LoadingRange; });

	const FTransform WorldPartitionTransform = WorldPartition->GetInstanceTransform();
	const FTransform2D LocalWPToGlobal(FQuat2D(FMath::DegreesToRadians(WorldPartitionTransform.Rotator().Yaw)), FVector2D(WorldPartitionTransform.GetLocation()));
	const FTransform2D GlobalToLocalWP = LocalWPToGlobal.Inverse();
	const FBox2D& WorldRegion = DrawContext.GetWorldRegion();
	const FVector2D WorldRegionSize = WorldRegion.GetSize();
	TArray<FVector2D> Points;
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + FVector2D(WorldRegionSize.X, 0)));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + WorldRegionSize));
	Points.Add(GlobalToLocalWP.TransformPoint(WorldRegion.Min + FVector2D(0, WorldRegionSize.Y)));
	const FBox2D LocalRegion(Points);

	const FBox2D& CanvasRegion = DrawContext.GetCanvasRegion();
	const int32 GridScreenWidthDivider = DrawContext.IsDetailedMode() ? FilteredStreamingGrids.Num() : 1;
	const float GridScreenWidthShrinkSize = GridScreenWidthDivider > 1 ? 20.f : 0.f;
	const float CanvasMaxScreenWidth = CanvasRegion.GetSize().X;
	const float GridMaxScreenWidth = CanvasMaxScreenWidth / GridScreenWidthDivider;
	const float GridEffectiveScreenWidth = FMath::Min(GridMaxScreenWidth, CanvasRegion.GetSize().Y) - GridScreenWidthShrinkSize;
	const FVector2D PartitionCanvasSize = FVector2D(CanvasRegion.GetSize().GetMin());
	const FVector2D GridScreenExtent = FVector2D(GridEffectiveScreenWidth, GridEffectiveScreenWidth);
	const FVector2D GridScreenHalfExtent = 0.5f * GridScreenExtent;
	const FVector2D GridScreenInitialOffset = CanvasRegion.Min;

	FBox GridsShapeBounds(ForceInit);
	FBox2D GridsBounds(ForceInit);
	int32 GridIndex = 0;

	for (const FSpatialHashStreamingGrid* StreamingGrid : FilteredStreamingGrids)
	{
		for (const FWorldPartitionStreamingSource& Source : Sources)
		{
			Source.ForEachShape(StreamingGrid->GetLoadingRange(), StreamingGrid->GridName, /*bProjectIn2D*/ true, [&GridsShapeBounds](const FSphericalSector& Shape)
			{
				GridsShapeBounds += Shape.CalcBounds();
			});
		}

		FVector2D GridReferenceWorldPos;
		FVector2D WorldRegionExtent;

		if (DrawContext.IsDetailedMode())
		{
			GridReferenceWorldPos = FVector2D(GridsShapeBounds.GetCenter());
			WorldRegionExtent = FVector2D(GridsShapeBounds.ExpandBy(GridsShapeBounds.GetExtent() * 0.1f).GetExtent().GetMax());
		}
		else
		{
			GridReferenceWorldPos = FVector2D(WorldRegion.GetCenter());
			WorldRegionExtent = FVector2D(WorldRegion.GetExtent().GetMax());
		}

		const FVector2D GridScreenOffset = GridScreenInitialOffset + ((float)GridIndex * FVector2D(GridMaxScreenWidth, 0.f)) + GridScreenHalfExtent + FVector2D(GridScreenWidthShrinkSize * 0.5f);
		const FVector2D WorldToScreenScale = GridScreenHalfExtent / WorldRegionExtent;
		const FBox2D GridScreenBounds(GridScreenOffset - GridScreenHalfExtent, GridScreenOffset + GridScreenHalfExtent);

		auto WorldToScreen = [&](const FVector2D& LocalWorldPos, bool bIsLocalWorldPos = true)
		{
			FVector2D GlocalPos = bIsLocalWorldPos ? LocalWPToGlobal.TransformPoint(LocalWorldPos) : LocalWorldPos;
			return (WorldToScreenScale * (GlocalPos - GridReferenceWorldPos)) + GridScreenOffset;
		};

		StreamingGrid->Draw2D(this, LocalRegion, GridScreenBounds, WorldToScreen, DrawContext);
		GridsBounds += GridScreenBounds;

		if (DrawContext.IsDetailedMode())
		{
			// Draw WorldPartition name
			FVector2D GridInfoPos = GridScreenOffset - GridScreenHalfExtent;

			FWorldPartitionCanvasMultiLineText MultiLineText;
			MultiLineText.Emplace(UWorld::RemovePIEPrefix(FPaths::GetBaseFilename(WorldPartition->GetPackage()->GetName())), FLinearColor::White);
			// Draw Grid name, loading range
			FString GridInfoText = FString::Printf(TEXT("%s | %d m"), *StreamingGrid->GridName.ToString(), int32(StreamingGrid->GetLoadingRange() * 0.01f));
			if (StreamingGrid->bClientOnlyVisible)
			{
				GridInfoText += TEXT(" | Client Only");
			}
#if !UE_BUILD_SHIPPING
			if (GFilterRuntimeSpatialHashGridLevel != INDEX_NONE)
			{
				GridInfoText += FString::Printf(TEXT(" | GridLevelFilter %d"), GFilterRuntimeSpatialHashGridLevel);
			}
#endif
			MultiLineText.Emplace(GridInfoText, FLinearColor::Yellow);
			FWorldPartitionCanvasMultiLineTextItem Item(GridInfoPos, MultiLineText);
			DrawContext.PushDrawText(Item);

			++GridIndex;
		}
	}

	FBox2D DesiredWorldBounds(ForceInit);
	if (GridsShapeBounds.IsValid)
	{
		// Convert to 2D
		FBox2D GridsShapeBounds2D = FBox2D(FVector2D(GridsShapeBounds.Min.X, GridsShapeBounds.Min.Y), FVector2D(GridsShapeBounds.Max.X, GridsShapeBounds.Max.Y));
		FVector2D CenterGlobalPos = LocalWPToGlobal.TransformPoint(GridsShapeBounds2D.GetCenter());
		// Use max extent of X/Y
		FVector2D Extent = FVector2D(GridsShapeBounds2D.GetExtent().GetMax());
		// Transform to global space
		GridsShapeBounds2D = FBox2D(CenterGlobalPos - Extent, CenterGlobalPos + Extent);
		// Expand by 10% computed bounds
		DesiredWorldBounds = GridsShapeBounds2D.ExpandBy(GridsShapeBounds2D.GetExtent() * 0.1f);
	}
	DrawContext.SetDesiredWorldBounds(DesiredWorldBounds);
	DrawContext.SetUsedCanvasBounds(GridsBounds);

	return true;
}

void UWorldPartitionRuntimeSpatialHash::Draw3D(const TArray<FWorldPartitionStreamingSource>& Sources) const
{
	UWorldPartition* WorldPartition = GetOuterUWorldPartition();
	FTransform Transform = WorldPartition->GetInstanceTransform();
	TArray<const FSpatialHashStreamingGrid*> FilteredStreamingGrids = GetFilteredStreamingGrids();
	for (const FSpatialHashStreamingGrid* StreamingGrid : FilteredStreamingGrids)
	{
		StreamingGrid->Draw3D(this, Sources, Transform);
	}
}

bool UWorldPartitionRuntimeSpatialHash::ContainsRuntimeHash(const FString& Name) const
{
	bool bResult = false;
	ForEachStreamingGridBreakable([&](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		bResult = StreamingGrid.GridName.ToString().Equals(Name, ESearchCase::IgnoreCase);
		return !bResult;
	});

	return bResult;
}

bool UWorldPartitionRuntimeSpatialHash::IsStreaming3D() const
{
	// If using Z culling, return true.
	return GetEffectiveEnableZCulling(bEnableZCulling);
}

TArray<const FSpatialHashStreamingGrid*> UWorldPartitionRuntimeSpatialHash::GetFilteredStreamingGrids() const
{
	TArray<const FSpatialHashStreamingGrid*> FilteredStreamingGrids;
	FilteredStreamingGrids.Reserve(GetNumGrids());

	ForEachStreamingGrid([this, &FilteredStreamingGrids](const FSpatialHashStreamingGrid& StreamingGrid)
	{
		if (IsCellRelevantFor(StreamingGrid.bClientOnlyVisible))
		{
			if (FWorldPartitionDebugHelper::IsDebugRuntimeHashGridShown(StreamingGrid.GridName))
			{
				FilteredStreamingGrids.Add(&StreamingGrid);
			}
		}
	});

	return FilteredStreamingGrids;
}

// ------------------------------------------------------------------------------------------------

#if WITH_EDITOR
void URuntimeSpatialHashExternalStreamingObject::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	Super::DumpStateLog(Ar);

	FHierarchicalLogArchive::FIndentScope IndentScope = Ar.PrintfIndent(TEXT("StreamingGrids"));
	for (FSpatialHashStreamingGrid& StreamingGrid : StreamingGrids)
	{
		StreamingGrid.DumpStateLog(Ar);
	}
}
#endif

#undef LOCTEXT_NAMESPACE

