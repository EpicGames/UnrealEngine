// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionRuntimeSpatialHashGridPreviewer.h"
#include "Materials/Material.h"
#include "WorldPartition/RuntimeSpatialHash/RuntimeSpatialHashGridHelper.h"
#include "Engine/World.h"
#include "Engine/PostProcessVolume.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionRuntimeSpatialHashGridPreviewer)

/*
 * FWorldPartitionRuntimeSpatialHashGridPreviewer
 */

FWorldPartitionRuntimeSpatialHashGridPreviewer::FWorldPartitionRuntimeSpatialHashGridPreviewer()
#if WITH_EDITORONLY_DATA
	: MID(nullptr)
	, Volume(nullptr)
	, bIsInitialized(false)
#endif
{
}

#if WITH_EDITOR
void FWorldPartitionRuntimeSpatialHashGridPreviewer::Draw(UWorld* World, const TArray<FSpatialHashRuntimeGrid>& Grids, bool bEnabled, int32 PreviewGridLevel, bool bUseAlignedGridLevels)
{
	if (!bIsInitialized)
	{
		Material = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/WorldPartition/WorldPartitionSpatialHashGridPreviewMaterial"));
		bIsInitialized = true;
	}

	if (bEnabled && Material)
	{
		if (!Volume)
		{
			FActorSpawnParameters SpawnParameters;
			SpawnParameters.ObjectFlags |= RF_Transient;
			SpawnParameters.ObjectFlags &= ~RF_Transactional;
			Volume = World->SpawnActor<APostProcessVolume>(SpawnParameters);
		}

		if (!MID && Volume)
		{
			MID = UMaterialInstanceDynamic::Create(Material, Volume);
			Volume->Settings.WeightedBlendables.Array.Emplace(1.0f, MID);
			Volume->bUnbound = true;
		}

		if (MID)
		{
			for (int i = 0; i < MAX_PREVIEW_GRIDS; ++i)
			{
				bool bGridEnabled = Grids.IsValidIndex(i) && Grids[i].CellSize > 0;
				FGridParametersCache& CachedParameters = GridParameters[i];

				if (CachedParameters.bEnabled != bGridEnabled)
				{
					MID->SetScalarParameterValue(*FString::Printf(TEXT("Grid%d_Enabled"), i), bGridEnabled ? 1.0f : 0.f);
					CachedParameters.bEnabled = bGridEnabled;
				}
				if (bGridEnabled)
				{
					const FSpatialHashRuntimeGrid& Grid = Grids[i];

					const int32 PreviewCellSize = (1 << FMath::Max(0, PreviewGridLevel)) * Grid.CellSize;

					if (CachedParameters.CellSize != PreviewCellSize)
					{
						MID->SetScalarParameterValue(*FString::Printf(TEXT("Grid%d_CellSize"), i), (float)PreviewCellSize);
						CachedParameters.CellSize = PreviewCellSize;
					}

					if (CachedParameters.LoadingRange != Grid.LoadingRange)
					{
						MID->SetScalarParameterValue(*FString::Printf(TEXT("Grid%d_LoadingRange"), i), Grid.LoadingRange);
						CachedParameters.LoadingRange = Grid.LoadingRange;
					}

					if (CachedParameters.GridColor != Grid.DebugColor)
					{
						MID->SetVectorParameterValue(*FString::Printf(TEXT("Grid%d_Color"), i), Grid.DebugColor);
						CachedParameters.GridColor = Grid.DebugColor;
					}

					FVector GridOffset = FVector(Grid.Origin, 0) + (bUseAlignedGridLevels ? FVector(0.5 * PreviewCellSize) : FVector::ZeroVector);
					if (CachedParameters.GridOffset != GridOffset)
					{
						MID->SetVectorParameterValue(*FString::Printf(TEXT("Grid%d_Offset"), i), GridOffset);
						CachedParameters.GridOffset = GridOffset;
					}
				}
			}
		}
	}

	if (Volume)
	{
		Volume->bEnabled = bEnabled;
	}
}
#endif
