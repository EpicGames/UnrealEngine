// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionSettings.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "UObject/SoftObjectPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionSettings)

UWorldPartitionSettings::UWorldPartitionSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

#if WITH_EDITOR
UWorldPartitionSettings::CookFilterCommandLineLoader::CookFilterCommandLineLoader()
{
	// First parse EDL filters passed through commandline like -EDLCookFilters=Value1+Value2+Value3
	FString SwitchValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("-EDLCookFilters="), SwitchValue))
	{
		TArray<FString> Tokens;
		SwitchValue.ParseIntoArray(Tokens, TEXT("+"));
		for (const FString& Filter : Tokens)
		{
			FTopLevelAssetPath Path;
			if (Path.TrySetPath(Filter))
			{
				EDLFilters.Add(Path);
			}
		}
	}

	// Second parse generators filters passed through commandline like -GeneratorPackageFilter=Value1+Value2+Value3
	// These restrict if the generator should be cooked more than once. When specified the generator will not be saved again.
	FString GeneratorPackageFilterString;
	if (FParse::Value(FCommandLine::Get(), TEXT("-GeneratorPackageFilter="), GeneratorPackageFilterString))
	{
		TArray<FString> Tokens;
		GeneratorPackageFilterString.ParseIntoArray(Tokens, TEXT("+"));
		for (const FString& Filter : Tokens)
		{
			GeneratorPackageFilters.Add(FName(Filter));
		}
	}

	// Then append filters found in config, e.g. in [Base/Default/etc.]Engine.ini
	// [/Script/Engine.WorldPartitionSettings]
	// +EDLCookFilters=/MyGFP/DataLayer/MyEDL.MyEDL
	for (const TSoftObjectPtr<UExternalDataLayerAsset>& EDL : GetDefault<UWorldPartitionSettings>()->EDLCookFilters)
	{
		FSoftObjectPath Path = EDL.ToSoftObjectPath();
		if (Path.IsValid())
		{
			EDLFilters.Add(Path.GetAssetPath());
		}
	}
}

const UWorldPartitionSettings::CookFilterCommandLineLoader& UWorldPartitionSettings::GetCookFilterCommandLineLoader()
{
	static UWorldPartitionSettings::CookFilterCommandLineLoader Loader;
	return Loader;
}

const TSet<FTopLevelAssetPath>& UWorldPartitionSettings::GetEDLCookFilters()
{
	return GetCookFilterCommandLineLoader().EDLFilters;
}

const bool UWorldPartitionSettings::ShouldCookEDLGenerator(const UPackage* Generator)
{
	if (!UWorldPartitionSettings::GetEDLCookFilters().IsEmpty())
	{
		return false;
	}

	return !GetCookFilterCommandLineLoader().GeneratorPackageFilters.Contains(Generator->GetFName());
}

#endif // #if WITH_EDITOR
