// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionStreamingDescriptor.h"

#if WITH_EDITOR

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "WorldPartition/WorldPartitionRuntimeCellDataSpatialHash.h"
#include "WorldPartition/WorldPartitionUtils.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionLHGrid.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "Engine/World.h"
#include "HAL/FileManager.h"
#include "Editor.h"

static FAutoConsoleCommand GenerateStreamingDescriptorCmd(
	TEXT("wp.editor.GenerateStreamingDescriptor"),
	TEXT("Generate the streaming descriptor for the current world in the speficied file."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() == 1)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (!World->IsGameWorld())
				{
					if (FArchive* LogFile = IFileManager::Get().CreateFileWriter(*Args[0]))
					{
						UE::Private::WorldPartition::FStreamingDescriptor StreamingDescriptor;
						UE::Private::WorldPartition::FStreamingDescriptor::GenerateStreamingDescriptor(World, StreamingDescriptor);
						{				
							FHierarchicalLogArchive HierarchicalLogAr(*LogFile);
							StreamingDescriptor.DumpStateLog(HierarchicalLogAr);
						}
						LogFile->Close();
						delete LogFile;
					}
				}
			}
		}
	})
);

namespace UE::Private::WorldPartition
{

void FStreamingDescriptor::FStreamingActor::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintIndent(*Path.ToString()));
	
	if (BaseClass.IsValid())
	{
		Ar.Printf(TEXT("  Base Class: %s"), *BaseClass.ToString());
	}

	Ar.Printf(TEXT("Native Class: %s"), *NativeClass.ToString());
	Ar.Printf(TEXT("     Package: %s"), *Package.ToString());
}

void FStreamingDescriptor::FStreamingCell::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	Ar.Printf(TEXT("          Bounds: %s"), *Bounds.ToString());
	Ar.Printf(TEXT("   Always Loaded: %s"), bIsAlwaysLoaded ? TEXT("1" : TEXT("0")));
	Ar.Printf(TEXT("Spatially Loaded: %s"), bIsSpatiallyLoaded ? TEXT("1" : TEXT("0")));

	if (DataLayers.Num())
	{
		UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Data Layers:")));
		for (FName DataLayer : DataLayers)
		{
			Ar.Print(*DataLayer.ToString());
		}
	}

	if (Actors.Num())
	{
		UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Actors:")));
		for (FStreamingActor& Actor : Actors)
		{
			Actor.DumpStateLog(Ar);
		}
	}
}

void FStreamingDescriptor::FStreamingGrid::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintIndent(*Name.ToString()));

	Ar.Printf(TEXT("       Bounds: %s"), *Bounds.ToString());
	Ar.Printf(TEXT("    Cell Size: %d"), CellSize);
	Ar.Printf(TEXT("Loading Range: %d"), LoadingRange);

	if (StreamingCells.Num())
	{
		UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Streaming Cells:")));
		for (FStreamingCell& StreamingCell : StreamingCells)
		{
			StreamingCell.DumpStateLog(Ar);
		}
	}
}

void FStreamingDescriptor::DumpStateLog(FHierarchicalLogArchive& Ar)
{
	UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Streaming Grids:")));

	for (FStreamingGrid& StreamingGrid : StreamingGrids)
	{
		StreamingGrid.DumpStateLog(Ar);
	}
}

bool FStreamingDescriptor::GenerateStreamingDescriptor(UWorld* InWorld, FStreamingDescriptor& OutStreamingDescriptor, const FStreamingDescriptorParams& InParams)
{
	check(InWorld);

	if (UWorldPartition* WorldPartition = InWorld->GetWorldPartition())
	{
		FWorldPartitionUtils::FSimulateCookSessionParams Params;
		Params.FilteredClasses = InParams.FilteredClasses;
		FWorldPartitionUtils::FSimulateCookedSession SimulateCookedSession(InWorld, Params);
		UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
		TMap<FName, FStreamingDescriptor::FStreamingGrid> StreamingGrids;

		if (const UWorldPartitionRuntimeSpatialHash* RuntimeSpatialHash = Cast<UWorldPartitionRuntimeSpatialHash>(WorldPartition->RuntimeHash))
		{
			RuntimeSpatialHash->ForEachStreamingGrid([&StreamingGrids](const FSpatialHashStreamingGrid& SpatialHashStreamingGrid)
			{
				FStreamingDescriptor::FStreamingGrid& StreamingGrid = StreamingGrids.Add(SpatialHashStreamingGrid.GridName);
				StreamingGrid.Name = SpatialHashStreamingGrid.GridName;
				StreamingGrid.Bounds = SpatialHashStreamingGrid.WorldBounds;
				StreamingGrid.CellSize = SpatialHashStreamingGrid.CellSize;
				StreamingGrid.LoadingRange = SpatialHashStreamingGrid.LoadingRange;
			});

			for (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>& InjectedExternalStreamingObject : RuntimeSpatialHash->InjectedExternalStreamingObjects)
			{
				if (InjectedExternalStreamingObject.IsValid())
				{
					URuntimeSpatialHashExternalStreamingObject* ExternalStreamingObject = CastChecked<URuntimeSpatialHashExternalStreamingObject>(InjectedExternalStreamingObject.Get());
					
					for (FSpatialHashStreamingGrid& SpatialHashGrid : ExternalStreamingObject->StreamingGrids)
					{
						FStreamingDescriptor::FStreamingGrid& StreamingGrid = StreamingGrids.FindOrAdd(SpatialHashGrid.GridName);
						StreamingGrid.ExternalStreamingObjects.AddUnique(ExternalStreamingObject->GetPackagePathToCreate());
					}					
				}
			}

			WorldPartition->RuntimeHash->ForEachStreamingCells([&StreamingGrids, DataLayerManager](const UWorldPartitionRuntimeCell* Cell)
			{
				UWorldPartitionRuntimeCellDataSpatialHash* CellDataSpatialHash = CastChecked<UWorldPartitionRuntimeCellDataSpatialHash>(Cell->RuntimeCellData);
				FStreamingDescriptor::FStreamingGrid& StreamingGrid = StreamingGrids.FindChecked(CellDataSpatialHash->GridName);
				FStreamingDescriptor::FStreamingCell& StreamingCell = *new(StreamingGrid.StreamingCells) FStreamingDescriptor::FStreamingCell;
				StreamingCell.Bounds = Cell->GetCellBounds();
				StreamingCell.bIsAlwaysLoaded = Cell->IsAlwaysLoaded();
				StreamingCell.bIsSpatiallyLoaded = Cell->IsSpatiallyLoaded();
				StreamingCell.CellPackage = Cell->GetLevelPackageName();

				StreamingCell.DataLayers = Cell->GetDataLayers();

				if (DataLayerManager)
				{
					for (FName& DataLayerName : StreamingCell.DataLayers)
					{
						if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromName(DataLayerName))
						{
							DataLayerName = *DataLayerInstance->GetDataLayerShortName();
						}
					}
				}

				const UWorldPartitionRuntimeLevelStreamingCell* LevelStreamingCell = CastChecked<const UWorldPartitionRuntimeLevelStreamingCell>(Cell);
				const TArray<FWorldPartitionRuntimeCellObjectMapping>& CellPackages = LevelStreamingCell->GetPackages();

				TArray<FStreamingDescriptor::FStreamingActor> Actors;
				StreamingCell.Actors.Reserve(CellPackages.Num());

				for (const FWorldPartitionRuntimeCellObjectMapping& CellPackage : CellPackages)
				{
					FStreamingDescriptor::FStreamingActor& StreaminActor = StreamingCell.Actors.AddDefaulted_GetRef();
					StreaminActor.BaseClass = CellPackage.BaseClass; 
					StreaminActor.NativeClass = CellPackage.NativeClass;
					StreaminActor.Path = CellPackage.Path.ToString();
					StreaminActor.Package = CellPackage.Package;
					StreaminActor.ActorGuid = CellPackage.ActorInstanceGuid;
				}

				return true;
			});
		}
		else if (const UWorldPartitionRuntimeHashSet* RuntimeHashSet = Cast<UWorldPartitionRuntimeHashSet>(WorldPartition->RuntimeHash))
		{
			TArray<const FRuntimePartitionStreamingData*> StreamingDataList;

			for (const FRuntimePartitionStreamingData& StreamingData : RuntimeHashSet->RuntimeStreamingData)
			{
				StreamingDataList.Add(&StreamingData);
			}

			for (const TWeakObjectPtr<URuntimeHashExternalStreamingObjectBase>& InjectedExternalStreamingObject : RuntimeHashSet->InjectedExternalStreamingObjects)
			{
				if (InjectedExternalStreamingObject.IsValid())
				{
					URuntimeHashSetExternalStreamingObject* ExternalStreamingObject = CastChecked<URuntimeHashSetExternalStreamingObject>(InjectedExternalStreamingObject.Get());

					for (const FRuntimePartitionStreamingData& StreamingData : ExternalStreamingObject->RuntimeStreamingData)
					{
						FStreamingDescriptor::FStreamingGrid& StreamingGrid = StreamingGrids.FindOrAdd(StreamingData.Name);
						StreamingGrid.ExternalStreamingObjects.AddUnique(ExternalStreamingObject->GetPackagePathToCreate());

						StreamingDataList.Add(&StreamingData);
					}
				}
			}

			for (const FRuntimePartitionStreamingData* StreamingData : StreamingDataList)
			{
				FStreamingDescriptor::FStreamingGrid& StreamingGrid = StreamingGrids.FindOrAdd(StreamingData->Name);

				if (StreamingGrid.Name.IsNone())
				{
					StreamingGrid.Name = StreamingData->Name;
					StreamingGrid.CellSize = 0;
					StreamingGrid.LoadingRange = StreamingData->LoadingRange;

					if (const FRuntimePartitionDesc* RuntimePartitionDesc = RuntimeHashSet->RuntimePartitions.FindByPredicate([PartitionName = StreamingData->Name](const FRuntimePartitionDesc& RuntimePartitionDesc) { return RuntimePartitionDesc.Name == PartitionName; }))
					{
						if (const URuntimePartitionLHGrid* LHGrid = Cast<URuntimePartitionLHGrid>(RuntimePartitionDesc->MainLayer))
						{
							StreamingGrid.CellSize = LHGrid->CellSize;
						}
					}
				}

				auto AddCells = [DataLayerManager, &StreamingGrid](const TArray<TObjectPtr<UWorldPartitionRuntimeCell>>& Cells)
				{
					for (const TObjectPtr<UWorldPartitionRuntimeCell>& Cell : Cells)
					{
						FStreamingDescriptor::FStreamingCell& StreamingCell = *new(StreamingGrid.StreamingCells) FStreamingDescriptor::FStreamingCell;

						StreamingCell.Bounds = Cell->GetCellBounds();
						StreamingCell.bIsAlwaysLoaded = Cell->IsAlwaysLoaded();
						StreamingCell.bIsSpatiallyLoaded = Cell->IsSpatiallyLoaded();
						StreamingCell.DataLayers = Cell->GetDataLayers();
						StreamingCell.CellPackage = Cell->GetLevelPackageName();

						if (DataLayerManager)
						{
							for (FName& DataLayerName : StreamingCell.DataLayers)
							{
								if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromName(DataLayerName))
								{
									DataLayerName = *DataLayerInstance->GetDataLayerShortName();
								}
							}
						}

						const UWorldPartitionRuntimeLevelStreamingCell* LevelStreamingCell = CastChecked<const UWorldPartitionRuntimeLevelStreamingCell>(Cell);
						const TArray<FWorldPartitionRuntimeCellObjectMapping>& CellPackages = LevelStreamingCell->GetPackages();

						TArray<FStreamingDescriptor::FStreamingActor> Actors;
						StreamingCell.Actors.Reserve(CellPackages.Num());

						for (const FWorldPartitionRuntimeCellObjectMapping& CellPackage : CellPackages)
						{
							FStreamingDescriptor::FStreamingActor& StreaminActor = StreamingCell.Actors.AddDefaulted_GetRef();
							StreaminActor.BaseClass = CellPackage.BaseClass;
							StreaminActor.NativeClass = CellPackage.NativeClass;
							StreaminActor.Path = CellPackage.Path.ToString();
							StreaminActor.Package = CellPackage.Package;
							StreaminActor.ActorGuid = CellPackage.ActorInstanceGuid;
						}

						StreamingGrid.Bounds += StreamingCell.Bounds;
					}
				};

				AddCells(StreamingData->SpatiallyLoadedCells);
				AddCells(StreamingData->NonSpatiallyLoadedCells);
			}
		}

		StreamingGrids.GenerateValueArray(OutStreamingDescriptor.StreamingGrids);

		WorldPartition->FlushStreaming();
		return true;
	}

	return false;
}

TMap<FName, TArray<FStreamingDescriptor::FGeneratedPackage>> FStreamingDescriptor::GenerateWorldPluginPackagesList(UWorld* InWorld)
{
	TMap<FName, TArray<FGeneratedPackage>> PluginPackages;

	FStreamingDescriptor Descriptor;
	FStreamingDescriptor::GenerateStreamingDescriptor(InWorld, Descriptor);

	for (FStreamingDescriptor::FStreamingGrid& Grid : Descriptor.StreamingGrids)
	{
		for (FStreamingDescriptor::FStreamingCell& Cell : Grid.StreamingCells)
		{
			FName PluginName = FName(FPathViews::GetMountPointNameFromPath(Cell.CellPackage.ToString()));			
			FName PackageName = Cell.CellPackage;

			PluginPackages.FindOrAdd(PluginName).Add({PackageName, true});
		}

		for (FString& ExternalStreamingObject : Grid.ExternalStreamingObjects)
		{
			FName PluginName = FName(FPathViews::GetMountPointNameFromPath(ExternalStreamingObject));			
			FName PackageName = FName(ExternalStreamingObject);

			PluginPackages.FindOrAdd(PluginName).Add({PackageName, false});
		}
	}

	return PluginPackages;
}

};


static FAutoConsoleCommand GenerateWorldPluginPackagesList(
	TEXT("wp.editor.GenerateWorldPluginPackagesList"),
	TEXT("Generate the world plugin package list for the current world in the speficied file."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() == 1)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (!World->IsGameWorld())
				{
					if (FArchive* LogFile = IFileManager::Get().CreateFileWriter(*Args[0]))
					{
						TMap<FName, TArray<UE::Private::WorldPartition::FStreamingDescriptor::FGeneratedPackage>> PluginsPackageList = UE::Private::WorldPartition::FStreamingDescriptor::GenerateWorldPluginPackagesList(World);
						{
							FHierarchicalLogArchive Ar(*LogFile);

							for (TPair<FName, TArray<UE::Private::WorldPartition::FStreamingDescriptor::FGeneratedPackage>> PluginPackageList : PluginsPackageList)
							{
								UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Plugin: %s"), *PluginPackageList.Key.ToString()));

								for (UE::Private::WorldPartition::FStreamingDescriptor::FGeneratedPackage& GeneratedPackage : PluginPackageList.Value)
								{
									Ar.Printf(TEXT("Package: %s, IsLevelPackage: %i"), *GeneratedPackage.PackagePathName.ToString(), GeneratedPackage.bIsLevelPackage);
								}
							}
						}
						LogFile->Close();
						delete LogFile;
					}
				}
			}
		}
	})
);

#endif