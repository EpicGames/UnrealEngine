// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "Misc/HierarchicalLogArchive.h"
#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"

#include "Editor.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "Algo/Count.h"
#include "Containers/ArrayView.h"
#include "Engine/LevelScriptBlueprint.h"
#include "ActorReferencesUtils.h"
#include "Misc/PackageName.h"
#include "ReferenceCluster.h"
#include "Misc/HashBuilder.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionSettings.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/WorldPartitionLevelStreamingPolicy.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstanceNames.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationNullErrorHandler.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationLogErrorHandler.h"
#include "WorldPartition/HLOD/CustomHLODActor.h"
#include "WorldPartition/HLOD/HLODSourceCellPlaceholderActor.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODOnlyLevelInstanceActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/StandaloneHLODActor.h"
#include "WorldPartition/HLOD/StandaloneHLODSubsystem.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeContainerResolving.h"
#include "HAL/FileManager.h"
#include "Misc/EditorPathHelper.h"
#include "UObject/UObjectGlobalsInternal.h"

#define LOCTEXT_NAMESPACE "WorldPartition"

static bool GIsStreamingGenerationLogAllowed = true;
static FAutoConsoleVariableRef CVarIsStreamingGenerationLogAllowed(
	TEXT("wp.Editor.IsStreamingGenerationLogAllowed"),
	GIsStreamingGenerationLogAllowed,
	TEXT("Can be used to disable the streaming generation logs."),
	ECVF_Default);

static FAutoConsoleCommand DumpStreamingGenerationLog(
	TEXT("wp.Editor.DumpStreamingGenerationLog"),
	TEXT("Dump the streaming generation log."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (!GEditor)
		{
			UE_LOGF(LogWorldPartition, Error, "Streaming generation log can only be created in the editor");
		}
		else if (GEditor->GetPIEWorldContext())
		{
			UE_LOGF(LogWorldPartition, Error, "Streaming generation log can't be created during a PIE session");
		}
		else if (UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			if (UWorldPartition* WorldPartition = World->GetWorldPartition())
			{
				UWorldPartition::FGenerateStreamingParams Params = UWorldPartition::FGenerateStreamingParams()
					.SetOutputLogType(TEXT("DumpStreamingGeneration"));
				UWorldPartition::FGenerateStreamingContext Context;
				WorldPartition->GenerateStreaming(Params, Context);
				WorldPartition->FlushStreaming();

				if (Context.OutputLogFilename.IsSet())
				{
					UE_LOGF(LogWorldPartition, Display, "Streaming generation details logged to '%ls'", *FPaths::ConvertRelativePathToFull(Context.OutputLogFilename.GetValue()));
				}
			}
			else
			{
				UE_LOGF(LogWorldPartition, Warning, "Current world is not a partitioned world, no streaming generation log created");
			}
		}
	})
);

class FGCTempActorContainerInstances : public FGCObject
{
	friend class FStreamingGenerationTempActorDescInstance;
public:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObjects(ContainerInstances);
	}
	virtual FString GetReferencerName() const { return TEXT("FTempActorContainer"); }
	//~ End FGCObject interface

protected:
	TSet<TObjectPtr<UActorDescContainerInstance>> ContainerInstances;
};

class FStreamingGenerationTempActorDescInstance : public FWorldPartitionActorDescInstance
{
public:
	FStreamingGenerationTempActorDescInstance(UActorDescContainerInstance* InContainerInstance, AActor* InActor)
	{
		ContainerInstance = InContainerInstance;

		ActorDescPtr = InActor->CreateActorDesc();
		ActorDescPtr->SetContainer(InContainerInstance->GetContainer());
		ActorDesc = ActorDescPtr.Get();

		if (IsChildContainerInstance())
		{
			RegisterChildContainerInstance();
		}
	}

	void PropagateDataLayersFromSource(const FWorldPartitionActorDescInstance& InSourceDescInstance, const UDataLayerManager* InDataLayerManager, const TArray<const FWorldDataLayersActorDesc*>& InDataLayerResolvers)
	{
		const FDataLayerInstanceNames DataLayerInstanceNames = FDataLayerUtils::ResolveDataLayerInstanceNames(InDataLayerManager, InSourceDescInstance.GetActorDesc(), InDataLayerResolvers);
		SetDataLayerInstanceNames(DataLayerInstanceNames);
	}

	FStreamingGenerationTempActorDescInstance(FStreamingGenerationTempActorDescInstance&& InOther) = delete;
	FStreamingGenerationTempActorDescInstance& operator=(FStreamingGenerationTempActorDescInstance&& InOther) = delete;

	virtual ~FStreamingGenerationTempActorDescInstance()
	{
		if (IsChildContainerInstance())
		{
			UnregisterChildContainerInstance();
		}
	}

	static TUniquePtr<FStreamingGenerationTempActorDescInstance> Create(AActor* InActor, const FStreamingGenerationContainerInstanceCollection& InContainerInstanceCollection)
	{
		UActorDescContainerInstance* HandlingContainer = const_cast<UActorDescContainerInstance*>(InContainerInstanceCollection.FindHandlingContainerInstance(InActor).Get());
		check(HandlingContainer);
		return MakeUnique<FStreamingGenerationTempActorDescInstance>(HandlingContainer, InActor);
	}

	//~ Begin FWorldPartitionActorDescInstance interface
	virtual void RegisterChildContainerInstance() override
	{
		if (!TempActorContainerInstances.IsValid())
		{
			TempActorContainerInstances = MakeUnique<FGCTempActorContainerInstances>();
		}

		ChildContainerInstance = GetActorDesc()->CreateChildContainerInstance(this);
		TempActorContainerInstances->ContainerInstances.Add(ChildContainerInstance);
	}
	virtual void UnregisterChildContainerInstance() override
	{
		check(TempActorContainerInstances.IsValid())
		{
			TempActorContainerInstances->ContainerInstances.Remove(ChildContainerInstance);
			if (TempActorContainerInstances->ContainerInstances.IsEmpty())
			{
				TempActorContainerInstances.Reset();
			}
		}

		ChildContainerInstance->Uninitialize();
		ChildContainerInstance = nullptr;
	}
	//~ End FWorldPartitionActorDescInstance interface

protected:
	TUniquePtr<FWorldPartitionActorDesc> ActorDescPtr;
	static TUniquePtr<FGCTempActorContainerInstances> TempActorContainerInstances;
};

TUniquePtr<FGCTempActorContainerInstances> FStreamingGenerationTempActorDescInstance::TempActorContainerInstances;

template <class T>
class TErrorHandlerSelector
{
public:
	TErrorHandlerSelector(IStreamingGenerationErrorHandler* InErrorHandler = nullptr)
	{
		ErrorHandler = InErrorHandler ? InErrorHandler : &BaseErrorHandler;
	}

	IStreamingGenerationErrorHandler* Get()
	{
		if (UWorldPartition::StreamingGenerationErrorHandlerOverride)
		{
			return (*UWorldPartition::StreamingGenerationErrorHandlerOverride)(ErrorHandler);
		}
		return ErrorHandler;
	}

private:
	T BaseErrorHandler;
	IStreamingGenerationErrorHandler* ErrorHandler;
};

FName FStreamingGenerationActorDescView::GetRuntimeGrid() const
{
	if (bIsForcedNoRuntimeGrid)
	{
		return NAME_None;
	}

	if (ParentView)
	{
		return ParentView->GetRuntimeGrid();
	}

	if (ResolvedRuntimeGrid.IsSet())
	{
		return ResolvedRuntimeGrid.GetValue();
	}

	return Super::GetRuntimeGrid();
}

bool FStreamingGenerationActorDescView::GetIsSpatiallyLoaded() const
{
	if (bIsForcedNonSpatiallyLoaded)
	{
		return false;
	}

	if (ParentView)
	{
		return ParentView->GetIsSpatiallyLoaded();
	}

	if (bIsSpatiallyLoaded.IsSet())
	{
		return bIsSpatiallyLoaded.GetValue();
	}

	return Super::GetIsSpatiallyLoaded();
}

FSoftObjectPath FStreamingGenerationActorDescView::GetHLODLayer() const
{
	if (bIsForceNoHLODLayer)
	{
		return FSoftObjectPath();
	}

	if (RuntimeHLODLayer.IsSet())
	{
		return RuntimeHLODLayer.GetValue();
	}

	return Super::GetHLODLayer();
}

bool FStreamingGenerationActorDescView::GetActorIsHLODRelevant() const
{
	if (bIsForcedHLODIrrelevant)
	{
		return false;
	}

	if (bIsHLODRelevant.IsSet())
	{
		return bIsHLODRelevant.GetValue();
	}

	return Super::GetActorIsHLODRelevant();
}

const FDataLayerInstanceNames& FStreamingGenerationActorDescView::GetDataLayerInstanceNames() const
{
	if (!bIsForcedNoDataLayers && ParentView)
	{
		return ParentView->GetDataLayerInstanceNames();
	}

	if (ResolvedDataLayerInstanceNames.IsSet())
	{
		// ResolvedDataLayerInstanceNames contains the bIsForcedNoDataLayers information internally and will return an empty non-EDL array when requested.
		check(ResolvedDataLayerInstanceNames->IsForcedEmptyNonExternalDataLayers() == bIsForcedNoDataLayers);
		return ResolvedDataLayerInstanceNames.GetValue();
	}

	if (bIsForcedNoDataLayers)
	{
		// Build a FDataLayerInstanceNames containing only the EDL (if any) and cache the result as we need to return a ref
		LastReturnedDataLayerInstanceNames = FDataLayerInstanceNames(TArray<FName>(), Super::GetDataLayerInstanceNames().GetExternalDataLayer());
		return LastReturnedDataLayerInstanceNames;
	}

	return Super::GetDataLayerInstanceNames();
}

const TArray<FGuid>& FStreamingGenerationActorDescView::GetReferences() const
{
	return RuntimeReferences.IsSet() ? RuntimeReferences.GetValue() : Super::GetReferences();
}

bool FStreamingGenerationActorDescView::IsEditorOnlyReference(const FGuid& ReferenceGuid) const
{
	// We consider forced invalid references to be Editor-Only references as they will be skipped by streaming generation and PIE
	return Super::IsEditorOnlyReference(ReferenceGuid) || ForcedInvalidReference.Contains(ReferenceGuid);
}

FString FStreamingGenerationActorDescView::ToString(FWorldPartitionActorDesc::EToStringMode Mode) const
{
	return ToString(FStreamingGenerationActorDescView::EToStringMode::Full, Mode, [](const TArray<FName>&, TArray<const UDataLayerInstance*>&) { return false; });
}

FString FStreamingGenerationActorDescView::ToString(FStreamingGenerationActorDescView::EToStringMode ActorDescViewMode, FWorldPartitionActorDesc::EToStringMode ActorDescMode, TFunctionRef<bool(const TArray<FName>&, TArray<const UDataLayerInstance*>&)> GetRuntimeDataLayerInstancesFromNamesFunc) const
{
	FWorldPartitionActorDescStringBuilder Result(ActorDescMode);

	if (ActorDescMode >= FWorldPartitionActorDesc::EToStringMode::Compact)
	{
		if (GetIsSpatiallyLoaded() != GetActorDesc()->GetIsSpatiallyLoaded())
		{
			Result.AppendFromBool(TEXT("SpatiallyLoaded"), GetIsSpatiallyLoaded());
			if (bIsForcedNonSpatiallyLoaded)
			{
				Result.AppendFromBool(TEXT("IsForcedNonSpatiallyLoaded"), bIsForcedNonSpatiallyLoaded);
			}
		}

		if (!GetEditorBounds().Equals(GetActorDesc()->GetEditorBounds()))
		{
			Result.AppendToString(TEXT("EditorBounds"), GetEditorBounds());
		}

		if (!GetRuntimeBounds().Equals(GetActorDesc()->GetRuntimeBounds()))
		{
			Result.AppendToString(TEXT("RuntimeBounds"), GetRuntimeBounds());
		}

		if (GetRuntimeGrid() != GetActorDesc()->GetRuntimeGrid())
		{
			Result.AppendToString(TEXT("RuntimeGrid"), GetRuntimeGrid());
			if (bIsForcedNoRuntimeGrid)
			{
				Result.AppendFromBool(TEXT("IsForcedNoRuntimeGrid"), bIsForcedNoRuntimeGrid);
			}
		}

		if (GetHLODLayer() != GetActorDesc()->GetHLODLayer())
		{
			Result.AppendToString(TEXT("HLODLayer"), GetHLODLayer());
			if (bIsForceNoHLODLayer)
			{
				Result.AppendFromBool(TEXT("IsForceNoHLODLayer"), bIsForceNoHLODLayer);
			}
		}

		if (GetActorIsHLODRelevant() != GetActorDesc()->GetActorIsHLODRelevant())
		{
			Result.AppendFromBool(TEXT("HLODRelevant"), GetActorIsHLODRelevant());
			if (bIsForcedHLODIrrelevant)
			{
				Result.AppendFromBool(TEXT("IsForcedHLODIrrelevant"), bIsForcedHLODIrrelevant);
			}
		}

		if (ActorDescMode >= FWorldPartitionActorDesc::EToStringMode::Full)
		{
			TArray<const UDataLayerInstance*> DataLayerInstances;
			if (GetRuntimeDataLayerInstancesFromNamesFunc(GetRuntimeDataLayerInstanceNames().ToArray(), DataLayerInstances))
			{
				TArray<FName> DataLayerInstanceDisplayNames;
				Algo::Transform(DataLayerInstances, DataLayerInstanceDisplayNames, [&](const UDataLayerInstance* DataLayerInstance)
				{
					return FName(*DataLayerInstance->GetDataLayerFullName());
				});
				if ((DataLayerInstanceDisplayNames.Num() != GetActorDesc()->GetDataLayers().Num()) ||
					(!TSet(DataLayerInstanceDisplayNames).Includes(TSet(GetActorDesc()->GetDataLayers()))))
				{
					Result.AppendFromString(TEXT("DataLayers"), DataLayerInstanceDisplayNames.IsEmpty() ? TEXT("<Empty>") : FString::JoinBy(DataLayerInstanceDisplayNames, TEXT(","), [&](const FName& DataLayerName) { return DataLayerName.ToString(); }));
					if (bIsForcedNoDataLayers)
					{
						Result.AppendFromBool(TEXT("IsForcedNoDataLayers"), bIsForcedNoDataLayers);
					}
				}
			}
		}
	}

	FWorldPartitionActorDescStringBuilder FinalResult(ActorDescMode);
	if (ActorDescViewMode == FStreamingGenerationActorDescView::EToStringMode::ViewOnly)
	{
		if (Result.Len())
		{
			FinalResult.AppendToString(TEXT("Guid"), GetGuid(), false);
			FinalResult.Append(Result.ToString());
		}
		return FinalResult.ToString();
	}
	
	FinalResult.Append(Super::ToString(ActorDescMode));
	if (Result.Len())
	{
		FinalResult.Append(TEXT(" View=("));
		FinalResult.Append(Result.ToString());
		FinalResult.Append(TEXT(" )"));
	}
	return FinalResult.ToString();
}

const TArray<FGuid>& FStreamingGenerationActorDescView::GetEditorReferences() const
{
	return EditorReferences;
}

void FStreamingGenerationActorDescView::SetParentView(const FStreamingGenerationActorDescView* InParentView)
{
	check(!ParentView);
	check(!EditorOnlyParentTransform.IsSet());
	check(GetParentActor().IsValid());
	ParentView = InParentView;
}

void FStreamingGenerationActorDescView::SetEditorOnlyParentTransform(const FTransform& InEditorOnlyParentTransform)
{
	check(!ParentView);
	check(!EditorOnlyParentTransform.IsSet());
	check(GetParentActor().IsValid());
	EditorOnlyParentTransform = InEditorOnlyParentTransform;
}

void FStreamingGenerationActorDescView::SetDataLayerInstanceNames(const FDataLayerInstanceNames& InDataLayerInstanceNames)
{
	check(!Super::HasResolvedDataLayerInstanceNames());
	check(!InDataLayerInstanceNames.IsForcedEmptyNonExternalDataLayers())
	ResolvedDataLayerInstanceNames = InDataLayerInstanceNames;
	if (bIsForcedNoDataLayers)
	{
		ResolvedDataLayerInstanceNames->ForceEmptyNonExternalDataLayers();
	}
}

const FStreamingGenerationActorDescView::FInvalidReference* FStreamingGenerationActorDescView::GetInvalidReference(const FGuid& InGuid) const
{
	return ForcedInvalidReference.Find(InGuid);
}

void FStreamingGenerationActorDescView::AddForcedInvalidReference(const FStreamingGenerationActorDescView* ReferenceView)
{ 
	const FGuid& ReferenceGuid = ReferenceView->GetGuid();
	if (!ForcedInvalidReference.Contains(ReferenceGuid))
	{
		check(GetReferences().Contains(ReferenceGuid));
		FInvalidReference& InvalidReference = ForcedInvalidReference.Add(ReferenceGuid);
		InvalidReference.ActorPackage = ReferenceView->GetActorPackage();
		InvalidReference.ActorSoftPath = ReferenceView->GetActorSoftPath();
		InvalidReference.BaseClass = ReferenceView->GetBaseClass();
		InvalidReference.NativeClass = ReferenceView->GetNativeClass();
		UE_LOGF(LogWorldPartition, Verbose, "Actor '%ls' forced invalid reference %ls", *GetActorLabelOrName().ToString(), *ReferenceGuid.ToString());
	}
}

void FStreamingGenerationActorDescView::SetIsSpatiallyLoaded(bool bValue)
{
	bIsForcedNonSpatiallyLoaded = false;
	bIsSpatiallyLoaded = bValue;
}

void FStreamingGenerationActorDescView::SetForcedNonSpatiallyLoaded()
{
	if (!bIsForcedNonSpatiallyLoaded)
	{
		bIsForcedNonSpatiallyLoaded = true;
		UE_LOGF(LogWorldPartition, Verbose, "Actor '%ls' forced to be non-spatially loaded", *GetActorLabelOrName().ToString());
	}
}

void FStreamingGenerationActorDescView::SetForcedNoRuntimeGrid()
{
	if (!bIsForcedNoRuntimeGrid)
	{
		bIsForcedNoRuntimeGrid = true;
		UE_LOGF(LogWorldPartition, Verbose, "Actor '%ls' runtime grid invalidated", *GetActorLabelOrName().ToString());
	}
}

void FStreamingGenerationActorDescView::SetForcedNoDataLayers()
{
	if (!bIsForcedNoDataLayers)
	{
		bIsForcedNoDataLayers = true;
		UE_LOGF(LogWorldPartition, Verbose, "Actor '%ls' data layers invalidated", *GetActorLabelOrName().ToString());

		if (ResolvedDataLayerInstanceNames.IsSet())
		{
			ResolvedDataLayerInstanceNames->ForceEmptyNonExternalDataLayers();
		}

		if (RuntimeDataLayerInstanceNames.IsSet())
		{
			RuntimeDataLayerInstanceNames->ForceEmptyNonExternalDataLayers();
		}
	}
}

void FStreamingGenerationActorDescView::SetRuntimeDataLayerInstanceNames(const FDataLayerInstanceNames& InRuntimeDataLayerInstanceNames)
{
	check(!InRuntimeDataLayerInstanceNames.IsForcedEmptyNonExternalDataLayers() ||
		(GetRuntimeDataLayerInstanceNames().IsForcedEmptyNonExternalDataLayers() == InRuntimeDataLayerInstanceNames.IsForcedEmptyNonExternalDataLayers()));
	RuntimeDataLayerInstanceNames = InRuntimeDataLayerInstanceNames;
	if (bIsForcedNoDataLayers)
	{
		RuntimeDataLayerInstanceNames->ForceEmptyNonExternalDataLayers();
	}
}

void FStreamingGenerationActorDescView::SetRuntimeGrid(FName InRuntimeGrid)
{
	bIsForcedNoRuntimeGrid = false;
	ResolvedRuntimeGrid = InRuntimeGrid;
}

void FStreamingGenerationActorDescView::SetRuntimeReferences(const TArray<FGuid>& InRuntimeReferences)
{
	RuntimeReferences = InRuntimeReferences;
}

void FStreamingGenerationActorDescView::SetEditorReferences(const TArray<FGuid>& InEditorReferences)
{
	EditorReferences = InEditorReferences;
}

void FStreamingGenerationActorDescView::SetForcedNoHLODLayer()
{
	if (!bIsForceNoHLODLayer)
	{
		bIsForceNoHLODLayer = true;
		UE_LOGF(LogWorldPartition, Verbose, "Actor '%ls' HLOD layer invalidated", *GetActorLabelOrName().ToString());
	}
}

void FStreamingGenerationActorDescView::SetRuntimeHLODLayer(const FSoftObjectPath& InHLODLayer)
{
	bIsForceNoHLODLayer = false;
	RuntimeHLODLayer = InHLODLayer;
}

void FStreamingGenerationActorDescView::SetForcedHLODIrrelevant()
{
	bIsForcedHLODIrrelevant = true;
}

void FStreamingGenerationActorDescView::SetIsHLODRelevant(bool bValue)
{
	bIsForcedHLODIrrelevant = false;
	bIsHLODRelevant = bValue;
}

const FDataLayerInstanceNames& FStreamingGenerationActorDescView::GetRuntimeDataLayerInstanceNames() const
{
	if (!RuntimeDataLayerInstanceNames.IsSet())
	{
		static FDataLayerInstanceNames EmptyDataLayers;
		return EmptyDataLayers;
	}

	if (!bIsForcedNoDataLayers && ParentView)
	{
		return ParentView->GetRuntimeDataLayerInstanceNames();
	}

	// RuntimeDataLayerInstanceNames contains the bIsForcedNoDataLayers information internally and will return an empty non-EDL array when requested.
	check(RuntimeDataLayerInstanceNames->IsForcedEmptyNonExternalDataLayers() == bIsForcedNoDataLayers);
	return RuntimeDataLayerInstanceNames.GetValue();
}

FStreamingGenerationActorDescViewMap::FStreamingGenerationActorDescViewMap()
{}

TArray<const FStreamingGenerationActorDescView*> FStreamingGenerationActorDescViewMap::FindByExactNativeClass(UClass* InExactNativeClass) const
{
	check(InExactNativeClass->IsNative());
	const FName NativeClassName = InExactNativeClass->GetFName();
	TArray<const FStreamingGenerationActorDescView*> Result;
	ActorDescViewsByClass.MultiFind(NativeClassName, Result);
	return Result;
}

FStreamingGenerationActorDescView* FStreamingGenerationActorDescViewMap::Emplace(const FGuid& InGuid, FStreamingGenerationActorDescView&& InActorDescView)
{
	FStreamingGenerationActorDescView* NewActorDescView = ActorDescViewList.Emplace_GetRef(MakeUnique<FStreamingGenerationActorDescView>(MoveTemp(InActorDescView))).Get();
	NewActorDescView->ActorDescViewMap = this;

	const UClass* NativeClass = NewActorDescView->GetActorNativeClass();
	const FName NativeClassName = NativeClass->GetFName();

	ActorDescViewsByGuid.Emplace(InGuid, NewActorDescView);
	ActorDescViewsByClass.Add(NativeClassName, NewActorDescView);

	return NewActorDescView;
}

FStreamingGenerationActorDescView* FStreamingGenerationActorDescViewMap::Emplace(const FWorldPartitionActorDescInstance* InActorDescInstance)
{
	return Emplace(InActorDescInstance->GetGuid(), FStreamingGenerationActorDescView(*this, InActorDescInstance));
}

UWorldPartition::FCheckForErrorsParams::FCheckForErrorsParams()
	: ErrorHandler(nullptr)
	, bEnableStreaming(false)
	, ActorDescContainerInstanceCollection(nullptr)
{}

class FWorldPartitionStreamingGenerator
{
	class FStreamingGenerationContext : public IStreamingGenerationContext
	{
	public:
		FStreamingGenerationContext(const FWorldPartitionStreamingGenerator* StreamingGenerator, const FStreamingGenerationContainerInstanceCollection& TopLevelActorDescCollection)
		{
			// Create the dataset required for IStreamingGenerationContext interface
			ContextBaseContainerActorSetContainerInstanceIndex = INDEX_NONE;
			ActorSetContainerInstances.Empty(StreamingGenerator->ContainerCollectionInstanceDescriptorsMap.Num());

			TMap<TWeakPtr<FStreamingGenerationContainerInstanceCollection>, int32> ActorSetContainerMap;
			for (const auto& [ContainerID, ContainerDescriptor] : StreamingGenerator->ContainerCollectionInstanceDescriptorsMap)
			{
				const int32 ContainerIndex = ActorSetContainerInstances.AddDefaulted();
				ActorSetContainerMap.Add(ContainerDescriptor.ContainerInstanceCollection, ContainerIndex);

				FActorSetContainerInstance& ActorSetContainer = ActorSetContainerInstances[ContainerIndex];
				ActorSetContainer.ActorDescViewMap = &ContainerDescriptor.ActorDescViewMap.Get();
				ActorSetContainer.DataLayerResolvers = &ContainerDescriptor.DataLayerResolvers;
				ActorSetContainer.ContainerInstanceCollection = ContainerDescriptor.ContainerInstanceCollection.Get();

				ActorSetContainer.ActorSets.Empty(ContainerDescriptor.Clusters.Num());
				for (const TArray<FGuid>& Cluster : ContainerDescriptor.Clusters)
				{
					FActorSet& ActorSet = *ActorSetContainer.ActorSets.Add_GetRef(MakeUnique<FActorSet>()).Get();
					ActorSet.Actors = Cluster;
				}

				if (ContainerDescriptor.ContainerInstanceCollection->GetBaseContainerInstancePackageName() == TopLevelActorDescCollection.GetBaseContainerInstancePackageName())
				{
					check(ContextBaseContainerActorSetContainerInstanceIndex == INDEX_NONE);
					ContextBaseContainerActorSetContainerInstanceIndex = ContainerIndex;
				}
			}
			check(StreamingGenerator->ContainerCollectionInstanceDescriptorsMap.IsEmpty() || (ContextBaseContainerActorSetContainerInstanceIndex != INDEX_NONE));

			ActorSetInstances.Empty();
			for (const auto& [ContainerID, ContainerCollectionInstanceDescriptor] : StreamingGenerator->ContainerCollectionInstanceDescriptorsMap)
			{
				const FActorSetContainerInstance& ActorSetContainer = ActorSetContainerInstances[ActorSetContainerMap.FindChecked(ContainerCollectionInstanceDescriptor.ContainerInstanceCollection)];
				const TSet<FGuid>* FilteredActors = StreamingGenerator->ContainerFilteredActors.Find(ContainerID);
				for (const TUniquePtr<FActorSet>& ActorSetPtr : ActorSetContainer.ActorSets)
				{
					const FActorSet& ActorSet = *ActorSetPtr;
					const FStreamingGenerationActorDescView& ReferenceActorDescView = ActorSetContainer.ActorDescViewMap->FindByGuidChecked(ActorSet.Actors[0]);

					bool bContainsUnfilteredActors = !FilteredActors;

					if (!bContainsUnfilteredActors)
					{
						for (const FGuid& ActorGuid : ActorSet.Actors)
						{
							if (FilteredActors && !FilteredActors->Contains(ActorGuid))
							{
								bContainsUnfilteredActors = true;
								break;
							}
						}
					}

					// Skip if all actors are filtered out for this container
					if (bContainsUnfilteredActors)
					{
						FActorSetInstance& ActorSetInstance = ActorSetInstances.Emplace_GetRef();
						ActorSetInstance.ActorSetContainerInstance = &ActorSetContainer;
						ActorSetInstance.ActorSet = &ActorSet;
						ActorSetInstance.FilteredActors = FilteredActors;
						ActorSetInstance.ContainerID = ContainerCollectionInstanceDescriptor.ID;
						ActorSetInstance.Transform = ContainerCollectionInstanceDescriptor.Transform;
						ActorSetInstance.bIsSpatiallyLoaded = ReferenceActorDescView.GetIsSpatiallyLoaded();
						ActorSetInstance.ContentBundleID = ContainerCollectionInstanceDescriptor.ContentBundleID;
						ActorSetInstance.RuntimeGrid = ReferenceActorDescView.GetRuntimeGrid();
						ActorSetInstance.DataLayers = StreamingGenerator->GetRuntimeDataLayerInstances(ReferenceActorDescView.GetRuntimeDataLayerInstanceNames().ToArray());

						ActorSetInstance.Bounds.Init();
						ActorSetInstance.ForEachActor([this, &ActorSetInstance](const FGuid& ActorGuid)
						{
							const FStreamingGenerationActorDescView& ActorDescView = ActorSetInstance.ActorSetContainerInstance->ActorDescViewMap->FindByGuidChecked(ActorGuid);
							const FBox RuntimeBounds = ActorDescView.GetRuntimeBounds();
							if (RuntimeBounds.IsValid)
							{
								ActorSetInstance.Bounds += RuntimeBounds;
							}
						});
					}
				}
			}

			WorldBounds = StreamingGenerator->ContainerCollectionInstanceDescriptorsMap.FindChecked(FActorContainerID::GetMainContainerID()).Bounds;
		}

		virtual ~FStreamingGenerationContext()
		{}

		//~Begin IStreamingGenerationContext interface
		virtual FBox GetWorldBounds() const override
		{
			return WorldBounds;
		}

		virtual const FActorSetContainerInstance* GetActorSetContainerForContextBaseContainerInstance() const override
		{
			return ActorSetContainerInstances.IsValidIndex(ContextBaseContainerActorSetContainerInstanceIndex) ? &ActorSetContainerInstances[ContextBaseContainerActorSetContainerInstanceIndex] : nullptr;
		}

		virtual void ForEachActorSetInstance(TFunctionRef<void(const FActorSetInstance&)> Func) const override
		{
			for (const FActorSetInstance& ActorSetInstance : ActorSetInstances)
			{
				Func(ActorSetInstance);
			}
		}

		virtual void ForEachActorSetContainerInstance(TFunctionRef<void(const FActorSetContainerInstance&)> Func) const override
		{
			for (const FActorSetContainerInstance& ActorSetContainerInstance : ActorSetContainerInstances)
			{
				Func(ActorSetContainerInstance);
			}
		}
		//~End IStreamingGenerationContext interface};

	private:
		FBox WorldBounds;
		// Represents the index of the ActorSetContainerInstance (in the ActorSetContainerInstances array)
		// that contains the a BaseContainerInstance matching this context 
		// (the same BaseContainerInstance of the collection provided at the construction)
		int32 ContextBaseContainerActorSetContainerInstanceIndex;
		TArray<FActorSetContainerInstance> ActorSetContainerInstances;
		TArray<FActorSetInstance> ActorSetInstances;
	};

	/**
	 * An actor container instance descriptor, one for the main world and one for every actor container instance
	 */
	struct FContainerCollectionInstanceDescriptor
	{
		FContainerCollectionInstanceDescriptor()
			: Bounds(ForceInit)
			, bIsHLODRelevant(true)
		{}

		FBox Bounds;
		FTransform Transform;
		TSharedPtr<FStreamingGenerationContainerInstanceCollection> ContainerInstanceCollection;
		EContainerClusterMode ClusterMode;
		FString OwnerName;
		FActorContainerID ID;
		FActorContainerID ParentID;
		FGuid ContentBundleID;
		FSoftObjectPath HLODLayer;
		bool bIsHLODRelevant;

		FContainerCollectionInstanceDescriptor(const FContainerCollectionInstanceDescriptor& Other) = delete;
		FContainerCollectionInstanceDescriptor(FContainerCollectionInstanceDescriptor&& Other) = default;

		FContainerCollectionInstanceDescriptor& operator=(const FContainerCollectionInstanceDescriptor& Other) = delete;
		FContainerCollectionInstanceDescriptor& operator=(FContainerCollectionInstanceDescriptor&& Other) = default;

		/** The actor descriptor views for for this descriptor (TUniqueObj so it is moveable without having to update FStreamingGenerationActorDescView::ActorDescViewMap pointer */
		TUniqueObj<FStreamingGenerationActorDescViewMap> ActorDescViewMap;

		/** List of FWorldDataLayerActorDesc used to resolve Data Layers when DataLayerManager is not available */
		TArray<const FWorldDataLayersActorDesc*> DataLayerResolvers;

		/** Set of editor-only actors that are not part of the actor descriptor views */
		TSet<FGuid> EditorOnlyActorDescSet;

		/** List of actor descriptor views that are containers (mainly level instances) */
		TArray<FStreamingGenerationActorDescView> ContainerCollectionInstanceViews;

		/** List of unsaved/dirty descriptor views, unique ptrs so that they are moveable without having to update references to them from FStreamingGenerationActorDescView::ActorDescView */
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>> UnsavedDirtyInstances;

		/** List of Standalone HLOD actor descriptor views. */
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>> StandaloneHLODActorInstances;

		/** List of transient placeholder actors used to determine source cell for Custom HLOD actors */
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>> CustomHLODActorPlaceholderInstances;

		/** List of actor clusters for this descriptor */
		TArray<TArray<FGuid>> Clusters;

		struct FCombinedData
		{
			bool bIsSpatiallyLoaded;
			FName RuntimeGrid;
			FDataLayerInstanceNames RuntimeDataLayerInstanceNames;
		};

		/** Data that contains the result of combining parent's FCombinedData with ContainerCollectionInstanceView */
		FCombinedData ContainerCombinedData;

		/** Map of actor descriptor mutators */
		TMap<FGuid, FActorDescViewMutator> ActorDescViewMutators;

		/** Map of editor-only parent actor transforms */
		TMap<FGuid, FTransform> EditorOnlyParentActorTransforms;
	};

	void ResolveRuntimeSpatiallyLoaded(FStreamingGenerationActorDescView& ActorDescView)
	{
		if (!bEnableStreaming)
		{
			ActorDescView.SetForcedNonSpatiallyLoaded();
		}
	}

	void ResolveRuntimeGrid(FStreamingGenerationActorDescView& ActorDescView)
	{
		if (!bEnableStreaming)
		{
			ActorDescView.SetForcedNoRuntimeGrid();
		}
	}

	FName GetResolvedRuntimeGridForHLODLayer(FName RuntimeGrid, const FSoftObjectPath& HLODLayerPath) const
	{
		if (WorldPartitionContext && WorldPartitionContext->RuntimeHash)
		{
			FName GridName = RuntimeGrid.IsNone() ? DefaultGrid : RuntimeGrid;
			return WorldPartitionContext->RuntimeHash->ResolveRuntimeGridNameForHLODLayer(GridName, HLODLayerPath);
		}
		return NAME_None;
	}

	void ResolveRuntimeDataLayers(FStreamingGenerationActorDescView& ActorDescView, const TArray<const FWorldDataLayersActorDesc*>& InDataLayerResolvers)
	{
		// Resolve DataLayerInstanceNames of ActorDescView only when necessary (i.e. when container is a template)
		if (!ActorDescView.HasResolvedDataLayerInstanceNames())
		{
			// Build a WorldDataLayerActorDescs if DataLayerManager can't resolve Data Layers (i.e. when validating changelists and World is not loaded)
			static const TArray<const FWorldDataLayersActorDesc*> Empty;
			const bool bDataLayerManagerCanResolve = DataLayerManager && DataLayerManager->CanResolveDataLayers();
			const TArray<const FWorldDataLayersActorDesc*>& DataLayerResolvers = !bDataLayerManagerCanResolve ? InDataLayerResolvers : Empty;
			const FDataLayerInstanceNames DataLayerInstanceNames = FDataLayerUtils::ResolveDataLayerInstanceNames(DataLayerManager, ActorDescView.GetActorDesc(), DataLayerResolvers);
			ActorDescView.SetDataLayerInstanceNames(DataLayerInstanceNames);
		}

		FDataLayerInstanceNames RuntimeDataLayerInstanceNames;
		if (FDataLayerUtils::ResolveRuntimeDataLayerInstanceNames(DataLayerManager, ActorDescView, InDataLayerResolvers, RuntimeDataLayerInstanceNames))
		{
			ActorDescView.SetRuntimeDataLayerInstanceNames(RuntimeDataLayerInstanceNames);
		}
	}

	void ResolveHLODLayer(FStreamingGenerationActorDescView& ActorDescView, const FSoftObjectPath& ParentHLODLayer)
	{
		// Only assign the parent layer to actors that don't have a valid HLOD layer set and that are HLOD relevant. 
		// HLOD actors will have their parent HLOD layer set during HLOD generation.
		if (!ActorDescView.GetHLODLayer().IsValid() && ActorDescView.GetIsSpatiallyLoaded() && ActorDescView.GetActorIsHLODRelevant())
		{
			ActorDescView.SetRuntimeHLODLayer(ParentHLODLayer);
		}
	}

	void ResolveHLODRelevant(FStreamingGenerationActorDescView& ActorDescView, bool bIsHLODRelevant)
	{
		if (!bIsHLODRelevant)
		{
			ActorDescView.SetForcedHLODIrrelevant();
		}
	}
	
	void ResolveParentView(FStreamingGenerationActorDescView& ActorDescView, const FStreamingGenerationActorDescViewMap& ActorDescViewMap, const TSet<FGuid>& EditorOnlyActorDescSet, const TMap<FGuid, FTransform>& EditorOnlyParentActorTransforms)
	{
		if (FGuid ParentGuid = ActorDescView.GetParentActor(); ParentGuid.IsValid())
		{
			if (const FStreamingGenerationActorDescView* ParentView = ActorDescViewMap.FindByGuid(ParentGuid))
			{
				ActorDescView.SetParentView(ParentView);
			}
			else if (const FTransform* EditorOnlyParentActorTransform = EditorOnlyParentActorTransforms.Find(ParentGuid))
			{
				ActorDescView.SetEditorOnlyParentTransform(*EditorOnlyParentActorTransform);
			}
		}
	}

	void CreateActorDescViewMap(FContainerCollectionInstanceDescriptor& InContainerCollectionInstanceDescriptor)
	{
		const FStreamingGenerationContainerInstanceCollection& InActorDescCollection = *InContainerCollectionInstanceDescriptor.ContainerInstanceCollection;
		FStreamingGenerationActorDescViewMap& OutActorDescViewMap = InContainerCollectionInstanceDescriptor.ActorDescViewMap.Get();
		TArray<const FWorldDataLayersActorDesc*>& OutDataLayerResolvers = InContainerCollectionInstanceDescriptor.DataLayerResolvers;
		TSet<FGuid>& OutEditorOnlyActorDescSet = InContainerCollectionInstanceDescriptor.EditorOnlyActorDescSet;
		const FActorContainerID& InContainerID = InContainerCollectionInstanceDescriptor.ID;
		TArray<FStreamingGenerationActorDescView>& OutContainerInstances = InContainerCollectionInstanceDescriptor.ContainerCollectionInstanceViews;
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>>& OutUnsavedDirtyInstances = InContainerCollectionInstanceDescriptor.UnsavedDirtyInstances;
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>>& OutStandaloneHLODActorInstances = InContainerCollectionInstanceDescriptor.StandaloneHLODActorInstances;
		TArray<TUniquePtr<FStreamingGenerationTempActorDescInstance>>& OutCustomHLODActorPlaceholderInstances = InContainerCollectionInstanceDescriptor.CustomHLODActorPlaceholderInstances;
		TMap<FGuid, FTransform>& EditorOnlyParentActorTransforms = InContainerCollectionInstanceDescriptor.EditorOnlyParentActorTransforms;

		// Custom HLOD placeholders need their data layers propagated after the main iteration completes, so OutDataLayerResolvers is fully populated.
		TArray<TPair<FStreamingGenerationTempActorDescInstance*, const FWorldPartitionActorDescInstance*>> PendingCustomHLODDataLayerPropagation;

		// Should we handle unsaved or newly created actors?
		const bool bShouldHandleUnsavedActors = bHandleUnsavedActors && InContainerID.IsMainContainer();

		// Consider all actors of a /Temp/ container package as Unsaved because loading them from disk will fail (Outer world name mismatch)
		const bool bIsTempContainerPackage = FPackageName::IsTempPackage((InActorDescCollection.GetBaseContainerInstancePackageName().ToString()));

		// Test whether an actor descriptor instance should be included in the ActorDescViewMap.
		auto ShouldRegisterActorDesc = [this](const FWorldPartitionActorDescInstance* InActorDescInstance, AActor** OutActor = nullptr)
		{
			for (UClass* FilteredClass : FilteredClasses)
			{
				if (InActorDescInstance->GetActorNativeClass()->IsChildOf(FilteredClass))
				{
					return false;
				}
			}

			if (!InActorDescInstance->IsRuntimeRelevant())
			{
				return false;
			}

			if (AActor* Actor = InActorDescInstance->GetActor())
			{
				if (OutActor)
				{
					if (bIsPIE && Actor->bIgnoreInPIE)
					{
						return false;
					}

					*OutActor = Actor;
				}

				return !Actor->IsEditorOnly() || (bIsPIE && Actor->IsEditorOnlyLoadedInPIE());
			}

			return !InActorDescInstance->GetActorIsEditorOnly() || (bIsPIE && InActorDescInstance->GetActorIsEditorOnlyLoadedInPIE());
		};

		// Register the actor descriptor view
		auto RegisterActorDescView = [this, &OutActorDescViewMap, &OutContainerInstances](FStreamingGenerationActorDescView&& InActorDescView)
		{
			if (InActorDescView.IsChildContainerInstance())
			{
				OutContainerInstances.Add(InActorDescView);
			}
			else
			{
				const FGuid ActorGuid = InActorDescView.GetGuid();
				OutActorDescViewMap.Emplace(ActorGuid, MoveTemp(InActorDescView));
			}
		};

		for (FStreamingGenerationContainerInstanceCollection::TConstIterator<> Iterator(&InActorDescCollection); Iterator; ++Iterator)
		{
			// @todo_ow: this is to validate that new parenting of container instance code is equivalent
			check(Iterator->GetContainerInstance()->GetContainerID() == InContainerID);

			if (Iterator->GetActorNativeClass()->IsChildOf<AWorldDataLayers>())
			{
				const FWorldDataLayersActorDesc* WorldDataLayersActorDesc = (const FWorldDataLayersActorDesc*)Iterator->GetActorDesc();
				if (WorldDataLayersActorDesc->IsValid())
				{
					OutDataLayerResolvers.Add(WorldDataLayersActorDesc);
				}
			}

			if (Iterator->GetActorNativeClass()->IsChildOf<AWorldPartitionCustomHLOD>())
			{
				if (WorldPartitionContext)
				{
					// Custom HLOD actors, unlike regular HLOD actors don't have their SourceCell set during creation, so we need to compute and store that information during streaming generation
					// We're doing this by adding a placeholder actor descriptor for each custom HLOD actor. Placeholder actor has the same transform, bounds, etc. as the Custom HLOD actor.
					// Later, when placing actors in the appropriate cells we'll save that information and remove the placeholder actors.
					// The easiest way to add actor desc view is to spawn an actual actor and use the same logic that is used to handle unsaved dirty actors. 
					// Right after we have actor desc view, we can destroy actor to avoid undesired side effects.
					// To keep streaming generation deterministic, we use a deterministic actor GUID and name.
					FActorSpawnParameters SpawnParameters;
					SpawnParameters.ObjectFlags = RF_Transient;
					SpawnParameters.bCreateActorPackage = false;
					SpawnParameters.OverrideActorGuid = FGuid::NewDeterministicGuid(TEXT("CustomHLODPlaceholder") + Iterator->GetActorSoftPath().ToString());
					SpawnParameters.Name = FName(SpawnParameters.OverrideActorGuid.ToString());
					SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

					// Spawn placeholder actor and initialize properties based on Custom HLOD actor
					AWorldPartitionHLODSourceCellPlaceholder* HLODPlaceholderActor = WorldPartitionContext->GetWorld()->SpawnActor<AWorldPartitionHLODSourceCellPlaceholder>(SpawnParameters);
					check(HLODPlaceholderActor);
					HLODPlaceholderActor->InitFrom(*Iterator);
					HLODPlaceholderActor->SetPlaceholderType(EHLODSourceCellPlaceholderType::CustomHLOD);

					if (UHLODLayer* HLODLayer = Cast<UHLODLayer>(Iterator->GetHLODLayer().TryLoad()))
					{
						if (HLODLayer->GetLayerType() == EHLODLayerType::CustomHLODActor)
						{
							if (UHLODLayer* LinkedHLODLayer = HLODLayer->GetLinkedLayer())
							{
								FName ResolvedRuntimeGrid = GetResolvedRuntimeGridForHLODLayer(HLODPlaceholderActor->GetRuntimeGrid(), LinkedHLODLayer);
								if (!ResolvedRuntimeGrid.IsNone())
								{
									HLODPlaceholderActor->SetRuntimeGrid(ResolvedRuntimeGrid);
								}
								else
								{
									HLODPlaceholderActor->SetIsSpatiallyLoaded(false);
								}
							}
						}
					}
				
					// Register placeholder actor desc view
					TUniquePtr<FStreamingGenerationTempActorDescInstance>& HLODPlaceholderRef = OutCustomHLODActorPlaceholderInstances.Add_GetRef(MakeUnique<FStreamingGenerationTempActorDescInstance>(Iterator->GetContainerInstance(), HLODPlaceholderActor));
					PendingCustomHLODDataLayerPropagation.Emplace(HLODPlaceholderRef.Get(), *Iterator);
					RegisterActorDescView(FStreamingGenerationActorDescView(OutActorDescViewMap, HLODPlaceholderRef.Get(), false));

					// Destroy placeholder actor since we only need its actor descriptor.
					WorldPartitionContext->GetWorld()->DestroyActor(HLODPlaceholderActor, false, false);
					TrashObject(HLODPlaceholderActor);
				}

				// If Custom HLOD actor is used with some HLOD layer different than CustomHLODActor layer don't register its actor descriptor
				if (bIncludeCustomHLODActorsWithOtherLayerType == false)
				{
					if (UHLODLayer* HLODLayer = Cast<UHLODLayer>(Iterator->GetHLODLayer().TryLoad()))
					{
						if (HLODLayer->GetLayerType() != EHLODLayerType::CustomHLODActor)
						{
							continue;
						}
					}
				}
			}

			AActor* Actor = nullptr;
			if (ShouldRegisterActorDesc(*Iterator, &Actor))
			{
				// Handle unsaved actors
				if (Actor)
				{
					// Deleted actors
					if (!IsValid(Actor))
					{
						continue;
					}

					// Dirty actors
					if (bShouldHandleUnsavedActors && (bIsTempContainerPackage || (Actor->GetPackage()->IsDirty() || Actor->ShouldForceDuplicateInPIE() || Actor->HasAllFlags(RF_Transient | RF_NonPIEDuplicateTransient))))
					{
						// Dirty, unsaved actor for PIE
						TUniquePtr<FStreamingGenerationTempActorDescInstance>& UnsavedDirtyRef = OutUnsavedDirtyInstances.Add_GetRef(FStreamingGenerationTempActorDescInstance::Create(Actor, InActorDescCollection));
						RegisterActorDescView(FStreamingGenerationActorDescView(OutActorDescViewMap, UnsavedDirtyRef.Get(), true));
						continue;
					}
				}

				// Non-dirty actor
				RegisterActorDescView(FStreamingGenerationActorDescView(OutActorDescViewMap, *Iterator));
			}
			else
			{
				OutEditorOnlyActorDescSet.Add(Iterator->GetGuid());
			}
		}

		for (const TPair<FStreamingGenerationTempActorDescInstance*, const FWorldPartitionActorDescInstance*>& Pending : PendingCustomHLODDataLayerPropagation)
		{
			Pending.Key->PropagateDataLayersFromSource(*Pending.Value, DataLayerManager, OutDataLayerResolvers);
		}

		// Register transforms from editor-only parents as the childs won't be properly offset if they are not present
		for (FStreamingGenerationContainerInstanceCollection::TConstIterator<> Iterator(&InActorDescCollection); Iterator; ++Iterator)
		{
			if (FGuid ParentGuid = Iterator->GetParentActor(); ParentGuid.IsValid())
			{
				if (OutEditorOnlyActorDescSet.Contains(ParentGuid) && !EditorOnlyParentActorTransforms.Contains(ParentGuid))
				{
					const FWorldPartitionActorDescInstance* ParentActorDescInstance = InActorDescCollection.GetActorDescInstance(ParentGuid);
					FTransform ParentActorDescInstanceTransform = ParentActorDescInstance->GetActorTransform();
					
					// Dirty actors
					if (AActor* Actor = ParentActorDescInstance->GetActor())
					{
						if (bShouldHandleUnsavedActors && (bIsTempContainerPackage || Actor->GetPackage()->IsDirty()))
						{
							ParentActorDescInstanceTransform = Actor->GetActorTransform();
						}
					}

					EditorOnlyParentActorTransforms.Add(ParentGuid, ParentActorDescInstanceTransform);
				}
			}
		}

		// Append new unsaved actors for the persistent level
		if (bShouldHandleUnsavedActors)
		{
			for (AActor* Actor : InActorDescCollection.GetWorld()->PersistentLevel->Actors)
			{
				// Here, FindHandlingContainer is used to make sure that the actor is handled by the collection
				// The main reason is that UWorldPartition::CheckForErrors currently builds a collection per ActorDescContainer of the WorldPartition.
				// This is probably a limitation introduced by ContentBundles. 
				if (IsValid(Actor)
					&& ((Actor->IsPackageExternal() && Actor->IsMainPackageActor()) || Actor->HasAllFlags(RF_Transient | RF_NonPIEDuplicateTransient))
					&& (!Actor->IsEditorOnly() || (bIsPIE && Actor->IsEditorOnlyLoadedInPIE()))
					&& InActorDescCollection.FindHandlingContainerInstance(Actor)
					&& !InActorDescCollection.GetActorDescInstance(Actor->GetActorGuid()))
				{
					TUniquePtr<FStreamingGenerationTempActorDescInstance> UnsavedViewPtr = FStreamingGenerationTempActorDescInstance::Create(Actor, InActorDescCollection);
					if (ShouldRegisterActorDesc(UnsavedViewPtr.Get()))
					{
						FStreamingGenerationActorDescView ModifiedActorDescView(OutActorDescViewMap, UnsavedViewPtr.Get(), true);
						OutUnsavedDirtyInstances.Add(MoveTemp(UnsavedViewPtr));
						RegisterActorDescView(MoveTemp(ModifiedActorDescView));
					}
				}
			}
		}

		if (bIncludeStandaloneHLODActors && InContainerID.IsMainContainer())
		{
			if (WorldPartitionStandaloneHLODSubsystem)
			{
				// Standalone HLOD actors
				WorldPartitionStandaloneHLODSubsystem->ForEachStandaloneHLODActorWithLevelInstanceGuid([this, &OutStandaloneHLODActorInstances, &OutCustomHLODActorPlaceholderInstances, &OutActorDescViewMap, &OutDataLayerResolvers, &InActorDescCollection, &RegisterActorDescView](AWorldPartitionStandaloneHLOD* StandaloneHLODActor, const FGuid& LevelInstanceGuid)
				{
					if (InActorDescCollection.FindHandlingContainerInstance(StandaloneHLODActor))
					{
						TUniquePtr<FStreamingGenerationTempActorDescInstance>& StandaloneHLODActorRef = OutStandaloneHLODActorInstances.Add_GetRef(FStreamingGenerationTempActorDescInstance::Create(StandaloneHLODActor, InActorDescCollection));
						RegisterActorDescView(FStreamingGenerationActorDescView(OutActorDescViewMap, StandaloneHLODActorRef.Get(), false));

						// Save information about "dependencies" between Standalone HLOD actors. We want all external child HLOD actors of a Standalone HLOD actor to end up in the same streaming cell
						// to maintain HLODActor -> SourceCell mapping, that is used for regular HLODActors to control visibility / loading state.
						// StandaloneHLODActorsDependencies saved here will be used later during clusters generation.
						if (UActorDescContainer* StandaloneHLODContainer = StandaloneHLODActorRef->GetActorDesc()->GetChildContainer())
						{
							for (UActorDescContainer::TConstIterator<AWorldPartitionHLOD> Iterator(StandaloneHLODContainer); Iterator; ++Iterator)
							{
								const FHLODActorDesc* HLODActorDesc = static_cast<const FHLODActorDesc*>(*Iterator);
								if (HLODActorDesc->GetExternalChildHLODActors().Num() > 1)
								{
									const TArray<FGuid>& ExternalChildHLODActors = HLODActorDesc->GetExternalChildHLODActors();
									for (int32 ExternalChildHLODActorIndex = 0; ExternalChildHLODActorIndex < ExternalChildHLODActors.Num(); ExternalChildHLODActorIndex++)
									{
										for (int32 OtherExternalChildHLODActorIndex = ExternalChildHLODActorIndex + 1; OtherExternalChildHLODActorIndex < ExternalChildHLODActors.Num(); OtherExternalChildHLODActorIndex++)
										{
											StandaloneHLODActorsDependencies.FindOrAdd(ExternalChildHLODActors[ExternalChildHLODActorIndex]).Add(ExternalChildHLODActors[OtherExternalChildHLODActorIndex]);
											StandaloneHLODActorsDependencies.FindOrAdd(ExternalChildHLODActors[OtherExternalChildHLODActorIndex]).Add(ExternalChildHLODActors[ExternalChildHLODActorIndex]);
										}
									}
								}
							}
						}

						// For HLOD-only Level Instances, create placeholders for HLOD0 actors so they get mapped to the containing map's MainGrid cells.
						// HLOD1+ actors are already handled via ExternalChildHLODActors (in RuntimeHashSetStreamingGeneration).
						// Without this, HLOD0 actors fall through to the orphan path at runtime and never hide when the containing map's content cells load.
						if (const FWorldPartitionActorDescInstance* LevelInstanceActorDescInstance = WorldPartitionContext->GetActorDescInstance(LevelInstanceGuid))
						{
							if (LevelInstanceActorDescInstance->GetActorNativeClass()->IsChildOf<AWorldPartitionHLODOnlyLevelInstance>())
							{
								if (UActorDescContainerInstance* StandaloneHLODContainerInstance = StandaloneHLODActorRef->GetChildContainerInstance())
								{
									for (UActorDescContainerInstance::TConstIterator<AWorldPartitionHLOD> Iterator(StandaloneHLODContainerInstance); Iterator; ++Iterator)
									{
										const FWorldPartitionActorDescInstance* HLODActorDescInstance = *Iterator;
										const FHLODActorDesc* HLODActorDesc = static_cast<const FHLODActorDesc*>(HLODActorDescInstance->GetActorDesc());

										// Only HLOD0 actors (no external child references). HLOD1+ have non-empty ExternalChildHLODActors.
										if (!HLODActorDesc->GetExternalChildHLODActors().IsEmpty())
										{
											continue;
										}

										FActorSpawnParameters SpawnParameters;
										SpawnParameters.ObjectFlags = RF_Transient;
										SpawnParameters.bCreateActorPackage = false;
										SpawnParameters.OverrideActorGuid = FGuid::NewDeterministicGuid(TEXT("StandaloneHLODPlaceholder") + HLODActorDescInstance->GetActorSoftPath().ToString());
										SpawnParameters.Name = FName(SpawnParameters.OverrideActorGuid.ToString());
										SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

										AWorldPartitionHLODSourceCellPlaceholder* HLODPlaceholderActor = WorldPartitionContext->GetWorld()->SpawnActor<AWorldPartitionHLODSourceCellPlaceholder>(SpawnParameters);
										check(HLODPlaceholderActor);
										HLODPlaceholderActor->InitFrom(HLODActorDescInstance);
										HLODPlaceholderActor->SetPlaceholderType(EHLODSourceCellPlaceholderType::StandaloneHLOD);
										// Override runtime grid to NAME_None so the placeholder lands in the containing map's MainGrid rather than the Level Instance's internal HLOD grid.
										HLODPlaceholderActor->SetRuntimeGrid(NAME_None);
										HLODPlaceholderActor->SetIsSpatiallyLoaded(true);

										TUniquePtr<FStreamingGenerationTempActorDescInstance>& HLODPlaceholderRef = OutCustomHLODActorPlaceholderInstances.Add_GetRef(MakeUnique<FStreamingGenerationTempActorDescInstance>(StandaloneHLODActorRef->GetContainerInstance(), HLODPlaceholderActor));
										HLODPlaceholderRef->PropagateDataLayersFromSource(*HLODActorDescInstance, DataLayerManager, OutDataLayerResolvers);
										RegisterActorDescView(FStreamingGenerationActorDescView(OutActorDescViewMap, HLODPlaceholderRef.Get(), false));

										WorldPartitionContext->GetWorld()->DestroyActor(HLODPlaceholderActor, false, false);
										TrashObject(HLODPlaceholderActor);
									}
								}
							}
						}
					}
				});
			}
		}
	}
	
	void CreateActorDescriptorViewsRecursive(FContainerCollectionInstanceDescriptor&& InContainerCollectionInstanceDescriptor)
	{
		// Inherited parent per-instance data logic
		auto InheritParentContainerData = [this](const FContainerCollectionInstanceDescriptor& ParentContainerCollectionInstanceDescriptor, const FStreamingGenerationActorDescView& InActorDescView)
		{
			const FContainerCollectionInstanceDescriptor::FCombinedData& InParentData = ParentContainerCollectionInstanceDescriptor.ContainerCombinedData;
			FContainerCollectionInstanceDescriptor::FCombinedData OutCombinedData;

			// Apply AND logic on spatially loaded flag
			OutCombinedData.bIsSpatiallyLoaded = InActorDescView.GetIsSpatiallyLoaded() && InParentData.bIsSpatiallyLoaded;

			// Runtime grid is inherited from the main world if the actor has its runtime grid set to none.
			OutCombinedData.RuntimeGrid = (ParentContainerCollectionInstanceDescriptor.ID.IsMainContainer() || InParentData.RuntimeGrid.IsNone()) ? InActorDescView.GetRuntimeGrid() : InParentData.RuntimeGrid;

			// Set RuntimeGrid for Custom HLOD Actor based on actor's HLOD Layer to have it directly included in the appropriate grid
			if (InActorDescView.GetActorNativeClass()->IsChildOf<AWorldPartitionCustomHLOD>())
			{
				if (UHLODLayer* HLODLayer = Cast<UHLODLayer>(InActorDescView.GetHLODLayer().TryLoad()))
				{
					if (HLODLayer->GetLayerType() == EHLODLayerType::CustomHLODActor)
					{
						FName ResolvedRuntimeGrid = GetResolvedRuntimeGridForHLODLayer(OutCombinedData.RuntimeGrid, InActorDescView.GetHLODLayer());
						if (!ResolvedRuntimeGrid.IsNone())
						{
							OutCombinedData.RuntimeGrid = ResolvedRuntimeGrid;
						}
						else
						{
							OutCombinedData.bIsSpatiallyLoaded = false;
						}
					}
				}
			}

			// Data layers are accumulated down the hierarchy chain, since level instances supports data layers assignation on actors
			OutCombinedData.RuntimeDataLayerInstanceNames = InActorDescView.GetRuntimeDataLayerInstanceNames();
			OutCombinedData.RuntimeDataLayerInstanceNames.Append(InParentData.RuntimeDataLayerInstanceNames);
			return OutCombinedData;
		};

		// Hold on to ID
		const FActorContainerID ContainerID = InContainerCollectionInstanceDescriptor.ID;
		{
			TArray<FStreamingGenerationActorDescView> ContainerCollectionInstanceViews;

			// ContainerInstanceDescriptor may be reallocated after this scope
			{
				// Create container instance descriptor
				check(!ContainerCollectionInstanceDescriptorsMap.Contains(ContainerID));

				FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.Add(ContainerID, MoveTemp(InContainerCollectionInstanceDescriptor));
				
				// Gather actor descriptor views for this container
				CreateActorDescViewMap(ContainerCollectionInstanceDescriptor);

				// Resolve actor descriptor views before validation
				ResolveContainerDescriptor(ContainerCollectionInstanceDescriptor);

				// Validate container, fixing anything illegal, etc.
				ValidateContainerDescriptor(ContainerCollectionInstanceDescriptor);

				// Update container, computing cluster, bounds, etc.
				UpdateContainerDescriptor(ContainerCollectionInstanceDescriptor);

				// Calculate Bounds of non-container actor descriptor views
				check(!ContainerCollectionInstanceDescriptor.Bounds.IsValid);
				ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([&ContainerCollectionInstanceDescriptor](const FStreamingGenerationActorDescView& ActorDescView)
				{
					if (ActorDescView.GetIsSpatiallyLoaded())
					{
						const FBox RuntimeBounds = ActorDescView.GetRuntimeBounds();
						// Test if RuntimeBounds is valid because GetIsSpatiallyLoaded() is affected by a valid ParentView
						// So the RuntimeBounds can be invalid in the case where its a non-spatial with a spatial parent.
						if (RuntimeBounds.IsValid)
						{
							ContainerCollectionInstanceDescriptor.Bounds += RuntimeBounds;
						}
					}
				});

				// Copy list as descriptor might get reallocated after this scope
				ContainerCollectionInstanceViews.Append(ContainerCollectionInstanceDescriptor.ContainerCollectionInstanceViews);
			}

			// Parse actor containers
			for (const FStreamingGenerationActorDescView& ContainerCollectionInstanceView : ContainerCollectionInstanceViews)
			{
				FWorldPartitionActorDesc::FContainerInstance SubContainerInstance;
				if (!ContainerCollectionInstanceView.GetChildContainerInstance(SubContainerInstance) || !SubContainerInstance.ContainerInstance)
				{
					continue;
				}

				FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.FindChecked(ContainerID);
				FContainerCollectionInstanceDescriptor SubContainerInstanceDescriptor;

				SubContainerInstanceDescriptor.ID = SubContainerInstance.ContainerInstance->GetContainerID();
				check(SubContainerInstanceDescriptor.ID == FActorContainerID(ContainerCollectionInstanceDescriptor.ID, ContainerCollectionInstanceView.GetGuid()));

				// @todo_ow: LevelInstance EDL Support
				// LevelInstance don't support Content Bundle containers nor EDL containers
				ensure(!SubContainerInstance.ContainerInstance->HasExternalContent());
				FStreamingGenerationContainerInstanceCollection SubContainerInstanceCollection({ SubContainerInstance.ContainerInstance }, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs);
				SubContainerInstanceDescriptor.ContainerInstanceCollection = MakeShared<FStreamingGenerationContainerInstanceCollection>(SubContainerInstanceCollection);
				SubContainerInstanceDescriptor.Transform = SubContainerInstance.ContainerInstance->GetTransform();

				// @todo_ow: this is to validate that new parenting of container instance code is equivalent
				const FTransform ValidationTransform = SubContainerInstance.Transform * ContainerCollectionInstanceDescriptor.Transform;
				check(SubContainerInstanceDescriptor.Transform.Equals(ValidationTransform));

				SubContainerInstanceDescriptor.ParentID = ContainerCollectionInstanceDescriptor.ID;
				SubContainerInstanceDescriptor.OwnerName = *ContainerCollectionInstanceView.GetActorLabelOrName().ToString();
				// Since Content Bundles streaming generation happens in its own context, all actor set instances must have the same content bundle GUID for now, so Level Instances
				// placed inside a Content Bundle will propagate their Content Bundle GUID to child instances.
				SubContainerInstanceDescriptor.ContentBundleID = ContainerCollectionInstanceDescriptor.ContentBundleID;
				SubContainerInstanceDescriptor.ContainerCombinedData = InheritParentContainerData(ContainerCollectionInstanceDescriptor, ContainerCollectionInstanceView);
				SubContainerInstanceDescriptor.HLODLayer = ContainerCollectionInstanceView.GetHLODLayer().IsValid() ? ContainerCollectionInstanceView.GetHLODLayer() : ContainerCollectionInstanceDescriptor.HLODLayer;
				SubContainerInstanceDescriptor.bIsHLODRelevant = ContainerCollectionInstanceView.GetActorIsHLODRelevant() && ContainerCollectionInstanceDescriptor.bIsHLODRelevant;

				if (WorldPartitionSubsystem && ContainerID.IsMainContainer() && ContainerCollectionInstanceView.GetChildContainerFilterType() == EWorldPartitionActorFilterType::Loading)
				{
					if (const FWorldPartitionActorFilter* ContainerFilter = ContainerCollectionInstanceView.GetChildContainerFilter())
					{
						ContainerFilteredActors.Append(WorldPartitionSubsystem->GetFilteredActorsPerContainer(SubContainerInstanceDescriptor.ID, ContainerCollectionInstanceView.GetChildContainerPackage().ToString(), *ContainerFilter));
					}
				}

				CreateActorDescriptorViewsRecursive(MoveTemp(SubContainerInstanceDescriptor));
			}
		}

		// Fetch the versions stored in the map as it can have been reallocated during recursion
		FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.FindChecked(ContainerID);

		if (!ContainerID.IsMainContainer())
		{
			FContainerCollectionInstanceDescriptor& ParentContainerCollection = ContainerCollectionInstanceDescriptorsMap.FindChecked(ContainerCollectionInstanceDescriptor.ParentID);
			ParentContainerCollection.Bounds += ContainerCollectionInstanceDescriptor.Bounds;
		}

		// Apply parent container data
		ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([&ContainerCollectionInstanceDescriptor, &InheritParentContainerData](FStreamingGenerationActorDescView& ActorDescView)
		{
			const FContainerCollectionInstanceDescriptor::FCombinedData CombinedData = InheritParentContainerData(ContainerCollectionInstanceDescriptor, ActorDescView);
			ActorDescView.SetIsSpatiallyLoaded(CombinedData.bIsSpatiallyLoaded);
			ActorDescView.SetRuntimeGrid(CombinedData.RuntimeGrid);
			ActorDescView.SetRuntimeDataLayerInstanceNames(CombinedData.RuntimeDataLayerInstanceNames);
		});
	}

	/**
	 * Creates the actor descriptor views for the specified container.
	 */
	void CreateActorContainers(const FStreamingGenerationContainerInstanceCollection& InContainerInstanceCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionStreamingGenerator::CreateActorContainers);

		FContainerCollectionInstanceDescriptor MainContainerCollectionInstance;
		MainContainerCollectionInstance.ContainerInstanceCollection = MakeShared<FStreamingGenerationContainerInstanceCollection>(InContainerInstanceCollection);
		MainContainerCollectionInstance.ClusterMode = EContainerClusterMode::Partitioned;
		MainContainerCollectionInstance.OwnerName = TEXT("MainContainer");
		MainContainerCollectionInstance.ContentBundleID = InContainerInstanceCollection.GetContentBundleGuid();
		MainContainerCollectionInstance.ContainerCombinedData.bIsSpatiallyLoaded = true; // Since we apply AND logic on spatially loaded flag recursively, startup value must be true
		MainContainerCollectionInstance.HLODLayer = WorldPartitionContext ? FSoftObjectPath(WorldPartitionContext->GetDefaultHLODLayer()) : FSoftObjectPath();
		MainContainerCollectionInstance.bIsHLODRelevant = true;

		// Create Child Containers
		CreateActorDescriptorViewsRecursive(MoveTemp(MainContainerCollectionInstance));
	}

	/**
	 * Creates the actor descriptor container resolver
	 */
	void CreateContainerResolver(const FStreamingGenerationContainerInstanceCollection& InContainerInstanceCollection)
	{
		ContainerResolver.SetMainContainerPackage(InContainerInstanceCollection.GetBaseContainerInstancePackageName());

		for (const auto& [ContainerID, ContainerDescriptor] : ContainerCollectionInstanceDescriptorsMap)
		{
			if (!ContainerResolver.ContainsContainer(ContainerDescriptor.ContainerInstanceCollection->GetBaseContainerInstancePackageName()))
			{
				FWorldPartitionRuntimeContainer& Container = ContainerResolver.AddContainer(ContainerDescriptor.ContainerInstanceCollection->GetBaseContainerInstancePackageName());

				for (const FStreamingGenerationActorDescView& ActorDescView : ContainerDescriptor.ContainerCollectionInstanceViews)
				{
					Container.AddContainerInstance(ActorDescView.GetActorName(), { ActorDescView.GetGuid(), ActorDescView.GetChildContainerPackage() });
				}
			}
		}

		ContainerResolver.BuildContainerIDToEditorPathMap();
	}

	/**
	 * Perform various validations on the container descriptor, and adjust it based on different requirements. This needs to happen before updating
	 * containers bounds because some actor descriptor views might change grid placement, etc.
	 */
	void ResolveContainerDescriptor(FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor)
	{
		auto ResolveActorDescView = [this, &ContainerCollectionInstanceDescriptor](FStreamingGenerationActorDescView& ActorDescView)
		{
			ResolveRuntimeSpatiallyLoaded(ActorDescView);
			ResolveRuntimeGrid(ActorDescView);
			ResolveRuntimeDataLayers(ActorDescView, ContainerCollectionInstanceDescriptor.DataLayerResolvers);
			ResolveHLODLayer(ActorDescView, ContainerCollectionInstanceDescriptor.HLODLayer);
			ResolveHLODRelevant(ActorDescView, ContainerCollectionInstanceDescriptor.bIsHLODRelevant);
			ResolveParentView(ActorDescView, ContainerCollectionInstanceDescriptor.ActorDescViewMap.Get(), ContainerCollectionInstanceDescriptor.EditorOnlyActorDescSet, ContainerCollectionInstanceDescriptor.EditorOnlyParentActorTransforms);
		};

		ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([this, &ResolveActorDescView](FStreamingGenerationActorDescView& ActorDescView)
		{
			ResolveActorDescView(ActorDescView);
		});

		for (FStreamingGenerationActorDescView& ContainerCollectionInstanceView : ContainerCollectionInstanceDescriptor.ContainerCollectionInstanceViews)
		{
			ResolveActorDescView(ContainerCollectionInstanceView);
		}
	}

	/**
	 * Perform various validations on the container descriptor, and adjust it based on different requirements. This needs to happen before updating
	 * containers bounds because some actor descriptor views might change grid placement, etc.
	 */
	void ValidateContainerDescriptor(FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor)
	{
		const bool bIsMainContainerNonContentBundle = ContainerCollectionInstanceDescriptor.ID.IsMainContainer() && !ContainerCollectionInstanceDescriptor.ContentBundleID.IsValid();
		if (bIsMainContainerNonContentBundle)
		{
			TArray<FGuid> WorldReferences;
			if (WorldPartitionContext)
			{
				// Gather all references to external actors from the world and make them non-spatially loaded
				const ActorsReferencesUtils::FGetActorReferencesParams Params = ActorsReferencesUtils::FGetActorReferencesParams(WorldPartitionContext->GetTypedOuter<UWorld>());
				TArray<ActorsReferencesUtils::FActorReference> WorldExternalActorReferences = ActorsReferencesUtils::GetActorReferences(Params);
				Algo::Transform(WorldExternalActorReferences, WorldReferences, [](const ActorsReferencesUtils::FActorReference& ActorReference) { return ActorReference.Actor->GetActorGuid(); });

				// Validate data layers
				if (DataLayerManager)
				{
					DataLayerManager->ForEachDataLayerInstance([this](const UDataLayerInstance* DataLayerInstance)
					{
						DataLayerInstance->Validate(ErrorHandler);
						return true;
					});
				}
			}
			else
			{
				ULevel::GetWorldExternalActorsReferencesFromPackage(ContainerCollectionInstanceDescriptor.ContainerInstanceCollection->GetBaseContainerInstancePackageName(), WorldReferences);
			}

			for (const FGuid& LevelScriptReferenceActorGuid : WorldReferences)
			{
				if (FStreamingGenerationActorDescView* ActorDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuid(LevelScriptReferenceActorGuid))
				{
					if (ActorDescView->GetIsSpatiallyLoaded())
					{
						ErrorHandler->OnInvalidWorldReference(*ActorDescView, IStreamingGenerationErrorHandler::EWorldReferenceInvalidReason::ReferencedActorIsSpatiallyLoaded);
						ActorDescView->SetForcedNonSpatiallyLoaded();
					}

					if (ActorDescView->GetRuntimeDataLayerInstanceNames().Num())
					{
						ErrorHandler->OnInvalidWorldReference(*ActorDescView, IStreamingGenerationErrorHandler::EWorldReferenceInvalidReason::ReferencedActorHasDataLayers);
						ActorDescView->SetForcedNoDataLayers();
					}
				}
			}
		}

		// Route standard CheckForErrors calls which should not modify actor descriptors in any ways
		ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([this](FStreamingGenerationActorDescView& ActorDescView)
		{
			ActorDescView.CheckForErrors(ErrorHandler);
		});

		for (FStreamingGenerationActorDescView& ContainerCollectionInstanceView : ContainerCollectionInstanceDescriptor.ContainerCollectionInstanceViews)
		{
			ContainerCollectionInstanceView.CheckForErrors(ErrorHandler);
		}

		// Perform various adjustements based on validations and report errors
		//
		// The first validation pass is used to report errors, subsequent passes are used to make corrections to the actor descriptor views.
		// Since the references can form cycles/long chains in the data fixes might need to be propagated in multiple passes.
		// 
		// This works because fixes are deterministic and always apply the same way to both Actors being modified, so there's no ordering issues possible.
		int32 NbErrorsDetected = INDEX_NONE;
		for (uint32 NbValidationPasses = 0; NbErrorsDetected; NbValidationPasses++)
		{
			// Type of work performed in this pass, for clarity
			enum class EPassType
			{
				ErrorReporting,
				Fixup
			};
			const EPassType PassType = NbValidationPasses == 0 ? EPassType::ErrorReporting : EPassType::Fixup;

			NbErrorsDetected = 0;

			ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([this, &ContainerCollectionInstanceDescriptor, &NbErrorsDetected, PassType](FStreamingGenerationActorDescView& ActorDescView)
			{
				// Validate grid placement
				auto IsReferenceGridPlacementValid = [](const FStreamingGenerationActorDescView& RefererActorDescView, const FStreamingGenerationActorDescView& ReferenceActorDescView)
				{
					const bool bIsActorDescSpatiallyLoaded = RefererActorDescView.GetIsSpatiallyLoaded();
					const bool bIsActorDescRefSpatiallyLoaded = ReferenceActorDescView.GetIsSpatiallyLoaded();
					return bIsActorDescSpatiallyLoaded == bIsActorDescRefSpatiallyLoaded;
				};

				// Validate external data layer
				auto IsReferenceExternalDataLayerValid = [](const FStreamingGenerationActorDescView& RefererActorDescView, const FStreamingGenerationActorDescView& ReferenceActorDescView)
				{
					return RefererActorDescView.GetRuntimeDataLayerInstanceNames().GetExternalDataLayer() == ReferenceActorDescView.GetRuntimeDataLayerInstanceNames().GetExternalDataLayer();
				};

				auto GetDataLayerLoadFilter = [](const UDataLayerInstance* DataLayerInstance)
				{
					EDataLayerLoadFilter LoadFilter = DataLayerInstance->IsClientOnly() ? EDataLayerLoadFilter::ClientOnly : DataLayerInstance->IsServerOnly() ? EDataLayerLoadFilter::ServerOnly : EDataLayerLoadFilter::None;
					return LoadFilter;
				};

				// Validate that all runtime data layers have the same DataLayerLoadFilter
				auto AreDataLayersLoadFilterValid = [this, GetDataLayerLoadFilter](const FStreamingGenerationActorDescView& ActorDescView)
				{
					const FDataLayerInstanceNames& RuntimeDLNames = ActorDescView.GetRuntimeDataLayerInstanceNames();
					TArray<const UDataLayerInstance*> RuntimeDataLayerInstances = GetRuntimeDataLayerInstances(RuntimeDLNames.ToArray());

					if (RuntimeDataLayerInstances.Num() > 1)
					{
						TOptional<EDataLayerLoadFilter> LoadFilter;
						for (const UDataLayerInstance* DataLayerInstance : RuntimeDataLayerInstances)
						{
							EDataLayerLoadFilter Current = GetDataLayerLoadFilter(DataLayerInstance);
							// Skip EDL if its load filter is None (doesn't constrain filtering)
							if (DataLayerInstance->IsA<UExternalDataLayerInstance>() && Current == EDataLayerLoadFilter::None)
							{
								continue;
							}
							if (!LoadFilter.IsSet())
							{
								LoadFilter = Current;
							}
							else if (LoadFilter.GetValue() != Current)
							{
								return false;
							}
						}
					}
					return true;
				};

				// Validate reference actor data layers
				auto IsReferenceDataLayersValid = [](const FStreamingGenerationActorDescView& RefererActorDescView, const FStreamingGenerationActorDescView& ReferenceActorDescView)
				{
					if (RefererActorDescView.GetRuntimeDataLayerInstanceNames().GetNonExternalDataLayers().Num() == ReferenceActorDescView.GetRuntimeDataLayerInstanceNames().GetNonExternalDataLayers().Num())
					{
						TSet<FName> RefererActorDescDataLayers;
						TSet<FName> ReferenceActorDescDataLayers;
						RefererActorDescDataLayers.Append(RefererActorDescView.GetRuntimeDataLayerInstanceNames().GetNonExternalDataLayers());
						ReferenceActorDescDataLayers.Append(ReferenceActorDescView.GetRuntimeDataLayerInstanceNames().GetNonExternalDataLayers());

						return RefererActorDescDataLayers.Includes(ReferenceActorDescDataLayers);
					}

					return false;
				};

				// Validate runtime grid references
				auto IsReferenceRuntimeGridValid = [](const FStreamingGenerationActorDescView& RefererActorDescView, const FStreamingGenerationActorDescView& ReferenceActorDescView)
				{
					return RefererActorDescView.GetRuntimeGrid() == ReferenceActorDescView.GetRuntimeGrid();
				};

				// Build references List
				struct FActorReferenceInfo
				{
					FGuid ActorGuid;
					FStreamingGenerationActorDescView* ActorDesc;
					FGuid ReferenceGuid;
					FStreamingGenerationActorDescView* ReferenceActorDesc;
				};

				if (!AreDataLayersLoadFilterValid(ActorDescView))
				{
					if (PassType == EPassType::ErrorReporting)
					{
						ErrorHandler->OnDataLayersLoadFilterMismatch(ActorDescView);
					}
					else
					{
						ActorDescView.SetForcedNoDataLayers();
					}
					NbErrorsDetected++;
				}

				TArray<FActorReferenceInfo> References;

				// Add normal actor references
				for (const FGuid& ReferenceGuid : ActorDescView.GetReferences())
				{
					if (ReferenceGuid != ActorDescView.GetParentActor()) // References to the parent are inversed in their handling 
					{
						// Filter out parent back references
						FStreamingGenerationActorDescView* ReferenceActorDesc = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuid(ReferenceGuid);
						if (!ReferenceActorDesc || (ReferenceActorDesc->GetParentActor() != ActorDescView.GetGuid()))
						{
							References.Emplace(FActorReferenceInfo{ ActorDescView.GetGuid(), &ActorDescView, ReferenceGuid, ReferenceActorDesc });
						}
					}
				}

				// Add attach reference for the topmost parent, this reference is inverted since we consider the top most existing 
				// parent to be refering to us, not the child to be referering the parent.
				{
					FGuid ParentGuid = ActorDescView.GetParentActor();
					FStreamingGenerationActorDescView* TopParentDescView = nullptr;

					while (ParentGuid.IsValid())
					{
						FStreamingGenerationActorDescView* ParentDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuid(ParentGuid);

						if (ParentDescView)
						{
							TopParentDescView = ParentDescView;
							ParentGuid = ParentDescView->GetParentActor();
						}
						else
						{
							if (PassType == EPassType::ErrorReporting)
							{
								// We had a guid but parent cannot be found, this will report a missing reference error, but no error in the subsequent passes
								References.Emplace(FActorReferenceInfo{ ActorDescView.GetGuid(), &ActorDescView, ParentGuid, nullptr });
							}

							break;
						}
					}

					if (TopParentDescView)
					{
						References.Emplace(FActorReferenceInfo{ TopParentDescView->GetGuid(), TopParentDescView, ActorDescView.GetGuid(), &ActorDescView });
					}
				}

				// Non-PIE editor-only actors can only be non-spatially loaded and can't have data layers
				if (ActorDescView.GetActorIsEditorOnlyLoadedInPIE() && (ActorDescView.GetIsSpatiallyLoaded() || ActorDescView.GetRuntimeDataLayerInstanceNames().GetNonExternalDataLayers().Num()))
				{
					if (ensure(PassType == EPassType::ErrorReporting))
					{
						// No error reporting here because this is not a data issue rather than a code issue. Actor classes returning true to IsEditorOnlyLoadedInPIE should be non-spatially loaded
						// and can't be associated to data layers, so make sure they also have CanChangeIsSpatiallyLoadedFlag and ActorTypeSupportsDataLayer return false.
					}
					else
					{
						ActorDescView.SetForcedNonSpatiallyLoaded();
						ActorDescView.SetForcedNoDataLayers();
					}

					NbErrorsDetected++;
				}

				for (FActorReferenceInfo& Info : References)
				{
					FStreamingGenerationActorDescView* RefererActorDescView = Info.ActorDesc;
					FStreamingGenerationActorDescView* ReferenceActorDescView = Info.ReferenceActorDesc;

					if (ReferenceActorDescView)
					{
						// The actor reference is not editor-only, but we are referencing it through an editor-only property
						if (!RefererActorDescView->IsEditorOnlyReference(ReferenceActorDescView->GetGuid()))
						{
							// Validate grid placement
							if (!IsReferenceGridPlacementValid(*RefererActorDescView, *ReferenceActorDescView))
							{
								if (PassType == EPassType::ErrorReporting)
								{
									ErrorHandler->OnInvalidReferenceGridPlacement(*RefererActorDescView, *ReferenceActorDescView);
								}
								else
								{
									RefererActorDescView->SetForcedNonSpatiallyLoaded();
									ReferenceActorDescView->SetForcedNonSpatiallyLoaded();
								}

								NbErrorsDetected++;
							}

							if (!IsReferenceExternalDataLayerValid(*RefererActorDescView, *ReferenceActorDescView))
							{
								if (PassType == EPassType::ErrorReporting)
								{
									ErrorHandler->OnInvalidReferenceDataLayers(*RefererActorDescView, *ReferenceActorDescView, IStreamingGenerationErrorHandler::EDataLayerInvalidReason::ReferencedActorDifferentExternalDataLayer);
								}
								else
								{
									RefererActorDescView->AddForcedInvalidReference(ReferenceActorDescView);
								}

								NbErrorsDetected++;
							}

							if (!IsReferenceDataLayersValid(*RefererActorDescView, *ReferenceActorDescView))
							{
								if (PassType == EPassType::ErrorReporting)
								{
									ErrorHandler->OnInvalidReferenceDataLayers(*RefererActorDescView, *ReferenceActorDescView, IStreamingGenerationErrorHandler::EDataLayerInvalidReason::ReferencedActorDifferentRuntimeDataLayers);
								}
								else
								{
									RefererActorDescView->SetForcedNoDataLayers();
									ReferenceActorDescView->SetForcedNoDataLayers();
								}

								NbErrorsDetected++;
							}

							if (!IsReferenceRuntimeGridValid(*RefererActorDescView, *ReferenceActorDescView))
							{
								if (PassType == EPassType::ErrorReporting)
								{
									ErrorHandler->OnInvalidReferenceRuntimeGrid(*RefererActorDescView, *ReferenceActorDescView);
								}
								else
								{
									RefererActorDescView->SetForcedNoRuntimeGrid();
									ReferenceActorDescView->SetForcedNoRuntimeGrid();
								}

								NbErrorsDetected++;
							}
						}
					}
					else
					{
						if (!ContainerCollectionInstanceDescriptor.EditorOnlyActorDescSet.Contains(Info.ReferenceGuid))
						{
							if (PassType == EPassType::ErrorReporting)
							{
								FWorldPartitionActorDescInstance* ReferencedActorDescInstance = nullptr;

								if (const UActorDescContainerInstance** ExistingReferencedContainerPtr = ActorGuidsToContainerInstanceMap.Find((Info.ReferenceGuid)))
								{
									ReferencedActorDescInstance = (*ExistingReferencedContainerPtr)->GetActorDescInstance(Info.ReferenceGuid);
								}

								// Don't report invalid references to non-existing actors as it can happen in valid scenarios, as the linker code will silently null-out
								// references when loading actors with missing references.
								if (ReferencedActorDescInstance)
								{
									FStreamingGenerationActorDescView InvalidReference(ReferencedActorDescInstance);
									ErrorHandler->OnInvalidReference(*RefererActorDescView, Info.ReferenceGuid, &InvalidReference);
								}

								NbErrorsDetected++;
							}
						}
					}
				}
			});
		}

		// Split runtime and editor references
		ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([this, &ContainerCollectionInstanceDescriptor](FStreamingGenerationActorDescView& ActorDescView)
		{
			TArray<FGuid> RuntimeReferences;
			TArray<FGuid> EditorReferences;

			RuntimeReferences.Reserve(ActorDescView.GetReferences().Num());
			EditorReferences.Reserve(ActorDescView.GetReferences().Num());

			for (const FGuid& ReferenceGuid : ActorDescView.GetReferences())
			{
				if (FStreamingGenerationActorDescView* ReferenceActorDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuid(ReferenceGuid))
				{
					// The actor reference is not editor-only, but we are referencing it through an editor-only property
					if (ActorDescView.IsEditorOnlyReference(ReferenceGuid))
					{
						EditorReferences.Add(ReferenceGuid);
					}
					else
					{
						RuntimeReferences.Add(ReferenceGuid);
					}
				}
				else if (ContainerCollectionInstanceDescriptor.EditorOnlyActorDescSet.Contains(ReferenceGuid))
				{
					EditorReferences.Add(ReferenceGuid);
				}
			}

			if (RuntimeReferences.Num() != ActorDescView.GetReferences().Num())
			{
				ActorDescView.SetRuntimeReferences(RuntimeReferences);
				ActorDescView.SetEditorReferences(EditorReferences);
			}
		});
	}

	/** 
	 * Experimental: apply actor descriptor view mutators.
	 */
	bool MutateContainerInstanceDescriptors(const FStreamingGenerationContainerInstanceCollection& ActorDescCollection)
	{
		TUniquePtr<const IStreamingGenerationContext> MutatorStreamingGenerationContext = MakeUnique<FStreamingGenerationContext>(this, ActorDescCollection);

		// Gather actor descriptor mutators
		if (WorldPartitionContext && WorldPartitionContext->OnGenerateStreamingActorDescsMutatePhase.IsBound())
		{
			TArray<FActorDescViewMutatorInstance> ActorDescsMutatorsInstances;
			WorldPartitionContext->OnGenerateStreamingActorDescsMutatePhase.Broadcast(MutatorStreamingGenerationContext.Get(), ActorDescsMutatorsInstances);

			// Apply actor descriptor mutators to their respective containers
			for (const FActorDescViewMutatorInstance& ActorDescMutatorInstance : ActorDescsMutatorsInstances)
			{
				FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.FindChecked(ActorDescMutatorInstance.ContainerId);
				FActorDescViewMutator& ActorDescViewMutator = ContainerCollectionInstanceDescriptor.ActorDescViewMutators.FindOrAdd(ActorDescMutatorInstance.ActorGuid);

				if (ActorDescMutatorInstance.bIsSpatiallyLoaded.IsSet())
				{
					ActorDescViewMutator.bIsSpatiallyLoaded = ActorDescMutatorInstance.bIsSpatiallyLoaded;
				}
				if (ActorDescMutatorInstance.RuntimeGrid.IsSet())
				{
					ActorDescViewMutator.RuntimeGrid = ActorDescMutatorInstance.RuntimeGrid;
				}
				if (ActorDescMutatorInstance.bIsHLODRelevant.IsSet())
				{
					ActorDescViewMutator.bIsHLODRelevant = ActorDescMutatorInstance.bIsHLODRelevant;
				}
				if (ActorDescMutatorInstance.HLODLayer.IsSet())
				{
					ActorDescViewMutator.HLODLayer = ActorDescMutatorInstance.HLODLayer;
				}
			}

			// Build the containers tree representation
			TMultiMap<FActorContainerID, FActorContainerID> InvertedContainersHierarchy;
			for (auto& [ContainerID, ContainerCollectionInstanceDescriptor] : ContainerCollectionInstanceDescriptorsMap)
			{
				if (!ContainerID.IsMainContainer())
				{
					InvertedContainersHierarchy.Add(ContainerCollectionInstanceDescriptor.ParentID, ContainerID);
				}
			}

			// Apply mutators to per instance data
			auto ApplyActorDescViewMutators = [this, &InvertedContainersHierarchy](const FActorContainerID& ContainerID, TMap<FGuid, FActorDescViewMutator> ActorDescViewMutators)
			{
				auto DumpContainerInstancesRecursive = [this, &InvertedContainersHierarchy](const FActorContainerID& ContainerID, TMap<FGuid, FActorDescViewMutator> ActorDescViewMutators, auto& RecursiveFunc) -> void
				{
					FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.FindChecked(ContainerID);
					ActorDescViewMutators.Append(ContainerCollectionInstanceDescriptor.ActorDescViewMutators);

					RejectClusterDivergentMutators(ContainerCollectionInstanceDescriptor, ActorDescViewMutators);

					for (const auto& [ActorGuid, ActorDescViewMutator] : ActorDescViewMutators)
					{
						if (FStreamingGenerationActorDescView* ActorDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuid(ActorGuid))
						{
							if (ActorDescViewMutator.bIsSpatiallyLoaded.IsSet())
							{
								ActorDescView->SetIsSpatiallyLoaded(ActorDescViewMutator.bIsSpatiallyLoaded.GetValue());
							}

							if (ActorDescViewMutator.RuntimeGrid.IsSet())
							{
								ActorDescView->SetRuntimeGrid(ActorDescViewMutator.RuntimeGrid.GetValue());
							}

							if (ActorDescViewMutator.bIsHLODRelevant.IsSet())
							{
								ActorDescView->SetIsHLODRelevant(ActorDescViewMutator.bIsHLODRelevant.GetValue());
							}

							if (ActorDescViewMutator.HLODLayer.IsSet() && ActorDescView->GetIsSpatiallyLoaded() && ActorDescView->GetActorIsHLODRelevant())
							{
								ActorDescView->SetRuntimeHLODLayer(ActorDescViewMutator.HLODLayer.GetValue());
							}
						}
					}

					TArray<FActorContainerID> ChildContainersIDs;
					InvertedContainersHierarchy.MultiFind(ContainerID, ChildContainersIDs);
					ChildContainersIDs.Sort();

					if (ChildContainersIDs.Num())
					{
						for (const FActorContainerID& ChildContainerID : ChildContainersIDs)
						{
							RecursiveFunc(ChildContainerID, ActorDescViewMutators, RecursiveFunc);
						}
					}
				};

				DumpContainerInstancesRecursive(ContainerID, ActorDescViewMutators, DumpContainerInstancesRecursive);
			};

			ApplyActorDescViewMutators(FActorContainerID(), TMap<FGuid, FActorDescViewMutator>());

			return true;
		}

		return false;
	}

	void RejectClusterDivergentMutators(FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor, TMap<FGuid, FActorDescViewMutator>& Mutators) const
	{
		const FStreamingGenerationActorDescViewMap& ActorDescViewMap = ContainerCollectionInstanceDescriptor.ActorDescViewMap.Get();

		auto ResolveRuntimeGrid = [&ActorDescViewMap, &Mutators](const FGuid& ActorGuid) -> FName
		{
			const FActorDescViewMutator* Mutator = Mutators.Find(ActorGuid);
			return (Mutator && Mutator->RuntimeGrid.IsSet()) ? Mutator->RuntimeGrid.GetValue() : ActorDescViewMap.FindByGuidChecked(ActorGuid).GetRuntimeGrid();
		};

		auto ResolveIsSpatiallyLoaded = [&ActorDescViewMap, &Mutators](const FGuid& ActorGuid) -> bool
		{
			const FActorDescViewMutator* Mutator = Mutators.Find(ActorGuid);
			return (Mutator && Mutator->bIsSpatiallyLoaded.IsSet()) ? Mutator->bIsSpatiallyLoaded.GetValue() : ActorDescViewMap.FindByGuidChecked(ActorGuid).GetIsSpatiallyLoaded();
		};

		for (const TArray<FGuid>& Cluster : ContainerCollectionInstanceDescriptor.Clusters)
		{
			const FName ReferenceRuntimeGrid = ResolveRuntimeGrid(Cluster[0]);
			const bool bReferenceIsSpatiallyLoaded = ResolveIsSpatiallyLoaded(Cluster[0]);

			bool bRuntimeGridDivergent = false;
			bool bIsSpatiallyLoadedDivergent = false;
			for (int32 Index = 1; Index < Cluster.Num() && (!bRuntimeGridDivergent || !bIsSpatiallyLoadedDivergent); ++Index)
			{
				if (!bRuntimeGridDivergent && ResolveRuntimeGrid(Cluster[Index]) != ReferenceRuntimeGrid)
				{
					bRuntimeGridDivergent = true;
				}
				if (!bIsSpatiallyLoadedDivergent && ResolveIsSpatiallyLoaded(Cluster[Index]) != bReferenceIsSpatiallyLoaded)
				{
					bIsSpatiallyLoadedDivergent = true;
				}
			}

			if (!bRuntimeGridDivergent && !bIsSpatiallyLoadedDivergent)
			{
				continue;
			}

			for (const FGuid& ActorGuid : Cluster)
			{
				FActorDescViewMutator* Mutator = Mutators.Find(ActorGuid);
				if (!Mutator)
				{
					continue;
				}

				if (bRuntimeGridDivergent && Mutator->RuntimeGrid.IsSet())
				{
					ErrorHandler->OnRejectedActorDescViewMutator(ActorDescViewMap.FindByGuidChecked(ActorGuid), *Mutator, IStreamingGenerationErrorHandler::EActorDescViewMutatorRejectedReason::DivergesClusterRuntimeGrid);
					Mutator->RuntimeGrid.Reset();
				}

				if (bIsSpatiallyLoadedDivergent && Mutator->bIsSpatiallyLoaded.IsSet())
				{
					ErrorHandler->OnRejectedActorDescViewMutator(ActorDescViewMap.FindByGuidChecked(ActorGuid), *Mutator, IStreamingGenerationErrorHandler::EActorDescViewMutatorRejectedReason::DivergesClusterIsSpatiallyLoaded);
					Mutator->bIsSpatiallyLoaded.Reset();
				}
			}
		}
	}

	/**
	 * Perform various validations on the container descriptor instance, and adjust it based on different requirements. This needs to happen before updating
	 * containers bounds because some actor descriptor views might change grid placement, etc.
	 */
	void ValidateContainerInstanceDescriptor(FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor, bool bIsMainContainer)
	{
		// Perform various adjustements based on validations and report errors
		//
		// The first validation pass is used to report errors, subsequent passes are used to make corrections to the actor descriptor views.
		// Since the references can form cycles/long chains in the data fixes might need to be propagated in multiple passes.
		// 
		// This works because fixes are deterministic and always apply the same way to both Actors being modified, so there's no ordering issues possible.
		int32 NbErrorsDetected = INDEX_NONE;
		for (uint32 NbValidationPasses = 0; NbErrorsDetected; NbValidationPasses++)
		{
			// Type of work performed in this pass, for clarity
			enum class EPassType
			{
				ErrorReporting,
				Fixup
			};
			const EPassType PassType = NbValidationPasses == 0 ? EPassType::ErrorReporting : EPassType::Fixup;

			NbErrorsDetected = 0;

			ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([this, &ContainerCollectionInstanceDescriptor, &NbErrorsDetected, PassType](FStreamingGenerationActorDescView& ActorDescView)
			{
				// Report invalid grid usage by spatially loaded actors
				if (ActorDescView.GetIsSpatiallyLoaded() && !IsValidGrid(ActorDescView.GetRuntimeGrid(), ActorDescView.GetActorNativeClass()))
				{
					if (PassType == EPassType::ErrorReporting)
					{
						ErrorHandler->OnInvalidRuntimeGrid(ActorDescView, ActorDescView.GetRuntimeGrid());
					}
					else
					{
						ActorDescView.SetRuntimeGrid(NAME_None);
					}

					NbErrorsDetected++;
				}

				// If project wants to enforce explicit HLOD Layer Partition assignation
				// Validate that the HLOD layer + runtime grid combination is expected.
				if (IWorldPartitionEditorModule::Get().GetRequireExplicitHLODLayerPartitionAssignation())
				{
				    if (ActorDescView.GetIsSpatiallyLoaded() && ActorDescView.GetHLODLayer().IsValid() && !IsValidHLODLayer(ActorDescView.GetRuntimeGrid(), ActorDescView.GetHLODLayer()))
				    {
					    if (PassType == EPassType::ErrorReporting)
					    {
						    ErrorHandler->OnInvalidHLODLayer(ActorDescView);
					    }
					    else
					    {
						    ActorDescView.SetForcedNoHLODLayer();
					    }

					    NbErrorsDetected++;
				    }
				}

				if (ActorDescView.GetIsSpatiallyLoaded() && ActorDescView.GetHLODLayer().IsValid())
				{
					if (UHLODLayer* HLODLayer = Cast<UHLODLayer>(ActorDescView.GetHLODLayer().TryLoad()))
					{
						if (const FHLODLayerTypeUnsupportedActorClasses* UnsupportedActorClasses = UWorldPartitionSettings::Get()->GetUnsupportedActorClassesPerHLODLayerType().Find(HLODLayer->GetLayerType()))
						{
							for (TSubclassOf<AActor> ActorClass : UnsupportedActorClasses->ActorClasses)
							{
								if (ActorDescView.GetActorNativeClass()->IsChildOf(ActorClass))
								{
									if (PassType == EPassType::ErrorReporting)
									{
										ErrorHandler->OnUnsupportedHLODLayer(ActorDescView, HLODLayer, IStreamingGenerationErrorHandler::EHLODLayerUnsupportedReason::ActorClassUnsupportedByLayerType);
									}
									else
									{
										ActorDescView.SetForcedNoHLODLayer();
									}

									NbErrorsDetected++;
								}
							}
						}
					}
				}

				if (TSet<FGuid>* FilteredActors = ContainerFilteredActors.Find(ContainerCollectionInstanceDescriptor.ID))
				{
					const bool IsReferencerFiltered = FilteredActors->Contains(ActorDescView.GetGuid());
					for (const FGuid& ReferenceGuid : ActorDescView.GetReferences())
					{
						const bool IsReferenceeFiltered = FilteredActors->Contains(ReferenceGuid);
						if (IsReferenceeFiltered && !IsReferencerFiltered)
						{
							if (PassType == EPassType::ErrorReporting)
							{
								const FStreamingGenerationActorDescView& ReferenceActorDesc = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuidChecked(ReferenceGuid);
								ErrorHandler->OnInvalidActorFilterReference(ActorDescView, ReferenceActorDesc);
							}
							else
							{
								FilteredActors->Remove(ReferenceGuid);
							}

							NbErrorsDetected++;
						}
					}
				}
			});
		}
	}

	/**
	 * Update the container descriptor containers to adjust their bounds from actor descriptor views.
	 */
	void UpdateContainerDescriptor(FContainerCollectionInstanceDescriptor& ContainerCollectionInstanceDescriptor)
	{
		// Build clusters for this container - at this point, all actors references should be in the same data layers, grid, etc because of actor descriptors validation.
		TArray<TPair<FGuid, TArray<FGuid>>> ActorsWithRefs;
		ContainerCollectionInstanceDescriptor.ActorDescViewMap->ForEachActorDescView([&ActorsWithRefs, this](const FStreamingGenerationActorDescView& ActorDescView)
		{ 
			TPair<FGuid, TArray<FGuid>>& ActorWithRefs = ActorsWithRefs.Emplace_GetRef(ActorDescView.GetGuid(), ActorDescView.GetReferences());

			// Include Standalone HLOD actors "dependencies". Make sure all child actors of a Standalone HLOD actor to end up in the same cluster.
			if (TSet<FGuid>* StandaloneHLODDependencies = StandaloneHLODActorsDependencies.Find(ActorWithRefs.Key))
			{
				for (const FGuid& StandaloneHLODDependency : *StandaloneHLODDependencies)
				{
					if (ActorDescView.GetActorDescViewMap().FindByGuid(StandaloneHLODDependency))
					{
						ActorWithRefs.Value.Add(StandaloneHLODDependency);
					}
				}
			}
		});
		ContainerCollectionInstanceDescriptor.Clusters = GenerateObjectsClusters(ActorsWithRefs);
	}

	/**
	 * Validate the streaming generator internal state.
	 */
	void ValidateContainerInstanceDescriptors()
	{
		for (auto& [ContainerID, ContainerCollectionInstanceDescriptor] : ContainerCollectionInstanceDescriptorsMap)
		{
			ValidateContainerInstanceDescriptor(ContainerCollectionInstanceDescriptor, ContainerCollectionInstanceDescriptor.ID.IsMainContainer());

			for (const TArray<FGuid>& Cluster : ContainerCollectionInstanceDescriptor.Clusters)
			{
				const FStreamingGenerationActorDescView& ReferenceActorDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuidChecked(Cluster[0]);

				for (const FGuid& ActorGuid : Cluster)
				{
					// Validate that all actors part of the same actor set share the same set of values
					const FStreamingGenerationActorDescView& ActorDescView = ContainerCollectionInstanceDescriptor.ActorDescViewMap->FindByGuidChecked(ActorGuid);
					check(ActorDescView.GetRuntimeGrid() == ReferenceActorDescView.GetRuntimeGrid());
					check(ActorDescView.GetIsSpatiallyLoaded() == ReferenceActorDescView.GetIsSpatiallyLoaded());
					check(ActorDescView.GetContentBundleGuid() == ReferenceActorDescView.GetContentBundleGuid());
					check(ActorDescView.GetExternalDataLayerAsset() == ReferenceActorDescView.GetExternalDataLayerAsset());
				}
			}
		}
	}

public:
	struct FWorldPartitionStreamingGeneratorParams
	{
		FWorldPartitionStreamingGeneratorParams()
			: WorldPartitionContext(nullptr)
			, ErrorHandler(&NullErrorHandler)
			, bEnableStreaming(false)
			, bCreateContainerResolver(false)
			, bHandleUnsavedActors(false)
			, bIsPIE(false)
			, bIncludeStandaloneHLODActors(false)
			, bIncludeCustomHLODActorsWithOtherLayerType(false)
		{}

		const UWorldPartition* WorldPartitionContext;
		IStreamingGenerationErrorHandler* ErrorHandler;
		bool bEnableStreaming;
		bool bCreateContainerResolver;
		bool bHandleUnsavedActors;
		bool bIsPIE;
		bool bIncludeStandaloneHLODActors;
		bool bIncludeCustomHLODActorsWithOtherLayerType;
		TMap<FGuid, const UActorDescContainerInstance*> ActorGuidsToContainerInstanceMap;
		TArray<TSubclassOf<AActor>> FilteredClasses;
		FName DefaultGrid;
		TFunction<bool(FName, const UClass*)> IsValidGrid;
		TFunction<bool(FName GridName, const FSoftObjectPath&)> IsValidHLODLayer;

		FWorldPartitionStreamingGeneratorParams& SetWorldPartitionContext(const UWorldPartition* InWorldPartitionContext) { WorldPartitionContext = InWorldPartitionContext; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetErrorHandler(IStreamingGenerationErrorHandler* InErrorHandler) { ErrorHandler = InErrorHandler; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetEnableStreaming(bool bInEnableStreaming) { bEnableStreaming = bInEnableStreaming; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetCreateContainerResolver(bool bInCreateContainerResolver) { bCreateContainerResolver = bInCreateContainerResolver; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetHandleUnsavedActors(bool bInHandleUnsavedActors) { bHandleUnsavedActors = bInHandleUnsavedActors; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetIsPIE(bool bInIsPIE) { bIsPIE = bInIsPIE; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetIncludeStandaloneHLODActors(bool bInIncludeStandaloneHLODActors) { bIncludeStandaloneHLODActors = bInIncludeStandaloneHLODActors; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetIncludeCustomHLODActorsWithOtherLayerType(bool bInIncludeCustomHLODActorsWithOtherLayerType) { bIncludeCustomHLODActorsWithOtherLayerType = bInIncludeCustomHLODActorsWithOtherLayerType; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetActorGuidsToContainerInstanceMap(const TMap<FGuid, const UActorDescContainerInstance*>& InActorGuidsToContainerInstanceMap) { ActorGuidsToContainerInstanceMap = InActorGuidsToContainerInstanceMap; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetFilteredClasses(const TArray<TSubclassOf<AActor>>& InFilteredClasses) { FilteredClasses = InFilteredClasses; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetDefaultGrid(FName InDefaultGrid) { DefaultGrid = InDefaultGrid; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetIsValidGrid(TFunction<bool(FName, const UClass*)> InIsValidGrid) { IsValidGrid = InIsValidGrid; return *this; }
		FWorldPartitionStreamingGeneratorParams& SetIsValidHLODLayer(TFunction<bool(FName GridName, const FSoftObjectPath&)> InIsValidHLODLayer) { IsValidHLODLayer = InIsValidHLODLayer; return *this; }
				
		inline static FStreamingGenerationNullErrorHandler NullErrorHandler;
	};

	FWorldPartitionStreamingGenerator(const FWorldPartitionStreamingGeneratorParams& Params)
		: WorldPartitionContext(Params.WorldPartitionContext)
		, bEnableStreaming(Params.bEnableStreaming)
		, bCreateContainerResolver(Params.bCreateContainerResolver)
		, bHandleUnsavedActors(Params.bHandleUnsavedActors)
		, bIsPIE(Params.bIsPIE)
		, bIncludeStandaloneHLODActors(Params.bIncludeStandaloneHLODActors)
		, bIncludeCustomHLODActorsWithOtherLayerType(Params.bIncludeCustomHLODActorsWithOtherLayerType)
		, FilteredClasses(Params.FilteredClasses)
		, DefaultGrid(Params.DefaultGrid)
		, IsValidGrid(Params.IsValidGrid)
		, IsValidHLODLayer(Params.IsValidHLODLayer)
		, ErrorHandler(Params.ErrorHandler)
		, ActorGuidsToContainerInstanceMap(Params.ActorGuidsToContainerInstanceMap)
	{
		UWorld* OwningWorld = WorldPartitionContext ? WorldPartitionContext->GetWorld() : nullptr;
		WorldPartitionSubsystem = OwningWorld ? UWorld::GetSubsystem<UWorldPartitionSubsystem>(OwningWorld) : nullptr;
		DataLayerManager = OwningWorld ? WorldPartitionContext->GetDataLayerManager() : nullptr;
		WorldPartitionStandaloneHLODSubsystem = OwningWorld ? UWorld::GetSubsystem<UWorldPartitionStandaloneHLODSubsystem>(OwningWorld) : nullptr;
	}

	void PreparationPhase(const FStreamingGenerationContainerInstanceCollection& ContainerInstanceCollection)
	{
		CreateActorContainers(ContainerInstanceCollection);

		if (bCreateContainerResolver)
		{
			CreateContainerResolver(ContainerInstanceCollection);
		}

		ValidateContainerInstanceDescriptors();

		if (MutateContainerInstanceDescriptors(ContainerInstanceCollection))
		{
			ValidateContainerInstanceDescriptors();
		}
	}

	static TUniquePtr<FArchive> CreateDumpStateLogArchive(const TCHAR* Suffix)
	{
		const FString LogDir = GIsBuildMachine ? FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs")) : FPaths::ProjectLogDir();
		const FString LogTimeStamp = GIsBuildMachine ? TEXT("") : FString::Printf(TEXT("-%08x-%s"), FPlatformProcess::GetCurrentProcessId(), *FDateTime::Now().ToIso8601().Replace(TEXT(":"), TEXT(".")));
		const FString StateLogOutputFilename = FPaths::Combine(LogDir, TEXT("WorldPartition"), FString::Printf(TEXT("StreamingGeneration-%s%s.log"), Suffix, *LogTimeStamp));
		return TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*StateLogOutputFilename));
	}

	void DumpStateLog(FHierarchicalLogArchive& Ar)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionStreamingGenerator::DumpStateLog);

		// Force mode to EToStringMode::Full when running automated tests
		FWorldPartitionActorDesc::EToStringMode Mode = GIsAutomationTesting ? FWorldPartitionActorDesc::EToStringMode::Full : FWorldPartitionActorDesc::EToStringMode::Compact;

		// Build the containers tree representation
		TMultiMap<FActorContainerID, FActorContainerID> InvertedContainersHierarchy;
		for (auto& [ContainerID, ContainerCollectionInstanceDescriptor] : ContainerCollectionInstanceDescriptorsMap)
		{
			if (!ContainerID.IsMainContainer())
			{
				InvertedContainersHierarchy.Add(ContainerCollectionInstanceDescriptor.ParentID, ContainerID);
			}
		}

		TSet<FString> UniqueContainerNames;

		UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Containers:")));
		for (auto& [ContainerID, ContainerCollectionInstanceDescriptor] : ContainerCollectionInstanceDescriptorsMap)
		{
			ContainerCollectionInstanceDescriptor.ContainerInstanceCollection->ForEachActorDescContainerInstance([&UniqueContainerNames, &Ar, Mode](const UActorDescContainerInstance* InActorDescContainerInstance)
			{
				const FString ContainerPackageName = InActorDescContainerInstance->GetContainerPackage().ToString();
				bool bIsAlreadySet = false;
				UniqueContainerNames.Add(ContainerPackageName, &bIsAlreadySet);
				if (!bIsAlreadySet)
				{
					UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Container: %s"), *ContainerPackageName));

					const UActorDescContainer* Container = InActorDescContainerInstance->GetContainer();
					if (Container->GetActorDescCount())
					{
						UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("ActorDescs:")));

						FActorDescList::FGuidActorDescMap SortedActorsByGuid = Container->GetActorsByGuid();
						SortedActorsByGuid.KeySort([](const FGuid& GuidA, const FGuid& GuidB) { return GuidA < GuidB; });

						for (auto& [ActorGuid, ActorDesc] : SortedActorsByGuid)
						{
							Ar.Print(*((*ActorDesc)->ToString(Mode)));
						}
					}
				}
			});
		}

		TSet<uint64> UniqueClusters;
		auto GetRuntimeDataLayerInstancesFromNames = [this](const TArray<FName>& InDataLayerInstanceNames, TArray<const UDataLayerInstance*>& OutDataLayerInstances)
		{
			OutDataLayerInstances = GetRuntimeDataLayerInstances(InDataLayerInstanceNames);
			return true;
		};

		Ar.Printf(TEXT("ContainerInstances:"));
		auto DumpContainerInstances = [this, &InvertedContainersHierarchy, &Ar, Mode, &GetRuntimeDataLayerInstancesFromNames, &UniqueClusters](const FActorContainerID& ContainerID)
		{
			auto DumpContainerInstancesRecursive = [this, &InvertedContainersHierarchy, &Ar, Mode, &GetRuntimeDataLayerInstancesFromNames, &UniqueClusters](const FActorContainerID& ContainerID, auto& RecursiveFunc) -> void
			{
				const FContainerCollectionInstanceDescriptor& ContainerInstanceDescriptor = ContainerCollectionInstanceDescriptorsMap.FindChecked(ContainerID);

				{
					UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("%s:"), *ContainerInstanceDescriptor.OwnerName));

					Ar.Printf(TEXT("          ID: %s"), *ContainerInstanceDescriptor.ID.ToString());
					Ar.Printf(TEXT("    ParentID: %s"), *ContainerInstanceDescriptor.ParentID.ToString());
					Ar.Printf(TEXT("      Bounds: %s"), *ContainerInstanceDescriptor.Bounds.ToString());
					Ar.Printf(TEXT("   Transform: %s"), *ContainerInstanceDescriptor.Transform.ToString());
					Ar.Printf(TEXT("   Container: %s"), *ContainerInstanceDescriptor.ContainerInstanceCollection->GetBaseContainerInstancePackageName().ToString());
					Ar.Printf(TEXT("HLODRelevant: %s"), ContainerInstanceDescriptor.bIsHLODRelevant ? TEXT("true") : TEXT("false"));

					if (ContainerInstanceDescriptor.Clusters.Num())
					{
						TArray<TArray<FGuid>> SortedClusters = ContainerInstanceDescriptor.Clusters;
						for (TArray<FGuid>& ActorGuids : SortedClusters)
						{
							ActorGuids.Sort([](const FGuid& GuidA, const FGuid& GuidB) { return GuidA < GuidB; });
						}
						SortedClusters.Sort([](const TArray<FGuid>& GuidA, const TArray<FGuid>& GuidB) { return GuidA[0] < GuidB[0]; });

						FHashBuilder HashBuilder;
						for (TArray<FGuid>& ActorGuids : SortedClusters)
						{
							for (const FGuid& ActorGuid : ActorGuids)
							{
								HashBuilder << ActorGuid;
							}
						}
						const uint64 ClustersHash = HashBuilder.GetHash();
						bool bIsAlreadySet = false;
						UniqueClusters.Add(ClustersHash, &bIsAlreadySet);
						
						UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("Clusters (Hash:%llX)%s"), ClustersHash, bIsAlreadySet ? TEXT("") : TEXT(":")));

						if (!bIsAlreadySet)
						{
							int ClusterIndex = 0;
							for (TArray<FGuid>& ActorGuids : SortedClusters)
							{
								UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("[%3d]"), ClusterIndex++));
								for (const FGuid& ActorGuid : ActorGuids)
								{
									const FStreamingGenerationActorDescView& ActorDescView = ContainerInstanceDescriptor.ActorDescViewMap->FindByGuidChecked(ActorGuid);
									Ar.Print(*ActorDescView.ToString(FWorldPartitionActorDesc::EToStringMode::Guid));
								}
							}
						}
					}

					TArray<FString> ActorDescViews;
					if (ContainerInstanceDescriptor.ActorDescViewMap->Num())
					{
						TMap<FGuid, FStreamingGenerationActorDescView*> SortedActorDescViewMap = ContainerInstanceDescriptor.ActorDescViewMap->ActorDescViewsByGuid;
						SortedActorDescViewMap.KeySort([](const FGuid& GuidA, const FGuid& GuidB) { return GuidA < GuidB; });
						for (auto& [ActorGuid, ActorDescView] : SortedActorDescViewMap)
						{
							FString ActorDescViewResult = ActorDescView->ToString(FStreamingGenerationActorDescView::EToStringMode::ViewOnly, Mode, GetRuntimeDataLayerInstancesFromNames);
							if (ActorDescViewResult.Len())
							{
								ActorDescViews.Add(MoveTemp(ActorDescViewResult));
							}
						}
					}

					if (ActorDescViews.Num())
					{
						UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("ActorDescViews:")));
						for (const FString& ActorDescView : ActorDescViews)
						{
							Ar.Print(*ActorDescView);
						}
					}
				}

				TArray<FActorContainerID> ChildContainersIDs;
				InvertedContainersHierarchy.MultiFind(ContainerID, ChildContainersIDs);
				ChildContainersIDs.Sort();

				if (ChildContainersIDs.Num())
				{
					UE_SCOPED_INDENT_LOG_ARCHIVE(Ar.PrintfIndent(TEXT("SubContainers:")));

					for (const FActorContainerID& ChildContainerID : ChildContainersIDs)
					{
						RecursiveFunc(ChildContainerID, RecursiveFunc);
					}
				}
			};

			DumpContainerInstancesRecursive(ContainerID, DumpContainerInstancesRecursive);
		};

		DumpContainerInstances(FActorContainerID());
	}

	const FStreamingGenerationContext* GetStreamingGenerationContext(const FStreamingGenerationContainerInstanceCollection& ContainerInstanceCollection)
	{
		if (!StreamingGenerationContext)
		{
			// Construct the streaming generation context
			StreamingGenerationContext = MakeUnique<FStreamingGenerationContext>(this, ContainerInstanceCollection);
		}

		return StreamingGenerationContext.Get();
	}

	TArray<const UDataLayerInstance*> GetRuntimeDataLayerInstances(const TArray<FName>& RuntimeDataLayers) const
	{
		return DataLayerManager ? DataLayerManager->GetRuntimeDataLayerInstances(RuntimeDataLayers) : TArray<const UDataLayerInstance*>();
	}

	const FWorldPartitionRuntimeContainerResolver& GetContainerResolver() const
	{
		return ContainerResolver;
	}

private:
	const UWorldPartition* WorldPartitionContext;
	UWorldPartitionSubsystem* WorldPartitionSubsystem;
	UWorldPartitionStandaloneHLODSubsystem* WorldPartitionStandaloneHLODSubsystem;
	const UDataLayerManager* DataLayerManager;
	bool bEnableStreaming;
	bool bCreateContainerResolver;
	bool bHandleUnsavedActors;
	bool bIsPIE;
	bool bIncludeStandaloneHLODActors;
	bool bIncludeCustomHLODActorsWithOtherLayerType;
	TArray<TSubclassOf<AActor>> FilteredClasses;
	FName DefaultGrid;
	TFunction<bool(FName, const UClass*)> IsValidGrid;
	TFunction<bool(FName, const FSoftObjectPath&)> IsValidHLODLayer;
	IStreamingGenerationErrorHandler* ErrorHandler;
	FWorldPartitionRuntimeContainerResolver ContainerResolver;

	/** Maps containers IDs to their container collection instance descriptor */
	TMap<FActorContainerID, FContainerCollectionInstanceDescriptor> ContainerCollectionInstanceDescriptorsMap;

	/** Data required for streaming generation interface */
	TUniquePtr<FStreamingGenerationContext> StreamingGenerationContext;

	/** List of container instances participating in this streaming generation step */
	TMap<FGuid, const UActorDescContainerInstance*> ActorGuidsToContainerInstanceMap;

	/** Maps containers IDs to their filtered actors use while creating FContainerCollectionInstanceDescriptor */
	TMap<FActorContainerID, TSet<FGuid>> ContainerFilteredActors;

	/** Maps Standalone HLOD actor guid to their "dependencies". Used to make sure all child actors of a Standalone HLOD actor end up in the same streaming cell */
	TMap<FGuid, TSet<FGuid>> StandaloneHLODActorsDependencies;
};

bool UWorldPartition::GenerateStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext)
{
	FGenerateStreamingParams Params = FGenerateStreamingParams(InParams).SetContainerInstanceCollection(*this, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs);

	OnPreGenerateStreaming.Broadcast(InContext.PackagesToGenerate);

	return GenerateContainerStreaming(Params, InContext);
}

bool UWorldPartition::GenerateContainerStreaming(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartition::GenerateContainerStreaming);

	const FString ContainerPackageName = InParams.ContainerInstanceCollection.GetBaseContainerInstancePackageName().ToString();
	FString ContainerShortName = FPackageName::GetShortName(ContainerPackageName);
	if (!ContainerPackageName.StartsWith(TEXT("/Game/")))
	{
		TArray<FString> SplitContainerPath;
		if (ContainerPackageName.ParseIntoArray(SplitContainerPath, TEXT("/")))
		{
			ContainerShortName += TEXT(".");
			ContainerShortName += SplitContainerPath[0];
		}
	}

	UE_SCOPED_TIMER(*FString::Printf(TEXT("GenerateStreaming for '%s'"), *ContainerShortName), LogWorldPartition, Display);

	// Dump streaming generation log
	TUniquePtr<FArchive> LogFileAr;
	TUniquePtr<FHierarchicalLogArchive> HierarchicalLogAr;

	const bool bIsStreamingGenerationLogAllowed = GIsStreamingGenerationLogAllowed && (!bIsPIE || IWorldPartitionEditorModule::Get().GetEnableStreamingGenerationLogOnPIE()) && (!InParams.OutputLogType.IsSet() || !InParams.OutputLogType.GetValue().IsEmpty());
	const bool bIsStreamingGenerationLogRelevant = !GIsBuildMachine || GIsAutomationTesting || IsRunningCookCommandlet();

	if (bIsStreamingGenerationLogAllowed && bIsStreamingGenerationLogRelevant)
	{
		TStringBuilder<256> StateLogSuffix;

		if (bIsPIE)
		{
			StateLogSuffix += TEXT("PIE");
		}
		else if (IsRunningDedicatedServer())
		{
			StateLogSuffix += TEXT("Server");
		}
		else if (IsRunningGame())
		{
			StateLogSuffix += TEXT("Game");
		}
		else if (IsRunningCookCommandlet())
		{
			StateLogSuffix += TEXT("Cook");
		}
		else if (GIsAutomationTesting)
		{
			StateLogSuffix += TEXT("UnitTest");
		}
		else if (InParams.OutputLogType.IsSet())
		{
			StateLogSuffix += *InParams.OutputLogType.GetValue();
		}
		else
		{
			StateLogSuffix += TEXT("Manual");
		}

		StateLogSuffix += TEXT("_");
		StateLogSuffix += ContainerShortName;
		LogFileAr = FWorldPartitionStreamingGenerator::CreateDumpStateLogArchive(*StateLogSuffix);

		if (LogFileAr.IsValid())
		{
			InContext.OutputLogFilename = LogFileAr->GetArchiveName();
			HierarchicalLogAr = MakeUnique<FHierarchicalLogArchive>(*LogFileAr);
		}
	}

	TErrorHandlerSelector<FStreamingGenerationLogErrorHandler> ErrorHandlerSelector(InParams.ErrorHandler);

	FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams StreamingGeneratorParams = FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams()
		.SetWorldPartitionContext(this)
		.SetHandleUnsavedActors(bIsPIE && !GetTypedOuter<UWorld>()->IsGameWorld())
		.SetIsPIE(bIsPIE)
		.SetIncludeStandaloneHLODActors(IsMainWorldPartition())
		.SetDefaultGrid(RuntimeHash ? RuntimeHash->GetDefaultGrid() : NAME_None)
		.SetIsValidGrid([this](FName GridName, const UClass* ActorClass) { return RuntimeHash->IsValidGrid(GridName, ActorClass); })
		.SetIsValidHLODLayer([this](FName GridName, const FSoftObjectPath& HLODLayerPath) { return RuntimeHash->IsValidHLODLayer(GridName, HLODLayerPath); })
		.SetErrorHandler(ErrorHandlerSelector.Get())
		.SetEnableStreaming(IsStreamingEnabled())
		.SetFilteredClasses(InParams.FilteredClasses)
		.SetCreateContainerResolver(FEditorPathHelper::IsEnabled());

	FWorldPartitionStreamingGenerator StreamingGenerator(StreamingGeneratorParams);

	// Preparation Phase
	StreamingGenerator.PreparationPhase(InParams.ContainerInstanceCollection);

	if (HierarchicalLogAr.IsValid())
	{
		StreamingGenerator.DumpStateLog(*HierarchicalLogAr);
	}

	auto GenerateRuntimeHash = [this, &HierarchicalLogAr, &StreamingGenerator, &InContext, &InParams](const UActorDescContainerInstance* InActorDescContainerInstance)
	{
		check(!StreamingPolicy);
		StreamingPolicy = NewObject<UWorldPartitionStreamingPolicy>(const_cast<UWorldPartition*>(this), WorldPartitionStreamingPolicyClass.Get(), NAME_None, bIsPIE ? RF_Transient : RF_NoFlags);

		FStreamingGenerationContextProxy GenerationContextProxy(StreamingGenerator.GetStreamingGenerationContext(InParams.ContainerInstanceCollection));
		GenerationContextProxy.SetActorSetInstanceFilter([ExternalDataLayerAsset = InActorDescContainerInstance->GetExternalDataLayerAsset()](const IStreamingGenerationContext::FActorSetInstance& InActorSetInstance) { return (InActorSetInstance.GetExternalDataLayerAsset() == ExternalDataLayerAsset); });

		check(RuntimeHash);
		if (RuntimeHash->GenerateStreaming(StreamingPolicy, &GenerationContextProxy, InContext.PackagesToGenerate))
		{
			StreamingPolicy->SetContainerResolver(StreamingGenerator.GetContainerResolver());
			StreamingPolicy->PrepareActorToCellRemapping();
			StreamingPolicy->SetShouldMergeStreamingSourceInfo(RuntimeHash->GetShouldMergeStreamingSourceInfo());
			return true;
		}
		return false;
	};

	// Generate streaming for External Data Layer container instances
	bool bStreamingGenerationSuccess = true;
	for (const UActorDescContainerInstance* ExternalDataLayerContainerInstance : InParams.ContainerInstanceCollection.GetExternalDataLayerContainerInstances())
	{
		const UExternalDataLayerAsset* ExternalDataLayerAsset = ExternalDataLayerContainerInstance->GetExternalDataLayerAsset();
		check(ExternalDataLayerAsset);
		bool bExternalDataLayerGenerationSuccess = GenerateRuntimeHash(ExternalDataLayerContainerInstance);
		// No need to create an ExternalStreamingObject and move the streaming content if it's empty
		if (bExternalDataLayerGenerationSuccess && RuntimeHash->HasStreamingContent())
		{
			URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = ExternalDataLayerManager->CreateExternalStreamingObjectUsingStreamingGeneration(ExternalDataLayerAsset);
			bExternalDataLayerGenerationSuccess = !!ExternalStreamingObject;
			if (bExternalDataLayerGenerationSuccess)
			{
				if (HierarchicalLogAr.IsValid())
				{
					ExternalStreamingObject->DumpStateLog(*HierarchicalLogAr);
				}

				if (InContext.GeneratedExternalStreamingObjects)
				{
					InContext.GeneratedExternalStreamingObjects->Add(ExternalStreamingObject);
				}
			}
		}
		bStreamingGenerationSuccess &= bExternalDataLayerGenerationSuccess;
		FlushStreaming();
	}

	// Generate streaming for the Base container instance
	const UActorDescContainerInstance* BaseContainerInstance = InParams.ContainerInstanceCollection.GetBaseContainerInstance();
	const bool bBaseContainerGenerationSuccess = GenerateRuntimeHash(BaseContainerInstance);
	if (bBaseContainerGenerationSuccess && HierarchicalLogAr.IsValid())
	{
		RuntimeHash->DumpStateLog(*HierarchicalLogAr);
	}
	bStreamingGenerationSuccess &= bBaseContainerGenerationSuccess;
	return bStreamingGenerationSuccess;
}

class FStreamingGenerationContextCopy : public FStreamingGenerationContextProxy
{
public:
	FStreamingGenerationContextCopy(const FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams& InParams)
		: FStreamingGenerationContextProxy(nullptr)
		, StreamingGenerator(InParams)
	{}

	void SetSourceContext(const IStreamingGenerationContext* InStreamingGenerationContext)
	{
		SourceContext = InStreamingGenerationContext;
	}

	FWorldPartitionStreamingGenerator StreamingGenerator;
};

TUniquePtr<IStreamingGenerationContext> UWorldPartition::GenerateStreamingGenerationContext(const FGenerateStreamingParams& InParams, FGenerateStreamingContext& InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartition::GenerateStreamingGenerationContext);

	TErrorHandlerSelector<FStreamingGenerationLogErrorHandler> ErrorHandlerSelector(InParams.ErrorHandler);

	FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams StreamingGeneratorParams = FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams()
		.SetWorldPartitionContext(this)
		.SetDefaultGrid(RuntimeHash ? RuntimeHash->GetDefaultGrid() : NAME_None)
		.SetIsValidGrid([this](FName GridName, const UClass* ActorClass) { return RuntimeHash->IsValidGrid(GridName, ActorClass); })
		.SetIsValidHLODLayer([this](FName GridName, const FSoftObjectPath& HLODLayerPath) { return RuntimeHash->IsValidHLODLayer(GridName, HLODLayerPath); })
		.SetErrorHandler(ErrorHandlerSelector.Get())
		.SetEnableStreaming(IsStreamingEnabled())
		.SetCreateContainerResolver(FEditorPathHelper::IsEnabled());

	TUniquePtr<FStreamingGenerationContextCopy> StreamingGenerationContextCopy = MakeUnique<FStreamingGenerationContextCopy>(StreamingGeneratorParams);

	StreamingGenerationContextCopy->StreamingGenerator.PreparationPhase(InParams.ContainerInstanceCollection);

	const IStreamingGenerationContext* StreamingGenerationContext = StreamingGenerationContextCopy->StreamingGenerator.GetStreamingGenerationContext(InParams.ContainerInstanceCollection);
	check(StreamingGenerationContext);

	StreamingGenerationContextCopy->SetSourceContext(StreamingGenerationContext);

	return MoveTemp(StreamingGenerationContextCopy);
}

void UWorldPartition::FlushStreaming()
{
	RuntimeHash->FlushStreamingContent();
	StreamingPolicy = nullptr;
	GeneratedLevelStreamingPackageNames.Empty();
}

bool UWorldPartition::HasStreamingContent() const
{
	return RuntimeHash && RuntimeHash->HasStreamingContent();
}

URuntimeHashExternalStreamingObjectBase* UWorldPartition::FlushStreamingToExternalStreamingObject()
{
	URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = RuntimeHash->StoreStreamingContentToExternalStreamingObject();
	check(ExternalStreamingObject);

	StreamingPolicy->StoreStreamingContentToExternalStreamingObject(*ExternalStreamingObject);

	FlushStreaming();
	return ExternalStreamingObject;
}

static void ExtractContentBundleContainerInstances(const FActorDescContainerInstanceCollection* InContainerInstanceCollection, TArray<const UActorDescContainerInstance*>& OutContentBundleContainerInstances, TArray<const UActorDescContainerInstance*>& OutNonContentBundleContainerInstances)
{
	InContainerInstanceCollection->ForEachActorDescContainerInstance([&OutContentBundleContainerInstances, &OutNonContentBundleContainerInstances](const UActorDescContainerInstance* InActorDescContainerInstance)
	{
		if (InActorDescContainerInstance->GetContentBundleGuid().IsValid())
		{
			OutContentBundleContainerInstances.Add(InActorDescContainerInstance);
		}
		else
		{
			OutNonContentBundleContainerInstances.Add(InActorDescContainerInstance);
		}
	});
}

void UWorldPartition::SetupHLODActors(const FSetupHLODActorsParams& Params)
{
	TArray<const UActorDescContainerInstance*> ContentBundleContainerInstances;
	TArray<const UActorDescContainerInstance*> BaseAndEDLContainerInstances;
	ExtractContentBundleContainerInstances(this, ContentBundleContainerInstances, BaseAndEDLContainerInstances);

	auto SetupHLODActorsForCollection = [this, &Params](const FStreamingGenerationContainerInstanceCollection& InContainerInstanceCollection)
	{
		TErrorHandlerSelector<FStreamingGenerationLogErrorHandler> ErrorHandlerSelector;
		FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams StreamingGeneratorParams = FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams()
			.SetWorldPartitionContext(this)
			.SetErrorHandler(ErrorHandlerSelector.Get())
			.SetEnableStreaming(IsStreamingEnabled())
			.SetFilteredClasses({ AWorldPartitionHLOD::StaticClass() })
			.SetDefaultGrid(RuntimeHash ? RuntimeHash->GetDefaultGrid() : NAME_None)
			.SetIsValidGrid([this](FName GridName, const UClass* ActorClass) { return RuntimeHash->IsValidGrid(GridName, ActorClass); })
			.SetIsValidHLODLayer([this](FName GridName, const FSoftObjectPath& HLODLayerPath) { return RuntimeHash->IsValidHLODLayer(GridName, HLODLayerPath); })
			.SetIncludeCustomHLODActorsWithOtherLayerType(true);

		FWorldPartitionStreamingGenerator StreamingGenerator(StreamingGeneratorParams);
		StreamingGenerator.PreparationPhase(InContainerInstanceCollection);

		TUniquePtr<FArchive> LogFileAr = FWorldPartitionStreamingGenerator::CreateDumpStateLogArchive(TEXT("HLOD"));
		if (LogFileAr.IsValid())
		{
			FHierarchicalLogArchive HierarchicalLogAr(*LogFileAr);
			StreamingGenerator.DumpStateLog(HierarchicalLogAr);
		}

		RuntimeHash->SetupHLODActors(StreamingGenerator.GetStreamingGenerationContext(InContainerInstanceCollection), Params);
	};

	UWorldPartitionRuntimeHash::ExecutePreSetupHLODActors(this, Params);

	// Process all Content Bundle container instances
	for (const UActorDescContainerInstance* ContentBundleContainerInstance : ContentBundleContainerInstances)
	{
		FStreamingGenerationContainerInstanceCollection ContentBundleCollection({ ContentBundleContainerInstance }, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAsContentBundle);
		SetupHLODActorsForCollection(ContentBundleCollection);
	}

	// Single pass for base and EDL container instances
	if (!BaseAndEDLContainerInstances.IsEmpty())
	{
		FStreamingGenerationContainerInstanceCollection Collection(BaseAndEDLContainerInstances, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs);
		SetupHLODActorsForCollection(Collection);
	}

	UWorldPartitionRuntimeHash::ExecutePostSetupHLODActors(this, Params);
}

FStreamingGenerationContainerInstanceCollection::FStreamingGenerationContainerInstanceCollection(std::initializer_list<TObjectPtr<const UActorDescContainerInstance>> ActorDescContainerInstanceArray, const ECollectionType& InCollectionType)
	: TActorDescContainerInstanceCollection<TObjectPtr<const UActorDescContainerInstance>>(ActorDescContainerInstanceArray)
	, CollectionType(InCollectionType)
{
	InitializeCollection();
}

FStreamingGenerationContainerInstanceCollection::FStreamingGenerationContainerInstanceCollection(const TArray<const UActorDescContainerInstance*>& ActorDescContainerInstances, const ECollectionType& InCollectionType)
	: TActorDescContainerInstanceCollection<TObjectPtr<const UActorDescContainerInstance>>(ActorDescContainerInstances)
	, CollectionType(InCollectionType)
{
	InitializeCollection();
}

UWorld* FStreamingGenerationContainerInstanceCollection::GetWorld() const
{
	UWorldPartition* WorldPartition = GetBaseContainerInstance()->GetOuterWorldPartition();
	check(WorldPartition);
	UWorld* World = WorldPartition->GetWorld();
	check(World);
	return World;
}

FGuid FStreamingGenerationContainerInstanceCollection::GetContentBundleGuid() const
{
	FGuid ContentBundleGuid;
	if (CollectionType == ECollectionType::BaseAsContentBundle)
	{
		check(GetActorDescContainerCount() == 1);
		ContentBundleGuid = GetBaseContainerInstance()->GetContentBundleGuid();
		check(ContentBundleGuid.IsValid());
	}
	return ContentBundleGuid;
}

//@todo_ow: Once ContentBundle code is removed, this function will always return a base ActorDescContainer that cannot have a valid content bundle
const UActorDescContainerInstance* FStreamingGenerationContainerInstanceCollection::GetBaseContainerInstance() const
{
	check(CollectionType != ECollectionType::Invalid);
	check(!IsEmpty());
	const UActorDescContainerInstance* ActorDescContainerInstance = ActorDescContainerInstanceCollection[BaseContainerIdx];
	check(ActorDescContainerInstance);
	// This function is not designed to return a valid ActorDescContainer if it has external content (except for ContentBundle collection type)
	check(!ActorDescContainerInstance->HasExternalContent() || (ActorDescContainerInstance->GetContentBundleGuid().IsValid() && (CollectionType == ECollectionType::BaseAsContentBundle || CollectionType == ECollectionType::BaseAndAny)));
	return ActorDescContainerInstance;
}

FName FStreamingGenerationContainerInstanceCollection::GetBaseContainerInstancePackageName() const
{
	return GetBaseContainerInstance()->GetContainerPackage();
}

TArrayView<const UActorDescContainerInstance* const> FStreamingGenerationContainerInstanceCollection::GetExternalDataLayerContainerInstances() const
{
	check(CollectionType != ECollectionType::Invalid);
	check(!IsEmpty());

	if (ExternalDataLayerStartIdx != INDEX_NONE)
	{
		int32 Size = ((ContentBundleStartIdx != INDEX_NONE) ? ContentBundleStartIdx : GetActorDescContainerCount()) - ExternalDataLayerStartIdx;
		return MakeArrayView(&ActorDescContainerInstanceCollection[ExternalDataLayerStartIdx], Size);
	}

	return TArrayView<const UActorDescContainerInstance*>();
}

TArrayView<const UActorDescContainerInstance* const> FStreamingGenerationContainerInstanceCollection::GetContentBundleContainerInstances() const
{
	check(CollectionType != ECollectionType::Invalid);
	check(!IsEmpty());

	if (ContentBundleStartIdx != INDEX_NONE)
	{
		int32 Size = GetActorDescContainerCount() - ContentBundleStartIdx;
		return MakeArrayView(&ActorDescContainerInstanceCollection[ContentBundleStartIdx], Size);
	}

	return TArrayView<const UActorDescContainerInstance*>();
}

void FStreamingGenerationContainerInstanceCollection::OnCollectionChanged()
{
	InitializeCollection();
}

void FStreamingGenerationContainerInstanceCollection::InitializeCollection()
{
	ExternalDataLayerStartIdx = INDEX_NONE;
	ContentBundleStartIdx = INDEX_NONE;

	check(!IsEmpty());
	check(CollectionType != ECollectionType::Invalid);
	if (IsEmpty())
	{
		return;
	}

	if (CollectionType == ECollectionType::BaseAsContentBundle)
	{
		check(GetActorDescContainerCount() == 1);
		check(GetContentBundleGuid().IsValid());
		ContentBundleStartIdx = 0;
		return;
	}

	check((CollectionType == ECollectionType::BaseAndEDLs) || (CollectionType == ECollectionType::BaseAndAny));
	check(!GetContentBundleGuid().IsValid());

	if (CollectionType == ECollectionType::BaseAndEDLs)
	{
		// When type is set to BaseAndEDL, we remove ContentBundle containers from the collection.
		// BaseAndEDL type assumes ContentBundle containers are generated separately one at a time.
		check(!ShouldRegisterDelegates());
		ActorDescContainerInstanceCollection.SetNum(Algo::RemoveIf(ActorDescContainerInstanceCollection, [](const UActorDescContainerInstance* ActorDescContainerInstance) { return ActorDescContainerInstance->GetContentBundleGuid().IsValid(); }));
	}

	int32 BaseContainerCount = Algo::CountIf(ActorDescContainerInstanceCollection, [](const UActorDescContainerInstance* ActorDescContainerInstance) { return !ActorDescContainerInstance->HasExternalContent(); });
	check(BaseContainerCount == 1);

	// Sort containers : Base, EDLs, ContentBundles
	if (GetActorDescContainerCount() > 1)
	{
		auto GetContainerSortValue = [](const UActorDescContainerInstance* ActorDescContainerInstance) { return (!!ActorDescContainerInstance->GetExternalDataLayerAsset() ? 1 : ActorDescContainerInstance->GetContentBundleGuid().IsValid() ? 2 : 0); };
		Algo::Sort(ActorDescContainerInstanceCollection, [&GetContainerSortValue](const UActorDescContainerInstance* A, const UActorDescContainerInstance* B)
		{
			int32 AValue = GetContainerSortValue(A);
			int32 BValue = GetContainerSortValue(B);
			return (AValue == BValue) ? A->GetContainerPackage().LexicalLess(B->GetContainerPackage()) : (AValue < BValue);
		});

		int32 Index = 0;
		for (const UActorDescContainerInstance* ContainerInstance : ActorDescContainerInstanceCollection)
		{
			if (ExternalDataLayerStartIdx == INDEX_NONE && ContainerInstance->GetExternalDataLayerAsset())
			{
				ExternalDataLayerStartIdx = Index;
			}
			else if (ContentBundleStartIdx == INDEX_NONE && ContainerInstance->GetContentBundleGuid().IsValid())
			{
				ContentBundleStartIdx = Index;
			}
			++Index;
		}

#if DO_CHECK
		// Validation
		check((ContentBundleStartIdx == INDEX_NONE) || (ContentBundleStartIdx > ExternalDataLayerStartIdx));
		check(GetBaseContainerInstance());
		TArrayView<const UActorDescContainerInstance* const> ExternalDataLayerContainers = GetExternalDataLayerContainerInstances();
		Algo::ForEach(ExternalDataLayerContainers, [](const UActorDescContainerInstance* ActorDescContainerInstance) { check(ActorDescContainerInstance->GetExternalDataLayerAsset()) });
		TArrayView<const UActorDescContainerInstance* const> ContentBundleContainers = GetContentBundleContainerInstances();
		Algo::ForEach(ContentBundleContainers, [](const UActorDescContainerInstance* ActorDescContainerInstance) { check(ActorDescContainerInstance->GetContentBundleGuid().IsValid()) });
#endif
	}
}

void UWorldPartition::CheckForErrors(IStreamingGenerationErrorHandler* ErrorHandler) const
{
	FCheckForErrorsParams Params = FCheckForErrorsParams()
		.SetErrorHandler(ErrorHandler)
		.SetActorDescContainerInstanceCollection(this)
		.SetEnableStreaming(IsStreamingEnabled());

	CheckForErrors(Params);
}

/* Static version, mainly used by changelist validation */
void UWorldPartition::CheckForErrors(const FCheckForErrorsParams& InParams)
{
	check(InParams.ErrorHandler);
	check(InParams.ActorDescContainerInstanceCollection);

	// Prepare ActorGuidsToContainerInstanceMap
	TMap<FGuid, const UActorDescContainerInstance*> ActorGuidsToContainerInstanceMap;
	InParams.ActorDescContainerInstanceCollection->ForEachActorDescContainerInstance([&ActorGuidsToContainerInstanceMap](const UActorDescContainerInstance* InContainerInstance)
	{
		for (UActorDescContainerInstance::TConstIterator<> Iterator(InContainerInstance); Iterator; ++Iterator)
		{
			check(!ActorGuidsToContainerInstanceMap.Contains(Iterator->GetGuid()));
			ActorGuidsToContainerInstanceMap.Add(Iterator->GetGuid(), InContainerInstance);
		}
	});

	// Changelist validation can pass Content Bundle containers that are not necessarily registered in the collection's BaseContainerInstance world partition.
	// Because these containers are validated one at a time, thus represent the base container for the generator's collection,
	// we need to setup the generator's WorldPartitionContext based on the Content Bundle container.
	// (Unregistered Content Bundle containers will differ from a registered BaseContainerInstance of the provided collection)
	auto ValidateCollection = [&InParams, &ActorGuidsToContainerInstanceMap](FStreamingGenerationContainerInstanceCollection& InCollection)
	{
		TErrorHandlerSelector<FStreamingGenerationLogErrorHandler> ErrorHandlerSelector(InParams.ErrorHandler);
		const UActorDescContainerInstance* BaseContainerInstance = InCollection.GetBaseContainerInstance();
		const UWorldPartition* WorldPartition = BaseContainerInstance->GetOuterWorldPartition();
		const TObjectPtr<UWorldPartitionRuntimeHash>& WorldPartitionRuntimeHash = WorldPartition ? WorldPartition->RuntimeHash : nullptr;

		FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams StreamingGeneratorParams = FWorldPartitionStreamingGenerator::FWorldPartitionStreamingGeneratorParams()
			.SetWorldPartitionContext(WorldPartition)
			.SetHandleUnsavedActors(!!WorldPartition)
			.SetIsPIE(WorldPartition ? WorldPartition->bIsPIE : false)
			.SetIncludeStandaloneHLODActors(WorldPartition ? WorldPartition->IsMainWorldPartition() : false)
			.SetErrorHandler(ErrorHandlerSelector.Get())
			.SetDefaultGrid(WorldPartitionRuntimeHash ? WorldPartitionRuntimeHash->GetDefaultGrid() : NAME_None)
			.SetIsValidGrid([WorldPartitionRuntimeHash](FName GridName, const UClass* ActorClass) { return WorldPartitionRuntimeHash ? WorldPartitionRuntimeHash->IsValidGrid(GridName, ActorClass) : true; })
			.SetIsValidHLODLayer([WorldPartitionRuntimeHash](FName GridName, const FSoftObjectPath& HLODLayerPath) { return WorldPartitionRuntimeHash ? WorldPartitionRuntimeHash->IsValidHLODLayer(GridName, HLODLayerPath) : true; })
			.SetEnableStreaming(InParams.bEnableStreaming)
			.SetActorGuidsToContainerInstanceMap(ActorGuidsToContainerInstanceMap);

		FWorldPartitionStreamingGenerator StreamingGenerator(StreamingGeneratorParams);
		StreamingGenerator.PreparationPhase(InCollection);
	};

	// @todo_ow : Once content bundles are remove, we will only do this validation in 1 pass
	TArray<const UActorDescContainerInstance*> ContentBundleContainerInstances;
	TArray<const UActorDescContainerInstance*> BaseAndEDLContainerInstances;
	ExtractContentBundleContainerInstances(InParams.ActorDescContainerInstanceCollection, ContentBundleContainerInstances, BaseAndEDLContainerInstances);

	if (!BaseAndEDLContainerInstances.IsEmpty())
	{
		FStreamingGenerationContainerInstanceCollection Collection(BaseAndEDLContainerInstances, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAndEDLs);
		ValidateCollection(Collection);
	}

	for (const UActorDescContainerInstance* ContentBundleContainer : ContentBundleContainerInstances)
	{
		FStreamingGenerationContainerInstanceCollection Collection({ ContentBundleContainer }, FStreamingGenerationContainerInstanceCollection::ECollectionType::BaseAsContentBundle);
		ValidateCollection(Collection);
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
