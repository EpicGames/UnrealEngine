// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionStreamingGenerationContext.h"
#include "WorldPartition/WorldPartitionStreamingGeneration.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"

#if WITH_EDITOR

const FStreamingGenerationActorDescView& IStreamingGenerationContext::FActorInstance::GetActorDescView() const
{
	return ActorSetInstance->ActorSetContainerInstance->ActorDescViewMap->FindByGuidChecked(ActorGuid);
}

const FActorContainerID& IStreamingGenerationContext::FActorInstance::GetContainerID() const
{
	return ActorSetInstance->ContainerID;
}

const FTransform& IStreamingGenerationContext::FActorInstance::GetTransform() const
{
	return ActorSetInstance->Transform;
}

const FBox IStreamingGenerationContext::FActorInstance::GetBounds() const
{
	return ActorSetInstance->Bounds;
}

const UExternalDataLayerAsset* IStreamingGenerationContext::FActorSetInstance::GetExternalDataLayerAsset() const
{
	auto IsAnExternalDataLayerPred = [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); };
	if (const UDataLayerInstance* const* ExternalDataLayerInstance = DataLayers.FindByPredicate(IsAnExternalDataLayerPred))
	{
		return CastChecked<UExternalDataLayerInstance>(*ExternalDataLayerInstance)->GetExternalDataLayerAsset();
	}

	return nullptr;
}
#endif
