// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * WorldPartitionStreamingPolicy Implementation
 */

#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "Algo/Find.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Level.h"
#include "Engine/NetConnection.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/HashBuilder.h"
#include "Misc/ScopeExit.h"
#include "Stats/Stats.h"
#include "UObject/Package.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/HLOD/HLODRuntimeSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionReplay.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionStreamingPolicy)

#define LOCTEXT_NAMESPACE "WorldPartitionStreamingPolicy"

int32 GBlockOnSlowStreaming = 1;
static FAutoConsoleVariableRef CVarBlockOnSlowStreaming(
	TEXT("wp.Runtime.BlockOnSlowStreaming"),
	GBlockOnSlowStreaming,
	TEXT("Set if streaming needs to block when to slow to catchup."));

#if !UE_BUILD_SHIPPING
bool GDisplayStreamingPerformanceForNonBlockingStreaming = false;
static FAutoConsoleVariableRef CVarDisplayStreamingPerformanceForNonBlockingStreaming(
	TEXT("wp.Runtime.DisplayStreamingPerformanceForNonBlockingStreaming"),
	GDisplayStreamingPerformanceForNonBlockingStreaming,
	TEXT("Display streaming performance updates for non blocking streaming"));
#endif

static FString GServerDisallowStreamingOutDataLayersString;
static FAutoConsoleVariableRef CVarServerDisallowStreamingOutDataLayers(
	TEXT("wp.Runtime.ServerDisallowStreamingOutDataLayers"),
	GServerDisallowStreamingOutDataLayersString,
	TEXT("Comma separated list of data layer names that aren't allowed to be unloaded or deactivated on the server"),
	ECVF_ReadOnly);

bool UWorldPartitionStreamingPolicy::IsUpdateOptimEnabled = true;
FAutoConsoleVariableRef UWorldPartitionStreamingPolicy::CVarUpdateOptimEnabled(
	TEXT("wp.Runtime.UpdateStreaming.EnableOptimization"),
	UWorldPartitionStreamingPolicy::IsUpdateOptimEnabled,
	TEXT("Set to 1 to enable an optimization that skips world partition streaming update\n")
	TEXT("if nothing relevant changed since last update."),
	ECVF_Default);

int32 UWorldPartitionStreamingPolicy::ForceUpdateFrameCount = 0;
FAutoConsoleVariableRef UWorldPartitionStreamingPolicy::CVarForceUpdateFrameCount(
	TEXT("wp.Runtime.UpdateStreaming.ForceUpdateFrameCount"),
	UWorldPartitionStreamingPolicy::ForceUpdateFrameCount,
	TEXT("Frequency (in frames) at which world partition streaming update will be executed regardless if no changes are detected."),
	ECVF_Default);

bool UWorldPartitionStreamingPolicy::IsAsyncUpdateStreamingStateEnabled = false;
FAutoConsoleVariableRef UWorldPartitionStreamingPolicy::CVarAsyncUpdateStreamingStateEnabled(
	TEXT("wp.Runtime.UpdateStreaming.EnableAsyncUpdate"),
	UWorldPartitionStreamingPolicy::IsAsyncUpdateStreamingStateEnabled,
	TEXT("Set to enable asynchronous World Partition UpdateStreamingState."),
	ECVF_Default);

UWorldPartitionStreamingPolicy::UWorldPartitionStreamingPolicy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) 
	, WorldPartition(nullptr)
	, CurrentStreamingPerformance(EWorldPartitionStreamingPerformance::Good)
	, bCurrentBlockOnSlowStreaming(false)
	, bShouldMergeStreamingSourceInfo(false)
	, bCriticalPerformanceRequestedBlockTillOnWorld(false)
	, CriticalPerformanceBlockTillLevelStreamingCompletedEpoch(0)
	, ProcessedToLoadCells(0)
	, ProcessedToActivateCells(0)
	, ServerStreamingStateEpoch(INT_MIN)
	, ServerStreamingEnabledEpoch(INT_MIN)
	, UpdateStreamingHash(0)
	, UpdateStreamingSourcesHash(0)
	, UpdateStreamingStateCounter(0)
	, AsyncUpdateTaskState(EAsyncUpdateTaskState::None)
	, AsyncShouldSkipUpdateCounter(0)
#if !UE_BUILD_SHIPPING
	, OnScreenMessageStartTime(0.0)
	, OnScreenMessageStreamingPerformance(EWorldPartitionStreamingPerformance::Good)
	, bOnScreenMessageShouldBlock(false)
#endif
{
	if (!IsTemplate())
	{
		WorldPartition = GetOuterUWorldPartition();
		check(WorldPartition);
	}
}

void UWorldPartitionStreamingPolicy::UpdateStreamingSources(bool bCanOptimizeUpdate)
{
	check(IsInGameThread());

	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingSources);

	if (!WorldPartition->CanStream())
	{
		CurrentState.StreamingSources.Reset();
		UpdateStreamingSourcesHash = 0;
		return;
	}

	const UWorldPartitionSubsystem* WorldPartitionSubsystem = GetWorld()->GetSubsystem<UWorldPartitionSubsystem>();
	const uint32 NewUpdateStreamingSourcesHash = WorldPartitionSubsystem->GetStreamingSourcesHash();
	if (bCanOptimizeUpdate && (UpdateStreamingSourcesHash == NewUpdateStreamingSourcesHash))
	{
		// Streaming Source Hash doesn't take velocity into account, we still need to update the streaming sources velocity
		WorldPartitionSubsystem->UpdateStreamingSourcesVelocity(WorldPartition, CurrentState.StreamingSources);
		return;
	}

	CurrentState.StreamingSources.Reset();
	WorldPartitionSubsystem->GetStreamingSources(WorldPartition, CurrentState.StreamingSources);
	UpdateStreamingSourcesHash = NewUpdateStreamingSourcesHash;
}

bool UWorldPartitionStreamingPolicy::IsInBlockTillLevelStreamingCompleted(bool bIsCausedByBadStreamingPerformance /* = false*/) const
{
	check(IsInGameThread());

	const UWorld* World = GetWorld();
	const bool bIsInBlockTillLevelStreamingCompleted = World->GetIsInBlockTillLevelStreamingCompleted();
	if (bIsCausedByBadStreamingPerformance)
	{
		return bIsInBlockTillLevelStreamingCompleted &&
				(CurrentStreamingPerformance != EWorldPartitionStreamingPerformance::Good) &&
				(CriticalPerformanceBlockTillLevelStreamingCompletedEpoch == World->GetBlockTillLevelStreamingCompletedEpoch());
	}
	return bIsInBlockTillLevelStreamingCompleted;
}

int32 UWorldPartitionStreamingPolicy::ComputeServerStreamingEnabledEpoch() const
{
	return WorldPartition->IsServer() ? (WorldPartition->IsServerStreamingEnabled() ? 1 : 0) : INT_MIN;
}

bool UWorldPartitionStreamingPolicy::IsUpdateStreamingOptimEnabled()
{
	return UWorldPartitionStreamingPolicy::IsUpdateOptimEnabled &&
		((FWorldPartitionStreamingSource::GetLocationQuantization() > 0) ||
		 (FWorldPartitionStreamingSource::GetRotationQuantization() > 0));
};

uint32 UWorldPartitionStreamingPolicy::ComputeUpdateStreamingHash(bool bCanOptimizeUpdate) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::ComputeUpdateStreamingHash);
	if (bCanOptimizeUpdate)
	{
		const bool bIsStreaming3D = WorldPartition->RuntimeHash && WorldPartition->RuntimeHash->IsStreaming3D();

		// Build hash that will be used to detect relevant changes
		FHashBuilder HashBuilder;
		if (WorldPartition->RuntimeHash)
		{
			HashBuilder << WorldPartition->RuntimeHash->ComputeUpdateStreamingHash();
#if !UE_BUILD_SHIPPING
			HashBuilder << UWorldPartitionSubsystem::GetOverriddenLoadingRangesEpoch();
#endif
		}
		HashBuilder << ComputeServerStreamingEnabledEpoch();
		HashBuilder << WorldPartition->GetStreamingStateEpoch();
		HashBuilder << bIsStreaming3D;
		for (const FWorldPartitionStreamingSource& Source : CurrentState.StreamingSources)
		{
			HashBuilder << Source.GetHash(bIsStreaming3D);
		}

		if (WorldPartition->IsServer())
		{
			HashBuilder << GetWorld()->GetSubsystem<UWorldPartitionSubsystem>()->GetServerClientsVisibleLevelsHash();
		}

#if WITH_EDITOR
		HashBuilder << GetWorld()->GetSubsystem<UWorldPartitionSubsystem>()->GetDynamicLoadingRangeScales().LinearScale;
#endif // WITH_EDITOR

		return HashBuilder.GetHash();
	}

	return 0;
};

bool UWorldPartitionStreamingPolicy::GetIntersectingCells(const TArray<FWorldPartitionStreamingQuerySource>& InSources, TArray<const IWorldPartitionCell*>& OutCells) const
{
	if (!WorldPartition || !WorldPartition->RuntimeHash)
	{
		return false;
	}

	FWorldPartitionQueryCache QueryCache;
	TSet<const UWorldPartitionRuntimeCell*> Cells;
	for (const FWorldPartitionStreamingQuerySource& Source : InSources)
	{
		WorldPartition->RuntimeHash->ForEachStreamingCellsQuery(Source, [&Cells](const UWorldPartitionRuntimeCell* Cell)
		{
			Cells.Add(Cell);
			return true;
		}, &QueryCache);
	}

	TArray<const UWorldPartitionRuntimeCell*> SortedCells = Cells.Array();
	Algo::Sort(SortedCells, [&QueryCache](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB)
	{
		int32 SortCompare = CellA->SortCompare(CellB);
		if (SortCompare == 0)
		{
			// Closest distance (lower value is higher prio)
			const double Diff = QueryCache.GetCellMinSquareDist(CellA) - QueryCache.GetCellMinSquareDist(CellB);
			if (FMath::IsNearlyZero(Diff))
			{
				return CellA->GetLevelPackageName().LexicalLess(CellB->GetLevelPackageName());
			}

			return Diff < 0;
		}
		return SortCompare < 0;
	});

	OutCells.Reserve(SortedCells.Num());
	for (const UWorldPartitionRuntimeCell* Cell : SortedCells)
	{
		OutCells.Add(Cell);
	}
	return true;
}

const TSet<FName>& UWorldPartitionStreamingPolicy::GetServerDisallowedStreamingOutDataLayers() const
{
	if (!CachedServerDisallowStreamingOutDataLayers.IsSet())
	{
		TSet<FName> ServerDisallowStreamingOutDataLayers;

		if (!GServerDisallowStreamingOutDataLayersString.IsEmpty())
		{
			TArray<FString> AllDLAssetsStrings;
			GServerDisallowStreamingOutDataLayersString.ParseIntoArray(AllDLAssetsStrings, TEXT(","));

			if (const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
			{
				for (const FString& DataLayerAssetName : AllDLAssetsStrings)
				{
					if (const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAssetName(FName(DataLayerAssetName)))
					{
						ServerDisallowStreamingOutDataLayers.Add(DataLayerInstance->GetFName());
					}
				}
			}
		}

		CachedServerDisallowStreamingOutDataLayers = ServerDisallowStreamingOutDataLayers;
	}
	
	return CachedServerDisallowStreamingOutDataLayers.GetValue();
}

void UWorldPartitionStreamingPolicy::UpdateStreamingState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState);

	++UpdateStreamingStateCounter;

	UWorld* World = GetWorld();
	check(World);
	check(World->IsGameWorld());

	const bool bLastUpdateCompletedLoadingAndActivation = ((ProcessedToActivateCells + ProcessedToLoadCells) == (TargetState.ToActivateCells.Num() + TargetState.ToLoadCells.Num()));

	ProcessedToLoadCells = 0;
	ProcessedToActivateCells = 0;
	TargetState.Reset();

	// Last update was asynchronous
	if (WaitForAsyncUpdateStreamingState())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateTargetStateFromAsyncTask);

		check(AsyncUpdateStreamingStateTask.IsCompleted());
		PostUpdateStreamingStateInternal_GameThread(AsyncTaskTargetState);

		// Update Target State using asynchronous task results
		// Filter result (AsyncTaskTargetState) using CurrentState as asynchronous tasks started with a snapshot version of ActivatedCells/LoadedCells
		for (const UWorldPartitionRuntimeCell* Cell : AsyncTaskTargetState.ToActivateCells)
		{
			if (!CurrentState.ActivatedCells.Contains(Cell))
			{
				TargetState.ToActivateCells.Add(Cell);
			}
		}
		for (const UWorldPartitionRuntimeCell* Cell : AsyncTaskTargetState.ToLoadCells)
		{
			if (!CurrentState.LoadedCells.Contains(Cell))
			{
				TargetState.ToLoadCells.Add(Cell);
			}
		}

		// Reset everything related to last asynchronous tasks
		check(AsyncUpdateTaskState == EAsyncUpdateTaskState::Started);
		AsyncUpdateTaskState = EAsyncUpdateTaskState::None;
		AsyncUpdateStreamingStateTask = UE::Tasks::TTask<void>();
		AsyncTaskCurrentState.Reset();
		AsyncTaskTargetState.Reset();
	}

	check(AsyncUpdateTaskState == EAsyncUpdateTaskState::None);

	// Determine if the World's BlockTillLevelStreamingCompleted was triggered by WorldPartitionStreamingPolicy
	if (bCriticalPerformanceRequestedBlockTillOnWorld && IsInBlockTillLevelStreamingCompleted())
	{
		bCriticalPerformanceRequestedBlockTillOnWorld = false;
		CriticalPerformanceBlockTillLevelStreamingCompletedEpoch = World->GetBlockTillLevelStreamingCompletedEpoch();
	}

	const bool bIsServer = WorldPartition->IsServer();
	const bool bCanStream = WorldPartition->CanStream();

	// If server (non-streaming) has nothing to do, early out
	if (bIsServer &&
		bCanStream &&
		bLastUpdateCompletedLoadingAndActivation &&
		!WorldPartition->IsServerStreamingEnabled() &&
		(ServerStreamingEnabledEpoch == ComputeServerStreamingEnabledEpoch()) &&
		(ServerStreamingStateEpoch == WorldPartition->GetStreamingStateEpoch()))
	{
		return;
	}

	const bool bForceFrameUpdate = (UWorldPartitionStreamingPolicy::ForceUpdateFrameCount > 0) ? ((UpdateStreamingStateCounter % UWorldPartitionStreamingPolicy::ForceUpdateFrameCount) == 0) : false;
	const bool bCanOptimizeUpdate =
		WorldPartition->RuntimeHash &&
		bCanStream &&
		!bForceFrameUpdate &&												// We guarantee to update every N frame to force some internal updates like UpdateStreamingPerformance
		IsUpdateStreamingOptimEnabled() &&									// Check CVars to see if optimization is enabled
		bLastUpdateCompletedLoadingAndActivation &&							// Don't optimize if last frame didn't process all cells to load/activate
		!IsInBlockTillLevelStreamingCompleted() &&							// Don't optimize when inside UWorld::BlockTillLevelStreamingCompleted
		CurrentState.ActivatedCells.GetPendingAddToWorldCells().IsEmpty(); 	// Don't optimize when remaining cells to add to world

	// Update streaming sources
	UpdateStreamingSources(bCanOptimizeUpdate);

	// Determine if update will be async or not
	const bool bIsDedicatedServer = (World->GetNetMode() == NM_DedicatedServer);
	const bool bCanUpdateAsync = UWorldPartitionStreamingPolicy::IsAsyncUpdateStreamingStateEnabled && !bIsDedicatedServer && bCanStream && WorldPartition->IsInitialized() && !IsInBlockTillLevelStreamingCompleted();

	// Detect if nothing relevant changed and early out
	const uint32 NewUpdateStreamingHash = ComputeUpdateStreamingHash(bCanOptimizeUpdate);
	const bool bIsUpdateStreamingHashIdentical = NewUpdateStreamingHash && (UpdateStreamingHash == NewUpdateStreamingHash);
	AsyncShouldSkipUpdateCounter = (bIsUpdateStreamingHashIdentical && bCanUpdateAsync) ? (AsyncShouldSkipUpdateCounter + 1) : 0;
	// Since the asynchronous update is working with a snapshot of the last frame, wait for 2 consecutive update without any changes before deciding to skip the update
	const bool bShouldSkipUpdate = bIsUpdateStreamingHashIdentical && (!bCanUpdateAsync || (AsyncShouldSkipUpdateCounter >= 2));
	if (bShouldSkipUpdate)
	{
		return;
	}

	// Update new streaming sources hash
	UpdateStreamingHash = NewUpdateStreamingHash;

	// UpdateStreamingStateInternal asynchronously or synchronously
	if (bCanUpdateAsync)
	{
		// Put state to Pending
		// Wait for WorldPartitionSubsystem to finish accessing streaming cells (used to sort)
		// Once it's the case, OnStreamingStateUpdated will be called and async tasks will be created and dispatched
		AsyncUpdateTaskState = EAsyncUpdateTaskState::Pending;
	}
	else
	{
		TargetState.Reset();
		UWorldPartitionStreamingPolicy::UpdateStreamingStateInternal(FUpdateStreamingStateParams(this, CurrentState), TargetState);
		PostUpdateStreamingStateInternal_GameThread(TargetState);
	}
}

void UWorldPartitionStreamingPolicy::UpdateStreamingStateInternal(const FUpdateStreamingStateParams& InParams, FWorldPartitionUpdateStreamingTargetState& OutTargetState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingStateInternal);

	OutTargetState.bUpdateServerEpoch = false;

	check(InParams.FrameActivateCells.IsEmpty());
	check(InParams.FrameLoadCells.IsEmpty());
	check(InParams.FrameAnalysePerformanceCells.IsEmpty());
	check(OutTargetState.IsEmpty());

	FWorldPartitionStreamingContext Context(InParams.DataLayersLogicOperator, InParams.GetWorldDataLayersEffectiveStates(), InParams.PolicyUpdateStreamingStateEpoch
#if WITH_EDITOR
		, InParams.DynamicLoadingRangeScales
#endif // WITH_EDITOR
	);

	ON_SCOPE_EXIT
	{
		// Reset frame cells to avoid reallocation at every Update call
		InParams.FrameActivateCells.Reset();
		InParams.FrameLoadCells.Reset();
		InParams.FrameAnalysePerformanceCells.Reset();
	};

	if (InParams.bCanStream)
	{
		if (!InParams.bIsServer || InParams.bIsServerStreamingEnabled || InParams.bIsPlaybackEnabled)
		{
			// When world partition can't stream, all cells must be unloaded
			if (InParams.RuntimeHash)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_ForEachStreamingCellsSources);

				InParams.RuntimeHash->ForEachStreamingCellsSources(InParams.CurrentState.StreamingSources, [&InParams](const UWorldPartitionRuntimeCell* Cell, EStreamingSourceTargetState SourceTargetState)
				{
					switch (SourceTargetState)
					{
					case EStreamingSourceTargetState::Loaded:
						InParams.FrameLoadCells.Add(Cell);
						break;
					case EStreamingSourceTargetState::Activated:
						InParams.FrameActivateCells.Add(Cell);
						break;
					default:
						check(0);
					}
					return true;
				}, Context);
			}
		}

		if (InParams.bIsServer)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_ServerUpdate);

			// Server will activate all non data layer cells at first and then load/activate/unload data layer cells only when the data layer states change
			const bool bCanServerDeactivateOrUnloadCells = InParams.bIsServerStreamingOutEnabled;

			OutTargetState.bUpdateServerEpoch = true;

			auto CanServerDeactivateOrUnloadDataLayerCell = [&InParams](const UWorldPartitionRuntimeCell* Cell)
			{
				return !Cell->HasDataLayers() || !Cell->HasAnyDataLayer(InParams.ServerDisallowedStreamingOutDataLayers);
			};

			auto AddServerFrameCell = [&InParams, &Context, CanServerDeactivateOrUnloadDataLayerCell](const UWorldPartitionRuntimeCell* Cell)
			{
				// Keep Data Layer cells in their current state if server cannot deactivate/unload data layer cells
				if (!CanServerDeactivateOrUnloadDataLayerCell(Cell))
				{
					// If cell was activated, keep it activated
					if (InParams.CurrentState.ActivatedCells.Contains(Cell))
					{
						InParams.FrameActivateCells.Add(Cell);
						return;
					}
					else
					{
						// If cell was loaded, keep it loaded except if it should become activated.
						// In the second case, let the standard code path process it and add it to FrameActivateCells.
						const bool bIsAnActivatedDataLayerCell = Cell->HasDataLayers() && (Cell->GetCellEffectiveWantedState(Context) == EDataLayerRuntimeState::Activated); 
						if (InParams.CurrentState.LoadedCells.Contains(Cell) && !bIsAnActivatedDataLayerCell)
						{
							InParams.FrameLoadCells.Add(Cell);
							return;
						}
					}
				}
				
				switch (Cell->GetCellEffectiveWantedState(Context))
				{
				case EDataLayerRuntimeState::Loaded:
					InParams.FrameLoadCells.Add(Cell);
					break;
				case EDataLayerRuntimeState::Activated:
					InParams.FrameActivateCells.Add(Cell);
					break;
				case EDataLayerRuntimeState::Unloaded:
					break;
				default:
					checkNoEntry();
				}
			};

			if (!InParams.bIsServerStreamingEnabled)
			{
				if (InParams.RuntimeHash)
				{
					InParams.RuntimeHash->ForEachStreamingCells([&AddServerFrameCell](const UWorldPartitionRuntimeCell* Cell)
					{
						AddServerFrameCell(Cell);
						return true;
					});
				}
			}
			else if (!bCanServerDeactivateOrUnloadCells)
			{
				// When server streaming-out is disabled, revisit existing loaded/activated cells and add them in the proper FrameLoadCells/FrameActivateCells
				for (const UWorldPartitionRuntimeCell* Cell : InParams.CurrentState.ActivatedCells.GetCells())
				{
					AddServerFrameCell(Cell);
				}
				for (const UWorldPartitionRuntimeCell* Cell : InParams.CurrentState.LoadedCells)
				{
					AddServerFrameCell(Cell);
				}
			}
		}
	}

	const TSet<FName>& ServerClientsVisibleLevelNames = InParams.World->GetSubsystem<UWorldPartitionSubsystem>()->ServerClientsVisibleLevelNames;
	auto ShouldWaitForClientVisibility = [&InParams, &OutTargetState, &ServerClientsVisibleLevelNames](const UWorldPartitionRuntimeCell* Cell)
	{
		check(InParams.bIsServer);
		if (Cell->ShouldServerWaitForClientLevelVisibility())
		{
			if (ULevel* Level = Cell->GetLevel())
			{
				if (ServerClientsVisibleLevelNames.Contains(Level->GetPackage()->GetFName()))
				{
					UE_CLOGF(OutTargetState.bUpdateServerEpoch, LogWorldPartition, Verbose, "Server epoch update delayed by client visibility");
					OutTargetState.bUpdateServerEpoch = false;
					return true;
				}
			}
		}
		return false;
	};

	auto ShouldSkipDisabledHLODCell = [](const UWorldPartitionRuntimeCell* Cell)
	{
		return Cell->GetIsHLOD() && !UWorldPartitionHLODRuntimeSubsystem::IsHLODEnabled();
	};

	auto ShouldSkipCellForPerformance = [&InParams](const UWorldPartitionRuntimeCell* Cell)
	{
		// When performance is degrading start skipping non blocking cells
		return !InParams.bIsServer && InParams.bIsBlockingCausedByBadStreamingPerformance && !Cell->GetBlockOnSlowLoading();
	};

	// Activation supersedes Loading
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_FrameLoadCells);
		if (InParams.FrameLoadCells.Num() && InParams.FrameActivateCells.Num())
		{
			InParams.FrameLoadCells = InParams.FrameLoadCells.Difference(InParams.FrameActivateCells);
		}
	}

	// Determine cells to activate
	if (InParams.bCanStream)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_ToActivateCells);
		for (const UWorldPartitionRuntimeCell* Cell : InParams.FrameActivateCells)
		{
			if (InParams.CurrentState.ActivatedCells.Contains(Cell))
			{
				// Update streaming source info for pending add to world cells
				if (InParams.bShouldMergeStreamingSourceInfo && InParams.CurrentState.ActivatedCells.GetPendingAddToWorldCells().Contains(Cell))
				{
					Cell->MergeStreamingSourceInfo();
				}
			}
			else if (!ShouldSkipCellForPerformance(Cell) && !ShouldSkipDisabledHLODCell(Cell))
			{
				if (InParams.bShouldMergeStreamingSourceInfo)
				{
					Cell->MergeStreamingSourceInfo();
				}
				OutTargetState.ToActivateCells.Add(Cell);
			}
		}
	}

	// Determine cells to load and server cells to deactivate
	if (InParams.bCanStream)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_ToLoadCells);
		for (const UWorldPartitionRuntimeCell* Cell : InParams.FrameLoadCells)
		{
			if (InParams.CurrentState.LoadedCells.Contains(Cell))
			{
				// Update streaming source info for pending load cells
				if (InParams.bShouldMergeStreamingSourceInfo && !Cell->GetLevel())
				{
					Cell->MergeStreamingSourceInfo();
				}
			}
			else
			{
				if (!ShouldSkipCellForPerformance(Cell) && !ShouldSkipDisabledHLODCell(Cell))
				{
					// Server deactivated cells are processed right away (see below for details)
					if (const bool bIsServerCellToDeactivate = InParams.bIsServer && InParams.CurrentState.ActivatedCells.Contains(Cell))
					{
						// Only deactivated server cells need to call ShouldWaitForClientVisibility (those part of ActivatedCells)
						if (!ShouldWaitForClientVisibility(Cell))
						{
							OutTargetState.ToDeactivateCells.Add(Cell);
						}
					}
					else
					{
						if (InParams.bShouldMergeStreamingSourceInfo)
						{
							Cell->MergeStreamingSourceInfo();
						}
						OutTargetState.ToLoadCells.Add(Cell);
					}
				}
			}
		}
	}

	// Determine cells to unload
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingState_ToUnloadCells);
		auto BuildCellsToUnload = [&InParams, &OutTargetState, ShouldWaitForClientVisibility](const TSet<TObjectPtr<const UWorldPartitionRuntimeCell>>& InCells)
		{
			for (const UWorldPartitionRuntimeCell* Cell : InCells)
			{
				if (!InParams.FrameActivateCells.Contains(Cell) && !InParams.FrameLoadCells.Contains(Cell))
				{
					if (!InParams.bCanStream || !InParams.bIsServer || !ShouldWaitForClientVisibility(Cell))
					{
						OutTargetState.ToUnloadCells.Add(Cell);
					}
				}
			}
		};

		BuildCellsToUnload(InParams.CurrentState.ActivatedCells.GetCells());
		BuildCellsToUnload(InParams.CurrentState.LoadedCells);
	}

	UE_SUPPRESS(LogWorldPartition, Verbose,
	if ((InParams.bIsStreamingInEnabled && (OutTargetState.ToActivateCells.Num() > 0 || OutTargetState.ToLoadCells.Num() > 0)) || OutTargetState.ToUnloadCells.Num() > 0)
	{
		UE_LOGF(LogWorldPartition, Verbose, "UWorldPartitionStreamingPolicy: CellsToActivate(%d), CellsToLoad(%d), CellsToUnload(%d)", OutTargetState.ToActivateCells.Num(), OutTargetState.ToLoadCells.Num(), OutTargetState.ToUnloadCells.Num());
		for (int i = 0; i < InParams.CurrentState.StreamingSources.Num(); ++i)
		{
			FVector ViewLocation = InParams.WorldPartitionInstanceTransform.TransformPosition(InParams.CurrentState.StreamingSources[i].Location);
			FRotator ViewRotation = InParams.WorldPartitionInstanceTransform.TransformRotation(InParams.CurrentState.StreamingSources[i].Rotation.Quaternion()).Rotator();
			UE_LOGF(LogWorldPartition, Verbose, "UWorldPartitionStreamingPolicy: Sources[%d] = %ls,%ls", i, *ViewLocation.ToString(), *ViewRotation.ToString());
		}
	});

#if !UE_BUILD_SHIPPING || UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	{
		TArray<TPair<FGuid, float>>* PriorityTraceData = nullptr;
	#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
		if (InParams.bShouldTraceContainerPriorities)
		{
			PriorityTraceData = &OutTargetState.DeferredPriorityTraceData;
		}
	#endif
		UWorldPartitionStreamingPolicy::UpdateDebugCellsStreamingPriority(InParams.FrameActivateCells, InParams.FrameLoadCells, InParams.bShouldMergeStreamingSourceInfo, PriorityTraceData);
	}
#endif

	if (InParams.RuntimeHash)
	{
		if (!InParams.bMatchStarted)
		{
			OutTargetState.StreamingPerformance = EWorldPartitionStreamingPerformance::Good;
			OutTargetState.bBlockOnSlowStreaming = false;
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::GetStreamingPerformance);
			const TSet<const UWorldPartitionRuntimeCell*>& PendingAddToWorldCells = InParams.CurrentState.ActivatedCells.GetPendingAddToWorldCells();
			for (const UWorldPartitionRuntimeCell* Cell : InParams.FrameActivateCells)
			{
				if (PendingAddToWorldCells.Contains(Cell) || !InParams.CurrentState.ActivatedCells.Contains(Cell))
				{
					InParams.FrameAnalysePerformanceCells.Add(Cell);
				}
			}
			OutTargetState.StreamingPerformance = InParams.RuntimeHash->GetStreamingPerformance(InParams.FrameAnalysePerformanceCells, OutTargetState.bBlockOnSlowStreaming);
		}
	}
}

void UWorldPartitionStreamingPolicy::PostUpdateStreamingStateInternal_GameThread(FWorldPartitionUpdateStreamingTargetState& InOutTargetState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::PostUpdateStreamingStateInternal_GameThread);

	check(IsInGameThread());

	// Unloaded cells
	if (!InOutTargetState.ToUnloadCells.IsEmpty())
	{
		SetCellsStateToUnloaded(InOutTargetState.ToUnloadCells);
		InOutTargetState.ToUnloadCells.Reset();
	}

	const bool bIsServer = WorldPartition->IsServer();
	check(bIsServer || InOutTargetState.ToDeactivateCells.IsEmpty());
	if (bIsServer)
	{
		if (!InOutTargetState.ToDeactivateCells.IsEmpty())
		{
			// Server deactivated cells (activated -> loaded)
			// 
			// Server deactivation is handle right away to ensure that even if WorldPartitionSubsystem::UpdateStreamingState
			// is running in incremental mode, server deactivated cells will make their streaming level ShouldBeVisible() 
			// return false. This way, UNetConnection::UpdateLevelVisibilityInternal will not allow clients to make their 
			// streaming level visible (see LevelVisibility.bTryMakeVisible).
			for (const UWorldPartitionRuntimeCell* ServerCellToDeactivate : InOutTargetState.ToDeactivateCells)
			{
				int32 DummyMaxCellToLoad = 0; // Deactivating is not concerned by MaxCellsToLoad
				SetCellStateToLoaded(ServerCellToDeactivate, DummyMaxCellToLoad);
			}
			InOutTargetState.ToDeactivateCells.Reset();
		}

		// Update Epoch if we aren't waiting for clients anymore
		if (InOutTargetState.bUpdateServerEpoch)
		{
			ServerStreamingStateEpoch = WorldPartition->GetStreamingStateEpoch();
			ServerStreamingEnabledEpoch = ComputeServerStreamingEnabledEpoch();
			UE_LOGF(LogWorldPartition, Verbose, "Server epoch updated");
		}
		else
		{
			// Invalidate UpdateStreamingHash as it was built with latest epochs 
			// and is now also used to optimize server's update streaming.
			UpdateStreamingHash = 0;
		}
	}

	// Evaluate streaming performance based on cells that should be activated
	UpdateStreamingPerformance(InOutTargetState.StreamingPerformance, InOutTargetState.bBlockOnSlowStreaming);

	// Emit deferred priority trace data (captured during UpdateStreamingStateInternal)
	UE_TRACE_WP_STREAMING_CONTAINER_PRIORITIES(GetWorld(), InOutTargetState.DeferredPriorityTraceData);
}

void UWorldPartitionStreamingPolicy::OnPreChangeStreamingContent()
{
	WaitForAsyncUpdateStreamingState();
}

bool UWorldPartitionStreamingPolicy::WaitForAsyncUpdateStreamingState()
{
	if (AsyncUpdateStreamingStateTask.IsValid())
	{
		check(AsyncUpdateTaskState == EAsyncUpdateTaskState::Started);
		// Wait for completion
		if (!AsyncUpdateStreamingStateTask.IsCompleted())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_WorldPartitionStreamingPolicy_WaitForAsyncUpdateStreamingState);
			AsyncUpdateStreamingStateTask.Wait();
			check(AsyncUpdateStreamingStateTask.IsCompleted());
		}
		return true;
	}
	return false;
}

void UWorldPartitionStreamingPolicy::OnStreamingStateUpdated()
{
	if (AsyncUpdateTaskState == EAsyncUpdateTaskState::Pending)
	{
		// Here, it's considered safe to start the asynchronous call to UWorldPartitionStreamingPolicy::UpdateStreamingStateInternal since the
		// WorldPartitionSubsystem is done working on the world partition streaming cells returned by UWorldPartitionStreamingPolicy::GetCellsToUpdate.
		// 
		// Any call that modifies the streaming content should first call UWorldPartitionStreamingPolicy::OnPreChangeStreamingContent to make sure
		// that any asynchronous update task completes before modifying the streaming content.
		// 
		// All the required input is prepared and copied in the FUpdateStreamingStateParams structure. 
		// Some members (like FrameActivateCells/FrameLoadCells) are direct references to UWorldPartitionStreamingPolicy members, these are considered 
		// safe to access read/write from the asynchronous task.
		// 
		// Note that some calls to world partition cells will cache information into the cell (thus modify it).
		// Some of this information is either cached for performance reasons, some is used to prioritize cells (see UWorldPartitionRuntimeCell::SortCompare)
		// Here's the list of calls that modify the cell:
		// - UWorldPartitionRuntimeCell::GetCellEffectiveWantedState
		// - UWorldPartitionRuntimeCellData::ResetStreamingSourceInfo
		// - UWorldPartitionRuntimeCellData::AppendStreamingSourceInfo
		// - UWorldPartitionRuntimeCellData::MergeStreamingSourceInfo
		// 
		// Note: This could be revisited at some point (FWorldPartitionStreamingContext could store this information).

		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::OnStreamingStateUpdated);
		{
			// Prepare async task input payload
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::PrepareAsyncTaskPayloads);
			AsyncTaskCurrentState.CopyFrom(CurrentState);
			check(AsyncTaskTargetState.IsEmpty());
		}
		{
			// Create async tasks
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::CreateAndDispatchAsyncTasks);
			FUpdateStreamingStateParams InputParams = FUpdateStreamingStateParams(this, AsyncTaskCurrentState).SetRequiredWorldDataLayersEffectiveStatesCopy(true);
			AsyncUpdateStreamingStateTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [InputParams = MoveTemp(InputParams), this]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_WorldPartitionStreamingPolicy_AsyncUpdateStreamingState);
				
				FGCScopeGuard GCGuard;
				UWorldPartitionStreamingPolicy::UpdateStreamingStateInternal(InputParams, AsyncTaskTargetState);

			}, UE::Tasks::ETaskPriority::Normal);
			AsyncUpdateTaskState = EAsyncUpdateTaskState::Started;
		}
	}
}

#if !UE_BUILD_SHIPPING || UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
void UWorldPartitionStreamingPolicy::UpdateDebugCellsStreamingPriority(const TSet<const UWorldPartitionRuntimeCell*>& InActivateStreamingCells, const TSet<const UWorldPartitionRuntimeCell*>& InLoadStreamingCells, bool bInShouldMergeStreamingSourceInfo, TArray<TPair<FGuid, float>>* OutTraceData)
{
	// @todo_ow: This code generates debug priority values local to this partitioned world.
	// To properly support multiple partitioned worlds, move the sorting pass and the priority update in the WorldPartitionSubsystem.
	const bool bIsDebugStreamingPriorityShown = FWorldPartitionDebugHelper::IsRuntimeSpatialHashCellStreamingPriorityShown();
	if (bIsDebugStreamingPriorityShown || OutTraceData)
	{
		TArray<const UWorldPartitionRuntimeCell*> Cells = InActivateStreamingCells.Array();
		Cells.Append(InLoadStreamingCells.Array());

		if (bInShouldMergeStreamingSourceInfo)
		{
			for (const UWorldPartitionRuntimeCell* Cell : Cells)
			{
				Cell->MergeStreamingSourceInfo();
			}
		}

		if (Cells.Num() > 1)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SortStreamingCellsByImportance);
			Algo::Sort(Cells, [](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB) { return CellA->SortCompare(CellB) < 0; });
		}

		const int32 CellCount = Cells.Num();
		if (OutTraceData)
		{
			OutTraceData->Reset();
			OutTraceData->Reserve(CellCount);
		}

		int32 CellPrio = 0;
		for (const UWorldPartitionRuntimeCell* SortedCell : Cells)
		{
			const float NormalizedPriority = float(CellPrio++) / CellCount;
			if (bIsDebugStreamingPriorityShown)
			{
				const_cast<UWorldPartitionRuntimeCell*>(SortedCell)->SetDebugStreamingPriority(NormalizedPriority);
			}
			if (OutTraceData)
			{
				OutTraceData->Emplace(SortedCell->GetGuid(), NormalizedPriority);
			}
		}
	}
}
#endif

void UWorldPartitionStreamingPolicy::UpdateStreamingPerformance(EWorldPartitionStreamingPerformance NewStreamingPerformance, bool bBlockOnSlowStreaming)
{
	check(IsInGameThread());

	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::UpdateStreamingPerformance);
	UWorld* World = GetWorld();
	// If we are currently in a blocked loading just reset the on screen message time and return
	if (CurrentStreamingPerformance >= EWorldPartitionStreamingPerformance::Critical && IsInBlockTillLevelStreamingCompleted())
	{
#if !UE_BUILD_SHIPPING
		OnScreenMessageStartTime = FPlatformTime::Seconds();
#endif
	}

	if (WorldPartition->RuntimeHash)
	{
		if (CurrentStreamingPerformance != NewStreamingPerformance)
		{
			UE_LOGF(LogWorldPartition, Log, "Streaming performance changed: %ls -> %ls",
				*StaticEnum<EWorldPartitionStreamingPerformance>()->GetDisplayNameTextByValue((int64)CurrentStreamingPerformance).ToString(),
				*StaticEnum<EWorldPartitionStreamingPerformance>()->GetDisplayNameTextByValue((int64)NewStreamingPerformance).ToString());

			CurrentStreamingPerformance = NewStreamingPerformance;
		}
		if (bCurrentBlockOnSlowStreaming != bBlockOnSlowStreaming)
		{
			bCurrentBlockOnSlowStreaming = bBlockOnSlowStreaming;
		}
	}

#if !UE_BUILD_SHIPPING
	if (CurrentStreamingPerformance != EWorldPartitionStreamingPerformance::Good)
	{
		if (bOnScreenMessageShouldBlock || GDisplayStreamingPerformanceForNonBlockingStreaming)
		{
			// performance still bad keep message alive
			OnScreenMessageStartTime = FPlatformTime::Seconds();
			OnScreenMessageStreamingPerformance = CurrentStreamingPerformance;
			bOnScreenMessageShouldBlock = bBlockOnSlowStreaming;
		}
	}
#endif
	
	if (CurrentStreamingPerformance >= EWorldPartitionStreamingPerformance::Critical && bCurrentBlockOnSlowStreaming)
	{
		const bool bIsServer = WorldPartition->IsServer();
		const bool bIsServerStreamingEnabled = WorldPartition->IsServerStreamingEnabled();
		const bool bCanBlockOnSlowStreaming = GBlockOnSlowStreaming && (!bIsServer || bIsServerStreamingEnabled);

		// This is a very simple implementation of handling of critical streaming conditions.
		if (bCanBlockOnSlowStreaming && !IsInBlockTillLevelStreamingCompleted())
		{
			World->bRequestedBlockOnAsyncLoading = true;
			bCriticalPerformanceRequestedBlockTillOnWorld = true;
		}
	}
}

#if !UE_BUILD_SHIPPING
void UWorldPartitionStreamingPolicy::GetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages)
{
	// Keep displaying for 2 seconds (or more if health stays bad)
	double DisplayTime = FPlatformTime::Seconds() - OnScreenMessageStartTime;
	if (DisplayTime < 2.0)
	{
		switch (OnScreenMessageStreamingPerformance)
		{
		case EWorldPartitionStreamingPerformance::Immediate:
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::Format(LOCTEXT("WPStreamingImmediate", "[Immediate] WorldPartition Streaming Performance [Blocking:{0}]"), bOnScreenMessageShouldBlock ? LOCTEXT("True", "True") : LOCTEXT("False", "False")));
			break;
		case EWorldPartitionStreamingPerformance::Critical:
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::Format(LOCTEXT("WPStreamingCritical", "[Critical] WorldPartition Streaming Performance [Blocking:{0}]"), bOnScreenMessageShouldBlock ? LOCTEXT("True", "True") : LOCTEXT("False", "False")));
			break;
		case EWorldPartitionStreamingPerformance::Slow:
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::Format(LOCTEXT("WPStreamingWarning", "[Slow] WorldPartition Streaming Performance [Blocking:{0}]"), bOnScreenMessageShouldBlock ? LOCTEXT("True", "True") : LOCTEXT("False", "False")));
			break;
		default:
			break;
		}
	}
	else
	{
		OnScreenMessageStreamingPerformance = EWorldPartitionStreamingPerformance::Good;
	}
}
#endif

void UWorldPartitionStreamingPolicy::GetCellsToUpdate(TArray<const UWorldPartitionRuntimeCell*>& OutToLoadCells, TArray<const UWorldPartitionRuntimeCell*>& OutToActivateCells)
{
	check(IsInGameThread());

	OutToLoadCells.Append(TargetState.ToLoadCells);
	OutToActivateCells.Append(TargetState.ToActivateCells);
}

void UWorldPartitionStreamingPolicy::GetCellsToReprioritize(TArray<const UWorldPartitionRuntimeCell*>& OutToReprioritizeLoadCells, TArray<const UWorldPartitionRuntimeCell*>& OutToReprioritizeActivateCells)
{
	check(IsInGameThread());

	for (const UWorldPartitionRuntimeCell* Cell : CurrentState.LoadedCells)
	{
		if (!Cell->GetLevel())
		{
			OutToReprioritizeLoadCells.Add(Cell);
		}
	}

	for (const UWorldPartitionRuntimeCell* Cell : CurrentState.ActivatedCells.GetPendingAddToWorldCells())
	{
		OutToReprioritizeActivateCells.Add(Cell);
	}
}

void UWorldPartitionStreamingPolicy::SetCellStateToLoaded(const UWorldPartitionRuntimeCell* InCell, int32& InOutMaxCellsToLoad)
{
	check(IsInGameThread());

	bool bLoadCell = false;
	if (CurrentState.ActivatedCells.Contains(InCell))
	{
		InCell->Deactivate();
		CurrentState.ActivatedCells.Remove(InCell);
		bLoadCell = true;
	}
	else if (WorldPartition->IsStreamingInEnabled())
	{
		if (InOutMaxCellsToLoad > 0)
		{
			InCell->Load();
			bLoadCell = true;
			if (!InCell->IsAlwaysLoaded())
			{
				--InOutMaxCellsToLoad;
			}
		}
	}

	if (bLoadCell)
	{
		UE_LOGF(LogWorldPartition, Verbose, "UWorldPartitionStreamingPolicy::SetCellStateToLoaded %ls", *InCell->GetName());
		CurrentState.LoadedCells.Add(InCell);
		++ProcessedToLoadCells;
	}
}

void UWorldPartitionStreamingPolicy::SetCellStateToActivated(const UWorldPartitionRuntimeCell* InCell, int32& InOutMaxCellsToLoad)
{
	check(IsInGameThread());

	if (!WorldPartition->IsStreamingInEnabled())
	{
		return;
	}

	bool bActivateCell = false;
	if (CurrentState.LoadedCells.Contains(InCell))
	{
		CurrentState.LoadedCells.Remove(InCell);
		bActivateCell = true;
	}
	else if (InOutMaxCellsToLoad > 0)
	{
		if (!InCell->IsAlwaysLoaded())
		{
			--InOutMaxCellsToLoad;
		}
		bActivateCell = true;
	}

	if (bActivateCell)
	{
		UE_LOGF(LogWorldPartition, Verbose, "UWorldPartitionStreamingPolicy::SetCellStateToActivated %ls", *InCell->GetName());
		CurrentState.ActivatedCells.Add(InCell);
		InCell->Activate();
		++ProcessedToActivateCells;
	}
}

void UWorldPartitionStreamingPolicy::SetCellsStateToUnloaded(const TArray<TObjectPtr<const UWorldPartitionRuntimeCell>>& InToUnloadCells)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::SetCellsStateToUnloaded);

	for (const UWorldPartitionRuntimeCell* Cell : InToUnloadCells)
	{
		if (Cell->CanUnload())
		{
			UE_LOGF(LogWorldPartition, Verbose, "UWorldPartitionStreamingPolicy::UnloadCells %ls", *Cell->GetName());
			Cell->Unload();
			CurrentState.ActivatedCells.Remove(Cell);
			CurrentState.LoadedCells.Remove(Cell);
		}
	}
}

bool UWorldPartitionStreamingPolicy::CanAddCellToWorld(const UWorldPartitionRuntimeCell* InCell) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionStreamingPolicy::CanAddCellToWorld);

	check(InCell);
	check(WorldPartition->IsInitialized());

	// Always allow AddToWorld in Dedicated server and Listen server
	if (WorldPartition->IsServer())
	{
		return true;
	}

	// Always allow AddToWorld when not inside UWorld::BlockTillLevelStreamingCompleted that was not triggered by bad streaming performance
	if (!IsInBlockTillLevelStreamingCompleted(/*bIsCausedByBadStreamingPerformance*/true))
	{
		return true;
	}

	// When performance is degrading, start skipping non blocking cells
	return InCell->GetBlockOnSlowLoading();
}

bool UWorldPartitionStreamingPolicy::IsStreamingCompleted(const TArray<FWorldPartitionStreamingSource>* InStreamingSources) const
{
	const UWorld* World = GetWorld();
	check(World);
	check(World->IsGameWorld());
	const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
	const bool bTestProvidedStreamingSource = !!InStreamingSources;
	
	// IsStreamingCompleted using streaming sources will be considered as completed
	// if the content is activated and the target state is loaded.
	const bool bExactState = false;

	// Always test non-spatial cells
	{
		// Test non-data layer and activated data layers
		TArray<FWorldPartitionStreamingQuerySource> QuerySources;
		FWorldPartitionStreamingQuerySource& QuerySource = QuerySources.Emplace_GetRef();
		QuerySource.bSpatialQuery = false;
		QuerySource.bDataLayersOnly = false;
		QuerySource.DataLayers = DataLayerManager->GetEffectiveActiveDataLayerNames().Array();
		if (!IsStreamingCompleted(EWorldPartitionRuntimeCellState::Activated, QuerySources, bExactState))
		{
			return false;
		}

		// Test only loaded data layers
		if (!DataLayerManager->GetEffectiveLoadedDataLayerNames().IsEmpty())
		{
			QuerySource.bDataLayersOnly = true;
			QuerySource.DataLayers = DataLayerManager->GetEffectiveLoadedDataLayerNames().Array();
			if (!IsStreamingCompleted(EWorldPartitionRuntimeCellState::Loaded, QuerySources, bExactState))
			{
				return false;
			}
		}
	}

	// Test spatially loaded cells using streaming sources (or provided streaming source)
	// Non-spatial cells were already queried above; skip them for each per-source spatial query.
	check(WorldPartition->RuntimeHash->bQueryNonSpatialCells); // Flag future re-entrancy issues
	TGuardValue<bool> QueryNonSpatialCells(WorldPartition->RuntimeHash->bQueryNonSpatialCells, false);
	TArrayView<const FWorldPartitionStreamingSource> QueriedStreamingSources = bTestProvidedStreamingSource ? *InStreamingSources : CurrentState.StreamingSources;
	for (const FWorldPartitionStreamingSource& StreamingSource : QueriedStreamingSources)
	{
		// Build a query source from a Streaming Source
		TArray<FWorldPartitionStreamingQuerySource> QuerySources;
		FWorldPartitionStreamingQuerySource& QuerySource = QuerySources.Emplace_GetRef();
		QuerySource.bSpatialQuery = true;
		QuerySource.Location = StreamingSource.Location;
		QuerySource.Rotation = StreamingSource.Rotation;
		QuerySource.TargetBehavior = StreamingSource.TargetBehavior;
		QuerySource.TargetGrids = StreamingSource.TargetGrids;
		QuerySource.Shapes = StreamingSource.Shapes;
		QuerySource.bUseGridLoadingRange = true;
		QuerySource.Radius = 0.f;
		QuerySource.bDataLayersOnly = false;
		// Always include active Data Layers even when testing loaded because we are querrying in non-exact state
		QuerySource.DataLayers = DataLayerManager->GetEffectiveActiveDataLayerNames().Array();
		if (StreamingSource.TargetState == EStreamingSourceTargetState::Loaded)
		{
			QuerySource.DataLayers.Append(DataLayerManager->GetEffectiveLoadedDataLayerNames().Array());
		}

		// Execute query
		const EWorldPartitionRuntimeCellState QueryState = (StreamingSource.TargetState == EStreamingSourceTargetState::Loaded) ? EWorldPartitionRuntimeCellState::Loaded : EWorldPartitionRuntimeCellState::Activated;
		if (!IsStreamingCompleted(QueryState, QuerySources, bExactState))
		{
			return false;
		}
	}

	return true;
}

bool UWorldPartitionStreamingPolicy::IsStreamingCompleted(EWorldPartitionRuntimeCellState QueryState, const TArray<FWorldPartitionStreamingQuerySource>& QuerySources, bool bExactState) const
{
	const FWorldPartitionStreamingContext StreamingContext = FWorldPartitionStreamingContext::Create(GetTypedOuter<UWorld>());
	const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();
	const bool bIsHLODEnabled = UWorldPartitionHLODRuntimeSubsystem::IsHLODEnabled();

	bool bResult = true;
	for (const FWorldPartitionStreamingQuerySource& QuerySource : QuerySources)
	{
		// Verify that the state of the query data layers are in the query target state (or greater if not querying for exact state)
		bool bAreQueryDLsInTargetState = true;
		for (const FName& QueryDataLayer : QuerySource.DataLayers)
		{
			EDataLayerRuntimeState QueryDataLayerState = StreamingContext.GetDataLayerEffectiveRuntimeState(QueryDataLayer);
			if ((QueryDataLayerState != QueryState) && (bExactState || QueryDataLayerState < QueryState))
			{
				bAreQueryDLsInTargetState = false;
				break;
			}
		}

		WorldPartition->RuntimeHash->ForEachStreamingCellsQuery(QuerySource, [QuerySource, QueryState, bExactState, bIsHLODEnabled, DataLayerManager, bAreQueryDLsInTargetState, &StreamingContext, &bResult](const UWorldPartitionRuntimeCell* Cell)
		{
			const EWorldPartitionRuntimeCellState CellState = Cell->GetCurrentState();

			if (CellState != QueryState)
			{
				bool bSkipCell = false;

				// Don't consider HLOD cells if HLODs are disabled.
				if (!bIsHLODEnabled)
				{
					bSkipCell = Cell->GetIsHLOD();
				}

				// Don't consider Custom HLOD Placeholder cells as they have no level streaming object
				if (!bSkipCell && Cell->GetIsCustomHLODPlaceholderCell())
				{
					bSkipCell = true;
				}

				if (!bSkipCell && Cell->HasDataLayers())
				{
					if (!bAreQueryDLsInTargetState)
					{
						bResult = false;
						return false;
					}

					// Because we already validated that the queried data layers are in the proper effective state
					// if the cell wanted state is not in the proper queried state, it's because of unquerried cell data layer states.
					// In this case, we must ignore the cell.
					const EDataLayerRuntimeState CellWantedState = Cell->GetCellEffectiveWantedState(StreamingContext);
					bSkipCell = ((CellWantedState != QueryState) && (bExactState || CellWantedState < QueryState));
				}

				if (!bSkipCell && (bExactState || CellState < QueryState))
				{
					bResult = false;
					return false;
				}
			}

			return true;
		});
	}
	return bResult;
}

bool UWorldPartitionStreamingPolicy::DrawRuntimeHash2D(FWorldPartitionDraw2DContext& DrawContext)
{
	if (CurrentState.StreamingSources.Num() > 0 && WorldPartition->RuntimeHash)
	{
		return WorldPartition->RuntimeHash->Draw2D(DrawContext);
	}
	return false;
}

void UWorldPartitionStreamingPolicy::DrawRuntimeHash3D()
{
	if (WorldPartition->IsInitialized() && WorldPartition->RuntimeHash)
	{
		WorldPartition->RuntimeHash->Draw3D(CurrentState.StreamingSources);
	}
}

void UWorldPartitionStreamingPolicy::OnCellShown(const UWorldPartitionRuntimeCell* InCell)
{
	CurrentState.ActivatedCells.OnAddedToWorld(InCell);
}

void UWorldPartitionStreamingPolicy::OnCellHidden(const UWorldPartitionRuntimeCell* InCell)
{
	CurrentState.ActivatedCells.OnRemovedFromWorld(InCell);
}

void FActivatedCells::Add(const UWorldPartitionRuntimeCell* InCell)
{
	Cells.Add(InCell);
	if (!InCell->IsAlwaysLoaded())
	{
		PendingAddToWorldCells.Add(InCell);
	}
}

void FActivatedCells::Remove(const UWorldPartitionRuntimeCell* InCell)
{
	Cells.Remove(InCell);
	PendingAddToWorldCells.Remove(InCell);
}

void FActivatedCells::Reset()
{
	Cells.Reset();
	PendingAddToWorldCells.Reset();
}

void FActivatedCells::OnAddedToWorld(const UWorldPartitionRuntimeCell* InCell)
{
	PendingAddToWorldCells.Remove(InCell);
}

void FActivatedCells::OnRemovedFromWorld(const UWorldPartitionRuntimeCell* InCell)
{
	if (Cells.Contains(InCell))
	{
		if (!InCell->IsAlwaysLoaded())
		{
			PendingAddToWorldCells.Add(InCell);
		}
	}
}

void FWorldPartitionUpdateStreamingCurrentState::Reset()
{
	StreamingSources.Reset();
	LoadedCells.Reset();
	ActivatedCells.Reset();
}

void FWorldPartitionUpdateStreamingCurrentState::CopyFrom(const FWorldPartitionUpdateStreamingCurrentState& InCurrentState)
{
	StreamingSources = InCurrentState.StreamingSources;
	LoadedCells = InCurrentState.LoadedCells;
	ActivatedCells = InCurrentState.ActivatedCells;
}

bool FWorldPartitionUpdateStreamingTargetState::IsEmpty() const
{
	return ToLoadCells.IsEmpty() && ToActivateCells.IsEmpty() && ToUnloadCells.IsEmpty() && ToDeactivateCells.IsEmpty();
}

void FWorldPartitionUpdateStreamingTargetState::Reset()
{
	ToLoadCells.Reset();
	ToActivateCells.Reset();
	ToDeactivateCells.Reset();
	ToUnloadCells.Reset();
	StreamingPerformance = EWorldPartitionStreamingPerformance::Good;
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	DeferredPriorityTraceData.Reset();
#endif
}

UWorldPartitionStreamingPolicy::FUpdateStreamingStateParams::FUpdateStreamingStateParams(UWorldPartitionStreamingPolicy* InPolicy, const FWorldPartitionUpdateStreamingCurrentState& InCurrentState)
	: World(InPolicy->GetWorld())
	, RuntimeHash(InPolicy->GetOuterUWorldPartition()->RuntimeHash)
	, bCanStream(InPolicy->GetOuterUWorldPartition()->CanStream())
	, bIsServer(InPolicy->GetOuterUWorldPartition()->IsServer())
	, bIsStreamingInEnabled(InPolicy->GetOuterUWorldPartition()->IsStreamingInEnabled())
	, bIsServerStreamingEnabled(InPolicy->GetOuterUWorldPartition()->IsServerStreamingEnabled())
	, bIsServerStreamingOutEnabled(InPolicy->GetOuterUWorldPartition()->IsServerStreamingOutEnabled())
	, bIsBlockingCausedByBadStreamingPerformance(InPolicy->IsInBlockTillLevelStreamingCompleted(true))
	, bIsPlaybackEnabled(AWorldPartitionReplay::IsPlaybackEnabled(InPolicy->GetWorld()))
	, bMatchStarted(InPolicy->GetWorld()->bMatchStarted)
	, bShouldMergeStreamingSourceInfo(InPolicy->bShouldMergeStreamingSourceInfo)
#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
	, bShouldTraceContainerPriorities(UWorldStreamingTraceSubsystem::ShouldTraceContainerPriorityThisFrame(InPolicy->GetWorld()))
#endif
	, PolicyUpdateStreamingStateEpoch(InPolicy->UpdateStreamingStateCounter)
	, DataLayersLogicOperator(InPolicy->GetOuterUWorldPartition()->GetDataLayersLogicOperator())
	, WorldPartitionInstanceTransform(InPolicy->GetOuterUWorldPartition()->GetInstanceTransform())
	, CurrentState(InCurrentState)
	, ServerDisallowedStreamingOutDataLayers(InPolicy->GetServerDisallowedStreamingOutDataLayers())
	, FrameActivateCells(InPolicy->FrameActivateCells)
	, FrameLoadCells(InPolicy->FrameLoadCells)
	, FrameAnalysePerformanceCells(InPolicy->FrameAnalysePerformanceCells)
#if WITH_EDITOR
	, DynamicLoadingRangeScales(
		[InPolicy]()
		{
			const UWorldPartitionSubsystem* Subsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(InPolicy->GetWorld());
			return (Subsystem) ? Subsystem->GetDynamicLoadingRangeScales() : FDynamicLoadingRangeScales{};
		}())
#endif // WITH_EDITOR
	, WorldDataLayersEffectiveStatesRef(FWorldDataLayersEffectiveStatesAccessor::Get(InPolicy->GetOuterUWorldPartition()->GetTypedOuter<UWorld>()->GetWorldDataLayers()))
{
	check(IsInGameThread());
	check(World->IsGameWorld());
}

UWorldPartitionStreamingPolicy::FUpdateStreamingStateParams& UWorldPartitionStreamingPolicy::FUpdateStreamingStateParams::SetRequiredWorldDataLayersEffectiveStatesCopy(bool bInRequiredEffectiveStatesCopy)
{
	check(IsInGameThread());
	if (bInRequiredEffectiveStatesCopy)
	{
		WorldDataLayersEffectiveStatesCopy = WorldDataLayersEffectiveStatesRef;
	}
	else
	{
		WorldDataLayersEffectiveStatesCopy.Reset();
	}
	return *this;
}

#undef LOCTEXT_NAMESPACE
