// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "Math/UnitConversion.h"
#include "Misc/HashBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionStreamingSource)

int32 FWorldPartitionStreamingSource::LocationQuantization = 400;
FAutoConsoleVariableRef FWorldPartitionStreamingSource::CVarLocationQuantization(
	TEXT("wp.Runtime.UpdateStreaming.LocationQuantization"),
	FWorldPartitionStreamingSource::LocationQuantization,
	TEXT("Distance (in Unreal units) used to quantize the streaming sources location to determine if a world partition streaming update is necessary."),
	ECVF_Default);

int32 FWorldPartitionStreamingSource::RotationQuantization = 10;
FAutoConsoleVariableRef FWorldPartitionStreamingSource::CVarRotationQuantization(
	TEXT("wp.Runtime.UpdateStreaming.RotationQuantization"),
	FWorldPartitionStreamingSource::RotationQuantization,
	TEXT("Angle (in degrees) used to quantize the streaming sources rotation to determine if a world partition streaming update is necessary."),
	ECVF_Default);

int32 FWorldPartitionStreamingSource::DebugDisplaySpeedUnit = 3;
FAutoConsoleVariableRef FWorldPartitionStreamingSource::CVarDebugDisplaySpeedUnit(
	TEXT("wp.Runtime.DebugDisplaySpeedUnit"),
	FWorldPartitionStreamingSource::DebugDisplaySpeedUnit,
	TEXT("Unit used for debug display to show speeds (0=cm/s, 1=m/s, 2=km/h, 3=mi/h), defaults to mi/h."),
	ECVF_Default);

FString FStreamingSourceShape::ToString() const
{
	TStringBuilder<128> StringBuilder;
	if (bIsSector)
	{
		StringBuilder += TEXT("IsSector ");
	}
	if (bUseGridLoadingRange)
	{
		StringBuilder += TEXT("UsesGridLoadingRange ");
		if (!FMath::IsNearlyEqual(LoadingRangeScale, 1.f))
		{
			StringBuilder.Appendf(TEXT("Scale: %3.2f"), LoadingRangeScale);
		}
	}
	else
	{
		StringBuilder.Appendf(TEXT("Radius: %d"), (int32)Radius);
	}
	return FString(StringBuilder);
}

FString FWorldPartitionStreamingSource::ToString() const
{
	TStringBuilder<256> StringBuilder;

	static EUnit SupportedUnits[] = { EUnit::CentimetersPerSecond, EUnit::MetersPerSecond, EUnit::KilometersPerHour, EUnit::MilesPerHour };
	const EUnit VelocityUnitEnum = SupportedUnits[FMath::Min<int32>(DebugDisplaySpeedUnit, (sizeof(SupportedUnits) / sizeof((SupportedUnits)[0])) - 1)];
	const int32 VelocityUnitValue = (int32)FUnitConversion::Convert(Velocity.Size(), EUnit::CentimetersPerSecond, VelocityUnitEnum);
	const TCHAR* VelocityUnitStr = FUnitConversion::GetUnitDisplayString(VelocityUnitEnum);

	StringBuilder.Appendf(
		TEXT("Priority: %u | %s | %s | %s | Pos: X=%lld,Y=%lld,Z=%lld | Rot: %s | Vel: %d %s"),
		uint32{EnumToUnderlyingType(Priority)},
		bRemote ? TEXT("Remote") : TEXT("Local"),
		GetStreamingSourceTargetStateName(TargetState),
		bBlockOnSlowLoading ? TEXT("Blocking") : TEXT("NonBlocking"),
		(int64)Location.X, (int64)Location.Y, (int64)Location.Z,
		*Rotation.ToCompactString(),
		VelocityUnitValue,
		VelocityUnitStr
	);

	if (bForce2D)
	{
		StringBuilder += TEXT(" | Force2D");
	}

	if (Shapes.Num())
	{
		StringBuilder += TEXT(" | ");
		int32 ShapeIndex = 0;
		for (const FStreamingSourceShape& Shape : Shapes)
		{
			StringBuilder.Appendf(TEXT("Shape[%d]: %s "), ShapeIndex++, *Shape.ToString());
		}
		StringBuilder.RemoveSuffix(1);
	}

	if (ExtraRadius > 0.f)
	{
		StringBuilder.Appendf(TEXT(" | Extra Radius: %d "), (int32)ExtraRadius);
	}

	if (ExtraAngle > 0.f)
	{
		StringBuilder.Appendf(TEXT(" | Extra Angle: %d "), (int32)ExtraAngle);
	}

	if (TargetGrids.Num())
	{
		StringBuilder.Appendf(TEXT(" | %s TargetGrids: "), (TargetBehavior == EStreamingSourceTargetBehavior::Include) ? TEXT("Included") : TEXT("Excluded"));
		for (const FName& TargetGrid : TargetGrids)
		{
			StringBuilder.Appendf(TEXT("%s, "), *TargetGrid.ToString());
		}
		StringBuilder.RemoveSuffix(2);
	}

	return FString(StringBuilder);
}

void FWorldPartitionStreamingSource::UpdateHash()
{
	// Update old values when they are changing enough, to avoid the case where we are on the edge of a quantization unit.
	if (FVector::Dist(Location, QuantizedLocation) > LocationQuantization)
	{
		QuantizedLocation = Location;
	}
	
	if (Rotation.GetManhattanDistance(QuantizedRotation) > RotationQuantization)
	{
		QuantizedRotation = Rotation;
	}

	FHashBuilder HashBuilder;
	const uint8 ValuesMask = (bBlockOnSlowLoading ? 1 : 0) | (bUseVelocityContributionToCellsSorting ? 2 : 0) | (bReplay ? 4 : 0) | (bRemote ? 8 : 0) | (bForce2D ? 16 : 0);
	HashBuilder	<< Name << TargetState << ValuesMask << Priority << TargetBehavior << TargetGrids << Shapes;

	if (ExtraRadius > 0.f)
	{
		HashBuilder << ExtraRadius;
	}

	if (ExtraAngle > 0.f)
	{
		HashBuilder << ExtraAngle;
	}

	if (LocationQuantization > 0)
	{
		HashBuilder << FMath::FloorToInt(QuantizedLocation.X / LocationQuantization) << FMath::FloorToInt(QuantizedLocation.Y / LocationQuantization);
	}
	else
	{
		HashBuilder << Location.X << Location.Y;
	}

	if (RotationQuantization > 0)
	{
		HashBuilder << FMath::FloorToInt(Rotation.Yaw / RotationQuantization);
	}
	else
	{
		HashBuilder << Rotation.Yaw;
	}

	Hash2D = HashBuilder.GetHash();

	if (LocationQuantization > 0)
	{
		HashBuilder << FMath::FloorToInt(QuantizedLocation.Z / LocationQuantization);
	}
	else
	{
		HashBuilder << Location.Z;
	}

	if (RotationQuantization > 0)
	{
		HashBuilder << FMath::FloorToInt(QuantizedRotation.Pitch / RotationQuantization) << FMath::FloorToInt(QuantizedRotation.Roll / RotationQuantization);
	}
	else
	{
		HashBuilder << QuantizedRotation.Pitch << QuantizedRotation.Roll;
	}

	Hash3D = HashBuilder.GetHash();
};

TArray<TPair<FVector, FVector>> FSphericalSector::BuildDebugMesh() const
{
	TArray<TPair<FVector, FVector>> Segments;
	if (!IsValid())
	{
		return Segments;
	}

	const int32 SegmentCount = FMath::Max(4, FMath::CeilToInt32(64 * (float)Angle / 360.f));
	const FReal AngleStep = Angle / FReal(SegmentCount);
	const FRotator ShapeRotation = FRotationMatrix::MakeFromX(Axis).Rotator();
	const FVector ScaledAxis = FVector::ForwardVector * Radius;
	const int32 RollCount = 16;

	Segments.Reserve(2 * (RollCount + 1) * (SegmentCount + 2));
	int32 LastArcStartIndex = -1;
	for (int32 i = 0; i <= RollCount; ++i)
	{
		const float Roll = 180.f * i / float(RollCount);
		const FTransform Transform(FRotator(0, 0, Roll) + ShapeRotation, Center);
		FVector SegmentStart = Transform.TransformPosition(FRotator(0, -0.5f * Angle, 0).RotateVector(ScaledAxis));
		Segments.Emplace(Center, SegmentStart);
		int32 CurrentArcStartIndex = Segments.Num();
		// Build sector arc
		for (int32 j = 1; j <= SegmentCount; j++)
		{
			FVector SegmentEnd = Transform.TransformPosition(FRotator(0, -0.5f * Angle + (AngleStep * j), 0).RotateVector(ScaledAxis));
			Segments.Emplace(SegmentStart, SegmentEnd);
			SegmentStart = SegmentEnd;
		}
		Segments.Emplace(Center, SegmentStart);
		if (i > 0)
		{
			// Connect sector arc to previous arc
			for (int32 j = 0; j < SegmentCount; j++)
			{
				Segments.Emplace(Segments[LastArcStartIndex + j].Key, Segments[CurrentArcStartIndex + j].Key);
			}
			Segments.Emplace(Segments[LastArcStartIndex + SegmentCount - 1].Value, Segments[CurrentArcStartIndex + SegmentCount - 1].Value);
		}
		LastArcStartIndex = CurrentArcStartIndex;
	}
	return Segments;
}

bool FSphericalSector::IntersectsBox(const FBox2D& InBox) const
{
	// First check if the box intersects the radius
	const FVector2D ClosestPoint = FVector2D::Max(InBox.Min, FVector2D::Min(FVector2D(Center), InBox.Max));
	if ((ClosestPoint - FVector2D(Center)).SizeSquared() > FMath::Square(Radius))
	{
		return false;
	}

	if (Angle < 360.0)
	{
		const FVector2D CenterToMinXMinY((FVector2D(InBox.Min.X, InBox.Min.Y) - FVector2D(Center)).GetSafeNormal());
		const FVector2D CenterToMaxXMinY((FVector2D(InBox.Max.X, InBox.Min.Y) - FVector2D(Center)).GetSafeNormal());
		const FVector2D CenterToMaxXMaxY((FVector2D(InBox.Max.X, InBox.Max.Y) - FVector2D(Center)).GetSafeNormal());
		const FVector2D CenterToMinXMaxY((FVector2D(InBox.Min.X, InBox.Max.Y) - FVector2D(Center)).GetSafeNormal());

		if (Angle <= 180.0f)
		{
			const FVector2D::FReal SinHalfAngle = FMath::Sin(Angle * 0.5f * UE_PI / 180.0);
			const FVector2D::FReal SinAngleMinXMinY = FVector2D::CrossProduct(FVector2D(Axis), CenterToMinXMinY);
			const FVector2D::FReal SinAngleMaxXMinY = FVector2D::CrossProduct(FVector2D(Axis), CenterToMaxXMinY);
			const FVector2D::FReal SinAngleMaxXMaxY = FVector2D::CrossProduct(FVector2D(Axis), CenterToMaxXMaxY);
			const FVector2D::FReal SinAngleMinXMaxY = FVector2D::CrossProduct(FVector2D(Axis), CenterToMinXMaxY);

			// Reject completly left
			if (SinAngleMinXMinY < -SinHalfAngle && SinAngleMaxXMinY < -SinHalfAngle && SinAngleMaxXMaxY < -SinHalfAngle && SinAngleMinXMaxY < -SinHalfAngle)
			{
				return false;
			}

			// Reject completly right
			if (SinAngleMinXMinY > SinHalfAngle && SinAngleMaxXMinY > SinHalfAngle && SinAngleMaxXMaxY > SinHalfAngle && SinAngleMinXMaxY > SinHalfAngle)
			{
				return false;
			}

			const FVector2D::FReal CosAngleMinXMinY = FVector2D::DotProduct(FVector2D(Axis), CenterToMinXMinY);
			const FVector2D::FReal CosAngleMaxXMinY = FVector2D::DotProduct(FVector2D(Axis), CenterToMaxXMinY);
			const FVector2D::FReal CosAngleMaxXMaxY = FVector2D::DotProduct(FVector2D(Axis), CenterToMaxXMaxY);
			const FVector2D::FReal CosAngleMinXMaxY = FVector2D::DotProduct(FVector2D(Axis), CenterToMinXMaxY);

			// Reject backward half circle
			if (CosAngleMinXMinY < 0 && CosAngleMaxXMinY < 0 && CosAngleMaxXMaxY < 0 && CosAngleMinXMaxY < 0)
			{
				return false;
			}
		}
		else
		{
			const FVector2D::FReal CosAngleMinXMinY = FVector2D::DotProduct(FVector2D(Axis),CenterToMinXMinY);
			const FVector2D::FReal CosAngleMaxXMinY = FVector2D::DotProduct(FVector2D(Axis),CenterToMaxXMinY);
			const FVector2D::FReal CosAngleMaxXMaxY = FVector2D::DotProduct(FVector2D(Axis),CenterToMaxXMaxY);
			const FVector2D::FReal CosAngleMinXMaxY = FVector2D::DotProduct(FVector2D(Axis),CenterToMinXMaxY);

			// Include forward half circle
			if (CosAngleMinXMinY >= 0 || CosAngleMaxXMinY >= 0 || CosAngleMaxXMaxY >= 0 || CosAngleMinXMaxY >= 0)
			{
				return true;
			}

			const FVector2D::FReal InvInSinHalfAngle = FMath::Sin((360.0 - Angle) * 0.5 * UE_PI / 180.0);
			const FVector2D::FReal InvSinAngleMinXMinY = FVector2D::CrossProduct(-FVector2D(Axis), CenterToMinXMinY);
			const FVector2D::FReal InvSinAngleMaxXMinY = FVector2D::CrossProduct(-FVector2D(Axis), CenterToMaxXMinY);
			const FVector2D::FReal InvSinAngleMaxXMaxY = FVector2D::CrossProduct(-FVector2D(Axis), CenterToMaxXMaxY);
			const FVector2D::FReal InvSinAngleMinXMaxY = FVector2D::CrossProduct(-FVector2D(Axis), CenterToMinXMaxY);

			// Reject completly inside inverted axis test
			if (InvSinAngleMinXMinY > -InvInSinHalfAngle && InvSinAngleMaxXMinY > -InvInSinHalfAngle && InvSinAngleMaxXMaxY > -InvInSinHalfAngle && InvSinAngleMinXMaxY > -InvInSinHalfAngle &&
				InvSinAngleMinXMinY < InvInSinHalfAngle && InvSinAngleMaxXMinY < InvInSinHalfAngle && InvSinAngleMaxXMaxY < InvInSinHalfAngle && InvSinAngleMinXMaxY < InvInSinHalfAngle)
			{
				return false;
			}
		}
	}

	return true;
}