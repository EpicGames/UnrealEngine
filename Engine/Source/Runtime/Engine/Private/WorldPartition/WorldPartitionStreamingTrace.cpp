// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionStreamingTrace.h"

#if UE_TRACE_WORLD_STREAMING_ENABLED
#include "Engine/World.h"
#include "Hash/xxhash.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"


UE::WorldStreamingTrace::EStreamingContainerState FWorldPartitionStreamingTrace::GetStreamingContainerState(ELevelStreamingState InState)
{
	switch (InState)
	{
	case ELevelStreamingState::Removed:
	case ELevelStreamingState::Unloaded:
	case ELevelStreamingState::FailedToLoad:
		return UE::WorldStreamingTrace::EStreamingContainerState::Unloaded;

	case ELevelStreamingState::Loading:
		return UE::WorldStreamingTrace::EStreamingContainerState::Loading;

	case ELevelStreamingState::LoadedNotVisible:
		return UE::WorldStreamingTrace::EStreamingContainerState::Loaded;

	case ELevelStreamingState::MakingVisible:
		return UE::WorldStreamingTrace::EStreamingContainerState::Activating;

	case ELevelStreamingState::LoadedVisible:
		return UE::WorldStreamingTrace::EStreamingContainerState::Active;

	case ELevelStreamingState::MakingInvisible:
		return UE::WorldStreamingTrace::EStreamingContainerState::Deactivating;

	default:
		checkNoEntry();
		return UE::WorldStreamingTrace::EStreamingContainerState::Unloaded;
	}
}

void FWorldPartitionStreamingTrace::TraceStreamingStateChange(const UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ELevelStreamingState InState)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingChannel))
	{
		return;
	}

	UWorldStreamingTraceSubsystem* Subsystem = UWorld::GetSubsystem<UWorldStreamingTraceSubsystem>(InWorld);
	if (!Subsystem)
	{
		return;
	}

	const UWorldPartitionLevelStreamingDynamic* WPStreamingLevel = Cast<const UWorldPartitionLevelStreamingDynamic>(InStreamingLevel);
	if (!WPStreamingLevel)
	{
		return;
	}

	const UWorldPartitionRuntimeCell* RuntimeCell = WPStreamingLevel->GetWorldPartitionRuntimeCell();
	if (!RuntimeCell)
	{
		return;
	}

	const FGuid CellGuid = RuntimeCell->GetGuid();
	const uint64 Id = FXxHash64::HashBuffer(&CellGuid, sizeof(FGuid)).Hash;

	if (!Subsystem->IsContainerDescribed(Id))
	{
		EnsureContainerHierarchyDescribed(Subsystem, Id, RuntimeCell);
	}

	Subsystem->TraceContainerStateChange(Id, GetStreamingContainerState(InState));
}

void FWorldPartitionStreamingTrace::EnsureContainerHierarchyDescribed(UWorldStreamingTraceSubsystem* InSubsystem, uint64 InCellId, const UWorldPartitionRuntimeCell* InCell)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionStreamingTrace::EnsureContainerHierarchyDescribed);

	if (!InCell->RuntimeCellData)
	{
		UE_LOGF(LogWorldStreamingTrace, Log, "Cell '%ls' has no RuntimeCellData - skipping trace description", *InCell->GetDebugName());
		return;
	}

	const UWorldPartitionRuntimeCellData* CellData = InCell->RuntimeCellData.Get();

	const FString GridNameString = CellData->GridName.ToString();
	const uint64 GridNameContainerId = FXxHash64::HashBuffer(*GridNameString, GridNameString.Len() * sizeof(TCHAR)).Hash;
	InSubsystem->EnsureContainerDescribed(GridNameContainerId, /*InParentId=*/0, GridNameString, FStringView(), FBox(ForceInit), {});

	const FString LevelString = (CellData->HierarchicalLevel == MAX_int32) ? GridNameString + TEXT("_NonSpatial") : FString::Printf(TEXT("%s_L%d"), *GridNameString, CellData->HierarchicalLevel);
	// Hash grid name + level as binary fields to prevent collision with grid-name IDs.
	FXxHash64Builder LevelHashBuilder;
	LevelHashBuilder.Update(*GridNameString, GridNameString.Len() * sizeof(TCHAR));
	LevelHashBuilder.Update(&CellData->HierarchicalLevel, sizeof(CellData->HierarchicalLevel));
	const uint64 LevelContainerId = LevelHashBuilder.Finalize().Hash;
	InSubsystem->EnsureContainerDescribed(LevelContainerId, GridNameContainerId, LevelString, FStringView(), FBox(ForceInit), {});

	const uint64 ParentId = LevelContainerId;

	TArray<uint64> Tags;
	if (!InCell->GetDataLayers().IsEmpty())
	{
		static const FString DataLayerTagName(TEXT("DataLayer"));
		static const uint64 DataLayerGroupTagId = FXxHash64::HashBuffer(*DataLayerTagName, DataLayerTagName.Len() * sizeof(TCHAR)).Hash;
		InSubsystem->EnsureTagGroupDescribed(DataLayerGroupTagId, DataLayerTagName);

		TArray<const UDataLayerInstance*> DataLayerInstances = InCell->GetDataLayerInstances();
		for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
		{
			const FString TagName = DataLayerInstance->GetDataLayerShortName();
			FXxHash64Builder Builder;
			Builder.Update(*DataLayerTagName, DataLayerTagName.Len() * sizeof(TCHAR));
			Builder.Update(TEXT(":"), sizeof(TCHAR));
			Builder.Update(*TagName, TagName.Len() * sizeof(TCHAR));
			const uint64 TagId = Builder.Finalize().Hash;
			InSubsystem->EnsureTagDescribed(TagId, DataLayerGroupTagId, /*InParentId=*/0, TagName);
			Tags.Add(TagId);
		}
	}

	const FName LevelPackageName = InCell->GetLevelPackageName();
	const FString PackageNameString = LevelPackageName.IsNone() ? FString() : LevelPackageName.ToString();
	InSubsystem->EnsureContainerDescribed(InCellId, ParentId, InCell->GetDebugName(), PackageNameString, InCell->GetCellBounds(), Tags);

#if UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
	InSubsystem->TraceDependenciesForPackage(InCellId, LevelPackageName);
#endif // UE_TRACE_WORLD_STREAMING_DEPENDENCIES_ENABLED
}

void FWorldPartitionStreamingTrace::TraceStreamingSourceUpdate(const UWorld* InWorld, const FWorldPartitionStreamingSource& InStreamingSource)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingChannel))
	{
		return;
	}
	UWorldStreamingTraceSubsystem* Subsystem = UWorld::GetSubsystem<UWorldStreamingTraceSubsystem>(InWorld);
	if (!Subsystem)
	{
		return;
	}

	const uint64 SourceId = InStreamingSource.Name.ToUnstableInt();
	if (!Subsystem->IsStreamingSourceDescribed(SourceId))
	{
		const FString NameString = InStreamingSource.Name.ToString();
		Subsystem->EnsureStreamingSourceDescribed(SourceId, NameString);
	}
	Subsystem->TraceStreamingSourceUpdate(SourceId, InStreamingSource.Location);
}

void FWorldPartitionStreamingTrace::EndStreamingSourceUpdates(const UWorld* InWorld)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingChannel))
	{
		return;
	}
	if (UWorldStreamingTraceSubsystem* Subsystem = UWorld::GetSubsystem<UWorldStreamingTraceSubsystem>(InWorld))
	{
		Subsystem->FlushInactiveStreamingSources();
	}
}

#if UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
void FWorldPartitionStreamingTrace::TraceStreamingContainerPriorities(const UWorld* InWorld, TArrayView<const TPair<FGuid, float>> InCellPriorities)
{
	if (!UE_TRACE_MINIMAL_CHANNELEXPR_IS_ENABLED(WorldStreamingPriorityChannel))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionStreamingTrace::TraceStreamingContainerPriorities);

	if (InCellPriorities.IsEmpty())
	{
		return;
	}

	UWorldStreamingTraceSubsystem* Subsystem = UWorld::GetSubsystem<UWorldStreamingTraceSubsystem>(InWorld);
	if (!Subsystem)
	{
		return;
	}

	// Hash FGuid -> uint64 and split into parallel arrays for the trace event
	TArray<uint64> ContainerIds;
	TArray<float> Priorities;
	ContainerIds.Reserve(InCellPriorities.Num());
	Priorities.Reserve(InCellPriorities.Num());

	for (const TPair<FGuid, float>& CellPriority : InCellPriorities)
	{
		ContainerIds.Add(FXxHash64::HashBuffer(&CellPriority.Key, sizeof(FGuid)).Hash);
		Priorities.Add(CellPriority.Value);
	}

	Subsystem->TraceContainerPriorityBatch(ContainerIds, Priorities);
}
#endif // UE_TRACE_WORLD_STREAMING_PRIORITY_ENABLED
#endif // #if UE_TRACE_WORLD_STREAMING_ENABLED