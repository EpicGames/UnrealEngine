// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionSubsystem.h"
#include "SceneView.h"
#include "UnrealEngine.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionDebugHelper.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "WorldPartition/WorldPartitionReplay.h"
#include "WorldPartition/WorldPartitionDraw2DContext.h"
#include "WorldPartition/WorldPartitionSettings.h"
#include "WorldPartition/WorldPartitionStreamingPolicy.h"
#include "WorldPartition/WorldPartitionStreamingTrace.h"
#include "WorldPartition/ActorDescContainerSubsystem.h"
#include "WorldPartition/DataLayer/WorldDataLayersActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"
#include "Engine/Canvas.h"
#include "Engine/CoreSettings.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingGCHelper.h"
#include "Engine/NetDriver.h"
#include "Engine/NetConnection.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "Engine/LevelBounds.h"
#include "Debug/DebugDrawService.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/ParallelFor.h"
#include "Algo/ForEach.h"
#include "Misc/HashBuilder.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "LevelUtils.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"
#endif

#if !UE_BUILD_SHIPPING
#include "Engine/Engine.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionSubsystem)


DECLARE_CYCLE_STAT(TEXT("Update Streaming State"), STAT_WorldPartitionUpdateStreamingState, STATGROUP_WorldPartition);
DECLARE_CYCLE_STAT(TEXT("Update Streaming State (External)"), STAT_WorldPartitionUpdateStreamingStateExternal, STATGROUP_WorldPartition);
CSV_DEFINE_CATEGORY(WorldPartition, (!UE_BUILD_SHIPPING));


extern int32 GBlockOnSlowStreaming;
static const FName NAME_WorldPartitionRuntimeHash("WorldPartitionRuntimeHash");

static int32 GServerStreamingSourceMinimumExtraRadius = 400;
static FAutoConsoleVariableRef CVarServerStreamingSourceMinimumExtraRadius(
	TEXT("wp.Runtime.ServerStreamingSourceMinimumExtraRadius"),
	GServerStreamingSourceMinimumExtraRadius,
	TEXT("Minimum value added to the radius of the streaming sources used by the server (in Unreal unit)."),
	ECVF_Default
);

static int32 GServerStreamingSourceMinimumExtraAngle = 1;
static FAutoConsoleVariableRef CVarServerStreamingSourceMinimumExtraAngle(
	TEXT("wp.Runtime.ServerStreamingSourceMinimumExtraAngle"),
	GServerStreamingSourceMinimumExtraAngle,
	TEXT("Minimum value added to the angle of the streaming source shape sector used by the server (in degree)."),
	ECVF_Default
);

static int32 GDrawWorldPartitionIndex = -1;
static FAutoConsoleCommand CVarDrawWorldPartitionIndex(
	TEXT("wp.Runtime.DrawWorldPartitionIndex"),
	TEXT("Sets the index of the wanted partitioned world to display debug draw.\n")
	TEXT("Sets < 0 to display debug draw all registered partitioned worlds.\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() == 1)
		{
			GDrawWorldPartitionIndex = FCString::Atoi(*Args[0]);
		}
	}));

static float GDrawRuntimeHash2DScaleFactor = 1;
static FAutoConsoleVariableRef CVarDrawRuntimeHash2DScaleFactor(
	TEXT("wp.Runtime.DrawRuntimeHash2DScaleFactor"),
	GDrawRuntimeHash2DScaleFactor,
	TEXT("Allows to reduce the size on screen of the 2D debug display of world partition runtime hash (value must be between 0.1 and 1)"),
	ECVF_Default
);

static int32 GDrawRuntimeHash3D = 0;
static FAutoConsoleCommand CVarDrawRuntimeHash3D(
	TEXT("wp.Runtime.ToggleDrawRuntimeHash3D"),
	TEXT("Toggles 3D debug display of world partition runtime hash."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawRuntimeHash3D = !GDrawRuntimeHash3D; }));

static int32 GDrawRuntimeHash2D = 0;
static FAutoConsoleCommand CVarDrawRuntimeHash2D(
	TEXT("wp.Runtime.ToggleDrawRuntimeHash2D"),
	TEXT("Toggles 2D debug display of world partition runtime hash."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawRuntimeHash2D = !GDrawRuntimeHash2D; }));

static int32 GDrawStreamingSources = 0;
static FAutoConsoleCommand CVarDrawStreamingSources(
	TEXT("wp.Runtime.ToggleDrawStreamingSources"),
	TEXT("Toggles debug display of world partition streaming sources."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawStreamingSources = !GDrawStreamingSources; }));

static int32 GDrawStreamingPerfs = 0;
static FAutoConsoleCommand CVarDrawStreamingPerfs(
	TEXT("wp.Runtime.ToggleDrawStreamingPerfs"),
	TEXT("Toggles debug display of world partition streaming perfs."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawStreamingPerfs = !GDrawStreamingPerfs; }));

static int32 GDrawLegends = 0;
static FAutoConsoleCommand CVarGDrawLegends(
	TEXT("wp.Runtime.ToggleDrawLegends"),
	TEXT("Toggles debug display of world partition legends."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawLegends = !GDrawLegends; }));

static int32 GDrawRuntimeCellsDetails = 0;
static FAutoConsoleCommand CVarDrawRuntimeCellsDetails(
	TEXT("wp.Runtime.ToggleDrawRuntimeCellsDetails"),
	TEXT("Toggles debug display of world partition runtime streaming cells."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawRuntimeCellsDetails = !GDrawRuntimeCellsDetails; }));

static int32 GDrawDataLayers = 0;
static FAutoConsoleCommand CVarDrawDataLayers(
	TEXT("wp.Runtime.ToggleDrawDataLayers"),
	TEXT("Toggles debug display of active data layers."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawDataLayers = !GDrawDataLayers; }));

int32 GDrawDataLayersLoadTime = 0;
static FAutoConsoleCommand CVarDrawDataLayersLoadTime(
	TEXT("wp.Runtime.ToggleDrawDataLayersLoadTime"),
	TEXT("Toggles debug display of active data layers load time."),
	FConsoleCommandDelegate::CreateLambda([] { GDrawDataLayersLoadTime = !GDrawDataLayersLoadTime; }));

int32 GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurgeForWP = 64;
static FAutoConsoleVariableRef CVarGLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurgeForWP(
	TEXT("wp.Runtime.LevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurgeForWP"),
	GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurgeForWP,
	TEXT("Force a GC update when there's more than the number of specified pending purge levels."),
	ECVF_Default
);

int32 GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurgeForWP = 0;	// Disabled by default
static FAutoConsoleVariableRef CVarGLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurgeForWP(
	TEXT("wp.Runtime.LevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurgeForWP"),
	GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurgeForWP,
	TEXT("Force a GC update when there's more than the number of specified Actors (from levels) pending purge."),
	ECVF_Default
);

static FAutoConsoleCommandWithOutputDevice GDumpStreamingSourcesCmd(
	TEXT("wp.Runtime.DumpStreamingSources"),
	TEXT("Dumps active streaming sources to the log"),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (const UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{				
					WorldPartitionSubsystem->DumpStreamingSources(OutputDevice);
				}
			}
		}
	})
);

static FAutoConsoleCommandWithOutputDevice GDumpWorldPartitionsCmd(
	TEXT("wp.Runtime.DumpWorldPartitions"),
	TEXT("Dumps active world partitions to the log"),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic([](FOutputDevice& OutputDevice)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (const UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>())
				{
					WorldPartitionSubsystem->DumpWorldPartitions(OutputDevice);
				}
			}
		}
	})
);

static int32 GMaxLoadingStreamingCells = 4;
static FAutoConsoleVariableRef CMaxLoadingStreamingCells(
	TEXT("wp.Runtime.MaxLoadingStreamingCells"),
	GMaxLoadingStreamingCells,
	TEXT("Used to limit the number of concurrent loading world partition streaming cells."));

static float GUpdateStreamingStateTimeLimit = 0.f;
static FAutoConsoleVariableRef CVarUdateStreamingStateTimeLimit(
	TEXT("wp.Runtime.UpdateStreamingStateTimeLimit"),
	GUpdateStreamingStateTimeLimit,
	TEXT("Maximum amount of time to spend doing World Partition UpdateStreamingState (ms per frame)."),
	ECVF_Default
);

#if !UE_BUILD_SHIPPING
TMap<FName, int32> UWorldPartitionSubsystem::OverriddenLoadingRanges;
uint32 UWorldPartitionSubsystem::OverriddenLoadingRangesEpoch = 0;
static const TCHAR* GOverrideLoadingRangeCommandName = TEXT("wp.Runtime.OverrideRuntimeLoadingRange");
static FDelegateHandle OnWorldPartitionSubsystemDeinitializedFDelegateHandle;
FAutoConsoleCommand UWorldPartitionSubsystem::OverrideLoadingRangeCommand(
	GOverrideLoadingRangeCommandName,
	TEXT("Sets runtime loading range. Args -grid=[Name] -range=[Range]"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		FString ArgString = FString::Join(InArgs, TEXT(" "));
		FName OverrideGridName;
		int32 OverrideLoadingRange = -1;
		FParse::Value(*ArgString, TEXT("grid="), OverrideGridName);
		FParse::Value(*ArgString, TEXT("range="), OverrideLoadingRange);

		if (!OnWorldPartitionSubsystemDeinitializedFDelegateHandle.IsValid())
		{
			OnWorldPartitionSubsystemDeinitializedFDelegateHandle = UWorldPartitionSubsystem::OnWorldPartitionSubsystemDeinitialized.AddLambda([](UWorldPartitionSubsystem* InWorldPartitionSubsystem, UWorld* InWorld)
			{
				if (InWorld && InWorld->IsGameWorld())
				{
					OverriddenLoadingRanges.Reset();
				}
			});
		}

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && World->IsGameWorld())
			{
				if (UWorld::HasSubsystem<UWorldPartitionSubsystem>(World))
				{
					FWorldPartitionHelpers::ServerExecConsoleCommand(World, GOverrideLoadingRangeCommandName, InArgs);
					UWorldPartitionSubsystem::SetOverrideLoadingRange(OverrideGridName, OverrideLoadingRange);
					break;
				}
			}
		}
	})
);
#endif

TMulticastDelegate<void(UWorldPartitionSubsystem*, UWorld*)> UWorldPartitionSubsystem::OnWorldPartitionSubsystemInitialized;
TMulticastDelegate<void(UWorldPartitionSubsystem*, UWorld*)> UWorldPartitionSubsystem::OnWorldPartitionSubsystemDeinitialized;

UWorldPartitionSubsystem::UWorldPartitionSubsystem()
	: NumWorldPartitionServerStreamingEnabled(0)
	, ServerClientsVisibleLevelsHash(0)
{}

UWorldPartition* UWorldPartitionSubsystem::GetWorldPartition()
{
	return GetWorld()->GetWorldPartition();
}

const UWorldPartition* UWorldPartitionSubsystem::GetWorldPartition() const
{
	return GetWorld()->GetWorldPartition();
}

#if WITH_EDITOR
FWorldPartitionActorFilter UWorldPartitionSubsystem::GetWorldPartitionActorFilter(const FString& InWorldPackage, EWorldPartitionActorFilterType InFilterTypes) const
{
	TSet<FString> VisitedPackageStack;
	return GetWorldPartitionActorFilterInternal(InWorldPackage, InFilterTypes, VisitedPackageStack);
}


FWorldPartitionActorFilter UWorldPartitionSubsystem::GetWorldPartitionActorFilterInternal(const FString& InWorldPackage, EWorldPartitionActorFilterType InFilterTypes, TSet<FString>& InOutVisitedPackageStack) const
{
	if (InOutVisitedPackageStack.Contains(InWorldPackage))
	{
		return FWorldPartitionActorFilter(InWorldPackage);
	}

	InOutVisitedPackageStack.Add(InWorldPackage);
	
	// Most of the time if this will return an existing Container but when loading a new LevelInstance (Content Browser Drag&Drop, Create LI) 
	// This will make sure Container exists.
	UActorDescContainer* LevelContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer(UActorDescContainer::FInitializeParams{ *InWorldPackage });
	check(LevelContainer);
	ON_SCOPE_EXIT{ UActorDescContainerSubsystem::GetChecked().UnregisterContainer(LevelContainer); };

	// Lazy create filter for now
	TArray<const FWorldPartitionActorDesc*> ContainerActorDescs;
	const FWorldDataLayersActorDesc* WorldDataLayersActorDesc = nullptr;

	for (FActorDescList::TConstIterator<> ActorDescIt(LevelContainer); ActorDescIt; ++ActorDescIt)
	{
		if (ActorDescIt->GetActorNativeClass()->IsChildOf<AWorldDataLayers>())
		{
			check(!WorldDataLayersActorDesc);
			WorldDataLayersActorDesc = static_cast<const FWorldDataLayersActorDesc*>(*ActorDescIt);
		}
		else if ((ActorDescIt->GetChildContainerFilterType() & InFilterTypes) != EWorldPartitionActorFilterType::None)
		{
			ContainerActorDescs.Add(*ActorDescIt);
		}
	}

	FWorldPartitionActorFilter Filter(InWorldPackage);

	if (WorldDataLayersActorDesc)
	{
		for (const FDataLayerInstanceDesc& DataLayerInstanceDesc : WorldDataLayersActorDesc->GetDataLayerInstances())
		{
			// For now consider all DataLayerInstances using Assets as filters that are included by default
			if (DataLayerInstanceDesc.SupportsActorFilters())
			{
				Filter.DataLayerFilters.Add(FSoftObjectPath(DataLayerInstanceDesc.GetAssetPath().ToString()), FWorldPartitionActorFilter::FDataLayerFilter(DataLayerInstanceDesc.GetShortName(), DataLayerInstanceDesc.IsIncludedInActorFilterDefault()));
			}
		}
	}

	for (const FWorldPartitionActorDesc* ContainerActorDesc : ContainerActorDescs)
	{
		// Get World Default Filter
		FWorldPartitionActorFilter* ChildFilter = new FWorldPartitionActorFilter(GetWorldPartitionActorFilterInternal(ContainerActorDesc->GetChildContainerPackage().ToString(), InFilterTypes, InOutVisitedPackageStack));
		ChildFilter->DisplayName = ContainerActorDesc->GetActorLabelOrName().ToString();

		// Apply Filter to Default
		if (const FWorldPartitionActorFilter* ContainerFilter = ContainerActorDesc->GetChildContainerFilter())
		{
			ChildFilter->Override(*ContainerFilter);
		}

		Filter.AddChildFilter(ContainerActorDesc->GetGuid(), ChildFilter);
	}

	verify(InOutVisitedPackageStack.Remove(InWorldPackage));

	return Filter;
}

TMap<FActorContainerID, TSet<FGuid>> UWorldPartitionSubsystem::GetFilteredActorsPerContainer(const FActorContainerID& InContainerID, const FString& InWorldPackage, const FWorldPartitionActorFilter& InActorFilter, EWorldPartitionActorFilterType InFilterTypes)
{
	TMap<FActorContainerID, TSet<FGuid>> FilteredActors;

	FWorldPartitionActorFilter ContainerFilter = GetWorldPartitionActorFilter(InWorldPackage, InFilterTypes);
	ContainerFilter.Override(InActorFilter);

	// Flatten Filter to FActorContainerID map
	TMap<FActorContainerID, TMap<FSoftObjectPath, FWorldPartitionActorFilter::FDataLayerFilter>> DataLayerFiltersPerContainer;
	TFunction<void(const FActorContainerID&, const FWorldPartitionActorFilter&)> ProcessFilter = [&DataLayerFiltersPerContainer, &ProcessFilter](const FActorContainerID& InContainerID, const FWorldPartitionActorFilter& InContainerFilter)
	{
		check(!DataLayerFiltersPerContainer.Contains(InContainerID));
		TMap<FSoftObjectPath,FWorldPartitionActorFilter::FDataLayerFilter>& DataLayerFilters = DataLayerFiltersPerContainer.Add(InContainerID);
		
		for (auto& [AssetPath, DataLayerFilter] : InContainerFilter.DataLayerFilters)
		{
			DataLayerFilters.Add(AssetPath, DataLayerFilter);
		}

		for (auto& [ActorGuid, WorldPartitionActorFilter] : InContainerFilter.GetChildFilters())
		{
			ProcessFilter(FActorContainerID(InContainerID, ActorGuid), *WorldPartitionActorFilter);
		}
	};

	ProcessFilter(InContainerID, ContainerFilter);

	// Keep track of registered containers to unregister them
	TMap<FName, UActorDescContainer*> RegisteredContainers;

	TFunction<UActorDescContainer* (FName)> FindOrRegisterContainer = [this, &RegisteredContainers](FName ContainerPackage)
	{
		if (UActorDescContainer** FoundContainer = RegisteredContainers.Find(ContainerPackage))
		{
			return *FoundContainer;
		}
	
		UActorDescContainer* RegisteredContainer = UActorDescContainerSubsystem::GetChecked().RegisterContainer(UActorDescContainer::FInitializeParams{ ContainerPackage });
		RegisteredContainers.Add(ContainerPackage, RegisteredContainer);
		return RegisteredContainer;
	};

	TFunction<void(const FActorContainerID&, const UActorDescContainer*, TSet<FName>&)> ProcessContainers = [&FindOrRegisterContainer, &DataLayerFiltersPerContainer, &FilteredActors, &ProcessContainers, &InFilterTypes](const FActorContainerID& InContainerID, const UActorDescContainer* InContainer, TSet<FName>& InOutVisitedPackageStack)
	{
		if(InOutVisitedPackageStack.Contains(InContainer->GetContainerPackage()))
		{
			return;
		}
		InOutVisitedPackageStack.Add(InContainer->GetContainerPackage());

		const TMap<FSoftObjectPath, FWorldPartitionActorFilter::FDataLayerFilter>& DataLayerFilters = DataLayerFiltersPerContainer.FindChecked(InContainerID);
		for (FActorDescList::TConstIterator<> ActorDescIt(InContainer); ActorDescIt; ++ActorDescIt)
		{
			if (ActorDescIt->GetDataLayers().Num() > 0)
			{
				bool bExcluded = false;
				for (FName DataLayerName : ActorDescIt->GetDataLayers())
				{
					FSoftObjectPath DataLayerAsset(DataLayerName.ToString());
					if (const FWorldPartitionActorFilter::FDataLayerFilter* DataLayerFilter = DataLayerFilters.Find(DataLayerAsset))
					{
						if (DataLayerFilter->bIncluded)
						{
							bExcluded = false;
							break;
						}
						else
						{
							bExcluded = true;
						}
					}
				}

				if (bExcluded)
				{
					FilteredActors.FindOrAdd(InContainerID).Add(ActorDescIt->GetGuid());
				}
			}

			if ((ActorDescIt->GetChildContainerFilterType() & InFilterTypes) != EWorldPartitionActorFilterType::None)
			{
				if (const FWorldPartitionActorFilter* ChildFilter = ActorDescIt->GetChildContainerFilter())
				{
					UActorDescContainer* ChildContainer = FindOrRegisterContainer(ActorDescIt->GetChildContainerPackage());
					check(ChildContainer);
					ProcessContainers(FActorContainerID(InContainerID, ActorDescIt->GetGuid()), ChildContainer, InOutVisitedPackageStack);
				}
			}
		}

		verify(InOutVisitedPackageStack.Remove(InContainer->GetContainerPackage()));
	};

	UActorDescContainer* Container = FindOrRegisterContainer(*InWorldPackage);

	TSet<FName> VisitedPackageStack;
	ProcessContainers(InContainerID, Container, VisitedPackageStack);
	verify(VisitedPackageStack.IsEmpty());
	
	// Unregister Containers
	UActorDescContainerSubsystem& ContainerSubsystem = UActorDescContainerSubsystem::GetChecked();
	for (auto& [Name, RegisteredContainer] : RegisteredContainers)
	{
		ContainerSubsystem.UnregisterContainer(RegisteredContainer);
	}

	return FilteredActors;
}


bool UWorldPartitionSubsystem::IsRunningConvertWorldPartitionCommandlet()
{
	static UClass* WorldPartitionConvertCommandletClass = FindObject<UClass>(nullptr, TEXT("/Script/UnrealEd.WorldPartitionConvertCommandlet"), EFindObjectFlags::ExactClass);
	check(WorldPartitionConvertCommandletClass);
	return GetRunningCommandletClass() && GetRunningCommandletClass()->IsChildOf(WorldPartitionConvertCommandletClass);
}

#endif

void UWorldPartitionSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

#if WITH_EDITOR
	bIsRunningConvertWorldPartitionCommandlet = IsRunningConvertWorldPartitionCommandlet();
	if(bIsRunningConvertWorldPartitionCommandlet)
	{
		return;
	}
#endif

	GetWorld()->OnWorldPartitionInitialized().AddUObject(this, &UWorldPartitionSubsystem::OnWorldPartitionInitialized);
	GetWorld()->OnWorldPartitionUninitialized().AddUObject(this, &UWorldPartitionSubsystem::OnWorldPartitionUninitialized);
	if (GetWorld()->IsGameWorld())
	{
#if WITH_EDITOR
		FWorldDelegates::OnStartGameInstance.AddUObject(this, &UWorldPartitionSubsystem::OnStartGameInstance);
#endif // WITH_EDITOR

		FLevelStreamingDelegates::OnLevelBeginMakingVisible.AddUObject(this, &UWorldPartitionSubsystem::OnLevelBeginMakingVisible);
		FLevelStreamingDelegates::OnLevelBeginMakingInvisible.AddUObject(this, &UWorldPartitionSubsystem::OnLevelBeginMakingInvisible);
		FLevelStreamingDelegates::OnLevelStreamingTargetStateChanged.AddUObject(this, &UWorldPartitionSubsystem::OnLevelStreamingTargetStateChanged);
		FLevelStreamingDelegates::OnLevelStreamingStateChanged.AddUObject(this, &UWorldPartitionSubsystem::OnLevelStreamingStateChanged);
	}

	OnWorldPartitionSubsystemInitialized.Broadcast(this, GetWorld());
}

void UWorldPartitionSubsystem::Deinitialize()
{
#if WITH_EDITOR
	if (bIsRunningConvertWorldPartitionCommandlet)
	{
		Super::Deinitialize();
		return;
	}
#endif

	OnWorldPartitionSubsystemDeinitialized.Broadcast(this, GetWorld());

	GetWorld()->OnWorldPartitionInitialized().RemoveAll(this);
	GetWorld()->OnWorldPartitionUninitialized().RemoveAll(this);
	if (GetWorld()->IsGameWorld())
	{
		FLevelStreamingDelegates::OnLevelBeginMakingVisible.RemoveAll(this);
		FLevelStreamingDelegates::OnLevelBeginMakingInvisible.RemoveAll(this);
		FLevelStreamingDelegates::OnLevelStreamingTargetStateChanged.RemoveAll(this);
		FLevelStreamingDelegates::OnLevelStreamingStateChanged.RemoveAll(this);

#if WITH_EDITOR
		FWorldDelegates::OnStartGameInstance.RemoveAll(this);
#endif // WITH_EDITOR
	}

	// At this point World Partition should be uninitialized
	check(!GetWorldPartition() || !GetWorldPartition()->IsInitialized());

	Super::Deinitialize();
}

// WorldPartitionSubsystem is required for WorldPartition to be properly initialized (even when world is Inactive like during Cook or world duplication).
bool UWorldPartitionSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::Inactive || WorldType == EWorldType::EditorPreview;
}

void UWorldPartitionSubsystem::ForEachWorldPartition(TFunctionRef<bool(UWorldPartition*)> Func)
{
	for (UWorldPartition* WorldPartition : RegisteredWorldPartitions)
	{
		if (!Func(WorldPartition))
		{
			return;
		}
	}
}

#if !UE_BUILD_SHIPPING
void UWorldPartitionSubsystem::SetOverrideLoadingRange(FName Name, int32 LoadingRange)
{
	if (LoadingRange >= 0)
	{
		OverriddenLoadingRanges.Add(Name, LoadingRange);
	}
	else
	{
		OverriddenLoadingRanges.Remove(Name);
	}
	OverriddenLoadingRangesEpoch++;
}

bool UWorldPartitionSubsystem::GetOverrideLoadingRange(FName Name, int32& LoadingRange)
{
	if (int32* OverriddenLoadingRange = OverriddenLoadingRanges.Find(Name))
	{
		LoadingRange = *OverriddenLoadingRange;
		return true;
	}
	return false;
}

uint32 UWorldPartitionSubsystem::GetOverriddenLoadingRangesEpoch()
{
	return OverriddenLoadingRangesEpoch;
}
#endif

#if WITH_EDITOR
void UWorldPartitionSubsystem::OnStartGameInstance(UGameInstance* GameInstance)
{
	InitDynamicLoadingRangeScale();
}
#endif // WITH_EDITOR

void UWorldPartitionSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	if (RegisteredWorldPartitions.IsEmpty())
	{
		DrawHandle = UDebugDrawService::Register(TEXT("Game"), FDebugDrawDelegate::CreateUObject(this, &UWorldPartitionSubsystem::Draw));

		// Enforce some GC settings when using World Partition
		if (GetWorld()->IsGameWorld())
		{
			LevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge = GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge;
			LevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge = GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge;
			LevelStreamingForceGCAfterLevelStreamedOut = GLevelStreamingForceGCAfterLevelStreamedOut;

			GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge = GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurgeForWP;
			GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge = GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurgeForWP;
			GLevelStreamingForceGCAfterLevelStreamedOut = 0;
		}
	}

	if (bHasBegunPlay)
	{
		InWorldPartition->OnBeginPlay();
	}

	check(!RegisteredWorldPartitions.Contains(InWorldPartition));
	RegisteredWorldPartitions.Add(InWorldPartition);
	IncrementalUpdateWorldPartitionsPendingAdd.Add(InWorldPartition);
	NumWorldPartitionServerStreamingEnabled += InWorldPartition->IsServerStreamingEnabled() ? 1 : 0;
	check(NumWorldPartitionServerStreamingEnabled >= 0);
}

void UWorldPartitionSubsystem::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	check(RegisteredWorldPartitions.Contains(InWorldPartition));
	RegisteredWorldPartitions.Remove(InWorldPartition);
	IncrementalUpdateWorldPartitionsPendingAdd.Remove(InWorldPartition);
	IncrementalUpdateWorldPartitions.Remove(InWorldPartition);
	const UWorld* OwningWorld = GetWorld();
	if (OwningWorld->IsGameWorld())
	{
		TOptional<TSet<TWeakObjectPtr<UWorldPartitionLevelStreamingDynamic>>*> PendingStreamingLevels;
		auto GetPendingStreamingLevels = [this, InWorldPartition, &PendingStreamingLevels]() -> TSet<TWeakObjectPtr<UWorldPartitionLevelStreamingDynamic>>&
		{
			if (!PendingStreamingLevels.IsSet())
			{
				PendingStreamingLevels = &WorldPartitionUninitializationPendingStreamingLevels.FindOrAdd(FSoftObjectPath(InWorldPartition));
			}
			return *PendingStreamingLevels.GetValue();
		};

		if (InWorldPartition->GetTypedOuter<UWorld>() != OwningWorld)
		{
			const FSoftObjectPath WorldPartition(InWorldPartition);
			for (ULevelStreaming* StreamingLevel : OwningWorld->GetStreamingLevels())
			{
				UWorldPartitionLevelStreamingDynamic* WorldPartitionStreamingLevel = Cast<UWorldPartitionLevelStreamingDynamic>(StreamingLevel);
				if (WorldPartitionStreamingLevel && 
					(WorldPartitionStreamingLevel->GetOuterWorldPartition() == WorldPartition) && 
					(WorldPartitionStreamingLevel->GetLevelStreamingState() != ELevelStreamingState::FailedToLoad))
				{
					GetPendingStreamingLevels().Add(WorldPartitionStreamingLevel);
				}
			}
		}
	}
	NumWorldPartitionServerStreamingEnabled -= InWorldPartition->IsServerStreamingEnabled() ? 1 : 0;
	check(NumWorldPartitionServerStreamingEnabled >= 0);

	if (RegisteredWorldPartitions.IsEmpty())
	{
		if (GetWorld()->IsGameWorld())
		{
			GLevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge = LevelStreamingContinuouslyIncrementalGCWhileLevelsPendingPurge;
			GLevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge = LevelStreamingContinuouslyIncrementalGCWhileActorsPendingPurge;
			GLevelStreamingForceGCAfterLevelStreamedOut = LevelStreamingForceGCAfterLevelStreamedOut;
		}

		if (DrawHandle.IsValid())
		{
			UDebugDrawService::Unregister(DrawHandle);
			DrawHandle.Reset();
		}
	}
}

bool UWorldPartitionSubsystem::HasUninitializationPendingStreamingLevels(const UWorldPartition* InWorldPartition) const
{
	if (const TSet<TWeakObjectPtr<UWorldPartitionLevelStreamingDynamic>>* PendingStreamingLevels = InWorldPartition ? WorldPartitionUninitializationPendingStreamingLevels.Find(FSoftObjectPath(InWorldPartition)) : nullptr)
	{
		if (ensure(!PendingStreamingLevels->IsEmpty()))
		{
			return true;
		}
	}
	return false;
}

static FORCEINLINE bool IsLoadingOrPendingLoadStreamingLevel(const ULevelStreaming* InStreamingLevel)
{
	// Consider loading and pending to load world partition streaming levels
	if (InStreamingLevel && InStreamingLevel->IsA<UWorldPartitionLevelStreamingDynamic>())
	{
		const ELevelStreamingState CurrentState = InStreamingLevel->GetLevelStreamingState();
		return ((CurrentState == ELevelStreamingState::Loading) || ((CurrentState == ELevelStreamingState::Removed || CurrentState == ELevelStreamingState::Unloaded) && InStreamingLevel->ShouldBeLoaded()));
	}
	return false;
}

void UWorldPartitionSubsystem::UpdateLoadingAndPendingLoadStreamingLevels(const ULevelStreaming* InStreamingLevel)
{
	if (IsLoadingOrPendingLoadStreamingLevel(InStreamingLevel))
	{
		WorldPartitionLoadingAndPendingLoadStreamingLevels.Add(InStreamingLevel);
	}
	else
	{
		WorldPartitionLoadingAndPendingLoadStreamingLevels.Remove(InStreamingLevel);
	}
}

void UWorldPartitionSubsystem::OnLevelStreamingStateChanged(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* LevelIfLoaded, ELevelStreamingState PreviousState, ELevelStreamingState NewState)
{
	if (InWorld != GetWorld())
	{
		return;
	}

	UE_TRACE_WP_STREAMING_CONTAINER_STATE(InWorld, InStreamingLevel, NewState);

	UpdateLoadingAndPendingLoadStreamingLevels(InStreamingLevel);

	if ((NewState == ELevelStreamingState::Removed) || (NewState == ELevelStreamingState::FailedToLoad))
	{
		if (const UWorldPartitionLevelStreamingDynamic* WorldPartitionStreamingLevel = Cast<const UWorldPartitionLevelStreamingDynamic>(InStreamingLevel))
		{
			check(InWorld->IsGameWorld());
			const FSoftObjectPath& WorldPartition = WorldPartitionStreamingLevel->GetOuterWorldPartition();
			if (TSet<TWeakObjectPtr<UWorldPartitionLevelStreamingDynamic>>* PendingStreamingLevels = WorldPartitionUninitializationPendingStreamingLevels.Find(WorldPartition))
			{
				if (PendingStreamingLevels->Remove(WorldPartitionStreamingLevel))
				{
					if (PendingStreamingLevels->IsEmpty())
					{
						WorldPartitionUninitializationPendingStreamingLevels.Remove(WorldPartition);
					}
				}
			}
		}
	}
}

void UWorldPartitionSubsystem::OnLevelBeginMakingVisible(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel)
{
	if ((InWorld != GetWorld()) || !GetWorld()->IsGameWorld())
	{
		return;
	}

	if (UWorldPartition* WorldPartition = InLoadedLevel->GetWorldPartition())
	{
		check(WorldPartition != InWorld->GetWorldPartition());
		WorldPartition->Initialize(InWorld, InStreamingLevel->LevelTransform);
	}
}

void UWorldPartitionSubsystem::OnLevelBeginMakingInvisible(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLoadedLevel)
{
	if ((InWorld != GetWorld()) || !GetWorld()->IsGameWorld())
	{
		return;
	}

	if (UWorldPartition* WorldPartition = InLoadedLevel->GetWorldPartition())
	{
		check(WorldPartition != InWorld->GetWorldPartition());
		check(WorldPartition->GetWorld() == GetWorld());
		check(InLoadedLevel == WorldPartition->GetTypedOuter<UWorld>()->PersistentLevel);
		WorldPartition->Uninitialize();
	}
}

void UWorldPartitionSubsystem::OnLevelStreamingTargetStateChanged(UWorld* InWorld, const ULevelStreaming* InStreamingLevel, ULevel* InLevelIfLoaded, ELevelStreamingState InCurrentState, ELevelStreamingTargetState InPrevTarget, ELevelStreamingTargetState InNewTarget)
{
	if (InWorld != GetWorld())
	{
		return;
	}

	UpdateLoadingAndPendingLoadStreamingLevels(InStreamingLevel);

	// Make sure when a WorldPartition is LevelStreamed that changing its state to remove it from world will update the target states of its Cells right away.
	if(InLevelIfLoaded && InNewTarget != ELevelStreamingTargetState::LoadedVisible)
	{
		UWorldPartition* WorldPartition = InLevelIfLoaded->GetTypedOuter<UWorld>()->GetWorldPartition();
		if (WorldPartition && WorldPartition->IsInitialized() && !WorldPartition->CanStream())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::OnLevelStreamingTargetStateChanged);
			UWorldPartitionSubsystem::UpdateStreamingStateInternal(InWorld, WorldPartition);
		}
	}
}

void UWorldPartitionSubsystem::RegisterStreamingSourceProvider(IWorldPartitionStreamingSourceProvider* StreamingSource)
{
	bool bIsAlreadyInSet = false;
	StreamingSourceProviders.Add(StreamingSource, &bIsAlreadyInSet);
	UE_CLOGF(bIsAlreadyInSet, LogWorldPartition, Warning, "Streaming source provider already registered.");
}

bool UWorldPartitionSubsystem::IsStreamingSourceProviderRegistered(IWorldPartitionStreamingSourceProvider* StreamingSource) const
{
	return StreamingSourceProviders.Contains(StreamingSource);
}

bool UWorldPartitionSubsystem::UnregisterStreamingSourceProvider(IWorldPartitionStreamingSourceProvider* StreamingSource)
{
	return !!StreamingSourceProviders.Remove(StreamingSource);
}

TSet<IWorldPartitionStreamingSourceProvider*> UWorldPartitionSubsystem::GetStreamingSourceProviders() const
{
	TSet<IWorldPartitionStreamingSourceProvider*> Result = StreamingSourceProviders;
	if (!Result.IsEmpty() && IsStreamingSourceProviderFiltered.IsBound())
	{
		for (auto It = Result.CreateIterator(); It; ++It)
		{
			if (IsStreamingSourceProviderFiltered.Execute(*It))
			{
				It.RemoveCurrent();
			}
		}
	}
	return Result;
}

void UWorldPartitionSubsystem::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	for (UWorldPartition* RegisteredWorldPartition : RegisteredWorldPartitions)
	{
		RegisteredWorldPartition->Tick(DeltaSeconds);

		if (GDrawRuntimeHash3D && CanDebugDraw())
		{
			RegisteredWorldPartition->DrawRuntimeHash3D();
		}

#if WITH_EDITOR
		if (!GetWorld()->IsGameWorld())
		{
			RegisteredWorldPartition->DrawRuntimeHashPreview();
		}
#endif
	}
}

ETickableTickType UWorldPartitionSubsystem::GetTickableTickType() const
{
	return IsTemplate() ? ETickableTickType::Never : ETickableTickType::Always;
}

TStatId UWorldPartitionSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UWorldPartitionSubsystem, STATGROUP_Tickables);
}

bool UWorldPartitionSubsystem::IsAllStreamingCompleted()
{
	return const_cast<UWorldPartitionSubsystem*>(this)->IsStreamingCompleted();
}

bool UWorldPartitionSubsystem::IsStreamingCompleted(const IWorldPartitionStreamingSourceProvider* InStreamingSourceProvider) const
{
	// Convert specified/optional streaming source provider to a world partition 
	// streaming source and pass it along to each registered world partition
	TArray<FWorldPartitionStreamingSource> WorldStreamingSources;

	if (InStreamingSourceProvider)
	{
		if (!InStreamingSourceProvider->GetStreamingSources(WorldStreamingSources))
		{
			return true;
		}
	}

	for (UWorldPartition* RegisteredWorldPartition : RegisteredWorldPartitions)
	{
		TArray<FWorldPartitionStreamingSource> LocalStreamingSources;

		if (InStreamingSourceProvider)
		{
			const FTransform WorldToLocal = RegisteredWorldPartition->GetInstanceTransform().Inverse();
			
			LocalStreamingSources = WorldStreamingSources;
			for (FWorldPartitionStreamingSource& StreamingSource : LocalStreamingSources)
			{
				StreamingSource.Location = WorldToLocal.TransformPosition(StreamingSource.Location);
				StreamingSource.Rotation = WorldToLocal.TransformRotation(StreamingSource.Rotation.Quaternion()).Rotator();
			}
		}

		if (!RegisteredWorldPartition->IsStreamingCompleted(InStreamingSourceProvider ? &LocalStreamingSources : nullptr))
		{
			return false;
		}
	}
	return true;
}

bool UWorldPartitionSubsystem::IsStreamingCompleted(EWorldPartitionRuntimeCellState QueryState, const TArray<FWorldPartitionStreamingQuerySource>& QuerySources, bool bExactState) const
{
	for (UWorldPartition* RegisteredWorldPartition : RegisteredWorldPartitions)
	{
		const FTransform WorldToLocal = RegisteredWorldPartition->GetInstanceTransform().Inverse();

		TArray<FWorldPartitionStreamingQuerySource> LocalQuerySources = QuerySources;
		for (FWorldPartitionStreamingQuerySource& QuerySource : LocalQuerySources)
		{
			QuerySource.Location = WorldToLocal.TransformPosition(QuerySource.Location);
			QuerySource.Rotation = WorldToLocal.TransformRotation(QuerySource.Rotation.Quaternion()).Rotator();
		}

		if (!RegisteredWorldPartition->IsStreamingCompleted(QueryState, LocalQuerySources, bExactState))
		{
			return false;
		}
	}

	return true;
}

void UWorldPartitionSubsystem::DumpWorldPartitions(FOutputDevice& OutputDevice) const
{
	if (RegisteredWorldPartitions.Num() > 0)
	{
		OutputDevice.Logf(TEXT("Registered World Partitions for %s:"), *GetWorld()->GetPathName());
		TArray<FString> WorldPartitions;
		Algo::ForEach(RegisteredWorldPartitions, [&WorldPartitions](const UWorldPartition* WorldPartition) { WorldPartitions.Add(WorldPartition->GetPathName()); });
		WorldPartitions.Sort();
		Algo::ForEach(WorldPartitions, [&OutputDevice](const FString& WorldPartition) { OutputDevice.Logf(TEXT("  - %s"), *WorldPartition); });
	}
}

void UWorldPartitionSubsystem::DumpStreamingSources(FOutputDevice& OutputDevice) const
{
	if (StreamingSources.Num() > 0)
	{
		OutputDevice.Logf(TEXT("Streaming Sources for %s:"), *GetWorld()->GetPathName());
		for (const FWorldPartitionStreamingSource& StreamingSource : StreamingSources)
		{
			OutputDevice.Logf(TEXT("  - %s: %s"), *StreamingSource.Name.ToString(), *StreamingSource.ToString());
		}
	}
}

static int32 GUpdateStreamingSources = 1;
static FAutoConsoleVariableRef CVarUpdateStreamingSources(
	TEXT("wp.Runtime.UpdateStreamingSources"),
	GUpdateStreamingSources,
	TEXT("Set to 0 to stop updating (freeze) world partition streaming sources."));

#if WITH_EDITOR
static const FName NAME_SIEStreamingSource("SIE");
#endif

bool UWorldPartitionSubsystem::IsServer(const UWorld* InWorld)
{
	const ENetMode NetMode = InWorld->GetNetMode();
	const bool bIsServer = (NetMode == NM_DedicatedServer || NetMode == NM_ListenServer);
	return bIsServer;
}

bool UWorldPartitionSubsystem::HasAnyWorldPartitionServerStreamingEnabled() const
{
	return (NumWorldPartitionServerStreamingEnabled > 0);
}

void UWorldPartitionSubsystem::UpdateServerClientsVisibleLevelNames()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::UpdateServerClientsVisibleLevelNames);
	UWorld* World = GetWorld();
	if (IsServer(World))
	{
		ServerClientsVisibleLevelNames.Reset();
		if (const UNetDriver* NetDriver = World->GetNetDriver())
		{
			for (UNetConnection* Connection : NetDriver->ClientConnections)
			{
				ServerClientsVisibleLevelNames.Add(Connection->GetClientWorldPackageName());
				ServerClientsVisibleLevelNames.Append(Connection->ClientVisibleLevelNames);
				ServerClientsVisibleLevelNames.Append(Connection->GetClientMakingVisibleLevelNames());
			}
		}
		FHashBuilder HashBuilder;
		TArray<FName> SortedServerClientsVisibleLevelNames = ServerClientsVisibleLevelNames.Array();
		SortedServerClientsVisibleLevelNames.Sort(FNameFastLess());
		HashBuilder << SortedServerClientsVisibleLevelNames;
		ServerClientsVisibleLevelsHash = HashBuilder.GetHash();
	}
}

void UWorldPartitionSubsystem::UpdateStreamingSources()
{
	if (!GUpdateStreamingSources)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::UpdateStreamingSources);

	StreamingSources.Reset();

	const UWorld* World = GetWorld();
	bool bIsUsingReplayStreamingSources = false;
	if (AWorldPartitionReplay::IsPlaybackEnabled(World))
	{
		if (const UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			bIsUsingReplayStreamingSources = WorldPartition->Replay->GetReplayStreamingSources(StreamingSources);
		}
	}

	if (!bIsUsingReplayStreamingSources)
	{
		const bool bIsServer = IsServer(World);
		const bool bServerStreamingEnabled = HasAnyWorldPartitionServerStreamingEnabled();
		if (!bIsServer || bServerStreamingEnabled || AWorldPartitionReplay::IsRecordingEnabled(World))
		{
			bool bAllowPlayerControllerStreamingSources = true;
#if WITH_EDITOR
			if (UWorldPartition::IsSimulating())
			{
				// We are in the SIE
				const FVector ViewLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
				const FRotator ViewRotation = GCurrentLevelEditingViewportClient->GetViewRotation();
				StreamingSources.Add(FWorldPartitionStreamingSource(NAME_SIEStreamingSource, ViewLocation, ViewRotation, EStreamingSourceTargetState::Activated, /*bBlockOnSlowLoading=*/false, EStreamingSourcePriority::Default, false));
				bAllowPlayerControllerStreamingSources = false;
			}
#endif

			{
				// This will include game-sepcific code called from GetStreamingSourceProviders if IsStreamingSourceProviderFiltered is bound, and also GetStreamingSources calls.
				SCOPE_CYCLE_COUNTER(STAT_WorldPartitionUpdateStreamingStateExternal);
				CSV_SCOPED_TIMING_STAT(WorldPartition, GetStreamingSourceProviders);

				TArray<FWorldPartitionStreamingSource> ProviderStreamingSources;
				for (IWorldPartitionStreamingSourceProvider* StreamingSourceProvider : GetStreamingSourceProviders())
				{
					if (bAllowPlayerControllerStreamingSources || !Cast<APlayerController>(StreamingSourceProvider->GetStreamingSourceOwner()))
					{
						ProviderStreamingSources.Reset();
						if (StreamingSourceProvider->GetStreamingSources(ProviderStreamingSources))
						{
							for (FWorldPartitionStreamingSource& ProviderStreamingSource : ProviderStreamingSources)
							{
								StreamingSources.Add(MoveTemp(ProviderStreamingSource));
							}
						}
					}
				}
			}
		}

		// Make sure the server streaming always loads a bit more than the client.
		// This is necessary to avoid making the client wait indefinitly for the server to finish loading cells
		// that are not even requested by the server because of a slight difference between client and server
		// streaming source locations / rotation.
		// Network quantization and world partition location/rotation quantization can contribute to this difference.
		if (bIsServer && bServerStreamingEnabled)
		{
			// Double location quantization for safety
			const int32 LocationQuantization = UWorldPartitionStreamingPolicy::IsUpdateStreamingOptimEnabled() && (FWorldPartitionStreamingSource::GetLocationQuantization() > 0.f) ? FWorldPartitionStreamingSource::GetLocationQuantization() : 0.f;
			const float ExtraRadius = FMath::Max<int32>(FMath::Max<int32>(GServerStreamingSourceMinimumExtraRadius, LocationQuantization * 2), 0);
			// Double rotation quantization for safety
			const int32 RotationQuantization = UWorldPartitionStreamingPolicy::IsUpdateStreamingOptimEnabled() && (FWorldPartitionStreamingSource::GetRotationQuantization() > 0.f) ? FWorldPartitionStreamingSource::GetRotationQuantization() : 0.f;
			const float ExtraAngle = FMath::Max<int32>(FMath::Max<int32>(GServerStreamingSourceMinimumExtraAngle, RotationQuantization * 2), 0);
			for (FWorldPartitionStreamingSource& StreamingSource : StreamingSources)
			{
				FSetStreamingSourceExtraRadius SetExtraRadius(StreamingSource, ExtraRadius);
				FSetStreamingSourceExtraAngle SetExtraAngle(StreamingSource, ExtraAngle);
			}
		}
	}

	for (auto& Pair : StreamingSourcesVelocity)
	{
		Pair.Value.Invalidate();
	}

	StreamingSourcesHash = 0;

	const double CurrentTime = World->GetTimeSeconds();
	for (FWorldPartitionStreamingSource& StreamingSource : StreamingSources)
	{
		// Update streaming sources velocity
		if (!StreamingSource.Name.IsNone())
		{
			FStreamingSourceVelocity& SourceVelocity = StreamingSourcesVelocity.FindOrAdd(StreamingSource.Name, FStreamingSourceVelocity(StreamingSource.Name));
			StreamingSource.Velocity = SourceVelocity.GetAverageVelocity(StreamingSource.Location, CurrentTime);
		}

		// Update streaming source hash
		StreamingSource.UpdateHash();
		// Build hash for all streaming sources
		StreamingSourcesHash = HashCombine(StreamingSourcesHash, StreamingSource.GetHash());

		UE_TRACE_WP_STREAMING_SOURCE_STATE(World, StreamingSource);
	}

	UE_TRACE_WP_STREAMING_SOURCE_END_UPDATES(World);

	// Cleanup StreamingSourcesVelocity
	for (auto It(StreamingSourcesVelocity.CreateIterator()); It; ++It)
	{
		if (!It.Value().IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

void UWorldPartitionSubsystem::GetStreamingSources(const UWorldPartition* InWorldPartition, TArray<FWorldPartitionStreamingSource>& OutStreamingSources) const
{
	const bool bIsServer = InWorldPartition->IsServer();
	const bool bIsServerStreamingEnabled = InWorldPartition->IsServerStreamingEnabled();
	const bool bIncludeStreamingSources = (!bIsServer || bIsServerStreamingEnabled || AWorldPartitionReplay::IsRecordingEnabled(GetWorld()));

	if (bIncludeStreamingSources)
	{
		OutStreamingSources.Append(StreamingSources);
	}
#if WITH_EDITOR
	else if (UWorldPartition::IsSimulating())
	{
		if (const FWorldPartitionStreamingSource* SIEStreamingSource = (StreamingSources.Num() > 0) && (StreamingSources[0].Name == NAME_SIEStreamingSource) ? &StreamingSources[0] : nullptr)
		{
			OutStreamingSources.Add(*SIEStreamingSource);
		}
	}
#endif

	// Transform to Local
	if (OutStreamingSources.Num() && InWorldPartition->HasInstanceTransform())
	{
		const FTransform WorldToLocal = InWorldPartition->GetInstanceTransform().Inverse();
		for (FWorldPartitionStreamingSource& StreamingSource : OutStreamingSources)
		{
			StreamingSource.Location = WorldToLocal.TransformPosition(StreamingSource.Location);
			StreamingSource.Rotation = WorldToLocal.TransformRotation(StreamingSource.Rotation.Quaternion()).Rotator();
			StreamingSource.Velocity = WorldToLocal.TransformVector(StreamingSource.Velocity);
		}
	}
}

void UWorldPartitionSubsystem::UpdateStreamingSourcesVelocity(const UWorldPartition* InWorldPartition, TArray<FWorldPartitionStreamingSource>& InOutStreamingSources) const
{
	if (InOutStreamingSources.Num())
	{
		const FTransform WorldToLocal = InWorldPartition->GetInstanceTransform().Inverse();
		for (FWorldPartitionStreamingSource& StreamingSourceToUpdate : InOutStreamingSources)
		{
			// Here we can't assume that StreamingSources matches InOutStreamingSources, so we find the streaming sources based on its name
			if (const FWorldPartitionStreamingSource* StreamingSource = StreamingSources.FindByPredicate([Name = StreamingSourceToUpdate.Name](const FWorldPartitionStreamingSource& Source) { return Source.Name == Name; }))
			{
				StreamingSourceToUpdate.Velocity = WorldToLocal.TransformVector(StreamingSource->Velocity);
			}
		}
	}
}

void UWorldPartitionSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);

	bHasBegunPlay = true;
	for (UWorldPartition* RegisteredWorldPartition : RegisteredWorldPartitions)
	{
		RegisteredWorldPartition->OnBeginPlay();
	}
}

void UWorldPartitionSubsystem::OnUpdateStreamingState()
{
	SCOPE_CYCLE_COUNTER(STAT_WorldPartitionUpdateStreamingState);

	UWorldPartitionSubsystem::UpdateStreamingStateInternal(GetWorld());
}

void UWorldPartitionSubsystem::OnWorldComponentsUpdated(UWorld& World)
{
	Super::OnWorldComponentsUpdated(World);

#if WITH_EDITOR
	// While Cooking Cells will get initialized for Save and UpdateWorldComponents will get called rerunning ConstructionScripts, we then want to apply Post Construction Script Overrides on Actors
	if (IsRunningCookCommandlet())
	{
		for (AActor* Actor : World.PersistentLevel->Actors)
		{
			if (IsValid(Actor))
			{
				FWorldPartitionLevelHelper::ApplyConstructionScriptPropertyOverridesFromAnnotation(Actor);
			}
		}
	}
#endif
}

bool UWorldPartitionSubsystem::IncrementalUpdateStreamingState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::IncrementalUpdateStreamingState);
	check(GUpdateStreamingStateTimeLimit > 0.f);

	// Make a snapshot of registered World Partitions to update
	if (IncrementalUpdateWorldPartitions.IsEmpty())
	{
		IncrementalUpdateWorldPartitions.Append(RegisteredWorldPartitions);
		IncrementalUpdateWorldPartitionsPendingAdd.Reset();
	}
	// Append all World Partitions added since last incremental update
	else if (!IncrementalUpdateWorldPartitionsPendingAdd.IsEmpty())
	{
		IncrementalUpdateWorldPartitions.Append(IncrementalUpdateWorldPartitionsPendingAdd);
		IncrementalUpdateWorldPartitionsPendingAdd.Reset();
	}

	const double StartTime = FPlatformTime::Seconds();
	for (auto It = IncrementalUpdateWorldPartitions.CreateIterator(); It; ++It)
	{
		UWorldPartition* RegisteredWorldPartition = *It;
		if (RegisteredWorldPartition->StreamingPolicy)
		{
			RegisteredWorldPartition->StreamingPolicy->UpdateStreamingState();
		}
		It.RemoveCurrent();

		const double DeltaTime = (FPlatformTime::Seconds() - StartTime) * 1000;
		if (DeltaTime > GUpdateStreamingStateTimeLimit)
		{
			break;
		}
	}
	return IncrementalUpdateWorldPartitions.IsEmpty();
}

static bool IsHighPriorityLoading(const UWorld* InWorld)
{
	const AWorldSettings* WorldSettings = InWorld ? InWorld->GetWorldSettings(false, false) : nullptr;
	const bool bHighPriorityLoading = ensure(WorldSettings) ? (WorldSettings->bHighPriorityLoadingLocal || WorldSettings->bHighPriorityLoading) : false;
	return bHighPriorityLoading;
}

void UWorldPartitionSubsystem::UpdateStreamingStateInternal(const UWorld* InWorld, UWorldPartition* InWorldPartition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::UpdateStreamingState);

	const UWorld* World = InWorld;
	if (!World->IsGameWorld())
	{
		return;
	}

	// Subsystem can be null during EndPlayMap. WorldPartition::Uninitialize will still call UpdateStreamingStateInternal to cleanup it's streaming levels
	UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(World);
	check(WorldPartitionSubsystem || InWorldPartition);
	if (!InWorldPartition && (WorldPartitionSubsystem == nullptr || WorldPartitionSubsystem->RegisteredWorldPartitions.IsEmpty()))
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		if (WorldPartitionSubsystem)
		{
			WorldPartitionSubsystem->OnStreamingStateUpdated().Broadcast();
		}
	};

	TArray<TObjectPtr<UWorldPartition>> RegisteredWorldPartitionsCopy;
	auto GetRegisteredWorldPartitionsCopy = [&RegisteredWorldPartitionsCopy, InWorldPartition, WorldPartitionSubsystem]()
	{
		if (RegisteredWorldPartitionsCopy.IsEmpty())
		{
			// Make temp copy of array as UpdateStreamingState may FlushAsyncLoading, which may add a new world partition to RegisteredWorldPartitions while iterating
			RegisteredWorldPartitionsCopy = InWorldPartition ? TArray<TObjectPtr<UWorldPartition>>({ InWorldPartition }) : (WorldPartitionSubsystem) ? WorldPartitionSubsystem->RegisteredWorldPartitions : TArray<TObjectPtr<UWorldPartition>>{};
		}
		return RegisteredWorldPartitionsCopy;
	};

	if (WorldPartitionSubsystem)
	{
		// Update streaming sources
		WorldPartitionSubsystem->UpdateStreamingSources();
		// Update server's clients visible levels
		WorldPartitionSubsystem->UpdateServerClientsVisibleLevelNames();
	}

	const bool bIsServer = IsServer(InWorld);
	const bool bServerStreamingEnabled = bIsServer && WorldPartitionSubsystem && WorldPartitionSubsystem->HasAnyWorldPartitionServerStreamingEnabled();
	const int32 WorldPartitionUpdateCount = InWorldPartition ? 1 : (WorldPartitionSubsystem) ? WorldPartitionSubsystem->RegisteredWorldPartitions.Num() : 0;

	const bool bIsInBlockTillLevelStreamingCompleted = InWorld->GetIsInBlockTillLevelStreamingCompleted();
	const bool bForceDisableIncrementalUpdate = IsHighPriorityLoading(InWorld) || !InWorld->bMatchStarted || InWorld->IsInSeamlessTravel() || bIsInBlockTillLevelStreamingCompleted;
	const bool bIncrementalUpdate = (GUpdateStreamingStateTimeLimit > 0.f) &&
									(WorldPartitionUpdateCount > 1) &&
									!bForceDisableIncrementalUpdate &&
									(!bIsServer || bServerStreamingEnabled); // No increment on server except if server streaming is enabled

#if WITH_EDITOR
	if (WorldPartitionSubsystem)
	{
		WorldPartitionSubsystem->UpdateDynamicLoadingRangeScale();
	}
#endif // WITH_EDITOR

	// Update streaming state of all registered world partitions
	if (bIncrementalUpdate)
	{
		// Early out if incremental update doesn't complete
		if (WorldPartitionSubsystem && !WorldPartitionSubsystem->IncrementalUpdateStreamingState())
		{
			return;
		}
	}
	else
	{
		for (UWorldPartition* RegisteredWorldPartition : GetRegisteredWorldPartitionsCopy())
		{
			if (RegisteredWorldPartition->StreamingPolicy)
			{
				RegisteredWorldPartition->StreamingPolicy->UpdateStreamingState();
			}
		}
	}

	// Cumulate cells to load and to activate
	TArray<const UWorldPartitionRuntimeCell*> ToLoadCells;
	TArray<const UWorldPartitionRuntimeCell*> ToActivateCells;
	for (UWorldPartition* RegisteredWorldPartition : GetRegisteredWorldPartitionsCopy())
	{
		if (RegisteredWorldPartition->StreamingPolicy)
		{
			RegisteredWorldPartition->StreamingPolicy->GetCellsToUpdate(ToLoadCells, ToActivateCells);
		}
	}

	// Sort all cells to activate and load
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SortStreamingCellsByImportance);
		Algo::Sort(ToActivateCells, [](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB) { return CellA->SortCompare(CellB) < 0; });
		Algo::Sort(ToLoadCells, [](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB) { return CellA->SortCompare(CellB) < 0; });
	}

	// Compute maximum number of cells to load
	int32 MaxCellsToLoad = MAX_int32;
	
	if (WorldPartitionSubsystem)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetMaxCellsToLoad);

		if (!bIsServer && !bIsInBlockTillLevelStreamingCompleted && (GMaxLoadingStreamingCells > 0))
		{
			MaxCellsToLoad = GMaxLoadingStreamingCells;

			for (auto It = WorldPartitionSubsystem->WorldPartitionLoadingAndPendingLoadStreamingLevels.CreateIterator(); It; ++It)
			{
				const ULevelStreaming* InStreamingLevel = It->Get();
				if (IsLoadingOrPendingLoadStreamingLevel(InStreamingLevel))
				{
					if (!--MaxCellsToLoad)
					{
						break;
					}
				}
				else
				{
					It.RemoveCurrent();
				}
			}
		}
	}

	// Process cells to activate
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToActivateCells);
		// Trigger cell activation. Depending on actual state of cell limit loading.
		for (const UWorldPartitionRuntimeCell* Cell : ToActivateCells)
		{
			Cell->GetOuterWorld()->GetWorldPartition()->StreamingPolicy->SetCellStateToActivated(Cell, MaxCellsToLoad);
		}
	}
	// Process cells to load
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToLoadCells);
		for (const UWorldPartitionRuntimeCell* Cell : ToLoadCells)
		{
			Cell->GetOuterWorld()->GetWorldPartition()->StreamingPolicy->SetCellStateToLoaded(Cell, MaxCellsToLoad);
		}
	}

	// Build and reprioritize cells that are still in a pending state (waiting to be loaded or activated)
	TArray<const UWorldPartitionRuntimeCell*> PendingToLoadCells;
	TArray<const UWorldPartitionRuntimeCell*> PendingToActivateCells;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildPendingCells);
		PendingToLoadCells.Reserve(ToLoadCells.Num());
		PendingToActivateCells.Reserve(ToActivateCells.Num());
		for (UWorldPartition* RegisteredWorldPartition : GetRegisteredWorldPartitionsCopy())
		{
			if (RegisteredWorldPartition->StreamingPolicy)
			{
				RegisteredWorldPartition->StreamingPolicy->GetCellsToReprioritize(PendingToLoadCells, PendingToActivateCells);
			}
		}
	}
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SortPendingCells);
			Algo::Sort(PendingToLoadCells, [](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB) { return CellA->SortCompare(CellB) < 0; });
			Algo::Sort(PendingToActivateCells, [](const UWorldPartitionRuntimeCell* CellA, const UWorldPartitionRuntimeCell* CellB) { return CellA->SortCompare(CellB) < 0; });
		}

		auto SetCellsStreamingPriority = [World](const TArray<const UWorldPartitionRuntimeCell*>& InCells, int32 InPriorityBias = 0)
		{
			const int32 MaxPrio = InCells.Num() + InPriorityBias;
			int32 SortedPriority = MaxPrio;
			if (World->HasAnyLevelMakingVisible())
			{
				// Skip cells referencing current pending visible levels as their priority is already handled by UWorld::UpdateStreamingLevel
				for (const UWorldPartitionRuntimeCell* Cell : InCells)
				{
					if (const ULevel* Level = Cell->GetLevel(); Level && (Level->GetPendingVisibilityState() != ELevelPendingVisibilityState::MakingVisible))
					{
						Cell->SetStreamingPriority(SortedPriority--);
					}
				}
			}
			else
			{
				for (const UWorldPartitionRuntimeCell* Cell : InCells)
				{
					Cell->SetStreamingPriority(SortedPriority--);
				}
			}
		};

		// Update level streaming priority so that UWorld::UpdateLevelStreaming will naturally process the levels in the correct order
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SetPendingCellsStreamingPriority);
			SetCellsStreamingPriority(PendingToLoadCells);
			SetCellsStreamingPriority(PendingToActivateCells, PendingToLoadCells.Num());
		}
	}
}

bool UWorldPartitionSubsystem::CanDebugDraw() const
{
#if WITH_EDITOR
	const ENetMode NetMode = GetWorld()->GetNetMode();
	const bool bIsDedicatedServer = (NetMode == NM_DedicatedServer);
	const bool bIsClient = (NetMode == NM_Client);
	if ((bIsDedicatedServer && !UWorldPartition::DebugDedicatedServerStreaming) ||
		(bIsClient && UWorldPartition::DebugDedicatedServerStreaming))
	{
		return false;
	}

	if (!bIsDedicatedServer && UWorldPartition::IsSimulating(false) && (!GEditor || (GetWorld() != GEditor->PlayWorld)))
	{
		return false;
	}
#endif
	return GetWorld()->IsGameWorld();
}

void UWorldPartitionSubsystem::Draw(UCanvas* Canvas, class APlayerController* PC)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::Draw);
	if (!Canvas || !Canvas->SceneView)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	if ((GDrawWorldPartitionIndex >= 0) && !RegisteredWorldPartitions.IsValidIndex(GDrawWorldPartitionIndex))
	{
		return;
	}

#if !WITH_STATE_STREAM // TODO: This needs to be fixed

	// Filter out views that don't match our world
	if (!GetWorld()->IsNetMode(NM_DedicatedServer) && !UWorldPartition::IsSimulating(false) &&
		(Canvas->SceneView->ViewActor == nullptr || Canvas->SceneView->ViewActor->GetWorld() != GetWorld()))
	{
		return;
	}

#endif

	// When GDrawWorldPartitionIndex < 0, draw all world partitions
	const int32 WPDrawCount = (GDrawWorldPartitionIndex < 0) ? RegisteredWorldPartitions.Num() : 1;
	UWorldPartition* SingleWorldPartition = (WPDrawCount == 1) ? RegisteredWorldPartitions[FMath::Max(GDrawWorldPartitionIndex, 0)] : nullptr;

	const FVector2D CanvasTopLeftPadding(10.f, 10.f);
	FVector2D CurrentOffset(CanvasTopLeftPadding);

	if (GDrawRuntimeHash2D)
	{
		const float MaxScreenRatio = 0.75f;
		const FVector2D CanvasBottomRightPadding(10.f, 10.f);
		const FVector2D CanvasMinimumSize(100.f, 100.f);
		const FVector2D CanvasMaxScreenSize = FVector2D::Max(MaxScreenRatio*FVector2D(Canvas->ClipX, Canvas->ClipY) - CanvasBottomRightPadding - CurrentOffset, CanvasMinimumSize);
		const float DrawRuntimeHash2DScaleFactor = FMath::Clamp<float>(GDrawRuntimeHash2DScaleFactor, 0.1, 1);
		const FVector2D PartitionCanvasSize = FVector2D(CanvasMaxScreenSize.X, CanvasMaxScreenSize.Y) * DrawRuntimeHash2DScaleFactor;

		FBox2D WorldRegion(ForceInit);
		WorldPartitionsDraw2DContext.SetNum(RegisteredWorldPartitions.Num(), EAllowShrinking::No);
		Algo::ForEach(WorldPartitionsDraw2DContext, [&](FWorldPartitionDraw2DContext& Context) { if (const FBox2D& Bounds = Context.GetDesiredWorldBounds(); Bounds.bIsValid) { WorldRegion += Bounds; } });
		if (!WorldRegion.bIsValid)
		{
			// Fallback on a default region centered around centroid of all streaming sources
			const float DefaultWorldRegionExtent = 25600.f;
			FVector2D Extent = FVector2D(DefaultWorldRegionExtent);
			FVector2D Pos = FVector2D::ZeroVector;
			for (const FWorldPartitionStreamingSource& Source : StreamingSources)
			{
				Pos += FVector2D(Source.Location.X, Source.Location.Y);
			}
			Pos /= StreamingSources.Num();
			WorldRegion = FBox2D(Pos - Extent, Pos + Extent);
		}

		const FBox2D CanvasBox = FBox2D(CurrentOffset, CurrentOffset + PartitionCanvasSize);
		Algo::ForEach(WorldPartitionsDraw2DContext, [&](FWorldPartitionDraw2DContext& Context) { Context.Initialize(CanvasBox, WorldRegion); });

		std::atomic<int32> LineCount;
		std::atomic<int32> BoxCount;
		bool bSucceeded = false;
		
		ParallelFor(WPDrawCount, [&](int32 ProcessIndex)
		{
			UWorldPartition* CurrentWorldPartition = SingleWorldPartition ? SingleWorldPartition : RegisteredWorldPartitions[ProcessIndex].Get();
			FWorldPartitionDraw2DContext& Context = WorldPartitionsDraw2DContext[ProcessIndex];
			Context.SetIsDetailedMode(WPDrawCount == 1);
			Context.SetDrawGridBounds(ProcessIndex == (WPDrawCount-1));
			Context.SetDrawGridAxis(ProcessIndex == (WPDrawCount-1));
			if (CurrentWorldPartition->DrawRuntimeHash2D(Context))
			{
				LineCount += Context.GetLineCount();
				BoxCount += Context.GetBoxCount();
			}
		});

		if (WPDrawCount > 1)
		{
			FWorldPartitionCanvasMultiLineText Text;
			Text.Emplace(FString::Printf(TEXT("Showing %d world partitions"), WPDrawCount), FLinearColor::White);
			FWorldPartitionCanvasMultiLineTextItem Item(CanvasBox.Min, Text);
			WorldPartitionsDraw2DContext[WPDrawCount - 1].PushDrawText(Item);
		}

		int32 TotalLineCount = LineCount.load();
		int32 TotalBoxCount = BoxCount.load();
		if (TotalLineCount + TotalBoxCount > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::DrawAllCanvasItems);
			FWorldPartitionDraw2DCanvas WorldPartitionCanvas(Canvas);
			WorldPartitionCanvas.PrepareDraw(TotalLineCount, TotalBoxCount);

			FBox2D MaxUsedBounds(ForceInit);
			for (FWorldPartitionDraw2DContext& Context : WorldPartitionsDraw2DContext)
			{
				WorldPartitionCanvas.Draw(Context.GetCanvasItems());
				
				if (Context.GetUsedCanvasBounds().bIsValid)
				{
					MaxUsedBounds += Context.GetUsedCanvasBounds();
				}
			}
			
			CurrentOffset.X = CanvasBottomRightPadding.X;
			CurrentOffset.Y += MaxUsedBounds.bIsValid ? MaxUsedBounds.Max.Y : CanvasBox.GetSize().Y;
		}
	}
	
	if (GDrawStreamingPerfs || GDrawRuntimeHash2D)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::DrawStreamingStatus);
		{
			FString StatusText;
			if (IsIncrementalPurgePending()) { StatusText += TEXT("(Purging) "); }
			if (IsIncrementalUnhashPending()) { StatusText += TEXT("(Unhashing) "); }
			if (IsAsyncLoading()) { StatusText += TEXT("(AsyncLoading) "); }
			if (StatusText.IsEmpty()) { StatusText = TEXT("(Idle) "); }
			if (IsHighPriorityLoading(GetWorld())) { StatusText += TEXT("(HighPriorityLoading) "); }

			FString DebugWorldText = FString::Printf(TEXT("(%s)"), *GetDebugStringForWorld(GetWorld()));
			if (SingleWorldPartition && SingleWorldPartition->IsServer())
			{
				DebugWorldText += FString::Printf(TEXT(" (Server Streaming %s)"), SingleWorldPartition->IsServerStreamingEnabled() ? TEXT("Enabled") : TEXT("Disabled"));
			}
			
			const FString Text = FString::Printf(TEXT("Streaming Status for %s: %s"), *DebugWorldText, *StatusText);
			FWorldPartitionDebugHelper::DrawText(Canvas, Text, GEngine->GetSmallFont(), FColor::White, CurrentOffset);
		}

		{
			auto GetStreamingPerformance = [&]()
			{
				EWorldPartitionStreamingPerformance StreamingPerformance = SingleWorldPartition ? SingleWorldPartition->GetStreamingPerformance() : EWorldPartitionStreamingPerformance::Good;
				if (!SingleWorldPartition)
				{
					for (UWorldPartition* RegisteredWorldPartition : RegisteredWorldPartitions)
					{
						StreamingPerformance = FMath::Max(StreamingPerformance, RegisteredWorldPartition->GetStreamingPerformance());
						if (StreamingPerformance == EWorldPartitionStreamingPerformance::Critical)
						{
							break;
						}
					}
				}
				return StreamingPerformance;
			};

			FString StatusText;
			EWorldPartitionStreamingPerformance StreamingPerformance = GetStreamingPerformance();
			switch (StreamingPerformance)
			{
			case EWorldPartitionStreamingPerformance::Good:
				StatusText = TEXT("Good");
				break;
			case EWorldPartitionStreamingPerformance::Slow:
				StatusText = TEXT("Slow");
				break;
			case EWorldPartitionStreamingPerformance::Critical:
				StatusText = TEXT("Critical");
				break;
			default:
				StatusText = TEXT("Unknown");
				break;
			}
			const FString Text = FString::Printf(TEXT("Streaming Performance: %s (Blocking %s)"), *StatusText, GBlockOnSlowStreaming ? TEXT("Enabled") : TEXT("Disabled"));
			FWorldPartitionDebugHelper::DrawText(Canvas, Text, GEngine->GetSmallFont(), FColor::White, CurrentOffset);
		}
	}

	if (GDrawStreamingSources || GDrawRuntimeHash2D)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::DrawStreamingSources);

		const TArray<FWorldPartitionStreamingSource>& LocalStreamingSources = SingleWorldPartition ? SingleWorldPartition->GetStreamingSources() : StreamingSources;
		if (LocalStreamingSources.Num() > 0)
		{
			FString Title("Streaming Sources");
			FWorldPartitionDebugHelper::DrawText(Canvas, Title, GEngine->GetSmallFont(), FColor::Yellow, CurrentOffset);

			FVector2D Pos = CurrentOffset;
			float MaxTextWidth = 0;
			for (const FWorldPartitionStreamingSource& StreamingSource : LocalStreamingSources)
			{
				FString StreamingSourceDisplay = StreamingSource.Name.ToString();
				if (StreamingSource.bReplay)
				{
					StreamingSourceDisplay += TEXT(" (Replay)");
				}
				FWorldPartitionDebugHelper::DrawText(Canvas, StreamingSourceDisplay, GEngine->GetSmallFont(), StreamingSource.GetDebugColor(), Pos, &MaxTextWidth);
			}
			Pos = CurrentOffset + FVector2D(MaxTextWidth + 10, 0.f);
			for (const FWorldPartitionStreamingSource& StreamingSource : LocalStreamingSources)
			{
				FWorldPartitionDebugHelper::DrawText(Canvas, *StreamingSource.ToString(), GEngine->GetSmallFont(), FColor::White, Pos);
			}
			CurrentOffset.Y = Pos.Y;
		}
	}

	if (GDrawLegends || GDrawRuntimeHash2D)
	{
		// Streaming Status Legend
		DrawStreamingStatusLegend(Canvas, CurrentOffset, SingleWorldPartition);
	}

	if (SingleWorldPartition)
	{
		if (GDrawDataLayers || GDrawDataLayersLoadTime || GDrawRuntimeHash2D)
		{
			if (UDataLayerManager* DataLayerManager = SingleWorldPartition->GetDataLayerManager())
			{
				DataLayerManager->DrawDataLayersStatus(Canvas, CurrentOffset);
			}
		}

		UContentBundleManager* ContentBundleManager = GetWorld()->ContentBundleManager;
		if (ContentBundleManager && FWorldPartitionDebugHelper::CanDrawContentBundles() && GDrawRuntimeHash2D)
		{
			ContentBundleManager->DrawContentBundlesStatus(GetWorld(), Canvas, CurrentOffset);
		}

		if (GDrawRuntimeCellsDetails)
		{
			SingleWorldPartition->DrawRuntimeCellsDetails(Canvas, CurrentOffset);
		}
	}
}

void UWorldPartitionSubsystem::DrawStreamingStatusLegend(class UCanvas* Canvas, FVector2D& Offset, const UWorldPartition* InWorldPartition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionSubsystem::DrawStreamingStatusLegend);

	check(Canvas);

	TArray<int32> StreamingStatus;
	StreamingStatus.SetNumZeroed((int32)LEVEL_StreamingStatusCount);
	
	const UWorld* StreamingWorld = InWorldPartition ? InWorldPartition->GetTypedOuter<UWorld>() : nullptr;
	// Cumulate counter stats
	for (ULevelStreaming* LevelStreaming : GetWorld()->GetStreamingLevels())
	{
		if (!StreamingWorld || (StreamingWorld == LevelStreaming->GetStreamingWorld()))
		{
			if (const UWorldPartitionRuntimeCell* Cell = Cast<const UWorldPartitionRuntimeCell>(LevelStreaming->GetWorldPartitionCell()))
			{
				StreamingStatus[(int32)Cell->GetStreamingStatus()]++;
			}
		}
	}

	// @todo_ow: This is not exactly the good value, as we could have pending unload level from Level Instances, etc.
	//           We could modify GetNumLevelsPendingPurge to return the number of pending purge levels from the grid, 
	//           but that will do for now.
	StreamingStatus[LEVEL_UnloadedButStillAround] = FLevelStreamingGCHelper::GetNumLevelsPendingPurge();

	// Draw legend
	FVector2D Pos = Offset;
	float MaxTextWidth = 0.f;
	FWorldPartitionDebugHelper::DrawText(Canvas, TEXT("Streaming Status Legend"), GEngine->GetSmallFont(), FColor::Yellow, Pos, &MaxTextWidth);

	for (int32 i = 0; i < (int32)LEVEL_StreamingStatusCount; ++i)
	{
		EStreamingStatus Status = (EStreamingStatus)i;
		const FColor& StatusColor = ULevelStreaming::GetLevelStreamingStatusColor(Status);
		FString DebugString = *FString::Printf(TEXT("%d) %s"), i, ULevelStreaming::GetLevelStreamingStatusDisplayName(Status));
		if (Status != LEVEL_Unloaded)
		{
			DebugString += *FString::Printf(TEXT(" (%d)"), StreamingStatus[(int32)Status]);
		}
		FWorldPartitionDebugHelper::DrawLegendItem(Canvas, *DebugString, GEngine->GetSmallFont(), StatusColor, FColor::White, Pos, &MaxTextWidth);
	}

	Offset.X += MaxTextWidth + 10;
}

#if WITH_EDITOR

namespace UE
{
	namespace Private
	{
		void AssignLinearScale(FDynamicLoadingRangeScales& Scales, float Linear)
		{
			Scales.LinearScale = Linear;
			Scales.Scale2D = FMath::Sqrt(Scales.LinearScale);
			Scales.Scale3D = FMath::Pow(Scales.LinearScale, 1.f / 3.f);
		}
	}
}

void UWorldPartitionSubsystem::InitDynamicLoadingRangeScale()
{
	if (GetWorld()->GetWorldSettings()->GetDynamicLoadingRangeScalingEnable())
	{
		FDynamicLoadingRangeScaling& ScaleSettings = GetWorld()->GetWorldSettings()->DynamicLoadingRangeScaling;
		UE::Private::AssignLinearScale(LoadingRangeScales, FMath::Clamp(ScaleSettings.InitialDynamicLoadingRangeScaling, 0.f, 100.f) * 0.01f);
		DynamicLoadingRangeScaleStartTime = GetWorld()->GetTimeSeconds();
	}
	else
	{
		UE::Private::AssignLinearScale(LoadingRangeScales, 1.f);
	}
}

void UWorldPartitionSubsystem::UpdateDynamicLoadingRangeScale()
{
	if (LoadingRangeScales.LinearScale < 1.f) // Can skip the update if the scale is at 1.f already
	{
		if (GetWorld()->GetWorldSettings()->GetDynamicLoadingRangeScalingEnable())
		{
			FDynamicLoadingRangeScaling& ScaleSettings = GetWorld()->GetWorldSettings()->DynamicLoadingRangeScaling;
			// ScaleSettings.InitialDynamicLoadingRangeScaling is a percentage value
			float InitialScale = FMath::Clamp(ScaleSettings.InitialDynamicLoadingRangeScaling * 0.01f, 0.f, 1.f);
			if (GetWorld()->GetIsInBlockTillLevelStreamingCompleted() && (LoadingRangeScales.LinearScale == InitialScale))
			{
				// Do not interpolate as long as we are still loading the levels, and use the initial loading range
				DynamicLoadingRangeScaleStartTime = GetWorld()->GetTimeSeconds(); // StartTime is the time when initial level loading is complete, so update the time
			}
			else
			{
				// Start interpolating
				float InterpolationTime = GetWorld()->GetTimeSeconds() - DynamicLoadingRangeScaleStartTime;
				float Delay = FMath::Max(ScaleSettings.DynamicLoadingRangeScalingDelay, 0.0f);
				float Duration = FMath::Max(ScaleSettings.DynamicLoadingRangeScalingDuration, 0.0f);

				float EffectiveTime = InterpolationTime - Delay;
				float Progress = (EffectiveTime < 0.f) ? 0.f : (Duration == 0.f) ? 1.f : FMath::Min(EffectiveTime / Duration, 1.f);
				UE::Private::AssignLinearScale(LoadingRangeScales, Progress * (1.f - InitialScale) + InitialScale);
			}
		}
		else
		{
			// Reset scale to 1.f if user disables the feature during runtime
			UE::Private::AssignLinearScale(LoadingRangeScales, 1.f);
		}
	}
}

#endif // WITH_EDITOR

/*
 * FStreamingSourceVelocity Implementation
 */

FStreamingSourceVelocity::FStreamingSourceVelocity(const FName& InSourceName)
	: bIsValid(false)
	, SourceName(InSourceName)
	, LastUpdateTime(-1.0)
	, LastPosition(FVector::Zero())
{}

FVector FStreamingSourceVelocity::GetAverageVelocity(const FVector& NewPosition, double CurrentTime)
{
	bIsValid = true;

	const bool bNewSource = (LastUpdateTime <= 0.0);
	const double DeltaSeconds = bNewSource ? 1.0 : (CurrentTime - LastUpdateTime);

	if (DeltaSeconds <= 0.0)
	{
		UE_LOGF(LogWorldPartition, Verbose, "Detected Invalid Delta Time: %ls", *SourceName.ToString());
		AvgVelocity = FVector::Zero();
	}
	else
	{
		LastUpdateTime = CurrentTime;

		const FVector AbsMovement = (NewPosition - LastPosition);
		const FVector AbsVelocity = AbsMovement / DeltaSeconds;
		LastPosition = NewPosition;

		const double TeleportDistance = 10000;
		const double MaxDeltaSeconds = 5.0;
		const double Distance = AbsMovement.Size();

		if (bNewSource)
		{
			UE_LOGF(LogWorldPartition, Verbose, "New Streaming Source: %ls -> Position: %ls", *SourceName.ToString(), *NewPosition.ToString());
			AvgVelocity = FVector::Zero();
		}
		else if (Distance > TeleportDistance)
		{
			UE_LOGF(LogWorldPartition, Verbose, "Detected Streaming Source Teleport: %ls -> Last Position: %ls -> New Position: %ls", *SourceName.ToString(), *LastPosition.ToString(), *NewPosition.ToString());
			AvgVelocity = FVector::Zero();
		}
		else if  (DeltaSeconds > MaxDeltaSeconds)
		{
			UE_LOGF(LogWorldPartition, Verbose, "Detected Inactive Streaming Source: %ls -> Last Position: %ls -> New Position: %ls", *SourceName.ToString(), *LastPosition.ToString(), *NewPosition.ToString());
			AvgVelocity = FVector::Zero();
		}
		else
		{
			// Compute the new value in a weighted moving average series
			const double AvgWeight = FMath::Clamp(DeltaSeconds * 100, 0, 1);
			AvgVelocity = AvgVelocity * (1.0 - AvgWeight) + AbsVelocity * AvgWeight;
		}
	}

	return AvgVelocity;
}
