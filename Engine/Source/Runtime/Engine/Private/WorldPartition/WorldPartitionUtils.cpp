// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "WorldPartition/WorldPartitionUtils.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/Cook/WorldPartitionCookPackageContext.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "WorldPartition"

FWorldPartitionUtils::FSimulateCookedSession::FSimulateCookedSession(UWorld* InWorld, const FSimulateCookSessionParams& Params)
	: CookContext(nullptr)
{
	UE_CLOGF(!InWorld, LogWorldPartition, Error, "FSimulateCookedSession was given an invalid world.");
	UE_CLOGF(InWorld && !InWorld->IsPartitionedWorld(), LogWorldPartition, Warning, "FSimulateCookedSession was given a non-partitioned world.");
	
	if (InWorld)
	{
		WorldPartition = InWorld->GetWorldPartition();
		if (WorldPartition.IsValid())
		{
			SimulateCook(Params);
		}
	}

	UE_CLOGF(!IsValid(), LogWorldPartition, Warning, "FSimulateCookedSession failed to generate streaming.");
}

bool FWorldPartitionUtils::FSimulateCookedSession::SimulateCook(const FSimulateCookSessionParams& InParams)
{
	check(WorldPartition.IsValid());
	if (!WorldPartition->CanGenerateStreaming() || IsRunningGame() || WorldPartition->GetWorld()->IsGameWorld() || IsRunningCookCommandlet())
	{
		UE_CLOGF(!WorldPartition->CanGenerateStreaming(), LogWorldPartition, Warning, "FSimulateCookedSession cannot generate streaming, world partition is already generating streaming.");
		UE_CLOGF(IsRunningGame() || WorldPartition->GetWorld()->IsGameWorld(), LogWorldPartition, Warning, "FSimulateCookedSession cannot generate streaming while running PIE/game.");
		UE_CLOGF(IsRunningCookCommandlet(), LogWorldPartition, Warning, "FSimulateCookedSession cannot generate streaming while cooking.");
		return false;
	}

	// Simulate cook
	check(!CookContext);
	CookContext = new FWorldPartitionCookPackageContext;
	WorldPartition->BeginCook(*CookContext);

	FWorldPartitionCookPackageContextParams Params;
	Params.FilteredClasses = InParams.FilteredClasses;

	if (!CookContext->GatherPackagesToCook(Params))
	{
		return false;
	}

	// Simulate Content Bundles injection
	TArray<TSharedPtr<FContentBundleEditor>> ContentBundles;
	WorldPartition->GetWorld()->ContentBundleManager->GetEditorContentBundle(ContentBundles);
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : ContentBundles)
	{
		if (ContentBundle->HasCookedContent())
		{
			URuntimeHashExternalStreamingObjectBase* ExternalStreamingObject = ContentBundle->GetStreamingObject();
			WorldPartition->RuntimeHash->InjectExternalStreamingObject(ExternalStreamingObject);
			InjectedStreamingObjects.Add(ExternalStreamingObject);
		}
	}

	// Simulate EDL injection
	UExternalDataLayerManager* EDLManager = WorldPartition->GetExternalDataLayerManager();
	for (const UDataLayerInstance* Instance : WorldPartition->GetWorld()->GetDataLayerManager()->GetDataLayerInstances())
	{	
		if (const UExternalDataLayerInstance* ExternalInstance = Cast<const UExternalDataLayerInstance>(Instance))
		{
			if (URuntimeHashExternalStreamingObjectBase* EDLStreamingObject = EDLManager->GetExternalStreamingObjectForDataLayerAsset(CastChecked<const UExternalDataLayerAsset>(ExternalInstance->GetAsset())))
			{
				WorldPartition->RuntimeHash->InjectExternalStreamingObject(EDLStreamingObject);
				InjectedStreamingObjects.Add(EDLStreamingObject);
			}
		}
	}

	return true;
}

FWorldPartitionUtils::FSimulateCookedSession::~FSimulateCookedSession()
{
	if (CookContext)
	{
		// Clean-up External Streaming Objects injection
		for(URuntimeHashExternalStreamingObjectBase* InjectedStreamingObject : InjectedStreamingObjects)
		{
			WorldPartition->RuntimeHash->RemoveExternalStreamingObject(InjectedStreamingObject);
			// Trash external streaming object
			InjectedStreamingObject->Rename(*MakeUniqueObjectName(GetTransientPackage(), URuntimeHashExternalStreamingObjectBase::StaticClass()).ToString(), GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors | REN_DoNotDirty);
		}		

		WorldPartition->EndCook(*CookContext);
		delete CookContext;
	}
}

bool FWorldPartitionUtils::FSimulateCookedSession::ForEachStreamingCells(TFunctionRef<void(const IWorldPartitionCell*)> Func)
{
	if (!WorldPartition.IsValid() || !WorldPartition->RuntimeHash)
	{
		return false;
	}

	WorldPartition->RuntimeHash->ForEachStreamingCells([Func](const UWorldPartitionRuntimeCell* Cell)
	{
		if (Cell)
		{
			Func(Cell);
		}
		return true;
	});

	return true;
}

bool FWorldPartitionUtils::FSimulateCookedSession::GetIntersectingCells(const TArray<FWorldPartitionStreamingQuerySource>& InSources, TArray<const IWorldPartitionCell*>& OutCells)
{
	if (WorldPartition.IsValid())
	{
		return WorldPartition->GetIntersectingCells(InSources, OutCells);
	}
	return false;
}
	
#undef LOCTEXT_NAMESPACE
	
#endif
