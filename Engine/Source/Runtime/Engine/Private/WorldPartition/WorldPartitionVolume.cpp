// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionVolume)

ADEPRECATED_WorldPartitionVolume::ADEPRECATED_WorldPartitionVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}
