// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionBlueprintLibrary.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionBlueprintLibrary)

#if WITH_EDITOR
#include "Subsystems/UnrealEditorSubsystem.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterActorList.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Editor.h"
#endif

FActorDesc::FActorDesc()
	: NativeClass(nullptr)
	, Bounds(ForceInit)
	, bIsSpatiallyLoaded(false)
	, bActorIsEditorOnly(false)
{}

#if WITH_EDITOR
FActorDesc::FActorDesc(const FWorldPartitionActorDescInstance& InActorDesc)
{
	Guid = InActorDesc.GetGuid();

	NativeClass = InActorDesc.GetActorNativeClass();

	if (!InActorDesc.GetBaseClass().IsNull())
	{
		Class = FSoftObjectPath(InActorDesc.GetBaseClass(), {});
	}
	else
	{
		Class = FSoftObjectPath(InActorDesc.GetActorNativeClass());
	}

	Name = InActorDesc.GetActorName();
	Label = InActorDesc.GetActorLabel();
	Bounds = InActorDesc.GetEditorBounds();
	RuntimeGrid = InActorDesc.GetRuntimeGrid();
	bIsSpatiallyLoaded = InActorDesc.GetIsSpatiallyLoaded();
	bActorIsEditorOnly = InActorDesc.GetActorIsEditorOnly();
	ActorPackage = InActorDesc.GetActorPackage();

	DataLayerAssets.Reserve(InActorDesc.GetDataLayers().Num());
	for (FName DataLayerAssetPath : InActorDesc.GetDataLayers())
	{
		DataLayerAssets.Add(FSoftObjectPath(DataLayerAssetPath.ToString()));
	}
	
	ActorPath = *InActorDesc.GetActorSoftPath().ToString();
}

TMap<UWorldPartition*, TUniquePtr<FLoaderAdapterActorList>> UWorldPartitionBlueprintLibrary::LoaderAdapterActorListMap;
FDelegateHandle UWorldPartitionBlueprintLibrary::OnWorldPartitionUninitializedHandle;

UWorld* UWorldPartitionBlueprintLibrary::GetEditorWorld()
{
	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	return UnrealEditorSubsystem ? UnrealEditorSubsystem->GetEditorWorld() : nullptr;
}

UWorldPartition* UWorldPartitionBlueprintLibrary::GetWorldPartition()
{
	if (UWorld* World = GetEditorWorld())
	{
		return World->GetWorldPartition();
	}
	return nullptr;
}

void UWorldPartitionBlueprintLibrary::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	if (LoaderAdapterActorListMap.Remove(InWorldPartition) && LoaderAdapterActorListMap.IsEmpty())
	{
		InWorldPartition->GetWorld()->OnWorldPartitionUninitialized().Remove(OnWorldPartitionUninitializedHandle);
	}
}

bool UWorldPartitionBlueprintLibrary::GetActorDescs(const UWorldPartition* InWorldPartition, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = true;

	InWorldPartition->ForEachActorDescContainerInstance([&](const UActorDescContainerInstance* InContainerInstance)
	{
		bResult &= GetActorDescs(InContainerInstance, OutActorDescs);
	});
	
	return bResult;
}

bool UWorldPartitionBlueprintLibrary::GetActorDescs(const UActorDescContainerInstance* InContainerInstance, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = true;

	for (UActorDescContainerInstance::TConstIterator<> Iterator(InContainerInstance); Iterator; ++Iterator)
	{
		OutActorDescs.Emplace(**Iterator);

		if (Iterator->IsChildContainerInstance())
		{
			FWorldPartitionActorDesc::FContainerInstance ContainerInstance;
			if (Iterator->GetChildContainerInstance(ContainerInstance))
			{
				bResult &= GetActorDescs(ContainerInstance.ContainerInstance, OutActorDescs);
			}
			else
			{
				bResult = false;
			}
		}
	}

	return bResult;
}

bool UWorldPartitionBlueprintLibrary::HandleIntersectingActorDesc(const FWorldPartitionActorDescInstance* ActorDescInstance, const FBox& InBox, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = true;

	OutActorDescs.Emplace(*ActorDescInstance);

	if (ActorDescInstance->IsChildContainerInstance())
	{
		FWorldPartitionActorDesc::FContainerInstance ContainerInstance;
		if (ActorDescInstance->GetChildContainerInstance(ContainerInstance))
		{
			bResult &= GetIntersectingActorDescs(ContainerInstance.ContainerInstance, InBox, OutActorDescs);
		}
		else
		{
			bResult = false;
		}
	}
	
	return bResult;
}

bool UWorldPartitionBlueprintLibrary::GetIntersectingActorDescs(UWorldPartition* WorldPartition, const FBox& InBox, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = true;

	FWorldPartitionHelpers::ForEachIntersectingActorDescInstance(WorldPartition, InBox, [&bResult, &InBox, &OutActorDescs](const FWorldPartitionActorDescInstance* ActorDesc)
	{
		bResult &= HandleIntersectingActorDesc(ActorDesc, InBox, OutActorDescs);
		return true;
	});

	return bResult;
}

bool UWorldPartitionBlueprintLibrary::GetIntersectingActorDescs(const UActorDescContainerInstance* InContainerInstance, const FBox& InBox, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = true;

	for (UActorDescContainerInstance::TConstIterator<> Iterator(InContainerInstance); Iterator; ++Iterator)
	{
		if (Iterator->GetEditorBounds().Intersect(InBox))
		{
			bResult &= HandleIntersectingActorDesc(*Iterator, InBox, OutActorDescs);
		}
	}

	return bResult;
}
#endif

FBox UWorldPartitionBlueprintLibrary::GetEditorWorldBounds()
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		return WorldPartition->GetEditorWorldBounds();
	}
#endif
	return FBox(ForceInit);
}

FBox UWorldPartitionBlueprintLibrary::GetRuntimeWorldBounds()
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		return WorldPartition->GetRuntimeWorldBounds();
	}
#endif
	return FBox(ForceInit);
}

void UWorldPartitionBlueprintLibrary::LoadActors(const TArray<FGuid>& InActorsToLoad)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		if (LoaderAdapterActorListMap.IsEmpty())
		{
			OnWorldPartitionUninitializedHandle = WorldPartition->GetWorld()->OnWorldPartitionUninitialized().AddStatic(&UWorldPartitionBlueprintLibrary::OnWorldPartitionUninitialized);
		}

		TUniquePtr<FLoaderAdapterActorList>& LoaderAdapterActorList = LoaderAdapterActorListMap.FindOrAdd(WorldPartition, MakeUnique<FLoaderAdapterActorList>(WorldPartition->GetWorld()));

		LoaderAdapterActorList->AddActors(InActorsToLoad);
	}
#endif
}

void UWorldPartitionBlueprintLibrary::UnloadActors(const TArray<FGuid>& InActorsToUnload)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		if (TUniquePtr<FLoaderAdapterActorList>* LoaderAdapterActorList = LoaderAdapterActorListMap.Find(WorldPartition))
		{
			(*LoaderAdapterActorList)->RemoveActors(InActorsToUnload);
		}
	}
#endif
}

void UWorldPartitionBlueprintLibrary::PinActors(const TArray<FGuid>& InActorsToPin)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		WorldPartition->PinActors(InActorsToPin);
	}
#endif
}

void UWorldPartitionBlueprintLibrary::UnpinActors(const TArray<FGuid>& InActorsToUnpin)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		WorldPartition->UnpinActors(InActorsToUnpin);
	}
#endif
}

bool UWorldPartitionBlueprintLibrary::GetActorDescs(TArray<FActorDesc>& OutActorDescs)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		return GetActorDescs(WorldPartition, OutActorDescs);
	}
#endif
	return false;
}

bool UWorldPartitionBlueprintLibrary::GetIntersectingActorDescs(const FBox& InBox, TArray<FActorDesc>& OutActorDescs)
{
#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		return GetIntersectingActorDescs(WorldPartition, InBox, OutActorDescs);
	}
#endif
	return false;
}

bool UWorldPartitionBlueprintLibrary::GetActorDescsForActors(const TArray<AActor*>& InActors, TArray<FActorDesc>& OutActorDescs)
{
	bool bResult = false;

#if WITH_EDITOR
	if (UWorldPartition* WorldPartition = GetWorldPartition())
	{
		for (AActor* Actor : InActors)
		{
			if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(Actor->GetActorGuid()))
			{
				OutActorDescs.Emplace(*ActorDescInstance);
				bResult = true;
			}
		}
	}
#endif

	return bResult;
}

UDataLayerManager* UWorldPartitionBlueprintLibrary::GetDataLayerManager(class UObject* WorldContextObject)
{
	return UDataLayerManager::GetDataLayerManager(WorldContextObject);
}

UDataLayerManager* UWorldPartitionBlueprintLibrary::GetDataLayerManagerForObject(class UObject* Object)
{
	return UDataLayerManager::GetDataLayerManager(Object);
}