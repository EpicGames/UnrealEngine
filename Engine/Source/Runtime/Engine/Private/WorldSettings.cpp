// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFramework/WorldSettings.h"
#include "Misc/MessageDialog.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineStats.h"
#include "SceneInterface.h"
#include "GameFramework/DefaultPhysicsVolume.h"
#include "EngineUtils.h"
#include "Engine/AssetUserData.h"
#include "Engine/Engine.h"
#include "Engine/WorldComposition.h"
#include "WorldPartition/WorldPartition.h"
#include "Net/UnrealNetwork.h"
#include "Net/Core/PushModel/PushModel.h"
#include "GameFramework/GameNetworkManager.h"
#include "AudioDevice.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Particles/ParticleEventManager.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "SceneManagement.h"
#include "AI/AISystemBase.h"
#include "AI/NavigationSystemConfig.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/BookMark.h"
#include "WorldSettingsCustomVersion.h"
#include "Materials/Material.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Algo/Transform.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Misc/TransactionObjectEvent.h"
#include "HierarchicalLOD.h"
#include "HLOD/HLODSetup.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldSettings)

#define LOCTEXT_NAMESPACE "ErrorChecking"

DEFINE_LOG_CATEGORY_STATIC(LogWorldSettings, Log, All);

// @todo vreditor urgent: Temporary hack to allow world-to-meters to be set before
// input is polled for motion controller devices each frame.
ENGINE_API float GNewWorldToMetersScale = 0.0f;

/** Allows globally disabling world bound checks. */
static int32 GDisableWorldBoundsChecks = 0;
static FAutoConsoleVariableRef CVarActorWorldBoundsOverride(
	TEXT("WorldBounds.DisableChecks"),
	GDisableWorldBoundsChecks,
	TEXT("If set to 1, will ignore all worlds bounds checks."));

#if WITH_EDITOR
AWorldSettings::FOnBookmarkClassChanged AWorldSettings::OnBookmarkClassChanged;
AWorldSettings::FOnNumberOfBookmarksChanged AWorldSettings::OnNumberOfBoomarksChanged;
AWorldSettings::FOnDefaultBookmarkChanged AWorldSettings::OnDefaultBookmarkChanged;

namespace WorldSettingsUtils
{
	bool CanDeleteOrReplaceCommon(const AWorldSettings* InActor, FText& OutReason)
	{
		const ULevel* OwnerLevel = InActor->GetLevel();
		if (OwnerLevel && OwnerLevel->GetWorldSettings() == InActor)
		{
			OutReason = NSLOCTEXT("WorldSettings", "CanDeleteOrReplace_Error_WorldSettings", "Can't delete or replace a level's world settings.");
			return false;
		}

		return true;
	}
}
#endif

AWorldSettings::FOnNaniteSettingsChanged AWorldSettings::OnNaniteSettingsChanged;

AWorldSettings::AWorldSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(TEXT("Sprite")))
	, WorldPartition(nullptr)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UClass> DmgType_Environmental_Object;
		FConstructorStatics()
			: DmgType_Environmental_Object(TEXT("/Engine/EngineDamageTypes/DmgTypeBP_Environmental.DmgTypeBP_Environmental_C"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	bEnableWorldBoundsChecks = true;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bEnableNavigationSystem = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	NavigationSystemConfig = nullptr;
	bEnableAISystem = true;
	AISystemClass = UAISystemBase::GetAISystemClassName();
	bUseAutomaticUGCNavigation = false;
	bEnableWorldComposition = false;
	bEnableWorldOriginRebasing = false;
#if WITH_EDITORONLY_DATA
	bEnableLargeWorlds_DEPRECATED = (UE_USE_UE4_WORLD_MAX != 0);
	bEnableHierarchicalLODSystem_DEPRECATED = true;
	NumHLODLevels = 0;
	bGenerateSingleClusterForLevel = false;
#endif

	KillZ = -UE_OLD_HALF_WORLD_MAX1;	// LWC_TODO: HALF_WORLD_MAX1? Something else?
	KillZDamageType = ConstructorStatics.DmgType_Environmental_Object.Object;

#if WITH_EDITORONLY_DATA
	InstancedFoliageGridSize = DefaultPlacementGridSize = LandscapeSplineMeshesGridSize = 25600;
	NavigationDataChunkGridSize = 102400;
	NavigationDataBuilderLoadingCellSize = 102400 * 4;
	bHideEnableStreamingWarning = false;
	bIncludeGridSizeInNameForFoliageActors = false;
	bIncludeGridSizeInNameForPartitionedActors = false;
#endif
	WorldToMeters = 100.f;

	DefaultPhysicsVolumeClass = ADefaultPhysicsVolume::StaticClass();
	GameNetworkManagerClass = AGameNetworkManager::StaticClass();
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	bAlwaysRelevant = true;
	TimeDilation = 1.0f;
	CinematicTimeDilation = 1.0f;
	DemoPlayTimeDilation = 1.0f;
	bAllowTimeDilation = true;
	PackedLightAndShadowMapTextureSize = 1024;
	SetHidden(false);

	DefaultColorScale = FVector(1.0f, 1.0f, 1.0f);
	DefaultMaxDistanceFieldOcclusionDistance = 600;
	GlobalDistanceFieldViewDistance = 20000;
	DynamicIndirectShadowsSelfShadowingIntensity = .8f;
	bPlaceCellsOnlyAlongCameraTracks = false;
	VisibilityCellSize = 200;
	VisibilityAggressiveness = VIS_LeastAggressive;
	
	VolumetricLightmapLoadingRange = 6400;

#if WITH_EDITORONLY_DATA
	bActorLabelEditable = false;
#endif // WITH_EDITORONLY_DATA

	bReplayRewindable = true;

	MaxNumberOfBookmarks = 10;

	DefaultBookmarkClass = UBookMark::StaticClass();
	LastBookmarkClass = DefaultBookmarkClass;

	LevelInstancePivotOffset = FVector::ZeroVector;

	bReuseAddressAndPort = false;
}

void AWorldSettings::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_NeedLoad|RF_WasLoaded|RF_ClassDefaultObject) == false)
	{
		TSubclassOf<UNavigationSystemConfig> NavSystemConfigClass = UNavigationSystemConfig::GetDefaultConfigClass();
		if (*NavSystemConfigClass)
		{
			NavigationSystemConfig = NewObject<UNavigationSystemConfig>(this, NavSystemConfigClass);
		}
	}

	if (MinGlobalTimeDilation < 0)
	{
		MinGlobalTimeDilation = 0;
	}

	if (MaxGlobalTimeDilation < 0)
	{
		MaxGlobalTimeDilation = 0;
	}

	if (MinCinematicTimeDilation < 0)
	{
		MinCinematicTimeDilation = 0;
	}

	if (MaxCinematicTimeDilation < 0)
	{
		MaxCinematicTimeDilation = 0;
	}

	if (MinUndilatedFrameTime < 0)
	{
		MinUndilatedFrameTime = 0;
	}

	if (MaxUndilatedFrameTime < 0)
	{
		MaxUndilatedFrameTime = 0;
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UpdateNumberOfBookmarks();
		UpdateBookmarkClass();
	}
}

void AWorldSettings::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	// create the emitter pool
	// we only need to do this for the persistent level's WorldSettings as sublevel actors will have their WorldSettings set to it on association
	if (GetNetMode() != NM_DedicatedServer && IsInPersistentLevel())
	{
		UWorld* World = GetWorld();
		check(World);		

		// only create once - 
		if (World->MyParticleEventManager == NULL && !GEngine->ParticleEventManagerClassPath.IsEmpty())
		{
			UObject* Object = StaticLoadObject(UClass::StaticClass(), NULL, *GEngine->ParticleEventManagerClassPath, NULL, LOAD_NoWarn, NULL);
			if (Object != NULL)
			{
				TSubclassOf<AParticleEventManager> ParticleEventManagerClass = Cast<UClass>(Object);
				if (ParticleEventManagerClass != NULL)
				{
					FActorSpawnParameters SpawnParameters;
					SpawnParameters.Owner = this;
					SpawnParameters.Instigator = GetInstigator();
					SpawnParameters.ObjectFlags |= RF_Transient;	// We never want to save particle event managers into a map
					World->MyParticleEventManager = World->SpawnActor<AParticleEventManager>(ParticleEventManagerClass, SpawnParameters);
				}
			}
		}
	}
}

void AWorldSettings::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	UWorld* World = GetWorld();
	if (FAudioDeviceHandle AudioDevice = World->GetAudioDevice())
	{
		AudioDevice->SetDefaultAudioSettings(World, DefaultReverbSettings, DefaultAmbientZoneSettings);
	}
}

UWorldPartition* AWorldSettings::GetWorldPartition() const
{
	return WorldPartition;
}

void AWorldSettings::SetWorldPartition(UWorldPartition* InWorldPartition)
{
	check(!InWorldPartition || !WorldPartition);
	WorldPartition = InWorldPartition;
	ApplyWorldPartitionForcedSettings();
}

void AWorldSettings::ApplyWorldPartitionForcedSettings()
{
	static const auto CVarAllowStaticLightingOnWPMaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowStaticLightingInWorldPartitionMaps"));

	bEnableWorldComposition = false;
	bPrecomputeVisibility = false;
	if (CVarAllowStaticLightingOnWPMaps->GetValueOnAnyThread() != 0)
	{
		// leave static lighting options alone
	}
	else
	{
		bForceNoPrecomputedLighting = true;	
	}
#if WITH_EDITOR

	LightmassSettings.bWorldPartition = true;
#endif
}

float AWorldSettings::GetGravityZ() const
{
	if (!bWorldGravitySet)
	{
		// try to initialize cached value
		AWorldSettings* const MutableThis = const_cast<AWorldSettings*>(this);
		MutableThis->WorldGravityZ = bGlobalGravitySet ? GlobalGravityZ : UPhysicsSettings::Get()->DefaultGravityZ;	//allows us to override DefaultGravityZ
	}

	return WorldGravityZ;
}

#if WITH_EDITOR
void AWorldSettings::SupportsWorldPartitionStreamingChanged()
{
	if (WorldPartition)
	{
		WorldPartition->OnEnableStreamingChanged();
	}
}
#endif

void AWorldSettings::OnRep_WorldGravityZ()
{
	bWorldGravitySet = true;
}

void AWorldSettings::OnRep_NaniteSettings()
{
	// Need to recreate scene proxies when Nanite settings changes.
	FGlobalComponentRecreateRenderStateContext Context;

	OnNaniteSettingsChanged.Broadcast(this);
}

void AWorldSettings::SetAllowMaskedMaterials(bool bState)
{
	if (GetLocalRole() == ROLE_Authority)
	{
		NaniteSettings.bAllowMaskedMaterials = bState;
		MARK_PROPERTY_DIRTY_FROM_NAME(AWorldSettings, NaniteSettings, this);
	}
}

float AWorldSettings::FixupDeltaSeconds(float DeltaSeconds, float RealDeltaSeconds)
{
	// DeltaSeconds is assumed to be fully dilated at this time, so we will dilate the clamp range as well
	float const Dilation = GetEffectiveTimeDilation();
	float const MinFrameTime = MinUndilatedFrameTime * Dilation;
	float const MaxFrameTime = MaxUndilatedFrameTime * Dilation;

	// clamp frame time according to desired limits
	return FMath::Clamp(DeltaSeconds, MinFrameTime, MaxFrameTime);	
}

float AWorldSettings::SetTimeDilation(float NewTimeDilation)
{
	TimeDilation = FMath::Clamp(NewTimeDilation, MinGlobalTimeDilation, MaxGlobalTimeDilation);
	return TimeDilation;
}

float AWorldSettings::SetCinematicTimeDilation(float NewCinematicTimeDilation)
{
	CinematicTimeDilation = FMath::Clamp(NewCinematicTimeDilation, MinCinematicTimeDilation, MaxCinematicTimeDilation);
	return CinematicTimeDilation;
}


void AWorldSettings::SetAllowTimeDilation(bool NewAllowTimeDilation) 
{ 
	bAllowTimeDilation = NewAllowTimeDilation; 
}

void AWorldSettings::NotifyBeginPlay()
{
	UWorld* World = GetWorld();
	if (!World->GetBegunPlay())
	{
		World->OnWorldPreBeginPlay.Broadcast();

		for (FActorIterator It(World); It; ++It)
		{
			SCOPE_CYCLE_COUNTER(STAT_ActorBeginPlay);
			const bool bFromLevelLoad = true;
			It->DispatchBeginPlay(bFromLevelLoad);
		}

		World->SetBegunPlay(true);
	}
}

void AWorldSettings::NotifyMatchStarted()
{
	UWorld* World = GetWorld();
	World->OnWorldMatchStarting.Broadcast();
	World->bMatchStarted = true;
}

void AWorldSettings::GetLifetimeReplicatedProps( TArray< FLifetimeProperty > & OutLifetimeProps ) const
{
	Super::GetLifetimeReplicatedProps( OutLifetimeProps );

	DOREPLIFETIME( AWorldSettings, PauserPlayerState );
	DOREPLIFETIME( AWorldSettings, TimeDilation );
	DOREPLIFETIME( AWorldSettings, CinematicTimeDilation );
	DOREPLIFETIME( AWorldSettings, WorldGravityZ );
	DOREPLIFETIME( AWorldSettings, bHighPriorityLoading );

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST( AWorldSettings, NaniteSettings, SharedParams );
}

#if WITH_EDITOR
bool AWorldSettings::GetDynamicLoadingRangeScalingEnable() const
{
	// The per-project, per-user settings are stored for the Editor world, not the PIE world.
	UWorld* EditorWorld = (GEditor) ? GEditor->GetEditorWorldContext().World() : nullptr;

	// DynamicLoadingRangeScaling is enabled in WorldSettings, but can also be disabled via a per-user, per-map override
	return DynamicLoadingRangeScaling.bEnabled && EditorWorld && !GetDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetForceDisableDynamicLoadingRangeScaling(EditorWorld);
}
#endif // WITH_EDITOR

const FGuid FWorldSettingCustomVersion::GUID(0x1ED048F4, 0x2F2E4C68, 0x89D053A4, 0xF18F102D);
// Register the custom version with core
FCustomVersionRegistration GRegisterWorldSettingCustomVersion(FWorldSettingCustomVersion::GUID, FWorldSettingCustomVersion::LatestVersion, TEXT("WorldSettingVer"));

void AWorldSettings::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FEnterpriseObjectVersion::GUID);
	Ar.UsingCustomVersion(FWorldSettingCustomVersion::GUID);

	if (Ar.UEVer() < VER_UE4_ADD_OVERRIDE_GRAVITY_FLAG)
	{
		//before we had override flag we would use GlobalGravityZ != 0
		if(GlobalGravityZ != 0.0f)
		{
			bGlobalGravitySet = true;
		}
	}
#if WITH_EDITOR	
	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::ConvertHLODScreenSize)
	{
		for (FHierarchicalSimplification& Setup : HierarchicalLODSetup)
		{
			const float OldScreenSize = Setup.TransitionScreenSize;

			const float HalfFOV = UE_PI * 0.25f;
			const float ScreenWidth = 1920.0f;
			const float ScreenHeight = 1080.0f;
			const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);

			const float DummySphereRadius = 16.0f;
			const float ScreenArea = OldScreenSize * (ScreenWidth * ScreenHeight);
			const float ScreenRadius = FMath::Sqrt(ScreenArea / UE_PI);
			const float ScreenDistance = FMath::Max(ScreenWidth / 2.0f * ProjMatrix.M[0][0], ScreenHeight / 2.0f * ProjMatrix.M[1][1]) * DummySphereRadius / ScreenRadius;

			Setup.TransitionScreenSize = ComputeBoundsScreenSize(FVector::ZeroVector, DummySphereRadius, FVector(0.0f, 0.0f, ScreenDistance), ProjMatrix);
		}
	}
#endif

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FEnterpriseObjectVersion::GUID) < FEnterpriseObjectVersion::BookmarkExtensibilityUpgrade)
		{
			const TObjectPtr<UBookmarkBase>* LocalBookmarks =
				reinterpret_cast<TObjectPtr<UBookmarkBase>*>(BookMarks);
			const int32 NumBookmarks = sizeof(BookMarks) / sizeof(UBookMark*);
			BookmarkArray = TArray<TObjectPtr<UBookmarkBase>>(LocalBookmarks, NumBookmarks);
			AdjustNumberOfBookmarks();
		}

		if (Ar.CustomVer(FWorldSettingCustomVersion::GUID) < FWorldSettingCustomVersion::DeprecatedEnableHierarchicalLODSystem)
		{
			if (!bEnableHierarchicalLODSystem_DEPRECATED)
			{
				ResetHierarchicalLODSetup();
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void AWorldSettings::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		RemoveUserDataOfClass(InUserData->GetClass());
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* AWorldSettings::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	const TArray<UAssetUserData*>* ArrayPtr = GetAssetUserDataArray();
	for (int32 DataIdx = 0; DataIdx < ArrayPtr->Num(); DataIdx++)
	{
		UAssetUserData* Datum = (*ArrayPtr)[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

const TArray<UAssetUserData*>* AWorldSettings::GetAssetUserDataArray() const
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	else
	{
		static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
		CachedAssetUserData.Reset();
		CachedAssetUserData.Append(AssetUserData);
		CachedAssetUserData.Append(AssetUserDataEditorOnly);
		return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
	}
#else
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
}

#if WITH_EDITOR
const TArray<FHierarchicalSimplification>& AWorldSettings::GetHierarchicalLODSetup() const
{
	const UHierarchicalLODSettings* HLODSettings = GetDefault<UHierarchicalLODSettings>();

	// If we have a HLOD asset set use this
	if (HLODSetupAsset.LoadSynchronous())
	{
		return HLODSetupAsset->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup;
	}
	else if (HLODSettings->bForceSettingsInAllMaps && HLODSettings->DefaultSetup.IsValid())
	{
		return HLODSettings->DefaultSetup->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup;
	}
	
	return HierarchicalLODSetup;
}

TArray<FHierarchicalSimplification>& AWorldSettings::GetHierarchicalLODSetup()
{
	UHierarchicalLODSettings* HLODSettings = GetMutableDefault<UHierarchicalLODSettings>();
	
	// If we have a HLOD asset set use this
	if (HLODSetupAsset.LoadSynchronous())
	{
		return HLODSetupAsset->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup;
	}
	else if (HLODSettings->bForceSettingsInAllMaps && HLODSettings->DefaultSetup.LoadSynchronous())
	{
		return HLODSettings->DefaultSetup->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup;
	}

	return HierarchicalLODSetup;
}

int32 AWorldSettings::GetNumHierarchicalLODLevels() const
{
	const UHierarchicalLODSettings* HLODSettings = GetDefault<UHierarchicalLODSettings>();

	// If we have a HLOD asset set use this
	if (HLODSetupAsset.LoadSynchronous())
	{
		return HLODSetupAsset->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup.Num();
	}
	else if (HLODSettings->bForceSettingsInAllMaps && HLODSettings->DefaultSetup.IsValid())
	{
		return HLODSettings->DefaultSetup->GetDefaultObject<UHierarchicalLODSetup>()->HierarchicalLODSetup.Num();
	}

	return HierarchicalLODSetup.Num();
}

UMaterialInterface* AWorldSettings::GetHierarchicalLODBaseMaterial() const
{
	UMaterialInterface* Material = GetDefault<UHierarchicalLODSettings>()->BaseMaterial.LoadSynchronous();

	if (!OverrideBaseMaterial.IsNull())
	{
		Material = OverrideBaseMaterial.LoadSynchronous();
	}

	if (HLODSetupAsset.LoadSynchronous())
	{
		if (!HLODSetupAsset->GetDefaultObject<UHierarchicalLODSetup>()->OverrideBaseMaterial.IsNull())
		{
			Material = HLODSetupAsset->GetDefaultObject<UHierarchicalLODSetup>()->OverrideBaseMaterial.LoadSynchronous();
		}
	}

	return Material;
}

void AWorldSettings::ResetHierarchicalLODSetup()
{
	HLODSetupAsset = nullptr;
	OverrideBaseMaterial = nullptr;
	HierarchicalLODSetup.Reset();
	NumHLODLevels = 0;
}

void AWorldSettings::SaveDefaultWorldPartitionSettings()
{
	ResetDefaultWorldPartitionSettings();

	if (WorldPartition)
	{
		DefaultWorldPartitionSettings.LoadedEditorRegions = WorldPartition->GetUserLoadedEditorRegions();

		if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(GetWorld()))
		{
			TArray<const UDataLayerInstance*> LoadedDataLayersPtrArray;
			TArray<const UDataLayerInstance*> NotLoadedDataLayersPtrArray;
			DataLayerManager->GetUserLoadedInEditorStates(LoadedDataLayersPtrArray, NotLoadedDataLayersPtrArray);
			DefaultWorldPartitionSettings.LoadedDataLayers.Reset(LoadedDataLayersPtrArray.Num());
			DefaultWorldPartitionSettings.NotLoadedDataLayers.Reset(NotLoadedDataLayersPtrArray.Num());
			Algo::Transform(LoadedDataLayersPtrArray, DefaultWorldPartitionSettings.LoadedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetAsset(); });
			Algo::Transform(NotLoadedDataLayersPtrArray, DefaultWorldPartitionSettings.NotLoadedDataLayers, [](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->GetAsset(); });
		}
	}
}

void AWorldSettings::ResetDefaultWorldPartitionSettings()
{
	Modify();
	DefaultWorldPartitionSettings.Reset();
}

const FWorldPartitionPerWorldSettings* AWorldSettings::GetDefaultWorldPartitionSettings() const
{
	if (WorldPartition)
	{
		return &DefaultWorldPartitionSettings;
	}

	return nullptr;
}
#endif // WITH_EDITOR

void AWorldSettings::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
#if WITH_EDITOR
	for (int32 DataIdx = 0; DataIdx < AssetUserDataEditorOnly.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserDataEditorOnly[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserDataEditorOnly.RemoveAt(DataIdx);
			return;
		}
	}
#endif
}

void AWorldSettings::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (WorldPartition)
	{
		// Force to re-apply WorldPartition restrictions on WorldSettings (in case they changed)	
		ApplyWorldPartitionForcedSettings();

		// Partitionned levels have no need for this legacy data
		ResetHierarchicalLODSetup();
	}
#endif// WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// temporarily using deprecated bEnableNavigationSystem for backwards compatibility
	if (bEnableNavigationSystem && NavigationSystemConfig == nullptr)
	{
		ULevel* Level = GetLevel();
		if (Level)
		{
			TSubclassOf<UNavigationSystemConfig> NavSystemConfigClass = UNavigationSystemConfig::GetDefaultConfigClass();
			if (*NavSystemConfigClass)
		{
				NavigationSystemConfig = NewObject<UNavigationSystemConfig>(this, NavSystemConfigClass);
			}
			bEnableNavigationSystem = false;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITORONLY_DATA
void AWorldSettings::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNavigationSystemConfig::StaticClass()));	
	TSubclassOf<UNavigationSystemConfig> NavSystemConfigClass = UNavigationSystemConfig::GetDefaultConfigClass();
	if (*NavSystemConfigClass)
	{
		OutConstructClasses.Add(FTopLevelAssetPath(NavSystemConfigClass));
	}
}
#endif

bool AWorldSettings::IsNavigationSystemEnabled() const
{
	return NavigationSystemConfig && NavigationSystemConfig->NavigationSystemClass.IsValid();
}

void AWorldSettings::SetNavigationSystemConfigOverride(UNavigationSystemConfig* NewConfig)
{
	NavigationSystemConfigOverride = NewConfig;
	if (NavigationSystemConfig)
	{
		NavigationSystemConfig->SetIsOverriden(NewConfig != nullptr && NewConfig != NavigationSystemConfig);
	}
}

bool AWorldSettings::AreWorldBoundsChecksEnabled() const
{
	return bEnableWorldBoundsChecks && !GDisableWorldBoundsChecks;
}

#if WITH_EDITOR

void AWorldSettings::CheckForErrors()
{
	Super::CheckForErrors();

	UWorld* World = GetWorld();
	// World is nullptr if save is done from a derived AWorldSettings blueprint
	if (World == nullptr)
	{
		return;
	}

	if ( World->GetWorldSettings() != this )
	{
		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_DuplicateLevelInfo", "Duplicate level info" ) ))
			->AddToken(FMapErrorToken::Create(FMapErrors::DuplicateLevelInfo));
	}

	if (!World->GetWorldSettings()->bForceNoPrecomputedLighting)
	{
		int32 NumLightingScenariosEnabled = 0;

		for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); LevelIndex++)
		{
			ULevel* Level = World->GetLevels()[LevelIndex];

			if (Level->bIsLightingScenario && Level->bIsVisible)
			{
				NumLightingScenariosEnabled++;
			}
		}
		if( World->NumLightingUnbuiltObjects > 0 && NumLightingScenariosEnabled <= 1 )
		{
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT( "MapCheck_Message_RebuildLighting", "Maps need lighting rebuilt" ) ))
				->AddToken(FMapErrorToken::Create(FMapErrors::RebuildLighting));
		}
	}
}

bool AWorldSettings::CanDeleteSelectedActor(FText& OutReason) const
{
	if (!Super::CanDeleteSelectedActor(OutReason))
	{
		return false;
	}

	return WorldSettingsUtils::CanDeleteOrReplaceCommon(this, OutReason);
}

bool AWorldSettings::CanReplaceSelectedActor(FText& OutReason) const
{
	if (!Super::CanReplaceSelectedActor(OutReason))
	{
		return false;
	}

	return WorldSettingsUtils::CanDeleteOrReplaceCommon(this, OutReason);
}


bool AWorldSettings::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (InProperty->GetOwner<UObject>() &&
			  InProperty->GetOwner<UObject>()->GetName() == TEXT("LightmassWorldInfoSettings"))
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, bGenerateAmbientOcclusionMaterialMask)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, DirectIlluminationOcclusionFraction)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, IndirectIlluminationOcclusionFraction)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, OcclusionExponent)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, FullyOccludedSamplesFraction)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, MaxOcclusionDistance)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, bVisualizeAmbientOcclusion))
			{
				return LightmassSettings.bUseAmbientOcclusion;
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, VolumetricLightmapDetailCellSize)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, VolumetricLightmapMaximumBrickMemoryMb)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, VolumetricLightmapSphericalHarmonicSmoothing)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, VolumetricLightmapLoadingCellSize)
				|| PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AWorldSettings, VolumetricLightmapLoadingRange))
			{
				return LightmassSettings.VolumeLightingMethod == VLM_VolumetricLightmap;
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, VolumeLightSamplePlacementScale))
			{
				return LightmassSettings.VolumeLightingMethod == VLM_SparseVolumeLightingSamples;
			}

			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FLightmassWorldInfoSettings, EnvironmentColor))
			{
				return LightmassSettings.EnvironmentIntensity > 0;
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AWorldSettings, bEnableWorldComposition ) ||
				 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AWorldSettings, bPrecomputeVisibility))
		{
			return !IsPartitionedWorld();
		}
		else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AWorldSettings, LandscapeSplineMeshesGridSize))
		{
			return IsPartitionedWorld();
		}
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(AWorldSettings, InstancedFoliageGridSize))
		{
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}

void AWorldSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UWorld* World = GetWorld();

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged)
	{
		InternalPostPropertyChanged(PropertyThatChanged->GetFName());
	}

	LightmassSettings.NumIndirectLightingBounces = FMath::Clamp(LightmassSettings.NumIndirectLightingBounces, 0, 100);
	LightmassSettings.NumSkyLightingBounces = FMath::Clamp(LightmassSettings.NumSkyLightingBounces, 0, 100);
	LightmassSettings.IndirectLightingSmoothness = FMath::Clamp(LightmassSettings.IndirectLightingSmoothness, .25f, 10.0f);
	LightmassSettings.VolumeLightSamplePlacementScale = FMath::Clamp(LightmassSettings.VolumeLightSamplePlacementScale, .1f, 100.0f);
	LightmassSettings.VolumetricLightmapDetailCellSize = FMath::Clamp(LightmassSettings.VolumetricLightmapDetailCellSize, 1.0f, 10000.0f);
	LightmassSettings.IndirectLightingQuality = FMath::Clamp(LightmassSettings.IndirectLightingQuality, .1f, 100.0f);
	LightmassSettings.StaticLightingLevelScale = FMath::Clamp(LightmassSettings.StaticLightingLevelScale, .001f, 1000.0f);
	LightmassSettings.EmissiveBoost = FMath::Max(LightmassSettings.EmissiveBoost, 0.0f);
	LightmassSettings.DiffuseBoost = FMath::Max(LightmassSettings.DiffuseBoost, 0.0f);
	LightmassSettings.DirectIlluminationOcclusionFraction = FMath::Clamp(LightmassSettings.DirectIlluminationOcclusionFraction, 0.0f, 1.0f);
	LightmassSettings.IndirectIlluminationOcclusionFraction = FMath::Clamp(LightmassSettings.IndirectIlluminationOcclusionFraction, 0.0f, 1.0f);
	LightmassSettings.OcclusionExponent = FMath::Max(LightmassSettings.OcclusionExponent, 0.0f);
	LightmassSettings.FullyOccludedSamplesFraction = FMath::Clamp(LightmassSettings.FullyOccludedSamplesFraction, 0.0f, 1.0f);
	LightmassSettings.MaxOcclusionDistance = FMath::Max(LightmassSettings.MaxOcclusionDistance, 0.0f);
	LightmassSettings.EnvironmentIntensity = FMath::Max(LightmassSettings.EnvironmentIntensity, 0.0f);

	const FName PropName = PropertyChangedEvent.GetPropertyName();
	if (PropName == GET_MEMBER_NAME_CHECKED(AWorldSettings, bEnableAISystem)
	|| PropName == GET_MEMBER_NAME_CHECKED(AWorldSettings, AISystemClass))
	{
		if (World)
		{
			World->CreateAISystem();
		}
	}

	// Ensure texture size is power of two between 512 and 4096.
	PackedLightAndShadowMapTextureSize = FMath::Clamp<uint32>( FMath::RoundUpToPowerOfTwo( PackedLightAndShadowMapTextureSize ), 512, 4096 );

	if (PropertyThatChanged != nullptr && World != nullptr && World->Scene)
	{
		World->Scene->UpdateSceneSettings(this);
	}

	for (UAssetUserData* Datum : AssetUserData)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	for (UAssetUserData* Datum : AssetUserDataEditorOnly)
	{
		if (Datum != nullptr)
		{
			Datum->PostEditChangeOwner(PropertyChangedEvent);
		}
	}

	if (PropName == GET_MEMBER_NAME_CHECKED(AWorldSettings, bShowInstancedFoliageGrid))
	{
		if (bShowInstancedFoliageGrid)
		{
			check(!InstancedFoliageGridGridPreviewer);
			InstancedFoliageGridGridPreviewer = MakeUnique<FWorldGridPreviewer>(GetTypedOuter<UWorld>(), true);
		}
		else
		{
			check(InstancedFoliageGridGridPreviewer);
			InstancedFoliageGridGridPreviewer.Reset();
		}

		if (InstancedFoliageGridGridPreviewer)
		{
			InstancedFoliageGridGridPreviewer->CellSize = InstancedFoliageGridSize;
			InstancedFoliageGridGridPreviewer->GridColor = FColor::White;
			InstancedFoliageGridGridPreviewer->GridOffset = FVector::ZeroVector;
			InstancedFoliageGridGridPreviewer->LoadingRange = MAX_int32;
			InstancedFoliageGridGridPreviewer->Update();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AWorldSettings::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		for (const FName& PropertyName : TransactionEvent.GetChangedProperties())
		{
			InternalPostPropertyChanged(PropertyName);
		}
	}
}

void AWorldSettings::InternalPostPropertyChanged(FName PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, DefaultReverbSettings) || PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, DefaultAmbientZoneSettings))
{
		UWorld* World = GetWorld();
		if (FAudioDeviceHandle AudioDevice = World->GetAudioDevice())
	{
			AudioDevice->SetDefaultAudioSettings(World, DefaultReverbSettings, DefaultAmbientZoneSettings);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, bForceNoPrecomputedLighting) && bForceNoPrecomputedLighting)
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("bForceNoPrecomputedLightingIsEnabled", "bForceNoPrecomputedLighting is now enabled, build lighting once to propagate the change (will remove existing precomputed lighting data)."));
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings,bEnableWorldComposition))
	{
		bEnableWorldComposition = UWorldComposition::EnableWorldCompositionEvent.IsBound() ? UWorldComposition::EnableWorldCompositionEvent.Execute(GetWorld(), bEnableWorldComposition): false;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, NavigationSystemConfig))
	{
		UWorld* World = GetWorld();
		if (World && World->PersistentLevel)
		{
			World->SetNavigationSystem(nullptr);
			if (NavigationSystemConfig)
			{
				FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorMode, NavigationSystemConfig);
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, MaxNumberOfBookmarks))
	{
		UpdateNumberOfBookmarks();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, DefaultBookmarkClass))
	{
		UpdateBookmarkClass();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, DefaultBookmark))
	{
		OnDefaultBookmarkChanged.Broadcast(this);
	}

	if (GetWorld() != nullptr && GetWorld()->PersistentLevel && GetWorld()->PersistentLevel->GetWorldSettings() == this)
	{
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, TransitionScreenSize))
		{
			GEditor->BroadcastHLODTransitionScreenSizeChanged();
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, HierarchicalLODSetup))
		{
			GEditor->BroadcastHLODLevelsArrayChanged();
			NumHLODLevels = HierarchicalLODSetup.Num();			
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(AWorldSettings, OverrideBaseMaterial))
		{
			if (!OverrideBaseMaterial.IsNull())
			{
				if (!UHierarchicalLODSettings::IsValidFlattenMaterial(OverrideBaseMaterial.LoadSynchronous(), true))
				{
					OverrideBaseMaterial = GEngine->DefaultHLODFlattenMaterial;
				}
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FNaniteSettings, bAllowMaskedMaterials))
		{
			// Need to recreate scene proxies when this flag changes.
			FGlobalComponentRecreateRenderStateContext Context;
		
			OnNaniteSettingsChanged.Broadcast(this);
		}
	}
}

void UHierarchicalLODSetup::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UHierarchicalLODSetup, OverrideBaseMaterial))
	{
		if (!OverrideBaseMaterial.IsNull())
		{
			if (!UHierarchicalLODSettings::IsValidFlattenMaterial(OverrideBaseMaterial.LoadSynchronous(), true))
			{
				OverrideBaseMaterial = GEngine->DefaultHLODFlattenMaterial;
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void AWorldSettings::CompactBookmarks()
{
	Modify();
	BookmarkArray.RemoveAll([&](const UBookmarkBase* Base) { return Base == nullptr; });

	// See if we can shrink the overall size of the bookmark array.
	const int32 DefaultMaxNumberOfBookmarks = Cast<AWorldSettings>(GetClass()->GetDefaultObject())->MaxNumberOfBookmarks;
	const int32 IntMappedBookmarks = static_cast<int32>(NumMappedBookmarks);

	MaxNumberOfBookmarks = FMath::Max(DefaultMaxNumberOfBookmarks, FMath::Max(IntMappedBookmarks, BookmarkArray.Num()));
	AdjustNumberOfBookmarks();
}

class UBookmarkBase* AWorldSettings::GetOrAddBookmark(const uint32 BookmarkIndex, const bool bRecreateOnClassMismatch)
{
	if (BookmarkArray.IsValidIndex(BookmarkIndex))
	{
		TObjectPtr<UBookmarkBase>& Bookmark = BookmarkArray[BookmarkIndex];

		if (Bookmark == nullptr || (bRecreateOnClassMismatch && Bookmark->GetClass() != GetDefaultBookmarkClass()))
		{
			Modify();
			Bookmark = NewObject<UBookmarkBase>(this, GetDefaultBookmarkClass(), NAME_None, RF_Transactional);
		}

		return Bookmark;
	}

	return nullptr;
}

UBookmarkBase* AWorldSettings::AddBookmark(const TSubclassOf<UBookmarkBase> BookmarkClass, const bool bExpandIfNecessarry)
{
	UBookmarkBase* Result = nullptr;
	
	UClass* NewBookmarkClass = BookmarkClass.Get();
	if (NewBookmarkClass == nullptr)
	{
		NewBookmarkClass = GetDefaultBookmarkClass();
	}

	if (NewBookmarkClass)
	{
		int32 UseIndex = INDEX_NONE;
		if (!BookmarkArray.Find(nullptr, UseIndex) && bExpandIfNecessarry)
		{
			Modify();
			BookmarkArray.AddZeroed();
			UseIndex = MaxNumberOfBookmarks;
			MaxNumberOfBookmarks = BookmarkArray.Num();
		}

		if (BookmarkArray.IsValidIndex(UseIndex))
		{
			Modify();
			Result = NewObject<UBookmarkBase>(this, NewBookmarkClass, NAME_None, RF_Transactional);
			BookmarkArray[UseIndex] = Result;
		}
	}

	return Result;
}

void AWorldSettings::ClearBookmark(const uint32 BookmarkIndex)
{
	if (BookmarkArray.IsValidIndex(BookmarkIndex))
	{
		if (TObjectPtr<UBookmarkBase>& Bookmark = BookmarkArray[BookmarkIndex])
		{
			Modify();
			Bookmark->OnCleared();
			Bookmark = nullptr;
		}
	}
}

void AWorldSettings::ClearAllBookmarks()
{
	Modify();
	for (TObjectPtr<UBookmarkBase>& Bookmark : BookmarkArray)
	{
		if (Bookmark)
		{
			Bookmark->OnCleared();
			Bookmark = nullptr;
		}
	}
}

#if WITH_EDITOR

UBookmarkBase* AWorldSettings::GetDefaultBookmark() const
{
	return DefaultBookmark;
}

void AWorldSettings::SetDefaultBookmark(UBookmarkBase* InDefaultBookmark)
{
	DefaultBookmark = InDefaultBookmark;
}

#endif

void AWorldSettings::AdjustNumberOfBookmarks()
{
	if (MaxNumberOfBookmarks < 0)
	{
		UE_LOGF(LogWorldSettings, Warning, "%ls: MaxNumberOfBookmarks cannot be below 0 (Value=%d). Defaulting to 10", *GetPathNameSafe(this), MaxNumberOfBookmarks);
		MaxNumberOfBookmarks = NumMappedBookmarks;
	}

	if (MaxNumberOfBookmarks < BookmarkArray.Num())
	{
		UE_LOGF(LogWorldSettings, Warning, "%ls: MaxNumberOfBookmarks set below current number of bookmarks. Clearing %d bookmarks.", *GetPathNameSafe(this), BookmarkArray.Num() - MaxNumberOfBookmarks);
	}

	if (MaxNumberOfBookmarks != BookmarkArray.Num())
	{
		Modify();
		BookmarkArray.SetNumZeroed(MaxNumberOfBookmarks);
	}
}

void AWorldSettings::UpdateNumberOfBookmarks()
{
	if (MaxNumberOfBookmarks != BookmarkArray.Num())
	{
		AdjustNumberOfBookmarks();

#if WITH_EDITOR
		if (!HasAnyFlags(RF_NeedInitialization))
		{
			OnNumberOfBoomarksChanged.Broadcast(this);
		}
#endif
	}
}

void AWorldSettings::SanitizeBookmarkClasses()
{
	if (UClass* ExpectedClass = GetDefaultBookmarkClass().Get())
	{
		bool bFoundInvalidBookmarks = false;
		for (int32 i = 0; i < BookmarkArray.Num(); ++i)
		{
			if (TObjectPtr<UBookmarkBase>& Bookmark = BookmarkArray[i])
			{
				if (Bookmark->GetClass() != ExpectedClass)
				{
					Modify();
					Bookmark->OnCleared();
					Bookmark = nullptr;
					bFoundInvalidBookmarks = true;
				}
			}
		}

		if (bFoundInvalidBookmarks)
		{
			UE_LOGF(LogWorldSettings, Warning, "%ls: Bookmarks found with invalid classes", *GetPathNameSafe(this));
		}
	}
	else
	{
		UE_LOGF(LogWorldSettings, Warning, "%ls: Invalid bookmark class, clearing existing bookmarks.", *GetPathNameSafe(this));
		DefaultBookmarkClass = UBookMark::StaticClass();
		SanitizeBookmarkClasses();
	}
}

void AWorldSettings::UpdateBookmarkClass()
{
	if (LastBookmarkClass != DefaultBookmarkClass)
	{

#if WITH_EDITOR
		if (!HasAnyFlags(RF_NeedInitialization))
		{
			OnBookmarkClassChanged.Broadcast(this);
		}
#endif

		// Explicitly done after OnBookmarkClassChanged, in case there's any upgrade work
		// that can be done.
		SanitizeBookmarkClasses();
		
		LastBookmarkClass = DefaultBookmarkClass;
	}
}

FSoftClassPath AWorldSettings::GetAISystemClassName() const
{
	return bEnableAISystem ? FSoftClassPath(AISystemClass.ToString()) : FSoftClassPath();
}

void AWorldSettings::RewindForReplay()
{
	Super::RewindForReplay();

	PauserPlayerState = nullptr;
	TimeDilation = 1.0;
	CinematicTimeDilation = 1.0;
	bWorldGravitySet = false;
	bHighPriorityLoading = false;
}

#undef LOCTEXT_NAMESPACE
