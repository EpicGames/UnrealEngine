// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for all packages containing WorldSetting
struct FWorldSettingCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Deprecated bEnableHierarchicalLODSystem
		DeprecatedEnableHierarchicalLODSystem = 1,

		// DO NOT ADD NEW VERSIONS. THIS SHOULD NOT BE USED
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

	FWorldSettingCustomVersion() = delete;
};
