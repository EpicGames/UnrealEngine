// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "Templates/SharedPointer.h"

struct FNavigationDirtyArea;

class FNavDataGenerator : public TSharedFromThis<FNavDataGenerator, ESPMode::ThreadSafe>
{
public:
	virtual ~FNavDataGenerator() = default;

	/** Rebuilds all known navigation data */
	virtual bool RebuildAll() { return false; };

	/** Blocks until build is complete */
	virtual void EnsureBuildCompletion() {};

	/** Cancels build, may block until current running async tasks are finished */
	virtual void CancelBuild() {};

	/** Ticks navigation build
	 *  If the generator is set to time sliced rebuild then this function will only get called when
	 *  there is sufficient time (effectively roughly once in n frames where n is the number of time sliced nav data / generators currently building)
	 */
	virtual void TickAsyncBuild(float DeltaSeconds) {};
	
	/**  */
	virtual void OnNavigationBoundsChanged() {};

	/** Asks generator to update navigation affected by DirtyAreas */
	virtual void RebuildDirtyAreas(const TArray<FNavigationDirtyArea>& DirtyAreas) {}

	/** determines whether this generator is performing navigation building actions at the moment, dirty areas are also checked */
	virtual bool IsBuildInProgressCheckDirty() const { return false; }

	virtual bool GetTimeSliceData(int32& OutNumRemainingBuildTasks, double& OutCurrentBuildTaskDuration) const { OutNumRemainingBuildTasks = 0; OutCurrentBuildTaskDuration = 0.; return false; }

	/** Returns number of remaining tasks till build is complete
	 */
	virtual int32 GetNumRemaningBuildTasks() const { return 0; };

	/** Returns number of currently running tasks
	 */
	virtual int32 GetNumRunningBuildTasks() const { return 0; };

	//----------------------------------------------------------------------//
	// debug
	//----------------------------------------------------------------------//
	virtual uint32 LogMemUsed() const { return 0; }

#if ENABLE_VISUAL_LOG
	virtual void ExportNavigationData(const FString& FileName) const {}
	virtual void GrabDebugSnapshot(struct FVisualLogEntry* Snapshot, const FBox& BoundingBox, const FName& CategoryName, ELogVerbosity::Type Verbosity) const {}
#endif
};
