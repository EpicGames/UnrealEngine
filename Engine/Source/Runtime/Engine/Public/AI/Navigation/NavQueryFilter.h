// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// For now leave all the real numbers here as floats instead of conversion to FVector::FReal. Its not worth changing the virtual functions and breaking
// any existing code. Its not required for heuristic scale or area costs to be larger or have greater precision than a float.
class INavigationQueryFilterInterface
{
public:
	virtual ~INavigationQueryFilterInterface()= default;

	virtual void Reset() = 0;

	virtual void SetAreaCost(uint8 AreaType, float Cost) = 0;
	virtual void SetFixedAreaEnteringCost(uint8 AreaType, float Cost) = 0;
	virtual void SetExcludedArea(uint8 AreaType) = 0;
	virtual void SetAllAreaCosts(const float* CostArray, const int32 Count) = 0;
	virtual void GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const = 0;
	virtual void SetBacktrackingEnabled(const bool bBacktracking) = 0;
	virtual bool IsBacktrackingEnabled() const = 0;
	virtual float GetHeuristicScale() const = 0;
	virtual bool IsEqual(const INavigationQueryFilterInterface* Other) const = 0;
	virtual void SetIncludeFlags(uint16 Flags) = 0;
	virtual uint16 GetIncludeFlags() const = 0;
	virtual void SetExcludeFlags(uint16 Flags) = 0;
	virtual uint16 GetExcludeFlags() const = 0;
	virtual FVector GetAdjustedEndLocation(const FVector& EndLocation) const { return EndLocation; }

	virtual INavigationQueryFilterInterface* CreateCopy() const = 0;
};

struct FNavigationQueryFilter;
typedef TSharedPtr<FNavigationQueryFilter, ESPMode::ThreadSafe> FSharedNavQueryFilter;
typedef TSharedPtr<const FNavigationQueryFilter, ESPMode::ThreadSafe> FSharedConstNavQueryFilter;

struct FNavigationQueryFilter : public TSharedFromThis<FNavigationQueryFilter, ESPMode::ThreadSafe>
{
	FNavigationQueryFilter() : QueryFilterImpl(NULL), MaxSearchNodes(DefaultMaxSearchNodes) {}
private:
	ENGINE_API FNavigationQueryFilter(const FNavigationQueryFilter& Source);
	ENGINE_API FNavigationQueryFilter(const FNavigationQueryFilter* Source);
	ENGINE_API FNavigationQueryFilter(const FSharedNavQueryFilter Source);
	ENGINE_API FNavigationQueryFilter& operator=(const FNavigationQueryFilter& Source);
public:

	/** set travel cost for area */
	ENGINE_API void SetAreaCost(uint8 AreaType, float Cost);

	/** set entering cost for area */
	ENGINE_API void SetFixedAreaEnteringCost(uint8 AreaType, float Cost);

	/** mark area as excluded from path finding */
	ENGINE_API void SetExcludedArea(uint8 AreaType);

	/** set travel cost for all areas */
	ENGINE_API void SetAllAreaCosts(const TArray<float>& CostArray);
	ENGINE_API void SetAllAreaCosts(const float* CostArray, const int32 Count);

	/** get travel & entering costs for all areas */
	ENGINE_API void GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const;

	/** set required flags of navigation nodes */
	ENGINE_API void SetIncludeFlags(uint16 Flags);

	/** get required flags of navigation nodes */
	ENGINE_API uint16 GetIncludeFlags() const;

	/** set forbidden flags of navigation nodes */
	ENGINE_API void SetExcludeFlags(uint16 Flags);

	/** get forbidden flags of navigation nodes */
	ENGINE_API uint16 GetExcludeFlags() const;

	/** set node limit for A* loop */
	void SetMaxSearchNodes(const uint32 MaxNodes) { MaxSearchNodes = MaxNodes; }

	/** get node limit for A* loop */
	inline uint32 GetMaxSearchNodes() const { return MaxSearchNodes; }

	/** get heuristic scaling factor */
	float GetHeuristicScale() const { return QueryFilterImpl->GetHeuristicScale(); }

	/** mark filter as backtracking - parse directional links in opposite direction
	*  (find path from End to Start, but all links works like on path from Start to End) */
	void SetBacktrackingEnabled(const bool bBacktracking) { QueryFilterImpl->SetBacktrackingEnabled(bBacktracking); }

	/** get backtracking status */
	bool IsBacktrackingEnabled() const { return QueryFilterImpl->IsBacktrackingEnabled(); }

	/** post processing for pathfinding's end point */
	FVector GetAdjustedEndLocation(const FVector& EndPoint) const { return QueryFilterImpl->GetAdjustedEndLocation(EndPoint);  }

	template<typename FilterType>
	void SetFilterType()
	{
		QueryFilterImpl = MakeShareable(new FilterType());
	}

	inline void SetFilterImplementation(const INavigationQueryFilterInterface* InQueryFilterImpl)
	{
		QueryFilterImpl = MakeShareable(InQueryFilterImpl->CreateCopy());
	}

	inline const INavigationQueryFilterInterface* GetImplementation() const { return QueryFilterImpl.Get(); }
	inline INavigationQueryFilterInterface* GetImplementation() { return QueryFilterImpl.Get(); }
	void Reset() { GetImplementation()->Reset(); }

	ENGINE_API FSharedNavQueryFilter GetCopy() const;

	inline bool operator==(const FNavigationQueryFilter& Other) const
	{
		const INavigationQueryFilterInterface* Impl0 = GetImplementation();
		const INavigationQueryFilterInterface* Impl1 = Other.GetImplementation();
		return Impl0 && Impl1 && Impl0->IsEqual(Impl1);
	}

	static ENGINE_API const uint32 DefaultMaxSearchNodes;

protected:
	ENGINE_API void Assign(const FNavigationQueryFilter& Source);

	TSharedPtr<INavigationQueryFilterInterface, ESPMode::ThreadSafe> QueryFilterImpl;
	uint32 MaxSearchNodes;
};

