// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "AI/Navigation/NavigationDataResolution.h"
#include "AI/Navigation/NavigationTypes.h" //FNavDataPerInstanceTransformDelegate

class UBrushComponent;
class UPrimitiveComponent;
class UNavAreaBase;

template<typename InElementType> class TNavStatArray;

struct FNavigationModifier
{
	FNavigationModifier() : bHasMetaAreas(false) {}
	inline bool HasMetaAreas() const { return !!bHasMetaAreas; }

protected:
	/** set to true if any of areas used by this modifier is a meta nav area (UNavArea::IsMetaArea is true)*/
	int32 bHasMetaAreas : 1;
};

namespace ENavigationShapeType
{
	enum Type
	{
		Unknown,
		Cylinder,
		Box,
		Convex,
		InstancedConvex,
	};
}

ENGINE_API FString LexToString(ENavigationShapeType::Type Type);

namespace ENavigationAreaMode
{
	enum Type
	{
		// apply area modifier on all voxels in bounds 
		Apply,
		
		// apply area modifier only on those voxels in bounds that are matching replace area Id
		Replace,

		// apply area modifier on all voxels in bounds, performed during low area prepass (see: ARecastNavMesh.bMarkLowHeightAreas)
		// (ReplaceInLowPass: mark ONLY "low" voxels that will be removed after prepass, ApplyInLowPass: mark all voxels, including "low" ones)
		ApplyInLowPass,

		// apply area modifier only on those voxels in bounds that are matching replace area Id, performed during low area prepass (see: ARecastNavMesh.bMarkLowHeightAreas)
		// (ReplaceInLowPass: mark ONLY "low" voxels that will be removed after prepass, ApplyInLowPass: mark all voxels, including "low" ones)
		ReplaceInLowPass,
	};
}

ENGINE_API FString LexToString(ENavigationAreaMode::Type Type);

namespace ENavigationCoordSystem
{
	enum Type
	{
		Unreal,
		Navigation,

		MAX
	};
}

/** Area modifier: cylinder shape */
struct FCylinderNavAreaData
{
	FVector Origin;
	float Radius;
	float Height;
};

/** Area modifier: box shape (AABB) */
struct FBoxNavAreaData
{
	FVector Origin;
	FVector Extent;
};

struct FConvexNavAreaData
{
	TArray<FVector> Points;
	FVector::FReal MinZ;
	FVector::FReal MaxZ;
};

/** Area modifier: base */
struct FAreaNavModifier : public FNavigationModifier
{
	/** transient value used for navigation modifiers sorting. If < 0 then not set*/
	float Cost;
	float FixedCost;

	ENGINE_API FAreaNavModifier();
	ENGINE_API FAreaNavModifier(float Radius, float Height, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const FVector& Extent, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const FBox& Box, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const TArray<FVector>& Points, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const TConstArrayView<FVector> Points, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const TArray<FVector>& Points, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API FAreaNavModifier(const TNavStatArray<FVector>& Points, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);
	UE_DEPRECATED(5.0, "FAreaNavModifier constructor with a UBrushComponent* parameter has been deprecated since it wasn't able to handle concave shapes. Use FCompositeNavModifier::CreateAreaModifiers instead")
	ENGINE_API FAreaNavModifier(const UBrushComponent* BrushComponent, const TSubclassOf<UNavAreaBase> AreaClass);

	ENGINE_API void InitializePerInstanceConvex(const TNavStatArray<FVector>& Points, const int32 FirstIndex, const int32 LastIndex, const TSubclassOf<UNavAreaBase> AreaClass);
	ENGINE_API void InitializeConvex(const TNavStatArray<FVector>& Points, const int32 FirstIndex, const int32 LastIndex, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass);

	inline const FBox& GetBounds() const { return Bounds; }
	inline ENavigationShapeType::Type GetShapeType() const { return ShapeType; }
	inline ENavigationAreaMode::Type GetApplyMode() const { return ApplyMode; }
	inline bool IsLowAreaModifier() const { return bIsLowAreaModifier; }
	inline bool ShouldExpandTopByCellHeight() const { return bExpandTopByCellHeight; }
	inline bool ShouldIncludeAgentHeight() const { return bIncludeAgentHeight; }
	inline void SetExpandTopByCellHeight(bool bExpand) { bExpandTopByCellHeight = bExpand; }
	inline FAreaNavModifier& SetIncludeAgentHeight(bool bInclude) { bIncludeAgentHeight = bInclude; return *this; }
	inline const TSubclassOf<UNavAreaBase> GetAreaClass() const { return TSubclassOf<UNavAreaBase>(AreaClassOb.Get()); }
	inline const TSubclassOf<UNavAreaBase> GetAreaClassToReplace() const { return TSubclassOf<UNavAreaBase>(ReplaceAreaClassOb.Get()); }

	/** navigation area applied by this modifier */
	ENGINE_API void SetAreaClass(const TSubclassOf<UNavAreaBase> AreaClass);

	/** operation mode, ReplaceInLowPass will always automatically use UNavArea_LowHeight as ReplaceAreaClass! */
	ENGINE_API void SetApplyMode(ENavigationAreaMode::Type InApplyMode);
	
	/** additional class for used by some ApplyModes, setting it will automatically change ApplyMode to keep backwards compatibility! */
	ENGINE_API void SetAreaClassToReplace(const TSubclassOf<UNavAreaBase> AreaClass);

	ENGINE_API void GetCylinder(FCylinderNavAreaData& Data) const;
	ENGINE_API void GetBox(FBoxNavAreaData& Data) const;
	ENGINE_API void GetConvex(FConvexNavAreaData& Data) const;
	ENGINE_API void GetPerInstanceConvex(const FTransform& InLocalToWorld, FConvexNavAreaData& OutData) const;

protected:
	/** this should take a value of a game specific navigation modifier	*/
	TWeakObjectPtr<UClass> AreaClassOb;
	TWeakObjectPtr<UClass> ReplaceAreaClassOb;
	FBox Bounds;
	
	TArray<FVector> Points;
	TEnumAsByte<ENavigationShapeType::Type> ShapeType;
	TEnumAsByte<ENavigationAreaMode::Type> ApplyMode;

	/** if set, area shape will be extended at the top by one cell height */
	uint8 bExpandTopByCellHeight : 1;

	/** if set, area shape will be extended by agent's height to cover area underneath like regular colliding geometry */
	uint8 bIncludeAgentHeight : 1;

	/** set when this modifier affects low spans in navmesh generation step */
	uint8 bIsLowAreaModifier : 1;

	ENGINE_API void Init(const TSubclassOf<UNavAreaBase> InAreaClass);
	/** @param CoordType specifies which coord system the input data is in */
	ENGINE_API void SetConvex(const FVector* InPoints, const int32 FirstIndex, const int32 LastIndex, ENavigationCoordSystem::Type CoordType, const FTransform& LocalToWorld);
	ENGINE_API void SetPerInstanceConvex(const FVector* InPoints, const int32 InFirstIndex, const int32 InLastIndex);
	ENGINE_API void SetBox(const FBox& Box, const FTransform& LocalToWorld);
	
	static ENGINE_API void FillConvexNavAreaData(const FVector* InPoints, const int32 InNumPoints, const FTransform& InLocalToWorld, FConvexNavAreaData& OutConvexData, FBox& OutBounds);
};

/**
 *	This modifier allows defining ad-hoc navigation links defining 
 *	connections in an straightforward way.
 */
struct FSimpleLinkNavModifier : public FNavigationModifier
{
	/** use Set/Append/Add function to update links, they will take care of meta areas */
	TArray<FNavigationLink> Links;
	TArray<FNavigationSegmentLink> SegmentLinks;
	FTransform LocalToWorld;

	UE_DEPRECATED(5.3, "Use NavLinkId instead, this id is no longer used in the engine.")
	int32 UserId;

	FNavLinkId NavLinkId;

	FSimpleLinkNavModifier() 
		: bHasFallDownLinks(false)
		, bHasMetaAreasPoint(false)
		, bHasMetaAreasSegment(false)
	{
	}

	FSimpleLinkNavModifier(const FNavigationLink& InLink, const FTransform& InLocalToWorld) 
		: LocalToWorld(InLocalToWorld)
		, bHasFallDownLinks(false)
		, bHasMetaAreasPoint(false)
		, bHasMetaAreasSegment(false)
	{
		NavLinkId = InLink.NavLinkId;
		AddLink(InLink);
	}	

	FSimpleLinkNavModifier(const TArray<FNavigationLink>& InLinks, const FTransform& InLocalToWorld) 
		: LocalToWorld(InLocalToWorld)
		, bHasFallDownLinks(false)
		, bHasMetaAreasPoint(false)
		, bHasMetaAreasSegment(false)
	{
		if (InLinks.Num() > 0)
		{
			NavLinkId = InLinks[0].NavLinkId;
			SetLinks(InLinks);
		}
	}

	FSimpleLinkNavModifier(const FNavigationSegmentLink& InLink, const FTransform& InLocalToWorld) 
		: LocalToWorld(InLocalToWorld)
		, bHasFallDownLinks(false)
		, bHasMetaAreasPoint(false)
		, bHasMetaAreasSegment(false)
	{
		NavLinkId = InLink.NavLinkId;
		AddSegmentLink(InLink);
	}	

	FSimpleLinkNavModifier(const TArray<FNavigationSegmentLink>& InSegmentLinks, const FTransform& InLocalToWorld) 
		: LocalToWorld(InLocalToWorld)
		, bHasFallDownLinks(false)
		, bHasMetaAreasPoint(false)
		, bHasMetaAreasSegment(false)
	{
		if (InSegmentLinks.Num() > 0)
		{
			NavLinkId = InSegmentLinks[0].NavLinkId;
			SetSegmentLinks(InSegmentLinks);
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSimpleLinkNavModifier(const FSimpleLinkNavModifier&) = default;
	FSimpleLinkNavModifier(FSimpleLinkNavModifier&& Other) = default;
	FSimpleLinkNavModifier& operator=(const FSimpleLinkNavModifier& Other) = default;
	FSimpleLinkNavModifier& operator=(FSimpleLinkNavModifier&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	inline bool HasFallDownLinks() const { return !!bHasFallDownLinks; }
	ENGINE_API void SetLinks(const TArray<FNavigationLink>& InLinks);
	ENGINE_API void SetSegmentLinks(const TArray<FNavigationSegmentLink>& InLinks);
	ENGINE_API void AppendLinks(const TArray<FNavigationLink>& InLinks);
	ENGINE_API void AppendSegmentLinks(const TArray<FNavigationSegmentLink>& InLinks);
	ENGINE_API void AddLink(const FNavigationLink& InLink);
	ENGINE_API void AddSegmentLink(const FNavigationSegmentLink& InLink);
	ENGINE_API void UpdateFlags();

protected:
	/** set to true if any of links stored is a "fall down" link, i.e. requires vertical snapping to geometry */
	int32 bHasFallDownLinks : 1;
	int32 bHasMetaAreasPoint : 1;
	int32 bHasMetaAreasSegment : 1;
};

struct FCustomLinkNavModifier : public FNavigationModifier
{
	FTransform LocalToWorld;

	ENGINE_API void Set(TSubclassOf<UNavLinkDefinition> LinkDefinitionClass, const FTransform& InLocalToWorld);
	inline const TSubclassOf<UNavLinkDefinition> GetNavLinkClass() const { return TSubclassOf<UNavLinkDefinition>(LinkDefinitionClassOb.Get()); }

protected:
	TWeakObjectPtr<UClass> LinkDefinitionClassOb;
};

struct FCollisionShape;

struct FCompositeNavModifier : public FNavigationModifier
{
	FCompositeNavModifier() 
		: bHasPotentialLinks(false)
		, bAdjustHeight(false)
		, bHasLowAreaModifiers(false)
		, bIsPerInstanceModifier(false)
		, bFillCollisionUnderneathForNavmesh(false)
		, bMaskFillCollisionUnderneathForNavmesh(false)
		, bRasterizeAsFilledConvexVolume(false)
		, NavMeshResolution(ENavigationDataResolution::Invalid)
	{}

	ENGINE_API void Shrink();
	ENGINE_API void Reset();
	ENGINE_API void Empty();

	inline bool IsEmpty() const 
	{ 
		return (Areas.Num() == 0) && (SimpleLinks.Num() == 0) && (CustomLinks.Num() == 0) && 
			!bFillCollisionUnderneathForNavmesh && 
			!bMaskFillCollisionUnderneathForNavmesh &&
			!bRasterizeAsFilledConvexVolume &&
			NavMeshResolution == ENavigationDataResolution::Invalid;
	}

	inline bool IsDynamic() const 
	{
		// Excluding bFillCollisionUnderneathForNavmesh, bMaskFillCollisionUnderneathForNavmesh and NavMeshResolution since they require full tile rebuild.
		return !Areas.IsEmpty() || !SimpleLinks.IsEmpty() || !CustomLinks.IsEmpty();
	}
	
	void Add(const FAreaNavModifier& Area)
	{
		Areas.Add(Area);
		bHasMetaAreas |= Area.HasMetaAreas(); 
		bAdjustHeight |= Area.ShouldIncludeAgentHeight();
		bHasLowAreaModifiers |= Area.IsLowAreaModifier();
	}

	void Add(const FSimpleLinkNavModifier& Link)
	{
		SimpleLinks.Add(Link);
		bHasMetaAreas |= Link.HasMetaAreas(); 
	}

	void Add(const FCustomLinkNavModifier& Link)
	{
		CustomLinks.Add(Link); 
		bHasMetaAreas |= Link.HasMetaAreas(); 
	}

	void Add(const FCompositeNavModifier& Modifiers)
	{
		Areas.Append(Modifiers.Areas); 
		SimpleLinks.Append(Modifiers.SimpleLinks); 
		CustomLinks.Append(Modifiers.CustomLinks); 
		bHasMetaAreas |= Modifiers.bHasMetaAreas; 
		bAdjustHeight |= Modifiers.HasAgentHeightAdjust();
		bHasLowAreaModifiers |= Modifiers.HasLowAreaModifiers();
		bFillCollisionUnderneathForNavmesh |= Modifiers.GetFillCollisionUnderneathForNavmesh();
		bMaskFillCollisionUnderneathForNavmesh |= Modifiers.GetMaskFillCollisionUnderneathForNavmesh();
		bRasterizeAsFilledConvexVolume |= Modifiers.GetRasterizeAsFilledConvexVolume();
		if (Modifiers.GetNavMeshResolution() != ENavigationDataResolution::Invalid)
		{
			NavMeshResolution = FMath::Max(NavMeshResolution, Modifiers.GetNavMeshResolution());	// Pick the highest resolution
		}
	}

	ENGINE_API void CreateAreaModifiers(const UPrimitiveComponent* PrimComp, const TSubclassOf<UNavAreaBase> AreaClass, const TSubclassOf<UNavAreaBase> AreaClassToReplace = {});
	ENGINE_API void CreateAreaModifiers(const FCollisionShape& CollisionShape, const FTransform& LocalToWorld, const TSubclassOf<UNavAreaBase> AreaClass, const bool bIncludeAgentHeight = false);

	inline const TArray<FAreaNavModifier>& GetAreas() const { return Areas; }
	inline const TArray<FSimpleLinkNavModifier>& GetSimpleLinks() const { return SimpleLinks; }
	inline const TArray<FCustomLinkNavModifier>& GetCustomLinks() const { return CustomLinks; }
	
	inline bool HasLinks() const { return (SimpleLinks.Num() > 0) || (CustomLinks.Num() > 0); }
	inline bool HasPotentialLinks() const { return bHasPotentialLinks; }
	inline bool HasAgentHeightAdjust() const { return bAdjustHeight; }
	inline bool HasAreas() const { return Areas.Num() > 0; }
	inline bool HasLowAreaModifiers() const { return bHasLowAreaModifiers; }
	inline bool IsPerInstanceModifier() const { return bIsPerInstanceModifier; }
	inline bool GetFillCollisionUnderneathForNavmesh() const { return bFillCollisionUnderneathForNavmesh; }
	inline void SetFillCollisionUnderneathForNavmesh(bool bValue) { bFillCollisionUnderneathForNavmesh = bValue; }
	inline bool GetMaskFillCollisionUnderneathForNavmesh() const { return bMaskFillCollisionUnderneathForNavmesh; }
	inline void SetMaskFillCollisionUnderneathForNavmesh(bool bValue) { bMaskFillCollisionUnderneathForNavmesh = bValue; }
	inline bool GetRasterizeAsFilledConvexVolume() const { return bRasterizeAsFilledConvexVolume; }
	inline void SetRasterizeAsFilledConvexVolume(bool bValue) { bRasterizeAsFilledConvexVolume = bValue; }
	inline ENavigationDataResolution GetNavMeshResolution() const { return bEnableNavMeshResolutions ? NavMeshResolution : ENavigationDataResolution::Invalid; }
	inline void SetNavMeshResolution(ENavigationDataResolution Resolution) { NavMeshResolution = Resolution; }
	inline void ReserveForAdditionalAreas(int32 AdditionalElementsCount) { Areas.Reserve(Areas.Num() + AdditionalElementsCount); }

	void MarkPotentialLinks() { bHasPotentialLinks = true; }
    void MarkAsPerInstanceModifier() { bIsPerInstanceModifier = true; }

	/** returns a copy of Modifier */
	ENGINE_API FCompositeNavModifier GetInstantiatedMetaModifier(const FNavAgentProperties* NavAgent, const TWeakObjectPtr<const UObject>& WeakOwnerPtr) const;
	UE_DEPRECATED(5.5, "Use the version taking a const reference instead.")
	ENGINE_API FCompositeNavModifier GetInstantiatedMetaModifier(const FNavAgentProperties* NavAgent, TWeakObjectPtr<UObject> WeakOwnerPtr) const;

	ENGINE_API uint32 GetAllocatedSize() const;

	TArray<FAreaNavModifier>& GetMutableAreas() { return Areas; }
	TArray<FSimpleLinkNavModifier>& GetSimpleLinks() { return SimpleLinks; }
	TArray<FCustomLinkNavModifier>& GetCustomLinks() { return CustomLinks; }

public:

	// This property is deprecated and will be removed in future versions. Use FNavigationRelevantData NavDataPerInstanceTransformDelegate instead.
	// Gathers per instance data for navigation area modifiers in a specified area box
	FNavDataPerInstanceTransformDelegate NavDataPerInstanceTransformDelegate;

	static ENGINE_API bool bEnableNavMeshResolutions;
	
private:
	TArray<FAreaNavModifier> Areas;
	TArray<FSimpleLinkNavModifier> SimpleLinks;
	TArray<FCustomLinkNavModifier> CustomLinks;
	uint32 bHasPotentialLinks : 1;
	uint32 bAdjustHeight : 1;
	uint32 bHasLowAreaModifiers : 1;
    uint32 bIsPerInstanceModifier : 1;
	uint32 bFillCollisionUnderneathForNavmesh : 1;
	uint32 bMaskFillCollisionUnderneathForNavmesh : 1;
	uint32 bRasterizeAsFilledConvexVolume : 1;
	ENavigationDataResolution NavMeshResolution;
};
