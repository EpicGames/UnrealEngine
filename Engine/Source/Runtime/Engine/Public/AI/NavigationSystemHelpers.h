// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Chaos/Core.h"

class UBodySetup;
struct FCompositeNavModifier;
struct FKAggregateGeom;
struct FKConvexElem;
struct FNavHeightfieldSamples;

template<typename InElementType> class TNavStatArray;

namespace Chaos
{
	class FHeightField;
	class FTriangleMeshImplicitObject;
}

struct FNavigableGeometryExport
{
	virtual ~FNavigableGeometryExport() = default;

	virtual void ExportChaosTriMesh(const Chaos::FTriangleMeshImplicitObject* const TriMesh, const FTransform& LocalToWorld) = 0;
	virtual void ExportChaosConvexMesh(const FKConvexElem* const Convex, const FTransform& LocalToWorld) = 0;
	virtual void ExportChaosHeightField(const Chaos::FHeightField* const Heightfield, const FTransform& LocalToWorld) = 0;
	virtual void ExportChaosHeightFieldSlice(const FNavHeightfieldSamples& PrefetchedHeightfieldSamples, const int32 NumRows, const int32 NumCols, const FTransform& LocalToWorld, const FBox& SliceBox) = 0;
	virtual void ExportRigidBodySetup(UBodySetup& BodySetup, const FTransform& LocalToWorld) = 0;
	virtual void ExportCustomMesh(const FVector* VertexBuffer, int32 NumVerts, const int32* IndexBuffer, int32 NumIndices, const FTransform& LocalToWorld) = 0;
	
	virtual void AddNavModifiers(const FCompositeNavModifier& Modifiers) = 0;

	// Optional delegate for geometry per instance transforms
	virtual void SetNavDataPerInstanceTransformDelegate(const FNavDataPerInstanceTransformDelegate& InDelegate) = 0;
};
