// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// IRVOAvoidanceInterface is an interface for objects that want to perform
// RVO avoidance. See UCharacterMovementComponent and UWheeledVehicleMovementComponent
// for example implementations.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "RVOAvoidanceInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class URVOAvoidanceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IRVOAvoidanceInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Store the AvoidanceUID generated by the Avoidance Manager **/
	virtual void SetRVOAvoidanceUID(int32 UID) = 0;

	/** Return the AvoidanceUID assigned by the Avoidance Manager during registration **/
	virtual int32 GetRVOAvoidanceUID() = 0;

	/** Store the AvoidanceWeight generated by the Avoidance Manager **/
	virtual void SetRVOAvoidanceWeight(float Weight) = 0;

	/** Returns the AvoidanceWeight assigned by the Avoidance Manager during registration **/
	virtual float GetRVOAvoidanceWeight() = 0;

	/** Get the Location from where the RVO avoidance should originate **/
	virtual FVector GetRVOAvoidanceOrigin() = 0;

	/** The scaled collider radius to consider for RVO avoidance **/
	virtual float GetRVOAvoidanceRadius() = 0;

	/** The scaled collider height to consider for RVO avoidance **/
	virtual float GetRVOAvoidanceHeight() = 0;

	/** The scaled collider radius to consider for RVO avoidance **/
	virtual float GetRVOAvoidanceConsiderationRadius() = 0;

	/** The velocity of the avoiding entity **/
	virtual FVector GetVelocityForRVOConsideration() = 0;

	/** Store the avoidance group mask **/
	virtual void SetAvoidanceGroupMask(int32 GroupFlags) = 0;

	/** This actor's avoidance group mask **/
	virtual int32 GetAvoidanceGroupMask() = 0;

	/** Store the groups to avoid mask **/
	virtual void SetGroupsToAvoidMask(int32 GroupFlags) = 0;

	/** Agent groups to avoid mask **/
	virtual int32 GetGroupsToAvoidMask() = 0;

	/** Store the groups to ignore mask **/
	virtual void SetGroupsToIgnoreMask(int32 GroupFlags) = 0;

	/** Agent groups to ignore **/
	virtual int32 GetGroupsToIgnoreMask() = 0;
};
