// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldCollision.h"
#include "Sound/SoundAttenuation.h"
#include "HAL/ThreadSafeBool.h"
#include "Audio.h"
#include "Audio/AudioDebug.h"
#include "AudioDynamicParameter.h"
#include "Components/AudioComponent.h"
#include "DSP/VolumeFader.h"
#include "IAudioExtensionPlugin.h"
#include "Sound/AudioVolume.h"
#include "Sound/SoundSourceBus.h"
#include "Sound/QuartzQuantizationUtilities.h"

class FAudioDevice;
class USoundBase;
class USoundSubmix;
class USoundSourceBus;
struct FSoundSubmixSendInfo;
struct FSoundSourceBusSendInfo;
struct FWaveInstance;
class USoundWave;
class USoundConcurrency;
struct FListener;
struct FAttenuationListenerData;

/**
 * Attenuation focus system data computed per update per active sound
 */
struct FAttenuationFocusData
{
	/** Azimuth of the active sound relative to the listener. Used by sound  focus. */
	float Azimuth = 0.0f;

	/** Absolute azimuth of the active sound relative to the listener. Used for 3d audio calculations. */
	float AbsoluteAzimuth = 0.0f;

	/** Value used to allow smooth interpolation in/out of focus */
	float FocusFactor = 1.0f;

	/** Cached calculation of the amount distance is scaled due to focus */
	float DistanceScale = 1.0f;

	/** The amount priority is scaled due to focus */
	float PriorityScale = 1.0f;

	/** Cached highest priority of the parent active sound's wave instances. */
	float PriorityHighest = 1.0f;

	/** The amount volume is scaled due to focus */
	float VolumeScale = 1.0f;

	/** If this is the first update for focus. Handles edge case of starting a sound in-focus or out-of-focus. */
	bool bFirstFocusUpdate = true;
};

/**
 *	Struct used for gathering the final parameters to apply to a wave instance
 */
struct FSoundParseParameters
{
	// A collection of finish notification hooks
	FNotifyBufferFinishedHooks NotifyBufferFinishedHooks;

	// The Sound Class to use the settings of
	USoundClass* SoundClass;

	// The transform of the sound (scale is not used)
	FTransform Transform;

	// The speed that the sound is moving relative to the listener
	FVector Velocity;

	// The volume product of the sound
	float Volume;

	// The attenuation of the sound due to distance attenuation
	float DistanceAttenuation;

	// The attenuation of the sound due to occlusion attenuation
	float OcclusionAttenuation;

	// A volume scale on the sound specified by user
	float VolumeMultiplier;

	// Attack time of the source envelope follower
	int32 EnvelopeFollowerAttackTime;

	// Release time of the source envelope follower
	int32 EnvelopeFollowerReleaseTime;

	// The multiplier to apply if the sound class desires
	float InteriorVolumeMultiplier;

	// The priority of sound, which is the product of the component priority and the USoundBased priority
	float Priority;

	// The pitch scale factor of the sound
	float Pitch;

	// Time offset from beginning of sound to start at
	float StartTime;

	// At what distance from the source of the sound should spatialization begin 
	float NonSpatializedRadiusStart;

	// At what distance from the source the sound is fully non-spatialized
	float NonSpatializedRadiusEnd;

	// Which mode to use for non-spatialized radius
	ENonSpatializedRadiusSpeakerMapMode NonSpatializedRadiusMode;

	// The distance over which the sound is attenuated
	float AttenuationDistance;

	// The distance from the listener to the sound
	float ListenerToSoundDistance;

	// The distance from the listener to the sound (ignores attenuation settings)
	float ListenerToSoundDistanceForPanning;

	// The absolute azimuth angle of the sound relative to the forward listener vector (359 degrees to left, 1 degrees to right)
	float AbsoluteAzimuth;

	// The sound submix to use for the wave instance
	USoundSubmixBase* SoundSubmix;

	// The submix sends. 
	TArray<FSoundSubmixSendInfo> SoundSubmixSends;
	TArray<FAttenuationSubmixSendSettings> AttenuationSubmixSends;

	// The source bus sends to use
	TArray<FSoundSourceBusSendInfo> BusSends[(int32)EBusSendType::Count];

	// Reverb wet-level parameters
	EReverbSendMethod ReverbSendMethod;
	FVector2D ReverbSendLevelRange;
	FVector2D ReverbSendLevelDistanceRange;
	float ManualReverbSendLevel;
	FRuntimeFloatCurve CustomReverbSendCurve;

	// The distance between left and right channels when spatializing stereo assets
	float StereoSpread;

	// Which spatialization algorithm to use
	ESoundSpatializationAlgorithm SpatializationMethod;

	// Whether the spatialization plugin is an external send
	bool bSpatializationIsExternalSend;

	// What occlusion plugin source settings to use
	USpatializationPluginSourceSettingsBase* SpatializationPluginSettings;

	// What occlusion plugin source settings to use
	UOcclusionPluginSourceSettingsBase* OcclusionPluginSettings;

	// What reverb plugin source settings to use
	UReverbPluginSourceSettingsBase* ReverbPluginSettings;

	// What source data override plugin source settings to use
	USourceDataOverridePluginSourceSettingsBase* SourceDataOverridePluginSettings;

	// If using AudioLink, this allows the settings to be overriden.
	UAudioLinkSettingsAbstract* AudioLinkSettingsOverride = nullptr;

	// What source effect chain to use
	USoundEffectSourcePresetChain* SourceEffectChain;

	// The lowpass filter frequency to apply (if enabled)
	float LowPassFilterFrequency;
	
	// The highpass filter frequency to apply (if enabled)
	float HighPassFilterFrequency;

	// The lowpass filter frequency to apply due to distance attenuation
	float AttenuationLowpassFilterFrequency;

	// The highpass filter frequency to apply due to distance attenuation
	float AttenuationHighpassFilterFrequency;

	// The lowpass filter to apply if the sound is occluded
	float OcclusionFilterFrequency;

	// The lowpass filter to apply if the sound is inside an ambient zone
	float AmbientZoneFilterFrequency;

	/** Whether or not to enable sending this audio's output to buses.*/
	uint32 bEnableBusSends : 1;

	/** Whether or not to render to the main submix */
	uint32 bEnableBaseSubmix : 1;

	/** Whether or not to enable Submix Sends in addition to the Main Submix*/
	uint32 bEnableSubmixSends : 1;

	uint32 bEnableSourceDataOverride : 1;

	uint32 bEnableSendToAudioLink : 1;

	// Whether the sound should be spatialized
	uint8 bUseSpatialization:1;

	// Whether the sound should be seamlessly looped
	uint8 bLooping:1;

	// Whether we have enabled low-pass filtering of this sound
	uint8 bEnableLowPassFilter:1;
	
	// Whether we have enabled high-pass filtering of this sound
	uint8 bEnableHighPassFilter:1;

	// Whether this sound is occluded
	uint8 bIsOccluded:1;

	// Whether or not this sound is manually paused (i.e. not by application-wide pause)
	uint8 bIsPaused:1;

	// Whether or not this sound can re-trigger
	uint8 bEnableRetrigger : 1;

	// Whether or not to apply a =6 dB attenuation to stereo spatialization sounds
	uint8 bApplyNormalizationToStereoSounds:1;

	FSoundParseParameters()
		: SoundClass(nullptr)
		, Velocity(ForceInit)
		, Volume(1.f)
		, DistanceAttenuation(1.f)
		, OcclusionAttenuation(1.f)
		, VolumeMultiplier(1.f)
		, EnvelopeFollowerAttackTime(10)
		, EnvelopeFollowerReleaseTime(100)
		, InteriorVolumeMultiplier(1.f)
		, Pitch(1.f)
		, StartTime(-1.f)
		, NonSpatializedRadiusStart(0.0f)
		, NonSpatializedRadiusEnd(0.0f)
		, NonSpatializedRadiusMode(ENonSpatializedRadiusSpeakerMapMode::OmniDirectional)
		, AttenuationDistance(0.0f)
		, ListenerToSoundDistance(0.0f)
		, ListenerToSoundDistanceForPanning(0.0f)
		, AbsoluteAzimuth(0.0f)
		, SoundSubmix(nullptr)
		, ReverbSendMethod(EReverbSendMethod::Linear)
		, ReverbSendLevelRange(0.0f, 0.0f)
		, ReverbSendLevelDistanceRange(0.0f, 0.0f)
		, ManualReverbSendLevel(0.0f)
		, StereoSpread(0.0f)
		, SpatializationMethod(ESoundSpatializationAlgorithm::SPATIALIZATION_Default)
		, bSpatializationIsExternalSend(false)
		, SpatializationPluginSettings(nullptr)
		, OcclusionPluginSettings(nullptr)
		, ReverbPluginSettings(nullptr)
		, SourceDataOverridePluginSettings(nullptr)
		, AudioLinkSettingsOverride(nullptr)
		, SourceEffectChain(nullptr)
		, LowPassFilterFrequency(MAX_FILTER_FREQUENCY)
		, HighPassFilterFrequency(MIN_FILTER_FREQUENCY)
		, AttenuationLowpassFilterFrequency(MAX_FILTER_FREQUENCY)
		, AttenuationHighpassFilterFrequency(MIN_FILTER_FREQUENCY)
		, OcclusionFilterFrequency(MAX_FILTER_FREQUENCY)
		, AmbientZoneFilterFrequency(MAX_FILTER_FREQUENCY)
		, bEnableBusSends(false)
		, bEnableBaseSubmix(false)
		, bEnableSubmixSends(false)
		, bEnableSourceDataOverride(false)
		, bUseSpatialization(false)
		, bLooping(false)
		, bEnableLowPassFilter(false)
		, bEnableHighPassFilter(false)
		, bIsOccluded(false)
		, bIsPaused(false)
		, bEnableRetrigger(false)
		, bApplyNormalizationToStereoSounds(false)
	{
	}
};

struct FActiveSound : public ISoundModulatable
{
public:

	ENGINE_API FActiveSound();
	ENGINE_API FActiveSound(const FActiveSound&);
	ENGINE_API ~FActiveSound();

	static ENGINE_API FActiveSound* CreateVirtualCopy(const FActiveSound& ActiveSoundToCopy, FAudioDevice& AudioDevice);

private:
	TWeakObjectPtr<UWorld> World;
	uint32 WorldID;

	TObjectPtr<USoundBase> Sound;
	TObjectPtr<USoundEffectSourcePresetChain> SourceEffectChain;
	TObjectPtr<USoundAttenuation> SoundAttenuation;

	uint64 AudioComponentID;
	FName AudioComponentUserID;
	uint32 OwnerID;

	FName AudioComponentName;
	FName OwnerName;

	FObjectKey OwnerKey;

	uint32 PlayOrder;

public:
	uint32 GetObjectId() const override { return Sound ? Sound->GetUniqueID() : INDEX_NONE; }
	ENGINE_API int32 GetPlayCount() const override;
	uint32 GetPlayOrder() const { return PlayOrder; }
	bool IsPreviewSound() const override { return bIsPreviewSound; }
	ENGINE_API void Stop() override;

	/** Returns a unique identifier for this active sound object */
	uint32 GetInstanceID() const { return PlayOrder; }

	uint64 GetAudioComponentID() const { return AudioComponentID; }
	FName GetAudioComponentUserID() const { return AudioComponentUserID; }
	ENGINE_API void ClearAudioComponent();
	ENGINE_API void SetAudioComponent(const FActiveSound& ActiveSound);
	ENGINE_API void SetAudioComponent(const UAudioComponent& Component);
	ENGINE_API void SetOwner(const AActor* Owner);
	ENGINE_API FString GetAudioComponentName() const;
	ENGINE_API FString GetOwnerName() const;

	uint32 GetWorldID() const { return WorldID; }
	TWeakObjectPtr<UWorld> GetWeakWorld() const { return World; }
	UWorld* GetWorld() const
	{
		return World.Get();
	}
	ENGINE_API void SetWorld(UWorld* World);

	ENGINE_API void SetPitch(float Value);
	ENGINE_API void SetVolume(float Value);

	float GetPitch() const { return PitchMultiplier; }

	/** Gets volume product all gain stages pertaining to active sound */
	ENGINE_API float GetVolume() const;

	USoundBase* GetSound() const { return Sound; }
	ENGINE_API void SetSound(USoundBase* InSound);

	USoundEffectSourcePresetChain* GetSourceEffectChain() const { return SourceEffectChain ? ToRawPtr(SourceEffectChain) : ToRawPtr(Sound->SourceEffectChain); }
	
	ENGINE_API void SetSourceEffectChain(USoundEffectSourcePresetChain* InSourceEffectChain);

	ENGINE_API void SetSoundClass(USoundClass* SoundClass);

	ENGINE_API void SetAttenuationSettingsAsset(TObjectPtr<USoundAttenuation> InSoundAttenuation);

	ENGINE_API void SetAttenuationSettingsOverride(bool bInIsAttenuationSettingsOverridden);

	void SetAudioDevice(FAudioDevice* InAudioDevice)
	{
		AudioDevice = InAudioDevice;
	}

	void SetSourceListener(FSharedISourceBufferListenerPtr InListener, bool bShouldZeroBuffer)
	{
		SourceBufferListener = InListener;
		bShouldSourceBufferListenerZeroBuffer = bShouldZeroBuffer;
	}

	ENGINE_API int32 GetClosestListenerIndex() const;

	/** Returns whether or not the active sound can be deleted. */
	bool CanDelete() const { return !bAsyncOcclusionPending; }

	/** Whether or not the active sound is a looping sound. */
	bool IsLooping() const { return Sound && Sound->IsLooping(); }

	/** Whether or not the active sound a one-shot sound. */
	bool IsOneShot() const { return !IsLooping(); }

	/** Whether or not the active sound is currently playing audible sound. */
	bool IsPlayingAudio() const { return bIsPlayingAudio; }

	/** Whether or not sound reference is valid and set to play when silent. */
	ENGINE_API bool IsPlayWhenSilent() const;

	/** The priority used for concurrency calculations */
	float GetConcurrencyPriority() const;

	EVirtualizationMode GetVirtualizationMode() const;

	/** Override the Virtualization settings set on Audio Component/Sound */
	void OverrideVirtualizationMode(EVirtualizationMode Mode);

	/** Stop using the Virtualization Mode Override */
	void ClearVirtualizationModeOverride();

	FAudioDevice* AudioDevice;

	/** The concurrent groups that this sound is actively playing in. */
	TMap<FConcurrencyGroupID, FConcurrencySoundData> ConcurrencyGroupData;

	/** Optional USoundConcurrency to override for the sound. */
	TSet<TObjectPtr<USoundConcurrency>> ConcurrencySet;

private:
	/** Optional SoundClass to override for the sound. */
	TObjectPtr<USoundClass> SoundClassOverride;

	/** Optional override the submix sends for the sound. */
	TArray<FSoundSubmixSendInfo> SoundSubmixSendsOverride;

	/** Optional override for the source bus sends for the sound. */
	TArray<FSoundSourceBusSendInfo> BusSendsOverride[(int32)EBusSendType::Count];

	TMap<UPTRINT, FWaveInstance*> WaveInstances;

	TSharedPtr<Audio::IParameterTransmitter> InstanceTransmitter;

	EVirtualizationMode VirtualizationModeOverride;

	uint8 bEnableVirtualizationModeOverride: 1;

public:
	Audio::IParameterTransmitter* GetTransmitter()
	{
		return InstanceTransmitter.Get();
	}

	const Audio::IParameterTransmitter* GetTransmitter() const
	{
		return InstanceTransmitter.Get();
	}

	void ClearTransmitter()
	{
		return InstanceTransmitter.Reset();
	}

	enum class EFadeOut : uint8
	{
		// Sound is not currently fading out
		None,

		// Client code (eg. AudioComponent) is requesting a fade out
		User,

		// The concurrency system is requesting a fade due to voice stealing
		Concurrency
	};

	/** Whether or not the sound has checked if it was occluded already. Used to initialize a sound as occluded and bypassing occlusion interpolation. */
	uint8 bHasCheckedOcclusion:1;

	/** Is this sound allowed to be spatialized? */
	uint8 bAllowSpatialization:1;

	/** Does this sound have attenuation settings specified. */
	uint8 bHasAttenuationSettings:1;

	/** Whether the wave instances should remain active if they're dropped by the prioritization code. Useful for e.g. vehicle sounds that shouldn't cut out. */
	uint8 bShouldRemainActiveIfDropped:1;

	/** Whether the current component has finished playing */
	uint8 bFinished:1;

	/** Whether or not the active sound is paused. Independently set vs global pause or unpause. */
	uint8 bIsPaused:1;

	/** Whether or not to stop this active sound due to max concurrency */
	uint8 bShouldStopDueToMaxConcurrency:1;

	/** Whether or not sound has been virtualized and then realized */
	uint8 bHasVirtualized:1;

	/** If true, the decision on whether to apply the radio filter has been made. */
	uint8 bRadioFilterSelected:1;

	/** If true, this sound will not be stopped when flushing the audio device. */
	uint8 bApplyRadioFilter:1;

	/** If true, the AudioComponent will be notified when a Wave is started to handle subtitles */
	uint8 bHandleSubtitles:1;

	/** If true, subtitles are being provided for the sound externally, so it still needs to make sure the sound plays to trigger the subtitles. */
	uint8 bHasExternalSubtitles:1;

	/** Whether the Location of the component is well defined */
	uint8 bLocationDefined:1;

	/** If true, this sound will not be stopped when flushing the audio device. */
	uint8 bIgnoreForFlushing:1;

	/** Whether to artificially prioritize the component to play */
	uint8 bAlwaysPlay:1;

	/** Whether or not this sound plays when the game is paused in the UI */
	uint8 bIsUISound:1;

	/** Whether or not this audio component is a music clip */
	uint8 bIsMusic:1;

	/** Whether or not the audio component should be excluded from reverb EQ processing */
	uint8 bReverb:1;

	/** Whether or not this sound class forces sounds to the center channel */
	uint8 bCenterChannelOnly:1;

	/** Whether or not this active sound is a preview sound */
	uint8 bIsPreviewSound:1;

#if WITH_EDITORONLY_DATA
	/** Whether or not this sound uses submix sends if it is a preview sound */
	uint8 bIsDryPreviewPlayback : 1;
#endif //WITH_EDITORONLY_DATA

	/** Whether we have queried for the interior settings at least once */
	uint8 bGotInteriorSettings:1;

	/** Whether some part of this sound will want interior sounds to be applied */
	uint8 bApplyInteriorVolumes:1;

#if !(NO_LOGGING || UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** For debugging purposes, output to the log once that a looping sound has been orphaned */
	uint8 bWarnedAboutOrphanedLooping:1;
#endif

	/** Whether or not we have a low-pass filter enabled on this active sound. */
	uint8 bEnableLowPassFilter : 1;
	
	/** Whether or not we have a low-pass filter enabled on this active sound. */
	uint8 bEnableHighPassFilter : 1;

	/** Whether or not this active sound will update play percentage. Based on set delegates on audio component. */
	uint8 bUpdatePlayPercentage:1;

	/** Whether or not this active sound will update the envelope value of every wave instance that plays a sound source. Based on set delegates on audio component. */
	uint8 bUpdateSingleEnvelopeValue:1;

	/** Whether or not this active sound will update the average envelope value of every wave instance that plays a sound source. Based on set delegates on audio component. */
	uint8 bUpdateMultiEnvelopeValue:1;

	/** Whether or not the active sound should update it's owning audio component's playback time. */
	uint8 bUpdatePlaybackTime:1;

	/** Whether or not this active sound is playing audio, as in making audible sounds. */
	uint8 bIsPlayingAudio:1;

	/** Whether or not the active sound is stopping. */
	uint8 bIsStopping:1;

	/** Whether or not we are overriding the routing enablement options on sounds. */
	uint8 bHasActiveBusSendRoutingOverride : 1;
	uint8 bHasActiveMainSubmixOutputOverride : 1;
	uint8 bHasActiveSubmixSendRoutingOverride : 1;

	/** What the value of the enablement overrides are. */
	uint8 bEnableBusSendRoutingOverride : 1;
	uint8 bEnableMainSubmixOutputOverride : 1;
	uint8 bEnableSubmixSendRoutingOverride : 1;

	uint8 bIsFirstAttenuationUpdate : 1;
	uint8 bStartedWithinNonBinauralRadius : 1;

	uint8 bModulationRoutingUpdated : 1;

	/** If this is true the active sound uses the overridden struct of the sound not the attenuation settings asset. */
	uint8 bIsAttenuationSettingsOverridden : 1;

	uint8 UserIndex;

	/** Type of fade out currently being applied */
	EFadeOut FadeOut;

	/** whether we were occluded the last time we checked */
	FThreadSafeBool bIsOccluded;

	/** Whether or not there is an async occlusion trace pending */
	FThreadSafeBool bAsyncOcclusionPending;

	/** Duration between now and when the sound has been started. */
	float PlaybackTime;

	/** Unscaled duration between now and when the sound has been started. This value is not scaled by the playback rate of the sound */
	float PlaybackTimeUnscaled;

	/** If virtualized, duration between last time virtualized and now. */
	float PlaybackTimeNonVirtualized;

	float MinCurrentPitch;
	float RequestedStartTime;

	TSortedMap<UPTRINT, float> SeekRestartTimes;
	
	float VolumeMultiplier;
	float PitchMultiplier;

	/** The low-pass filter frequency to apply if bEnableLowPassFilter is true. */
	float LowPassFilterFrequency;
	
	/** The high-pass filter frequency to apply if bEnableHighPassFilter is true. */
	float HighPassFilterFrequency;

	/** Fader that tracks component volume */
	Audio::FVolumeFader ComponentVolumeFader;

	/** The interpolated parameter for the low-pass frequency due to occlusion. */
	FDynamicParameter CurrentOcclusionFilterFrequency;

	/** The interpolated parameter for the volume attenuation due to occlusion. */
	FDynamicParameter CurrentOcclusionVolumeAttenuation;

	float SubtitlePriority;
	
	/** The product of the component priority and the USoundBase priority */
	float Priority;

	/** The volume used to determine concurrency resolution for "quietest" active sound.
	// If negative, tracking is disabled for lifetime of ActiveSound */
	float VolumeConcurrency;

	/** The source's final mixed Modulation value, used to evaluate VolumeConcurrency */
	float MaxSourceModulationValue;
	
	/** The time in seconds with which to check for occlusion from its closest listener */
	float OcclusionCheckInterval;

	/** Last time we checked for occlusion */
	float LastOcclusionCheckTime;

	/** The max distance this sound will be audible. */
	float MaxDistance;

	FTransform Transform;

	/**
	 * Cached data pertaining to focus system updated each frame
	 */
	FAttenuationFocusData FocusData;

	/** Location last time playback was updated */
	FVector LastLocation;

	FSoundAttenuationSettings AttenuationSettings;

	/** Quantization information */
	Audio::FQuartzQuantizedRequestData QuantizedRequestData;

	/** Source buffer listener */
	FSharedISourceBufferListenerPtr SourceBufferListener;
	bool bShouldSourceBufferListenerZeroBuffer = false;

	/** Cache what volume settings we had last time so we don't have to search again if we didn't move */
	FInteriorSettings InteriorSettings;
	TArray<FAudioVolumeSubmixSendSettings> AudioVolumeSubmixSendSettings;
	TArray<FAudioVolumeSubmixSendSettings> PreviousAudioVolumeSubmixSendSettings;

	uint32 AudioVolumeID;

	// To remember where the volumes are interpolating to and from
	double LastUpdateTime;
	float SourceInteriorVolume;
	float SourceInteriorLPF;
	float CurrentInteriorVolume;
	float CurrentInteriorLPF;

	// Envelope follower attack and release time parameters
	int32 EnvelopeFollowerAttackTime;
	int32 EnvelopeFollowerReleaseTime;

	TMap<UPTRINT,uint32> SoundNodeOffsetMap;
	TArray<uint8> SoundNodeData;

	// Whether or not there are Source Bus Sends that have not been sent to the render thread
	bool bHasNewBusSends;

	// Bus send(s) that have not yet been sent to the render thread
	TArray<TTuple<EBusSendType, FSoundSourceBusSendInfo>> NewBusSends;

	FSoundModulationDefaultRoutingSettings ModulationRouting;

#if ENABLE_AUDIO_DEBUG
	FColor DebugColor;
#endif // ENABLE_AUDIO_DEBUG

	ENGINE_API void UpdateInterfaceParameters(const TArray<FListener>& InListeners);

	// Updates the wave instances to be played.
	ENGINE_API void UpdateWaveInstances(TArray<FWaveInstance*> &OutWaveInstances, const float DeltaTime);

	/**
	 * Find an existing waveinstance attached to this audio component (if any)
	 */
	ENGINE_API FWaveInstance* FindWaveInstance(const UPTRINT WaveInstanceHash);

	ENGINE_API void RemoveWaveInstance(const UPTRINT WaveInstanceHash);

	const TMap<UPTRINT, FWaveInstance*>& GetWaveInstances() const
	{
		return WaveInstances;
	}

	/**
	 * Add newly created wave instance to active sound
	 */
	ENGINE_API FWaveInstance& AddWaveInstance(const UPTRINT WaveInstanceHash);

	/**
	 * Check whether to apply the radio filter
	 */
	ENGINE_API void ApplyRadioFilter(const FSoundParseParameters& ParseParams);

	/** Gets total concurrency gain stage based on all concurrency memberships of sound */
	ENGINE_API float GetTotalConcurrencyVolumeScale() const;

	ENGINE_API void CollectAttenuationShapesForVisualization(TMultiMap<EAttenuationShape::Type, FBaseAttenuationSettings::AttenuationShapeDetails>& ShapeDetailsMap) const;

	/**
	 * Friend archive function used for serialization.
	 */
	friend FArchive& operator<<( FArchive& Ar, FActiveSound* ActiveSound );

	ENGINE_API void AddReferencedObjects( FReferenceCollector& Collector );

	/**
	 * Get the sound class to apply on this sound instance
	 */
	ENGINE_API USoundClass* GetSoundClass() const;

	/**
	* Get the sound submix to use for this sound instance
	*/
	ENGINE_API USoundSubmixBase* GetSoundSubmix() const;

	/** Gets the sound submix sends to use for this sound instance. */
	ENGINE_API void GetSoundSubmixSends(TArray<FSoundSubmixSendInfo>& OutSends) const;

	/** Gets the sound source bus sends to use for this sound instance. */
	ENGINE_API void GetBusSends(EBusSendType BusSendType, TArray<FSoundSourceBusSendInfo>& OutSends) const;

	/**
	 * Checks whether there are Source Bus Sends that have not yet been updated
	 * @return true when there are new Source Bus Sends, false otherwise
	 */
	ENGINE_API bool HasNewBusSends() const;

	/** Lets the audio thread know if additional Source Bus Send information has been added 
	*
	*  @return the array of Sound Bus Sends that have not yet been added to the render thread
	*/
	ENGINE_API TArray< TTuple<EBusSendType, FSoundSourceBusSendInfo> > const & GetNewBusSends() const;

	/** Resets internal data of new Source Bus Sends */
	ENGINE_API void ResetNewBusSends();

	/* Gives new Modulation Routing settings to the ActiveSound. This overwrites all previous Modulation Routing settings. */
	ENGINE_API void SetNewModulationRouting(const FSoundModulationDefaultRoutingSettings& NewRouting);

	/* Adds additional Modulators to the ActiveSound, if possible. 
	 * This function forces the Destination's routing method to Union.
	 * To replace existing modulators or set a different routing method, use SetNewModulationRouting.
	 */
	ENGINE_API void AddModulationRouting(const TSet<TObjectPtr<USoundModulatorBase>>& NewModulators, EModulationDestination Destination);

	/* Removes given Modulators from the ActiveSound, if possible. */
	ENGINE_API void RemoveModulationRouting(const TSet<TObjectPtr<USoundModulatorBase>>& NewModulators, EModulationDestination Destination);

	/** Creates a copy of the modulation configuration for all WaveInstances on this ActiveSound.
	 * Each element in the array corresponds to a Wave Instance's modulation settings, factoring in the routing settings on the wave instances. */
	ENGINE_API TArray<TSet<TObjectPtr<USoundModulatorBase>>> GetEffectiveVolumeModulators() const;
	
	/* Determines which of the provided listeners is the closest to the sound */
	ENGINE_API int32 FindClosestListener( const TArray<struct FListener>& InListeners ) const;

	/* Determines which listener is the closest to the sound */
	ENGINE_API int32 FindClosestListener() const;

	/** Returns the unique ID of the active sound's owner if it exists. Returns 0 if the sound doesn't have an owner. */
	FSoundOwnerObjectID GetOwnerID() const { return OwnerID; }

	/** Returns the unique Key of the active sound's owner if it exists. Returns default FObjectKey() if the sound doesn't have an owner. */
	FObjectKey GetOwnerKey() const { return OwnerKey; }

	/** Gets the sound concurrency handles applicable to this sound instance*/
	ENGINE_API void GetConcurrencyHandles(TArray<FConcurrencyHandle>& OutConcurrencyHandles) const;

	ENGINE_API bool GetConcurrencyFadeDuration(float& OutFadeDuration) const;

	/** Delegate callback function when an async occlusion trace completes */
	static ENGINE_API void OcclusionTraceDone(const FTraceHandle& TraceHandle, FTraceDatum& TraceDatum);

	/** Applies the active sound's attenuation settings to the input parse params using the given listener */
	ENGINE_API void ParseAttenuation(FSoundParseParameters& OutParseParams, int32 ListenerIndex, const FSoundAttenuationSettings& InAttenuationSettings);

	/** Returns whether or not sound or any active wave instances it manages are set to always play */
	ENGINE_API bool GetAlwaysPlay() const;

	/** Returns the highest effective priority of the child wave instances. If bIgnoreAlwaysPlay set to true, gives highest
	  * priority disregarding always play priority override.
	  */
	ENGINE_API float GetHighestPriority(bool bIgnoreAlwaysPlay = false) const;

	/** Sets the amount of audio from this active sound to send to the submix. */
	ENGINE_API void SetSubmixSend(const FSoundSubmixSendInfo& SubmixSendInfo);

	/** Sets the amount of audio from this active sound to send to the source bus. */
	ENGINE_API void SetSourceBusSend(EBusSendType BusSendType, const FSoundSourceBusSendInfo& SourceBusSendInfo);

	/** Updates the active sound's attenuation settings to the input parse params using the given listener */
	ENGINE_API void UpdateAttenuation(float DeltaTime, FSoundParseParameters& ParseParams, int32 ListenerIndex, const FSoundAttenuationSettings* SettingsAttenuationNode = nullptr);

	/** Updates the provided focus data using the local */
	ENGINE_API void UpdateFocusData(float DeltaTime, const FAttenuationListenerData& ListenerData, FAttenuationFocusData* OutFocusData = nullptr);

	/** Apply the submix sends to our parse params as appropriate */
	ENGINE_API void AddVolumeSubmixSends(FSoundParseParameters& ParseParams, EAudioVolumeLocationState LocationState);

private:

	struct FAsyncTraceDetails
	{
		Audio::FDeviceId AudioDeviceID;
		FActiveSound* ActiveSound;
	};

	static ENGINE_API TMap<FTraceHandle, FAsyncTraceDetails> TraceToActiveSoundMap;

	static ENGINE_API FTraceDelegate ActiveSoundTraceDelegate;

	/** Cached index to the closest listener. So we don't have to do the work to find it twice. */
	mutable int32 ClosestListenerIndex = INDEX_NONE;

	void CacheClosestListener() const;

	/** This is a friend so the audio device can call Stop() on the active sound. */
	friend class FAudioDevice;

	/**
	  * Marks the active sound as pending delete and begins termination of internal resources.
	  * Only to be called from the owning audio device.
	  */
	ENGINE_API void MarkPendingDestroy (bool bDestroyNow);

	/** Whether or not the active sound is stopping. */
	bool IsStopping() const { return bIsStopping; }

	/** Called when an active sound has been stopped but needs to update it's stopping sounds. Returns true when stopping sources have finished stopping. */
	ENGINE_API bool UpdateStoppingSources(uint64 CurrentTick, bool bEnsureStopped);

	/** Updates ramping concurrency volume scalars */
	ENGINE_API void UpdateConcurrencyVolumeScalars(const float DeltaTime);

	/** if OcclusionCheckInterval >= 0.0, periodically checks if the sound has become (un)occluded during playback
	 * When the interval is < 0.0, perform the initial occlusion check, but do not retest occlusion.
	 * @param ListenerLocation - location of the closest listener to the sound
	 * @param SoundLocation - location of the sound
	 * @param AttenuationSettingsPtr - Attenuation settings used (subject to various overrides)
	 */
	ENGINE_API void CheckOcclusion(const FVector ListenerLocation, const FVector SoundLocation, const FSoundAttenuationSettings* AttenuationSettingsPtr);

	/** Gather the interior settings needed for the sound */
	ENGINE_API void GatherInteriorData(FSoundParseParameters& ParseParams);

	/** Apply the interior settings to the ambient sound as appropriate */
	ENGINE_API void HandleInteriorVolumes(FSoundParseParameters& ParseParams);

	/** Helper function which retrieves attenuation frequency value for HPF and LPF distance-based filtering. */
	ENGINE_API float GetAttenuationFrequency(const FSoundAttenuationSettings* InSettings, const FAttenuationListenerData& ListenerData, const FVector2D& FrequencyRange, const FRuntimeFloatCurve& CustomCurve);
};
