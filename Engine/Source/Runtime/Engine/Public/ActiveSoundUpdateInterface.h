// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ActiveSoundUpdateInterface.generated.h"

// Forward Declarations 
struct FActiveSound;
struct FSoundParseParameters;

/** Interface for modifying active sounds during their update */
UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class UActiveSoundUpdateInterface : public UInterface
{
	GENERATED_BODY()
};

class IActiveSoundUpdateInterface
{
	GENERATED_BODY()

public:

	/**
	 * Gathers interior data that can affect the active sound.  Non-const as this step can be used to track state about the sound on the implementing object
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The active sound affected
	 * @param ParseParams	The parameters to apply to the wave instances
	 */
	virtual void GatherInteriorData(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) {}

	/**
	 * Applies interior data previously collected to the active sound and parse parameters.
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The active sound affected
	 * @param ParseParams	The parameters to apply to the wave instances
	 */
	virtual void ApplyInteriorSettings(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) {}

	/**
	 * Allows subsystems to modify SoundParseParameters just before Sound.Parse checks them to populate WaveInstances.
	 * 
	 * Modifying Transform is not recommended. It is used for calculations before this function is called,
	 * which will not be able to observe any modifications done here.
	 * 
	 * Modifying SoundClass is allowed but not compatible with use of FAudioDevice::SetSoundClassDistanceScale.
	 * The scale value will always be taken from the old class, never the new one.
	 * 
	 * NOTE! Called on the AudioThread during each audio frame Update
	 *
	 * @param ActiveSound	The active sound affected
	 * @param ParseParams	The parameters to apply to the wave instances
	 */
	virtual void SubsystemsAdjustParseParams(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) {}

	/**
	 * Called while an active sound is being constructed. Will be followed by either
	 * NotifyActiveSoundCreated or NotifyVirtualizedSoundCreated.
	 * 
	 * NOTE! Called on the GameThread
	 *
	 * @param ActiveSound   The active sound being associated
	 * @param Owner         The owner it is associated with, or nullptr
	 */
	virtual void NotifyActiveSoundOwner(FActiveSound& ActiveSound, const UObject* Owner) {}

	/**
	 * Called when an active sound has just been added to the audio engine,
	 * both for brand new sounds and for virtualized sounds that have just become active.
	 * In the latter case, a corresponding NotifyVirtualizedSoundDeleting will be received.
	 * You can correlate the two objects by matching their GetPlayOrder() value.
	 *
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The active sound being created
	 */
	virtual void NotifyActiveSoundCreated(FActiveSound& ActiveSound) {}

	/**
	 * Called when an active sound has just been removed from the audio engine, by being stopped or virtualized.
	 * In either case, the referenced ActiveSound object is about to be deleted; any pointers to it should be discarded.
	 * 
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The active sound being deleted
	 */
	virtual void NotifyActiveSoundDeleting(const FActiveSound& ActiveSound) {}

	/**
	 * Called when a virtualized sound has just been added to the audio engine,
	 * both for brand new sounds and for active sounds that have just become virtualized.
	 * When virtualizing, the corresponding NotifyActiveSoundDeleting will arrive after any fade-out has finished.
	 * You can correlate the two objects by matching their GetPlayOrder() value.
	 * 
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The virtualized sound being created
	 */
	virtual void NotifyVirtualizedSoundCreated(FActiveSound& ActiveSound) {}

	/**
	 * Called when a virtualized sound has just been removed from the audio engine, by being stopped or re-triggered.
	 * In either case, the referenced ActiveSound object is about to be deleted; any pointers to it should be discarded.
	 * 
	 * NOTE! Called on the AudioThread
	 *
	 * @param ActiveSound	The active sound being removed
	 */
	virtual void NotifyVirtualizedSoundDeleting(const FActiveSound& ActiveSound) {}


	UE_DEPRECATED(5.5, "OnNotifyAddActiveSound is deprecated. Use NotifyActiveSoundCreated instead.")
	virtual void OnNotifyAddActiveSound(FActiveSound& ActiveSound) {}

	UE_DEPRECATED(5.5, "OnNotifyPendingDelete is deprecated. Use NotifyActiveSoundDeleting and/or NotifyVirtualizedSoundDeleting instead.")
	virtual void OnNotifyPendingDelete(const FActiveSound& ActiveSound) {}
};
