// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"
#include "Containers/Deque.h"
#include "IAssetCompilingManager.h"

class FAsyncCompilationNotification;
class AActor;
struct FAssetCompileData;

#if WITH_EDITOR

class FActorDeferredScriptManager : IAssetCompilingManager
{
public:
	ENGINE_API static FActorDeferredScriptManager& Get();

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown() override;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

	/**
	 * empty implementation since deferred actor scripts are not actually async
	 */
	void FinishAllCompilation() override
	{
		ProcessAsyncTasks(false);
	}

	/** 
	 * Add an actor for the manager to run the construction of once dependent asset compilation are done (i.e Static Mesh)
	 */
	ENGINE_API void AddActor(AActor* InActor);

private:
	friend class FAssetCompilingManager;

	FActorDeferredScriptManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	void UpdateCompilationNotification();

	void OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets);
	void OnWorldCleanup(class UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

	void EnsureEventRegistered();

	/** Actors awaiting outstanding asset compilation prior to running their non trivial construction scripts after loading this level. */
	TArray<TWeakObjectPtr<AActor>> PendingConstructionScriptActors;

	/** Notification for the amount of pending construction scripts to run */
	TUniquePtr<FAsyncCompilationNotification> Notification;

	/** We need to monitor when static meshes have finished compiling to only work when necessary */
	FDelegateHandle OnAssetChangeDelegateHandle;

	/** We need to monitor when worlds are cleaned up to avoid triggering any construction script past this point */
	FDelegateHandle OnWorldCleanupDelegateHandle;

	/** Where to start on the next ProcessAsyncTasks iteration. */
	int32 NextIndexToProcess = 0;
	
	/** What's left to process if we were aborted because of execution time limit. */
	int32 NumLeftToProcess = 0;
};

#endif
