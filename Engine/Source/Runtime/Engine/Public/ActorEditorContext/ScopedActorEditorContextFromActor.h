// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreFwd.h"

class AActor;

/**
 * Pushes a new context initialized from the provided actor.
 */
class FScopedActorEditorContextFromActor
{
public:
	ENGINE_API FScopedActorEditorContextFromActor(AActor* InActor);
	ENGINE_API ~FScopedActorEditorContextFromActor();
};

#endif
