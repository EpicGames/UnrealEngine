// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreFwd.h"

class UExternalDataLayerAsset;

/**
 * Pushes a copy of the existing context and overrides the current External Data Layer Asset.
 */
class FScopedActorEditorContextSetExternalDataLayerAsset
{
public:
	ENGINE_API FScopedActorEditorContextSetExternalDataLayerAsset(const UExternalDataLayerAsset* InExternalDataLayerAsset);
	ENGINE_API ~FScopedActorEditorContextSetExternalDataLayerAsset();
};

#endif
