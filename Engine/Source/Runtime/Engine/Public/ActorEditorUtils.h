// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class UActorComponent;

namespace FActorEditorUtils
{
	/**
	* Return whether this actor is a builder brush or not.
	*
	* @return true if this actor is a builder brush, false otherwise
	*/
	ENGINE_API bool IsABuilderBrush( const AActor* InActor );

	/**
	 * Return whether this actor is an ABrush or not.
	 *
	 * @return true if this actor is an ABrush, false otherwise.
	 */
	ENGINE_API bool IsABrush( const AActor* InActor );

	/** 
	* Return whether this actor is a preview actor or not.
	*
	* @return true if this actor is a preview actor, false otherwise
	*/
	ENGINE_API bool IsAPreviewOrInactiveActor( const AActor* InActor );

	/**
	 * Gets a list of components, from an actor, which can be externally modified.  Editable components have properties visible when viewing their owner actor in a details view
	 *
	 * @param InActor		The actor to get components from
	 * @param OutEditableComponents The populated list of editable components on InActor
	 */
	ENGINE_API void GetEditableComponents( const AActor* InActor, TArray<UActorComponent*>& OutEditableComponents );

	/**
	 * Traverse an actor sub-tree (defined by attached children), parent first.
	 * 
	 * @param InActor				The Actor to traverse. InPredicate is only invoked for this actor where bIncludeThisActor is true
	 * @param InPredicate			Predicate to call for each actor of the traversal. Return true to continue, false to terminate the traversal
	 * @param bIncludeThisActor		True to include this actor in the traversal, false otherwise
	 * @return true on expected completion, false where the client prematurely terminated the traversal
	 */
	ENGINE_API bool TraverseActorTree_ParentFirst(AActor* InActor, TFunctionRef<bool(AActor*)> InPredicate, bool bIncludeThisActor = true);

	/**
	 * Traverse an actor sub-tree (defined by attached children), child first.
	 * 
	 * @param InActor				The Actor to traverse. InPredicate is only invoked for this actor where bIncludeThisActor is true
	 * @param InPredicate			Predicate to call for each actor of the traversal. Return true to continue, false to terminate the traversal
	 * @param bIncludeThisActor		True to include this actor in the traversal, false otherwise
	 * @return true on expected completion, false where the client prematurely terminated the traversal
	 */
	ENGINE_API bool TraverseActorTree_ChildFirst(AActor* InActor, TFunctionRef<bool(AActor*)> InPredicate, bool bIncludeThisActor = true);

	/**
	 * Validates that a name is suitable for an actor
	 *
	 * @param InName			The proposed actor name
	 * @param OutErrorMessage	Any error messages generated as a result of validation
	 * @return true if the actor name is valid, false if not.
	 */ 
	ENGINE_API bool ValidateActorName(const FText& InName, FText& OutErrorMessage);

	/**
	 * Validates that a name is suitable for an actor
	 *
	 * @param InName			The proposed actor name
	 * @param OutErrorMessage	Any error messages generated as a result of validation
	 * @return true if the actor name is valid, false if not.
	 */ 
	ENGINE_API bool ValidateActorLabel(FStringView InName, FText& OutErrorMessage);
	
#if WITH_EDITOR
	/**
	 * Return whether this actor is displayable in common UI (i.e. SceneOutliner), checks for things like
	 * validity, if it is a BuilderBrush, etc.
	 *
	 * @return true if this actor is displayable in common UI
	 */
	ENGINE_API bool IsActorDisplayable(const AActor* InActor);
#endif // WITH_EDITOR

};
