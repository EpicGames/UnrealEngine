// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Folder.h"

#include "ActorFolder.generated.h"

class ULevel;
struct FActorFolderDesc;

/**
 * This class is reserved for low-level Engine use, therefore relevant APIs are marked UE_INTERNAL.
 * The public, non-engine API for Folder operations is through the FActorFolders struct.
 */
UCLASS(Within = Level, MinimalAPI)
class UActorFolder : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	// Helper that creates a new UActorFolder
	UE_INTERNAL static ENGINE_API UActorFolder* Create(ULevel* InOuter, const FString& InFolderLabel, UActorFolder* InParent);

	// Returns actor folder info stored in its package
	UE_INTERNAL static ENGINE_API FActorFolderDesc GetAssetRegistryInfoFromPackage(FName PackageName);
	
	//~ Begin UObject
	UE_INTERNAL ENGINE_API virtual bool IsAsset() const override;
	UE_INTERNAL ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_INTERNAL ENGINE_API virtual void PostLoad() override;
	//~ End UObject

	UE_INTERNAL ENGINE_API void SetParent(UActorFolder* InParent);
	UE_INTERNAL ENGINE_API UActorFolder* GetParent(bool bSkipDeleted = true) const;

	UE_INTERNAL ENGINE_API void SetLabel(const FString& InFolderLabel);
	UE_INTERNAL const FString& GetLabel() const { return FolderLabel; }

	// Control whether the folder is initially expanded or not
	UE_INTERNAL ENGINE_API void SetIsInitiallyExpanded(bool bInFolderInitiallyExpanded);
	UE_INTERNAL bool IsInitiallyExpanded() const { return bFolderInitiallyExpanded; }

	UE_INTERNAL const FGuid& GetGuid() const { return FolderGuid; }
	UE_INTERNAL ENGINE_API FName GetPath() const;
	UE_INTERNAL ENGINE_API FString GetDisplayName() const;

	UE_INTERNAL ENGINE_API void MarkAsDeleted();
	UE_INTERNAL bool IsMarkedAsDeleted() const { return bIsDeleted; }
	
	// Checks if folder is valid (if it's not deleted)
	UE_INTERNAL bool IsValid() const
	{
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		return !IsMarkedAsDeleted();
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	}

	// Remaps parent folder to the first parent folder not marked as deleted
	UE_INTERNAL ENGINE_API void Fixup();
	// Detects and clears invalid parent folder
	UE_INTERNAL ENGINE_API void FixupParentFolder();

	UE_INTERNAL ENGINE_API FFolder GetFolder() const;

	/**
	 * Set the folder packaging mode.
	* @param bExternal will set the folder packaging mode to external if true, to internal otherwise
	* @param bShouldDirty should dirty or not the level package
	*/
	UE_INTERNAL ENGINE_API void SetPackageExternal(bool bExternal, bool bShouldDirty = true);

private:
	ENGINE_API FName GetPathInternal(UActorFolder* InSkipFolder) const;
#endif

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	FGuid ParentFolderGuid;

	UPROPERTY()
	FGuid FolderGuid;

	UPROPERTY()
	FString FolderLabel;

	UPROPERTY()
	bool bFolderInitiallyExpanded = true;

	UPROPERTY()
	bool bIsDeleted;
#endif
};
