// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "Containers/Map.h"

#if WITH_EDITOR

struct FActorFolderDesc
{
public:
	ENGINE_API FActorFolderDesc();
	const FGuid& GetParentFolderGuid() const { return ParentFolderGuid; }
	const FGuid& GetFolderGuid() const { return FolderGuid; }
	const FString& GetFolderLabel() const { return FolderLabel; }
	bool IsFolderInitiallyExpanded() const { return bFolderInitiallyExpanded; }
	bool IsFolderDeleted() const { return bFolderIsDeleted; }
	FString GetOuterPackageName() const { return OuterPackageName; }
	ENGINE_API FString GetDisplayName() const;
	ENGINE_API FString GetPath() const;

private:
	FGuid ParentFolderGuid;
	FGuid FolderGuid;
	FString FolderLabel;
	bool bFolderInitiallyExpanded;
	bool bFolderIsDeleted;
	FString OuterPackageName;

	friend class UActorFolder;
};

class FActorFolderDescsContext
{
public:
	FActorFolderDescsContext(const FActorFolderDesc& InActorFolderDesc);
	const FActorFolderDesc* GetActorFolderDesc(const FGuid& InFolderGuid);
	const FActorFolderDesc* GetParentActorFolderDesc(const FActorFolderDesc& InActorFolderDesc);

private:
	TMap<FGuid, FActorFolderDesc> ActorFolders;
};

#endif
