// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Templates/SubclassOf.h"
#include "Misc/HashBuilder.h"
#include "Misc/Guid.h"
#include "ActorPartition/PartitionActor.h"
#include "ActorPartitionSubsystem.generated.h"

class FBaseActorPartition;
class UWorldPartition;

#if WITH_EDITOR

static inline constexpr auto DefaultPartitionActorCreated = [](APartitionActor*) {};

/**
 * FActorPartitionGetParam
 */
struct FActorPartitionGetParams
{
	ENGINE_API FActorPartitionGetParams(const TSubclassOf<APartitionActor>& InActorClass, bool bInCreate, ULevel* InLevelHint, const FVector& InLocationHint, uint32 InGridSize = 0, const FGuid& InGuidHint = FGuid(), bool bInBoundsSearch = true, TFunctionRef<void(APartitionActor*)> InActorCreated = DefaultPartitionActorCreated);

	/* Class of Actor we are getting from the subsystem. */
	TSubclassOf<APartitionActor> ActorClass;
	
	/* Tells Subsystem if it needs to create Actor if it doesn't exist. */
	bool bCreate;
	
	/* Depending on the world LocationHint can be used to find/create the Actor. */
	FVector LocationHint;
	
	/* Depending on the world LevelHint can be used to find/create the Actor. */
	ULevel* LevelHint;

	/* Guid can be used to distinguish between actors of the same type. */
	FGuid GuidHint;

	/* If greater than 0, use this instead of the Actor CDO Grid size*/
	int32 GridSize;

	/* If true existing actors will be searched in the cell bounds only */
	bool bBoundsSearch;

	/* If set, a callback to use if an actor is created. */
	TFunctionRef<void(APartitionActor*)> ActorCreatedCallback;
};

/**
 * FActorPartitionIdentifier
 */

struct FActorPartitionIdentifier
{
	FActorPartitionIdentifier(UClass* InClass, const FGuid& InGridGuid, const uint32& InContextHash)
		: Class(InClass)
		, GridGuid(InGridGuid)
		, ContextHash(InContextHash)
	{}

	bool operator == (const FActorPartitionIdentifier& Other) const { return Class == Other.Class && GridGuid == Other.GridGuid && ContextHash == Other.ContextHash; }

	friend uint32 GetTypeHash(const FActorPartitionIdentifier& Id)
	{
		return GetTypeHash(Id.Class.Get()->GetName()) ^ GetTypeHash(Id.GridGuid) ^ GetTypeHash(Id.ContextHash);
	}

	const TSubclassOf<APartitionActor>& GetClass() const { return Class; }
	const FGuid& GetGridGuid() const { return GridGuid; }
	uint32 GetContextHash() const { return ContextHash; }
	static const uint32 EmptyContextHash = 0;
private:
	TSubclassOf<APartitionActor> Class;
	FGuid GridGuid;
	uint32 ContextHash;
};

#endif

/**
 * UActorPartitionSubsystem
 */

UCLASS(MinimalAPI)
class UActorPartitionSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	ENGINE_API UActorPartitionSubsystem();

	struct FCellCoord
	{
		FCellCoord()
		{}

		FCellCoord(int64 InX, int64 InY, int64 InZ, ULevel* InLevel)
			: X(InX)
			, Y(InY)
			, Z(InZ)
			, Level(InLevel)
		{}

		int64 X;
		int64 Y;
		int64 Z;
		ULevel* Level;

		bool operator==(const FCellCoord& Other) const
		{
			return (X == Other.X) && (Y == Other.Y) && (Z == Other.Z) && (Level == Other.Level);
		}

		friend uint32 GetTypeHash(const FCellCoord& CellCoord)
		{
			FHashBuilder HashBuilder;
			HashBuilder << CellCoord.X << CellCoord.Y << CellCoord.Z << PointerHash(CellCoord.Level);
			return HashBuilder.GetHash();
		}

		static FCellCoord GetCellCoord(FVector InPos, ULevel* InLevel, uint32 InGridSize)
		{
			return FCellCoord(
				FMath::FloorToInt(InPos.X / InGridSize),
				FMath::FloorToInt(InPos.Y / InGridSize),
				FMath::FloorToInt(InPos.Z / InGridSize),
				InLevel
			);
		}

		static FCellCoord GetCellCoord(FIntPoint InPos, ULevel* InLevel, uint32 InGridSize)
		{
			const int64 GridSize = (int64)InGridSize;
			const int64 CellCoordX = InPos.X < 0 ? (InPos.X - (GridSize - 1)) / GridSize : InPos.X / GridSize;
			const int64 CellCoordY = InPos.Y < 0 ? (InPos.Y - (GridSize - 1)) / GridSize : InPos.Y / GridSize;

			return FCellCoord(CellCoordX, CellCoordY, 0, InLevel);
		}
		
		static FBox GetCellBounds(const FCellCoord& InCellCoord, uint32 InGridSize)
		{
			return FBox(
				FVector(
					static_cast<FVector::FReal>(InCellCoord.X * InGridSize),
					static_cast<FVector::FReal>(InCellCoord.Y * InGridSize),
					static_cast<FVector::FReal>(InCellCoord.Z * InGridSize)
				),
				FVector(
					static_cast<FVector::FReal>(InCellCoord.X * InGridSize + InGridSize),
					static_cast<FVector::FReal>(InCellCoord.Y * InGridSize + InGridSize),
					static_cast<FVector::FReal>(InCellCoord.Z * InGridSize + InGridSize)
				)
			);
		}
	};

#if WITH_EDITOR
	ENGINE_API APartitionActor* GetActor(const FActorPartitionGetParams& GetParam);

	/**
	 * Returns a matching actor based on the parameters being provided
	 * 
	 * @param InActorClass The type of actor we are searching for.
	 * @param InCellCoords The cell coordinate of that actor.
	 * @param bInCreate If the actor doesn't existe should we created it.
	 * @param InGuid Optional, if multiple actors of the same InActorClass exist user can provide Guid id.
	 * @param InGridSize Optional, if not provided the InActorClass CDO will be used to determine GridSize.
	 * @param bInBoundsSearch Optional, if not specified existing actors will be searched in the cell bounds only.
	 */
	ENGINE_API APartitionActor* GetActor(const TSubclassOf<APartitionActor>& InActorClass, const FCellCoord& InCellCoords, bool bInCreate, const FGuid& InGuid = FGuid(), uint32 InGridSize = 0, bool bInBoundsSearch = true, TFunctionRef<void(APartitionActor*)> InActorCreated = DefaultPartitionActorCreated);

	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;

	ENGINE_API void ForEachRelevantActor(const TSubclassOf<APartitionActor>& InActorClass, const FBox& IntersectionBounds, TFunctionRef<bool(APartitionActor*)>InOperation) const;

	/**
	 * Updates the partition actor data layers to match the provided ones.
	 * The partition actor name and its external package will change accordingly.
	 */
	ENGINE_API bool MoveActorToDataLayers(APartitionActor* InActor, const TArray<UDataLayerInstance*>& InDataLayerInstances);
#endif
	ENGINE_API bool IsLevelPartition() const;

protected:
	ENGINE_API virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;

private:

#if WITH_EDITOR
	APartitionActor* GetActor(const FActorPartitionIdentifier& InActorPartitionId, const FCellCoord& InCellCoords, bool bInCreate, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated = DefaultPartitionActorCreated);

	void OnActorPartitionHashInvalidated(const FCellCoord& Hash);
	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);

	void InitializeActorPartition();
	void UninitializeActorPartition();

	TMap<FCellCoord, TMap<FActorPartitionIdentifier, TWeakObjectPtr<APartitionActor>>> PartitionedActors;
	TUniquePtr<FBaseActorPartition> ActorPartition;
	
	FDelegateHandle ActorPartitionHashInvalidatedHandle;
#endif
};

#if WITH_EDITOR
/**
 * FBaseActorPartition
 */
class FBaseActorPartition
{
public:
	FBaseActorPartition(UWorld* InWorld) : World(InWorld) {}
	virtual ~FBaseActorPartition() = default;

	virtual UActorPartitionSubsystem::FCellCoord GetActorPartitionHash(const FActorPartitionGetParams& GetParams) const = 0;
	virtual APartitionActor* GetActor(const FActorPartitionIdentifier& InActorPartitionId, bool bInCreate, const UActorPartitionSubsystem::FCellCoord& InCellCoord, uint32 InGridSize, bool bInBoundsSearch, TFunctionRef<void(APartitionActor*)> InActorCreated) = 0;
	virtual void ForEachRelevantActor(const TSubclassOf<APartitionActor>& InActorClass, const FBox& IntersectionBounds, TFunctionRef<bool(APartitionActor*)>InOperation) const = 0;

	DECLARE_EVENT_OneParam(FBaseActorPartition, FOnActorPartitionHashInvalidated, const UActorPartitionSubsystem::FCellCoord&);
	FOnActorPartitionHashInvalidated& GetOnActorPartitionHashInvalidated() { return OnActorPartitionHashInvalidated; }
protected:
	UWorld* World;

	FOnActorPartitionHashInvalidated OnActorPartitionHashInvalidated;

	friend class FActorPartitionGridHelper;
};


class FActorPartitionGridHelper
{
public:
	static ENGINE_API void ForEachIntersectingCell(const TSubclassOf<APartitionActor>& InActorClass, const FBox& InBounds, ULevel* InLevel, TFunctionRef<bool(const UActorPartitionSubsystem::FCellCoord&, const FBox&)> InOperation, uint32 InGridSize = 0);
	static ENGINE_API void ForEachIntersectingCell(const TSubclassOf<APartitionActor>& InActorClass, const FIntRect& InBounds, ULevel* InLevel, TFunctionRef<bool(const UActorPartitionSubsystem::FCellCoord&, const FIntRect&)> InOperation, uint32 InGridSize = 0);
};
#endif
