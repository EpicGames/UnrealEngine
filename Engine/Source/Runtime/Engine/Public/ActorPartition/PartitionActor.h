// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Misc/Guid.h"
#include "PartitionActor.generated.h"

#if WITH_EDITOR
struct FActorPartitionIdentifier;
#endif

// Actor base class for instance containers placed on a grid.
// See UActorPartitionSubsystem.
UCLASS(Abstract, MinimalAPI)
class APartitionActor: public AActor
{
	GENERATED_BODY()

public:
	ENGINE_API APartitionActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	//~ Begin AActor Interface
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	virtual uint32 GetDefaultGridSize(UWorld* InWorld) const PURE_VIRTUAL(APartitionActor, return 0;)
	ENGINE_API virtual bool ShouldIncludeGridSizeInName(UWorld * InWorld, const FActorPartitionIdentifier& InIdentifier) const;
	virtual bool ShouldIncludeGridSizeInLabel() const { return false; }
	virtual FGuid GetGridGuid() const { return FGuid(); }
	ENGINE_API virtual bool IsUserManaged() const override;
	//~ End AActor Interface

	virtual bool IsPartitionActorNameAffectedByDataLayers() const { return true; }
#endif

#if WITH_EDITORONLY_DATA
	ENGINE_API uint32 GetGridSize() const;
	ENGINE_API void SetGridSize(uint32 InGridSize);

	/** The grid size this actors was generated for */
	UE_DEPRECATED(5.3, "Use GetGridSize()/SetGridSize() instead.")
	UPROPERTY()
	uint32 GridSize;

	UE_DEPRECATED(5.4, "Use version that takes FActorPartitionIdentifier as second parameter")
	static ENGINE_API FString GetActorName(UWorld* World, const UClass* Class, const FGuid& GridGuid, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ, uint32 ContextHash);

	static ENGINE_API FString GetActorName(UWorld* World, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ);
	static ENGINE_API void SetLabelForActor(APartitionActor* PartitionActor, const FActorPartitionIdentifier& ActorPartitionId, uint32 GridSize, int32 CellCoordsX, int32 CellCoordsY, int32 CellCoordsZ);
#endif

	static ENGINE_API bool IsPartitioned(const APartitionActor* Actor);
};

DEFINE_ACTORDESC_TYPE(APartitionActor, FPartitionActorDesc);
