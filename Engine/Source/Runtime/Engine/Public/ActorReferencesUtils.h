// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

class AActor;

namespace ActorsReferencesUtils
{
	/**
	 * Gather direct references to external actors from the root object.
	 */
	UE_DEPRECATED(5.3, "GetExternalActorReferences is deprecated. Use GetActorReferences with params instead.")
	ENGINE_API TArray<AActor*> GetExternalActorReferences(UObject* Root, bool bRecursive = false);

	/**
	 * Gather direct references to actors from the root object.
	 */
	UE_DEPRECATED(5.3, "GetActorReferences is deprecated. Use GetActorReferences with params instead.")
	ENGINE_API TArray<AActor*> GetActorReferences(UObject* Root, EObjectFlags RequiredFlags = RF_NoFlags, bool bRecursive = false);

	struct FGetActorReferencesParams
	{
		FGetActorReferencesParams(UObject* InRoot)
			: Root(InRoot)
		{}

		UObject* Root = nullptr;
		EObjectFlags RequiredFlags = RF_NoFlags;
		bool bRecursive = false;

		FGetActorReferencesParams& SetRequiredFlags(EObjectFlags InRequiredFlags) { RequiredFlags = InRequiredFlags; return *this; }
		FGetActorReferencesParams& SetRecursive(bool bInRecursive) { bRecursive = bInRecursive; return *this; }
	};

	struct FActorReference
	{
		AActor* Actor;
		bool bIsEditorOnly;
	};

	/**
	 * Gather references to actors from the root object.
	 */
	ENGINE_API TArray<FActorReference> GetActorReferences(const FGetActorReferencesParams& InParams);
}