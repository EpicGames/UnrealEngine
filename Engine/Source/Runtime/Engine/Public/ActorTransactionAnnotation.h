// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "ComponentInstanceDataCache.h"

#if WITH_EDITOR
#include "Misc/TransactionCommon.h"
#include "Misc/ITransactionObjectAnnotation.h"
#endif

class AActor;
class USceneComponent;

/** Internal struct used to store information about an actor's components during reconstruction */
struct FActorRootComponentReconstructionData
{
	/** Struct to store info about attached actors */
	struct FAttachedActorInfo
	{
		TWeakObjectPtr<AActor> Actor;
		TWeakObjectPtr<USceneComponent> AttachParent;
		FName AttachParentName;
		FName SocketName;
		FTransform RelativeTransform;

		friend FArchive& operator<<(FArchive& Ar, FAttachedActorInfo& ActorInfo);
	};

	/** The RootComponent's transform */
	FTransform Transform;

	/** The RootComponent's relative rotation cache (enforces using the same rotator) */
	FRotationConversionCache TransformRotationCache;

	/** The Actor the RootComponent is attached to */
	FAttachedActorInfo AttachedParentInfo;

	/** Actors that are attached to this RootComponent */
	TArray<FAttachedActorInfo> AttachedToInfo;

	friend FArchive& operator<<(FArchive& Ar, FActorRootComponentReconstructionData& RootComponentData);
};

struct FActorTransactionAnnotationData
{
	TWeakObjectPtr<const AActor> Actor;
	FComponentInstanceDataCache ComponentInstanceData;

	bool bRootComponentDataCached;
	FActorRootComponentReconstructionData RootComponentData;

	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FActorTransactionAnnotationData& ActorTransactionAnnotationData);
};

#if WITH_EDITOR
/** Internal struct to track currently active transactions */
class FActorTransactionAnnotation : public ITransactionObjectAnnotation
{
public:
	/** Create an empty instance */
	static TSharedRef<FActorTransactionAnnotation> Create();

	/** Create an instance from the given actor, optionally caching root component data */
	static TSharedRef<FActorTransactionAnnotation> Create(const AActor* InActor, const bool InCacheRootComponentData = true);

	/** Create an instance from the given actor if required (UActorTransactionAnnotation::HasInstanceData would return true), optionally caching root component data */
	static TSharedPtr<FActorTransactionAnnotation> CreateIfRequired(const AActor* InActor, const bool InCacheRootComponentData = true);

	//~ ITransactionObjectAnnotation interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual bool SupportsAdditionalObjectChanges() const override { return true; }
	virtual void ComputeAdditionalObjectChanges(const ITransactionObjectAnnotation* OriginalAnnotation, TMap<UObject*, FTransactionObjectChange>& OutAdditionalObjectChanges) override;

	bool HasInstanceData() const;

	FActorTransactionAnnotationData ActorTransactionAnnotationData;

private:
	FActorTransactionAnnotation();
	FActorTransactionAnnotation(const AActor* InActor, FComponentInstanceDataCache&& InComponentInstanceData, const bool InCacheRootComponentData = true);
	
	struct FDiffableComponentInfo
	{
		FDiffableComponentInfo() = default;
		explicit FDiffableComponentInfo(const UActorComponent* Component);

		FActorComponentInstanceSourceInfo ComponentSourceInfo;
		UE::Transaction::FDiffableObject DiffableComponent;
	};

	TArray<FDiffableComponentInfo> DiffableComponentInfos;
};
#endif // WITH_EDITOR
