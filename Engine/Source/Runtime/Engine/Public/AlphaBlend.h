// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AlphaBlend.generated.h"

class UCurveFloat;

UENUM(BlueprintType)
enum class EAlphaBlendOption : uint8
{
	Linear = 0 UMETA(Grouping = Linear, DisplayName = "Linear", ToolTip = "Linear interpolation"),
	Cubic UMETA(Grouping = Cubic, DisplayName = "Cubic In", ToolTip = "Cubic-in interpolation"),
	HermiteCubic UMETA(Grouping = Cubic, DisplayName = "Hermite-Cubic InOut", ToolTip = "Hermite-Cubic"),
	Sinusoidal UMETA(Grouping = Sinusoidal, DisplayName = "Sinusoidal", ToolTip = "Sinusoidal interpolation"),
	QuadraticInOut UMETA(Grouping = Quadratic, DisplayName = "Quadratic InOut", ToolTip = "Quadratic in-out interpolation"),
	CubicInOut UMETA(Grouping = Cubic, DisplayName = "Cubic InOut", ToolTip = "Cubic in-out interpolation"),
	QuarticInOut UMETA(Grouping = Quartic, DisplayName = "Quartic InOut", ToolTip = "Quartic in-out interpolation"),
	QuinticInOut UMETA(Grouping = Quintic, DisplayName = "Quintic InOut", ToolTip = "Quintic in-out interpolation"),
	CircularIn UMETA(Grouping = Circular, DisplayName = "Circular In", ToolTip = "Circular-in interpolation"),
	CircularOut UMETA(Grouping = Circular, DisplayName = "Circular Out", ToolTip = "Circular-out interpolation"),
	CircularInOut UMETA(Grouping = Circular, DisplayName = "Circular InOut", ToolTip = "Circular in-out interpolation"),
	ExpIn UMETA(Grouping = Exponential, DisplayName = "Exponential In", ToolTip = "Exponential-in interpolation"),
	ExpOut UMETA(Grouping = Exponential, DisplayName = "Exponential Out", ToolTip = "Exponential-Out interpolation"),
	ExpInOut UMETA(Grouping = Exponential, DisplayName = "Exponential InOut", ToolTip = "Exponential in-out interpolation"),
	Custom UMETA(Grouping = Custom, DisplayName = "Custom", ToolTip = "Custom interpolation, will use custom curve inside an FAlphaBlend or linear if none has been set"),
};

namespace UE::Anim
{
	inline constexpr EAlphaBlendOption DefaultBlendOption = EAlphaBlendOption::HermiteCubic;
}

/**
 * Alpha Blend construction arguments. Used for creation of an AlphaBlend.
 */
USTRUCT(BlueprintType)
struct FAlphaBlendArgs
{
	GENERATED_BODY()

	ENGINE_API FAlphaBlendArgs();
	ENGINE_API FAlphaBlendArgs(float InBlendTime);
	ENGINE_API FAlphaBlendArgs(const struct FAlphaBlend& InAlphaBlend);

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	bool Serialize(FStructuredArchive::FSlot Slot);
	void PostSerialize(const FArchive& Ar);
#endif // WITH_EDITORONLY_DATA 

	/** If you're using Custom BlendOption, you can specify curve */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta=(DisplayAfter="BlendOption"))
	TObjectPtr<UCurveFloat> CustomCurve;

	/** Blend Time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend", meta = (UIMin = "0", ClampMin = "0"))
	float BlendTime;

	/** Type of blending used (Linear, Cubic, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend")
	EAlphaBlendOption BlendOption;
};

/**
 * Alpha Blend class that supports different blend options as well as custom curves
 */
USTRUCT(BlueprintType)
struct FAlphaBlend
{
	GENERATED_BODY()
private:
	/** 
	 * Please note that changing this variable would get applied in the NEXT UPDATE. 
	 * This does not change the Alpha and BlendedValue right away and it is intentional
	 */

	/** If you're using Custom BlendOption, you can specify curve */
	UPROPERTY(EditAnywhere, Category = "Blend", meta=(DisplayAfter="BlendOption"))
	TObjectPtr<UCurveFloat> CustomCurve;

public:
	/* Constructor */
	ENGINE_API FAlphaBlend(float NewBlendTime = 0.2f);

	/* Constructor */
	ENGINE_API FAlphaBlend(const FAlphaBlend& Other, float NewBlendTime);
	
#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	bool Serialize(FStructuredArchive::FSlot Slot);
	void PostSerialize(const FArchive& Ar);
#endif // WITH_EDITORONLY_DATA 
	
	/* Constructor */
	ENGINE_API explicit FAlphaBlend(const FAlphaBlendArgs& InArgs);

	/** Setters - need to refresh cached value */
	ENGINE_API void SetBlendOption(EAlphaBlendOption InBlendOption);
	ENGINE_API void SetCustomCurve(UCurveFloat* InCustomCurve);
	/** Update transition blend time. This new value will be applied in the next Update. */
	ENGINE_API void SetBlendTime(float InBlendTime);

	/** Sets the range of values to map to the interpolation
	 *
	 * @param Begin : this is start value
	 * @param Desired : this is target value 
	 *
	 * This can be (0, 1) if you'd like to increase, or it can be (1, 0) if you'd like to get to 0
	 */
	ENGINE_API void SetValueRange(float Begin, float Desired);

	/** Sets the final desired value for the blended value */
	ENGINE_API void SetDesiredValue(float InDesired);

	/** Sets the Lerp alpha value directly. PLEASE NOTE that this modifies the Blended Value right away.  */
	ENGINE_API void SetAlpha(float InAlpha);

	/** Update interpolation, has to be called once every frame.
	 *
	 * @return How much time remains after the blend completed if applicable
	 * e.g. if we have 0.01s left on the blend and update at 30Hz (~0.033s) we would return ~0.023s
	 */
	ENGINE_API float Update(float InDeltaTime);

	/** Gets whether or not the blend is complete */
	ENGINE_API bool IsComplete() const;

	/** Gets the current 0..1 alpha value. Changed to AlphaLerp to match with SetAlpha function */
	float GetAlpha() const { return AlphaLerp; }

	/** Gets the current blended value */
	float GetBlendedValue() const { return BlendedValue; }

	/** Getters */
	float GetBlendTime() const { return BlendTime; }
	float GetBlendTimeRemaining() const { return BlendTimeRemaining; }
	EAlphaBlendOption GetBlendOption() const { return BlendOption; }
	UCurveFloat* GetCustomCurve() const { return CustomCurve; }

	/** Get the current begin value */
	float GetBeginValue() const { return BeginValue; }
	
	/** Get the current desired value */
	float GetDesiredValue() const { return DesiredValue; }

	/** Converts InAlpha from a linear 0...1 value into the output alpha described by InBlendOption 
	 *  @param InAlpha In linear 0...1 alpha
	 *  @param InBlendOption The type of blend to use
	 *  @param InCustomCurve The curve to use when blend option is set to custom
	 */
	static ENGINE_API float AlphaToBlendOption(float InAlpha, EAlphaBlendOption InBlendOption, UCurveFloat* InCustomCurve = nullptr);

private:
	/** Blend Time */
	UPROPERTY(EditAnywhere, Category = "Blend", meta = (UIMin = "0", ClampMin = "0"))
	float	BlendTime;

	/** Internal Lerped value for Alpha */
	float	AlphaLerp;

	/** Resulting Alpha value, between 0.f and 1.f */
	float	AlphaBlend;

	/** Time left to reach target */
	float	BlendTimeRemaining;

	/** The current blended value derived from the begin and desired values. 
	  * This value should not change unless Update. 
	  */
	float BlendedValue;

	/** The Start value. It is 'from' range */
	float BeginValue;

	/**  The Target value. It is 'to' range */
	float DesiredValue;

	/** 
	 * Reset functions - 
	 *
	 * The distinction is important. 
	 * 
	 * We have 3 different reset functions depending on usability
	 * 
	 * Reset function : will change Blended Value to BeginValue, so that it can start blending
	 * That is only supposed to be used when you want to clear it up and restart
	 *
	 * ResetAlpha will change AlphaLerp/AlphaBlend to match with current Blend Value, 
	 * that way we keep the current blended value and move to target using the current value
	 * That will make sure this doesn't pop when you change direction of Desired
	 *
	 * ResetBlendTime will change BlendTimeRemaining as well as possibly weight because if BlendTimeRemaining <= 0.f, 
	 * We'll arrive to the destination
	 *
	 * The reason we need ResetAlpha and ResetBlendTime is that we don't want to modify blend time if direction changes or we don't want to reset alpha if blend time changes
	 * Those two have to work independently
	 * */

public:
	/** Reset to zero / restart the blend. This resets whole thing.  */
	ENGINE_API void Reset();

	/** Reset alpha, this keeps current BlendedValue but modify Alpha to keep the blending state.  */
	ENGINE_API void ResetAlpha();

private:
	/* Reset Blend Time, this modifies BlendTimeRemaining and possibly Weight when BlendTimeRemaining <= 0.f */
	ENGINE_API void ResetBlendTime();

	/** Converts internal lerped alpha into the output alpha type */
	ENGINE_API float AlphaToBlendOption();

	/** 
	* Please note that changing this variable would get applied in the NEXT UPDATE. 
	* This does not change the Alpha and BlendedValue right away and it is intentional
	*/

	/** Type of blending used (Linear, Cubic, etc.) */
	UPROPERTY(EditAnywhere, Category = "Blend")
	EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;

	/** internal flag to reset the alpha value */
	bool bNeedsToResetAlpha;

	/** internal flat to reset blend time. The two distinction is required. Otherwise, you'll modify blend time when you just change range */
	bool bNeedsToResetBlendTime;

	mutable bool bNeedsToResetCachedDesiredBlendedValue;

	/** Cached Desired Value with Alpha 1 so that we can check if reached or not */
	mutable float CachedDesiredBlendedValue;
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FAlphaBlendArgs> : public TStructOpsTypeTraitsBase2<FAlphaBlendArgs>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializer = true,
		WithPostSerialize = true,
	};
};


template<> struct TStructOpsTypeTraits<FAlphaBlend> : public TStructOpsTypeTraitsBase2<FAlphaBlend>
{
	enum
	{
		WithSerializer = true,
		WithStructuredSerializer = true,
		WithPostSerialize = true,
	};
};
#endif //#if WITH_EDITORONLY_DATA