// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"

class IAnalyticsPropertyStore;
class FAnalyticsSessionSummaryWriter;
struct FUserActivity;

/**
 * Collects engine events/stats and stores a summary on disk until reported by the analytics session summary manager.
 */
class FEngineAnalyticsSessionSummary
{
public:
	/**
	 * Constructs an analytics summary collector.
	 * @param Storage Where to store the engine summary properties that are sent on exit.
	 * @param MonitorProcessId The process ID of CRC launched at start up in 'monitor mode', zero otherwise.
	 */
	ENGINE_API FEngineAnalyticsSessionSummary(TSharedPtr<IAnalyticsPropertyStore> Storage, uint32 MonitorProcessId);

	/** Destructor. */
	virtual ~FEngineAnalyticsSessionSummary() = default;

	/** Ticks the session. This calls UpdateSessionProgress() at the configured update period which calls UpdateSessionProgressInternal() for derived classes. */
	ENGINE_API void Tick(float DeltaTime);

	/** Shuts down the session, unregistering the callbacks and closing the session. */
	ENGINE_API void Shutdown();

	/** Set the period used to persist the properties to disk. Some events ignore this setting and persist the properties right away. */
	void SetPersistPeriod(const FTimespan& Period) { PersistPeriod = Period; }

	/** Get the period used to update and persist the properties to disk. */
	FTimespan GetPersistPeriod() const { return PersistPeriod; }

	/** Invoked by the engine when the user is running low on disk space.. */
	ENGINE_API void LowDriveSpaceDetected();

protected:
	/** Invoked during Shutdown() to let derived classes hook in the shutdown. Default implementation does nothing.*/
	virtual void ShutdownInternal() {}

	/** Invoked during UpdateSessionProgress() to let derived classes hook in the update. Default implementation does nothing. Returns true to force persisting the properties to disk now. */
	virtual bool UpdateSessionProgressInternal(bool bCrashing) { return false; }

	/** Returns the property store used by the summary. */
	IAnalyticsPropertyStore* GetStore() { return Store.Get(); }

private:
	/** Invoked periodically and during important events to update the session progression.*/
	ENGINE_API void UpdateSessionProgress(bool bFlushAsync = true, const FTimespan& FlushTimeout = FTimespan::Zero(), bool bCrashing = false);

	/** Invoked when the engine is crashing.. */
	ENGINE_API void OnCrashing();

	/** Invoked when the engine fatally hung */
	ENGINE_API void OnHung();

	/** Invoked when the engine is requested to terminate. */
	ENGINE_API void OnTerminate();

	/** Invoked when the user is logging in or logging out. Only handles when the user logs out. */
	ENGINE_API void OnUserLoginChanged(bool bLoggingIn, int32, int32);

	/** Invoked when the user activity changes. */
	ENGINE_API void OnUserActivity(const FUserActivity& UserActivity);

	/** Invoked if the engine detects some local changes.. */
	ENGINE_API void OnVanillaStateChanged(bool bIsVanilla);

	/** Check on CRC and return true if its running state state changed (so that the change is persisted on disk). */
	ENGINE_API bool UpdateExternalProcessReporterState(bool bQuickCheck);

	/** Returns whether the debugger state changed. */
	ENGINE_API bool UpdateDebuggerStates();

private:
	/** The store used to persist the session properties until they are sent. */
	TSharedPtr<IAnalyticsPropertyStore> Store;

	/** Session timestamp from FDateTime::UtcNow(). Unreliable if user change system date/time (daylight saving or user altering it). */
	FDateTime SessionStartTimeUtc;

	/** Session timestamp from FPlatformTime::Seconds(). Lose precision when computing long time spans (+/- couple of seconds over a day). */
	double SessionStartTimeSecs;

	/** The number of time the summary was ticked. */
	uint64 CurrSessionTickCount = 0;

	/** The monitor process id. CrashReportClient (CRC) acts as the monitoring/reporting process. */
	uint32 CrcProcessId = 0;

	/** The nominal period at which the session properties are persisted to disk. */
	FTimespan PersistPeriod;

	/** The next time the session properties should be persisted to disk. */
	double NextPersistTimeSeconds = 0.0;

	/** Cache CRC exit code (if CRC was spawned in background and unexpectedly died before the engine). */
	TOptional<int32> CrcExitCode;

	/** Indicates whether Shutdown() method was called. */
	bool bShutdown = false;

	/** Indicates whether the setting to ignore the debugger was set. */
	bool bDebuggerIgnored = false;

	/** Indicates whether the presence of the debugger was detected. */
	bool bWasEverDebugged = false;
};
