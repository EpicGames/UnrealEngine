// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Net/Core/Analytics/NetAnalytics.h"
#include "Net/Core/Connection/NetCloseResult.h"


/**
 * Container class for separating analytics variables and processing, from the main NetConnection code
 */
struct FNetConnAnalyticsVars
{
	friend struct FNetConnAnalyticsData;

	using FNetResult = UE::Net::FNetResult;

public:
	/** Default constructor */
	FNetConnAnalyticsVars();

	bool operator == (const FNetConnAnalyticsVars& A) const;

	void CommitAnalytics(FNetConnAnalyticsVars& AggregatedData);


	/** Accessors */

	void IncreaseOutOfOrderPacketsLostCount(int32 Count=1)
	{
		OutOfOrderPacketsLostCount += Count;
	}

	void IncreaseOutOfOrderPacketsRecoveredCount(int32 Count=1)
	{
		OutOfOrderPacketsRecoveredCount += Count;
	}

	void IncreaseOutOfOrderPacketsDuplicateCount(int32 Count=1)
	{
		OutOfOrderPacketsDuplicateCount += Count;
	}

	void AddFailedPingAddressICMP(FString PingAddress)
	{
		if (FailedPingAddressesICMP.Num() < 32)
		{
			FailedPingAddressesICMP.AddUnique(PingAddress);
		}
	}

	void AddFailedPingAddressUDP(FString PingAddress)
	{
		if (FailedPingAddressesICMP.Num() < 32)
		{
			FailedPingAddressesUDP.AddUnique(PingAddress);
		}
	}


public:
	/** The number of packets that were exclusively ack packets */
	uint64 OutAckOnlyCount;

	/** The number of packets that were just keep-alive packets */
	uint64 OutKeepAliveCount;

private:
	/** The number of out of order packets lost */
	uint64 OutOfOrderPacketsLostCount = 0;

	/** The number of out of order packets recovered */
	uint64 OutOfOrderPacketsRecoveredCount = 0;

	/** The number of out of order packets that were duplicates */
	uint64 OutOfOrderPacketsDuplicateCount = 0;

public:
	/** The result/reason for triggering NetConnection Close (local) */
	TUniquePtr<FNetResult> CloseReason;

	/** The remotely-communicated result/reason for triggering NetConnection Close (remote, server only) */
	TArray<FString> ClientCloseReasons;

	/** NetConnection faults that were recovered from, and the number of times they were recovered from */
	TMap<FString, int32> RecoveredFaults;

private:
	/** List of IP addresses a client failed to ping with ICMP. Should correlate against NetConnection count, to determine overall percent. */
	TArray<FString> FailedPingAddressesICMP;

	/** List of IP addresses a client failed to ping with UDP. Should correlate against NetConnection count, to determine overall percent. */
	TArray<FString> FailedPingAddressesUDP;

public:
	/** Aggregation variables */

	struct FPerNetConnData
	{
		TUniquePtr<FNetResult> CloseReason;
		TArray<FString> ClientCloseReasons;
	};

	/** (Not filled until final commit) Contains Per-NetConnection data which can't be aggregated until final commit */
	TArray<FPerNetConnData> PerConnectionData;
};


/**
 * NetConnection implementation for basic aggregated net analytics data
 */
struct FNetConnAnalyticsData : public TBasicNetAnalyticsData<FNetConnAnalyticsVars>
{
public:
	virtual void SendAnalytics() override;
};
