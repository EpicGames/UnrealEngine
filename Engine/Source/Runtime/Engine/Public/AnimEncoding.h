// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding.h: Skeletal mesh animation compression.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "AnimationCompression.h"

// switches to toggle subsets of the new animation codec system
#define USE_ANIMATION_CODEC_BATCH_SOLVER 1

// all past encoding package version numbers should be listed here
#define ANIMATION_ENCODING_PACKAGE_ORIGINAL 0

// the current animation encoding package version
#define CURRENT_ANIMATION_ENCODING_PACKAGE_VERSION ANIMATION_ENCODING_PACKAGE_ORIGINAL

class FMemoryWriter;

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Interfaces For Working With Encoded Animations
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 *	Structure to hold an Atom and Track index mapping for a requested bone. 
 *	Used in the bulk-animation solving process
 */
struct BoneTrackPair
{
	int32 AtomIndex;
	int32 TrackIndex;

	BoneTrackPair(){}
	BoneTrackPair(int32 Atom, int32 Track):AtomIndex(Atom),TrackIndex(Track){}
};

/**
 *	Fixed-size array of BoneTrackPair elements.
 *	Used in the bulk-animation solving process.
 */
#define MAX_BONES 65536 // DesiredBones is passed to the decompression routines as a TArray<FBoneIndexType>, so we know this max is appropriate
typedef TArray<BoneTrackPair> BoneTrackArray;


/** Array of FTransform using the game memory stack */
typedef TArray< FTransform, TMemStackAllocator<> > FTransformArray;


/**
 * Extracts statistics about a given Animation Sequence
 *
 * @param	Seq					An Animation Sequence.
 * @param	NumTransTracks		The total number of Translation Tracks found.
 * @param	NumRotTracks		The total number of Rotation Tracks found.
 * @param	TotalNumTransKeys	The total number of Translation Keys found.
 * @param	TotalNumRotKeys		The total number of Rotation Keys found.
 * @param	TranslationKeySize	The average size (in BYTES) of a single Translation Key.
 * @param	RotationKeySize		The average size (in BYTES) of a single Rotation Key.
 * @param	OverheadSize		The size (in BYTES) of overhead (offsets, scale tables, key->frame lookups, etc...)
 * @param	NumTransTracksWithOneKey	The total number of Translation Tracks found containing a single key.
 * @param	NumRotTracksWithOneKey		The total number of Rotation Tracks found containing a single key.
*/
ENGINE_API void AnimationFormat_GetStats(const FUECompressedAnimData& CompressedData, 
  								int32& NumTransTracks,
								int32& NumRotTracks,
								int32& NumScaleTracks,
  								int32& TotalNumTransKeys,
								int32& TotalNumRotKeys,
								int32& TotalNumScaleKeys,
								float& TranslationKeySize,
								float& RotationKeySize,
								float& ScaleKeySize,
								int32& OverheadSize,
								int32& NumTransTracksWithOneKey,
								int32& NumRotTracksWithOneKey,
								int32& NumScaleTracksWithOneKey);

/**
 * Sets the internal Animation Codec Interface Links within an Animation Sequence
 *
 * @param	Seq					An Animation Sequence to setup links within.
*/
template<typename CompressedDataType>
void AnimationFormat_SetInterfaceLinks(CompressedDataType& CompressedData);

#if WITH_EDITORONLY_DATA
#define AC_UnalignedSwap( MemoryArchive, Data, Len )		\
	MemoryArchive.ByteOrderSerialize( (Data), (Len) );		\
	(Data) += (Len);
#else
	// No need to swap on consoles, as the cooker will have ordered bytes for the target platform.
#define AC_UnalignedSwap( MemoryArchive, Data, Len )		\
	MemoryArchive.Serialize( (Data), (Len) );				\
	(Data) += (Len);
#endif // !WITH_EDITORONLY_DATA

extern ENGINE_API const int32 CompressedTranslationStrides[ACF_MAX];
extern ENGINE_API const int32 CompressedTranslationNum[ACF_MAX];
extern ENGINE_API const int32 CompressedRotationStrides[ACF_MAX];
extern ENGINE_API const int32 CompressedRotationNum[ACF_MAX];
extern ENGINE_API const int32 CompressedScaleStrides[ACF_MAX];
extern ENGINE_API const int32 CompressedScaleNum[ACF_MAX];
extern ENGINE_API const uint8 PerTrackNumComponentTable[ACF_MAX*8];

class FMemoryWriter;
class FMemoryReader;

void PadMemoryWriter(FMemoryWriter* MemoryWriter, uint8*& TrackData, const int32 Alignment);
void PadMemoryReader(FMemoryReader* MemoryReader, uint8*& TrackData, const int32 Alignment);


class AnimEncoding
{
public:
	/**
	 * Handles Byte-swapping incoming animation data from a MemoryReader
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryReader		The MemoryReader object to read from.
	 * @param	SourceArVersion		The version of the archive that the data is coming from.
	 */
	virtual void ByteSwapIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader) PURE_VIRTUAL(AnimEncoding::ByteSwapIn,);

	/**
	 * Handles Byte-swapping outgoing animation data to an array of BYTEs
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	SerializedData		The output buffer.
	 * @param	ForceByteSwapping	true is byte swapping is not optional.
	 */
	virtual void ByteSwapOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter) PURE_VIRTUAL(AnimEncoding::ByteSwapOut, );

#if USE_ANIMATION_CODEC_BATCH_SOLVER

	/**
	 * Decompress all requested rotation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseRotations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) PURE_VIRTUAL(AnimEncoding::GetPoseRotations,);

	/**
	 * Decompress all requested translation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseTranslations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) PURE_VIRTUAL(AnimEncoding::GetPoseTranslations,);

	/**
	 * Decompress all requested translation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseScales(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) PURE_VIRTUAL(AnimEncoding::GetPoseScales,);
#endif

protected:

	/**
	 * Utility function to determine the two key indices to interpolate given a relative position in the animation
	 *
	 * @param	SequenceLength	The length of the anim sequence 
	 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
	 * @param	NumKeys			The number of keys present in the track being solved.
	 * @param	Interpolation	The Interpolation type of the sequence
	 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
	 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
	 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
	 */
	static float TimeToIndex(
		float SequenceLength,
		float RelativePos,
		int32 NumKeys,
		EAnimInterpolationType Interpolation,
		int32 &PosIndex0Out,
		int32 &PosIndex1Out);

	/**
	 * Utility function to determine the two key indices to interpolate given a relative position in the animation
	 *
	 * @param	Interpolation	The Interpolation type of the sequence
	 * @param	NumberOfFrames	The number of frames in the original sequence
	 * @param	FrameTable		The frame table containing a frame index for each key.
	 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
	 * @param	NumKeys			The number of keys present in the track being solved.
	 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
	 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
	 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
	 */
	static float TimeToIndex(
		EAnimInterpolationType Interpolation,
		int32 NumberOfFrames,
		const uint8* FrameTable,
		float RelativePos,
		int32 NumKeys,
		int32 &PosIndex0Out,
		int32 &PosIndex1Out);
};


/**
 * This class serves as the base to AEFConstantKeyLerpShared, introducing the per-track serialization methods called by
 * ByteSwapIn/ByteSwapOut and individual GetBoneAtomRotation / GetBoneAtomTranslation calls, which GetBoneAtom calls on
 * Seq.TranslationCodec or Seq.RotationCodec.
 */
class AnimEncodingLegacyBase : public AnimEncoding
{
public:
	/**
	 * Decompress the Rotation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomRotation(
		FTransform& OutAtom,
		const FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex) PURE_VIRTUAL(AnimEncoding::GetBoneAtomRotation,);

	/**
	 * Decompress the Translation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomTranslation(
		FTransform& OutAtom,
		const FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex) PURE_VIRTUAL(AnimEncoding::GetBoneAtomTranslation,);

	/**
	 * Decompress the Scale component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomScale(
		FTransform& OutAtom,
		const FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex) PURE_VIRTUAL(AnimEncoding::GetBoneAtomScale,);

	/**
	 * Handles Byte-swapping incoming animation data from a MemoryReader
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryReader		The MemoryReader object to read from.
	 */
	virtual void ByteSwapIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader) override;

	/**
	 * Handles Byte-swapping outgoing animation data to an array of BYTEs
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	SerializedData		The output buffer.
	 * @param	ForceByteSwapping	true is byte swapping is not optional.
	 */
	virtual void ByteSwapOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter) override;

	/**
	 * Handles the ByteSwap of compressed animation data on import
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @param	SourceArVersion	The version number of the source archive stream.
	 * @return					The adjusted Stream position after import. 
	 */
	virtual void ByteSwapRotationIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapRotationIn,);

	/**
	 * Handles the ByteSwap of compressed animation data on import
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @param	SourceArVersion	The version number of the source archive stream.
	 * @return					The adjusted Stream position after import. 
	 */
	virtual void ByteSwapTranslationIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapTranslationIn,);

	/**
	 * Handles the ByteSwap of compressed animation data on import
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @param	SourceArVersion	The version number of the source archive stream.
	 * @return					The adjusted Stream position after import. 
	 */
	virtual void ByteSwapScaleIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapScaleIn,);

	/**
	 * Handles the ByteSwap of compressed animation data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryReader to write to.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @return					The adjusted Stream position after export. 
	 */
	virtual void ByteSwapRotationOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapRotationOut,);

	/**
	 * Handles the ByteSwap of compressed animation data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryReader to write to.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @return					The adjusted Stream position after export. 
	 */
	virtual void ByteSwapTranslationOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapTranslationOut,);

	/**
	 * Handles the ByteSwap of compressed animation data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryReader to write to.
	 * @param	Stream			The compressed animation data.
	 * @param	NumKeys			The number of keys present in Stream.
	 * @return					The adjusted Stream position after export. 
	 */
	virtual void ByteSwapScaleOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& Stream,
		int32 NumKeys) PURE_VIRTUAL(AnimEncoding::ByteSwapScaleOut,);
};


/**
 * Utility function to determine the two key indices to interpolate given a relative position in the animation
 *
 * @param	Seq				The UAnimSequence container.
 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
 * @param	NumKeys			The number of keys present in the track being solved.
 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
 */
inline float AnimEncoding::TimeToIndex(
	float SequenceLength,
	float RelativePos,
	int32 NumKeys,
	EAnimInterpolationType Interpolation,
	int32 &PosIndex0Out,
	int32 &PosIndex1Out)
{
	float Alpha;

	if (NumKeys < 2)
	{
		checkSlow(NumKeys == 1); // check if data is empty for some reason.
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		return 0.0f;
	}
	// Check for before-first-frame case.
	if( RelativePos <= 0.f )
	{
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		Alpha = 0.0f;
	}
	else
	{
		NumKeys -= 1; // never used without the minus one in this case
		// Check for after-last-frame case.
		if( RelativePos >= 1.0f )
		{
			// If we're not looping, key n-1 is the final key.
			PosIndex0Out = NumKeys;
			PosIndex1Out = NumKeys;
			Alpha = 0.0f;
		}
		else
		{
			// For non-looping animation, the last frame is the ending frame, and has no duration.
			const float KeyPos = RelativePos * float(NumKeys);
			checkSlow(KeyPos >= 0.0f);
			const float KeyPosFloor = floorf(KeyPos);
			PosIndex0Out = FMath::Min( FMath::TruncToInt(KeyPosFloor), NumKeys );
			Alpha = (Interpolation == EAnimInterpolationType::Step) ? 0.0f : KeyPos - KeyPosFloor;
			PosIndex1Out = FMath::Min( PosIndex0Out + 1, NumKeys );
		}
	}
	return Alpha;
}

/**
 * Utility function to find the key before the specified search value.
 *
 * @param	FrameTable		The frame table, containing on frame index value per key.
 * @param	NumKeys			The total number of keys in the table.
 * @param	SearchFrame		The Frame we are attempting to find.
 * @param	KeyEstimate		An estimate of the best location to search from in the KeyTable.
 * @return	The index of the first key immediately below the specified search frame.
 */
template <typename TABLE_TYPE>
inline int32 FindLowKeyIndex(
	const TABLE_TYPE* FrameTable, 
	int32 NumKeys, 
	int32 SearchFrame, 
	int32 KeyEstimate)
{
	const int32 LastKeyIndex = NumKeys-1;
	int32 LowKeyIndex = KeyEstimate;

	if (FrameTable[KeyEstimate] <= SearchFrame)
	{
		// unless we find something better, we'll default to the last key
		LowKeyIndex = LastKeyIndex;

		// search forward from the estimate for the first value greater than our search parameter
		// if found, this is the high key and we want the one just prior to it
		for (int32 i = KeyEstimate+1; i <= LastKeyIndex; ++i)
		{
			if (FrameTable[i] > SearchFrame)
			{
				LowKeyIndex= i-1;
				break;
			}
		}
	}
	else
	{
		// unless we find something better, we'll default to the first key
		LowKeyIndex = 0;

		// search backward from the estimate for the first value less than or equal to the search parameter
		// if found, this is the low key we are searching for
		for (int32 i = KeyEstimate-1; i > 0; --i)
		{
			if (FrameTable[i] <= SearchFrame)
			{
				LowKeyIndex= i;
				break;
			}
		}
	}

	return LowKeyIndex;
}

/**
 * Utility function to determine the two key indices to interpolate given a relative position in the animation
 *
 * @param	Seq				The UAnimSequence container.
 * @param	FrameTable		The frame table containing a frame index for each key.
 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
  * @param	NumKeys			The number of keys present in the track being solved.
 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
 */
inline float AnimEncoding::TimeToIndex(
	EAnimInterpolationType Interpolation,
	int32 NumberOfFrames,
	const uint8* FrameTable,
	float RelativePos,
	int32 NumKeys,
	int32 &PosIndex0Out,
	int32 &PosIndex1Out)
{
	float Alpha = 0.0f;

	check(NumKeys != 0);
	
	const int32 LastKey= NumKeys-1;
	
	int32 TotalFrames = NumberOfFrames -1;
	int32 EndingKey = LastKey;

	if (NumKeys < 2 || RelativePos <= 0.f)
	{
		// return the first key
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		Alpha = 0.0f;
	}
	else if( RelativePos >= 1.0f )
	{
		// return the ending key
		PosIndex0Out = EndingKey;
		PosIndex1Out = EndingKey;
		Alpha = 0.0f;
	}
	else
	{
		// find the proper key range to return
		const int32 LastFrame= TotalFrames-1;
		const float KeyPos = RelativePos * (float)LastKey;
		const float FramePos = RelativePos * (float)TotalFrames;
		const int32 FramePosFloor = FMath::Clamp(FMath::TruncToInt(FramePos), 0, LastFrame);
		const int32 KeyEstimate = FMath::Clamp(FMath::TruncToInt(KeyPos), 0, LastKey);

		int32 LowFrame = 0;
		int32 HighFrame = 0;
		
		// find the pair of keys which surround our target frame index
		if (NumberOfFrames > 0xFF)
		{
			const uint16* Frames= (uint16*)FrameTable;
			PosIndex0Out = FindLowKeyIndex<uint16>(Frames, NumKeys, FramePosFloor, KeyEstimate);
			LowFrame = Frames[PosIndex0Out];

			PosIndex1Out = PosIndex0Out + 1;
			if (PosIndex1Out > LastKey)
			{
				PosIndex1Out= EndingKey;
			}
			HighFrame= Frames[PosIndex1Out];
		}
		else
		{
			const uint8* Frames= (uint8*)FrameTable;
			PosIndex0Out = FindLowKeyIndex<uint8>(Frames, NumKeys, FramePosFloor, KeyEstimate);
			LowFrame = Frames[PosIndex0Out];

			PosIndex1Out = PosIndex0Out + 1;
			if (PosIndex1Out > LastKey)
			{
				PosIndex1Out= EndingKey;
			}
			HighFrame= Frames[PosIndex1Out];
		}

		// compute the blend parameters for the keys we have found
		int32 Delta= FMath::Max(HighFrame - LowFrame, 1);
		const float Remainder = (FramePos - (float)LowFrame);
		Alpha = Interpolation == EAnimInterpolationType::Step ? 0.f : (Remainder / (float)Delta);
	}
	
	return Alpha;
}
