// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding_ConstantKeyLerp.h: Constant key compression.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "AnimEncoding.h"
#include "AnimationCompression.h"

class FMemoryWriter;

/**
 * Base class for all Animation Encoding Formats using consistently-spaced key interpolation.
 */
class AEFConstantKeyLerpShared : public AnimEncodingLegacyBase
{
public:
	/**
	 * Handles the ByteSwap of compressed rotation data on import
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	RotTrackData	The compressed rotation data stream.
	 * @param	NumKeysRot		The number of keys present in the stream.
	 */
	virtual void ByteSwapRotationIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& RotTrackData,
		int32 NumKeysRot) override;

	/**
	 * Handles the ByteSwap of compressed translation data on import
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	TransTrackData	The compressed translation data stream.
	 * @param	NumKeysTrans	The number of keys present in the stream.
	 */
	virtual void ByteSwapTranslationIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& TransTrackData,
		int32 NumKeysTrans) override;

	/**
	 * Handles the ByteSwap of compressed Scale data on import
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryReader	The FMemoryReader to read from.
	 * @param	ScaleTrackData	The compressed Scale data stream.
	 * @param	NumKeysScale	The number of keys present in the stream.
	 */
	virtual void ByteSwapScaleIn(
		FUECompressedAnimData& CompressedData,
		FMemoryReader& MemoryReader,
		uint8*& ScaleTrackData,
		int32 NumKeysScale) override;


	/**
	 * Handles the ByteSwap of compressed rotation data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryWriter to write to.
	 * @param	RotTrackData	The compressed rotation data stream.
	 * @param	NumKeysRot		The number of keys to write to the stream.
	 */
	virtual void ByteSwapRotationOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& RotTrackData,
		int32 NumKeysRot) override;

	/**
	 * Handles the ByteSwap of compressed translation data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryWriter to write to.
	 * @param	TransTrackData	The compressed translation data stream.
	 * @param	NumKeysTrans	The number of keys to write to the stream.
	 */
	virtual void ByteSwapTranslationOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& TransTrackData,
		int32 NumKeysTrans) override;

	/**
	 * Handles the ByteSwap of compressed Scale data on export
	 *
	 * @param	CompressedData	The compressed animation data being operated on.
	 * @param	MemoryWriter	The FMemoryWriter to write to.
	 * @param	TransTrackData	The compressed Scale data stream.
	 * @param	NumKeysTrans	The number of keys to write to the stream.
	 */
	virtual void ByteSwapScaleOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter,
		uint8*& ScaleTrackData,
		int32 NumKeysScale) override;
};

template<int32 FORMAT>
class AEFConstantKeyLerp : public AEFConstantKeyLerpShared
{
public:
	/**
	 * Decompress the Rotation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomRotation(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex) override;

	/**
	 * Decompress the Translation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomTranslation(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex) override;

	/**
	 * Decompress the Scale component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	virtual void GetBoneAtomScale(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex) override;

#if USE_ANIMATION_CODEC_BATCH_SOLVER

	/**
	 * Decompress all requested rotation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	void GetPoseRotations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext);

	/**
	 * Decompress all requested translation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	void GetPoseTranslations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext);

	/**
	 * Decompress all requested Scale components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	void GetPoseScales(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext);
#endif

};



/**
 * Decompress the Rotation component of a BoneAtom
 *
 * @param	OutAtom			The FTransform to fill in.
 * @param	DecompContext	The decompression context to use.
 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetBoneAtomRotation(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

	const int32* RESTRICT TrackData = AnimData.CompressedTrackOffsets.GetData() + (TrackIndex * 4);
	int32 RotKeysOffset = TrackData[2];
	int32 NumRotKeys = TrackData[3];
	const uint8* RESTRICT RotStream = AnimData.CompressedByteStream.GetData() + RotKeysOffset;

	if (NumRotKeys == 1)
	{
		// For a rotation track of n=1 keys, the single key is packed as an FQuatFloat96NoW.
		FQuat4f R0;
		DecompressRotation<ACF_Float96NoW>( R0, RotStream, RotStream );
		OutAtom.SetRotation( FQuat(R0) );
	}
	else
	{
		int32 Index0;
		int32 Index1;
		float Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumRotKeys, DecompContext.Interpolation, Index0, Index1);

		const int32 RotationStreamOffset = (FORMAT == ACF_IntervalFixed32NoW) ? (sizeof(float)*6) : 0; // offset past Min and Range data

		if (Index0 != Index1)
		{

			// unpack and lerp between the two nearest keys
			const uint8* RESTRICT KeyData0= RotStream + RotationStreamOffset +(Index0*CompressedRotationStrides[FORMAT]*CompressedRotationNum[FORMAT]);
			const uint8* RESTRICT KeyData1= RotStream + RotationStreamOffset +(Index1*CompressedRotationStrides[FORMAT]*CompressedRotationNum[FORMAT]);
			FQuat4f R0;
			FQuat4f R1;
			DecompressRotation<FORMAT>( R0, RotStream, KeyData0 );
			DecompressRotation<FORMAT>( R1, RotStream, KeyData1 );

			// Fast linear quaternion interpolation.
			FQuat4f BlendedQuat = FQuat4f::FastLerp(R0, R1, Alpha);
			BlendedQuat.Normalize();
			OutAtom.SetRotation( FQuat(BlendedQuat) );
		}
		else // (Index0 == Index1)
		{
			
			// unpack a single key
			const uint8* RESTRICT KeyData= RotStream + RotationStreamOffset +(Index0*CompressedRotationStrides[FORMAT]*CompressedRotationNum[FORMAT]);
			FQuat4f R0;
			DecompressRotation<FORMAT>( R0, RotStream, KeyData );
			OutAtom.SetRotation( FQuat(R0) );
		}
	}
}

/**
 * Decompress the Translation component of a BoneAtom
 *
 * @param	OutAtom			The FTransform to fill in.
 * @param	DecompContext	The decompression context to use.
 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetBoneAtomTranslation(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

	const int32* RESTRICT TrackData = AnimData.CompressedTrackOffsets.GetData() + (TrackIndex * 4);
	int32 TransKeysOffset = TrackData[0];
	int32 NumTransKeys = TrackData[1];
	const uint8* RESTRICT TransStream = AnimData.CompressedByteStream.GetData() + TransKeysOffset;

	int32 Index0;
	int32 Index1;
	float Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumTransKeys, DecompContext.Interpolation, Index0, Index1);

	const int32 TransStreamOffset = ((FORMAT == ACF_IntervalFixed32NoW) && NumTransKeys > 1) ? (sizeof(float)*6) : 0; // offset past Min and Range data

	if (Index0 != Index1)
	{
		const uint8* RESTRICT KeyData0 = TransStream + TransStreamOffset + Index0*CompressedTranslationStrides[FORMAT]*CompressedTranslationNum[FORMAT];
		const uint8* RESTRICT KeyData1 = TransStream + TransStreamOffset + Index1*CompressedTranslationStrides[FORMAT]*CompressedTranslationNum[FORMAT];
		FVector3f P0;
		FVector3f P1;
		DecompressTranslation<FORMAT>( P0, TransStream, KeyData0 );
		DecompressTranslation<FORMAT>( P1, TransStream, KeyData1 );
		OutAtom.SetTranslation( (FVector)FMath::Lerp( P0, P1, Alpha ) );
	}
	else // (Index0 == Index1)
	{
		// unpack a single key
		const uint8* RESTRICT KeyData = TransStream + TransStreamOffset + Index0*CompressedTranslationStrides[FORMAT]*CompressedTranslationNum[FORMAT];
		FVector3f P0;
		DecompressTranslation<FORMAT>( P0, TransStream, KeyData);
		OutAtom.SetTranslation((FVector)P0);
	}
}

/**
 * Decompress the Scale component of a BoneAtom
 *
 * @param	OutAtom			The FTransform to fill in.
 * @param	DecompContext	The decompression context to use.
 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
 */
template<int32 FORMAT>
inline void AEFConstantKeyLerp<FORMAT>::GetBoneAtomScale(FTransform& OutAtom, const FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex)
{
	const FUECompressedAnimData& AnimData = static_cast<const FUECompressedAnimData&>(DecompContext.CompressedAnimData);

	const int32 ScaleKeysOffset = AnimData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 0);
	const int32 NumScaleKeys = AnimData.CompressedScaleOffsets.GetOffsetData(TrackIndex, 1);
	const uint8* RESTRICT ScaleStream = AnimData.CompressedByteStream.GetData() + ScaleKeysOffset;

	int32 Index0;
	int32 Index1;
	float Alpha = TimeToIndex(DecompContext.GetPlayableLength(), DecompContext.GetRelativePosition(), NumScaleKeys, DecompContext.Interpolation, Index0, Index1);

	const int32 ScaleStreamOffset = ((FORMAT == ACF_IntervalFixed32NoW) && NumScaleKeys > 1) ? (sizeof(float)*6) : 0; // offset past Min and Range data

	if (Index0 != Index1)
	{
		const uint8* RESTRICT KeyData0 = ScaleStream + ScaleStreamOffset + Index0*CompressedScaleStrides[FORMAT]*CompressedScaleNum[FORMAT];
		const uint8* RESTRICT KeyData1 = ScaleStream + ScaleStreamOffset + Index1*CompressedScaleStrides[FORMAT]*CompressedScaleNum[FORMAT];
		FVector3f P0;
		FVector3f P1;
		DecompressScale<FORMAT>( P0, ScaleStream, KeyData0 );
		DecompressScale<FORMAT>( P1, ScaleStream, KeyData1 );
		OutAtom.SetScale3D( (FVector)FMath::Lerp( P0, P1, Alpha ) );
	}
	else // (Index0 == Index1)
	{
		// unpack a single key
		const uint8* RESTRICT KeyData = ScaleStream + ScaleStreamOffset + Index0*CompressedScaleStrides[FORMAT]*CompressedScaleNum[FORMAT];
		FVector3f P0;
		DecompressScale<FORMAT>( P0, ScaleStream, KeyData);
		OutAtom.SetScale3D((FVector)P0);
	}
}
