// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimEncoding_PerTrackCompression.h: Per-track decompressor.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "AnimEncoding.h"

class FMemoryArchive;

/**
 * Decompression codec for the per-track compressor.
 */
class AEFPerTrackCompressionCodec : public AnimEncoding
{
public:
	/**
	 * Handles Byte-swapping incoming animation data from a MemoryReader
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryReader		The MemoryReader object to read from.
	 * @param	SourceArVersion		The version of the archive that the data is coming from.
	 */
	virtual void ByteSwapIn(FUECompressedAnimData& CompressedData, FMemoryReader& MemoryReader) override;

	/**
	 * Handles Byte-swapping outgoing animation data to an array of BYTEs
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	SerializedData		The output buffer.
	 * @param	ForceByteSwapping	true is byte swapping is not optional.
	 */
	virtual void ByteSwapOut(
		FUECompressedAnimData& CompressedData,
		FMemoryWriter& MemoryWriter) override;

#if USE_ANIMATION_CODEC_BATCH_SOLVER

	/**
	 * Decompress all requested rotation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseRotations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) override;

	/**
	 * Decompress all requested translation components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseTranslations(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) override;

	/**
	 * Decompress all requested Scale components from an Animation Sequence
	 *
	 * @param	Atoms			The FTransform array to fill in.
	 * @param	DesiredPairs	Array of requested bone information
	 * @param	DecompContext	The decompression context to use.
	 */
	virtual void GetPoseScales(
		TArrayView<FTransform>& Atoms,
		const BoneTrackArray& DesiredPairs,
		FAnimSequenceDecompressionContext& DecompContext) override;
#endif

protected:
	/**
	 * Handles Byte-swapping a single track of animation data from a MemoryReader or to a MemoryWriter
	 *
	 * @param	CompressedData		The compressed animation data being operated on.
	 * @param	MemoryStream		The MemoryReader or MemoryWriter object to read from/write to.
	 * @param	Offset				The starting offset into the compressed byte stream for this track (can be INDEX_NONE to indicate an identity track)
	 */
	template<class TArchive>
	static void ByteSwapOneTrack(FUECompressedAnimData& CompressedData, TArchive& MemoryStream, int32 BufferStart, int32 Offset);

	/**
	 * Preserves 4 byte alignment within a stream
	 *
	 * @param	TrackData [inout]	The current data offset (will be returned four byte aligned from the start of the compressed byte stream)
	 * @param	MemoryStream		The MemoryReader or MemoryWriter object to read from/write to.
	 */
	static void PreservePadding(uint8*& TrackData, FMemoryArchive& MemoryStream);

	/**
	 * Decompress the Rotation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	static void GetBoneAtomRotation(
		FTransform& OutAtom,
		FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex);

	/**
	 * Decompress the Translation component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	static void GetBoneAtomTranslation(
		FTransform& OutAtom,
		FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex);

	/**
	 * Decompress the Scale component of a BoneAtom
	 *
	 * @param	OutAtom			The FTransform to fill in.
	 * @param	DecompContext	The decompression context to use.
	 * @param	TrackIndex		The index of the track desired in the Animation Sequence.
	 */
	static void GetBoneAtomScale(
		FTransform& OutAtom,
		FAnimSequenceDecompressionContext& DecompContext,
		int32 TrackIndex);

	friend class UAnimCompress_PerTrackCompression;
};
