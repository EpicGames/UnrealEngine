// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InterpFilter.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

//======================================================================================================================
struct UE_DEPRECATED(5.0, "FFIRFilter is no longer used or supported") FFIRFilter
{
public:
	FFIRFilter()
	{
	}

	FFIRFilter(int32 WindowLen)
	{
		Initialize(WindowLen);
	}

	void Initialize(int32 WindowLen)
	{
		if ( WindowLen > 0 )
		{
			FilterData.AddZeroed(WindowLen);
			Coefficients.AddZeroed(WindowLen);
			CurrentStack = 0;
		}
		else
		{
			FilterData.Reset();
			Coefficients.Reset();
			CurrentStack = 0;		
		}
	}

	void CalculateCoefficient(EFilterInterpolationType InterpolationType);
	float GetFilteredData(float Input);
	bool IsValid() const { return FilterData.Num() > 0; }
	float				LastOutput;
private:

	// CurrentStack is latest till CurrentStack + 1 is oldest
	// note that this works in reverse order with Coefficient
	TArray<float>		FilterData;
	// n-1 is latest till 0 is oldest
	TArray<float>		Coefficients;
	int32					CurrentStack;

	float GetStep() const
	{
		check( IsValid() );
		return 1.f/(float)Coefficients.Num();
	}

	float GetInterpolationCoefficient (EFilterInterpolationType InterpolationType, int32 CoefficientIndex) const;
	float CalculateFilteredOutput() const;
};

//======================================================================================================================
struct FFilterData
{
	float Input;
	float Time;

	FFilterData()
	{
	}

	FFilterData(const float InInput, const float InTime)
		: Input(InInput)
		, Time(InTime)
	{
	}

	void EnsureTimeIsValid(const float CurrentTime, const float ValidationWindow)
	{
		if (Diff(CurrentTime) > ValidationWindow)
		{
			Time = 0.f;
		}
	}

	bool IsValid() const
	{
		return (Time > 0.f);
	}

	float Diff(const float InTime) const
	{
		return (InTime - Time);
	}

	void SetInput(const float InData, const float InTime)
	{
		Input = InData;
		Time = InTime;
	}
};

//======================================================================================================================
struct FFIRFilterTimeBased
{
public:
	FFIRFilterTimeBased()
	{
	}

	FFIRFilterTimeBased(float InWindowDuration, EFilterInterpolationType InInterpolationType, float InDampingRatio,
	                    float InMin, float InMax, float InMaxSpeed, bool bInClamp)
	{
		Initialize(InWindowDuration, InInterpolationType, InDampingRatio, InMin, InMax, InMaxSpeed, bInClamp);
	}

	void Initialize(float InWindowDuration, EFilterInterpolationType InInterpolationType, float InDampingRatio,
	                float InMinValue, float InMaxValue, float InMaxSpeed, bool bInClamp)
	{
		InterpolationType = InInterpolationType;
		FilterData.Empty();
		CurrentStackIndex = 0;
		WindowDuration = InWindowDuration;
		DampingRatio = InDampingRatio;
		MinValue = InMinValue;
		MaxValue = InMaxValue;
		MaxSpeed = InMaxSpeed;
		bClamp = bInClamp;
		CurrentTime = 0.f;
		LastOutput = 0.f;
	}

	// These parameters can be modified at runtime without needing to re-initialize
	void SetParams(float InDampingRatio, float InMinValue, float InMaxValue, float InMaxSpeed, bool bInClamp)
	{
		DampingRatio = InDampingRatio;
		MinValue = InMinValue;
		MaxValue = InMaxValue;
		MaxSpeed = InMaxSpeed;
		bClamp = bInClamp;
	}

	// This adds Input to the stack representing an update of DeltaTime, and returns the new filtered value. 
	ENGINE_API float UpdateAndGetFilteredData(float Input, float DeltaTime);

	// Wraps the internal state by steps of Range so that it is as close as possible to Input
	ENGINE_API void WrapToValue(float Input, float Range);

	// Sets internal state to the specified value
	ENGINE_API void SetToValue(float Input);

	// Filter is considered valid if the WindowDuration is > 0
	bool IsValid() const { return WindowDuration > 0.f; }

	void SetWindowDuration(float InWindowDuration)
	{
		WindowDuration = InWindowDuration;
	}

#if WITH_EDITOR
	bool NeedsUpdate(const EFilterInterpolationType InType, const float InWindowDuration)
	{
		return InterpolationType != InType || WindowDuration != InWindowDuration;
	}
#endif // WITH_EDITOR

	float LastOutput;

private:
	EFilterInterpolationType InterpolationType;
	int32 CurrentStackIndex;
	float WindowDuration;
	float DampingRatio;
	float MinValue;
	float MaxValue;
	float MaxSpeed;
	bool  bClamp;
	float CurrentTime;
	TArray<FFilterData> FilterData;

	ENGINE_API float GetInterpolationCoefficient(const FFilterData& Data) const;
	ENGINE_API float CalculateFilteredOutput();
	ENGINE_API int32 GetSafeCurrentStackIndex();
	ENGINE_API void RefreshValidFilters();
};
