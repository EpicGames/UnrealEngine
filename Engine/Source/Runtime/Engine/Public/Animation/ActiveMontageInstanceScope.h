// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeMessages.h"

namespace UE::Anim {
	
// Notify Context Data containing the Instance ID of the montage that triggered this notify
// This is useful i.e When there's simultaneous instances of the same montage, to trigger start/end events of its notify states.
class FAnimNotifyMontageInstanceContext : public UE::Anim::IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE_API(FAnimNotifyMontageInstanceContext, ENGINE_API)
public:
	ENGINE_API FAnimNotifyMontageInstanceContext(const int32 InMontageInstanceID);
	const int32 MontageInstanceID;
};

}	// namespace UE::Anim
