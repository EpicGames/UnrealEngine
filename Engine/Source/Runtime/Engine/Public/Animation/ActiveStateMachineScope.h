// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSync.h"
#include "ActiveStateMachineScope.generated.h"

struct FAnimationBaseContext;
struct FAnimInstanceProxy;
struct FAnimNode_StateMachine;

USTRUCT()
struct FEncounteredStateMachineStack
{
	GENERATED_BODY()
	FEncounteredStateMachineStack() {} 
	ENGINE_API FEncounteredStateMachineStack(const FEncounteredStateMachineStack& ParentStack, int32 InStateMachineIndex, int32 InStateIndex);
	ENGINE_API FEncounteredStateMachineStack(int32 InStateMachineIndex, int32 InStateIndex);
	struct FStateMachineEntry
	{
		FStateMachineEntry() {}
		FStateMachineEntry(int32 InStateMachineIndex, int32 InStateIndex)
		: StateMachineIndex(InStateMachineIndex)
		, StateIndex(InStateIndex)
		{}
		
		int32 StateMachineIndex = INDEX_NONE;
		int32 StateIndex = INDEX_NONE;;
	};
	TArray<FStateMachineEntry, TInlineAllocator<4, TMemStackAllocator<>>> StateStack;
	static ENGINE_API FEncounteredStateMachineStack InitStack(const FEncounteredStateMachineStack& ParentStack, int32 InStateMachineIndex, int32 InStateIndex);
	static ENGINE_API FEncounteredStateMachineStack InitStack(int32 InStateMachineIndex, int32 InStateIndex);
};

namespace UE { namespace Anim {
	
class FAnimNotifyStateMachineContext : public UE::Anim::IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyStateMachineContext)
public:
	ENGINE_API FAnimNotifyStateMachineContext(const FEncounteredStateMachineStack& InEncounteredStateMachines);
	ENGINE_API bool IsStateMachineInContext(int32 StateMachineIndex) const;
	ENGINE_API bool IsStateInStateMachineInContext(int32 StateMachineIndex, int32 StateIndex) const; 
	const FEncounteredStateMachineStack EncounteredStateMachines;
};

class FActiveStateMachineScope : public UE::Anim::IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE(FActiveStateMachineScope);
public:
	ENGINE_API FActiveStateMachineScope(const FAnimationBaseContext& InContext, FAnimNode_StateMachine* StateMachine, int32 InStateIndex);

	static ENGINE_API int32 GetStateMachineIndex(FAnimNode_StateMachine* StateMachine, const FAnimationBaseContext& Context);
	ENGINE_API virtual TUniquePtr<const IAnimNotifyEventContextDataInterface> MakeUniqueEventContextData() const override;
	
	const FEncounteredStateMachineStack ActiveStateMachines; 
};

}}	// namespace UE::Anim
