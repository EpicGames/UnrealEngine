// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "UObject/NameTypes.h"

namespace UE::Anim
{

// Built-in attributes that most nodes will share
struct FAttributes
{
	static ENGINE_API const FName Pose;
	static ENGINE_API const FName Curves;
	static ENGINE_API const FName Attributes;
};

}	// namespace UE::Anim
