// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"

struct FAnimationBaseContext;

namespace UE { namespace Anim {

// Scoped graph message used to synchronize animations at various points in an anim graph
class FAnimBlendDebugScope : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FAnimBlendDebugScope, ENGINE_API);

public:
	ENGINE_API FAnimBlendDebugScope(const FAnimationBaseContext& InContext, const int32& InBlendIndex = INDEX_NONE, const int32& InNumBlends = 0, const FColor& InDebugColor = FColor::Black);

	// The index for this anim, if any
	int32 BlendIndex = INDEX_NONE;

	// Number of current blends, if any
	int32 NumBlends = 0;

	// A color used to identify this anim
	FColor DebugColor = FColor::Black;

private:
	// The node ID that was used when this scope was entered
	int32 NodeId;

};

}}	// namespace UE::Anim
