// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/NamedValueArray.h"

namespace UE::Anim
{

/** Name/index pair, used in FBulkCurves */
struct FNamedIndexElement
{
	FNamedIndexElement() = default;

	FNamedIndexElement(FName InName, int32 InIndex)
		: Name(InName)
		, Index(InIndex)
	{}

	FName Name = NAME_None;
	int32 Index = INDEX_NONE;
};

/**
 * Named value array used for bulk get/set of curves.
 * @see UE::Anim::FCurveUtils::BulkGet
 * @see UE::Anim::FCurveUtils::BulkSet
 */
struct FBulkCurves : TNamedValueArray<FDefaultAllocator, FNamedIndexElement>
{
	/**
	 * Add a named element and index it
	 * Note that this should only really be used when building a fresh value array, as using this during runtime can
	 * introduce duplicate values. 
	 */
	void AddIndexed(FName InName)
	{
		Elements.Emplace(InName, Elements.Num());
		bSorted = false;
	}
};

};
