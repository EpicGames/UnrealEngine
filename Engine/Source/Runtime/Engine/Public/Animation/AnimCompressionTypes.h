// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Misc/MemStack.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimationAsset.h"

#include "Async/MappedFileHandle.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/Paths.h"
#include "Serialization/BulkData.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "ProfilingDebugging/CsvProfiler.h"
#include "Animation/AnimationDecompression.h"
#include "BoneContainer.h"

#if WITH_EDITOR
#include "ReferenceSkeleton.h"
#endif

#include "AnimCompressionTypes.generated.h"

/**
 * Indicates animation data key format.
 */
UENUM()
enum AnimationKeyFormat : int
{
	AKF_ConstantKeyLerp,
	AKF_VariableKeyLerp,
	AKF_PerTrackCompression,
	AKF_MAX,
};

class FMemoryReader;
class FMemoryWriter;

class UAnimCompress;
class UAnimCurveCompressionSettings;
class UAnimBoneCompressionSettings;
class UAnimBoneCompressionCodec;
class USkeleton;
struct FCompactPose;
struct FBoneAnimationTrack;

template<typename ArrayClass>
struct FCompressedOffsetDataBase
{
	ArrayClass OffsetData;

	int32 StripSize;

	FCompressedOffsetDataBase(int32 InStripSize = 2)
		: StripSize(InStripSize)
	{}

	void SetStripSize(int32 InStripSize)
	{
		ensure(InStripSize > 0);
		StripSize = InStripSize;
	}

	const int32 GetOffsetData(int32 StripIndex, int32 Offset) const
	{
		checkSlow(OffsetData.IsValidIndex(StripIndex * StripSize + Offset));

		return OffsetData[StripIndex * StripSize + Offset];
	}

	void SetOffsetData(int32 StripIndex, int32 Offset, int32 Value)
	{
		checkSlow(OffsetData.IsValidIndex(StripIndex * StripSize + Offset));
		OffsetData[StripIndex * StripSize + Offset] = Value;
	}

	void AddUninitialized(int32 NumOfTracks)
	{
		OffsetData.AddUninitialized(NumOfTracks*StripSize);
	}

	void Empty(int32 NumOfTracks = 0)
	{
		OffsetData.Empty(NumOfTracks*StripSize);
	}

	int32 GetMemorySize() const
	{
		return sizeof(int32)*OffsetData.Num() + sizeof(int32);
	}

	int32 GetNumTracks() const
	{
		return OffsetData.Num() / StripSize;
	}

	bool IsValid() const
	{
		return (OffsetData.Num() > 0);
	}
};

/**
* A set of error statistics for an animation, gathered by FAnimationUtils::ComputeCompressionError
*/
USTRUCT()
struct FAnimationErrorStats
{
	GENERATED_USTRUCT_BODY()

	/** Average world-space translation error across all end-effectors **/
	float AverageError;
	/** The worst error encountered across all end effectors **/
	float MaxError;
	/** Time at which the worst error occurred */
	float MaxErrorTime;
	/** Bone on which the worst error occurred */
	int32 MaxErrorBone;

	FAnimationErrorStats()
		: AverageError(0.f)
		, MaxError(0.f)
		, MaxErrorTime(0.f)
		, MaxErrorBone(0)
	{}
};

FArchive& operator<<(FArchive& Ar, FAnimationErrorStats& ErrorStats);

// Helper for buiilding DDC keys of settings
struct FArcToHexString
{
private:
	TArray<uint8> TempBytes;

public:
	FMemoryWriter Ar;

	FArcToHexString()
		: Ar(TempBytes)
	{
		TempBytes.Reserve(64);
	}

	FString MakeString() const
	{
		FString Key;
		const uint8* SettingsAsBytes = TempBytes.GetData();
		Key.Reserve(TempBytes.Num() + 1);
		for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
		{
			ByteToHex(SettingsAsBytes[ByteIndex], Key);
		}
		return Key;
	}
};

struct FCompressedOffsetData : public FCompressedOffsetDataBase<TArray<int32>>
{
};

FArchive& operator<<(FArchive& Ar, FCompressedOffsetData& D);

// Mechanism for cancelling in flight compression
struct FCancelCompressionSignal
{
private:
	FThreadSafeBool Signal;

public:
	FCancelCompressionSignal() = default;
	FCancelCompressionSignal(const FCancelCompressionSignal&) = default;

	FCancelCompressionSignal& operator=(const FCancelCompressionSignal& Other)
	{
		Signal = (bool)Other.Signal;
		return *this;
	}

	void Cancel()
	{
		Signal = true;
	}

	bool IsCancelled() const
	{
		return Signal;
	}
};

#if WITH_EDITOR
struct FCompressibleAnimData
{
public:
	ENGINE_API FCompressibleAnimData();

	ENGINE_API FCompressibleAnimData(UAnimBoneCompressionSettings* InBoneCompressionSettings, UAnimCurveCompressionSettings* InCurveCompressionSettings, USkeleton* InSkeleton, EAnimInterpolationType InInterpolation, float InSequenceLength, int32 InNumberOfKeys, const ITargetPlatform* InTargetPlatform);

	ENGINE_API FCompressibleAnimData(class UAnimSequence* InSeq, const bool bPerformStripping, const ITargetPlatform* InTargetPlatform);

	ENGINE_API FCompressibleAnimData(const FCompressibleAnimData&);
	ENGINE_API FCompressibleAnimData& operator=(const FCompressibleAnimData&);

	TObjectPtr<UAnimCurveCompressionSettings> CurveCompressionSettings;

	TObjectPtr<UAnimBoneCompressionSettings> BoneCompressionSettings;

	// Data from USkeleton
	TArray<FTransform> RefLocalPoses;
	FReferenceSkeleton RefSkeleton;

	TArray<FTrackToSkeletonMap> TrackToSkeletonMapTable;

	TArray<FRawAnimSequenceTrack> RawAnimationData;

	TArray<FRawAnimSequenceTrack> AdditiveBaseAnimationData;

	EAnimInterpolationType Interpolation;

	TArray<FBoneData> BoneData;

	TArray<FFloatCurve> RawFloatCurves;

	double SequenceLength;

	/** Number of keys within the (non-uniform) RawAnimationData tracks */
	int32 NumberOfKeys;

	int32 GetNumberOfFrames() const { return FMath::Max(NumberOfKeys - 1, 1); }

	EAdditiveAnimationType AdditiveType = AAT_None;
	bool bIsValidAdditive;

	float ErrorThresholdScale;

	//For Logging
	FString Name;
	FString FullName;
	FName   AnimFName;

	FFrameRate SampledFrameRate;

	FCancelCompressionSignal IsCancelledSignal;

	bool bShouldPerformStripping = false;
	TWeakObjectPtr<UAnimSequence> WeakSequence;
	bool bDataFetched = false;

	const ITargetPlatform* TargetPlatform = nullptr;

	static int64 GetApproxRawDataArraySize(const TArray<FRawAnimSequenceTrack>& AnimData)
	{
		int64 Total = sizeof(FRawAnimSequenceTrack) * int64(AnimData.Num());
		for (int32 i = 0; i < AnimData.Num(); ++i)
		{
			const FRawAnimSequenceTrack& RawTrack = AnimData[i];
			Total +=
				sizeof(FVector) * int64(RawTrack.PosKeys.Num()) +
				sizeof(FQuat) * int64(RawTrack.RotKeys.Num()) +
				sizeof(FVector) * int64(RawTrack.ScaleKeys.Num());
		}

		return Total;
	}

	int64 GetApproxRawBoneSize() const
	{
		return GetApproxRawDataArraySize(RawAnimationData);
	}

	int64 GetApproxRawCurveSize() const
	{
		int64 Total = 0;
		for (const FFloatCurve& Curve : RawFloatCurves)
		{
			Total += sizeof(FFloatCurve);
			Total += sizeof(FRichCurveKey) * int64(Curve.FloatCurve.Keys.Num());
		}
		return Total;
	}

	int64 GetApproxRawSize() const
	{
		return GetApproxRawBoneSize() + GetApproxRawCurveSize();
	}

	uint64 GetApproxMemoryUsage() const
	{
		const uint64 MemUsage	= GetApproxRawSize()
								+ TrackToSkeletonMapTable.GetAllocatedSize()
								+ GetApproxRawDataArraySize(AdditiveBaseAnimationData)
								+ BoneData.GetAllocatedSize();
		return MemUsage;
	}

	ENGINE_API bool FetchData(const ITargetPlatform* InPlatform);
	
	ENGINE_API void Update(struct FCompressedAnimSequence& CompressedData) const;

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(BoneCompressionSettings);
		Collector.AddReferencedObject(CurveCompressionSettings);
	}

	bool IsCancelled() const
	{
		return IsCancelledSignal.IsCancelled();
	}
	
	static int32 GUseHighFidelityAdditiveCurveCompression;

protected:
	ENGINE_API void BakeOutAdditiveIntoRawData(const FFrameRate& SampleRate, TArray<FBoneAnimationTrack>& ResampledTrackData, TArray<FFloatCurve>& FloatCurves);
	ENGINE_API void ResampleAnimationTrackData(const FFrameRate& SampleRate, TArray<FBoneAnimationTrack>& ResampledTrackData) const;

private:
	ENGINE_API void WriteCompressionDataToJSON(TArrayView<FName> OriginalTrackNames, TArrayView<FRawAnimSequenceTrack> FinalRawAnimationData, TArrayView<FName> FinalTrackNames) const;
};

typedef TSharedPtr<FCompressibleAnimData, ESPMode::ThreadSafe> FCompressibleAnimPtr;
typedef TSharedRef<FCompressibleAnimData, ESPMode::ThreadSafe> FCompressibleAnimRef;

#endif // WITH_EDITOR

// Wrapper Code
template <typename T>
struct TArrayMaker
{
	using Type = TArray<T>;
};


template <typename T>
struct TNonConstArrayViewMaker
{
	using Type = TArrayView<T>;
};

template <typename T>
struct TArrayViewMaker
{
	using Type = TArrayView<const T>;
};

template <template <typename> class ContainerTypeMakerTemplate>
struct FCompressedAnimDataBase
{
	/**
	 * An array of 4*NumTrack ints, arranged as follows: - PerTrack is 2*NumTrack, so this isn't true any more
	 *   [0] Trans0.Offset
	 *   [1] Trans0.NumKeys
	 *   [2] Rot0.Offset
	 *   [3] Rot0.NumKeys
	 *   [4] Trans1.Offset
	 *   . . .
	 */
	typename ContainerTypeMakerTemplate<int32>::Type CompressedTrackOffsets;

	/**
	 * An array of 2*NumTrack ints, arranged as follows:
		if identity, it is offset
		if not, it is num of keys
	 *   [0] Scale0.Offset or NumKeys
	 *   [1] Scale1.Offset or NumKeys

	 * @TODO NOTE: first implementation is offset is [0], numkeys [1]
	 *   . . .
	 */
	FCompressedOffsetDataBase<typename ContainerTypeMakerTemplate<int32>::Type>  CompressedScaleOffsets;

	typename ContainerTypeMakerTemplate<uint8>::Type CompressedByteStream;

	/**
	 * The runtime interface to decode and byte swap the compressed animation
	 * May be NULL. Set at runtime - does not exist in editor
	 */
	class AnimEncoding* TranslationCodec;
	class AnimEncoding* RotationCodec;
	class AnimEncoding* ScaleCodec;

	enum AnimationKeyFormat KeyEncodingFormat;

	/** The compression format that was used to compress tracks parts. */
	AnimationCompressionFormat TranslationCompressionFormat;
	AnimationCompressionFormat RotationCompressionFormat;
	AnimationCompressionFormat ScaleCompressionFormat;
	
	template <template <typename> class OtherContainerMaker>
	explicit FCompressedAnimDataBase(FCompressedAnimDataBase<OtherContainerMaker>& InCompressedData)
		: CompressedTrackOffsets(InCompressedData.CompressedTrackOffsets)
		, CompressedByteStream(InCompressedData.CompressedByteStream)

		, TranslationCodec(InCompressedData.TranslationCodec)
		, RotationCodec(InCompressedData.RotationCodec)
		, ScaleCodec(InCompressedData.ScaleCodec)

		, KeyEncodingFormat(InCompressedData.KeyEncodingFormat)
		, TranslationCompressionFormat(InCompressedData.TranslationCompressionFormat)
		, RotationCompressionFormat(InCompressedData.RotationCompressionFormat)
		, ScaleCompressionFormat(InCompressedData.ScaleCompressionFormat)
	{
		CompressedScaleOffsets.OffsetData = InCompressedData.CompressedScaleOffsets.OffsetData;
		CompressedScaleOffsets.StripSize = InCompressedData.CompressedScaleOffsets.StripSize;
	}

	FCompressedAnimDataBase()
		: TranslationCodec(nullptr)
		, RotationCodec(nullptr)
		, ScaleCodec(nullptr)

		, KeyEncodingFormat((AnimationKeyFormat)0)
		, TranslationCompressionFormat((AnimationCompressionFormat)0)
		, RotationCompressionFormat((AnimationCompressionFormat)0)
		, ScaleCompressionFormat((AnimationCompressionFormat)0)
	{
	}
};

struct ICompressedAnimData
{
	/* Common data */
	int32 CompressedNumberOfKeys;

#if WITH_EDITORONLY_DATA
	/** The error stats from the current bone compression codec. */
	FAnimationErrorStats BoneCompressionErrorStats;
#endif

	ICompressedAnimData() = default;
	ENGINE_API ICompressedAnimData(const ICompressedAnimData&);
	ENGINE_API ICompressedAnimData& operator=(const ICompressedAnimData&);
	
	/* Virtual interface codecs must implement */
	virtual ~ICompressedAnimData() = default;

	UE_DEPRECATED(5.5, "SerializeCompressedData now accepts the DataOwner as an argument")
	ENGINE_API virtual void SerializeCompressedData(class FArchive& Ar);

	ENGINE_API virtual void SerializeCompressedData(UObject* DataOwner, class FArchive& Ar);
	virtual void Bind(const TArrayView<uint8> BulkData) = 0;

	virtual int64 GetApproxCompressedSize() const = 0;
	virtual FString GetDebugString() const { return FString(); }
	virtual bool IsValid() const = 0;
};

struct FCompressibleAnimDataResult
{
	TArray<uint8> CompressedByteStream;
	TUniquePtr<ICompressedAnimData> AnimData;

	UAnimBoneCompressionCodec* Codec;

	// Give compression code a place to store some extra data during compression
	// (Needed for PerTrackCompression)
	void* CompressionUserData;

	FCompressibleAnimDataResult()
		: CompressedByteStream()
		, AnimData()
		, Codec(nullptr)
		, CompressionUserData(nullptr)
	{}
};

template<typename T>
TArrayView<T> RebaseTArrayView(const TArrayView<T>& ArrayView, const uint8* OriginalBase, const uint8* NewBase)
{
	if (ArrayView.GetData() != nullptr)
	{
		uint32 Offset = ArrayView.GetData() - OriginalBase;
		T* NewData = (T*)(NewBase + Offset);
		return TArrayView<T>(NewData, ArrayView.Num());
	}
	return ArrayView;
}

struct FUECompressedAnimDataMutable : public ICompressedAnimData, public FCompressedAnimDataBase<TArrayMaker>
{
	FUECompressedAnimDataMutable() = default;

	ENGINE_API void BuildFinalBuffer(TArray<uint8>& OutCompressedByteStream);

	// ICompressedAnimData implementation
	virtual void Bind(const TArrayView<uint8> BulkData) {}
	virtual int64 GetApproxCompressedSize() const override { return (int64)CompressedTrackOffsets.GetTypeSize() * (int64)CompressedTrackOffsets.Num() + (int64)CompressedByteStream.Num() + (int64)CompressedScaleOffsets.GetMemorySize(); }
	virtual bool IsValid() const override { return CompressedByteStream.Num() > 0 || (TranslationCompressionFormat == ACF_Identity && RotationCompressionFormat == ACF_Identity && ScaleCompressionFormat == ACF_Identity); }
};

struct FUECompressedAnimData : public ICompressedAnimData, public FCompressedAnimDataBase<TNonConstArrayViewMaker>
{
	FUECompressedAnimData() = default;

	explicit FUECompressedAnimData(FUECompressedAnimDataMutable& InCompressedData)
		: ICompressedAnimData(InCompressedData)
		, FCompressedAnimDataBase(InCompressedData)
	{}

	ENGINE_API void InitViewsFromBuffer(const TArrayView<uint8> BulkData);

	template<typename TArchive>
	void ByteSwapData(TArrayView<uint8> CompresedData, TArchive& MemoryStream);

	void ByteSwapIn(TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) { ByteSwapData(CompressedData, MemoryStream); }
	void ByteSwapOut(TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) { ByteSwapData(CompressedData, MemoryStream); }

	// ICompressedAnimData implementation
	ENGINE_API virtual void SerializeCompressedData(class FArchive& Ar) override;
	virtual void Bind(const TArrayView<uint8> BulkData) override { InitViewsFromBuffer(BulkData); }

	virtual int64 GetApproxCompressedSize() const override { return (int64)CompressedTrackOffsets.GetTypeSize() * (int64)CompressedTrackOffsets.Num() + (int64)CompressedByteStream.Num() + (int64)CompressedScaleOffsets.GetMemorySize(); }
	ENGINE_API virtual FString GetDebugString() const override;
	virtual bool IsValid() const override { return CompressedByteStream.Num() > 0 || (TranslationCompressionFormat == ACF_Identity && RotationCompressionFormat == ACF_Identity && ScaleCompressionFormat == ACF_Identity); }
};

namespace UE::Animation::Private
{
	[[noreturn]] ENGINE_API void OnInvalidMaybeMappedAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
}

template<uint32 Alignment = DEFAULT_ALIGNMENT>
class TMaybeMappedAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	class ForAnyElementType
	{
	public:

		/** Default constructor. */
		ForAnyElementType()
			: Data(nullptr)
			, MappedHandle(nullptr)
			, MappedRegion(nullptr)
		{}

		/**
		 * Moves the state of another allocator into this one.
		 * Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		 * @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		 */
		void MoveToEmpty(ForAnyElementType& Other)
		{
			checkSlow(this != &Other);

			Reset();

			Data = Other.Data;
			Other.Data = nullptr;

			MappedRegion = Other.MappedRegion;
			Other.MappedRegion = nullptr;

			MappedHandle = Other.MappedHandle;
			Other.MappedHandle = nullptr;
		}

		/** Destructor. */
		~ForAnyElementType()
		{
			Reset();
		}

		// FContainerAllocatorInterface
		FScriptContainerElement* GetAllocation() const
		{
			return Data;
		}
		void ResizeAllocation(
			SizeType CurrentNum,
			SizeType NewMax,
			SIZE_T NumBytesPerElement
		)
		{
			if (MappedRegion || MappedHandle)
			{
				check(NewMax == 0); // Currently we can only support resizing of memory mapped regions to 0 size (ie delete)

				delete MappedRegion;
				delete MappedHandle;
				MappedRegion = nullptr;
				MappedHandle = nullptr;
				Data = nullptr; // make sure we don't try to free this pointer
			}
			else if (Data || NewMax)
			{
				static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

				// Check for under/overflow
				if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
				{
				    UE::Animation::Private::OnInvalidMaybeMappedAllocatorNum(NewMax, NumBytesPerElement);
				}

				// Avoid calling FMemory::Realloc( nullptr, 0 ) as ANSI C mandates returning a valid pointer which is not what we want.
				//checkSlow(((uint64)NewMax*(uint64)ElementTypeInfo.GetSize() < (uint64)INT_MAX));
				Data = (FScriptContainerElement*)FMemory::Realloc(Data, NewMax*NumBytesPerElement, Alignment);
			}
		}
		SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			check(!MappedHandle && !MappedRegion); // this could be supported, but it probably is never what you want, so we will just assert.
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, true, Alignment);
		}
		SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			check(!MappedHandle && !MappedRegion); // this could be supported, but it probably is never what you want, so we will just assert.
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}
		SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			check(!MappedHandle && !MappedRegion); // this could be supported, but it probably is never what you want, so we will just assert.
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, true, Alignment);
		}

		SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

		void AcceptFileMapping(IMappedFileHandle* InMappedHandle, IMappedFileRegion* InMappedRegion, void *MallocPtr)
		{
			check(!MappedHandle && !Data); // we could support stuff like this, but that usually isn't what we want for streamlined loading
			Reset(); // just in case
			if (InMappedHandle || InMappedRegion)
			{
				MappedHandle = InMappedHandle;
				MappedRegion = InMappedRegion;
				Data = (FScriptContainerElement*)MappedRegion->GetMappedPtr(); //@todo mapped files should probably be const-correct
			}
			else
			{
				Data = (FScriptContainerElement*)MallocPtr;
			}
			check(IsAligned(Data, alignof(FScriptContainerElement)));
		}

		bool IsMapped() const
		{
			return MappedRegion || MappedHandle;
		}
	private:

		FScriptContainerElement* Data;
		IMappedFileHandle* MappedHandle;
		IMappedFileRegion* MappedRegion;

		void Reset()
		{
			if (MappedRegion || MappedHandle)
			{
				delete MappedRegion;
				delete MappedHandle;
				MappedRegion = nullptr;
				MappedHandle = nullptr;
				Data = nullptr; // make sure we don't try to free this pointer
			}
			if (Data)
			{
				FMemory::Free(Data);
				Data = nullptr;
			}
		}


		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);
	};

	template<typename ElementType>
	class ForElementType : public ForAnyElementType
	{
	public:

		/** Default constructor. */
		ForElementType()
		{}

		ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

// Define the ResizeAllocation function with the regular alignment as exported to avoid bloat
extern template ENGINE_API FORCENOINLINE void TMaybeMappedAllocator<DEFAULT_ALIGNMENT>::ForAnyElementType::ResizeAllocation(SizeType PreviousNumElements, SizeType NumElements, SIZE_T NumBytesPerElement);

template<typename T, uint32 Alignment = DEFAULT_ALIGNMENT>
class TMaybeMappedArray : public TArray<T, TMaybeMappedAllocator<Alignment>>
{
public:
	TMaybeMappedArray()
	{
	}
	TMaybeMappedArray(TMaybeMappedArray&&) = default;
	TMaybeMappedArray(const TMaybeMappedArray&) = default;
	TMaybeMappedArray& operator=(TMaybeMappedArray&&) = default;
	TMaybeMappedArray& operator=(const TMaybeMappedArray&) = default;

	void AcceptOwnedBulkDataPtr(FOwnedBulkDataPtr* OwnedPtr, int32 Num)
	{
		this->ArrayNum = Num;
		this->ArrayMax = Num;
		this->AllocatorInstance.AcceptFileMapping(OwnedPtr->GetMappedHandle(), OwnedPtr->GetMappedRegion(), (void*)OwnedPtr->GetPointer());
		OwnedPtr->RelinquishOwnership();
	}
};

template<typename T, uint32 Alignment>
struct TIsContiguousContainer<TMaybeMappedArray<T, Alignment>>
{
	static constexpr bool Value = TIsContiguousContainer<TArray<T, TMaybeMappedAllocator<Alignment>>>::Value;
};

USTRUCT()
struct FAnimCompressedCurveIndexedName
{
	GENERATED_BODY()

	FAnimCompressedCurveIndexedName() = default;
	
	bool operator<(const FAnimCompressedCurveIndexedName& InOther) const
	{
		return InOther.CurveName.FastLess(CurveName);
	}

	friend FArchive& operator<<(FArchive& Ar, FAnimCompressedCurveIndexedName& Item)
	{
		Ar << Item.CurveName;
		if(Ar.IsCountingMemory())
		{
			Ar << Item.CurveIndex;
		}
		return Ar;
	}
	
	// Name of the curve, used for sorting
	UPROPERTY()
	FName CurveName = NAME_None;

	// Index into the compressed buffer
	int32 CurveIndex = INDEX_NONE;
};

struct FCompressedAnimSequence
{
public:
	UE_NONCOPYABLE(FCompressedAnimSequence)

	/**
	 * Version of TrackToSkeletonMapTable for the compressed tracks. Due to baking additive data
	 * we can end up with a different amount of tracks to the original raw animation and so we must index into the
	 * compressed data using this
	 */
	TArray<struct FTrackToSkeletonMap> CompressedTrackToSkeletonMapTable;

	/** Curve names for each compressed curve track, indices sorted by FName */
	TArray<FAnimCompressedCurveIndexedName> IndexedCurveNames;

	/**
	 * ByteStream for compressed animation data.
	 * The memory layout is dependent on the algorithm used to compress the anim sequence.
	 */
#if WITH_EDITOR
	TArray<uint8> CompressedByteStream;
	FByteBulkData OptionalBulk;
#else
	TMaybeMappedArray<uint8> CompressedByteStream;
#endif

	/* Compressed curve data stream used by AnimCurveCompressionCodec */
	TArray<uint8> CompressedCurveByteStream;

	/* Compressed bone data stream used by AnimBoneCompressionCodec */
	TUniquePtr<ICompressedAnimData> CompressedDataStructure;

	/** The codec used by the compressed data as determined by the bone compression settings. */
	TObjectPtr<class UAnimBoneCompressionCodec> BoneCompressionCodec;

	/** The codec used by the compressed data as determined by the curve compression settings. */
	TObjectPtr<class UAnimCurveCompressionCodec> CurveCompressionCodec;

	// The size of the raw data used to create the compressed data
	int32 CompressedRawDataSize;

	//Temp debug DDC issues
#if WITH_EDITOR
	TArray<FRawAnimSequenceTrack> CompressedRawData; //Temp debug DDC issues
	FName OwnerName;
#endif

#if WITH_EDITOR
	// Whether compressed data has been populated via DDC fetch or local compression.
	bool bHasValidCompressedData = false;
#endif

	FCompressedAnimSequence()
		: BoneCompressionCodec(nullptr)
		, CurveCompressionCodec(nullptr)
		, CompressedRawDataSize(0)
	{}

	ENGINE_API void SerializeCompressedData(FArchive& Ar, bool bDDCData, UObject* DataOwner, USkeleton* Skeleton, UAnimBoneCompressionSettings* BoneCompressionSettings, UAnimCurveCompressionSettings* CurveCompressionSettings, bool bCanUseBulkData=true);

	ENGINE_API void RebuildCurveIndexTable();
	
	int32 GetSkeletonIndexFromTrackIndex(const int32 TrackIndex) const
	{
		return CompressedTrackToSkeletonMapTable[TrackIndex].BoneTreeIndex;
	}

	int32 GetTrackIndexFromSkeletonIndex(const int32 BoneIndex) const
	{
		return CompressedTrackToSkeletonMapTable.IndexOfByPredicate([BoneIndex](const FTrackToSkeletonMap& Entry) { return Entry.BoneTreeIndex == BoneIndex; });
	}

	// Return the number of bytes used
	ENGINE_API SIZE_T GetMemorySize() const;

	ENGINE_API void Reset();
	UE_DEPRECATED(5.8, "ClearCompressedBoneData is not intended for external use and will become private. Use Reset() to clear all compressed data.")
	ENGINE_API void ClearCompressedBoneData();
	UE_DEPRECATED(5.8, "ClearCompressedCurveData is not intended for external use and will become private. Use Reset() to clear all compressed data.")
	ENGINE_API void ClearCompressedCurveData();

	ENGINE_API bool IsValid(const UAnimSequence* AnimSequence, bool bLogInformation=false) const;
	ENGINE_API bool IsBoneDataValid(const UAnimSequence* AnimSequence, bool bLogInformation=false) const;
	ENGINE_API bool IsCurveDataValid(const UAnimSequence* AnimSequence, bool bLogInformation=false) const;
};

struct FRootMotionReset
{

	FRootMotionReset(bool bInEnableRootMotion, ERootMotionRootLock::Type InRootMotionRootLock, bool bInForceRootLock, FTransform InAnimFirstFrame, bool bInIsValidAdditive)
		: AnimFirstFrame(InAnimFirstFrame)
		, RootMotionRootLock(InRootMotionRootLock)
		, bEnableRootMotion(bInEnableRootMotion)
		, bForceRootLock(bInForceRootLock)
		, bIsValidAdditive(bInIsValidAdditive)
	{
	}

	FTransform AnimFirstFrame;

	ERootMotionRootLock::Type RootMotionRootLock;

	bool bEnableRootMotion;

	bool bForceRootLock;

	bool bIsValidAdditive;

	void ResetRootBoneForRootMotion(FTransform& BoneTransform, const FTransform& RefPoseRootTransform) const
	{
		switch (RootMotionRootLock)
		{
		case ERootMotionRootLock::AnimFirstFrame: BoneTransform = AnimFirstFrame; break;
		case ERootMotionRootLock::Zero: BoneTransform = FTransform::Identity; break;
		default:
		case ERootMotionRootLock::RefPose: BoneTransform = RefPoseRootTransform; break;
		}

		if (bIsValidAdditive && RootMotionRootLock != ERootMotionRootLock::AnimFirstFrame)
		{
			//Need to remove default scale here for additives
			BoneTransform.SetScale3D(BoneTransform.GetScale3D() - FVector(1.f));
		}
	}

	void ResetRootBoneForRootMotion(FTransform& BoneTransform, const FBoneContainer& RequiredBones) const
	{
		ResetRootBoneForRootMotion(BoneTransform, RequiredBones.GetRefPoseArray()[0]);
	}
};

#if WITH_EDITORONLY_DATA
namespace UE::Anim::Compression {
	struct FAnimDDCKeyArgs
	{
		ENGINE_API FAnimDDCKeyArgs(const UAnimSequenceBase& AnimSequence);
		ENGINE_API FAnimDDCKeyArgs(const UAnimSequenceBase& AnimSequence, const ITargetPlatform* TargetPlatform);

		const UAnimSequenceBase& AnimSequence;
		const ITargetPlatform* TargetPlatform;
	};

	ENGINE_API FFrameRate GetCompressionFrameRate(const UAnimSequence& AnimSequence, const ITargetPlatform* TargetPlatform);
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
extern FGuid GenerateGuidFromRawAnimData(const TArray<FRawAnimSequenceTrack>& RawAnimationData, const FRawCurveTracks& RawCurveData);
#endif
