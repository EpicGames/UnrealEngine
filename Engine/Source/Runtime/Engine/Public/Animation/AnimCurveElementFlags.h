// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NamedValueArray.h"

namespace UE::Anim
{

// Flags for each curve element
enum class ECurveElementFlags : uint8
{
	// No flags
	None		= 0,

	// Curve applies to materials
	Material	= 0x01,

	// Curve applies to morph targets
	MorphTarget	= 0x02,
};

ENUM_CLASS_FLAGS(ECurveElementFlags);

// An element for named curve flags
struct FCurveElementFlags
{
	FCurveElementFlags() = default;

	FCurveElementFlags(FName InName)
		: Name(InName)
	{}

	FCurveElementFlags(FName InName, ECurveElementFlags InFlags)
		: Name(InName)
		, Flags(InFlags) 
	{}

	FName Name = NAME_None;
	ECurveElementFlags Flags = ECurveElementFlags::None;
};

/**
 * Named value array used for bulk storage of curve flags
 */
struct FBulkCurveFlags : TNamedValueArray<FDefaultAllocator, FCurveElementFlags> {};

}
