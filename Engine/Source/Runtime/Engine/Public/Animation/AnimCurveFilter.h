// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/Sort.h"
#include "AnimCurveElementFlags.h"
#include "NamedValueArray.h"
#include "Misc/MemStack.h"

namespace UE::Anim
{

struct FCurveUtils;

// Flags for each curve filter element
enum class ECurveFilterFlags : uint8
{
	// No flags
	None		= 0,

	// Curve is explicitly disallowed (i.e. not processed at a particular LOD, bone linkage etc.).
	// This is used in combination with the FilterMode of the filter.
	Disallowed	= 0x01,

	// Curve is allowed/disallowed.
	// This is used in combination with the FilterMode of the filter for allow/deny lists of curves.
	Filtered	= 0x02,
};

ENUM_CLASS_FLAGS(ECurveFilterFlags);

// The various ways curves can be filtered
enum class ECurveFilterMode : uint8
{
	// No filtering
	None,

	// Curves are all filtered/disallowed (equivalent of all elements filtered)
	DisallowAll,

	// Elements marked as 'filtered' are the only ones that are allowed
	AllowOnlyFiltered,

	// Elements marked as 'filtered' are disallowed, all others are allowed
	DisallowFiltered,
};

struct FCurveFilterElement
{
	FCurveFilterElement() = default;

	FCurveFilterElement(FName InName)
		: Name(InName)
	{}

	FCurveFilterElement(FName InName, ECurveFilterFlags InFlags)
		: Name(InName)
		, Flags(InFlags)
	{}

	FName Name = NAME_None;
	ECurveFilterFlags Flags = ECurveFilterFlags::None;
};

/** Named value array that can act as a filter, creating allow/deny lists of curve names */
struct FCurveFilter : TNamedValueArray<FDefaultAllocator, FCurveFilterElement>
{
	typedef TNamedValueArray<AllocatorType, FCurveFilterElement> Super;
	
	friend FCurveUtils;

	void Empty()
	{
		Super::Empty();
		FilterMode = ECurveFilterMode::None;
	}

	bool IsEmpty() const
	{
		return FilterMode == ECurveFilterMode::None || (FilterMode == ECurveFilterMode::DisallowFiltered && Elements.Num() == 0);
	}

	void Add(FName InName, ECurveFilterFlags InFlags = ECurveFilterFlags::Filtered)
	{
		Elements.Emplace(InName, InFlags);
		bSorted = false;
	}

	/** Add an array of names to filter */
	void AppendNames(TArrayView<const FName> InNameArray)
	{
		Elements.Reserve(Elements.Num() + InNameArray.Num());
		for(const FName& Name : InNameArray)
		{
			Elements.Emplace(Name, ECurveFilterFlags::Filtered);
		}
		bSorted = false;
	}

	/** Add an array of names/flags to filter */
	void AppendNamedFlags(std::initializer_list<TTuple<const FName, ECurveFilterFlags>> InInputArgs)
	{
		Elements.Reserve(Elements.Num() + InInputArgs.size());
		for(const TTuple<const FName, ECurveFilterFlags>& Arg : InInputArgs)
		{
			Elements.Emplace(Arg.Get<0>(), Arg.Get<1>());
		}
		bSorted = false;
	}

	/** Set the filter mode used in Filter() */
	void SetFilterMode(ECurveFilterMode InFilterMode)
	{
		FilterMode = InFilterMode;
	}

private:
	// The filtering mode
	ECurveFilterMode FilterMode = ECurveFilterMode::None;
};

}
