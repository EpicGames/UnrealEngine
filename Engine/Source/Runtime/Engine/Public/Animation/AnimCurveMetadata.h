// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/BoneReference.h"
#include "Engine/AssetUserData.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "UObject/Interface.h"
#include "AnimCurveMetadata.generated.h"

/** in the future if we need more bools, please convert to bitfield 
 * These are not saved in asset but per skeleton. 
 */
USTRUCT()
struct FAnimCurveType
{
	GENERATED_BODY()

	UPROPERTY()
	bool bMaterial;

	UPROPERTY()
	bool bMorphtarget;

	FAnimCurveType(bool bInMorphtarget = false, bool bInMaterial = false)
		: bMaterial(bInMaterial)
		, bMorphtarget(bInMorphtarget)
	{
	}
};

/** Curve Meta Data for each name
 * Unfortunately this should be linked to FName, but no GUID because we don't have GUID in run-time
 * We only add this if anything changed, by default, it is attribute curve
 */
USTRUCT()
struct FCurveMetaData
{
	GENERATED_BODY()

	/** connected bones to this meta data */
	UPROPERTY()
	TArray<FBoneReference> LinkedBones;
	
	/* max LOD (lowest LOD) to evaluate this. -1 means it will evaluate all the time. */
	UPROPERTY()
	uint8 MaxLOD;

	/** Curve type flags */
	UPROPERTY()
	FAnimCurveType Type;

	friend FArchive& operator<<(FArchive& Ar, FCurveMetaData& B);

	ENGINE_API bool Serialize(FArchive& Ar);

	ENGINE_API FCurveMetaData();
};

UINTERFACE()
class UInterface_AnimCurveMetaData : public UInterface
{
	GENERATED_BODY()
};

/** Interface for curve metadata-hosting objects */
class IInterface_AnimCurveMetaData
{
	GENERATED_BODY()

public:
	/**
	 * Iterate over all curve metadata entries, calling InFunction on each
	 * @param	InFunction	The function to call
	 **/
	virtual void ForEachCurveMetaData(const TFunctionRef<void(FName, const FCurveMetaData&)>& InFunction) const = 0;

	/** @return the number of curve metadata entries **/
	virtual int32 GetNumCurveMetaData() const = 0;

	/**
	 * Get the curve metadata entry with the specified name
	 * @param	InCurveName		The name of the curve to find
	 * @return the curve metadata, if found, otherwise nullptr
	 */
	virtual const FCurveMetaData* GetCurveMetaData(FName InCurveName) const = 0;

	/**
	 * Adds a curve metadata entry with the specified name
	 * @param	InCurveName			The name of the curve to find
	 * @param   bInTransact         If true record a new transaction
	 * @return true if an entry was added, false if an entry already existed
	 */
	virtual bool AddCurveMetaData(FName InCurveName, bool bInTransact = true) = 0;

	/**
	 * Get the curve metadata entry with the specified name
	 * @param	InCurveName		The name of the curve to find
	 * @return the curve metadata, if found, otherwise nullptr
	 */
	virtual FCurveMetaData* GetCurveMetaData(FName InCurveName) = 0;

	/**
	 * Get an array of all curve metadata names
	 * @param	OutNames		The array to receive the metadata names 
	 */
	virtual void GetCurveMetaDataNames(TArray<FName>& OutNames) const = 0;

	/**
	 * Refresh the indices of any linked bone references
	 * @param	InSkeleton		The skeleton to use
	 **/
	virtual void RefreshBoneIndices(USkeleton* InSkeleton) {}

	/** @return the version number for this metadata. Can be used to regenerate cached values against the metadata. */
	virtual uint16 GetVersionNumber() const = 0;

#if WITH_EDITOR
	/**
	 * Renames a curve metadata entry. Metadata is preserved, but assigned to a different curve name.
	 * @param OldName	The name of an existing curve entry
	 * @param NewName	The name to change the entry to
	 * @return			true if the rename was successful (the old name was found and the new name didnt collide with an
	 *					existing entry)
	 */	
	virtual bool RenameCurveMetaData(FName OldName, FName NewName) = 0;

	/**
	 * Removes a curve metadata entry for the specified name.
	 * @param CurveName	The name of the curve to remove the metadata for
	 * @return true if the entry was successfully removed (i.e. it existed)
	 */	
	virtual bool RemoveCurveMetaData(FName CurveName) = 0;

	/**
	 * Removes a group of curve metadata entries for the specified names.
	 * @param CurveNames	The names of the curves to remove the metadata for
	 * @return true if any of the entries were successfully removed (i.e. something changed)
	 */	
	virtual bool RemoveCurveMetaData(TArrayView<FName> CurveNames) = 0;

	/**
	 * Register a delegate to be called when curve metadata changes
	 * @param	InOnCurveMetaDataChanged	Delegate to register
	 * @return delegate handle
	 */
	virtual FDelegateHandle RegisterOnCurveMetaDataChanged(const FSimpleMulticastDelegate::FDelegate& InOnCurveMetaDataChanged) = 0;

	/**
	 * Gets the material flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return	true if the flag has been set
	 */
	virtual bool GetCurveMetaDataMaterial(FName CurveName) const = 0;

	/**
	 * Gets the morph target flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return	true if the flag has been set
	 */
	virtual bool GetCurveMetaDataMorphTarget(FName CurveName) const = 0;

	/**
	 * Set the material flag for a curve's metadata
	 * @param	CurveName			The name of the curve to set
	 * @param	bOverrideMaterial	Whether to set the material flag
	 */
	virtual void SetCurveMetaDataMaterial(FName CurveName, bool bOverrideMaterial) = 0;

	/**
	 * Set the morph target flag for a curve's metadata
	 * @param	CurveName				The name of the curve to set
	 * @param	bOverrideMorphTarget	Whether to set the morph target flag
	 */	
	virtual void SetCurveMetaDataMorphTarget(FName CurveName, bool bOverrideMorphTarget) = 0;

	/**
	 * Set the bone links for a curve's metadata
	 * @param	CurveName		The name of the curve to set
	 * @param	BoneLinks		Array of linked bones
	 * @param	InMaxLOD		The max LOD to apply the curve at
	 * @param	InSkeleton		The skeleton used to initialize bone references
	 */	
	virtual void SetCurveMetaDataBoneLinks(FName CurveName, const TArray<FBoneReference>& BoneLinks, uint8 InMaxLOD, USkeleton* InSkeleton) = 0;
	
	/**
	 * Unregister a delegate to be called when curve metadata changes
	 * @param	InHandle	Delegate handle
	 */
	virtual void UnregisterOnCurveMetaDataChanged(FDelegateHandle InHandle) = 0;

#endif// WITH_EDITOR
};


/** Asset user data used to supply curve meta data for specific assets */
UCLASS(MinimalAPI)
class UAnimCurveMetaData : public UAssetUserData, public IInterface_AnimCurveMetaData
{
	GENERATED_BODY()

	// Friend with skeleton for upgrade purposes
	friend class USkeleton;

public:
#if WITH_EDITOR
	// UObject interface
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
#endif

	// IInterface_AnimCurveMetaData interface
	virtual void ForEachCurveMetaData(const TFunctionRef<void(FName, const FCurveMetaData&)>& InFunction) const override;
	virtual int32 GetNumCurveMetaData() const override { return CurveMetaData.Num(); }
	virtual const FCurveMetaData* GetCurveMetaData(FName InCurveName) const override;
	virtual FCurveMetaData* GetCurveMetaData(FName InCurveName) override;
	virtual bool AddCurveMetaData(FName InCurveName, bool bInTransact = true) override;
	virtual void GetCurveMetaDataNames(TArray<FName>& OutNames) const override;
	virtual void RefreshBoneIndices(USkeleton* InSkeleton) override;
	virtual uint16 GetVersionNumber() const override; 
#if WITH_EDITOR
	virtual bool RenameCurveMetaData(FName OldName, FName NewName) override;
	virtual bool RemoveCurveMetaData(FName CurveName) override;
	virtual bool RemoveCurveMetaData(TArrayView<FName> CurveNames) override;
	virtual bool GetCurveMetaDataMaterial(FName CurveName) const override;
	virtual bool GetCurveMetaDataMorphTarget(FName CurveName) const override;
	virtual void SetCurveMetaDataMaterial(FName CurveName, bool bOverrideMaterial) override;
	virtual void SetCurveMetaDataMorphTarget(FName CurveName, bool bOverrideMorphTarget) override;
	virtual void SetCurveMetaDataBoneLinks(FName CurveName, const TArray<FBoneReference>& BoneLinks, uint8 InMaxLOD, USkeleton* InSkeleton) override;
	virtual FDelegateHandle RegisterOnCurveMetaDataChanged(const FSimpleMulticastDelegate::FDelegate& InOnCurveMetaDataChanged) override;
	virtual void UnregisterOnCurveMetaDataChanged(FDelegateHandle InHandle) override;
#endif
private:
	/** Increase the version number on changes */
	void IncreaseVersionNumber();

private:
	/** Map of name -> metadata */
	UPROPERTY()
	TMap<FName, FCurveMetaData> CurveMetaData;

	/** Delegate called when a curve metadata is changed */
	FSimpleMulticastDelegate OnCurveMetaDataChanged;

	/** Version number used for caching (0 = invalid) */
	uint16 VersionNumber = 0;
};
