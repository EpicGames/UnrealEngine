// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/SmartName.h"
#include "Curves/RichCurve.h"
#include "Misc/EnumRange.h"
#include "AnimCurveElementFlags.h"
#include "Animation/NamedValueArray.h"

#include "AnimCurveTypes.generated.h"

typedef SmartName::UID_Type SkeletonAnimCurveUID;

class USkeleton;
struct FBoneContainer;

namespace UE::Anim
{
struct FCurveUtils;
struct FCurveFilter;
}

UENUM()
enum class EAnimCurveType : uint8 
{
	AttributeCurve,
	MaterialCurve, 
	MorphTargetCurve, 
	// make sure to update MaxCurve 
	MaxAnimCurveType UMETA(Hidden)
};

ENUM_RANGE_BY_COUNT(EAnimCurveType, EAnimCurveType::MaxAnimCurveType);

/** This is curve flags that are saved in asset and **/
UENUM(BlueprintType, meta=(Bitflags))
enum EAnimAssetCurveFlags : int
{
	AACF_NONE = 0 UMETA(Hidden),
	// Used as morph target curve
	AACF_DriveMorphTarget_DEPRECATED = 0x00000001 UMETA(Hidden), // This has moved to FAnimCurveType:bMorphTarget. Set per skeleton. DO NOT REMOVE UNTIL FrameworkObjectVersion.MoveCurveTypesToSkeleton expires.
	// Used as triggering event
	AACF_DriveAttribute_DEPRECATED = 0x00000002 UMETA(Hidden), // Set per skeleton. DO NOT REMOVE UNTIL FrameworkObjectVersion.MoveCurveTypesToSkeleton expires.
	// Is editable in Sequence Editor
	AACF_Editable						= 0x00000004 UMETA(DisplayName = "Editable"), // per asset
	// Used as a material curve
	AACF_DriveMaterial_DEPRECATED = 0x00000008 UMETA(Hidden), // This has moved to FAnimCurveType:bMaterial. Set per skeleton. DO NOT REMOVE UNTIL FrameworkObjectVersion.MoveCurveTypesToSkeleton expires.
	// Is a metadata 'curve'
	AACF_Metadata						= 0x00000010 UMETA(DisplayName = "Metadata"), // per asset
	// motifies bone track
	AACF_DriveTrack = 0x00000020 UMETA(Hidden), // @Todo: remove?
	// disabled, right now it's used by track
	AACF_Disabled = 0x00000040 UMETA(Hidden), // per asset
};
inline constexpr EAnimAssetCurveFlags AACF_DefaultCurve = AACF_Editable;

ENUM_RANGE_BY_FIRST_AND_LAST(EAnimAssetCurveFlags, AACF_DriveMorphTarget_DEPRECATED, AACF_Disabled);

/** DEPRECATED - no longer used */
USTRUCT()
struct FAnimCurveParam
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = FAnimCurveParam)
	FName Name;

	// name UID for fast access
	SmartName::UID_Type UID;

	FAnimCurveParam()
		: UID(SmartName::MaxUID)
	{}

	UE_DEPRECATED(5.3, "FAnimCurveParam is no longer used.")
	void Initialize(USkeleton* Skeleton) {}

	UE_DEPRECATED(5.3, "FAnimCurveParam is no longer used.")
	bool IsValid() const { return false; }
	
	UE_DEPRECATED(5.3, "FAnimCurveParam is no longer used.")
	bool IsValidToEvaluate() const { return false; }
};
/**
 * Float curve data for one track
 */
USTRUCT(BlueprintType)
struct FAnimCurveBase
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName		LastObservedName_DEPRECATED;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	FSmartName	Name_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	FLinearColor Color;

	UPROPERTY()
	FString Comment;
#endif

private:
	UPROPERTY()
	FName CurveName;

	// this flag is mostly used by editor only now
	// however I can't remove this to editor only because 
	// we need DEPRECATED Flag to be loaded in game
	// because those data are stored in asset, and skeleton might not be saved with it. 
	/** Curve Type Flags */
	UPROPERTY()
	int32		CurveTypeFlags;

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS	// Constructors implicitly initialization deprecated FSmartName
	FAnimCurveBase() 
		: CurveTypeFlags(0)
	{
#if WITH_EDITORONLY_DATA
		Color = MakeColor(CurveName);
#endif
	}

	FAnimCurveBase(FName InName, int32 InCurveTypeFlags)
		: CurveName(InName)
		, CurveTypeFlags(InCurveTypeFlags)
	{
#if WITH_EDITORONLY_DATA
		Color = MakeColor(CurveName);
#endif
	}

	UE_DEPRECATED(5.3, "Please use the cosntructor that takes an FName.")
	FAnimCurveBase(FSmartName InName, int32 InCurveTypeFlags)
		: CurveName(InName.DisplayName)
		, CurveTypeFlags(InCurveTypeFlags)
	{
#if WITH_EDITORONLY_DATA
		Color = MakeColor(CurveName);
#endif
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS // FSmartName deprecation

	// This allows loading data that was saved between VER_UE4_SKELETON_ADD_SMARTNAMES and FFrameworkObjectVersion::SmartNameRefactor
	ENGINE_API void PostSerializeFixup(FArchive& Ar);

	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);

	/**
	 * Set InFlag to bValue
	 */
	ENGINE_API void SetCurveTypeFlag(EAnimAssetCurveFlags InFlag, bool bValue);

	/**
	 * Toggle the value of the specified flag
	 */
	ENGINE_API void ToggleCurveTypeFlag(EAnimAssetCurveFlags InFlag);

	/**
	 * Return true if InFlag is set, false otherwise 
	 */
	ENGINE_API bool GetCurveTypeFlag(EAnimAssetCurveFlags InFlag) const;

	/**
	 * Set CurveTypeFlags to NewCurveTypeFlags
	 * This just overwrites CurveTypeFlags
	 */
	ENGINE_API void SetCurveTypeFlags(int32 NewCurveTypeFlags);
	/** 
	 * returns CurveTypeFlags
	 */
	ENGINE_API int32 GetCurveTypeFlags() const;

	/** Get the name of this curve */
	FName GetName() const
	{
		return CurveName;
	}

	/** Set the name of this curve */
	void SetName(FName InName)
	{
		CurveName = InName;
	}

#if WITH_EDITORONLY_DATA
	/** Get the color used to display this curve in the editor */
	FLinearColor GetColor() const { return Color; }

	/** Make an initial color */
	static ENGINE_API FLinearColor MakeColor(const FName& CurveName);
#endif
};

template<>
struct TStructOpsTypeTraits<FAnimCurveBase> : public TStructOpsTypeTraitsBase2<FAnimCurveBase>
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FFloatCurve : public FAnimCurveBase
{
	GENERATED_USTRUCT_BODY()

	/** Curve data for float. */
	UPROPERTY()
	FRichCurve	FloatCurve;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FFloatCurve(){}

	UE_DEPRECATED(5.3, "Please use the constructor that takes an FName.")
	FFloatCurve(FSmartName InName, int32 InCurveTypeFlags)
		: FAnimCurveBase(InName, InCurveTypeFlags)
	{
	}

	FFloatCurve(FName InName, int32 InCurveTypeFlags)
		: FAnimCurveBase(InName, InCurveTypeFlags)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
	ENGINE_API void CopyCurve(const FFloatCurve& SourceCurve);
	ENGINE_API float Evaluate(float CurrentTime) const;
	ENGINE_API void UpdateOrAddKey(float NewKey, float CurrentTime);
	ENGINE_API void GetKeys(TArray<float>& OutTimes, TArray<float>& OutValues) const;
	ENGINE_API void Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime);
};

template<>
struct TStructOpsTypeTraits<FFloatCurve> : public TStructOpsTypeTraitsBase2<FFloatCurve>
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

USTRUCT()
struct FVectorCurve : public FAnimCurveBase
{
	GENERATED_USTRUCT_BODY()

	enum class EIndex
	{
		X = 0, 
		Y, 
		Z, 
		Max
	};

	/** Curve data for float. */
	UPROPERTY()
	FRichCurve	FloatCurves[3];

	FVectorCurve(){}

	FVectorCurve(FName InName, int32 InCurveTypeFlags)
		: FAnimCurveBase(InName, InCurveTypeFlags)
	{
	}

	// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
	void CopyCurve(const FVectorCurve& SourceCurve);
	ENGINE_API FVector Evaluate(float CurrentTime, float BlendWeight) const;
	ENGINE_API void UpdateOrAddKey(const FVector& NewKey, float CurrentTime);
	ENGINE_API void GetKeys(TArray<float>& OutTimes, TArray<FVector>& OutValues) const;
	bool DoesContainKey() const { return (FloatCurves[0].GetNumKeys() > 0 || FloatCurves[1].GetNumKeys() > 0 || FloatCurves[2].GetNumKeys() > 0);}
	ENGINE_API void Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime);
	ENGINE_API int32 GetNumKeys() const;
};

template<>
struct TStructOpsTypeTraits<FVectorCurve> : public TStructOpsTypeTraitsBase2<FVectorCurve>
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FTransformCurve: public FAnimCurveBase
{
	GENERATED_USTRUCT_BODY()

	/** Curve data for each transform. */
	UPROPERTY()
	FVectorCurve	TranslationCurve;

	/** Rotation curve - right now we use euler because quat also doesn't provide linear interpolation - curve editor can't handle quat interpolation
	 * If you hit gimbal lock, you should add extra key to fix it. This will cause gimbal lock. 
	 * @TODO: Eventually we'll need FRotationCurve that would contain rotation curve - that will interpolate as slerp or as quaternion 
	 */
	UPROPERTY()
	FVectorCurve	RotationCurve;

	UPROPERTY()
	FVectorCurve	ScaleCurve;

	FTransformCurve(){}

	FTransformCurve(FName InName, int32 InCurveTypeFlags)
		: FAnimCurveBase(InName, InCurveTypeFlags)
	{
	}

	// we don't want to have = operator. This only copies curves, but leaving naming and everything else intact. 
	ENGINE_API void CopyCurve(const FTransformCurve& SourceCurve);
	ENGINE_API FTransform Evaluate(float CurrentTime, float BlendWeight) const;
	ENGINE_API void UpdateOrAddKey(const FTransform& NewKey, float CurrentTime);
	ENGINE_API void GetKeys(TArray<float>& OutTimes, TArray<FTransform>& OutValues) const;
	ENGINE_API void Resize(float NewLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime);
	
	ENGINE_API const FVectorCurve* GetVectorCurveByIndex(int32 Index) const;
	ENGINE_API FVectorCurve* GetVectorCurveByIndex(int32 Index);
};

template<>
struct TStructOpsTypeTraits<FTransformCurve> : public TStructOpsTypeTraitsBase2<FTransformCurve>
{
	enum 
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FCachedFloatCurve
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Curve Settings")
	FName CurveName;

public:

	ENGINE_API bool IsValid(const UAnimSequenceBase* InAnimSequence) const;
	ENGINE_API float GetValueAtPosition(const UAnimSequenceBase* InAnimSequence, const float& InPosition) const;
	ENGINE_API const FFloatCurve* GetFloatCurve(const UAnimSequenceBase* InAnimSequence) const;

protected:
	UE_DEPRECATED(5.3, "Please just use CurveName.")
	SmartName::UID_Type GetAnimCurveUID(const UAnimSequenceBase* InAnimSequence) const { return 0; }
};

/**
* This is array of curves that run when collecting curves natively 
*/
struct UE_DEPRECATED(5.3, "FCurveElement in the global namespace is no longer used.") FCurveElement
{
	/** Curve Value */
	float					Value;
	/** Whether this value is set or not */
	bool					bValid;

	FCurveElement(float InValue)
		:  Value(InValue)
		,  bValid (true)
	{}

	FCurveElement()
		: Value(0.f)
		, bValid(false)
	{}

	bool IsValid() const 
	{
		return bValid;
	}

	void SetValue(float InValue)
	{
		Value = InValue;
		bValid = true;
	}
};

namespace UE::Anim
{

// An element for a named curve value
struct FCurveElement
{
	FCurveElement() = default;

	FCurveElement(FName InName)
		: Name(InName)
	{}

	FCurveElement(FName InName, float InValue)
		: Name(InName)
		, Value(InValue)
	{}

	FCurveElement(FName InName, ECurveElementFlags InFlags)
		: Name(InName)
		, Flags(InFlags) 
	{}
	
	FCurveElement(FName InName, float InValue, ECurveElementFlags InFlags)
		: Name(InName)
		, Value(InValue)
		, Flags(InFlags) 
	{}
	
	FName Name = NAME_None;
	float Value = 0.0f;
	ECurveElementFlags Flags = ECurveElementFlags::None;
};

// An element for a named curve value that is also indexed.
// Useful in bulk curve operations that need to pay heed to disabled curves
struct FCurveElementIndexed : public UE::Anim::FCurveElement
{
	FCurveElementIndexed() = default;

	FCurveElementIndexed(FName InName, int32 InIndex)
		: UE::Anim::FCurveElement(InName)
		, Index(InIndex)
	{}

	int32 Index = INDEX_NONE;
};

// Deprecated base for TBaseBlendedCurve for legacy support
// We need to use this non-templated base to avoid the deprecated (and now static) members taking up per-instance memory
struct FBaseBlendedCurve_DEPRECATED
{
	UE_DEPRECATED(5.3, "Direct access to CurveWeights is no longer allowed as it is no longer used.")
	static ENGINE_API TArray<float> CurveWeights;
	
	UE_DEPRECATED(5.3, "Direct access to ValidCurveWeights is no longer allowed as it is no longer used.")
	static ENGINE_API TBitArray<> ValidCurveWeights;
	
	UE_DEPRECATED(5.3, "Direct access to UIDToArrayIndexLUT is no longer allowed as it is no longer used.")
	static ENGINE_API TArray<uint16> const * UIDToArrayIndexLUT;
	
	UE_DEPRECATED(5.3, "Direct access to NumValidCurveCount is no longer allowed as it is no longer used.")
	static ENGINE_API uint16 NumValidCurveCount;
	
	UE_DEPRECATED(5.3, "Direct access to bInitialized is no longer allowed.")
	static ENGINE_API bool bInitialized;
};

}

/**
 * This struct is used to create curve snap shot of current time when extracted
 */
template <typename InAllocatorType = FAnimStackAllocator, typename InElementType = UE::Anim::FCurveElement>
struct TBaseBlendedCurve : public UE::Anim::TNamedValueArray<InAllocatorType, InElementType>, public UE::Anim::FBaseBlendedCurve_DEPRECATED
{
public:
	typedef InAllocatorType AllocatorType;
	typedef InElementType ElementType;

	template<typename OtherAllocator, typename OtherElementType>
	friend struct TBaseBlendedCurve;

	friend struct UE::Anim::FCurveUtils;

	typedef UE::Anim::TNamedValueArray<AllocatorType, ElementType> Super;

private:
	// Filter to use when building curves
	const UE::Anim::FCurveFilter* Filter = nullptr;

public:
	UE_DEPRECATED(5.3, "InitFrom can no longer be called with a LUT, please use another initialization method or just dont initialize the curve.")
	void InitFrom(TArray<uint16> const * InUIDToArrayIndexLUT)
	{
		Super::Elements.Reset();
		Filter = nullptr;
		Super::bSorted = false;
	}

	/** Initialize from another curve. Just copies the filter and empties 'this'. */
	template <typename OtherAllocator, typename OtherElementType>
	void InitFrom(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& InCurveToInitFrom)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_InitFrom);
		
		if (ensure(&InCurveToInitFrom != this))
		{
			Super::Elements.Reset();
			Super::bSorted = false;
			Filter = InCurveToInitFrom.Filter;
		}
	}

	UE_DEPRECATED(5.3, "Please use InvalidateCurveWeight with a curve name")
	void InvalidateCurveWeight(SmartName::UID_Type InUid)
	{
	}

	/** Invalidate value of the named curve */
	void InvalidateCurveWeight(FName InName)
	{
		const int32 ElementIndex = Super::IndexOf(InName);
		if(ElementIndex != INDEX_NONE)
		{
			Super::Elements.RemoveAt(ElementIndex);
		}
	}
	
	UE_DEPRECATED(5.3, "Please use Set with a curve name or consider using one of the bulk APIs (e.g. UE::Anim::FCurveUtils::BulkSet).")
	void Set(SmartName::UID_Type InUid, float InValue)
	{
	}

	/**
	 * Set value of curve named InName to InValue. Inserts a new value if an element for InName does not already exist.
	 * Note that this performs a binary search per-call and can potentially cause a re-sort. Consider using a combiner
	 * operation to set multiple element's values.
	 * @param	InName	the name of the curve to set
	 * @param	InValue	the value of the curve to set
	 */
	void Set(FName InName, float InValue)
	{
		if(ElementType* CurveElement = Super::Find(InName))
		{
			CurveElement->Value = InValue;
		}
		else
		{
			Super::Add(InName, InValue);
		}
	}

	/**
	 * Set flags of curve named InName to InFlags.  Inserts a new default value if an element for InName does not
	 * already exist and then sets its flags to InFlags.
	 * Note that this performs a binary search per-call and can potentially cause a re-sort. Consider using a combiner
	 * operation to set multiple element's flags.
	 * @param	InName	the name of the curve to set
	 * @param	InFlags	the flags of the curve to set
	 */
	void SetFlags(FName InName, UE::Anim::ECurveElementFlags InFlags)
	{
		if(ElementType* CurveElement = Super::Find(InName))
		{
			CurveElement->Flags = InFlags;
		}
		else
		{
			Super::Add(InName, InFlags);
		}
	}

	UE_DEPRECATED(5.3, "Please use Get with a curve name or consider using one of the bulk APIs (e.g. UE::Anim::FCurveUtils::BulkGet)")
	float Get(SmartName::UID_Type InUid) const
	{
		return 0.f;
	}

	/**
	 * Get value of curve element named InName.
	 * Note that this performs a binary search per-call. Consider using a ForEach* or a bulk API call
	 * (e.g. UE::Anim::FCurveUtils::BulkGet) to get multiple element's values.
	 * @param	InName	the name of the curve element to get
	 * @return  the value of the curve element. If this curve does not contain an element with the supplied name, returns 0.0 
	 */
	float Get(FName InName) const
	{
		if(const ElementType* CurveElement = Super::Find(InName))
		{
			return CurveElement->Value;
		}
		return 0.0f;
	}

	/**
	 * Get flags for curve element named InName
	 * Note that this performs a binary search per-call. Consider using a ForEach* call to get multiple element's flags.
	 * @param	InName	the name of the curve element to get
	 * @return  the flags of the curve element. If this curve does not contain an element with the supplied name, returns UE::Anim::ECurveElementFlags::None 
	 */
	UE::Anim::ECurveElementFlags GetFlags(FName InName) const
	{
		if(const ElementType* CurveElement = Super::Find(InName))
		{
			return CurveElement->Flags;
		}
		return UE::Anim::ECurveElementFlags::None;
	}

	UE_DEPRECATED(5.3, "Please use Get with a curve name or consider using one of the bulk APIs (e.g. UE::Anim::FCurveUtils::BulkGet).")
	float Get(SmartName::UID_Type InUid, bool& OutIsValid, float InDefaultValue=0.f) const
	{
		return 0.0f;
	}

	/**
	 * Get Value of curve named InName with validation and default value
	 * Note that this performs a binary search per-call. Consider using a ForEach* or one of the bulk APIs
	 * (e.g. UE::Anim::FCurveUtils::BulkGet) call to get multiple element's flags.
	 * @param	InName	the name of the curve element to get
	 * @param	HasElement	whether this curve contains the supplied named element 
	 * @param	InDefaultValue	the default value to use in case the curve does not contain the supplied element
	 * @return  the value of the curve element. If this curve does not contain an element with the supplied name, returns InDefaultValue
	 */
	float Get(FName InName, bool& OutHasElement, float InDefaultValue = 0.0f) const
	{
		if(const ElementType* CurveElement = Super::Find(InName))
		{
			OutHasElement = true;
			return CurveElement->Value;
		}
		
		OutHasElement = false;
		return InDefaultValue;
	}

	/**
	 * Mirror the values & flags of the two named curves if both exist.
	 * If only InName0 exists, InName0 will be renamed to InName1.
	 * If only InName1 exists, InName1 will be renamed to InName0.
	 * @param	InName0	the name of first curve to mirror
	 * @param	InName1	the name of second curve to mirror
	 */
	void Mirror(FName InName0, FName InName1)
	{
		ElementType* CurveElement0 = Super::Find(InName0);
		ElementType* CurveElement1 = Super::Find(InName1);

		// Both exist, so swap values & flags
		if(CurveElement0 && CurveElement1)
		{
			Swap(CurveElement0->Value, CurveElement1->Value);
			Swap(CurveElement0->Flags, CurveElement1->Flags);
		}
		// First exists, but not second. Rename to second
		else if(CurveElement0)
		{
			CurveElement0->Name = InName1;
			Super::bSorted = false;
		}
		// Second exists, but not first. Rename to first
		else if(CurveElement1)
		{
			CurveElement1->Name = InName0;
			Super::bSorted = false;
		}
	}
	
	UE_DEPRECATED(5.3, "This function has been removed.")
	int32 GetArrayIndexByUID(SmartName::UID_Type InUid) const
	{
		return INDEX_NONE;
	}

	UE_DEPRECATED(5.3, "Element validity/enabled state is now handled via UE::Anim::FCurveFilter.")
	bool IsEnabled(SmartName::UID_Type InUid) const
	{
		return false;
	}
	
	UE_DEPRECATED(5.3, "Element validity/enabled state is now handled via UE::Anim::FCurveFilter.")
	static int32 GetValidElementCount(TArray<uint16> const* InUIDToArrayIndexLUT) 
	{
		return 0;
	}

public:
	/**
	 * Blend (A, B) using Alpha. Lone valid elements are not preserved.
	 */
	template<typename AllocatorA, typename ElementTypeA, typename AllocatorB, typename ElementTypeB>
	void Lerp(const TBaseBlendedCurve<AllocatorA, ElementTypeA>& A, const TBaseBlendedCurve<AllocatorB, ElementTypeB>& B, float Alpha)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Lerp);
		
		if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha)))
		{
			// if blend is all the way for child1, then just copy its bone atoms
			Override(A);
		}
		else if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha - 1.0f)))
		{
			// if blend is all the way for child2, then just copy its bone atoms
			Override(B);
		}
		else
		{
			// Combine using Lerp. Result is a merged set of curves in 'this'. 
			UE::Anim::FNamedValueArrayUtils::Union(*this, A, B, [&Alpha](ElementType& OutResult, const ElementTypeA& InElement0, const ElementTypeB& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				OutResult.Value = FMath::Lerp(InElement0.Value, InElement1.Value, Alpha);
				OutResult.Flags = InElement0.Flags | InElement1.Flags;
			});
		}
	}

	/**
	 * Blend with Other using Alpha. Lone valid elements are not preserved. Same as Lerp
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void LerpTo(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& Other, float Alpha)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_LerpTo);
		
		if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha)))
		{
			return;
		}
		else if (!FAnimWeight::IsRelevant(FMath::Abs(Alpha - 1.0f)))
		{
			// if blend is all the way for child2, then just copy its bone atoms
			Override(Other);
		}
		else
		{
			// Combine using Lerp. Result is a merged set of curves in 'this'. 
			UE::Anim::FNamedValueArrayUtils::Union(*this, Other, [&Alpha](ElementType& InOutThisElement, const OtherElementType& InOtherElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutThisElement.Value = FMath::Lerp(InOutThisElement.Value, InOtherElement.Value, Alpha);
				InOutThisElement.Flags |= InOtherElement.Flags;
			});
		}
	}

	/**
	* Blend with Other using Alpha when both elements are valid, otherwise preserve valid element.
	*/
	template<typename AllocatorA, typename ElementTypeA, typename AllocatorB, typename ElementTypeB>
	void LerpValid(const TBaseBlendedCurve<AllocatorA, ElementTypeA>& A, const TBaseBlendedCurve<AllocatorB, ElementTypeB>& B, float Alpha)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_LerpValid);
		
		// Combine using Lerp. Result is a merged set of curves in 'this'. Elements where only one input is valid will be preserved.
		UE::Anim::FNamedValueArrayUtils::Union(*this, A, B, [&Alpha](UE::Anim::FCurveElement& OutResult, const ElementTypeA& InElement0, const ElementTypeB& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::BothArgsValid))
			{
				OutResult.Value = FMath::Lerp(InElement0.Value, InElement1.Value, Alpha);
				OutResult.Flags = InElement0.Flags | InElement1.Flags;
			}
			else if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg0))
			{
				OutResult.Value = InElement0.Value;
				OutResult.Flags = InElement0.Flags;
			}
			else if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				OutResult.Value = InElement1.Value;
				OutResult.Flags = InElement1.Flags;
			}				
		});
	}

	/**
	* Blend with Other using Alpha when both elements are valid, otherwise preserve valid element. Same as LerpValid
	*/
	template<typename OtherAllocator, typename OtherElementType>
	void LerpToValid(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& Other, float Alpha)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_LerpToValid);
		
		// Combine using Lerp. Result is a merged set of curves in 'this'. Elements where only one input is valid will be preserved.
		UE::Anim::FNamedValueArrayUtils::Union(*this, Other, [&Alpha](ElementType& InOutThisElement, const OtherElementType& InOtherElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::BothArgsValid))
			{
				InOutThisElement.Value = FMath::Lerp(InOutThisElement.Value, InOtherElement.Value, Alpha);
				InOutThisElement.Flags |= InOtherElement.Flags;
			}
			else if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				InOutThisElement.Value = InOtherElement.Value;
				InOutThisElement.Flags = InOtherElement.Flags;
			}			
		});
	}

	/**
	 * Convert current curves to Additive (this - BaseCurve) if overlapping entries are found
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void ConvertToAdditive(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& BaseCurve)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_ConvertToAdditive);
		
		UE::Anim::FNamedValueArrayUtils::Union(*this, BaseCurve, [](ElementType& InOutThisElement, const OtherElementType& InBaseCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			InOutThisElement.Value -= InBaseCurveElement.Value;
			InOutThisElement.Flags |= InBaseCurveElement.Flags;
		});
	}

	/**
	 * Accumulate the input curve with input Weight
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void Accumulate(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& AdditiveCurve, float Weight)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Accumulate);
		
		if (FAnimWeight::IsRelevant(Weight))
		{
			UE::Anim::FNamedValueArrayUtils::Union(*this, AdditiveCurve, [Weight](ElementType& InOutThisElement, const OtherElementType& InAdditiveCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				InOutThisElement.Value += InAdditiveCurveElement.Value * Weight;
				InOutThisElement.Flags |= InAdditiveCurveElement.Flags;
			});
		}
	}

	/**
	 * This doesn't blend but combines MAX(current value, CurveToCombine value)
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void UseMaxValue(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToCombine)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_UseMaxValue);
		
		UE::Anim::FNamedValueArrayUtils::Union(*this, CurveToCombine, [](ElementType& InOutThisElement, const OtherElementType& InCurveToCombineElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::BothArgsValid))
			{
				InOutThisElement.Value = FMath::Max(InOutThisElement.Value, InCurveToCombineElement.Value);
				InOutThisElement.Flags |= InCurveToCombineElement.Flags;
			}
			else if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				InOutThisElement.Value = InCurveToCombineElement.Value;
				InOutThisElement.Flags = InCurveToCombineElement.Flags;
			}
		});
	}

	/**
	 * This doesn't blend but combines MIN(current weight, CurveToCombine weight)
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void UseMinValue(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToCombine)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_UseMinValue);
		
		UE::Anim::FNamedValueArrayUtils::Union(*this, CurveToCombine, [](ElementType& InOutThisElement, const OtherElementType& InCurveToCombineElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::BothArgsValid))
			{
				InOutThisElement.Value = FMath::Min(InOutThisElement.Value, InCurveToCombineElement.Value);
				InOutThisElement.Flags |= InCurveToCombineElement.Flags;
			}
			else if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				InOutThisElement.Value = InCurveToCombineElement.Value;
				InOutThisElement.Flags = InCurveToCombineElement.Flags;
			}
		});
	}

	/**
	 * If 'this' does not contain a valid element, then the value in 'this' is set, otherwise the value
	 * is not modified.
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void CombinePreserved(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToCombine)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_CombinePreserved);
		
		UE::Anim::FNamedValueArrayUtils::Union(*this, CurveToCombine, [](ElementType& InOutThisElement, const OtherElementType& InCurveToCombineElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if(!EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg0))
			{
				InOutThisElement.Value = InCurveToCombineElement.Value;
				InOutThisElement.Flags = InCurveToCombineElement.Flags;
			}
		});
	}

	/**
	 * If CurveToCombine contains a valid element, then the value in 'this' is overridden, otherwise the value is not
	 * modified.
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void Combine(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToCombine)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Combine);
		
		UE::Anim::FNamedValueArrayUtils::Union(*this, CurveToCombine, [](ElementType& InOutThisElement, const OtherElementType& InCurveToCombineElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				InOutThisElement.Value = InCurveToCombineElement.Value;
				InOutThisElement.Flags = InCurveToCombineElement.Flags;
			}
		});
	}

	/**
	 * Override with input curve * weight
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void Override(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToOverrideFrom, float Weight)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Override_Weighted);
		
		CopyFrom(CurveToOverrideFrom);

		if (!FMath::IsNearlyEqual(Weight, 1.0f))
		{
			for (ElementType& Element : Super::Elements)
			{
				Element.Value *= Weight;
			}
		}
	}

public:
	/**
	 * Override with input curve 
	 */
	template<typename OtherAllocator, typename OtherElementType>
	void Override(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& CurveToOverrideFrom)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Override);
		
		CopyFrom(CurveToOverrideFrom);
	}
	
	/**
	 * Override with input curve, leaving input curve invalid
	 */
	void OverrideMove(TBaseBlendedCurve& CurveToOverrideFrom)
	{
		// make sure this doesn't happen
		if (ensure(&CurveToOverrideFrom != this))
		{
			Super::Elements = MoveTemp(CurveToOverrideFrom.Elements);
			Super::bSorted = CurveToOverrideFrom.bSorted;
			CurveToOverrideFrom.bSorted = false;
		}
	}

	UE_DEPRECATED(5.3, "Please use Num(). Element validity/enabled state is now handled via UE::Anim::FCurveFilter.")
	int32 NumValid() const { return Super::Elements.Num(); }

	/** Copy elements between curves that have different allocators & element types. Does not copy filter. */
	template <typename OtherAllocator, typename OtherElementType>
	void CopyFrom(const TBaseBlendedCurve<OtherAllocator, OtherElementType>& InCurveToCopyFrom)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_CopyFrom);
		
		if constexpr(std::is_same<ElementType, OtherElementType>::value)
		{
			Super::Elements = InCurveToCopyFrom.Elements;
			Super::bSorted = InCurveToCopyFrom.bSorted;
		}
		else
		{
			Super::Elements.Reset();

			UE::Anim::FNamedValueArrayUtils::Union(*this, InCurveToCopyFrom,
				[](ElementType& InOutResultElement, const OtherElementType& InParamElement, UE::Anim::ENamedValueUnionFlags InFlags)
				{
					InOutResultElement.Value = InParamElement.Value;
					InOutResultElement.Flags = InParamElement.Flags;
				});

			Super::bSorted = InCurveToCopyFrom.bSorted;
		}
	}

	/** Once moved, source is invalid */
	void MoveFrom(TBaseBlendedCurve& CurveToMoveFrom)
	{
		Super::Elements = MoveTemp(CurveToMoveFrom.Elements);
		Super::bSorted = CurveToMoveFrom.bSorted;
		CurveToMoveFrom.bSorted = false;
	}

	/** Reserves memory for InNumElements */
	void Reserve(int32 InNumElements)
	{
		CURVE_PROFILE_CYCLE_COUNTER(TBaseBlendedCurve_Reserve);
		Super::Elements.Reserve(InNumElements);
	}
	
	UE_DEPRECATED(5.3, "Curves are now always valid.")
	bool IsValid() const
	{
		return true;
	}

	/** Set the filter referenced by this curve. */
	void SetFilter(const UE::Anim::FCurveFilter* InFilter)
	{
		Filter = InFilter;
	}

	/** Get the filter referenced by this curve. */
	const UE::Anim::FCurveFilter* GetFilter() const
	{
		return Filter;
	}
};

template<typename AllocatorType>
struct UE_DEPRECATED(5.3, "FBaseBlendedCurve was renamed to TBaseBlendedCurve.") FBaseBlendedCurve : public TBaseBlendedCurve<AllocatorType>
{
};

struct FBlendedCurve : public TBaseBlendedCurve<FAnimStackAllocator>
{
	using TBaseBlendedCurve<FAnimStackAllocator>::InitFrom; 
	
	/** Initialize from bone container's filtered curves */
	ENGINE_API void InitFrom(const FBoneContainer& InBoneContainer);
};

struct FBlendedHeapCurve : public TBaseBlendedCurve<FDefaultAllocator>
{
	using TBaseBlendedCurve<FDefaultAllocator>::InitFrom;
	
	/** Initialize from bone container's filtered curves */
	ENGINE_API void InitFrom(const FBoneContainer& InBoneContainer);
};

UENUM(BlueprintType)
enum class ERawCurveTrackTypes : uint8
{
	RCT_Float UMETA(DisplayName = "Float Curve"),
	RCT_Vector UMETA(DisplayName = "Vector Curve", Hidden),
	RCT_Transform UMETA(DisplayName = "Transformation Curve"),
	RCT_MAX
};

/**
 * Raw Curve data for serialization
 */
USTRUCT()
struct FRawCurveTracks
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FFloatCurve>		FloatCurves;

#if WITH_EDITORONLY_DATA
	/**
	 * @note : Currently VectorCurves are not evaluated or used for anything else but transient data for modifying bone track
	 *			Note that it doesn't have UPROPERTY tag yet. In the future, we'd like this to be serialized, but not for now
	 **/
	UPROPERTY(transient)
	TArray<FVectorCurve>	VectorCurves;

	/**
	 * @note : TransformCurves are used to edit additive animation in editor. 
	 **/
	UPROPERTY()
	TArray<FTransformCurve>	TransformCurves;
#endif // #if WITH_EDITORONLY_DATA

	/**
	 * Evaluate curve data at the time CurrentTime, and add to Instance. It only evaluates Float Curve for now
	 *
	 * return true if curve exists, false otherwise
	 */
	ENGINE_API void EvaluateCurveData( FBlendedCurve& Curves, float CurrentTime ) const;

#if WITH_EDITOR
	/**
	 *	Evaluate transform curves 
	 */
	ENGINE_API void EvaluateTransformCurveData(USkeleton * Skeleton, TMap<FName, FTransform>&OutCurves, float CurrentTime, float BlendWeight) const;

	/**
	* Add new float curve from the given UID if not existing and add the key with time/value
	*/
	ENGINE_API void AddFloatCurveKey(const FName& NewCurve, int32 CurveFlags, float Time, float Value);

	ENGINE_API void RemoveRedundantKeys(float Tolerance = UE_SMALL_NUMBER, FFrameRate SampleRate = FFrameRate(0,0));

#endif // WITH_EDITOR

	/**
	* Find curve data based on the curve name
	*/
	ENGINE_API FAnimCurveBase * GetCurveData(FName Name, ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float);
	ENGINE_API const FAnimCurveBase * GetCurveData(FName Name, ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float) const;

	/**
	 * Add new curve from the provided name and return true if success
	 * bVectorInterpCurve == true, then it will create FVectorCuve, otherwise, FFloatCurve
	 */
	ENGINE_API bool AddCurveData(const FName& NewCurve, int32 CurveFlags = AACF_DefaultCurve, ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float);

	/**
	 * Delete curve data
	 */
	ENGINE_API bool DeleteCurveData(const FName& CurveToDelete, ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float);

	/**
	 * Delete all curve data
	 */
	ENGINE_API void DeleteAllCurveData(ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float);

	/**
	 * Duplicate curve data
	 */
	ENGINE_API bool DuplicateCurveData(const FName& CurveToCopy, const FName& NewCurve, ERawCurveTrackTypes SupportedCurveType = ERawCurveTrackTypes::RCT_Float);

	// This allows loading data that was saved between VER_UE4_SKELETON_ADD_SMARTNAMES and FFrameworkObjectVersion::SmartNameRefactor
	void PostSerializeFixup(FArchive& Ar);

	/*
	 * resize curve length. If longer, it doesn't do any. If shorter, remove previous keys and add new key to the end of the frame. 
	 */
	void Resize(float TotalLength, bool bInsert/* whether insert or remove*/, float OldStartTime, float OldEndTime);
	/** 
	 * Clear all keys
	 */
	void Empty()
	{
		FloatCurves.Empty();
#if WITH_EDITORONLY_DATA
		VectorCurves.Empty();
		TransformCurves.Empty();
#endif
	}
	/** 
	 * Clear all keys preserving allocations
	 */
	void Reset()
	{
		FloatCurves.Reset();
#if WITH_EDITORONLY_DATA
		VectorCurves.Reset();
		TransformCurves.Reset();
#endif
	}

private:
	/** 
	 * Adding vector curve support - this is all transient data for now. This does not save and all these data will be baked into RawAnimationData
	 */

	/**
	* Find curve data based on the curve name
	*/
	template <typename DataType>
	DataType * GetCurveDataImpl(TArray<DataType>& Curves, FName Name);
	
	/**
	* Find curve data based on the curve name
	*/
	template <typename DataType>
	const DataType * GetCurveDataImpl(const TArray<DataType>& Curves, FName Name) const;
	
	/**
	 * Add new curve from the provided UID and return true if success
	 * bVectorInterpCurve == true, then it will create FVectorCuve, otherwise, FFloatCurve
	 */
	template <typename DataType>
	bool AddCurveDataImpl(TArray<DataType>& Curves, const FName& NewCurve, int32 CurveFlags);
	/**
	 * Delete curve data 
	 */
	template <typename DataType>
	bool DeleteCurveDataImpl(TArray<DataType>& Curves, const FName& CurveToDelete);
	/**
	 * Duplicate curve data
	 * 
	 */
	template <typename DataType>
	bool DuplicateCurveDataImpl(TArray<DataType>& Curves, const FName& CurveToCopy, const FName& NewCurve);
};

FArchive& operator<<(FArchive& Ar, FRawCurveTracks& D);
