// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimBulkCurves.h"
#include "AnimCurveTypes.h"
#include "AnimCurveFilter.h"


namespace UE::Anim
{

struct FCurveUtils
{
private:
	/** Helper function used to apply filtering to elements. */
	static bool ElementPassesFilter(ECurveFilterMode InFilterMode, ECurveFilterFlags InFilterFlags)
	{
		if(EnumHasAnyFlags(InFilterFlags, ECurveFilterFlags::Disallowed))
		{
			return false;
		}

		switch(InFilterMode)
		{
		case ECurveFilterMode::None:
			return true;
		case ECurveFilterMode::DisallowAll:
			return false;
		case ECurveFilterMode::AllowOnlyFiltered:
			return EnumHasAnyFlags(InFilterFlags, ECurveFilterFlags::Filtered);
		case ECurveFilterMode::DisallowFiltered:
			return !EnumHasAnyFlags(InFilterFlags, ECurveFilterFlags::Filtered);
		}

		return false;
	};

	// Helper function for building curves, applying any filtering
	// Assumes (and enforces in debug builds) elements being built are in FName sorted order
	template<typename NamePredicateType, typename ValuePredicateType, typename CurveAllocatorType, typename CurveElementType>
	static void BuildSortedFiltered(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate, const FCurveFilter& InFilter)
	{
		// Early out if we are filtering all curves
		if(InFilter.FilterMode == ECurveFilterMode::DisallowAll)
		{
			OutCurve.Empty();
			return;
		}

		InFilter.SortElementsIfRequired();

		OutCurve.Empty();
		OutCurve.Reserve(InNumElements);

		const int32 NumElements0 = InNumElements;
		const int32 NumElements1 = InFilter.Elements.Num();

		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

#if DO_ANIM_NAMED_VALUE_SORTING_CHECKS
		FName LastCurveName;
#endif

		// Perform dual-iteration on the two sorted arrays
		while(ElementIndex0 < NumElements0 || ElementIndex1 < NumElements1)
		{
			if(ElementIndex0 == NumElements0 && ElementIndex1 < NumElements1)
			{
				// Reached end of user data with remaining in filter, we can just early out
				break;
			}
			else if(ElementIndex1 == NumElements1 && ElementIndex0 < NumElements0)
			{
				// Reached end of filter with remaining in user data, we can just run straight through the user data
				OutCurve.Elements.Reserve(OutCurve.Elements.Num() + (NumElements0 - ElementIndex0));
				for( ; ElementIndex0 < NumElements0; ++ElementIndex0)
				{
					if(ElementPassesFilter(InFilter.FilterMode, ECurveFilterFlags::None))
					{
						OutCurve.Elements.Emplace(InNamePredicate(ElementIndex0), InValuePredicate(ElementIndex0));
					}
				}
				break;
			}
			
			const FName CurveName = InNamePredicate(ElementIndex0);
			const FCurveFilterElement* RESTRICT Element1 = &InFilter.Elements[ElementIndex1];

#if DO_ANIM_NAMED_VALUE_SORTING_CHECKS
			// Check sorting invariants
			if(ElementIndex0 > 0)
			{
				check(LastCurveName == CurveName || LastCurveName.FastLess(CurveName))
			}
			LastCurveName = CurveName;
#endif

			if(CurveName == Element1->Name)
			{
				// Elements match, check filter & write to curve
				if(ElementPassesFilter(InFilter.FilterMode, Element1->Flags))
				{
					OutCurve.Elements.Emplace(CurveName, InValuePredicate(ElementIndex0));
				}

				++ElementIndex0;
				++ElementIndex1;
			}
			else if(CurveName.FastLess(Element1->Name))
			{
				// Element exists only in user data
				if(ElementPassesFilter(InFilter.FilterMode, ECurveFilterFlags::None))
				{
					OutCurve.Elements.Emplace(CurveName, InValuePredicate(ElementIndex0));
				}
				
				++ElementIndex0;
			}
			else
			{
				// Element exists only in filter, skip
				++ElementIndex1;
			}
		}

		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves
	template<typename NamePredicateType, typename ValuePredicateType, typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildLinearUnfiltered(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate)
	{
		OutCurve.Empty();
		OutCurve.Reserve(InNumElements);

		for(int32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
		{
			OutCurve.Elements.Emplace(InNamePredicate(ElementIndex), InValuePredicate(ElementIndex));
		}
		
		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves
	template<typename NamePredicateType, typename ValuePredicateType, typename ValidityPredicateType, typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildLinearUnfiltered(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate, ValidityPredicateType InValidityPredicate)
	{
		OutCurve.Empty();
		OutCurve.Reserve(InNumElements);

		for(int32 ElementIndex = 0; ElementIndex < InNumElements; ++ElementIndex)
		{
			if (InValidityPredicate(ElementIndex))
			{
				OutCurve.Elements.Emplace(InNamePredicate(ElementIndex), InValuePredicate(ElementIndex));
			}
		}
		
		OutCurve.CheckDuplicates();
	}

public:
	// Helper function for building curves, applying any filtering
	// Assumes (and enforces in debug builds) elements being built are in FName sorted order
	template<typename NamePredicateType, typename ValuePredicateType, typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildSorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate, const FCurveFilter* InFilter = nullptr)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildSorted);

		if(InFilter != nullptr && !InFilter->IsEmpty())
		{
			BuildSortedFiltered(OutCurve, InNumElements, InNamePredicate, InValuePredicate, *InFilter);
		}
		else
		{
			BuildLinearUnfiltered(OutCurve, InNumElements, InNamePredicate, InValuePredicate);
		}

		OutCurve.bSorted = true;
		OutCurve.CheckSorted();
	}

	// Helper function for building curves
	template<typename NamePredicateType, typename ValuePredicateType, typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsortedUnfiltered(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedUnfiltered);

		BuildLinearUnfiltered(OutCurve, InNumElements, InNamePredicate, InValuePredicate);
	}

	// Helper function for building curves from a map
	template<typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, const TMap<FName, float>& InMap)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedFromMap);

		OutCurve.Empty();
		OutCurve.Reserve(InMap.Num());

		for(const TPair<FName, float>& NameValuePair : InMap)
		{
			OutCurve.Elements.Emplace(NameValuePair.Key, NameValuePair.Value);
		}
	}

	// Helper function for building curves from an array view of name/value pairs
	template<typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, TConstArrayView<TTuple<FName, float>> InInputArrayView)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedFromArrayView);

		OutCurve.Empty();
		OutCurve.Reserve(InInputArrayView.Num());

		for(const TTuple<FName, float>& NameValueTuple : InInputArrayView)
		{
			OutCurve.Elements.Emplace(NameValueTuple.Get<0>(), NameValueTuple.Get<1>());
		}

		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves from an initializer list of name/value pairs
	template<typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, std::initializer_list<TTuple<FName, float>> InInputArgs)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedFromArrayView);

		OutCurve.Empty();
		OutCurve.Reserve(InInputArgs.size());

		for(const TTuple<FName, float>& NameValueTuple : InInputArgs)
		{
			OutCurve.Elements.Emplace(NameValueTuple.Get<0>(), NameValueTuple.Get<1>());
		}
		
		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves from an initializer list of name/flag pairs
	template<typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, std::initializer_list<TTuple<FName, UE::Anim::ECurveElementFlags>> InInputArgs)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedFromArrayView);

		OutCurve.Empty();
		OutCurve.Reserve(InInputArgs.size());

		for(const TTuple<FName, UE::Anim::ECurveElementFlags>& NameFlagTuple : InInputArgs)
		{
			OutCurve.Elements.Emplace(NameFlagTuple.Get<0>(), NameFlagTuple.Get<1>());
		}

		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves from an initializer list of name/value/flag tuples
	template<typename CurveAllocatorType, typename CurveElementType>
	inline static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, std::initializer_list<TTuple<FName, float, UE::Anim::ECurveElementFlags>> InInputArgs)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsortedFromArrayView);

		OutCurve.Empty();
		OutCurve.Reserve(InInputArgs.size());

		for(const TTuple<FName, float, UE::Anim::ECurveElementFlags>& NameValueFlagTuple : InInputArgs)
		{
			OutCurve.Elements.Emplace(NameValueFlagTuple.Get<0>(), NameValueFlagTuple.Get<1>(), NameValueFlagTuple.Get<2>());
		}

		OutCurve.CheckDuplicates();
	}

	// Helper function for building curves, applying any filtering
	// Note: uses TMemStackAllocator internally when filtering, so requires a FMemMark to be set somewhere in the callstack 
	template<typename NamePredicateType, typename ValuePredicateType, typename CurveAllocatorType, typename CurveElementType>
	static void BuildUnsorted(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate, const FCurveFilter* InFilter = nullptr)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsorted);
		
		if(InFilter != nullptr && !InFilter->IsEmpty())
		{
			// To use the filter, we need to pre-sort so we can effectively merge
			// Build a set of indices
			TArray<int32, FAnimStackAllocator> SortedIndices;
			SortedIndices.SetNumUninitialized(InNumElements);
			for(int32 NameIndex = 0; NameIndex < InNumElements; ++NameIndex)
			{
				SortedIndices[NameIndex] = NameIndex;
			}

			// Sort indices by name
			SortedIndices.Sort([&InNamePredicate](int32 InLHS, int32 InRHS)
			{
				return InNamePredicate(InLHS).FastLess(InNamePredicate(InRHS));
			});

			auto GetSortedName = [&SortedIndices, &InNamePredicate](int32 InIndex)
			{
				return InNamePredicate(SortedIndices[InIndex]);
			};

			auto GetSortedValue = [&SortedIndices, &InValuePredicate](int32 InIndex)
			{
				return InValuePredicate(SortedIndices[InIndex]);
			};
			
			BuildSortedFiltered(OutCurve, InNumElements, GetSortedName, GetSortedValue, *InFilter);
			OutCurve.bSorted = true;
			OutCurve.CheckSorted();
		}
		else
		{
			BuildLinearUnfiltered(OutCurve, InNumElements, InNamePredicate, InValuePredicate);
		}
	}

	// Helper function for building curves, applying filtering through InValidityPredicate
	template<typename NamePredicateType, typename ValuePredicateType, typename ValidityPredicateType, typename CurveAllocatorType, typename CurveElementType>
	static void BuildUnsortedValidated(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& OutCurve, int32 InNumElements, NamePredicateType InNamePredicate, ValuePredicateType InValuePredicate, ValidityPredicateType InValidityPredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BuildUnsorted);
		BuildLinearUnfiltered(OutCurve, InNumElements, InNamePredicate, InValuePredicate, InValidityPredicate);
	}

	/**
	 * Remove any curves in InOutCurve that are filtered by InFilter
	 */
	template<typename CurveAllocatorType, typename CurveElementType>
	static void Filter(TBaseBlendedCurve<CurveAllocatorType, CurveElementType>& InOutCurve, const FCurveFilter& InFilter)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_Filter);
		
		switch(InFilter.FilterMode)
		{
		case ECurveFilterMode::None:
			// No filtering, early out
			return;
		case ECurveFilterMode::DisallowAll:
			// Filtering all curves, so just clear curve
			InOutCurve.Elements.Reset();
			return;
		case ECurveFilterMode::AllowOnlyFiltered:
			if(InFilter.Num() == 0)
			{
				// Allow only filtered, and no filtered, so just clear curve
				InOutCurve.Elements.Reset();
				return;
			}
			break;
		default:
			break;
		}

		// Sort both inputs if required
		InOutCurve.SortElementsIfRequired();
		InFilter.SortElementsIfRequired();

		int32 NumElements0 = InOutCurve.Num();
		const int32 NumElements1 = InFilter.Num();
		
		// Perform dual-iteration on the two sorted arrays
		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

		while(true)
		{
			if(ElementIndex0 == NumElements0 && ElementIndex1 < NumElements1)
			{
				// Reached end of curve with remaining in filter, we can just early out
				break;
			}
			else if(ElementIndex1 == NumElements1 && ElementIndex0 < NumElements0)
			{
				// Reached end of filter with remaining in curve, we can just run straight through the curve
				while(ElementIndex0 < NumElements0)
				{
					if(!ElementPassesFilter(InFilter.FilterMode, ECurveFilterFlags::None))
					{
						InOutCurve.Elements.RemoveAt(ElementIndex0, EAllowShrinking::No);
						NumElements0 = InOutCurve.Num();
					}
					else
					{
						++ElementIndex0;
					}
				}
				break;
			}
			else if(ElementIndex0 == NumElements0 && ElementIndex1 == NumElements1)
			{
				// All elements exhausted, exit
				break;
			}
			
			const CurveElementType* RESTRICT Element0 = &InOutCurve.Elements[ElementIndex0];
			const FCurveFilterElement* RESTRICT Element1 = &InFilter.Elements[ElementIndex1];
			
			if(Element0->Name == Element1->Name)
			{
				// Elements match so check filter flags to see if it should be removed from curve
				if(!ElementPassesFilter(InFilter.FilterMode, Element1->Flags))
				{
					InOutCurve.Elements.RemoveAt(ElementIndex0, EAllowShrinking::No);
					NumElements0 = InOutCurve.Num();
					++ElementIndex1;
				}
				else
				{
					++ElementIndex0;
					++ElementIndex1;
				}
			}
			else if(Element0->Name.FastLess(Element1->Name))
			{
				// Element exists only in curve, check filter
				if(!ElementPassesFilter(InFilter.FilterMode, ECurveFilterFlags::None))
				{
					InOutCurve.Elements.RemoveAt(ElementIndex0, EAllowShrinking::No);
					NumElements0 = InOutCurve.Num();
				}
				else
				{
					++ElementIndex0;
				}
			}
			else
			{
				// Element exists only in filter, skip
				++ElementIndex1;
			}
		}

		InOutCurve.CheckSorted();
	}

	/**
	 * Extracts elements from curves in bulk.
	 * This is more efficient that just calling Curve.Get() repeatedly.
	 * @param	InCurve				The curve to get values from
	 * @param	InBulkCurves		The curve specifying values to get
	 * @param	InValuePredicate	Predicate to get each value. Signature: (const CurveType1::ElementType& InElement1, float InValue) -> void
	 **/
	template<typename CurveType0, typename CurveType1, typename ValuePredicateType>
	static void BulkGet(const CurveType0& InCurve, const CurveType1& InBulkCurves, ValuePredicateType InValuePredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BulkGet);
		
		FNamedValueArrayUtils::Intersection(InCurve, InBulkCurves,
			[&InValuePredicate](const typename CurveType0::ElementType& InElement0, const typename CurveType1::ElementType& InElement1)
			{
				InValuePredicate(InElement1, InElement0.Value);
			});
	}

	/**
	 * Inserts elements to curves in bulk.
	 * This is more efficient that just calling Curve.Set() repeatedly.
	 * @param	InCurve				The curve to set values into
	 * @param	InBulkCurves		The curve specifying values to set
	 * @param	InValuePredicate	Predicate to set each value. Signature: (const CurveType1::ElementType& InElement1) -> float
	 **/
	template<typename CurveType0, typename CurveType1, typename ValuePredicateType>
	static void BulkSet(CurveType0& InCurve, const CurveType1& InBulkCurves, ValuePredicateType InValuePredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FCurveUtils_BulkSet);
		
		FNamedValueArrayUtils::Union(InCurve, InBulkCurves,
			[&InValuePredicate](typename CurveType0::ElementType& InOutElement0, const typename CurveType1::ElementType& InElement1, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				if(EnumHasAllFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
				{
					InOutElement0.Value = InValuePredicate(InElement1);
				}
			});
	}
};

}
