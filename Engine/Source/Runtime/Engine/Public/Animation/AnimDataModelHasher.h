// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Curves/RichCurve.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"

#if WITH_EDITOR
#include "Misc/StringBuilder.h"
#include "String/BytesToHex.h"
#endif

namespace UE::Anim::DataModel
{

/**
 * Base class of Hasher classes used by IAnimationDataModel classes to share code between GenerateGuid and
 * GenerateDebugStateString.
 * This class uses no virtuals for performance; the function that takes a Hasher should be templated on the
 * type of the Hasher.
 *
 * This base class shows the required interface and some convenience functions. It uses the Curiously Recurring
 * Template Pattern to call the Subclass's version of internal function calls.
 */
template <typename SubClassType>
class FHasherBase
{
public:
	// Output functions; the base class does nothing and subclasses must override

	void UpdateBytes(const uint8* Data, int32 Size, const TCHAR* Name)
	{
		checkf(false, TEXT("Subclass must implement"));
	}
	void BeginObject(const TCHAR* Name = nullptr)
	{
		checkf(false, TEXT("Subclass must implement"));
	}
	void EndObject()
	{
		checkf(false, TEXT("Subclass must implement"));
	}

	// Convenience functions; the base class implementation does not need to be overridden

	template <typename T>
	void UpdateData(const T& Data, const TCHAR* Name)
	{
		TypedThis()->UpdateBytes(reinterpret_cast<const uint8*>(&Data), sizeof(Data), Name);
	}
	template <typename T>
	void UpdateArray(const TArray<T>& Array, const TCHAR* Name)
	{
		TypedThis()->UpdateBytes(reinterpret_cast<const uint8*>(Array.GetData()), Array.Num() * Array.GetTypeSize(), Name);
	}
	template <typename T>
	void UpdateArray(TConstArrayView<T> Array, const TCHAR* Name)
	{
		TypedThis()->UpdateBytes(reinterpret_cast<const uint8*>(Array.GetData()), Array.Num() * Array.GetTypeSize(), Name);
	}
	void UpdateString(const FString& Data, const TCHAR* Name)
	{
		TypedThis()->UpdateBytes(reinterpret_cast<const uint8*>(*Data), Data.Len() * sizeof((*Data)[0]), Name);
	}
	/**
	 * Previously, some code was serializing strings as GetCharArray, which includes the null terminator,
	 * and we did not want to change behavior because it would change the key. Those sites call UpdateLegacyString
	 * instead, and the subclasses that want to preserve the key override it.
	 */
	void UpdateLegacyString(const FString& Data, const TCHAR* Name)
	{
		TypedThis()->UpdateString(Data, Name);
	}

	void UpdateRichCurve(const FRichCurve& Curve, const TCHAR* Name)
	{
		SubClassType& Hasher = *TypedThis();

		Hasher.BeginObject(Name);
		Hasher.UpdateData(Curve.DefaultValue, TEXT("D"));
		Hasher.UpdateArray(Curve.GetConstRefOfKeys(), TEXT("K"));
		Hasher.UpdateData(Curve.PreInfinityExtrap, TEXT("E"));
		Hasher.UpdateData(Curve.PostInfinityExtrap, TEXT("O"));
		Hasher.EndObject();
	};

	void UpdateVectorCurve(const FVectorCurve& VectorCurve, const TCHAR* Name)
	{
		SubClassType& Hasher = *TypedThis();

		Hasher.BeginObject(Name);
		for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
		{
			Hasher.UpdateRichCurve(VectorCurve.FloatCurves[ChannelIndex], TEXT("C"));
		}
		Hasher.EndObject();
	};

	void UpdateAnimatedBoneAttributes(TConstArrayView<FAnimatedBoneAttribute> AnimatedBoneAttributes, const TCHAR* Name)
	{
		SubClassType& Hasher = *TypedThis();

		Hasher.BeginObject(Name);
		for (const FAnimatedBoneAttribute& Attribute : AnimatedBoneAttributes)
		{
			const UScriptStruct* TypeStruct = Attribute.Identifier.GetType();
			const uint32 StructSize = TypeStruct->GetPropertiesSize();
			const bool bHasTypeHash = TypeStruct->GetCppStructOps()->HasGetTypeHash();

			Hasher.BeginObject();
			Hasher.UpdateLegacyString(Attribute.Identifier.GetName().ToString(), TEXT("N"));
			Hasher.UpdateLegacyString(Attribute.Identifier.GetBoneName().ToString(), TEXT("BN"));
			Hasher.UpdateData(Attribute.Identifier.GetBoneIndex(), TEXT("BI"));
			Hasher.UpdateLegacyString(TypeStruct->GetFName().ToString(), TEXT("T"));
			Hasher.BeginObject(TEXT("K"));
			for (const FAttributeKey& Key : Attribute.Curve.GetConstRefOfKeys())
			{
				Hasher.BeginObject();
				Hasher.UpdateData(Key.Time, TEXT("T"));
				if (bHasTypeHash)
				{
					const uint32 KeyHash = TypeStruct->GetStructTypeHash(Key.GetValuePtr<uint8>());
					Hasher.UpdateData(KeyHash, TEXT("H"));
				}
				else
				{
					Hasher.UpdateBytes(Key.GetValuePtr<uint8>(), StructSize, TEXT("B"));
				}
				Hasher.EndObject();
			}
			Hasher.EndObject();
			Hasher.EndObject();
		}
		Hasher.EndObject();
	}

	void UpdateTransformCurves(TConstArrayView<FTransformCurve> TransformCurves, const TCHAR* Name)
	{
		SubClassType& Hasher = *TypedThis();

		Hasher.BeginObject(Name);
		for (const FTransformCurve& Curve : TransformCurves)
		{
			Hasher.BeginObject();
			Hasher.UpdateLegacyString(Curve.GetName().ToString(), TEXT("N"));
			Hasher.UpdateVectorCurve(Curve.TranslationCurve, TEXT("T"));
			Hasher.UpdateVectorCurve(Curve.RotationCurve, TEXT("R"));
			Hasher.UpdateVectorCurve(Curve.ScaleCurve, TEXT("S"));
			Hasher.EndObject();
		}
		Hasher.EndObject();
	}

private:
	SubClassType* TypedThis()
	{
		return static_cast<SubClassType*>(this);
	}

};

/** A hasher used for GenerateGuid, writes SHA and converts the SHA to a guid. */
class FHasherSha : public FHasherBase<FHasherSha>
{
public:
	void UpdateBytes(const uint8* Data, int32 Size, const TCHAR* Name)
	{
		Sha.Update(Data, Size);
	}
	void UpdateString(const FString& Data, const TCHAR* Name)
	{
		Sha.UpdateWithString(*Data, Data.Len());
	}
	void UpdateLegacyString(const FString& Data, const TCHAR* Name)
	{
		UpdateArray(Data.GetCharArray(), Name);
	}
	void BeginObject(const TCHAR* Name = nullptr)
	{
	}
	void EndObject()
	{
	}

	FGuid FinalGuid()
	{
		checkf(!bFinalized, TEXT("Calling FinalGuid more than once is not implemented."));
		bFinalized = true;
		Sha.Final();

		uint32 Hash[5];
		Sha.GetHash(reinterpret_cast<uint8*>(Hash));
		return FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	}

private:
	FSHA1 Sha;
	bool bFinalized = false;
};

#if WITH_EDITOR
/** A hasher used for GenerateDebugStateString. Writes data as string, untyped data as BytesToHex. */
class FHasherCopyToText : public FHasherBase<FHasherCopyToText>
{
public:
	void UpdateBytes(const uint8* Data, int32 Size, const TCHAR* Name)
	{
		Text << Divider << Name << Divider;
		UE::String::BytesToHex(TConstArrayView<uint8>(Data, Size), Text);
	}
	void UpdateString(const FString& Data, const TCHAR* Name)
	{
		Text << Divider << Name << Divider << Data;
	}
	void BeginObject(const TCHAR* Name = nullptr)
	{
		Text << Divider << (Name ? Name : TEXT("")) << Open;
	}
	void EndObject()
	{
		Text << Close;
	}

	FStringView GetStringView() const
	{
		return Text;
	}
	FString GetString() const
	{
		return FString(Text);
	}
private:
	TStringBuilder<256> Text;
	constexpr static TCHAR Divider = '_';
	constexpr static TCHAR Open = '{';
	constexpr static TCHAR Close = '}';
};
#endif


} // namespace UE::Anim::DataModel

