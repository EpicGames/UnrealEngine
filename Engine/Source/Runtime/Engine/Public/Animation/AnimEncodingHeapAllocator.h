// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"

/**
 * Heap allocator for animation decompression codec that want to avoid range checks for performance reasons.
 */
class FAnimEncodingHeapAllocator : public FHeapAllocator
{
public:
	/** Don't want to lose performance on range checks in performance-critical animation decompression code. */
	enum { RequireRangeCheck = false };
};
