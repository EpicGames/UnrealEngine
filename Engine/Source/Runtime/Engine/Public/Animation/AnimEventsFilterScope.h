// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"
#include "AnimNotifyQueue.h"

namespace UE { namespace Anim
{

class IAnimEventsFilterContext : public IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE_API(IAnimEventsFilterContext, ENGINE_API)

public:
	
	virtual bool ShouldFilterNotify(const FAnimNotifyEventReference & InNotifyEventRef) const = 0;
};
	
}}