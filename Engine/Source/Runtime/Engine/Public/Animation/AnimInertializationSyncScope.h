// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimNodeMessages.h"

namespace UE { namespace Anim {

// Scoped graph message used to synchronize tick records using inertialization 
class FAnimInertializationSyncScope : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FAnimInertializationSyncScope, ENGINE_API)
};

}} // namespace UE::Anim
