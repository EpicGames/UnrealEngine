// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimSync.h"
#include "Animation/AnimInertializationRequest.h"

#include "AnimInstanceProxy.generated.h"

struct FComponentSpacePoseContext;
class UAnimInstance;
class UAnimNotify;
class UAnimNotifyState;
class UBlendProfile;
class UBlendSpace;
class FTokenizedMessage;
namespace EMessageSeverity { enum Type : int; }
struct FAnimationEvaluationContext;
struct FAnimationPoseData;
struct FAnimBlueprintDebugData_NodeVisit;
struct FAnimBlueprintDebugData_AttributeRecord;
struct FAnimGroupInstance;
struct FAnimNodePoseWatch;
struct FAnimNode_AssetPlayerBase;
struct FAnimNode_AssetPlayerRelevancyBase;
struct FAnimNode_Base;
struct FAnimNode_LinkedInputPose;
struct FAnimNode_SaveCachedPose;
struct FAnimNode_StateMachine;
struct FAnimTickRecord;
struct FBakedAnimationStateMachine;
struct FCompactPose;
struct FMontageActiveSlotTracker;
struct FMontageEvaluationState;
struct FNativeStateBinding;
struct FNativeTransitionBinding;
struct FNodeDebugData;
struct FPoseContext;
struct FPoseSnapshot;
enum class ETransitionRequestQueueMode : uint8;
enum class ETransitionRequestOverwriteMode : uint8;

namespace UE::Anim
{
	class FAnimSyncGroupScope;
	class FActiveStateMachineScope;
	struct FAnimSyncParams;
	// DEPRECATED use FInertializationRequest instead
	using FSlotInertializationRequest = TPair<float, const UBlendProfile*>;
}

template<class PoseType> struct FCSPose;

// Disable debugging information for shipping and test builds.
#define ENABLE_ANIM_DRAW_DEBUG (1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

// Disable node logging for shipping and test builds
#define ENABLE_ANIM_LOGGING (1 && !NO_LOGGING && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

extern const FName NAME_AnimBlueprintLog;
extern const FName NAME_Evaluate;
extern const FName NAME_Update;
extern const FName NAME_AnimGraph;

DECLARE_DELEGATE_RetVal(bool, FCanTakeTransition);
DECLARE_DELEGATE_ThreeParams(FOnGraphStateChanged, const struct FAnimNode_StateMachine&, int32, int32);

UENUM()
namespace EDrawDebugItemType
{
	enum Type : int
	{
		DirectionalArrow,
		Sphere,
		Line,
		OnScreenMessage,
		CoordinateSystem,
		Point,
		Circle,
		Cone,
		InWorldMessage,
		Capsule,
	};
}

struct FQueuedDrawDebugItem 
{
	struct FVectorEntry
	{
		FVectorEntry(const FVector& InVector)
			: Value(InVector)
		{}

		FVector Value;
	};

	TEnumAsByte<EDrawDebugItemType::Type> ItemType = EDrawDebugItemType::DirectionalArrow;

	union
	{
		FVector StartLoc;
		struct
		{
			float Length;
			float AngleWidth;
			float AngleHeight;
		};
	};
	
	union
	{
		FVectorEntry EndLoc;
		FVectorEntry Direction;
	};

	FVector Center = FVector(0.f);
	FRotator Rotation = FRotator(0.f);
	float Radius = 0.f;
	float Size = 0.f;
	int32 Segments = 0;
	FColor Color = FColor(0);
	bool bPersistentLines = false;
	float LifeTime = 0.f;
	float Thickness = 0.f;
	FString Message;
	FVector2D TextScale = FVector2D(0.f);
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = SDPG_World;
	FQueuedDrawDebugItem() 
		: StartLoc(FVector(0.f))
		, EndLoc(FVector(0.f))
	{};
};

/** Proxy object passed around during animation tree update in lieu of a UAnimInstance */
USTRUCT(meta = (DisplayName = "Native Variables"))
struct FAnimInstanceProxy
{
	GENERATED_USTRUCT_BODY()

public:
	using FSyncGroupMap = TMap<FName, FAnimGroupInstance>;

	ENGINE_API FAnimInstanceProxy();
	ENGINE_API FAnimInstanceProxy(UAnimInstance* Instance);

	ENGINE_API FAnimInstanceProxy(const FAnimInstanceProxy&);
	ENGINE_API FAnimInstanceProxy& operator=(FAnimInstanceProxy&&);
	ENGINE_API FAnimInstanceProxy& operator=(const FAnimInstanceProxy&);
	ENGINE_API virtual ~FAnimInstanceProxy();

	// Get the IAnimClassInterface associated with this context, if there is one.
	// Note: This can return NULL, so check the result.
	IAnimClassInterface* GetAnimClassInterface() const
	{
		return AnimClassInterface;
	}

	/** Get the last DeltaSeconds passed into PreUpdate() */
	float GetDeltaSeconds() const
	{
		return CurrentDeltaSeconds;
	}

	/** Get the last time dilation, gleaned from world settings */
	float GetTimeDilation() const
	{
		return CurrentTimeDilation;
	}

#if WITH_EDITORONLY_DATA
	/** Whether the UAnimInstance this context refers to is currently being debugged in the editor */
	bool IsBeingDebugged() const
	{
		return bIsBeingDebugged;
	}

	/** Record a visited node in the debugger */
	ENGINE_API void RecordNodeVisit(int32 TargetNodeIndex, int32 SourceNodeIndex, float BlendWeight);

	/** Record a node attribute in the debugger */
	ENGINE_API void RecordNodeAttribute(const FAnimInstanceProxy& InSourceProxy, int32 InTargetNodeIndex, int32 InSourceNodeIndex, FName InAttribute);

	/** Record a node sync in the debugger */
	void RecordNodeSync(int32 InSourceNodeIndex, FName InSyncGroup)
	{
		NodeSyncsThisFrame.FindOrAdd(InSourceNodeIndex, InSyncGroup);
	}

	UAnimBlueprint* GetAnimBlueprint() const
	{
		UClass* ActualAnimClass = IAnimClassInterface::GetActualAnimClass(AnimClassInterface);
		return ActualAnimClass ? Cast<UAnimBlueprint>(ActualAnimClass->ClassGeneratedBy) : nullptr;
	}

	// Record pose for node of ID LinkID if it is currently being watched
	UE_DEPRECATED(5.6, "Please use RegisterWatchedPose that takes a pose context")
	ENGINE_API void RegisterWatchedPose(const FCompactPose& Pose, const FBlendedCurve& InCurve, int32 LinkID);
	UE_DEPRECATED(5.6, "Please use RegisterWatchedPose that takes a pose context")
	ENGINE_API void RegisterWatchedPose(const FCSPose<FCompactPose>& Pose, const FBlendedCurve& InCurve, int32 LinkID);

	// Record pose for node of ID LinkID if it is currently being watched
	ENGINE_API void RegisterWatchedPose(const FPoseContext& Pose, int32 LinkID);
	ENGINE_API void RegisterWatchedPose(const FComponentSpacePoseContext& Pose, int32 LinkID);
#endif

	UE_DEPRECATED(5.0, "Function renamed to FlipBufferWriteIndex as this no longer deals with sync groups.")
	void TickSyncGroupWriteIndex()
	{ 
		FlipBufferWriteIndex();
	}

	// flip buffer read/write indices
	void FlipBufferWriteIndex()
	{ 
		BufferWriteIndex = GetBufferReadIndex();
	}

	UE_DEPRECATED(5.0, "Sync groups are no longer stored in arrays, please use GetSyncGroupMapRead.")
	const TArray<FAnimGroupInstance>& GetSyncGroupRead() const
	{
		static const TArray<FAnimGroupInstance> Dummy;
		return Dummy; 
	}

	/** Get the sync group we are currently reading from */
	const FSyncGroupMap& GetSyncGroupMapRead() const
	{ 
		return Sync.GetSyncGroupMapRead(); 
	}

	/** Get the ungrouped active player we are currently reading from */
	const TArray<FAnimTickRecord>& GetUngroupedActivePlayersRead() 
	{ 
		return Sync.GetUngroupedActivePlayersRead(); 
	}

	UE_DEPRECATED(5.0, "Do not call directly. Instead sync and asset player ticking is controlled via UE::Anim::FSync and UE::Anim::FSyncScope.")
	ENGINE_API void TickAssetPlayerInstances(float DeltaSeconds);

	UE_DEPRECATED(5.0, "Do not call directly. Instead sync and asset player ticking is controlled via UE::Anim::FSync and UE::Anim::FSyncScope.")
	ENGINE_API void TickAssetPlayerInstances();

	/** Queues an Anim Notify from the shared list on our generated class */
	ENGINE_API void AddAnimNotifyFromGeneratedClass(int32 NotifyIndex);

	/** Trigger any anim notifies */
	ENGINE_API void TriggerAnimNotifies(USkeletalMeshComponent* SkelMeshComp, float DeltaSeconds);

	UE_DEPRECATED(5.2, "Skeleton compatibility is now an editor-only concern, this function is no longer used.")
	bool IsSkeletonCompatible(USkeleton const* InSkeleton) const
	{
		return Skeleton != nullptr && InSkeleton != nullptr;
	}

	/** Check whether we should extract root motion */
	bool ShouldExtractRootMotion() const 
	{ 
		return bShouldExtractRootMotion;
	}
	
	/** Save a pose snapshot to the internal snapshot cache */
	ENGINE_API void SavePoseSnapshot(USkeletalMeshComponent* InSkeletalMeshComponent, FName SnapshotName);

	/** Add an empty pose snapshot to the internal snapshot cache (or recycle an existing pose snapshot if the name is already in use) */
	ENGINE_API FPoseSnapshot& AddPoseSnapshot(FName SnapshotName);

	/** Remove a previously saved pose snapshot from the internal snapshot cache */
	ENGINE_API void RemovePoseSnapshot(FName SnapshotName);

	/** Get a cached pose snapshot by name */
	ENGINE_API const FPoseSnapshot* GetPoseSnapshot(FName SnapshotName) const;

	/** Access various counters */
	const FGraphTraversalCounter& GetInitializationCounter() const { return InitializationCounter; }
	const FGraphTraversalCounter& GetCachedBonesCounter() const  { return CachedBonesCounter; }
	const FGraphTraversalCounter& GetUpdateCounter() const  { return UpdateCounter; }
	const FGraphTraversalCounter& GetEvaluationCounter() const { return EvaluationCounter; }
	const FGraphTraversalCounter& GetSlotNodeInitializationCounter() const { return SlotNodeInitializationCounter; }
	
	void ResetUpdateCounter() { UpdateCounter.Reset(); }

	/** Access root motion params */
	FRootMotionMovementParams& GetExtractedRootMotion() { return ExtractedRootMotion; }

	/** Access UObject base of UAnimInstance */
	UObject* GetAnimInstanceObject() { return AnimInstanceObject; }
	const UObject* GetAnimInstanceObject() const { return AnimInstanceObject; }

	/** Gets an unchecked (can return nullptr) node given an index into the node property array */
	ENGINE_API FAnimNode_Base* GetMutableNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType);

	/** Gets an unchecked (can return nullptr) node given an index into the node property array */
	ENGINE_API const FAnimNode_Base* GetNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType) const;

	/** Gets a checked node given an index into the node property array */
	ENGINE_API FAnimNode_Base* GetCheckedMutableNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType);

	/** Gets a checked node given an index into the node property array */
	ENGINE_API const FAnimNode_Base* GetCheckedNodeFromIndexUntyped(int32 NodeIdx, UScriptStruct* RequiredStructType) const;

	/** Gets a checked node given an index into the node property array */
	template<class NodeType>
	const NodeType* GetCheckedNodeFromIndex(int32 NodeIdx) const
	{
		return (NodeType*)GetCheckedNodeFromIndexUntyped(NodeIdx, NodeType::StaticStruct());
	}

	/** Gets a checked node given an index into the node property array */
	template<class NodeType>
	NodeType* GetCheckedMutableNodeFromIndex(int32 NodeIdx) const
	{
		return (NodeType*)GetCheckedMutableNodeFromIndexUntyped(NodeIdx, NodeType::StaticStruct());
	}

	/** Gets an unchecked (can return nullptr) node given an index into the node property array */
	template<class NodeType>
	const NodeType* GetNodeFromIndex(int32 NodeIdx) const
	{
		return (NodeType*)GetNodeFromIndexUntyped(NodeIdx, NodeType::StaticStruct());
	}

	/** Gets an unchecked (can return nullptr) node given an index into the node property array */
	template<class NodeType>
	NodeType* GetMutableNodeFromIndex(int32 NodeIdx) const
	{
		return (NodeType*)GetNodeFromIndexUntyped(NodeIdx, NodeType::StaticStruct());
	}

	/** const access to required bones array */
	const FBoneContainer& GetRequiredBones() const 
	{ 
		return *RequiredBones; 
	}

	/** access to required bones array */
	FBoneContainer& GetRequiredBones() 
	{ 
		return *RequiredBones;
	}

	/** Access to LODLevel */
	int32 GetLODLevel() const
	{
		return LODLevel;
	}

	UE_DEPRECATED(5.0, "Please use GetComponentTransform")
	const FTransform& GetSkelMeshCompLocalToWorld() const
	{
		return ComponentTransform;
	}

	UE_DEPRECATED(5.0, "Please use GetActorTransform")
	const FTransform& GetSkelMeshCompOwnerTransform() const
	{
		return ActorTransform;
	}

	/** There are times when the skeleton is not set - for example if it has been deleted and we are in an editor window update. In these situations, use this to check before calling GetSkeleton() */
	bool IsSkeletonValid() const
	{
		return Skeleton != nullptr;
	}

	/** Get the current skeleton we are using. Note that this will return nullptr outside of pre/post update */
	USkeleton* GetSkeleton() 
	{ 
		// Skeleton is only available during update/eval. If you're calling this function outside of it, it will return null. 
		// adding ensure here so that we can catch them earlier
		ensureAlways(Skeleton);
		return Skeleton; 
	}

	/** Get the current skeletal mesh component we are running on. Note that this will return nullptr outside of pre/post update */
	USkeletalMeshComponent* GetSkelMeshComponent() const
	{ 
		// Skeletal mesh component is only available during update/eval. If you're calling this function outside of it, it will return null. 
		// adding ensure here so that we can catch them earlier
		ensureAlways(SkeletalMeshComponent);
		return SkeletalMeshComponent; 
	}

	/** Get the current main instance proxy. Note that this will return nullptr outside of pre/post update, and may return nullptr anyway if we dont have a main instance */
	FAnimInstanceProxy* GetMainInstanceProxy() const
	{ 
		// Main instance proxy is only available during update/eval. If you're calling this function outside of it, it will return null. 
		return MainInstanceProxy;
	}

	UE_DEPRECATED(4.26, "Please use the overload that takes a group FName")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecord(int32 GroupIndex, FAnimGroupInstance*& OutSyncGroupPtr);

	UE_DEPRECATED(4.26, "Please use the overload that takes a group FName")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecordInScope(int32 GroupIndex, EAnimSyncGroupScope Scope, FAnimGroupInstance*& OutSyncGroupPtr);

	UE_DEPRECATED(5.0, "Please use FAnimSyncGroupScope")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecord(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName);

	UE_DEPRECATED(5.0, "Please use FAnimSyncGroupScope")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecordInScope(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName, EAnimSyncGroupScope Scope);

	UE_DEPRECATED(5.0, "Please use the FAnimTickRecord constructor that takes a UAnimSequenceBase")
	ENGINE_API void MakeSequenceTickRecord(FAnimTickRecord& TickRecord, UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, float FinalBlendWeight, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord) const;

	UE_DEPRECATED(5.0, "Please use the FAnimTickRecord constructor that takes a UBlendSpace")
	ENGINE_API void MakeBlendSpaceTickRecord(FAnimTickRecord& TickRecord, UBlendSpace* BlendSpace, const FVector& BlendInput, TArray<FBlendSampleData>& BlendSampleDataCache, FBlendFilter& BlendFilter, bool bLooping, float PlayRate, float FinalBlendWeight, float& CurrentTime, FMarkerTickRecord& MarkerTickRecord) const;

	UE_DEPRECATED(5.0, "Please use the FAnimTickRecord constructor that takes a UPoseAsset")
	ENGINE_API void MakePoseAssetTickRecord(FAnimTickRecord& TickRecord, class UPoseAsset* PoseAsset, float FinalBlendWeight) const;

	// Adds a tick record in the list for the correct group or the ungrouped array.
	void AddTickRecord(const FAnimTickRecord& InTickRecord, const UE::Anim::FAnimSyncParams& InSyncParams = UE::Anim::FAnimSyncParams())
	{
		Sync.AddTickRecord(InTickRecord, InSyncParams);
	}

	/**
	 * Get Slot Node Weight : this returns new Slot Node Weight, Source Weight, Original TotalNodeWeight
	 *							this 3 values can't be derived from each other
	 *
	 * @param SlotNodeName : the name of the slot node you're querying
	 * @param out_SlotNodeWeight : The node weight for this slot node in the range of [0, 1]
	 * @param out_SourceWeight : The Source weight for this node. 
	 * @param out_TotalNodeWeight : Total weight of this node
	 */
	ENGINE_API void GetSlotWeight(const FName& SlotNodeName, float& out_SlotNodeWeight, float& out_SourceWeight, float& out_TotalNodeWeight) const;

	/** Evaluate a pose for a named montage slot */
	UE_DEPRECATED(4.26, "Use SlotEvaluatePose with other signature")
	ENGINE_API void SlotEvaluatePose(const FName& SlotNodeName, const FCompactPose& SourcePose, const FBlendedCurve& SourceCurve, float InSourceWeight, FCompactPose& BlendedPose, FBlendedCurve& BlendedCurve, float InBlendWeight, float InTotalNodeWeight);

	ENGINE_API void SlotEvaluatePose(const FName& SlotNodeName, const FAnimationPoseData& SourceAnimationPoseData, float InSourceWeight, FAnimationPoseData& OutBlendedAnimationPoseData, float InBlendWeight, float InTotalNodeWeight);
	
	// Allow slot nodes to store off their weight during ticking
	ENGINE_API void UpdateSlotNodeWeight(const FName& SlotNodeName, float InLocalMontageWeight, float InNodeGlobalWeight);

	UE_DEPRECATED(5.5, "Please use GetSlotInertializationRequestData instead.")
	ENGINE_API bool GetSlotInertializationRequest(const FName& SlotName, UE::Anim::FSlotInertializationRequest& OutRequest);

	ENGINE_API bool GetSlotInertializationRequestData(const FName& SlotName, FInertializationRequest& OutRequest);

	/** Register a named slot */
	ENGINE_API void RegisterSlotNodeWithAnimInstance(const FName& SlotNodeName);

	/** Check whether we have a valid root node */
	bool HasRootNode() const
	{ 
		return RootNode != nullptr; 
	}

	/** @todo: remove after deprecation */
	FAnimNode_Base* GetRootNode() 
	{ 
		return RootNode;
	}

	/** Gather debug data from this instance proxy and the blend tree for display */
	ENGINE_API void GatherDebugData(FNodeDebugData& DebugData);

	/** Gather debug data from this instance proxy and the specified blend tree root for display */
	ENGINE_API void GatherDebugData_WithRoot(FNodeDebugData& DebugData, FAnimNode_Base* InRootNode, FName InLayerName);

#if ENABLE_ANIM_DRAW_DEBUG
	mutable TArray<FQueuedDrawDebugItem> QueuedDrawDebugItems;

	ENGINE_API void AnimDrawDebugOnScreenMessage(const FString& DebugMessage, const FColor& Color, const FVector2D& TextScale = FVector2D::UnitVector, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugInWorldMessage(const FString& DebugMessage, const FVector& TextLocation, const FColor& Color, float TextScale);
	ENGINE_API void AnimDrawDebugLine(const FVector& StartLoc, const FVector& EndLoc, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugCoordinateSystem(FVector const& AxisLoc, FRotator const& AxisRot, float Scale = 1.f, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugPlane(const FTransform& BaseTransform, float Radii, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugPoint(const FVector& Loc, float Size, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	ENGINE_API void AnimDrawDebugCircle(const FVector& Center, float Radius, int32 Segments, const FColor& Color, const FVector& UpVector = FVector::UpVector, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f);
	ENGINE_API void AnimDrawDebugCone(const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, float AngleHeight, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f);
	ENGINE_API void AnimDrawDebugCapsule(const FVector& Center, float HalfHeight, float Radius, const FRotator& Rotation, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
#else
	void AnimDrawDebugOnScreenMessage(const FString& DebugMessage, const FColor& Color, const FVector2D& TextScale = FVector2D::UnitVector, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugInWorldMessage(const FString& DebugMessage, const FVector& TextLocation, const FColor& Color, float TextScale) {}
	void AnimDrawDebugLine(const FVector& StartLoc, const FVector& EndLoc, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugSphere(const FVector& Center, float Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugCoordinateSystem(FVector const& AxisLoc, FRotator const& AxisRot, float Scale = 1.f, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugPlane(const FTransform& BaseTransform, float Radii, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugPoint(const FVector& Loc, float Size, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
	void AnimDrawDebugCircle(const FVector& Center, float Radius, int32 Segments, const FColor& Color, const FVector& UpVector = FVector::UpVector, bool bPersistentLines = false, float LifeTime=-1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f) {}
	void AnimDrawDebugCone(const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, float AngleHeight, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.f) {}
	void AnimDrawDebugCapsule(const FVector& Center, float HalfHeight, float Radius, const FRotator& Rotation, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = SDPG_World) {}
#endif // ENABLE_ANIM_DRAW_DEBUG

#if ENABLE_ANIM_LOGGING
	const FString& GetActorName() const 
	{
		return ActorName; 
	}
#endif

	FString GetAnimInstanceName() const
	{ 
		return AnimInstanceObject->GetFullName();
	}

	// Get the root motion mode assigned to this anim instance proxy
	ERootMotionMode::Type GetRootMotionMode() const
	{
		return RootMotionMode;
	}

	/** Gets the runtime instance of the specified state machine by Name */
	ENGINE_API const FAnimNode_StateMachine* GetStateMachineInstanceFromName(FName MachineName) const;

	/** Gets the runtime instance of the specified state machine */
	ENGINE_API const FAnimNode_StateMachine* GetStateMachineInstance(int32 NodeIndex) const;

	/** Get the machine description for the specified instance. Does not rely on PRIVATE_MachineDescription being initialized */
	static ENGINE_API const FBakedAnimationStateMachine* GetMachineDescription(IAnimClassInterface* AnimBlueprintClass, const FAnimNode_StateMachine* MachineInstance);

	/** 
	 * Get the index of the specified instance asset player. Useful to pass to GetInstanceAssetPlayerLength (etc.).
	 * Passing NAME_None to InstanceName will return the first (assumed only) player instance index found.
	 */
	ENGINE_API int32 GetInstanceAssetPlayerIndex(FName MachineName, FName StateName, FName InstanceName = NAME_None) const;

	ENGINE_API float GetRecordedMachineWeight(const int32 InMachineClassIndex) const;
	ENGINE_API void RecordMachineWeight(const int32 InMachineClassIndex, const float InMachineWeight);

	ENGINE_API float GetRecordedStateWeight(const int32 InMachineClassIndex, const int32 InStateIndex) const;
	ENGINE_API void RecordStateWeight(const int32 InMachineClassIndex, const int32 InStateIndex, const float InStateWeight, const float InElapsedTime);

	ENGINE_API bool IsSlotNodeRelevantForNotifies(const FName& SlotNodeName) const;
	/** Reset any dynamics running simulation-style updates (e.g. on teleport, time skip etc.) */
	ENGINE_API void ResetDynamics(ETeleportType InTeleportType);

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<const FAnimNode_AssetPlayerBase*> GetInstanceAssetPlayers(const FName& GraphName) const;

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<FAnimNode_AssetPlayerBase*> GetMutableInstanceAssetPlayers(const FName& GraphName);

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerRelevancyBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<const FAnimNode_AssetPlayerRelevancyBase*> GetInstanceRelevantAssetPlayers(const FName& GraphName) const;

	/** Returns all Animation Nodes of FAnimNode_AssetPlayerRelevancyBase class within the specified (named) Animation Graph */
	ENGINE_API TArray<FAnimNode_AssetPlayerRelevancyBase*> GetMutableInstanceRelevantAssetPlayers(const FName& GraphName);

	/** Returns true if SyncGroupName is valid (exists, and if is based on markers, has valid markers) */
	ENGINE_API bool IsSyncGroupValid(FName InSyncGroupName) const;

	UE_DEPRECATED(4.20, "Please use ResetDynamics with a ETeleportType argument")
	ENGINE_API void ResetDynamics();

	/** Get the relative transform of the component we are running on */
	const FTransform& GetComponentRelativeTransform() const { return ComponentRelativeTransform; }

	/** Get the component to world transform of the component we are running on */
	const FTransform& GetComponentTransform() const { return ComponentTransform; }

	/** Get the transform of the actor we are running on */
	const FTransform& GetActorTransform() const { return ActorTransform; }

	// Trace montage debug data for the specified slot
	ENGINE_API void TraceMontageEvaluationData(const FAnimationUpdateContext& InContext, const FName& InSlotName);

	/** Get the debug data for this instance's anim bp */
	ENGINE_API FAnimBlueprintDebugData* GetAnimBlueprintDebugData() const;

	/**
	 * Add anim notifies to the proxies notify queue
	 * @param NewNotifies		The notifies to add
	 * @param InstanceWeight	The effective weight of the notifies (used for trigger filtering)
	 **/
	ENGINE_API void AddAnimNotifies(const TArray<FAnimNotifyEventReference>& NewNotifies, const float InstanceWeight);

	/** Set the optional animation interpolation type override. When not set, it will use the interpolation type set inside the animation asset. */
	ENGINE_API void SetInterpolationOverride(TOptional<EAnimInterpolationType> InterpolationType);

	/** Get the interpolation type override. If not set, it will use the iterpolation type set inside the animation asset. */
	ENGINE_API TOptional<EAnimInterpolationType> GetInterpolationOverride() const;

	/** Only restricted classes can access the protected interface */
	friend class UAnimInstance;
	friend class UAnimSingleNodeInstance;
	friend class USkeletalMeshComponent;
	friend struct FAnimNode_LinkedAnimGraph;
	friend struct FAnimNode_LinkedAnimLayer;
	friend struct FAnimationBaseContext;
	friend struct FAnimTrace;
	friend struct UE::Anim::FAnimSync;
	friend class UE::Anim::FAnimSyncGroupScope;
	friend class UE::Anim::FActiveStateMachineScope;
	friend struct FAnimNode_ControlRigInputPose;
	friend struct FSequencerMixedAnimInstanceProxy;
	
protected:
	/** Called when our anim instance is being initialized */
	ENGINE_API virtual void Initialize(UAnimInstance* InAnimInstance);

	/** Called when our anim instance is being uninitialized */
	ENGINE_API virtual void Uninitialize(UAnimInstance* InAnimInstance);

	/** Let us copy the full notify queue from the last frame */
	ENGINE_API void UpdateActiveAnimNotifiesSinceLastTick(const FAnimNotifyQueue& AnimInstanceQueue);
	
	/** Called before update so we can copy any data we need */
	ENGINE_API virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds);

	/** Called during PreUpdate, if SkelMesh LOD has changed since last update */
	ENGINE_API void OnPreUpdateLODChanged(const int32 PreviousLODIndex, const int32 NewLODIndex);

	/** Update override point */
	virtual void Update(float DeltaSeconds) {}

	/** Updates the anim graph */
	ENGINE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext);

	/** Updates the anim graph using a specified root node */
	ENGINE_API virtual void UpdateAnimationNode_WithRoot(const FAnimationUpdateContext& InContext, FAnimNode_Base* InRootNode, FName InLayerName);

	/** Called on the game thread pre-evaluate. */
	ENGINE_API virtual void PreEvaluateAnimation(UAnimInstance* InAnimInstance);

	/** Called when the anim instance is being initialized. If we are not using a blueprint instance, this root node can be provided*/
	virtual FAnimNode_Base* GetCustomRootNode()
	{
		return nullptr;
	}

	/** Called when the anim instance is being initialized. If we are not using a blueprint instance, these nodes can be provided */
	virtual void GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes)
	{
	}
	
	/** 
	 * Cache bones override point. You should call CacheBones on any nodes that need it here.
	 * bBoneCachesInvalidated is used to only perform this when needed (e.g. when a LOD changes), 
	 * as it is usually an expensive operation.
	 */
	ENGINE_API virtual void CacheBones();

	/** 
	 * Cache bones override point. You should call CacheBones on any nodes that need it here.
	 * bBoneCachesInvalidated is used to only perform this when needed (e.g. when a LOD changes), 
	 * as it is usually an expensive operation.
	 */
	ENGINE_API virtual void CacheBones_WithRoot(FAnimNode_Base* InRootNode);

	/** 
	 * Evaluate override point 
	 * @return true if this function is implemented, false otherwise.
	 * Note: the node graph will not be evaluated if this function returns true
	 */
	virtual bool Evaluate(FPoseContext& Output) { return false; }

	/** 
	 * Evaluate override point with root node override.
	 * @return true if this function is implemented, false otherwise.
	 * Note: the node graph will not be evaluated if this function returns true
	 */
	virtual bool Evaluate_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode) { return Evaluate(Output); }

	/** Called after update so we can copy any data we need */
	ENGINE_API virtual void PostUpdate(UAnimInstance* InAnimInstance) const;

	/** Called after evaluate so we can do any game thread work we need to */
	ENGINE_API virtual void PostEvaluate(UAnimInstance* InAnimInstance);

	/** Copy any UObjects we might be using. Called Pre-update and pre-evaluate. */
	ENGINE_API virtual void InitializeObjects(UAnimInstance* InAnimInstance);

	/** 
	 * Clear any UObjects we might be using. Called at the end of the post-evaluate phase.
	 * This is to ensure that objects are not used by anything apart from animation nodes.
	 * Please make sure to call the base implementation if this is overridden.
	 */
	ENGINE_API virtual void ClearObjects();

	/** Calls Update(), updates the anim graph, ticks asset players */
	ENGINE_API void UpdateAnimation();

	/** Calls Update(), updates the anim graph from the specified root, ticks asset players */
	ENGINE_API void UpdateAnimation_WithRoot(const FAnimationUpdateContext& InContext, FAnimNode_Base* InRootNode, FName InLayerName);

	/** Evaluates the anim graph if Evaluate() returns false */
	ENGINE_API void EvaluateAnimation(FPoseContext& Output);

	/** Evaluates the anim graph given the specified root if Evaluate() returns false */
	ENGINE_API void EvaluateAnimation_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode);

	/** Evaluates the anim graph */
	ENGINE_API void EvaluateAnimationNode(FPoseContext& Output);

	/** Evaluates the anim graph given the specified root */
	ENGINE_API void EvaluateAnimationNode_WithRoot(FPoseContext& Output, FAnimNode_Base* InRootNode);

	// @todo document
	ENGINE_API void SequenceAdvanceImmediate(UAnimSequenceBase* Sequence, bool bLooping, float PlayRate, float DeltaSeconds, /*inout*/ float& CurrentTime, FMarkerTickRecord& MarkerTickRecord);

	// @todo document
	ENGINE_API void BlendSpaceAdvanceImmediate(UBlendSpace* BlendSpace, const FVector& BlendInput, TArray<FBlendSampleData> & BlendSampleDataCache, FBlendFilter & BlendFilter, bool bLooping, float PlayRate, float DeltaSeconds, /*inout*/ float& CurrentTime, FMarkerTickRecord& MarkerTickRecord);

	UE_DEPRECATED(5.0, "Function renamed to GetBufferReadIndex as this no longer deals with sync groups.")
	int32 GetSyncGroupReadIndex() const 
	{ 
		return GetBufferReadIndex(); 
	}

	UE_DEPRECATED(5.0, "Function renamed to GetBufferWriteIndex as this no longer deals with sync groups.")
	int32 GetSyncGroupWriteIndex() const 
	{ 
		return GetBufferWriteIndex(); 
	}

	// Gets the buffer we should be reading from
	int32 GetBufferReadIndex() const 
	{ 
		return 1 - BufferWriteIndex; 
	}

	// Gets the sync group we should be writing to
	int32 GetBufferWriteIndex() const 
	{ 
		return BufferWriteIndex; 
	}

	/** Returns the baked sync group index from the compile step */
	ENGINE_API int32 GetSyncGroupIndexFromName(FName SyncGroupName) const;

	ENGINE_API bool GetTimeToClosestMarker(FName SyncGroup, FName MarkerName, float& OutMarkerTime) const;

	ENGINE_API bool HasMarkerBeenHitThisFrame(FName SyncGroup, FName MarkerName) const;

	ENGINE_API bool IsSyncGroupBetweenMarkers(FName InSyncGroupName, FName PreviousMarker, FName NextMarker, bool bRespectMarkerOrder = true) const;

	ENGINE_API FMarkerSyncAnimPosition GetSyncGroupPosition(FName InSyncGroupName) const;

	// Set the mirror data table to sync animations
	void SetSyncMirror(const UMirrorDataTable* MirrorDataTable)
	{
		Sync.SetMirror(MirrorDataTable);
	}
	
	// slot node run-time functions
	ENGINE_API void ReinitializeSlotNodes();

	// if it doesn't tick, it will keep old weight, so we'll have to clear it in the beginning of tick
	ENGINE_API void ClearSlotNodeWeights();

	/** Get global weight in AnimGraph for this slot node. 
	 * Note: this is the weight of the node, not the weight of any potential montage it is playing. */
	ENGINE_API float GetSlotNodeGlobalWeight(const FName& SlotNodeName) const;

	/** Get Global weight of any montages this slot node is playing. 
	 * If this slot is not currently playing a montage, it will return 0. */
	ENGINE_API float GetSlotMontageGlobalWeight(const FName& SlotNodeName) const;

	/** Get local weight of any montages this slot node is playing.
	* If this slot is not currently playing a montage, it will return 0. 
	* This is double buffered, will return last frame data if called from Update or Evaluate. */
	ENGINE_API float GetSlotMontageLocalWeight(const FName& SlotNodeName) const;

	/** Get local weight of any montages this slot is playing.
	* If this slot is not current playing a montage, it will return 0.
	* This will return up to date data if called during Update or Evaluate. */
	ENGINE_API float CalcSlotMontageLocalWeight(const FName& SlotNodeName) const;

	/** 
	 * Recalculate Required Bones [RequiredBones]
	 * Is called when bRequiredBonesUpToDate = false
	 * The provided asset must be a SkeletalMesh or a Skeleton.
	 */
	ENGINE_API void RecalcRequiredBones(USkeletalMeshComponent* Component, UObject* Asset);

	/** 
	 * Recalculate required curve list for animation - if you call RecalcRequiredBones, this should already happen
	 */
	ENGINE_API void RecalcRequiredCurves(const UE::Anim::FCurveFilterSettings& CurveFilterSettings);

	UE_DEPRECATED(5.3, "Please use RecalcRequiredCurves that takes a FCurveEvaluationOption.")
	ENGINE_API void RecalcRequiredCurves(const FCurveEvaluationOption& CurveEvalOption);
	
	UE_DEPRECATED(5.3, "This function is no longer used.")
	void UpdateCurvesToComponents(USkeletalMeshComponent* Component) {}

	/** Get Currently active montage evaluation state.
		Note that there might be multiple Active at the same time. This will only return the first active one it finds. **/
	ENGINE_API const FMontageEvaluationState* GetActiveMontageEvaluationState() const;

	ENGINE_API TMap<FName, FInertializationRequest>& GetSlotGroupInertializationRequestDataMap();

	/** Access montage array data */
	ENGINE_API TArray<FMontageEvaluationState>& GetMontageEvaluationData();

	/** Access montage array data */
	ENGINE_API const TArray<FMontageEvaluationState>& GetMontageEvaluationData() const;

	/** Check whether we have active morph target curves */

	/** Gets the most relevant asset player in a specified state */
	UE_DEPRECATED(5.1, "Please use GetRelevantAssetPlayerInterfaceFromState")
	const FAnimNode_AssetPlayerBase* GetRelevantAssetPlayerFromState(int32 MachineIndex, int32 StateIndex) const
	{
		return nullptr;
	}

	/** Gets the most relevant asset player in a specified state */
	ENGINE_API const FAnimNode_AssetPlayerRelevancyBase* GetRelevantAssetPlayerInterfaceFromState(int32 MachineIndex, int32 StateIndex) const;

	/** Gets an unchecked (can return nullptr) node given a property of the anim instance */
	template<class NodeType>
	NodeType* GetNodeFromProperty(FProperty* Property)
	{
		return (NodeType*)Property->ContainerPtrToValuePtr<NodeType>(AnimInstanceObject);
	}

	/** Gets the length in seconds of the asset referenced in an asset player node */
	ENGINE_API float GetInstanceAssetPlayerLength(int32 AssetPlayerIndex) const;

	/** Get the current accumulated time in seconds for an asset player node */
	ENGINE_API float GetInstanceAssetPlayerTime(int32 AssetPlayerIndex) const;

	/** Get the current accumulated time as a fraction for an asset player node */
	ENGINE_API float GetInstanceAssetPlayerTimeFraction(int32 AssetPlayerIndex) const;

	/** Get the time in seconds from the end of an animation in an asset player node */
	ENGINE_API float GetInstanceAssetPlayerTimeFromEnd(int32 AssetPlayerIndex) const;

	/** Get the time as a fraction of the asset length of an animation in an asset player node */
	ENGINE_API float GetInstanceAssetPlayerTimeFromEndFraction(int32 AssetPlayerIndex) const;

	/** Get the blend weight of a specified state */
	ENGINE_API float GetInstanceMachineWeight(int32 MachineIndex) const;

	/** Get the blend weight of a specified state */
	ENGINE_API float GetInstanceStateWeight(int32 MachineIndex, int32 StateIndex) const;

	/** Get the current elapsed time of a state within the specified state machine */
	ENGINE_API float GetInstanceCurrentStateElapsedTime(int32 MachineIndex) const;

	/** Get the crossfade duration of a specified transition */
	ENGINE_API float GetInstanceTransitionCrossfadeDuration(int32 MachineIndex, int32 TransitionIndex) const;

	/** Get the elapsed time in seconds of a specified transition */
	ENGINE_API float GetInstanceTransitionTimeElapsed(int32 MachineIndex, int32 TransitionIndex) const;

	/** Get the elapsed time as a fraction of the crossfade duration of a specified transition */
	ENGINE_API float GetInstanceTransitionTimeElapsedFraction(int32 MachineIndex, int32 TransitionIndex) const;

	/** Get the time remaining in seconds for the most relevant animation in the source state */
	ENGINE_API float GetRelevantAnimTimeRemaining(int32 MachineIndex, int32 StateIndex) const;

	/** Get the time remaining as a fraction of the duration for the most relevant animation in the source state */
	ENGINE_API float GetRelevantAnimTimeRemainingFraction(int32 MachineIndex, int32 StateIndex) const;

	/** Get the length in seconds of the most relevant animation in the source state */
	ENGINE_API float GetRelevantAnimLength(int32 MachineIndex, int32 StateIndex) const;

	/** Get the current accumulated time in seconds for the most relevant animation in the source state */
	ENGINE_API float GetRelevantAnimTime(int32 MachineIndex, int32 StateIndex) const;

	/** Get the current accumulated time as a fraction of the length of the most relevant animation in the source state */
	ENGINE_API float GetRelevantAnimTimeFraction(int32 MachineIndex, int32 StateIndex) const;

	/** Get whether a particular notify state was active in any state machine last tick. */
	ENGINE_API bool WasAnimNotifyStateActiveInAnyState(TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const;

	/** Get whether a particular notify state is active in a specific state machine last tick. */
	ENGINE_API bool WasAnimNotifyStateActiveInStateMachine(int32 MachineIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const;

	/** Get whether the most relevant animation was in a particular notify state last tick. */
	ENGINE_API bool WasAnimNotifyStateActiveInSourceState(int32 MachineIndex, int32 StateIndex, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const;
	
	/** Get whether the most relevant animation triggered the given notify last tick. */
	ENGINE_API bool WasAnimNotifyTriggeredInSourceState(int32 MachineIndex, int32 StateIndex, TSubclassOf<UAnimNotify> AnimNotifyType) const;

	/** Get whether the most relevant animation triggered the given notify last tick. */
	ENGINE_API bool WasAnimNotifyNameTriggeredInSourceState(int32 MachineIndex, int32 StateIndex, FName NotifyName) const;

	/** Get whether a particular notify type was active in a specific state machine last tick.  */
	ENGINE_API bool WasAnimNotifyTriggeredInStateMachine(int32 MachineIndex, TSubclassOf<UAnimNotify> AnimNotifyType) const;

	/**  Get whether an animation notify of a given type was triggered last tick. */
    ENGINE_API bool WasAnimNotifyTriggeredInAnyState(TSubclassOf<UAnimNotify> AnimNotifyType) const;

	/** Get whether the animation notify with the specified name triggered last tick. */
    ENGINE_API bool WasAnimNotifyNameTriggeredInAnyState(FName NotifyName) const;
	
	/** Get whether the given state machine triggered the animation notify with the specified name last tick. */
    ENGINE_API bool WasAnimNotifyNameTriggeredInStateMachine(int32 MachineIndex, FName NotifyName);

	/** Attempts to queue a transition request, returns true if successful */
	ENGINE_API bool RequestTransitionEvent(const FName& EventName, const double RequestTimeout, const ETransitionRequestQueueMode& QueueMode, const ETransitionRequestOverwriteMode& OverwriteMode);
	
	/** Removes all queued transition requests with the given event name */
	ENGINE_API void ClearTransitionEvents(const FName& EventName);

	/** Removes all queued transition requests */
	ENGINE_API void ClearAllTransitionEvents();

	/** Returns whether or not the given event transition request has been queued */
	ENGINE_API bool QueryTransitionEvent(int32 NodeIndex, int32 TransitionIndex, const FName& EventName) const;

	/** Behaves like QueryTransitionEvent but additionally marks the event for consumption */
	ENGINE_API bool QueryAndMarkTransitionEvent(int32 NodeIndex, int32 TransitionIndex, const FName& EventName);

	// Sets up a native transition delegate between states with PrevStateName and NextStateName, in the state machine with name MachineName.
	// Note that a transition already has to exist for this to succeed
	ENGINE_API void AddNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, const FCanTakeTransition& NativeTransitionDelegate, const FName& TransitionName = NAME_None);

	// Check for whether a native rule is bound to the specified transition
	ENGINE_API bool HasNativeTransitionBinding(const FName& MachineName, const FName& PrevStateName, const FName& NextStateName, FName& OutBindingName);

	// Sets up a native state entry delegate from state with StateName, in the state machine with name MachineName.
	ENGINE_API void AddNativeStateEntryBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeEnteredDelegate, const FName& BindingName = NAME_None);
	
	// Check for whether a native entry delegate is bound to the specified state
	ENGINE_API bool HasNativeStateEntryBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName);

	// Sets up a native state exit delegate from state with StateName, in the state machine with name MachineName.
	ENGINE_API void AddNativeStateExitBinding(const FName& MachineName, const FName& StateName, const FOnGraphStateChanged& NativeExitedDelegate, const FName& BindingName = NAME_None);

	// Check for whether a native exit delegate is bound to the specified state
	ENGINE_API bool HasNativeStateExitBinding(const FName& MachineName, const FName& StateName, FName& OutBindingName);

	/** Bind any native delegates that we have set up */
	ENGINE_API void BindNativeDelegates();

	/** Gets the runtime instance desc of the state machine specified by name */
	ENGINE_API const FBakedAnimationStateMachine* GetStateMachineInstanceDesc(FName MachineName) const;

	/** Gets the node index of the state machine matching MachineName */
	ENGINE_API int32 GetStateMachineIndex(FName MachineName) const;

	/** Gets the node index of the state machine */
	ENGINE_API int32 GetStateMachineIndex(FAnimNode_StateMachine* StateMachine) const;

	ENGINE_API void GetStateMachineIndexAndDescription(FName InMachineName, int32& OutMachineIndex, const FBakedAnimationStateMachine** OutMachineDescription) const;

	/** Initialize the root node - split into a separate function for backwards compatibility (initialization order) reasons */
	ENGINE_API void InitializeRootNode(bool bInDeferRootNodeInitialization = false);

	/** Initialize the specified root node */
	ENGINE_API void InitializeRootNode_WithRoot(FAnimNode_Base* InRootNode);

	/** Manually add object references to GC */
	ENGINE_API virtual void AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector);

	/** Allow nodes to register log messages to be processed on the game thread */
	ENGINE_API void LogMessage(FName InLogType, EMessageSeverity::Type InSeverity, const FText& InMessage) const;
	ENGINE_API void LogMessage(FName InLogType, const TSharedRef<FTokenizedMessage>& InMessage) const;

	/** Get the current value of all animation curves **/
	TMap<FName, float>& GetAnimationCurves(EAnimCurveType InCurveType) { return AnimationCurves[(uint8)InCurveType]; }
	const TMap<FName, float>& GetAnimationCurves(EAnimCurveType InCurveType) const { return AnimationCurves[(uint8)InCurveType]; }

	/** Reset Animation Curves */
	ENGINE_API void ResetAnimationCurves();

	/** Pushes blended heap curve to output curves in the proxy using required bones cached data */
	ENGINE_API void UpdateCurvesToEvaluationContext(const FAnimationEvaluationContext& InContext);

	/** Update curves once evaluation has taken place. Mostly pushes curves to materials/morphs */
	ENGINE_API virtual void UpdateCurvesPostEvaluation(USkeletalMeshComponent* SkelMeshComp);

	/** Check whether we have any active curves */
	ENGINE_API bool HasActiveCurves() const;

	
	/** Add a curve value */
	ENGINE_API void AddCurveValue(const FName& CurveName, float Value, bool bMorphtarget = false, bool bMaterial = false);
	
	/** Custom proxy Init/Cache/Update/Evaluate functions */
	static ENGINE_API void InitializeInputProxy(FAnimInstanceProxy* InputProxy, UAnimInstance* InAnimInstance);
	static ENGINE_API void GatherInputProxyDebugData(FAnimInstanceProxy* InputProxy, FNodeDebugData& DebugData);
	static ENGINE_API void CacheBonesInputProxy(FAnimInstanceProxy* InputProxy);
	static ENGINE_API void UpdateInputProxy(FAnimInstanceProxy* InputProxy, const FAnimationUpdateContext& Context);
	static ENGINE_API void EvaluateInputProxy(FAnimInstanceProxy* InputProxy, FPoseContext& Output);
	static ENGINE_API void ResetCounterInputProxy(FAnimInstanceProxy* InputProxy);

	/** Reset the sync system to its starting state */
	void ResetSync()
	{
		Sync.ResetAll();
	}

#if ENABLE_ANIM_DRAW_DEBUG
	/** Send any queued DrawDebug commands and reset the queue */
	void FlushQueuedDebugDrawItems(AActor* InActor, UWorld* InWorld) const;
#endif

private:

	FName GetTargetLogNameForCurrentWorldType() const;

	/** Executes the provided functor on each valid state machine on this anim instance */
	ENGINE_API void ForEachStateMachine(const TFunctionRef<void(FAnimNode_StateMachine&)>& Functor);

	/** Evaluate the slot when there are blend spaces involved in any of the active anim montages. */
	ENGINE_API void SlotEvaluatePoseWithBlendProfiles(const FName& SlotNodeName, const FAnimationPoseData& SourceAnimationPoseData, float InSourceWeight, FAnimationPoseData& OutBlendedAnimationPoseData, float InBlendWeight);

	/** Initialize cached class data - state machines, pre-update nodes etc. */
	ENGINE_API void InitializeCachedClassData();
	
	/** The component to world transform of the component we are running on */
	FTransform ComponentTransform;

	/** The relative transform of the component we are running on */
	FTransform ComponentRelativeTransform;

	/** The transform of the actor we are running on */
	FTransform ActorTransform;

	/** Object ptr to our UAnimInstance */
	mutable UObject* AnimInstanceObject;

	/** Our anim blueprint generated class */
	IAnimClassInterface* AnimClassInterface;

	/** Skeleton we are using, only used for comparison purposes. Note that this will be nullptr outside of pre/post update */
	USkeleton* Skeleton;

	/** Skeletal mesh component we are attached to. Note that this will be nullptr outside of pre/post update */
	USkeletalMeshComponent* SkeletalMeshComponent;

	/** Cached ptr to the main instance proxy, which may be "this" */
	FAnimInstanceProxy* MainInstanceProxy;

	/** The last time passed into PreUpdate() */
	float CurrentDeltaSeconds;

	/** The last dime dilation (gleaned from world settings) */
	float CurrentTimeDilation;

#if WITH_EDITORONLY_DATA
	/** Array of visited nodes this frame */
	TArray<FAnimBlueprintDebugData_NodeVisit> UpdatedNodesThisFrame;

	/** Map of node attributes this frame */
	TMap<int32, TArray<FAnimBlueprintDebugData_AttributeRecord>> NodeInputAttributesThisFrame;
	TMap<int32, TArray<FAnimBlueprintDebugData_AttributeRecord>> NodeOutputAttributesThisFrame;

	/** Map of node syncs this frame - maps from player node index to graph-determined group name */
	TMap<int32, FName> NodeSyncsThisFrame;
#endif

#if ENABLE_ANIM_LOGGING
	/** Actor name for debug logging purposes */
	FString ActorName;
#endif

	/** Anim graph */
	FAnimNode_Base* RootNode;

	/** Default linked instance input node if available */
	FAnimNode_LinkedInputPose* DefaultLinkedInstanceInputNode;

	/** Map of layer name to saved pose nodes to process after the graph has been updated */
	TMap<FName, TArray<FAnimNode_SaveCachedPose*>> SavedPoseQueueMap;

	/** Synchronizes animations according to sync groups/markers */
	UE::Anim::FAnimSync Sync;

	/** Buffers containing read/write buffers for all current machine weights */
	TArray<float> MachineWeightArrays[2];

	/** Buffers containing read/write buffers for all current state weights */
	TArray<float> StateWeightArrays[2];

	/** Map that transforms state class indices to base offsets into the weight array */
	TMap<int32, int32> StateMachineClassIndexToWeightOffset;

	// Current buffer index
	int32 BufferWriteIndex;

	/** Animation Notifies that has been triggered in the latest tick **/
	FAnimNotifyQueue NotifyQueue;

	// Root motion mode duplicated from the anim instance
	ERootMotionMode::Type RootMotionMode;

	// Read/write buffers Tracker map for slot name->weights/relevancy
	TMap<FName, int32> SlotNameToTrackerIndex;
	TArray<FMontageActiveSlotTracker> SlotWeightTracker[2];

	/** Curves in an easily looked-up form **/
	TMap<FName, float> AnimationCurves[(uint8)EAnimCurveType::MaxAnimCurveType];

	/** Material parameters that we had been changing and now need to clear */
	TSet<FName> MaterialParametersToClear;

protected:
	// Animation Notifies that has been triggered since the last tick. These can be safely consumed at any point.
	TArray<FAnimNotifyEventReference> ActiveAnimNotifiesSinceLastTick;
	
	// Counters for synchronization
	FGraphTraversalCounter InitializationCounter;
	FGraphTraversalCounter CachedBonesCounter;
	FGraphTraversalCounter UpdateCounter;
	FGraphTraversalCounter EvaluationCounter;
	FGraphTraversalCounter SlotNodeInitializationCounter;

	// Sync counter
	uint64 FrameCounterForUpdate;
	uint64 FrameCounterForNodeUpdate;

	/** Optional override of the animation interpolation type. If not set, it is ignored. */
	TOptional<EAnimInterpolationType> InterpolationOverride;

private:
	// Root motion extracted from animation since the last time ConsumeExtractedRootMotion was called
	FRootMotionMovementParams ExtractedRootMotion;

	/**
	  * Temporary array of bone indices required this frame. Should be subset of Skeleton and Mesh's RequiredBones.
	  * Shared between all anim instances on a skeletal mesh component if it has a skeletal mesh and a skeleton present.
	  */
	TSharedPtr<FBoneContainer> RequiredBones;

	/** LODLevel used by RequiredBones */
	int32 LODLevel = 0;

	/** Counter used to control CacheBones recursion behavior - makes sure we cache bones correctly when recursing into different subgraphs */
	int32 CacheBonesRecursionCounter;

	/** During animation update and eval, records the number of frames we will skip due to URO */
	int16 NumUroSkippedFrames_Update;
	int16 NumUroSkippedFrames_Eval;

private:
	/** Copy of UAnimInstance::MontageInstances data used for update & evaluation */
	TArray<FMontageEvaluationState> MontageEvaluationData;

	// Inertialization request for each slot.
	UPROPERTY(Transient)
	TMap<FName, FInertializationRequest> SlotGroupInertializationRequestDataMap;

	/** Delegate fired on the game thread before update occurs */
	TArray<FAnimNode_Base*> GameThreadPreUpdateNodes;

	/** When GameThreadPreUpdateNodes are disabled due to LOD, they are stored here. To be potentially restored later. */
	TArray<FAnimNode_Base*> LODDisabledGameThreadPreUpdateNodes;

	/** All nodes that need to be reset on DynamicReset() */
	TArray<FAnimNode_Base*> DynamicResetNodes;
	
	/** Native transition rules */
	TArray<FNativeTransitionBinding> NativeTransitionBindings;

	/** Native state entry bindings */
	TArray<FNativeStateBinding> NativeStateEntryBindings;

	/** Native state exit bindings */
	TArray<FNativeStateBinding> NativeStateExitBindings;

	/** Array of snapshots. Each entry contains a name for finding specific pose snapshots */
	TArray<FPoseSnapshot> PoseSnapshots;

#if ENABLE_ANIM_LOGGING
	/** Logged message queues. Allows nodes to report messages to MessageLog even though they may be running
	 *  on a worked thread
	 */
	typedef TSharedRef<FTokenizedMessage> FLogMessageEntry;
	mutable TMap<FName, TArray<FLogMessageEntry>> LoggedMessagesMap;

	/** Cache of guids generated from previously sent messages so we can stop spam*/
	mutable TArray<FGuid> PreviouslyLoggedMessages;
#endif

	/** Scope guard to prevent duplicate work on re-entracy */
	bool bUpdatingRoot;

protected:
	/** When RequiredBones mapping has changed, AnimNodes need to update their bones caches. */
	uint8 bBoneCachesInvalidated : 1;

private:
	// Diplicate of bool result of ShouldExtractRootMotion()
	uint8 bShouldExtractRootMotion : 1;

	/** We can defer initialization until first update */
	uint8 bDeferRootNodeInitialization : 1;

#if WITH_EDITORONLY_DATA
	/** Whether this UAnimInstance is currently being debugged in the editor */
	uint8 bIsBeingDebugged : 1;

	/** Whether the world of this proxy's skeletal mesh component is a game world. */
	uint8 bIsGameWorld : 1;
#endif

	// Whether subsystems should be initialized
	uint8 bInitializeSubsystems : 1;

	uint8 bUseMainInstanceMontageEvaluationData : 1;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Animation/AnimTrace.h"
#endif
