// Copyright Epic Games, Inc. All Rights Reserved.

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

// HEADER_UNIT_SKIP - Included through other header

DO_ANIMSTAT_PROCESSING(PerformAnimEvaluation);
DO_ANIMSTAT_PROCESSING(SkeletalComponentAnimEvaluate);
DO_ANIMSTAT_PROCESSING(EvaluateAnimGraph);
DO_ANIMSTAT_PROCESSING(EvaluateAnimInstance);
DO_ANIMSTAT_PROCESSING(DecompressAnimationPose);
DO_ANIMSTAT_PROCESSING(BlendPosesInGraph);
DO_ANIMSTAT_PROCESSING(EvaluateAnimStateMachine);
DO_ANIMSTAT_PROCESSING(FillComponentSpaceTransforms);
DO_ANIMSTAT_PROCESSING(ProxyUpdateAnimation);
DO_ANIMSTAT_PROCESSING(PoseBlendNodeEvaluate);
DO_ANIMSTAT_PROCESSING(PoseAssetGetAnimationPose);

// Verbose nodes
#if ANIMNODE_STATS_VERBOSE
DO_ANIMSTAT_PROCESSING(BlendSpacePlayer);
DO_ANIMSTAT_PROCESSING(Inertialization);
DO_ANIMSTAT_PROCESSING(ControlRig);
DO_ANIMSTAT_PROCESSING(IKRig);
DO_ANIMSTAT_PROCESSING(PreviewRetargetPose);
DO_ANIMSTAT_PROCESSING(RemapCurves);
DO_ANIMSTAT_PROCESSING(RemapCurvesFromMesh);
DO_ANIMSTAT_PROCESSING(CorrectPose);
DO_ANIMSTAT_PROCESSING(MotionMatching);
DO_ANIMSTAT_PROCESSING(PoseSearchHistoryCollector);
DO_ANIMSTAT_PROCESSING(AimOffsetLookAt);
DO_ANIMSTAT_PROCESSING(BlendBoneByChannel);
DO_ANIMSTAT_PROCESSING(CopyPoseFromMesh);
DO_ANIMSTAT_PROCESSING(CurveSource);
DO_ANIMSTAT_PROCESSING(MakeDynamicAdditive);
DO_ANIMSTAT_PROCESSING(ModifyCurve);
DO_ANIMSTAT_PROCESSING(MultiWayBlend);
DO_ANIMSTAT_PROCESSING(RandomPlayer);
DO_ANIMSTAT_PROCESSING(RotateRootBone);
DO_ANIMSTAT_PROCESSING(Slot);
DO_ANIMSTAT_PROCESSING(TwoWayBlend);
DO_ANIMSTAT_PROCESSING(ScaleChainLength);
DO_ANIMSTAT_PROCESSING(ApplyAdditive);
DO_ANIMSTAT_PROCESSING(ApplyMeshSpaceAdditive);
DO_ANIMSTAT_PROCESSING(ConvertComponentToLocalSpace);
DO_ANIMSTAT_PROCESSING(SaveCachedPose);
DO_ANIMSTAT_PROCESSING(PoseSearchComponentSpaceHistoryCollector);
DO_ANIMSTAT_PROCESSING(MeshSpaceRefPose);
DO_ANIMSTAT_PROCESSING(SkeletalControlBase);
DO_ANIMSTAT_PROCESSING(ConvertLocalToComponentSpace);
DO_ANIMSTAT_PROCESSING(HandIKRetargeting);
DO_ANIMSTAT_PROCESSING(StageCoachWheelController);
DO_ANIMSTAT_PROCESSING(WheelController);
DO_ANIMSTAT_PROCESSING(ApplyLimits);
DO_ANIMSTAT_PROCESSING(BoneDrivenController);
DO_ANIMSTAT_PROCESSING(CCDIK);
DO_ANIMSTAT_PROCESSING(Constraint);
DO_ANIMSTAT_PROCESSING(CopyBone);
DO_ANIMSTAT_PROCESSING(CopyBoneDelta);
DO_ANIMSTAT_PROCESSING(Fabrik);
DO_ANIMSTAT_PROCESSING(LookAt);
DO_ANIMSTAT_PROCESSING(ModifyBone);
DO_ANIMSTAT_PROCESSING(ObserveBone);
DO_ANIMSTAT_PROCESSING(ResetRoot);
DO_ANIMSTAT_PROCESSING(SplineIK);
DO_ANIMSTAT_PROCESSING(SpringBone);
DO_ANIMSTAT_PROCESSING(RotationMultiplier);
DO_ANIMSTAT_PROCESSING(RemoveCurve);
DO_ANIMSTAT_PROCESSING(ModularVehicleController);
#endif
