// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimNodeConstantData.generated.h"

class IAnimClassInterface;

// Any constant class data an anim node uses should be derived from this type.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNodeConstantData
{
	GENERATED_BODY()

	/** The class we are part of */
	const IAnimClassInterface& GetAnimClassInterface() const { check(AnimClassInterface); return *AnimClassInterface; }

	/** Get the node index for this constant data block. */
	int32 GetNodeIndex() const { return NodeIndex; }

private:
	friend class FAnimBlueprintCompilerContext;

	/** The class we are part of */
	UPROPERTY()
	TScriptInterface<IAnimClassInterface> AnimClassInterface = nullptr;

	/** 
	 * The index of the node for this constant data block in the class that it is held in. 
	 * INDEX_NONE if this node is not in a generated class or is per-instance data. 
	 */
	UPROPERTY()
	int32 NodeIndex = INDEX_NONE;
};
