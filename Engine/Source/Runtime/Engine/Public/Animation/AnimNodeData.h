// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimNodeData.generated.h"

class IAnimClassInterface;
struct FAnimNode_Base;
struct FAnimNodeData;

// Flag used to specify whether an anim node data entry is held on an instance or the class
#define ANIM_NODE_DATA_INVALID_ENTRY 0xffffffff
#define ANIM_NODE_DATA_INSTANCE_DATA_FLAG 0x80000000
#define ANIM_NODE_DATA_INSTANCE_DATA_MASK ~ANIM_NODE_DATA_INSTANCE_DATA_FLAG

namespace UE { namespace Anim {

// Identifier used to access folded node data.
struct FNodeDataId
{
	friend struct ::FAnimNodeData;
	friend struct ::FAnimNode_Base;

	// Check if this ID is valid
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

#if WITH_EDITORONLY_DATA	
	// Get the node struct associated with this ID
	const UScriptStruct* GetStruct() const
	{
		return Struct;
	}
	
	// Get the node property associated with this ID
	const FProperty* GetProperty() const
	{
		return Property;
	}
#endif

	FNodeDataId() = default;
	
	// Construct from a property name and struct
	ENGINE_API FNodeDataId(FName InPropertyName, const FAnimNode_Base* InNode, const UScriptStruct* InNodeStruct);
	
private:
	// Name-based ID
	FName Id = NAME_None;

	// Index into the folded data table
	int32 Index = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	// Struct of the node
	const UScriptStruct* Struct = nullptr;

	// Property of the data on the node
	const FProperty* Property = nullptr;
#endif
};

} }	// namespace UE::Anim

// The flags field of FAnimNodeData
// Primarily this is used to prevent the extra work asscicated with recovering folded properties for anim node functions
UENUM()
enum class EAnimNodeDataFlags : uint32
{
	None						= 0x00000000,
	
	// This node binds its initial update function
	HasInitialUpdateFunction	= 0x00000001,

	// This node binds its become relevant function
	HasBecomeRelevantFunction	= 0x00000002,

	// This node binds its update function
	HasUpdateFunction			= 0x00000004,

	AllFunctions = HasInitialUpdateFunction | HasBecomeRelevantFunction | HasUpdateFunction,
};

ENUM_CLASS_FLAGS(EAnimNodeDataFlags);

// Any constant/folded class data an anim node can be accessed via this struct
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNodeData
{
	GENERATED_BODY()

public:
	/**
	 * Get the specified data for the specified instance. The data may or may not reside on an anim instance itself as it may have been folded into constants.
	 * @param	InId				The data ID to use
	 * @param	InNode				The anim node that this data is for
	 * @param	InCurrentObject		The object that the node is held on. This is optional as it can be recovered via the node where needed, but can accelerate the operation if supplied.
	 * @return the raw data value, either held on sparse class data or on the instance 
	 */
	ENGINE_API const void* GetData(UE::Anim::FNodeDataId InId, const FAnimNode_Base* InNode, const UObject* InCurrentObject = nullptr) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Get the specified mutable data for the specified instance. The data may or may not reside on an anim instance itself as it may have been folded into constants.
	 * Only available in editor to support patching constant data during compilation. Not for use at runtime. 
	 */
	ENGINE_API void* GetMutableData(UE::Anim::FNodeDataId InId, FAnimNode_Base* InNode, UObject* InCurrentObject = nullptr) const;
#endif

	/** Get the specified mutable data for the specified instance. If the data is not held on the instance this will return nullptr. */
	ENGINE_API void* GetInstanceData(UE::Anim::FNodeDataId InId, FAnimNode_Base* InNode, UObject* InCurrentObject = nullptr) const;
	
	/** The class we are part of */
	const IAnimClassInterface& GetAnimClassInterface() const { check(AnimClassInterface); return *AnimClassInterface; }

	/** Get the node index for this constant data block. */
	int32 GetNodeIndex() const { return NodeIndex; }

	/** Check whether the node has the specified flags */
	bool HasNodeAnyFlags(EAnimNodeDataFlags InFlags) const { return EnumHasAnyFlags(static_cast<EAnimNodeDataFlags>(Flags), InFlags); }
	
private:
	friend class FAnimBlueprintCompilerContext;
	friend struct UE::Anim::FNodeDataId;

	/** Set whether the node has the specified flags */
	void SetNodeFlags(EAnimNodeDataFlags InFlags) { return EnumAddFlags(*reinterpret_cast<EAnimNodeDataFlags*>(&Flags), InFlags); }
	
	/** The class we are part of */
	UPROPERTY()
	TScriptInterface<IAnimClassInterface> AnimClassInterface = nullptr;

	/**
	 * Flags & Indices for table entries. Used to look up indices from NodePropertyIndex->FoldedDataIndex.
	 * If the MSB is set for an entry it is assumed to be on an instance, if not, it is assumed to be stored on the class.
	 */
	UPROPERTY()
	TArray<uint32> Entries;
	
	/** 
	 * The index of the node for this constant data block in the class that it is held in. 
	 * INDEX_NONE if this node is not in a generated class or is per-instance data. 
	 */
	UPROPERTY()
	int32 NodeIndex = INDEX_NONE;

	/** Common flags for this node */
	UPROPERTY()
	uint32 Flags = static_cast<uint32>(EAnimNodeDataFlags::None);
};

/**
 * Data about editor-side struct property indices.
 * Used to map property name to editor-only property indices. This allows us to avoid TMap lookups at runtime.   
 */
USTRUCT()
struct FAnimNodeStructData
{
	GENERATED_BODY()
	
	FAnimNodeStructData() = default;

	ENGINE_API FAnimNodeStructData(const UScriptStruct* InNodeType);

	ENGINE_API int32 GetPropertyIndex(FName InPropertyName) const;

	ENGINE_API int32 GetNumProperties() const;

#if WITH_EDITORONLY_DATA
	// Verifies the layout of another struct data against this one
	// Note: uses name + index, so not robust to type/size changes. This is OK however as tagged property serializaton
	// will deal with those and names/indices are all we need to be consistent at this level (as we inderect by name/index)
	ENGINE_API bool DoesLayoutMatch(const FAnimNodeStructData& InOther) const;
#endif
	
private:
	UPROPERTY()
	TMap<FName, int32> NameToIndexMap;

	UPROPERTY()
	int32 NumProperties = 0;
};
