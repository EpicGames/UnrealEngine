// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTypeTraits.h"

struct FAnimationUpdateSharedContext;
struct FAnimationBaseContext;
struct FAnimNode_Base;
class IAnimNotifyEventContextDataInterface;

// Simple RTTI implementation for graph messages

// Macro to be used in declaration of any new IGraphMessage types
#define DECLARE_ANIMGRAPH_MESSAGE(ClassName) \
	public: \
		static FName GetStaticTypeName() { return ClassName::TypeName; } \
		virtual FName GetTypeName() const final { return ClassName::GetStaticTypeName(); } \
	private: \
		static const FName TypeName;

#define DECLARE_ANIMGRAPH_MESSAGE_API(ClassName, ModuleApi) \
	public: \
		static FName GetStaticTypeName() { return ClassName::TypeName; } \
		virtual FName GetTypeName() const final { return ClassName::GetStaticTypeName(); } \
	private: \
		ModuleApi static const FName TypeName;

// Macro to be used in the implementation of any new IGraphMessage types
#define IMPLEMENT_ANIMGRAPH_MESSAGE(ClassName) \
	const FName ClassName::TypeName = TEXT(#ClassName);

#define IMPLEMENT_NOTIFY_CONTEXT_INTERFACE(ClassName) \
	IMPLEMENT_ANIMGRAPH_MESSAGE(ClassName);

#define DECLARE_NOTIFY_CONTEXT_INTERFACE(ClassName) \
	DECLARE_ANIMGRAPH_MESSAGE(ClassName)

#define DECLARE_NOTIFY_CONTEXT_INTERFACE_API(ClassName, ModuleApi) \
	DECLARE_ANIMGRAPH_MESSAGE_API(ClassName, ModuleApi)

namespace UE { namespace Anim {

class IAnimNotifyEventContextDataInterface
{
public:
	virtual ~IAnimNotifyEventContextDataInterface() = default;
	// RTTI support functions
	static FName GetStaticTypeName()
	{ 
		return BaseTypeName;
	}

	virtual FName GetTypeName() const = 0;
	
	template<typename Type> 
	bool Is() const 
	{ 
		return GetTypeName() == Type::GetStaticTypeName(); 
	}

	template<typename Type> 
	const Type& As() const 
	{ 
		return *static_cast<const Type*>(this);
	}

	template<typename Type> 
    Type& As() 
	{ 
		return *static_cast<Type*>(this);
	}
private:
	static ENGINE_API const FName BaseTypeName;
};
	
// Base class for all messages/events/scopes that are fired during execution
// Note that these messages are events/callbacks only, no blending is involved.
class IGraphMessage
{
public:
	virtual ~IGraphMessage() = default;

public:
	// RTTI support functions
	static FName GetStaticTypeName()
	{ 
		return BaseTypeName;
	}

	virtual FName GetTypeName() const = 0;

	template<typename Type> 
	bool Is() const 
	{ 
		return GetTypeName() == Type::GetStaticTypeName(); 
	}

	template<typename Type> 
	const Type& As() const 
	{ 
		return *static_cast<const Type*>(this);
	}

	template<typename Type> 
	Type& As() 
	{ 
		return *static_cast<Type*>(this);
	}

	UE_DEPRECATED(5.0, "No longer in use, please use MakeUniqueEventContextData")
	virtual TSharedPtr<const IAnimNotifyEventContextDataInterface> MakeEventContextData() const
	{
		TSharedPtr<const IAnimNotifyEventContextDataInterface> NullPtr;
		return NullPtr; 
	}

	virtual TUniquePtr<const IAnimNotifyEventContextDataInterface> MakeUniqueEventContextData() const
	{
		return TUniquePtr<const IAnimNotifyEventContextDataInterface>(); 
	}

private:
	static ENGINE_API const FName BaseTypeName;
};

// Pushes a simple tag onto the shared context stack
struct FScopedGraphTag
{
	ENGINE_API FScopedGraphTag(const FAnimationBaseContext& InContext, FName InTag);
	ENGINE_API ~FScopedGraphTag();

private:
	// Shared context that we push messages into
	FAnimationUpdateSharedContext* SharedContext;

	// The tag that we pushed
	FName Tag;
};

// Base helper class
struct FScopedGraphMessage
{
protected:
	ENGINE_API FScopedGraphMessage(const FAnimationBaseContext& InContext);

	// Helper functions
	ENGINE_API void PushMessage(const FAnimationBaseContext& InContext, TSharedRef<IGraphMessage> InMessage, FName InMessageType);
	ENGINE_API void PopMessage();

	// Shared context that we push messages into
	FAnimationUpdateSharedContext* SharedContext;

	// The type of the message we pushed
	FName MessageType;
};

// Pushes a message onto the shared context stack
template<typename TGraphMessageType>
struct TScopedGraphMessage : FScopedGraphMessage
{
	template<typename... TArgs>
	TScopedGraphMessage(const FAnimationBaseContext& InContext, TArgs&&... Args)
		: FScopedGraphMessage(InContext)
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		TSharedRef<TGraphMessageType> Message = MakeShared<TGraphMessageType>(Forward<TArgs>(Args)...);
		PushMessage(InContext, Message, Message->GetTypeName());
	}

	~TScopedGraphMessage()
	{
		PopMessage();
	}
};

// Optionally pushes a message onto the shared context stack
template<typename TGraphMessageType>
struct TOptionalScopedGraphMessage : FScopedGraphMessage
{
	template<typename... TArgs>
	TOptionalScopedGraphMessage(bool bInCondition, const FAnimationBaseContext& InContext, TArgs&&... Args)
		: FScopedGraphMessage(InContext)
		, bCondition(bInCondition)
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		if(bInCondition)
		{
			TSharedRef<TGraphMessageType> Message = MakeShared<TGraphMessageType>(Forward<TArgs>(Args)...);
			PushMessage(InContext, Message, Message->GetTypeName());
		}
	}

	~TOptionalScopedGraphMessage()
	{
		if(bCondition)
		{
			PopMessage();
		}
	}

private:
	// Record of condition
	bool bCondition;
};

// Stack of tags & events used to track context during graph execution
struct FMessageStack
{
public:
	friend struct FScopedGraphTag;
	friend struct FScopedGraphMessage;

	FMessageStack() = default;

	// Non-copyable
	FMessageStack(FMessageStack&) = delete;
	FMessageStack& operator=(const FMessageStack&) = delete;

	FMessageStack(FMessageStack&& InMessageStack) = default;
	FMessageStack& operator=(FMessageStack&&) = default;

	// Return value for the various ForEach* enumeration functions
	enum class EEnumerate
	{
		Stop,
		Continue,
	};

	// Info about a node
	struct FNodeInfo
	{
		FAnimNode_Base* Node;
		const UScriptStruct* NodeStruct;
	};

	// Call the supplied function with each node subscribed to the specified message in this stack
	// @param	InFunction		The function to call with the specified message. Function should return whether to continue or stop the enumeration.
	template<typename TGraphMessageType>
	void ForEachMessage(TFunctionRef<EEnumerate(TGraphMessageType&)> InFunction) const
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		ForEachMessage(TGraphMessageType::GetStaticTypeName(), [&InFunction](IGraphMessage& InMessage)
		{
			return InFunction(static_cast<TGraphMessageType&>(InMessage));
		});
	}

	// Call the supplied function on the top-most node subscribed to the specified message in this stack
	// @param	InFunction		The function to call with the specified message.
	template<typename TGraphMessageType>
	void TopMessage(TFunctionRef<void(TGraphMessageType&)> InFunction) const
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		TopMessage(TGraphMessageType::GetStaticTypeName(), [&InFunction](IGraphMessage& InMessage)
		{
			InFunction(static_cast<TGraphMessageType&>(InMessage));
		});
	}

	// Call the supplied function on the top-most node subscribed to the specified message in this stack
	// @param	InFunction		The function to call with the specified message.
	template<typename TGraphMessageType>
	void TopMessageWeakPtr(TFunctionRef<void(TWeakPtr<TGraphMessageType>&)> InFunction)
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		TopMessageSharedPtr(TGraphMessageType::GetStaticTypeName(), [&InFunction](TSharedPtr<IGraphMessage>& InMessage)
			{
				TSharedPtr<TGraphMessageType> PinPtr = StaticCastSharedPtr<TGraphMessageType>(InMessage);
				TWeakPtr<TGraphMessageType> WeakPtr(PinPtr); 
				InFunction(WeakPtr);
			});
	}

	// @return true if a message of the specified type is present
	template<typename TGraphMessageType>
	bool HasMessage() const
	{
		static_assert(TIsDerivedFrom<TGraphMessageType, IGraphMessage>::IsDerived, "Argument TGraphMessageType must derive from IGraphMessage");
		static_assert(!std::is_same_v<TGraphMessageType, IGraphMessage>, "Argument TGraphMessageType must not be IGraphMessage");

		return HasMessage(TGraphMessageType::GetStaticTypeName());
	}

	// Call the supplied function with each node tagged with the specified tag
	// @param	InTagId			The tag to check
	// @param	InFunction		The function to call with the specified tagged node. Function should return whether to continue or stop the enumeration.
	ENGINE_API void ForEachTag(FName InTagId, TFunctionRef<EEnumerate(FNodeInfo)> InFunction) const;

	// Copies the relevant parts of each stack only, ready for cached update.
	// @param	InStack		The stack to copy from
	ENGINE_API void CopyForCachedUpdate(const FMessageStack& InStack);

	// Call MakeEventContextData for the top entry of each MessageType, returning interfaces for types that return event data
	// @param	ContextData		An array of valid IAnimNotifyEventContextDataInterface.  Only one entry will exist per message type
	ENGINE_API void MakeEventContextData(TArray<TUniquePtr<const IAnimNotifyEventContextDataInterface>>& ContextData) const;

private:
	// Push a message onto the stack
	// @param	InNode		The node that this message was pushed from
	// @param	InMessage	The message to push
	ENGINE_API void PushMessage(FName InMessageType, FAnimNode_Base* InNode, const UScriptStruct* InStruct, TSharedRef<IGraphMessage> InMessage);

	// Pop a message off the stack
	ENGINE_API void PopMessage(FName InMessageType);

	// Helper function for the templated function of the same name above
	ENGINE_API void ForEachMessage(FName InMessageType, TFunctionRef<EEnumerate(IGraphMessage&)> InFunction) const;

	// Helper function for the templated function of the same name above
	ENGINE_API void TopMessage(FName InMessageType, TFunctionRef<void(IGraphMessage&)> InFunction) const;

	// Helper function for the templated function of the same name above
	void TopMessageSharedPtr(FName InMessageType, TFunctionRef<void(TSharedPtr<IGraphMessage>&)> InFunction);

	// Helper function for the templated function of the same name above
	ENGINE_API void BottomMessage(FName InMessageType, TFunctionRef<void(IGraphMessage&)> InFunction) const;

	// @return true if a message of the specified type is present
	ENGINE_API bool HasMessage(FName InMessageType) const;

	// Push a tag onto the stack
	// @param	InNode	The node that this tag was pushed from
	// @param	InTag	The tag to push
	ENGINE_API void PushTag(FAnimNode_Base* InNode, const UScriptStruct* InStruct, FName InTag);

	// Pop a tag off the stack
	// @param	InTag	The tag to pop
	ENGINE_API void PopTag(FName InTag);

private:
	// Holds a message that has been pushed onto the stack.
	// Message is held by a shared ptr because the same stack scope can potentially be
	// referenced by multiple cached poses (pesky DAG)
	struct FMessageEntry
	{
		TSharedPtr<IGraphMessage> Message;
		FNodeInfo NodeInfo;
	};

	using FMessageStackEntry = TArray<FMessageEntry, TInlineAllocator<8>>;
	using FMessageMap = TMap<FName, FMessageStackEntry, TInlineSetAllocator<8>>;

	// Message stack
	FMessageMap MessageStacks;

	using FTagStackEntry = TArray<FNodeInfo, TInlineAllocator<4>>;
	using FTagMap = TMap<FName, FTagStackEntry, TInlineSetAllocator<4>>;

	// Tag stack
	FTagMap TagStacks;
};

}}	// namespace UE::Anim