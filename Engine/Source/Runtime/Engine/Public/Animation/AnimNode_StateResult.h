// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimNode_Root.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "AnimNode_StateResult.generated.h"

struct FAnimNode_StateMachine;
class UAnimGraphNode_StateResult;

// Root node of an state machine state (sink node).
// We dont use AnimNode_Root to let us distinguish these nodes in the property list at link time.
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_StateResult : public FAnimNode_Root
{
	GENERATED_USTRUCT_BODY()

	/** Used to upgrade old FAnimNode_Roots to FAnimNode_StateResult */
	ENGINE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITORONLY_DATA
protected:

	/** The index of the state this node belongs to. Filled in during the owning state machine's compilation. */
	UPROPERTY(meta = (FoldProperty))
	int32 StateIndex = -1;
#endif

public:
#if WITH_EDITORONLY_DATA
	void SetStateIndex(int32 InStateIndex) { StateIndex = InStateIndex; }	
#endif

	ENGINE_API int32 GetStateIndex() const;
	
	/** Gets the anim node function called on state entry, state became the current state of its state machine */
	ENGINE_API const FAnimNodeFunctionRef& GetStateEntryFunction() const;

	/** Gets the anim node function called on state fully blended in */
	ENGINE_API const FAnimNodeFunctionRef& GetStateFullyBlendedInFunction() const;

	/** Gets the anim node function called on state exit, state stopped being the current state of its state machine  */
	ENGINE_API const FAnimNodeFunctionRef& GetStateExitFunction() const;

	/** Gets the anim node function called on state fully blended out */
	ENGINE_API const FAnimNodeFunctionRef& GetStateFullyBlendedOutFunction() const;

private:
	friend struct FAnimNode_StateMachine;
	friend class UAnimGraphNode_StateResult;

#if WITH_EDITORONLY_DATA
	
	/** The function called on state entry */
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef StateEntryFunction;

	/** The function called on state fully blended in */
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef StateFullyBlendedInFunction;

	/** The function called on state exit */
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef StateExitFunction;

	/** The function called on state fully blended out */
	UPROPERTY(meta=(FoldProperty))
	FAnimNodeFunctionRef StateFullyBlendedOutFunction;

#endif
};

template<>
struct TStructOpsTypeTraits<FAnimNode_StateResult>
	: public TStructOpsTypeTraitsBase2<FAnimNode_StateResult>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
