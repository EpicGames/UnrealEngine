// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeMessages.h"

namespace UE::Anim {
	
// Notify Context Data containing if the notify state reached the end or was cancelled
// This is useful i.e When there's simultaneous instances of the same montage, to trigger start/end events of its notify states.
class FAnimNotifyEndDataContext : public UE::Anim::IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyEndDataContext)

public:
	ENGINE_API explicit FAnimNotifyEndDataContext(bool bInReachedEnd);

	const bool bReachedEnd = false;
};

}	// namespace UE::Anim
