// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimNotifyLibrary.generated.h"


struct FAnimNotifyEventReference;

/**
*	A library of commonly used functionality for Notifies, exposed to blueprint.
*/
UCLASS(meta = (ScriptName = "UAnimNotifyLibrary"), MinimalAPI)
class UAnimNotifyLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Get whether the notify state reached the end (was not cancelled)
	*
	* @param EventReference		The event to inspect
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API bool NotifyStateReachedEnd(const FAnimNotifyEventReference& EventReference);

	/**
	* Get the start time for this notify. For a notify this is the trigger time.
	*
	* @param EventReference		The event to inspect
	* @return the time in at which this notify started, relative to the owning asset i.e. play length.
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API float GetNotifyStartTime(const FAnimNotifyEventReference& EventReference);

	/**
	 * Get the current anim notify time in seconds for when this notify was fired
	 *
	 * @param EventReference		The event to inspect
	 * @return the time in seconds through the current animation for when this notify was fired
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API float GetCurrentAnimationTime(const FAnimNotifyEventReference& EventReference);

	/**
	 * Get the current anim notify time as a ratio (0 -> 1) through the animation for when this notify was fired
	 *
	 * @param EventReference		The event to inspect
	 * @return the time as a ratio (0 -> 1) through the animation for when this notify was fired
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API float GetCurrentAnimationTimeRatio(const FAnimNotifyEventReference& EventReference);
	
	/**
	 * Gets the current time in seconds relative to the start of the notify state, clamped to the range of the notify
	 * state
	 *
	 * @param EventReference		The event to inspect
	 * @return  the current time in seconds relative to the start of the notify state, clamped to the range of the
	 *			notify state
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API float GetCurrentAnimationNotifyStateTime(const FAnimNotifyEventReference& EventReference);

	/**
	 * Gets the current time as a ratio (0 -> 1) relative to the start of the notify state
	 *
	 * @param EventReference		The event to inspect
	 * @return  the current time as a ratio (0 -> 1) relative to the start of the notify state
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API float GetCurrentAnimationNotifyStateTimeRatio(const FAnimNotifyEventReference& EventReference);

	/**
	 * Gets whether this notify comes from a source that is blending out.
	 *
	 * @param EventReference		The event to inspect
	 * @return  Whether this notify comes from a source that is blending out.
	 */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies" , meta = (ScriptMethod))
	static ENGINE_API bool IsBlendingOut(const FAnimNotifyEventReference& EventReference);
};
