// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimNotifyMirrorInspectionLibrary.generated.h"


struct FAnimNotifyEventReference;
/**
*	A library of commonly used functionality for Notifies related to mirroring, exposed to blueprint.
*/
UCLASS(meta = (ScriptName = "UAnimNotifyMirrorInspectionLibrary"), MinimalAPI)
class UAnimNotifyMirrorInspectionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Get whether the animation which triggered this notify was mirrored.
	*
	* @param EventReference		The event to inspect
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies")
    static ENGINE_API bool IsTriggeredByMirroredAnimation(const FAnimNotifyEventReference& EventReference);

	/** If the notify is mirrored, return the mirror data table that was active.
	*
	* @param EventReference		The event to inspect
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies")
    static ENGINE_API const UMirrorDataTable* GetMirrorDataTable(const FAnimNotifyEventReference& EventReference);
    

};
