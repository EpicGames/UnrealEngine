// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/RandomStream.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeMessages.h"
#include "AnimNotifyQueue.generated.h"

#define UE_API ENGINE_API

class USkeletalMeshComponent;
struct FAnimInstanceProxy;
struct FAnimNotifyEvent;
class UMirrorDataTable;
struct FAnimTickRecord;
class UWorld;

USTRUCT(BlueprintType)
struct FAnimNotifyEventReference
{
	GENERATED_BODY()

	FAnimNotifyEventReference() = default;

	FAnimNotifyEventReference(const FAnimNotifyEvent* InNotify, const UObject* InNotifySource)
		: Notify(InNotify)
		, MirrorTable(nullptr)
		, NotifySource(InNotifySource)
	{}

	FAnimNotifyEventReference(const FAnimNotifyEvent* InNotify, const UObject* InNotifySource, const UMirrorDataTable* MirrorDataTable)
    : Notify(InNotify)
	, MirrorTable(MirrorDataTable)
    , NotifySource(InNotifySource)
	{}


	const FAnimNotifyEvent* GetNotify() const
	{
		return NotifySource ? Notify : nullptr;
	}

	void SetNotify(const FAnimNotifyEvent* InNotify)
	{
		if (NotifySource)
		{
			Notify = InNotify;
		}
	}

	const UMirrorDataTable* GetMirrorDataTable() const
	{
		return MirrorTable.Get();
	}
	
	friend bool operator==(const FAnimNotifyEventReference& Lhs, const FAnimNotifyEventReference& Rhs)
	{
		return(
			(Lhs.Notify == Rhs.Notify) ||
			(Lhs.Notify && Rhs.Notify && *Lhs.Notify == *Rhs.Notify)
		);
	}

	friend bool operator==(const FAnimNotifyEventReference& Lhs, const FAnimNotifyEvent& Rhs);
	
	template<typename Type> 
	const Type* GetContextData() const 
	{
		if(ContextData.IsValid())
		{
			for(const TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>& DataInterface : *ContextData)
			{
				if (DataInterface->Is<Type>())
				{
					return &(DataInterface->As<Type>()); 
				}
			}
		}
		return nullptr; 
	}

	// Pulls relevant data from the tick record
	UE_API void GatherTickRecordData(const FAnimTickRecord& InTickRecord);

	// Allows adding extra context data after GatherTickRecordData has been exectuted
	template<typename Type, typename... TArgs>
	void AddContextData(TArgs&&... Args)
	{
		static_assert(TPointerIsConvertibleFromTo<Type, const UE::Anim::IAnimNotifyEventContextDataInterface>::Value, "'Type' template parameter to MakeContextData must be derived from IAnimNotifyEventContextDataInterface");
		if (!ContextData.IsValid())
		{
			ContextData = MakeShared<TArray<TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>>>();
		}

		ContextData->Add(MakeUnique<Type>(Forward<TArgs>(Args)...));
	}

	// Gets the source object of this notify (e.g. anim sequence), if any
	const UObject* GetSourceObject() const
	{
		return NotifySource;
	}

	// Gets the current animation's time that this notify was fired at
	float GetCurrentAnimationTime() const
	{
		return CurrentAnimTime;
	}

	// Whether the context this notify was fired from is active or not (active == not blending out).
	bool IsActiveContext() const
	{
		return bActiveContext;
	}

	// NotifyInstanceID is generated when we dispatch the NotifyBegin/Notify call.
	// In the case of a non-unique/existing Notify, we re-use/propagate the ID from the previous notify. 
	int32 GetNotifyInstanceID() const { return NotifyInstanceID; }
	void SetNotifyInstanceID(int32 InNewInstanceID) { NotifyInstanceID = InNewInstanceID; }

private:
	// Context data gleaned from the tick record
	TSharedPtr<TArray<TUniquePtr<const UE::Anim::IAnimNotifyEventContextDataInterface>>> ContextData;
	
	const FAnimNotifyEvent* Notify = nullptr;

	// If set, the Notify has been mirrored.  The mirrored name can be found in MirrorTable->AnimNotifyToMirrorAnimNotifyMap
	UPROPERTY(transient)
	TObjectPtr<const UMirrorDataTable> MirrorTable = nullptr; 

	UPROPERTY(transient)
	TObjectPtr<const UObject> NotifySource = nullptr;

	// The recorded time from the tick record that this notify event was fired at
	float CurrentAnimTime = 0.0f;

	// Unique per-notify instance ID. This gets populated before firing the notifies and should only be used after when notifies are actually fired.
	UPROPERTY(transient, Category=Notifies, BlueprintReadOnly, meta=(AllowPrivateAccess="true"))
	int32 NotifyInstanceID = INDEX_NONE;

	// Whether the context this notify was fired from is active or not (active == not blending out).
	bool bActiveContext = false;
};

USTRUCT()
struct FAnimNotifyArray
{
	GENERATED_BODY()

	UPROPERTY(transient)
	TArray<FAnimNotifyEventReference> Notifies;
};

USTRUCT()
struct FAnimNotifyContext
{
	GENERATED_BODY()
	FAnimNotifyContext() {}
	FAnimNotifyContext(const FAnimTickRecord& InTickRecord)
    : TickRecord(&InTickRecord)
	{}
	const FAnimTickRecord* TickRecord = nullptr;
	TArray<FAnimNotifyEventReference> ActiveNotifies;
};

USTRUCT()
struct FAnimNotifyQueue
{
	GENERATED_BODY()

	FAnimNotifyQueue()
		: PredictedLODLevel(-1)
	{
		RandomStream.Initialize(0x05629063);
	}

	/** Should the notifies current filtering mode stop it from triggering */
	bool PassesFiltering(const FAnimNotifyEvent* Notify) const;

	/** Work out whether this notify should be triggered based on its chance of triggering value */
	bool PassesChanceOfTriggering(const FAnimNotifyEvent* Event) const;

	/** Add notify to queue*/
	void AddAnimNotify(const FAnimNotifyEvent* Notify, const UObject* NotifySource);

	/** Add anim notifies **/
	UE_API void AddAnimNotifies(bool bSrcIsLeader, const TArray<FAnimNotifyEventReference>& NewNotifies, const float InstanceWeight);

	/** Add anim notifies from montage**/
	void AddAnimNotifies(bool bSrcIsLeader, const TMap<FName, TArray<FAnimNotifyEventReference>>& NewNotifies, const float InstanceWeight);

	/** Wrapper functions for when we aren't coming from a sync group **/
	void AddAnimNotifies(const TArray<FAnimNotifyEventReference>& NewNotifies, const float InstanceWeight) { AddAnimNotifies(true, NewNotifies, InstanceWeight); }
	void AddAnimNotifies(const TMap<FName, TArray<FAnimNotifyEventReference>>& NewNotifies, const float InstanceWeight) { AddAnimNotifies(true, NewNotifies, InstanceWeight); }

	/** Reset queue & update LOD level */
	void Reset(USkeletalMeshComponent* Component);

	/** Append one queue to another */
	void Append(const FAnimNotifyQueue& Queue);
	
	/** 
	 *	Best LOD that was 'predicted' by UpdateSkelPose. Copied form USkeletalMeshComponent.
	 *	This is what bones were updated based on, so we do not allow rendering at a better LOD than this. 
	 */
	int32 PredictedLODLevel;

	/** Internal random stream */
	FRandomStream RandomStream;

	/** Animation Notifies that has been triggered in the latest tick **/
	UPROPERTY(transient)
	TArray<FAnimNotifyEventReference> AnimNotifies;

	/** Animation Notifies from montages that still need to be filtered by slot weight*/
	UPROPERTY(transient)
	TMap<FName, FAnimNotifyArray> UnfilteredMontageAnimNotifies;

	/** Takes the cached notifies from playing montages and adds them if they pass a slot weight check */
	void ApplyMontageNotifies(const FAnimInstanceProxy& Proxy);
private:
	/** Implementation for adding notifies*/
	void AddAnimNotifiesToDest(bool bSrcIsLeader, const TArray<FAnimNotifyEventReference>& NewNotifies, TArray<FAnimNotifyEventReference>& DestArray, const float InstanceWeight);

	/** Adds the contents of the NewNotifies array to the DestArray (maintaining uniqueness of notify states*/
	void AddAnimNotifiesToDestNoFiltering(const TArray<FAnimNotifyEventReference>& NewNotifies, TArray<FAnimNotifyEventReference>& DestArray) const;

	UPROPERTY(Transient)
	TWeakObjectPtr<UWorld> World;
};

#undef UE_API