// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNotifyQueue.h"
#include "AnimNotifyStateMachineInspectionLibrary.generated.h"

struct FAnimNotifyEventReference;
class USkeletalMeshComponent;
/**
*	A library of commonly used functionality for Notifies related to state machines, exposed to blueprint.
*/
UCLASS(meta = (ScriptName = "UAnimNotifyStateMachineInspectionLibrary"), MinimalAPI)
class UAnimNotifyStateMachineInspectionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** Get whether the notify was triggered from the specified state machine
	*
    * @param EventReference		The event to inspect
    * @param MeshComp			The skeletal mesh that contains the animation instance that produced the event
    * @param StateMachineName	The Name of a state machine to test
    */
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies")
    static ENGINE_API bool IsTriggeredByStateMachine(const FAnimNotifyEventReference& EventReference,UAnimInstance* AnimInstance, FName StateMachineName);
	
	/** Get whether a particular state in a specific state machine triggered the notify
	*
	* @param EventReference		The event to inspect
	* @param MeshComp			The skeletal mesh that contains the animation instance that produced the event
	* @param StateMachineName	The name of a state machine to test
	* @param StateName			The name of a state to test
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies")
	static ENGINE_API bool IsTriggeredByStateInStateMachine(const FAnimNotifyEventReference& EventReference, UAnimInstance* AnimInstance, FName StateMachineName, FName StateName);

	/** Get whether a state with the given name in any state machine triggered the notify
	*
	* @param EventReference		The event to inspect
	* @param MeshComp			The skeletal mesh that contains the animation instance that produced the event
	* @param StateName			The name of a state to test
	*/
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|Notifies")
    static ENGINE_API bool IsTriggeredByState(const FAnimNotifyEventReference& EventReference, UAnimInstance* AnimInstance, FName StateName);
    
	/** Get whether the Reference ContextData has the given machine index in a UAnimNotifyStateMachineContext.  This function should not be exposed to blueprint
	*
	* @param Reference		The event to inspect
	* @param StateMachineIndex	The MachineIndex as defined in UAnimInstance
	*/
	static ENGINE_API bool IsStateMachineInEventContext(const FAnimNotifyEventReference& Reference, int32 StateMachineIndex);

	/** Get whether the Reference ContextData has the given state and machine index in a UAnimNotifyStateMachineContext.
	*	
	* @param Reference			The event to inspect
	* @param StateIndex			Index of a state inside a state machine as defined in UAnimInstance
	* @param StateMachineIndex	The MachineIndex as defined in UAnimInstance
	*/
	static ENGINE_API bool IsStateInStateMachineInEventContext(const FAnimNotifyEventReference& Reference, int32 StateMachineIndex, int32 StateIndex); 
};
