// Copyright Epic Games, Inc. All Rights Reserved.

// This code is based upon and adapted to Unreal from the code 
// provided in the Sandbox project here:
// https://github.com/melax/sandbox

// The license for Sandbox is reproduced below:

/*
	Copyright (c) 2014 Stan Melax

	Permission is hereby granted, free of charge, to any person obtaining a copy
	of this software and associated documentation files (the "Software"), to deal
	in the Software without restriction, including without limitation the rights
	to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
	copies of the Software, and to permit persons to whom the Software is
	furnished to do so, subject to the following conditions:

	The above copyright notice and this permission notice shall be included in all
	copies or substantial portions of the Software.

	THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
	IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
	FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
	AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
	LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
	OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
	SOFTWARE.
*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"

#include "AnimPhysicsSolver.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Dynamics Physics Update"), STAT_AnimDynamicsUpdate, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Linear Limit Pre-Update"), STAT_AnimDynamicsLinearPre, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Linear Limit Post-Update"), STAT_AnimDynamicsLinearPost, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Angular Limit Pre-Update"), STAT_AnimDynamicsAngularPre, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Angular Limit Post-Update"), STAT_AnimDynamicsAngularPost, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Velocity Init"), STAT_AnimDynamicsVelocityInit, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Pose Update"), STAT_AnimDynamicsPoseUpdate, STATGROUP_Physics, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Limits Update"), STAT_AnimDynamicsLimitUpdate, STATGROUP_Physics, ENGINE_API);

class FAnimPhys;

namespace AnimPhysicsConstants
{
	// Added bias for angular joints
	inline const float		JointBiasFactor = 0.3f;

	// Damping for linear momentum (1 = critically damped)
	inline const float		LinearDamping = 0.7f;

	// Damping for angular momentum (1 = critically damped)
	inline const float		AngularDamping = 0.7f;

	inline const float		DefaultSpringConstantLinear = 12.0f;
	inline const float		DefaultSpringConstantAngular = 4.0f;
}

// Enum for picking current angular twist axis
UENUM()
enum class AnimPhysTwistAxis : uint8
{
	AxisX,
	AxisY,
	AxisZ
};

UENUM()
enum class AnimPhysCollisionType : uint8
{
	CoM UMETA(DisplayName="CoM", DisplayValue="CoM", ToolTip="Only limit the center of mass from crossing planes."),
	CustomSphere UMETA(ToolTip="Use the specified sphere radius to collide with planes."),
	InnerSphere UMETA(ToolTip="Use the largest sphere that fits entirely within the body extents to collide with planes."),
	OuterSphere UMETA(ToolTip="Use the smallest sphere that wholely contains the body extents to collide with planes.")
};

struct FAnimPhysShape
{
	/** Makes a box with the given extents
	 *  @param Extents Extents of the resulting box
	 */
	static ENGINE_API FAnimPhysShape MakeBox(const FVector& Extents);

	ENGINE_API FAnimPhysShape();
	ENGINE_API FAnimPhysShape(TArray<FVector>& InVertices, TArray<FIntVector>& InTriangles);

	/** Transforms each vertex in the shape
	 *  @param InTransform Transform to apply to the verts
	 */
	ENGINE_API void TransformVerts(FTransform& InTransform);

	// Vertex positions defining the shape
	TArray<FVector> Vertices;

	// Triangles defining the shape
	TArray<FIntVector> Triangles;

	// Volume of the shape
	float Volume;

	// Center of mass for the chape
	FVector CenterOfMass;
};

/**
  * Defines a transform (Position/Orientation) for an anim phys object without scaling
  */
struct FAnimPhysPose
{
	FVector Position;
	FQuat Orientation;
	float Scale;

	FAnimPhysPose(const FVector& InPosition, const FQuat& InOrient)
		: Position(InPosition)
		, Orientation(InOrient)
		, Scale(1.0)
	{}

	FAnimPhysPose(const FVector& InPosition, const FQuat& InOrient, const float& InScale)
		: Position(InPosition)
		, Orientation(InOrient)
		, Scale(InScale)
	{}

	FAnimPhysPose()
		: Position(FVector::ZeroVector)
		, Orientation(FQuat::Identity)
		, Scale(1.0)
	{

	}

	FAnimPhysPose Inverse() const
	{
		FQuat InverseOrient = Orientation.Inverse();
		FVector InversePosition = InverseOrient * Position;
		return FAnimPhysPose(InversePosition, InverseOrient);
	}

	FMatrix Matrix() const
	{
		return FTransform(Orientation, Position).ToMatrixNoScale();
	}

	FVector operator*(const FVector& InPoint) const
	{
		return Position + (Orientation * InPoint);
	}

	FAnimPhysPose operator*(const FAnimPhysPose& InPose) const
	{
		return FAnimPhysPose(*this *InPose.Position, Orientation * InPose.Orientation);
	}

	FPlane TransformPlane(const FPlane& InPlane)
	{
		FVector NewNormal(InPlane.X, InPlane.Y, InPlane.Z);
		NewNormal = Orientation * NewNormal;
		return FPlane(NewNormal, InPlane.W - FVector::DotProduct(Position, NewNormal));
	}
};

/**
  * Defines a single closed, convex shape within the rigid body
  */
class FAnimPhysState
{
  public:
	ENGINE_API FAnimPhysState();
	ENGINE_API FAnimPhysState(const FVector& InPosition, const FQuat& InOrient, const FVector& InLinearMomentum, const FVector& InAngularMomentum);

	FAnimPhysPose Pose;

	FVector LinearMomentum;   
	FVector AngularMomentum;

    FAnimPhysPose&           GetPose() { return Pose; }
    const FAnimPhysPose&     GetPose() const { return Pose; }

	FAnimPhysState&          GetState() { return *this; }
	const FAnimPhysState&    GetState() const { return *this; }
};

/** 
  * Simple struct holding wind params passed into simulation
  */
struct FAnimPhysWindData
{
	// Scale for the final velocity
	float BodyWindScale;

	// Current wind speed
	float WindSpeed;

	// World space wind direction
	FVector WindDirection;

	// Mirrors APEX adaption, adds some randomness / billow
	float WindAdaption;
};

/**
  * A collection of shapes grouped for simulation as a rigid body
  */
class FAnimPhysRigidBody : public FAnimPhysState
{
  public:

	ENGINE_API FAnimPhysRigidBody(TArray<FAnimPhysShape>& InShapes, const FVector& InPosition);

	// Spin / Omega for the body
	ENGINE_API FVector Spin();

	// Mass of this body
	float				Mass;

	// 1.0f / Mass
	float				InverseMass;

	// Mass free inverse tensor (easier to scale mass)
	FMatrix				InverseTensorWithoutMass;  

	// Full (with mass) inverse tensor in world space
	FMatrix				InverseWorldSpaceTensor;    // Inverse Inertia Tensor rotated into world space 

	FMatrix				InertiaTensor;

	// Maintained state, Start/Prev/Next
	FVector				NextPosition;
	FQuat				NextOrientation;
	FVector				PreviousPosition;
	FQuat				PreviousOrientation;
	FVector				StartPosition;
	FQuat				StartOrientation;

	// Whether to use wind forces on this body
	bool				bWindEnabled;

	// Per-body wind data (speed, direction etc.)
	FAnimPhysWindData	WindData;

	// Override angular damping for this body
	bool				bAngularDampingOverriden;
	float				AngularDamping;

	// Override linear damping for this body
	bool				bLinearDampingOverriden;
	float				LinearDamping;

	// Override gravity scale for this body
	float				GravityScale;

	//Use manual gravity override
	bool bUseGravityOverride;

	//Direct gravity override value
	FVector GravityOverride;

	// Previous motion state (linear/angular momentum)
	FAnimPhysState		PreviousState;

	// Body center of mass (CoM of all shapes)
	FVector				CenterOfMass;

	// Collision Data (Only how we interact with planes currently)
	AnimPhysCollisionType CollisionType;

	// Radius to use when not using CoM collision mode
	float SphereCollisionRadius;
	
	// Shapes contained within this body
	TArray<FAnimPhysShape>		Shapes;
};

/**
  * Base class for constraint limits
  */
class FAnimPhysLimit
{
public:
	FAnimPhysRigidBody* Bodies[2];

	ENGINE_API FAnimPhysLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody);
};

/**
  * Angular limit, keeps angular torque around an axis within a defined range
  */
class FAnimPhysAngularLimit : public FAnimPhysLimit
{
public:

	// Axis of the limit in world space
	FVector WorldSpaceAxis;

	// Rotational impulse
	float  Torque;

	// The required spin required to align the limit
	float  TargetSpin;

	// Minimum torque this limit can apply
	float  MinimumTorque;

	// Maximum torque this limit can apply
	float  MaximumTorque;

	// Cached spin to torque value that is independant of iterations
	float CachedSpinToTorque;

	ENGINE_API FAnimPhysAngularLimit();
	ENGINE_API FAnimPhysAngularLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody, const FVector& InWorldSpaceAxis, float InTargetSpin = 0, float InMinimumTorque = -MAX_flt, float InMaximumTorque = MAX_flt);	

	/** Remove bias added to solve the limit
	 */
	ENGINE_API void RemoveBias();

	/** Solve the limit
	 */
	ENGINE_API void Iter(float DeltaTime);

	ENGINE_API void UpdateCachedData();
};

class FAnimPhysLinearLimit : public FAnimPhysLimit
{
 public:

	// Position of anchor on first object (in first object local space)
	FVector FirstPosition;

	// Position of anchor on second object (in second object local space)
	FVector SecondPosition;

	// Normal along which the limit is applied
	FVector LimitNormal;

	// Target speed needed to solve the limit
	float  TargetSpeed; 

	// Target speed of the limit without bias (force added just to solve limit)
	float  TargetSpeedWithoutBias;

	// Minimum force this limit can apply along the normal
	float  MinimumForce;

	// Maximum force this limit can apply along the normal
	float  Maximumforce;

	// Sum of impulses applied
	float  SumImpulses;

	// Cached denominator of the impulse calculation, doesn't change across iterations
	float InverseInertiaImpulse;

	// Cached world space position on body 0
	FVector WorldSpacePosition0;

	// Cached world space position on body 1
	FVector WorldSpacePosition1;

	ENGINE_API FAnimPhysLinearLimit();
	ENGINE_API FAnimPhysLinearLimit(FAnimPhysRigidBody* InFirstBody, FAnimPhysRigidBody* InSecondBody, const FVector& InFirstPosition, const FVector& InSecondPosition,
		const FVector& InNormal = FVector(0.0f, 0.0f, 1.0f), float InTargetSpeed = 0.0f, float InTargetSpeedWithoutBias = 0.0f, const FVector2D& InForceRange = FVector2D(-MAX_flt, MAX_flt));

	/** Remove bias added to solve the limit
	*/
	ENGINE_API void RemoveBias();

	/** Solve the limit
	*/
	ENGINE_API void Iter(float DetlaTime);

	ENGINE_API void UpdateCachedData();
};

struct FAnimPhysSpring
{
	// Bodies connected by the spring (Or nullptr for world)
	FAnimPhysRigidBody* Body0;
	FAnimPhysRigidBody* Body1;

	// Extra orientation applied on top of Body0's orientation to the target direction
	FQuat TargetOrientationOffset;

	// Target in body0 space for the target axis on body1 to reach
	FVector AngularTarget;
	
	// The axis of body1's orientation to match to the angular target
	AnimPhysTwistAxis AngularTargetAxis;

	// Local space anchor positions (in space of Body0 and Body1 respectively)
	FVector Anchor0;
	FVector Anchor1;

	// Spring constant for linear springs
	float SpringConstantLinear;

	// Sprint constant for angular springs
	float SpringConstantAngular;

	// Whether to apply linear spring force
	bool bApplyLinear;

	// Whether to apply angular spring force
	bool bApplyAngular;

	/** Applies forces to the bound bodies
	 */
	ENGINE_API void ApplyForces(float DeltaTime);
};

/**
  * Lightweight rigid body motion solver (no collision) used for cosmetic secondary motion in an animation graph
  * without invoking something heavier like using PhysX to simulate constraints which could be cost prohibitive
  */
class FAnimPhys
{
public:

	// Runtime CVar toggle for detailed/verbose profiling
	static ENGINE_API bool bEnableDetailedStats;

	// How much of the coriolis force to apply to the sim bodies
	static ENGINE_API float CoriolisAlpha;

	// How much of the centrifugal force to apply to the sim bodies
	static ENGINE_API float CentrifugalAlpha;

	// How much of the euler force to apply to the sim bodies
	static ENGINE_API float EulerAlpha;

	// How much of the simulation frame's angular acceleration to pass onto the sim bodies.
	static constexpr float AngularAccelerationAlpha = 1.0f;

	/** Calculates the volume of a shape
	 *  @param InVertices Verts in the shape
	 *  @param InTriangles Triangles represented in InVertices
	 */
	static ENGINE_API float CalculateVolume(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles);
	
	/** Calculates the volume of a collection of shapes
	 *  @param InShapes Collection of shapes to use
	 */
	static ENGINE_API float CalculateVolume(const TArray<FAnimPhysShape>& InShapes);

	/** Calculates the center of mass of a shape
	 *  @param InVertices Verts in the shape
	 *  @param InTriangles Triangles represented in InVertices
	 */
	static ENGINE_API FVector CalculateCenterOfMass(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles);
	
	/** Calculates the centre of mass of a collection of shapes
	 *  @param InShapes Collection of shapes to use
	 */
	static ENGINE_API FVector CalculateCenterOfMass(const TArray<FAnimPhysShape>& InShapes);

	/** Calculate the inertia tensor of a shape
	 *  @param InVertices Verts in the shape
	 *  @param InTriangles Triangles represented in InVertices
	 *  @param InCenterOfMass Center of mass of the shape
	 */
	static ENGINE_API FMatrix CalculateInertia(const TArray<FVector>& InVertices, const TArray<FIntVector>& InTriangles, const FVector& InCenterOfMass);

	/** Calculate the inertia tensor of a collection of shapes
	 *  @param InShapes Shapes to use
	 *  @param InCenterOfMass Center of mass of the collection
	 */
	static ENGINE_API FMatrix CalculateInertia(const TArray<FAnimPhysShape>& InShapes, const FVector& InCenterOfMass);

	/** Scale the mass and inertia properties of a rigid body
	 *  @param InOutRigidBody Body to modify
	 *  @param Scale Scale to use
	 */
	static ENGINE_API void ScaleRigidBodyMass(FAnimPhysRigidBody* InOutRigidBody, float Scale);  // scales the mass and all relevant inertial properties by multiplier s

	/** Update the inertial properties based on a scale change to the underlying shapes of the bodies independent of mass scaling
	 *  @param InOutRigidBody Body to modify
	 *  @param Scale Scale to use
	 */
	static ENGINE_API void RigidBodyShapeScaleUpdate(FAnimPhysRigidBody* InOutRigidBody, float Scale);


	/** Apply an impulse to a body
	 *  @param InOutRigidBody Body to apply the impulse to
	 *  @param InWorldOrientedImpactPoint Location of the impulse
	 *  @param InImpulse Impulse to apply
	 */
	static ENGINE_API void ApplyImpulse(FAnimPhysRigidBody* InOutRigidBody, const FVector& InWorldOrientedImpactPoint, const FVector& InImpulse);

	
	/** Performs a physics update on the provided state objects
	*  @param DeltaTime Time for the frame / update
	*  @param Bodies Bodies to integrate
	*  @param LinearLimits Linear limits to apply to the bodies
	*  @param AngularLimits Angular limits to apply to the bodies
	*  @param Springs Linear/Angular springs to apply to the bodies prior to solving
	*  @param NumPreIterations Number of times to iterate the limits before performing the integration
	*  @param NumPostIterations Number of times to iterae the limits after performing the integration
	*/
	static ENGINE_API void PhysicsUpdate(float DeltaTime, TArray<FAnimPhysRigidBody*>& Bodies, TArray<FAnimPhysLinearLimit>& LinearLimits, TArray<FAnimPhysAngularLimit>& AngularLimits, TArray<FAnimPhysSpring>& Springs, const FVector& GravityDirection, const FVector& ExternalForce, const FVector& ExternalLinearAcc, const FVector& ExternalAngularAcc, const FVector& ExternalAngularVelocity, int32 NumPreIterations = 8, int32 NumPostIterations = 2);

	//////////////////////////////////////////////////////////////////////////
	// Constraint functions

	/** Constrain bodies along a provided axis
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in constraint (or nullptr for world)
	 *  @param FirstPosition Local position on first body to apply constraint
	 *  @param SecondBody Second body in constraint
	 *  @param SecondPosition Local position on second body to apply constraint
	 *  @param AxisToConstrain Axis to constrain between bodies
	 *  @param Limits Limit on the provided axis so that Limits.X(min) < Final Position < Limits.Y(max)
	 *  @param MinimumForce Minimum force that can be applied to satisfy the maximum limit (default -MAX_flt)
	 *  @param MaximumForce Maximum force that can be applied to satisfy the minimum limit (default MAX_flt)
	 */
	static ENGINE_API void ConstrainAlongDirection(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody *SecondBody, const FVector& SecondPosition, const FVector& AxisToConstrain, const FVector2D Limits, float MinimumForce = -MAX_flt, float MaximumForce = MAX_flt);

	/** Constrain bodies together as if fixed or nailed (linear only, bodies can still rotate)
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in constraint (or nullptr for world)
	 *  @param FirstPosition Local position on first body to apply constraint
	 *  @param SecondBody Second body in constraint
	 *  @param SecondPosition Local position on second body to apply constraint
	 */
	static ENGINE_API void ConstrainPositionNailed(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody *SecondBody, const FVector& SecondPosition);

	/** Constrain bodies together with linear limits forming a box or prism around the constraint
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in the constraint (or nullptr for world)
	 *  @param FirstPosition Local position on first body to apply constraint
	 *  @param SecondBody Second body in the constraint
	 *  @param SecondPosition Local position on the second body to apply constraint
	 *  @param PrismRotation Rotation to apply to the prism axes, only necessary when constraining to world, otherwise the rotation of the first body is used
	 *  @param LimitsMin Minimum limits along axes
	 *  @param LimitsMax Maximum limits along axes
	 */
	static ENGINE_API void ConstrainPositionPrismatic(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* FirstBody, const FVector& FirstPosition, FAnimPhysRigidBody* SecondBody, const FVector& SecondPosition, const FQuat& PrismRotation, const FVector& LimitsMin, const FVector& LimitsMax);

	/** Constraint two bodies together with angular limits, limiting the relative rotation between them.
	 *  Note that this allows TWO axes to rotate, the twist axis will always be locked
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in the constraint (or nullptr for world)
	 *  @param SecondBody Second body in constraint
	 *  @param JointFrame Frame/Rotation of the joint
	 *  @param TwistAxis The axis to regard as the twist axis
	 *  @param JointLimitMin Minimum limits along each axis (twist axis ignored)
	 *  @param JointLimitMax Maximum limits along each axis (twist axis ignored)
	 *  @param InJointBias Bias towards second body's forces (1.0f = 100%)
	 */
	static ENGINE_API void ConstrainAngularRange(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, FAnimPhysRigidBody *SecondBody, const FQuat& JointFrame, AnimPhysTwistAxis TwistAxis, const FVector& JointLimitMin, const FVector& JointLimitMax, float InJointBias);

	/** Constraints the rotation between two bodies into a cone
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in the constraint (or nullptr for world)
	 *  @param Normal0 Normal for the first side of the constraint
	 *  @param SecondBody Second body in the constraint
	 *  @param Normal1 Normal for the second side of the constraint
	 *  @param LimitAngle Angle to limit the cone to
	 *  @param InJointBias Bias towards second body's forces (1.0f = 100%)
	 */
	static ENGINE_API void ConstrainConeAngle(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody* FirstBody, const FVector& Normal0, FAnimPhysRigidBody* SecondBody, const FVector& Normal1, float LimitAngle, float InJointBias);  // a hinge is a cone with 0 limitangle

	/** Constrains the position of a body to one side of a plane placed at PlaneTransform (plane normal is Z axis)
	*  @param LimitContainer Container to add limits to
	*  @param Body The body to constrain to the plane
	*  @param PlaneTransform Transform of the plane, with the normal facing along the Z axis of the orientation
	*/
	static ENGINE_API void ConstrainPlanar(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& PlaneTransform);

	/** Constrains the position of a body within the requested sphere */
	static ENGINE_API void ConstrainSphericalInner(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& SphereTransform, float SphereRadius);

	/** Constrains the position of a body outside of the requested sphere */
	static ENGINE_API void ConstrainSphericalOuter(float DeltaTime, TArray<FAnimPhysLinearLimit>& LimitContainer, FAnimPhysRigidBody* Body, const FTransform& SphereTransform, float SphereRadius);

	//////////////////////////////////////////////////////////////////////////
	// Spring creation methods
	static ENGINE_API void CreateSpring(TArray<FAnimPhysSpring>& SpringContainer, FAnimPhysRigidBody* Body0, FVector Position0, FAnimPhysRigidBody* Body1, FVector Position1);

private:

	/** Calculate differentiated orientation quaternion
	*  @param InOrientation Orientation of the body
	*  @param InInverseTensor Inverse inertia tensor for the body
	*  @param InAngularMomentum Current angular momentum of the body
	*/
	static FQuat DiffQ(const FQuat& InOrientation, const FMatrix &InInverseTensor, const FVector& InAngularMomentum);

	/** Perform an RK update of the provided orientation
	*  @param InOrient Orientation to update
	*  @param InInverseTensor Inverse inertia tensor of the body
	*  @param InAngularMomentum Angular momentum of the body
	*  @param InDeltaTime Delta time to process the update over
	*/
	static FQuat UpdateOrientRK(const FQuat& InOrient, const FMatrix& InInverseTensor, const FVector& InAngularMomentum, float InDeltaTime);

	/** Initialize the velocity for a given body
	 *  @param InBody Body to initialize
	 */
	static void InitializeBodyVelocity(float DeltaTime, FAnimPhysRigidBody *InBody, const FVector& GravityDirection);

	/** Using calculated linear and angular momentum, integrate the position and orientation of a body
	 *  @param InBody Body to integrate
	 */
	static void CalculateNextPose(float DeltaTime, FAnimPhysRigidBody* InBody);

	/** Update previous and current pose state
	 *  @param InBody Body to update
	 */
	static void UpdatePose(FAnimPhysRigidBody* InBody);

	/** Internal version of constrain angular. ConstrainAngularRange can work out some of these params so wraps this
	 *  @param LimitContainer Container to add limits to
	 *  @param FirstBody First body in the constraint (or nullptr for world)
	 *  @param JointFrame0 Frame/Rotation of the first side of the joint
	 *  @param SecondBody Second body in the constraint
	 *  @param JointFrame1 Frame/Rotation of the second side of the joint
	 *  @param TwistAxis Axis to consider as the twist axis
	 *  @param InJointLimitMin Minimum limits for the joint (twist axis ignored, always locked)
	 *  @param InJointLimitMax Maximum limits for the joint (twist axis ignored, always locked)
	 */
	static void ConstrainAngularRangeInternal(float DeltaTime, TArray<FAnimPhysAngularLimit>& LimitContainer, FAnimPhysRigidBody *FirstBody, const FQuat& JointFrame0, FAnimPhysRigidBody *SecondBody, const FQuat& JointFrame1, AnimPhysTwistAxis TwistAxis, const FVector& InJointLimitMin, const FVector& InJointLimitMax, float InJointBias);
};
