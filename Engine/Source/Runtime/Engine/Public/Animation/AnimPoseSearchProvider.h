// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

#define UE_API ENGINE_API

struct FAnimationBaseContext;
class UAnimationAsset;

namespace UE::Anim
{

/** Modular feature interface for PoseSearch */
class IPoseSearchProvider : public IModularFeature
{
public:
	virtual ~IPoseSearchProvider() = default;

	static UE_API FName GetModularFeatureName();
	static UE_API bool IsAvailable();
	static UE_API IPoseSearchProvider* Get();
	
	struct FSearchResult
	{
		UObject* SelectedAsset = nullptr;
		float TimeOffsetSeconds = 0.f;
		float Dissimilarity = MAX_flt;
		bool bIsFromContinuingPlaying = false;
		bool bMirrored = false;
		float WantedPlayRate = 1.f;
		FVector BlendParameters = FVector::ZeroVector;
	};

	struct FSearchPlayingAsset
	{
		const UObject* Asset = nullptr;
		float AccumulatedTime = 0.f;
		bool bMirrored = false;
		FVector BlendParameters = FVector::ZeroVector;
	};

	struct FSearchFutureAsset : FSearchPlayingAsset
	{
		float IntervalTime = 0.f;
	};

	/**
	* Finds a matching pose in the input Object given the current graph context
	* 
	* @param	GraphContext					Graph execution context used to construct a pose search query
	* @param	AssetsToSearch					The assets to search for the pose query
	* @param	PlayingAsset.Asset				The currently playing asset, used to bias the score of the eventually found continuing pose
	* @param	PlayingAsset.AccumulatedTime	The accumulated time of the currently playing asset
	* @param	FutureAsset.Asset				The asset that will play in the future after FutureAssetIntervalTime seconds
	* @param	FutureAsset.AccumulatedTime		The FutureAsset accumulated time in seconds when it'll start play
	* @param	FutureAsset.IntervalTime		The requested time interval before FutureAsset will start playing at FutureAssetAccumulatedTime
	* 
	* @return	FSearchResult					The search result identifying the asset from AssetsToSearch or PlayingAsset that most closely matches the query
	*/
	virtual FSearchResult Search(const FAnimationBaseContext& GraphContext, TArrayView<const UObject*> AssetsToSearch, const FSearchPlayingAsset& PlayingAsset, const FSearchFutureAsset& FutureAsset) const = 0;
};

} // namespace UE::Anim

#undef UE_API
