// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UAnimSequence;
class UPoseAsset;

namespace UE::Anim::RetargetHelpers
{

#if WITH_EDITOR

enum class ERetargetSourceAssetStatus
{
	NoRetargetDataSet = 0,
	RetargetSourceMissing,
	RetargetDataOk,
};

// Warning, calling this on PostLoad generates asset dependenciy check failures
static ERetargetSourceAssetStatus CheckRetargetSourceAssetData(const UAnimSequence* InAsset);

// Warning, calling this on PostLoad generates asset dependenciy check failures
static ERetargetSourceAssetStatus CheckRetargetSourceAssetData(const UPoseAsset* InAsset);

#endif // WITH_EDITOR

} // end namespace UE::Anim::RetargetHelpers
