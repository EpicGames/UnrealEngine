// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "UObject/ObjectMacros.h"

class UAnimSequence;

struct FDeltaTimeRecord;

namespace UE { namespace Anim {

struct FStackAttributeContainer;

// Modular feature interface for AnimationWarping
class IAnimRootMotionProvider : public IModularFeature
{
public:
	static ENGINE_API const FName ModularFeatureName; // "AnimationWarping"
	static ENGINE_API const FName AttributeName;

	virtual ~IAnimRootMotionProvider() = default;

	static ENGINE_API bool IsAvailable();
	static ENGINE_API const IAnimRootMotionProvider* Get();
	
	// Given the specified time range, sequence, and looping behavior, sample a root motion delta and store it in the specified attribute container
	virtual void SampleRootMotion(const FDeltaTimeRecord& SampleRange, const UAnimSequence& Sequence, bool bLoopingSequence, FStackAttributeContainer& OutAttributes) const = 0;
	
	// Directly sets the root motion attribute. Does not require root motion to exist already.
	virtual bool SetRootMotion(const FTransform& RootMotionDelta, FStackAttributeContainer& OutAttributes) const { return false; }

	// Override the currently stored root motion delta in the specified attribute container. Requires root motion to exist prior to overriding.
	virtual bool OverrideRootMotion(const FTransform& RootMotionDelta, FStackAttributeContainer& OutAttributes) const = 0;
	
	// Extract the currently stored root motion delta in the specified attribute container.
	virtual bool ExtractRootMotion(const FStackAttributeContainer& Attributes, FTransform& OutRootMotionDelta) const = 0;
	
	// Query whether or not a computed root motion delta exists within the specified attribute container.
	virtual bool HasRootMotion(const FStackAttributeContainer& Attributes) const = 0;
};
}}
