// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTypes.h"
#include "Misc/FrameRate.h"

struct ICompressedAnimData;
class USkeleton;

/* Encapsulates decompression related data used by bone compression codecs. */
struct FAnimSequenceDecompressionContext
{
	UE_DEPRECATED(5.1, "his constructor is deprecated. Use the other constructor by passing the ref pose and track to skeleton map to ensure safe usage with all codecs")
	FAnimSequenceDecompressionContext(float SequenceLength_, EAnimInterpolationType Interpolation_, const FName& AnimName_, const ICompressedAnimData& CompressedAnimData_)
		:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SequenceLength(SequenceLength_),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Interpolation(Interpolation_), AnimName(AnimName_), CompressedAnimData(CompressedAnimData_)
	{
	}

	UE_DEPRECATED(5.2, "FAnimSequenceDecompressionContext signature has been deprecated, use different one with sampling rate and number of frames")
	FAnimSequenceDecompressionContext(float SequenceLength_, EAnimInterpolationType Interpolation_, const FName& AnimName_, const ICompressedAnimData& CompressedAnimData_,
		const TArray<FTransform>& RefPoses_, const TArray<FTrackToSkeletonMap>& TrackToSkeletonMap_)
		:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SequenceLength(SequenceLength_),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Interpolation(Interpolation_), AnimName(AnimName_), CompressedAnimData(CompressedAnimData_),
		RefPoses(RefPoses_.GetData(), RefPoses_.Num()), TrackToSkeletonMap(TrackToSkeletonMap_.GetData(), TrackToSkeletonMap_.Num())
	{
	}

	UE_DEPRECATED(5.3, "FAnimSequenceDecompressionContext signature has been deprecated, use different one with additive type")
	FAnimSequenceDecompressionContext(const FFrameRate& InSamplingRate, const int32 InNumberOfFrames, EAnimInterpolationType Interpolation_, const FName& AnimName_, const ICompressedAnimData& CompressedAnimData_, const TArray<FTransform>& RefPoses_, const TArray<FTrackToSkeletonMap>& TrackToSkeletonMap_, const USkeleton* InSourceSkeleton, bool bIsBakedAdditive)
		:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SequenceLength(static_cast<float>(InSamplingRate.AsSeconds(InNumberOfFrames))),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Interpolation(Interpolation_), AnimName(AnimName_), CompressedAnimData(CompressedAnimData_),
		RefPoses(RefPoses_.GetData(), RefPoses_.Num()), TrackToSkeletonMap(TrackToSkeletonMap_.GetData(), TrackToSkeletonMap_.Num()),
		SamplingRate(InSamplingRate),
		SamplingTime(0),
		NumberOfFrames(InNumberOfFrames),
		SourceSkeleton(InSourceSkeleton),
		bAdditiveAnimation(bIsBakedAdditive)
	{}

	FAnimSequenceDecompressionContext(
		const FFrameRate& InSamplingRate,
		const int32 InNumberOfFrames,
		EAnimInterpolationType InInterpolation,
		const FName& InAnimName,
		const ICompressedAnimData& InCompressedAnimData,
		const TArray<FTransform>& InRefPoses,
		const TArray<FTrackToSkeletonMap>& InTrackToSkeletonMap,
		const USkeleton* InSourceSkeleton,
		bool bIsBakedAdditive,
		EAdditiveAnimationType InAdditiveType)
		:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SequenceLength(static_cast<float>(InSamplingRate.AsSeconds(InNumberOfFrames))),
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Interpolation(InInterpolation), AnimName(InAnimName), CompressedAnimData(InCompressedAnimData),
		RefPoses(InRefPoses.GetData(), InRefPoses.Num()), TrackToSkeletonMap(InTrackToSkeletonMap.GetData(), InTrackToSkeletonMap.Num()),
		SamplingRate(InSamplingRate),
		SamplingTime(0),
		NumberOfFrames(InNumberOfFrames),
		SourceSkeleton(InSourceSkeleton),
		AdditiveType(InAdditiveType),
		bAdditiveAnimation(bIsBakedAdditive)
	{}

	// Anim info
	UE_DEPRECATED(5.1, "Direct access to SequenceLength has been deprecated use GetPlayableLength instead")
	float SequenceLength = 0.0f;
	EAnimInterpolationType Interpolation = EAnimInterpolationType::Linear;
	FName AnimName;

	const ICompressedAnimData& CompressedAnimData;

public:
	UE_DEPRECATED(5.1, "Direct access to Time has been deprecated use GetEvaluationTime or GetInterpolatedEvaluationTime instead")
	float Time = 0.0f;
	
	UE_DEPRECATED(5.1, "Direct access to RelativePos has been deprecated use GetRelativePosition instead")
	float RelativePos = 0.0f;

	double GetPlayableLength() const
	{
		return SamplingRate.AsSeconds(NumberOfFrames);
	}
	double GetEvaluationTime() const
	{
		return SamplingRate.AsSeconds(SamplingTime);
	}
	double GetRelativePosition() const
	{
		return FMath::Min(SamplingTime.AsDecimal() / static_cast<double>(NumberOfFrames), 1.0);
	}

	double GetInterpolatedEvaluationTime() const
	{
		const FFrameTime InterpolationTime = Interpolation == EAnimInterpolationType::Step ? SamplingTime.FloorToFrame() : SamplingTime;
		return SamplingRate.AsSeconds(InterpolationTime);
	}

	UE_DEPRECATED(5.1, "Seek with float precision sampling time has been deprecated use double signature instead")
	void Seek(float SampleAtTime)
	{
		Seek(static_cast<double>(SampleAtTime));
	}

	void Seek(double SampleAtTime)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Time = static_cast<float>(SampleAtTime);
		RelativePos = SequenceLength != 0.f ? static_cast<float>(SampleAtTime) / SequenceLength : 0.f;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		SamplingTime = SamplingRate.AsFrameTime(SampleAtTime);
	}

	const USkeleton* GetSourceSkeleton() const { return SourceSkeleton; }
	bool IsAdditiveAnimation() const { return bAdditiveAnimation; }
	EAdditiveAnimationType GetAdditiveType() const { return AdditiveType; }

	const TArrayView<const FTransform>& GetRefLocalPoses() const
	{
		return RefPoses;
	}

	const TArrayView<const FTrackToSkeletonMap>& GetTrackToSkeletonMap() const
	{
		return TrackToSkeletonMap;
	}

protected:
	// Immutable reference/bind pose from the source skeleton, maps a bone index to its reference transform.
	// The same skeleton reference pose must be provided during compression and decompression. This is the
	// source skeleton reference pose (the one assigned to an anim sequence). Re-targeting is taken into account
	// elsewhere. Additionally, if the source skeleton reference pose isn't used, the TrackToSkeletonMap indices
	// may not match the provided pose. This could cause us to read incorrect bone transforms or possibly out
	// of bounds.
	TArrayView<const FTransform> RefPoses;

	// Immutable array that maps from a track index to a bone index. This is built during compression using
	// the source skeleton (the one assigned to an anim sequence).
	TArrayView<const FTrackToSkeletonMap> TrackToSkeletonMap;

	FFrameRate SamplingRate;
	FFrameTime SamplingTime;
	int32 NumberOfFrames = 0;
	const USkeleton* SourceSkeleton = nullptr;
	EAdditiveAnimationType AdditiveType = AAT_None;
	bool bAdditiveAnimation = false;
};

