// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "AnimSingleNodeInstanceProxy.generated.h"

struct FAnimSingleNodeInstanceProxy;
class UMirrorDataTable;

/** 
 * Local anim node for extensible processing. 
 * Cant be used outside of this context as it has no graph node counterpart 
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_SingleNode : public FAnimNode_Base
{
	friend struct FAnimSingleNodeInstanceProxy;

	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	// Slot to use if we are evaluating a montage
	FName ActiveMontageSlot;

	// FAnimNode_Base interface
	ENGINE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	ENGINE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

private:
	/** Parent proxy */
	FAnimSingleNodeInstanceProxy* Proxy;
};

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FAnimSingleNodeInstanceProxy : public FAnimInstanceProxy
{
	friend struct FAnimNode_SingleNode;

	GENERATED_BODY()

public:
	FAnimSingleNodeInstanceProxy()
	{
#if WITH_EDITOR
		bCanProcessAdditiveAnimations = false;
		bIgnoreRootLock = false;
		bIgnoreRootMotion = false;
#endif
	}

	FAnimSingleNodeInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
		, CurrentAsset(nullptr)
		, MirrorDataTable(nullptr)
		, BlendSpacePosition(0.0f, 0.0f, 0.0f)
		, CurrentTime(0.0f)
		, DeltaTimeRecord()
#if WITH_EDITORONLY_DATA
		, PreviewPoseCurrentTime(0.0f)
#endif
		, PlayRate(1.f)
		, bLooping(true)
		, bPlaying(true)
		, bReverse(false)
	{
		SingleNode.Proxy = this;

#if WITH_EDITOR
		bCanProcessAdditiveAnimations = false;
		bIgnoreRootLock = false;
		bIgnoreRootMotion = false;
#endif
	}

	ENGINE_API virtual ~FAnimSingleNodeInstanceProxy();

	ENGINE_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	ENGINE_API virtual bool Evaluate(FPoseContext& Output) override;
	ENGINE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	ENGINE_API virtual void PostUpdate(UAnimInstance* InAnimInstance) const override;
	ENGINE_API virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	ENGINE_API virtual void InitializeObjects(UAnimInstance* InAnimInstance) override;
	ENGINE_API virtual void ClearObjects() override;

	void SetPlaying(bool bIsPlaying)
	{
		bPlaying = bIsPlaying;
	}

	bool IsPlaying() const
	{
		return bPlaying;
	}

	bool IsReverse() const
	{
		return bReverse;
	}

	void SetLooping(bool bIsLooping)
	{
		bLooping = bIsLooping;
	}

	bool IsLooping() const
	{
		return bLooping;
	}

	void ResetWeightInfo()
	{
		WeightInfo.Reset();
	}

	ENGINE_API virtual void SetAnimationAsset(UAnimationAsset* NewAsset, USkeletalMeshComponent* MeshComponent, bool bIsLooping, float InPlayRate);

	ENGINE_API void UpdateBlendspaceSamples(FVector InBlendInput);

	void SetCurrentTime(float InCurrentTime)
	{
		if (InCurrentTime != CurrentTime)
		{
			// If the current time is changed externally then our record of where we are in relation to markers will be
			// out of sync, so reset it and it will be updated when necessary.
			MarkerTickRecord.Reset();
		}
		CurrentTime = InCurrentTime;
	}

	float GetCurrentTime() const
	{
		return CurrentTime;
	}

	float GetPlayRate() const
	{
		return PlayRate;
	}

	void SetPlayRate(float InPlayRate)
	{
		PlayRate = InPlayRate;
	}

	FVector GetFilterLastOutput()
	{
		return BlendFilter.GetFilterLastOutput();
	}

	ENGINE_API void SetReverse(bool bInReverse);

	/** Sets the target blend space position */
	ENGINE_API void SetBlendSpacePosition(const FVector& InPosition);

	/**
	 * Returns the current target/requested blend space position, and the filtered (smoothed) position.
	 */
	ENGINE_API void GetBlendSpaceState(FVector& OutPosition, FVector& OutFilteredPosition) const;

	/** 
	* Returns the length (seconds), not including any rate multipliers, calculated by 
	* weighting the currently active samples 
	*/
	ENGINE_API float GetBlendSpaceLength() const;

#if WITH_EDITOR
	bool CanProcessAdditiveAnimations() const
	{
		return bCanProcessAdditiveAnimations;
	}
#endif

	ENGINE_API void SetMirrorDataTable(const UMirrorDataTable* InMirrorDataTable);

	ENGINE_API const UMirrorDataTable* GetMirrorDataTable();

#if WITH_EDITORONLY_DATA
	ENGINE_API void PropagatePreviewCurve(FPoseContext& Output);
#endif // WITH_EDITORONLY_DATA

	ENGINE_API void SetPreviewCurveOverride(const FName& PoseName, float Value, bool bRemoveIfZero);

	// Update internal weight structures for supplied slot name
	ENGINE_API void UpdateMontageWeightForSlot(const FName CurrentSlotNodeName, float InGlobalNodeWeight);

	// Set the montage slot to preview
	ENGINE_API void SetMontagePreviewSlot(FName PreviewSlot);

private:
	ENGINE_API void InternalBlendSpaceEvaluatePose(class UBlendSpace* BlendSpace, TArray<FBlendSampleData>& BlendSampleDataCache, FPoseContext& OutContext);

protected:
#if WITH_EDITOR
	/** If this is being used by a user (for instance on a skeletal mesh placed in a level) we don't want to allow
	additives. But we need to be able to override this for editor preview windows */
	bool bCanProcessAdditiveAnimations;

	/** Allows editor previews to ignore root lock and show root motion in the editor, regardless of setting on sequence*/
	bool bIgnoreRootLock;

	/** Allows editor previews to ignore root motion in the editor, regardless of setting on sequence*/
	bool bIgnoreRootMotion;
#endif

	/** Pose Weight value that can override curve data. In the future, we'd like to have UCurveSet that can play by default**/
	TMap<FName, float> PreviewCurveOverride;

	/** Current Asset being played. Note that this will be nullptr outside of pre/post update **/
	UAnimationAsset* CurrentAsset;

	/** If set the result will be mirrored using the table */ 
	const UMirrorDataTable* MirrorDataTable;
	
	/** The internal anim node that does our processing */
	FAnimNode_SingleNode SingleNode;

private:
	/** Random cached values to play each asset **/
	FVector BlendSpacePosition;

	/** Random cached values to play each asset **/
	TArray<FBlendSampleData> BlendSampleData;

	/** Random cached values to play each asset **/
	FBlendFilter BlendFilter;

	/** Slot node weight transient data */
	FSlotNodeWeightInfo WeightInfo;

	/** Shared parameters for previewing blendspace or animsequence **/
	float CurrentTime;

	FDeltaTimeRecord DeltaTimeRecord;

#if WITH_EDITORONLY_DATA
	float PreviewPoseCurrentTime;
#endif

	/** Cache for data needed during marker sync */
	FMarkerTickRecord MarkerTickRecord;

	float PlayRate;
	bool bLooping;
	bool bPlaying;
	bool bReverse;
};
