// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimStats.h:	Animation stats
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

/** Skeletal stats */
DECLARE_CYCLE_STAT_EXTERN(TEXT("SkinnedMeshComp Tick"), STAT_SkinnedMeshCompTick, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("TickUpdateRate"), STAT_TickUpdateRate, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Tick Time"), STAT_AnimTickTime, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("StateMachine Update"), STAT_AnimStateMachineUpdate, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("StateMachine Find Transition"), STAT_AnimStateMachineFindTransition, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RefreshBoneTransforms"), STAT_RefreshBoneTransforms, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Post Anim Evaluation"), STAT_PostAnimEvaluation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trigger Notifies"), STAT_AnimTriggerAnimNotifies, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Anim Decompression"), STAT_GetAnimationPose, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("InterpolateSkippedFrames"), STAT_InterpolateSkippedFrames, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateKinematicBonesToAnim"), STAT_UpdateRBBones, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateRBJointsMotors"), STAT_UpdateRBJoints, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("FinalizeAnimationUpdate"), STAT_FinalizeAnimationUpdate, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SkelComp UpdateTransform"), STAT_SkelCompUpdateTransform, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("MeshObject Update"), STAT_MeshObjectUpdate, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calc SkelMesh Bounds"), STAT_CalcSkelMeshBounds, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BlendInPhysics"), STAT_BlendInPhysics, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SkinPerPolyVertices);"), STAT_SkinPerPolyVertices, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateTriMeshVertices"), STAT_UpdateTriMeshVertices, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("AnimGameThreadTime"), STAT_AnimGameThreadTime, STATGROUP_Anim, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("PreUpdateAnimation"), STAT_PreUpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateAnimation"), STAT_UpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PostUpdateAnimation"), STAT_PostUpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BlueprintUpdateAnimation"), STAT_BlueprintUpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("NativeUpdateAnimation"), STAT_NativeUpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("NativeThreadSafeUpdateAnimation"), STAT_NativeThreadSafeUpdateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BlueprintPostEvaluateAnimation"), STAT_BlueprintPostEvaluateAnimation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("MontageAdvance"), STAT_Montage_Advance, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("MontageUpdateWeight"), STAT_Montage_UpdateWeight, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateCurves"), STAT_UpdateCurves, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateCurvesToEvaluationContext"), STAT_UpdateCurvesToEvaluationContext, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateCurvesPostEvaluation"), STAT_UpdateCurvesPostEvaluation, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("TickAssetPlayerInstances"), STAT_TickAssetPlayerInstances, STATGROUP_Anim, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("TickAssetPlayerInstance"), STAT_TickAssetPlayerInstance, STATGROUP_Anim, );

#if STATS
	// Set to 1 for verbose anim node profiling
	#define ANIMNODE_STATS_VERBOSE 0
#else
	#define ANIMNODE_STATS_VERBOSE 0
#endif 

#define DO_ANIMSTAT_PROCESSING(StatName) DECLARE_CYCLE_STAT_EXTERN(TEXT(#StatName), STAT_ ## StatName, STATGROUP_Anim, ENGINE_API)
#include "Animation/AnimMTStats.h"
#undef DO_ANIMSTAT_PROCESSING

#define DO_ANIMSTAT_PROCESSING(StatName) DECLARE_CYCLE_STAT_EXTERN(TEXT(#StatName) TEXT("_WorkerThread"), STAT_ ## StatName ## _WorkerThread, STATGROUP_Anim, ENGINE_API)
#include "Animation/AnimMTStats.h"
#undef DO_ANIMSTAT_PROCESSING

#if STATS
#define ANIM_MT_SCOPE_CYCLE_COUNTER(StatName, bIsMultithreaded) \
	TStatId CycleCountID_##StatName = (bIsMultithreaded ? GET_STATID(STAT_ ## StatName ## _WorkerThread) : GET_STATID(STAT_ ## StatName)); \
	FScopeCycleCounter CycleCount_##StatName(CycleCountID_##StatName);
	#if ANIMNODE_STATS_VERBOSE
		#define ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(StateName, bIsMultithreaded) ANIM_MT_SCOPE_CYCLE_COUNTER(StateName, bIsMultithreaded)
	#else
		#define ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(StateName, bIsMultithreaded)
	#endif
#else
	#define ANIM_MT_SCOPE_CYCLE_COUNTER(StatName, bIsMultithreaded)
	#define ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(StateName, bIsMultithreaded)
#endif
