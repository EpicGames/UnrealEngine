// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSubsystem.generated.h"

class UAnimInstance;
struct FAnimSubsystem;
struct FAnimSubsystemInstance;
struct FAnimInstanceProxy;
class IAnimClassInterface;

struct FAnimSubsystemContext
{
	FAnimSubsystemContext(const FAnimSubsystem& InSubsystem, const UScriptStruct* InSubsystemStruct)
        : Subsystem(InSubsystem)
        , SubsystemStruct(InSubsystemStruct)
	{}
	
	const FAnimSubsystem& Subsystem;
	const UScriptStruct* SubsystemStruct;
};

struct FAnimSubsystemInstanceContext
{
	FAnimSubsystemInstanceContext(const FAnimSubsystem& InSubsystem, const UScriptStruct* InSubsystemStruct, FAnimSubsystemInstance& InSubsystemInstance, const UScriptStruct* InSubsystemInstanceStruct)
        : Subsystem(InSubsystem)
        , SubsystemStruct(InSubsystemStruct)
        , SubsystemInstance(InSubsystemInstance)
        , SubsystemInstanceStruct(InSubsystemInstanceStruct)
	{}
	
	const FAnimSubsystem& Subsystem;
	const UScriptStruct* SubsystemStruct;
	FAnimSubsystemInstance& SubsystemInstance;
	const UScriptStruct* SubsystemInstanceStruct;
};

struct FAnimSubsystemUpdateContext
{
	FAnimSubsystemUpdateContext(const FAnimSubsystemInstanceContext& InContext, UAnimInstance* InAnimInstance, float InDeltaTime)
		: InnerContext(InContext)
		, AnimInstance(InAnimInstance)
		, DeltaTime(InDeltaTime)
	{}

	const FAnimSubsystemInstanceContext& InnerContext;
	UAnimInstance* AnimInstance;
	float DeltaTime;
};

struct FAnimSubsystemParallelUpdateContext
{
	FAnimSubsystemParallelUpdateContext(const FAnimSubsystemInstanceContext& InContext, FAnimInstanceProxy& InProxy, float InDeltaTime)
        : InnerContext(InContext)
        , Proxy(InProxy)
        , DeltaTime(InDeltaTime)
	{}

	const FAnimSubsystemInstanceContext& InnerContext;
	FAnimInstanceProxy& Proxy;
	float DeltaTime;
};

struct FAnimSubsystemPostLoadContext
{
	FAnimSubsystemPostLoadContext(const FAnimSubsystemContext& InContext, IAnimClassInterface& InAnimClassInterface)
        : InnerContext(InContext)
        , AnimClassInterface(InAnimClassInterface)
	{}

	const FAnimSubsystemContext& InnerContext;
	IAnimClassInterface& AnimClassInterface;
};

struct FAnimSubsystemPostLoadDefaultsContext
{
	FAnimSubsystemPostLoadDefaultsContext(const FAnimSubsystemInstanceContext& InContext, UObject* InDefaultAnimInstance)
        : InnerContext(InContext)
        , DefaultAnimInstance(InDefaultAnimInstance)
	{}

	const FAnimSubsystemInstanceContext& InnerContext;
	UObject* DefaultAnimInstance;
};

struct FAnimSubsystemLinkContext
{
	FAnimSubsystemLinkContext(const FAnimSubsystemContext& InContext, IAnimClassInterface& InAnimClassInterface)
		: InnerContext(InContext)
		, AnimClassInterface(InAnimClassInterface)
	{}

	const FAnimSubsystemContext& InnerContext;
	IAnimClassInterface& AnimClassInterface;
};

/** Base structure for all anim subsystem class data */
USTRUCT()
struct FAnimSubsystem
{
	GENERATED_BODY()

	/** Virtual destructor */
	virtual ~FAnimSubsystem() = default;

	/** Override point to process game-thread data per-frame. Called before event-graph-related work (e.g. NativeUpdateAnimation and BlueprintUpdateAnimation) */
	virtual void OnPreUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const {}

	/** Override point to process game-thread data per-frame. Called after event-graph-related work (e.g. NativeUpdateAnimation and BlueprintUpdateAnimation) */
	virtual void OnPostUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const {}

	/** Override point to process worker-thread data per-frame. Called before proxy Update and BlueprintThreadSafeUpdateAnimation. */
	virtual void OnPreUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const {}

	/** Override point to process worker-thread data per-frame. Called after proxy Update and BlueprintThreadSafeUpdateAnimation. */
	virtual void OnPostUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const {}	

	/** Override point to perform subsystem instance data initialization post-load */
	virtual void OnPostLoadDefaults(FAnimSubsystemPostLoadDefaultsContext& InContext) {}
	
	/** Override point to perform subsystem class data initialization post-load */
	virtual void OnPostLoad(FAnimSubsystemPostLoadContext& InContext) {}

#if WITH_EDITORONLY_DATA
	/** Override point to perform subsystem class data initialization on class link */
	virtual void OnLink(FAnimSubsystemLinkContext& InContext) {}
#endif
};