// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSubsystemInstance.generated.h"

/** Base structure for all anim subsystem instance data */
USTRUCT()
struct FAnimSubsystemInstance
{
	GENERATED_BODY()

	virtual ~FAnimSubsystemInstance() = default;

	// Override point used to initialize per-instance data. Called on a worker thread.
	virtual void Initialize_WorkerThread() {}
};
