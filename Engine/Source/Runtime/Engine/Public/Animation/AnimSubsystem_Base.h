// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSubsystem.h"
#include "Animation/ExposedValueHandler.h"
#include "AnimSubsystem_Base.generated.h"

/** Provides common built-in anim node functionality as a subsystem */
USTRUCT()
struct FAnimSubsystem_Base : public FAnimSubsystem
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_Base;

	// Override copy/copy-assignment to avoid racing base class PatchValueHandlers with copying
	// derived class sparse class data
	FAnimSubsystem_Base() = default;
	ENGINE_API FAnimSubsystem_Base(const FAnimSubsystem_Base& Other);
	ENGINE_API FAnimSubsystem_Base& operator =(const FAnimSubsystem_Base& Other);
	
	// FAnimSubsystem interface
	ENGINE_API virtual void OnPostLoadDefaults(FAnimSubsystemPostLoadDefaultsContext& InContext) override;

	const TArray<FExposedValueHandler>& GetExposedValueHandlers() const { return ExposedValueHandlers; }

	ENGINE_API void PatchValueHandlers(UClass* InClass);

private:
	mutable FCriticalSection ValueHandlersCriticalSection;
	TArray<FExposedValueHandler> ExposedValueHandlers;
};
