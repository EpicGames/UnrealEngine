// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimSubsystem.h"
#include "Containers/ArrayView.h"
#include "AnimSubsystem_BlendSpaceGraph.generated.h"

class UBlendSpace;

USTRUCT()
struct FAnimSubsystem_BlendSpaceGraph : public FAnimSubsystem
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_BlendSpaceGraph;

	/** Access the property access library */
	TArrayView<const TObjectPtr<UBlendSpace>> GetBlendSpaces() const { return BlendSpaces; }

private:
	// Any internal blendspaces we host
	UPROPERTY()
	TArray<TObjectPtr<UBlendSpace>> BlendSpaces;
};
