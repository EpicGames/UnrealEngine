// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSubsystemInstance.h"
#include "Animation/AnimTypes.h"
#include "AnimSubsystem_NodeRelevancy.generated.h"

struct FAnimationUpdateContext;
struct FAnimNode_Base;
class UAnimInstance;

// Relevancy status of a animation node tracked by this subsystem
struct FAnimNodeRelevancyStatus
{
public:
	// Get the current weight
	ENGINE_API float GetCurrentWeight() const;

	// Get the previous weight
	ENGINE_API float GetPreviousWeight() const;
	
	// Node has zero weight
	ENGINE_API bool HasFullyBlendedOut() const;

	// Node has just become relevant
	bool HasJustBecomeRelevant() const { return IsStartingBlendingIn(); }
	
	// Node has weight and had zero weight last update
	ENGINE_API bool IsStartingBlendingIn() const;

	// Node has full weight and had non-full weight last update
	ENGINE_API bool IsFinishingBlendingIn() const;

	// Node has full weight and had full weight last update
	ENGINE_API bool HasFullyBlendedIn() const;

	// Node has non-full weight and had full weight last update
	ENGINE_API bool IsStartingBlendingOut() const;

	// Node has zero weight and had non-zero weight last update
	ENGINE_API bool IsFinishingBlendingOut() const;

private:
	friend struct FAnimSubsystemInstance_NodeRelevancy;
	
	// Previous weight 
	float PreviousWeight = 0.0f;

	// Current weight
	float CurrentWeight = 0.0f;
};

// Initialization status of a node
enum class EAnimNodeInitializationStatus
{
	NotUpdated = 0,
	
	InitialUpdate,

	Updated,
};

/** Allows anim node's relevancy (when they receive and lose weight in the graph) to be tracked */
USTRUCT()
struct FAnimSubsystemInstance_NodeRelevancy : public FAnimSubsystemInstance
{
	GENERATED_BODY()

	// FAnimSubsystemInstance interface
	ENGINE_API virtual void Initialize_WorkerThread() override;
	
	// Update the relevancy of the passed-in node using the supplied context
	ENGINE_API FAnimNodeRelevancyStatus UpdateNodeRelevancy(const FAnimationUpdateContext& InContext, const FAnimNode_Base& InNode);

	// Get the tracked relevancy of the passed-in node. If the node is not tracked the relevancy will be default (zero weighted).
	ENGINE_API FAnimNodeRelevancyStatus GetNodeRelevancy(const FAnimNode_Base& InNode) const;

	// Update the initialization state of the passed-in node using the supplied context
	ENGINE_API EAnimNodeInitializationStatus UpdateNodeInitializationStatus(const FAnimationUpdateContext& InContext, const FAnimNode_Base& InNode);

	// Get the tracked initialization state of the passed-in node. If the node is not tracked the initialization state will be default (NotUpdated)
	ENGINE_API EAnimNodeInitializationStatus GetNodeInitializationStatus(const FAnimNode_Base& InNode) const;
	
private:
	// Tracks the relevancy of a node
	struct FTracker
	{
		// Counter used to determine relevancy
		FGraphTraversalCounter Counter;

		// Status of the node's relevancy
		FAnimNodeRelevancyStatus Status;
	};

private:
	// Map of tracked anim nodes
	TMap<const FAnimNode_Base*, FTracker> NodeTrackers;

	// Map of tracked anim nodes initialization
	TMap<const FAnimNode_Base*, EAnimNodeInitializationStatus> NodeInitTrackers;
};
