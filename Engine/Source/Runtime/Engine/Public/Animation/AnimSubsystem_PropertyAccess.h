// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimSubsystem.h"
#include "PropertyAccess.h"
#include "AnimSubsystem_PropertyAccess.generated.h"

// The various call sites we can call into the property access library
UENUM()
enum class EAnimPropertyAccessCallSite
{
	// Access is made on a worker thread in the anim graph or in a BP function
	WorkerThread_Unbatched UMETA(DisplayName="Thread Safe"),

	// Access is made on a worker thread before BlueprintThreadSafeUpdateAnimation is run
	WorkerThread_Batched_PreEventGraph UMETA(DisplayName="Pre-Thread Safe Update Animation"),

	// Access is made on a worker thread after BlueprintThreadSafeUpdateAnimation is run
	WorkerThread_Batched_PostEventGraph UMETA(DisplayName="Post-Thread Safe Update Animation"),
	
	// Access is made on the game thread before the event graph (and NativeUpdateAnimation) is run
	GameThread_Batched_PreEventGraph UMETA(DisplayName="Pre-Event Graph"),

	// Access is made on the game thread after the event graph (and NativeUpdateAnimation) is run
	GameThread_Batched_PostEventGraph UMETA(DisplayName="Post-Event Graph"),
};

USTRUCT()
struct FAnimSubsystem_PropertyAccess : public FAnimSubsystem
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_PropertyAccess;

	/** FAnimSubsystem interface */
	ENGINE_API virtual void OnPreUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const override;
	ENGINE_API virtual void OnPostUpdate_GameThread(FAnimSubsystemUpdateContext& InContext) const override;
	ENGINE_API virtual void OnPreUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const override;
	ENGINE_API virtual void OnPostUpdate_WorkerThread(FAnimSubsystemParallelUpdateContext& InContext) const override;
	ENGINE_API virtual void OnPostLoad(FAnimSubsystemPostLoadContext& InContext) override;
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void OnLink(FAnimSubsystemLinkContext& InContext) override;
#endif

	/** Access the property access library */
	const FPropertyAccessLibrary& GetLibrary() const { return Library; }

private:
	UPROPERTY()
	FPropertyAccessLibrary Library;
};
