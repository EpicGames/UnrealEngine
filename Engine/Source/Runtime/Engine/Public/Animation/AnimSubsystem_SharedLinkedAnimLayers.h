// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimInstance.h"
#include "Templates/SubclassOf.h"
#include "Animation/AnimSubsystemInstance.h"
#include "AnimSubsystem_SharedLinkedAnimLayers.generated.h"

class UAnimInstance;
class USkeletalMeshComponent;

// Define to 1 for more extensive runtime validation of linked anim layers data
#define LINKEDANIMLAYERSDATA_INTEGRITYCHECKS 0

// Linked layer instance info
USTRUCT()
struct FLinkedAnimLayerInstanceData
{
	GENERATED_USTRUCT_BODY()

	FLinkedAnimLayerInstanceData() {}
	FLinkedAnimLayerInstanceData(UAnimInstance* AnimInstance, bool bInIsPersistent) { Instance = AnimInstance; bIsPersistent = bInIsPersistent; }

	bool IsPersistent() const { return bIsPersistent; }
	void SetPersistence(bool bInIsPersistent) { bIsPersistent = bInIsPersistent; }
	
	// Mark a function as linked
	void AddLinkedFunction(FName Function, UAnimInstance* AnimInstance);
	// Unmark a function as linked
	void RemoveLinkedFunction(FName Function);

	// Return map of currently linked functions
	const TMap<FName, TWeakObjectPtr<UAnimInstance>>& GetLinkedFunctions() const {return LinkedFunctions;}
	
	UPROPERTY(Transient)
	TObjectPtr<UAnimInstance> Instance;

private:
	
	UPROPERTY(Transient)
	TMap<FName, TWeakObjectPtr<UAnimInstance>> LinkedFunctions;

	// True if this is a persistent instance that remains alive when unlinked
	bool bIsPersistent = false;
};

// Linked layer class info
USTRUCT()
struct FLinkedAnimLayerClassData
{
	GENERATED_BODY()

	FLinkedAnimLayerClassData() {}
	FLinkedAnimLayerClassData(TSubclassOf<UAnimInstance> AnimClass, bool bInIsPersistent) { Class = AnimClass; bIsPersistent = bInIsPersistent; }

	bool IsPersistent() const { return bIsPersistent; }
	void SetPersistence(bool bInIsPersistent);

	// Find instance data for given instance if it exists, returns nullptr otherwise
	FLinkedAnimLayerInstanceData* FindInstanceData(const UAnimInstance* AnimInstance);

	// Find an existing instance to link given function if one is available, creates one otherwise
	UAnimInstance* FindOrAddInstanceForLinking(UAnimInstance* OwningInstance, FName Function, bool& IsNewInstance);
	
	TSubclassOf<UAnimInstance> GetClass() const {return Class;}
	const TArray<FLinkedAnimLayerInstanceData>& GetInstancesData() const {return InstancesData;}

	void RemoveLinkedFunction(UAnimInstance* AnimInstance, FName Function);

private:
	// Add given instance
	FLinkedAnimLayerInstanceData& AddInstance(UAnimInstance* AnimInstance);
	// Remove given instance
	void RemoveInstance(UAnimInstance* AnimInstance);

	TSubclassOf<UAnimInstance> Class;

	UPROPERTY(Transient)
	TArray<FLinkedAnimLayerInstanceData> InstancesData;

	// If true, one instance will be kept alive when unlinked so that we don't recreate it from scratch next time it's linked
	bool bIsPersistent;
};

// Data for shared linked anim instances module
USTRUCT()
struct FAnimSubsystem_SharedLinkedAnimLayers : public FAnimSubsystemInstance
{
	GENERATED_BODY()

	// Retrieve subsystem from skeletal mesh component
	static ENGINE_API FAnimSubsystem_SharedLinkedAnimLayers* GetFromMesh(USkeletalMeshComponent* SkelMesh);
	
	// Clear all linked layers data
	ENGINE_API void Reset();

	// Check if a given anim instance is shared
	bool IsSharedInstance(const UAnimInstance* AnimInstance) { return FindInstanceData(AnimInstance) != nullptr; }

	const TArray<FLinkedAnimLayerClassData>& GetClassesData() const { return ClassesData; }

	// Add a class to the persistent class array, insuring one instance is kept alive even when unlinked
	void AddPersistentAnimLayerClass(TSubclassOf<UAnimInstance> AnimInstanceClass) { PersistentClasses.AddUnique(AnimInstanceClass); }

	// Remove a class from the persistent class array. 
	ENGINE_API void RemovePersistentAnimLayerClass(TSubclassOf<UAnimInstance> AnimInstanceClass);

	// Add a linked function to be managed by a shared anim instance. Returns the instance to bind to the function
	ENGINE_API UAnimInstance* AddLinkedFunction(UAnimInstance* OwningInstance, TSubclassOf<UAnimInstance> AnimClass, FName Function, bool& bIsNewInstance);

	// Remove a linked function, cleaning the instance if it's not used anymore and not persistent
	ENGINE_API void RemoveLinkedFunction(UAnimInstance* AnimInstance, FName Function);

private:
	// Find instance data for given instance if it exists, returns nullptr otherwise
	ENGINE_API FLinkedAnimLayerInstanceData* FindInstanceData(const UAnimInstance* AnimInstance);

	// Find class data for given anim class if it exists, returns nullptr otherwise
	ENGINE_API FLinkedAnimLayerClassData* FindClassData(TSubclassOf<UAnimInstance> AnimClass);

	// Find or add ClassData for given AnimClass
	ENGINE_API FLinkedAnimLayerClassData& FindOrAddClassData(TSubclassOf<UAnimInstance> AnimClass);

	UPROPERTY(Transient)
	TArray<FLinkedAnimLayerClassData> ClassesData;

	// Anim instance classes that should be kept alive even when unlinked
	UPROPERTY(Transient)
	TArray<TSubclassOf<UAnimInstance>> PersistentClasses;
};
