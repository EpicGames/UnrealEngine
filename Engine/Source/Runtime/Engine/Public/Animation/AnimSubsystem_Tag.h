// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSubsystem.h"
#include "AnimSubsystem_Tag.generated.h"

struct FAnimNode_Base;
class UAnimInstance;

/** Allows anim nodes to be tagged by name at edit time and retrieved by tag at runtime */
USTRUCT()
struct FAnimSubsystem_Tag : public FAnimSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Find a node's index by tag.
	 * @param	InTag		The name tag for the node to find
	 * @return INDEX_NONE if the tagged node was not found
	 */	
	ENGINE_API int32 FindNodeIndexByTag(FName InTag) const;
	
	/**
	 * Find a node by tag.
	 * @param	InTag		The name tag for the node to find
	 * @param	InInstance	The instance that the node is in
	 * @return nullptr if the tagged node was not found
	 */
	template<typename NodeType>
	NodeType* FindNodeByTag(FName InTag, UAnimInstance* InInstance) const
	{
		return static_cast<NodeType*>(FindNodeByTag_Internal(InTag, InInstance, NodeType::StaticStruct()));
	}

	/**
	 * Find a node by tag (const version)
	 * @param	InTag		The name tag for the node to find
	 * @param	InInstance	The instance that the node is in
	 * @return nullptr if the tagged node was not found
	 */	
	template<typename NodeType>
	const NodeType* FindNodeByTag(FName InTag, const UAnimInstance* InInstance) const
	{
		return static_cast<const NodeType*>(FindNodeByTag_Internal(InTag, const_cast<UAnimInstance*>(InInstance), NodeType::StaticStruct()));
	}
	
private:
	// FAnimSubsystem interface
	ENGINE_API virtual void OnPostLoadDefaults(FAnimSubsystemPostLoadDefaultsContext& InContext) override;

	// Internal impl for FindNodeByTag
	ENGINE_API FAnimNode_Base* FindNodeByTag_Internal(FName InTag, UAnimInstance* InInstance, UScriptStruct* InNodeType) const;
	
private:
	friend class UAnimBlueprintExtension_Tag;
	
	// Map of name->node index
	UPROPERTY()
	TMap<FName, int32> NodeIndices;

	// Anim class that contains this subsystem
	IAnimClassInterface* AnimClass = nullptr;
};
