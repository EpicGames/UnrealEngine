// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"
#include "Animation/AnimationAsset.h"

struct FAnimationBaseContext;
struct FAnimInstanceProxy;
class UAnimInstance;
class UMirrorDataTable; 

namespace UE { namespace Anim {

// Parameters for group/marker-based sync 
struct FAnimSyncParams
{
	FAnimSyncParams(FName InGroupName = NAME_None, EAnimGroupRole::Type InRole = EAnimGroupRole::CanBeLeader, EAnimSyncMethod InMethod = EAnimSyncMethod::DoNotSync, bool InOverridePositionWhenJoiningSyncGroupAsLeader = false)
		: GroupName(InGroupName)
		, Role(InRole)
		, Method(InMethod)
		, bOverridePositionWhenJoiningSyncGroupAsLeader(InOverridePositionWhenJoiningSyncGroupAsLeader)
	{}
	
	FName GroupName = NAME_None;
	EAnimGroupRole::Type Role = EAnimGroupRole::CanBeLeader;
	EAnimSyncMethod Method = EAnimSyncMethod::DoNotSync;
	bool bOverridePositionWhenJoiningSyncGroupAsLeader = false;
};

// Wraps up functionality for ticking and syncing animations according to group (via normalized time) or marker
struct FAnimSync
{
	static ENGINE_API const FName Attribute;

	using FSyncGroupMap = TMap<FName, FAnimGroupInstance>;

	// Reset internal buffers ready for writing
	ENGINE_API void Reset();

	// Completely reset the sync system back to its starting state
	ENGINE_API void ResetAll();
	
	// Adds a tick record in the list for the correct group or the ungrouped array.
	ENGINE_API void AddTickRecord(const FAnimTickRecord& InTickRecord, const FAnimSyncParams& InSyncParams = FAnimSyncParams());

	ENGINE_API void SetMirror(const UMirrorDataTable* MirrorTable);

	// Tick all of the asset player tick records that are registered with us
	ENGINE_API void TickAssetPlayerInstances(FAnimInstanceProxy& InProxy, float InDeltaSeconds);

	ENGINE_API bool GetTimeToClosestMarker(FName SyncGroup, FName MarkerName, float& OutMarkerTime) const;

	ENGINE_API bool HasMarkerBeenHitThisFrame(FName SyncGroup, FName MarkerName) const;

	ENGINE_API bool IsSyncGroupBetweenMarkers(FName InSyncGroupName, FName PreviousMarker, FName NextMarker, bool bRespectMarkerOrder = true) const;

	ENGINE_API FMarkerSyncAnimPosition GetSyncGroupPosition(FName InSyncGroupName) const;

	ENGINE_API bool IsSyncGroupValid(FName InSyncGroupName) const;

	// flip sync group read/write indices
	void TickSyncGroupWriteIndex()
	{ 
		SyncGroupWriteIndex = GetSyncGroupReadIndex();
	}

	// Gets the sync group we should be reading from
	int32 GetSyncGroupReadIndex() const 
	{ 
		return 1 - SyncGroupWriteIndex; 
	}

	// Gets the sync group we should be writing to
	int32 GetSyncGroupWriteIndex() const 
	{ 
		return SyncGroupWriteIndex; 
	}

	/** Get the sync group we are currently reading from */
	const FSyncGroupMap& GetSyncGroupMapRead() const
	{ 
		return SyncGroupMaps[GetSyncGroupReadIndex()]; 
	}

	/** Get the ungrouped active player we are currently reading from */
	const TArray<FAnimTickRecord>& GetUngroupedActivePlayersRead() 
	{ 
		return UngroupedActivePlayerArrays[GetSyncGroupReadIndex()]; 
	}

	UE_DEPRECATED(5.0, "Legacy API support - do not use")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecord(FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName);

	UE_DEPRECATED(5.0, "Legacy API support - do not use")
	ENGINE_API FAnimTickRecord& CreateUninitializedTickRecordInScope(FAnimInstanceProxy& InProxy, FAnimGroupInstance*& OutSyncGroupPtr, FName GroupName, EAnimSyncGroupScope Scope);

	// GC support
	ENGINE_API void AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector);

private:
	/** The list of animation assets which are going to be evaluated this frame and need to be ticked (ungrouped) */
	TArray<FAnimTickRecord> UngroupedActivePlayerArrays[2];

	/** The set of tick groups for this anim instance */
	FSyncGroupMap SyncGroupMaps[2];
		
	/** If not null, the sync markers will be mirrored based on the data in the table **/ 
	const UMirrorDataTable* MirrorDataTable = nullptr; 

	/** Current sync group buffer index */
	int32 SyncGroupWriteIndex = 0;
};

}}	// namespace UE::Anim
