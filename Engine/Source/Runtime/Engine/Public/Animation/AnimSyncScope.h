// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSync.h"

struct FAnimationBaseContext;
struct FAnimInstanceProxy;

namespace UE { namespace Anim {

// Debug info for a sync group scope request
struct FAnimSyncDebugInfo
{
	FAnimSyncDebugInfo() = default;

	ENGINE_API FAnimSyncDebugInfo(const FAnimationBaseContext& InContext);

	FAnimSyncDebugInfo(FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId)
		: SourceProxy(&InSourceProxy)
		, SourceNodeId(InSourceNodeId)
	{}

	FAnimInstanceProxy* SourceProxy = nullptr;
	int32 SourceNodeId = INDEX_NONE;
};

// Scoped graph message used to synchronize animations at various points in an anim graph
class FAnimSyncGroupScope : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FAnimSyncGroupScope, ENGINE_API);

public:
	ENGINE_API FAnimSyncGroupScope(const FAnimationBaseContext& InContext, FName InSyncGroup = NAME_None, EAnimGroupRole::Type InGroupRole = EAnimGroupRole::CanBeLeader);

	// Adds a tick record in the list for the correct group or the ungrouped array.
	ENGINE_API void AddTickRecord(const FAnimTickRecord& InTickRecord, const FAnimSyncParams& InSyncParams = FAnimSyncParams(), const FAnimSyncDebugInfo& InDebugInfo = FAnimSyncDebugInfo());

	// Set a mirror table that any tick records added to the group will use
	ENGINE_API void SetMirror(const UMirrorDataTable* MirrorDataTable);

private:
	// The node ID that was used when this scope was entered
	int32 NodeId;

	// The proxy that we are running
	FAnimInstanceProxy& Proxy;

	// The outer proxy, if any, to forward tick records to
	FAnimInstanceProxy* OuterProxy;

	// The sync group that this scope uses.
	FName SyncGroup;

	// The role assumed by the specified sync group
	EAnimGroupRole::Type GroupRole;
};

}}	// namespace UE::Anim
