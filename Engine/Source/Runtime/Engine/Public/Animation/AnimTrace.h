// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "ObjectTrace.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "Animation/AnimTraceDefines.h"

#if ANIM_TRACE_ENABLED

#include "Animation/AnimAttributes.h"
#include "BoneIndices.h"
#include "Math/Color.h"
#include "Math/Vector.h"

UE_TRACE_CHANNEL_EXTERN(AnimationChannel, ENGINE_API);

class FAutoConsoleVariable;
class UAnimInstance;
class UPoseWatchPoseElement;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FAnimationBaseContext;
struct FAnimationCacheBonesContext;
struct FAnimationInitializeContext;
struct FAnimationUpdateContext;
struct FAnimInstanceProxy;
struct FAnimMontageInstance;
struct FAnimNode_SequencePlayerBase;
struct FAnimNotifyEvent;
struct FAnimSyncMarker;
struct FAnimTickRecord;
struct FBlendedHeapCurve;
struct FComponentSpacePoseContext;
struct FPassedMarker;
struct FPoseContext;

extern ENGINE_API FAutoConsoleVariable CVarRecordExternalMorphTargets;

struct FAnimTrace
{
	/** The various phases of anim graph processing */
	enum class EPhase : uint8
	{
		Initialize = 0,
		PreUpdate = 1,
		Update = 2,
		CacheBones = 3,
		Evaluate = 4,
	};

	/** The various events called on notifies */
	enum class ENotifyEventType : uint8
	{
		Event = 0,
		Begin = 1,
		End = 2,
		Tick = 3,
		SyncMarker = 4	// We 'fake' sync markers with a notify type for convenience
	};
	
	enum class EInertializationType : uint8
    {
    	Inertialization = 0,
    	DeadBlending = 1
    };

	/** Helper for outputting anim nodes */
	struct FScopedAnimNodeTrace
	{
		FScopedAnimNodeTrace(const FAnimationInitializeContext& InContext);
		FScopedAnimNodeTrace(const FAnimationUpdateContext& InContext);
		FScopedAnimNodeTrace(const FAnimationCacheBonesContext& InContext);
		FScopedAnimNodeTrace(const FPoseContext& InContext);
		FScopedAnimNodeTrace(const FComponentSpacePoseContext& InContext);

		~FScopedAnimNodeTrace();

		const FAnimationBaseContext& Context;
	};

	/** Helper for outputting anim graphs */
	struct FScopedAnimGraphTrace
	{
		FScopedAnimGraphTrace(const FAnimationInitializeContext& InContext);
		FScopedAnimGraphTrace(const FAnimationUpdateContext& InContext);
		FScopedAnimGraphTrace(const FAnimationCacheBonesContext& InContext);
		FScopedAnimGraphTrace(const FPoseContext& InContext);
		FScopedAnimGraphTrace(const FComponentSpacePoseContext& InContext);

		~FScopedAnimGraphTrace();

		uint64 StartCycle;
		const FAnimationBaseContext& Context;
		EPhase Phase;
	};

	/** Helper for suspending anim node tracing */
	struct FScopedAnimNodeTraceSuspend
	{
		FScopedAnimNodeTraceSuspend();
		~FScopedAnimNodeTraceSuspend();
	};

	/** Describes a debug line output to the world */
	struct FDebugLine
	{
		FDebugLine(const FVector& InStartLocation, const FVector& InEndLocation, const FColor& InColor, bool bInPersistentLines = false, float InLifeTime = -1.0f, float InThickness = 0.0f)
			: StartLocation(InStartLocation)
			, EndLocation(InEndLocation)
			, Color(InColor)
			, LifeTime(InLifeTime)
			, Thickness(InThickness)
			, bPersistentLines(bInPersistentLines)
		{}

		FVector StartLocation;
		FVector EndLocation;
		FColor Color;
		float LifeTime;
		float Thickness;
		bool bPersistentLines;
	};

	/** Reset Caches so a new trace can be started*/
	ENGINE_API static void Reset();

	/** Helper function to output a tick record */
	ENGINE_API FORCENOINLINE static void OutputAnimTickRecord(const FAnimationBaseContext& InContext, const FAnimTickRecord& InTickRecord);

	/** Helper function to output a skeletal mesh */
	ENGINE_API FORCENOINLINE static void OutputSkeletalMesh(const USkeletalMesh* InMesh);

	/** Helper function to output a skeletal mesh pose, curves etc. */
	ENGINE_API FORCENOINLINE static void OutputSkeletalMeshComponent(const USkeletalMeshComponent* InComponent);

	/** Helper function to output a skeletal mesh frame marker */
	ENGINE_API FORCENOINLINE static void OutputSkeletalMeshFrame(const USkeletalMeshComponent* InComponent);

	/** Helper function to output an anim graph's execution event */
	ENGINE_API FORCENOINLINE static void OutputAnimGraph(const FAnimationBaseContext& InContext, uint64 InStartCycle, uint64 InEndCycle, uint8 InPhase);

	/** Helper function to output an anim node's execution event */
	ENGINE_API FORCENOINLINE static void OutputAnimNodeStart(const FAnimationBaseContext& InContext, uint64 InStartCycle, int32 InPreviousNodeId, int32 InNodeId, float InBlendWeight, float InRootMotionWeight, uint8 InPhase);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeEnd(const FAnimationBaseContext& InContext, uint64 InEndCycle);

	/** Output the current set of attributes in the supplied context */
	ENGINE_API FORCENOINLINE static void OutputAnimNodeAttribute(const FAnimInstanceProxy& InTargetProxy, const FAnimInstanceProxy& InSourceProxy, int32 InTargetNodeId, int32 InSourceNodeId, FName InAttribute);

	/** Output the current set of attributes in the supplied context */
	template<typename ContextType>
	FORCENOINLINE static void OutputAnimNodeBlendableAttributes(const ContextType& InContext, int32 InTargetNodeId, int32 InSourceNodeId)
	{
		if(InContext.CustomAttributes.ContainsData())
		{
			OutputAnimNodeAttribute(*InContext.AnimInstanceProxy, *InContext.AnimInstanceProxy, InTargetNodeId, InSourceNodeId, UE::Anim::FAttributes::Attributes);
		}

		if(InContext.Curve.Num() > 0)
		{
			OutputAnimNodeAttribute(*InContext.AnimInstanceProxy, *InContext.AnimInstanceProxy, InTargetNodeId, InSourceNodeId, UE::Anim::FAttributes::Curves);
		}
	}

	/** Helper function to output a tracked value for an anim node */
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, bool InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, int32 InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, float InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FVector2D& InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FVector& InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FRotator& InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const FName& InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const TCHAR* InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const UClass* InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValue(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, const UObject* InValue);
	ENGINE_API FORCENOINLINE static void OutputAnimNodeValueAnimNode(const FAnimationBaseContext& InContext, uint32 NodeIndex, const TCHAR* InKey, int32 InValue, const UObject* InValueAnimInstanceId);

	/** Helper function to output debug info for sequence player nodes */
	ENGINE_API static void OutputAnimSequencePlayer(const FAnimationBaseContext& InContext, const FAnimNode_SequencePlayerBase& InNode);

	/** 
	 * Helper function to output a name to the trace stream, referenced by ID. 
	 * @return the ID used to reference the name
	 */
	ENGINE_API static uint32 OutputName(const FName& InName);

	/** Helper function to output a state machine state's info */
	ENGINE_API static void OutputStateMachineState(const FAnimationBaseContext& InContext, int32 InStateMachineIndex, int32 InStateIndex, float InStateWeight, float InElapsedTime);

	/** Helper function to output an anim notify event */
	ENGINE_API static void OutputAnimNotify(UAnimInstance* InAnimInstance, const FAnimNotifyEvent& InNotifyEvent, ENotifyEventType InEventType);

	/** Helper function to output an anim sync marker event */
	ENGINE_API static void OutputAnimSyncMarker(UAnimInstance* InAnimInstance, const FPassedMarker& InPassedSyncMarker);

	/** Helper function to output a montage instance's info */
	ENGINE_API static void OutputMontage(UAnimInstance* InAnimInstance, const FAnimMontageInstance& InMontageInstance);

	/** Helper function to output a sync record */
	ENGINE_API static void OutputSync(const FAnimInstanceProxy& InSourceProxy, int32 InSourceNodeId, FName InGroupName);

	/** Helper function to output a pose watch record */
	ENGINE_API static void OutputPoseWatch(const FAnimInstanceProxy& InSourceProxy, UPoseWatchPoseElement* InPoseWatchElement, int32 InPoseWatchId, const TArray<FTransform>& BoneTransforms, const FBlendedHeapCurve& InCurves, const TArray<FBoneIndexType>& RequiredBones, const FTransform& WorldTransform, const bool bIsEnabled);
	
	/** Helper function to output inertialization state */
	ENGINE_API static void OutputInertialization(const FAnimInstanceProxy& InSourceProxy, int32 InNodeId, float InWeight, EInertializationType InType);
};

#define TRACE_ANIM_TICK_RECORD(Context, TickRecord) \
	FAnimTrace::OutputAnimTickRecord(Context, TickRecord);

#define TRACE_SKELETAL_MESH(Mesh) \
	FAnimTrace::OutputSkeletalMesh(Mesh);

#define TRACE_SKELETAL_MESH_COMPONENT(Component) \
	FAnimTrace::OutputSkeletalMeshComponent(Component);

#define TRACE_SKELETALMESH_FRAME(Component) \
	FAnimTrace::OutputSkeletalMeshFrame(Component);

#define TRACE_SCOPED_ANIM_GRAPH(Context) \
	FAnimTrace::FScopedAnimGraphTrace _ScopedAnimGraphTrace(Context);

#define TRACE_SCOPED_ANIM_NODE(Context) \
	FAnimTrace::FScopedAnimNodeTrace _ScopedAnimNodeTrace(Context);

#define TRACE_ANIM_NODE_ATTRIBUTE(TargetProxy, SourceProxy, TargetNodeId, SourceNodeId, Name) \
	FAnimTrace::OutputAnimNodeAttribute(TargetProxy, SourceProxy, TargetNodeId, SourceNodeId, Name);

#define TRACE_ANIM_NODE_BLENDABLE_ATTRIBUTES(Context, TargetNodeId, SourceNodeId) \
	FAnimTrace::OutputAnimNodeBlendableAttributes(Context, TargetNodeId, SourceNodeId);

#define TRACE_SCOPED_ANIM_NODE_SUSPEND \
	FAnimTrace::FScopedAnimNodeTraceSuspend _ScopedAnimNodeTraceSuspend;

#define TRACE_ANIM_NODE_VALUE(Context, Key, Value) \
	FAnimTrace::OutputAnimNodeValue(Context, Context.GetCurrentNodeId(), Key, Value);

#define TRACE_ANIM_NODE_VALUE_WITH_ID(Context, NodeId, Key, Value) \
	FAnimTrace::OutputAnimNodeValue(Context, NodeId, Key, Value);

#define TRACE_ANIM_NODE_VALUE_WITH_ID_ANIM_NODE(Context, NodeId, Key, Value, ValueAnimInstanceId) \
	FAnimTrace::OutputAnimNodeValueAnimNode(Context, NodeId, Key, Value, ValueAnimInstanceId);

#define TRACE_ANIM_SEQUENCE_PLAYER(Context, Node) \
	FAnimTrace::OutputAnimSequencePlayer(Context, Node);

#define TRACE_ANIM_STATE_MACHINE_STATE(Context, StateMachineIndex, StateIndex, StateWeight, ElapsedTime) \
	FAnimTrace::OutputStateMachineState(Context, StateMachineIndex, StateIndex, StateWeight, ElapsedTime);

#define TRACE_ANIM_NOTIFY(AnimInstance, NotifyEvent, EventType) \
	FAnimTrace::OutputAnimNotify(AnimInstance, NotifyEvent, FAnimTrace::ENotifyEventType::EventType);

#define TRACE_ANIM_SYNC_MARKER(AnimInstance, SyncMarker) \
	FAnimTrace::OutputAnimSyncMarker(AnimInstance, SyncMarker);

#define TRACE_ANIM_MONTAGE(AnimInstance, MontageInstance) \
	FAnimTrace::OutputMontage(AnimInstance, MontageInstance);

#define TRACE_ANIM_NODE_SYNC(SourceProxy, SourceNodeId, GroupName) \
	FAnimTrace::OutputSync(SourceProxy, SourceNodeId, GroupName);

#define TRACE_ANIM_POSE_WATCH(SourceProxy, PoseWatchElement, PoseWatchId, BoneTransforms, Curves, RequiredBones, WorldTransform, bIsEnabled) \
	FAnimTrace::OutputPoseWatch(SourceProxy, PoseWatchElement, PoseWatchId, BoneTransforms, Curves, RequiredBones, WorldTransform, bIsEnabled);

#define TRACE_ANIM_INERTIALIZATION(SourceProxy, NodeId, Weight, Type) \
	FAnimTrace::OutputInertialization(SourceProxy, NodeId, Weight, Type);

#else

#define TRACE_ANIM_TICK_RECORD(Context, TickRecord)
#define TRACE_SKELETAL_MESH(Mesh)
#define TRACE_SKELETAL_MESH_COMPONENT(Component)
#define TRACE_SKELETALMESH_FRAME(Component)
#define TRACE_SCOPED_ANIM_GRAPH(Context)
#define TRACE_SCOPED_ANIM_NODE(Context)
#define TRACE_ANIM_NODE_ATTRIBUTE(TargetProxy, SourceProxy, TargetNodeId, SourceNodeId, Name)
#define TRACE_ANIM_NODE_BLENDABLE_ATTRIBUTES(Context, TargetNodeId, SourceNodeId)
#define TRACE_SCOPED_ANIM_NODE_SUSPEND
#define TRACE_ANIM_NODE_VALUE(Context, Key, Value)
#define TRACE_ANIM_NODE_VALUE_WITH_ID(Context, NodeId, Key, Value)
#define TRACE_ANIM_NODE_VALUE_WITH_ID_ANIM_NODE(Context, NodeId, Key, Value, ValueAnimInstanceId)
#define TRACE_ANIM_SEQUENCE_PLAYER(Context, Node)
#define TRACE_ANIM_STATE_MACHINE_STATE(Context, StateMachineIndex, StateIndex, StateWeight, ElapsedTime)
#define TRACE_ANIM_NOTIFY(AnimInstance, NotifyEvent, EventType)
#define TRACE_ANIM_SYNC_MARKER(AnimInstance, SyncMarker)
#define TRACE_ANIM_MONTAGE(AnimInstance, MontageInstance)
#define TRACE_ANIM_NODE_SYNC(SourceProxy, SourceNodeId, GroupName)
#define TRACE_ANIM_POSE_WATCH(SourceProxy, PoseWatchElement, PoseWatchId, BoneTransforms, Curves, RequiredBones, WorldTransform, bIsEnabled)
#define TRACE_ANIM_INERTIALIZATION(SourceProxy, NodeId, Weight, Type)

#endif
