// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectTraceDefines.h"

#ifndef ANIM_TRACE_ENABLED
#define ANIM_TRACE_ENABLED (OBJECT_TRACE_ENABLED && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif