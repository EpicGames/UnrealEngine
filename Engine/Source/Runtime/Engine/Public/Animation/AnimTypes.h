// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/MemStack.h"
#include "Algo/Transform.h"
#include "Animation/AnimLinkableElement.h"
#include "Animation/AnimEnums.h"
#include "Misc/SecureHash.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "AnimTypes.generated.h"

struct FMarkerPair;
struct FMarkerSyncAnimPosition;
struct FPassedMarker;

class FMemoryReader;
class FMemoryWriter;
class UMirrorDataTable;

// Disable debugging information for shipping and test builds.
#define ENABLE_ANIM_DEBUG (1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#define DEFAULT_SAMPLERATE			30.f
#define MINIMUM_ANIMATION_LENGTH	(1/DEFAULT_SAMPLERATE)

// Enable this if you want to locally measure detailed anim perf.  Disabled by default as it is introduces a lot of additional profile markers and associated overhead.
#define ENABLE_VERBOSE_ANIM_PERF_TRACKING 0

#define MAX_ANIMATION_TRACKS 65535

namespace EAnimEventTriggerOffsets
{
	enum Type
	{
		OffsetBefore,
		OffsetAfter,
		NoOffset
	};
}

ENGINE_API float GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::Type OffsetType);

/** Enum for specifying a specific axis of a bone */
UENUM()
enum EBoneAxis : int
{
	BA_X UMETA(DisplayName = "X Axis"),
	BA_Y UMETA(DisplayName = "Y Axis"),
	BA_Z UMETA(DisplayName = "Z Axis"),
};


/** Enum for controlling which reference frame a controller is applied in. */
UENUM(BlueprintType)
enum EBoneControlSpace : int
{
	/** Set absolute position of bone in world space. */
	BCS_WorldSpace UMETA(DisplayName = "World Space"),
	/** Set position of bone in SkeletalMeshComponent's reference frame. */
	BCS_ComponentSpace UMETA(DisplayName = "Component Space"),
	/** Set position of bone relative to parent bone. */
	BCS_ParentBoneSpace UMETA(DisplayName = "Parent Bone Space"),
	/** Set position of bone in its own reference frame. */
	BCS_BoneSpace UMETA(DisplayName = "Bone Space"),
	BCS_MAX,
};

/** Enum for specifying the source of a bone's rotation. */
UENUM()
enum EBoneRotationSource : int
{
	/** Don't change rotation at all. */
	BRS_KeepComponentSpaceRotation UMETA(DisplayName = "No Change (Preserve Existing Component Space Rotation)"),
	/** Keep forward direction vector relative to the parent bone. */
	BRS_KeepLocalSpaceRotation UMETA(DisplayName = "Maintain Local Rotation Relative to Parent"),
	/** Copy rotation of target to bone. */
	BRS_CopyFromTarget UMETA(DisplayName = "Copy Target Rotation"),
};

/** Ticking method for AnimNotifies in AnimMontages. */
UENUM()
namespace EMontageNotifyTickType
{
	enum Type : int
	{
		/** Queue notifies, and trigger them at the end of the evaluation phase (faster). Not suitable for changing sections or montage position. */
		Queued,
		/** Trigger notifies as they are encountered (Slower). Suitable for changing sections or montage position. */
		BranchingPoint,
	};
}

/** Filtering method for deciding whether to trigger a notify. */
UENUM()
namespace ENotifyFilterType
{
	enum Type : int
	{
		/** No filtering. */
		NoFiltering,

		/** Filter By Skeletal Mesh LOD. */
		LOD,
	};
}

USTRUCT(BlueprintType)
struct FPerBoneBlendWeight
{
	GENERATED_USTRUCT_BODY()

	/** Source index of the buffer. */
	UPROPERTY()
	int32 SourceIndex;

	UPROPERTY()
	float BlendWeight;

	FPerBoneBlendWeight()
		: SourceIndex(0)
		, BlendWeight(0.0f)
	{
	}
};

USTRUCT(BlueprintType)
struct FPerBoneBlendWeights
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FPerBoneBlendWeight> BoneBlendWeights;


	FPerBoneBlendWeights() {}

};

struct FGraphTraversalCounter
{
private:
	// Number of the last external frame where we updated the counter (total global frame count)
	uint64 LastSyncronizedFrame;

	// Number of counter increments since last reset
	int16 Counter;

	// Number of external frames that URO is currently scheduled to skip
	int16 SkipFrames;

public:

	FGraphTraversalCounter()
		: LastSyncronizedFrame(INDEX_NONE)
		, Counter(INDEX_NONE)
		, SkipFrames(0)

	{}

	int16 Get() const
	{
		return Counter;
	}

	bool HasEverBeenUpdated() const
	{
		return (Counter != INDEX_NONE) && (LastSyncronizedFrame != INDEX_NONE);
	}

	/** Increases the internal counter, and refreshes the current global frame counter */
	void Increment()
	{
		Counter++;
		LastSyncronizedFrame = GFrameCounter;

		// Avoid wrapping over back to INDEX_NONE, as this means 'never been traversed'
		if (Counter == INDEX_NONE)
		{
			Counter++;
		}
	}

	/** Sets the number of expected frames to be skipped by URO */
	void SetMaxSkippedFrames(int16 InMaxSkippedFrames)
	{
		SkipFrames = InMaxSkippedFrames;
	}

	/** Gets the number of expected frames to be skipped by URO */
	int16 GetMaxSkippedFrames() const
	{
		return SkipFrames;
	}

	/** Clear the internal counters and frame skip */
	void Reset()
	{
		Counter = INDEX_NONE;
		LastSyncronizedFrame = INDEX_NONE;
		SkipFrames = 0;
	}

	/** Sync this counter with another counter */
	void SynchronizeWith(const FGraphTraversalCounter& InMasterCounter)
	{
		Counter = InMasterCounter.Counter;
		LastSyncronizedFrame = InMasterCounter.LastSyncronizedFrame;
		SkipFrames = InMasterCounter.SkipFrames;
	}

	/** Check whether the internal counter is synchronized between this and another counter */
	bool IsSynchronized_Counter(const FGraphTraversalCounter& InOtherCounter) const
	{
		return HasEverBeenUpdated() && Counter == InOtherCounter.Counter;
	}

	/** Check whether this counter and another were synchronized on the same global frame */
	bool IsSynchronized_Frame(const FGraphTraversalCounter& InOtherCounter) const
	{
		return HasEverBeenUpdated() && LastSyncronizedFrame == InOtherCounter.LastSyncronizedFrame;
	}

	/** Check that both the internal counter and global frame are both in sync between this counter and another */
	bool IsSynchronized_All(const FGraphTraversalCounter& InOtherCounter) const
	{
		return HasEverBeenUpdated() && Counter == InOtherCounter.Counter && LastSyncronizedFrame == InOtherCounter.LastSyncronizedFrame;
	}

	/** Check if this counter is either synchronized with another or is one update behind */
	bool WasSynchronizedCounter(const FGraphTraversalCounter& InOtherCounter) const
	{
		// Test if we're currently in sync with our master counter
		if(IsSynchronized_Counter(InOtherCounter))
		{
			return true;
		}

		// If not, test if the Master Counter is a frame ahead of us
		FGraphTraversalCounter TestCounter(*this);
		TestCounter.Increment();

		return TestCounter.IsSynchronized_Counter(InOtherCounter);
	}

	/** 
	 * Check whether this counter and another were either synchronized this global frame or were
	 * synced one frame interval previously. A frame interval is the size of SkipFrames. This means
	 * that if under URO a number of frames are skipped, this check will consider counters from the
	 * last frame before the skip and the first frame after the skip as "was synchronized"
	 *
	 * Use this to test whether or not there's been a period when the owning animation instance has
	 * not been updated.
	 */
	bool WasSynchronizedLastFrame(const FGraphTraversalCounter& InOtherCounter) const
	{
		// Test if we're currently in sync with our master counter
		if(IsSynchronized_Frame(InOtherCounter))
		{
			return true;
		}

		// If we aren't synced then check if we were synced last frame
		return LastSyncronizedFrame >= (GFrameCounter - SkipFrames) - 1;
	}
};

/**
 * Triggers an animation notify.  Each AnimNotifyEvent contains an AnimNotify object
 * which has its Notify method called and passed to the animation.
 */
USTRUCT(BlueprintType)
struct FAnimNotifyEvent : public FAnimLinkableElement
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	/** The user requested time for this notify */
	UPROPERTY()
	float DisplayTime_DEPRECATED;
#endif

	/** An offset from the DisplayTime to the actual time we will trigger the notify, as we cannot always trigger it exactly at the time the user wants */
	UPROPERTY()
	float TriggerTimeOffset;

	/** An offset similar to TriggerTimeOffset but used for the end scrub handle of a notify state's duration */
	UPROPERTY()
	float EndTriggerTimeOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnimNotifyEvent)
	float TriggerWeightThreshold;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AnimNotifyEvent)
	FName NotifyName;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category=AnimNotifyEvent)
	TObjectPtr<class UAnimNotify>  Notify;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadWrite, Category=AnimNotifyEvent)
	TObjectPtr<class UAnimNotifyState>  NotifyStateClass;

	UPROPERTY()
	float Duration;

	/** Linkable element to use for the end handle representing a notify state duration */
	UPROPERTY()
	FAnimLinkableElement EndLink;

	/** If TRUE, this notify has been converted from an old BranchingPoint. */
	UPROPERTY()
	bool bConvertedFromBranchingPoint;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnimNotifyEvent)
	TEnumAsByte<EMontageNotifyTickType::Type> MontageTickType;

	/** Defines the chance of this notify triggering, 0 = No Chance, 1 = Always triggers */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float NotifyTriggerChance;

	/** Defines a method for filtering notifies (stopping them triggering) e.g. by looking at the meshes current LOD */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings)
	TEnumAsByte<ENotifyFilterType::Type> NotifyFilterType;

	/** LOD to start filtering this notify from.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings, meta = (ClampMin = "0"))
	int32 NotifyFilterLOD;

	/** Allow notify event to be filtered if requested at runtime (e. g. via an Anim Graph Message) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings)
	bool bCanBeFilteredViaRequest;
	
	/** If disabled this notify will be skipped on dedicated servers */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings)
	bool bTriggerOnDedicatedServer;

	/** If enabled this notify will trigger when the animation is a follower in a sync group (by default only the sync group leaders notifies trigger */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AnimNotifyTriggerSettings)
	bool bTriggerOnFollower;

#if WITH_EDITORONLY_DATA
	/** Color of Notify in editor */
	UPROPERTY()
	FColor NotifyColor;

	/** Guid for tracking notifies in editor */
	UPROPERTY(meta = (ExcludeFromHash))
	FGuid Guid;
#endif // WITH_EDITORONLY_DATA

	/** 'Track' that the notify exists on, used for visual placement in editor and sorting priority in runtime */
	UPROPERTY()
	int32 TrackIndex;

private:
	/** NotifyName appended to "AnimNotify_" */
	mutable FName CachedNotifyEventName;

	/** NotifyName used to generate CachedNotifyEventName, for invalidation */
	mutable FName CachedNotifyEventBaseName;

public:
	FAnimNotifyEvent()
		: FAnimLinkableElement()
#if WITH_EDITORONLY_DATA
		, DisplayTime_DEPRECATED(0)
#endif
		, TriggerTimeOffset(0)
		, EndTriggerTimeOffset(0)
		, TriggerWeightThreshold(ZERO_ANIMWEIGHT_THRESH)
		, Notify(nullptr)
		, NotifyStateClass(nullptr)
		, Duration(0)
		, bConvertedFromBranchingPoint(false)
		, MontageTickType(EMontageNotifyTickType::Queued)
		, NotifyTriggerChance(1.f)
		, NotifyFilterType(ENotifyFilterType::NoFiltering)
		, NotifyFilterLOD(0)
		, bCanBeFilteredViaRequest(true)
		, bTriggerOnDedicatedServer(true)
		, bTriggerOnFollower(false)
#if WITH_EDITORONLY_DATA
		, NotifyColor(FColor::Black)
#endif // WITH_EDITORONLY_DATA
		, TrackIndex(0)
		, CachedNotifyEventName(NAME_None)
		, CachedNotifyEventBaseName(NAME_None)
	{
	}

	virtual ~FAnimNotifyEvent() = default;

#if WITH_EDITORONLY_DATA
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif

	/** Updates trigger offset based on a combination of predicted offset and current offset */
	ENGINE_API void RefreshTriggerOffset(EAnimEventTriggerOffsets::Type PredictedOffsetType);

	/** Updates end trigger offset based on a combination of predicted offset and current offset */
	ENGINE_API void RefreshEndTriggerOffset(EAnimEventTriggerOffsets::Type PredictedOffsetType);

	/** Returns the actual trigger time for this notify. In some cases this may be different to the DisplayTime that has been set */
	ENGINE_API float GetTriggerTime() const;

	/** Returns the actual end time for a state notify. In some cases this may be different from DisplayTime + Duration */
	ENGINE_API float GetEndTriggerTime() const;

	ENGINE_API float GetDuration() const;

	ENGINE_API void SetDuration(float NewDuration);

	/** Returns true is this AnimNotify is a BranchingPoint */
	ENGINE_API bool IsBranchingPoint() const;

	/** Returns true if this is blueprint or native class-based notify (i.e. not a named notify) */
	bool IsBlueprintNotify() const
	{
		return Notify != nullptr || NotifyStateClass != nullptr;
	}

	bool operator ==(const FAnimNotifyEvent& Other) const
	{
		return(
			(Notify && Notify == Other.Notify) || 
			(NotifyStateClass && NotifyStateClass == Other.NotifyStateClass) ||
			(!IsBlueprintNotify() && NotifyName == Other.NotifyName)
			);
	}

	/** This can be used with the Sort() function on a TArray of FAnimNotifyEvents to sort the notifies array by time, earliest first. */
	ENGINE_API bool operator <(const FAnimNotifyEvent& Other) const;

	ENGINE_API virtual void SetTime(float NewTime, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) override;

	/** Get the NotifyName pre-appended to "AnimNotify_", for calling the event */
	ENGINE_API FName GetNotifyEventName() const;
	
	/** Get the mirrored NotifyName pre-appended to "AnimNotify_", for calling the event. If Notify is not mirrored returns result of GetNotifyEventName()*/
	ENGINE_API FName GetNotifyEventName(const UMirrorDataTable* MirrorDataTable) const;
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FAnimNotifyEvent> : public TStructOpsTypeTraitsBase2<FAnimNotifyEvent>
{
	enum 
	{ 
		WithSerializer = true,
		WithPostSerialize = true
	};
};
#endif

// Used by UAnimSequenceBase::SortNotifies() to sort its Notifies array
inline bool FAnimNotifyEvent::operator<(const FAnimNotifyEvent& Other) const
{
	float ATime = GetTriggerTime();
	float BTime = Other.GetTriggerTime();

	// When we've got the same time sort on the track index. Explicitly
	// using SMALL_NUMBER here incase the underlying default changes as
	// notifies can have an offset of KINDA_SMALL_NUMBER to be consider
	// distinct
	if (FMath::IsNearlyEqual(ATime, BTime, UE_SMALL_NUMBER))
	{
		return TrackIndex < Other.TrackIndex;
	}
	else
	{
		return ATime < BTime;
	}
}

USTRUCT(BlueprintType)
struct FAnimSyncMarker
{
	GENERATED_BODY()

	FAnimSyncMarker()
		: MarkerName(NAME_None)
		, Time(0.0f)
#if WITH_EDITORONLY_DATA
		, TrackIndex(0)
#endif
	{}

	// The name of this marker
	UPROPERTY(BlueprintReadOnly, Category=Animation)
	FName MarkerName;

	// Time in seconds of this marker
	UPROPERTY(BlueprintReadOnly, Category = Animation)
	float Time;

#if WITH_EDITORONLY_DATA
	// The editor track this marker sits on
	UPROPERTY()
	int32 TrackIndex;

	UPROPERTY()
	FGuid Guid;
#endif
	
#if WITH_EDITORONLY_DATA
	ENGINE_API bool Serialize(FArchive& Ar);
#endif

	/** This can be used with the Sort() function on a TArray of FAnimSyncMarker to sort the notifies array by time, earliest first. */
	bool operator <(const FAnimSyncMarker& Other) const { return Time < Other.Time; }

	bool operator ==(const FAnimSyncMarker& Other) const
	{
		return MarkerName == Other.MarkerName &&
#if WITH_EDITORONLY_DATA
			TrackIndex == Other.TrackIndex &&
			Guid == Other.Guid &&
#endif
			Time == Other.Time;
	}
};

#if WITH_EDITORONLY_DATA
template<> struct TStructOpsTypeTraits<FAnimSyncMarker> : public TStructOpsTypeTraitsBase2<FAnimSyncMarker>
{
	enum 
	{ 
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
#endif

/**
 * Keyframe position data for one track.  Pos(i) occurs at Time(i).  Pos.Num() always equals Time.Num().
 */
USTRUCT()
struct FAnimNotifyTrack
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName TrackName;

	UPROPERTY()
	FLinearColor TrackColor;

	TArray<FAnimNotifyEvent*> Notifies;
	
	TArray<FAnimSyncMarker*> SyncMarkers;

	FAnimNotifyTrack()
		: TrackName(TEXT(""))
		, TrackColor(FLinearColor::White)
	{
	}

	FAnimNotifyTrack(FName InTrackName, FLinearColor InTrackColor)
		: TrackName(InTrackName)
		, TrackColor(InTrackColor)
	{
	}
};

/** 
 * Indicates whether an animation is additive, and what kind.
 */
UENUM()
enum EAdditiveAnimationType : int
{
	/** No additive. */
	AAT_None  UMETA(DisplayName="No additive"),
	/* Create Additive from LocalSpace Base. */
	AAT_LocalSpaceBase UMETA(DisplayName="Local Space"),
	/* Create Additive from MeshSpace Rotation Only, Translation still will be LocalSpace. */
	AAT_RotationOffsetMeshSpace UMETA(DisplayName="Mesh Space"),
	AAT_MAX,
};

UENUM()
namespace ECurveBlendOption
{
	enum Type : int
	{
		/* Last pose that contains valid curve value override it. */
		Override, // redirect from MaxWeight old legacy behavior
		/** Only set the value if the previous pose doesn't have the curve value. */
		DoNotOverride, 
		/** Normalize By Sum of Weight and use it to blend. */
		NormalizeByWeight,
		/** Blend By Weight without normalizing*/
		BlendByWeight, 
		/** Use Base Pose for all curve values. Do not blend */
		UseBasePose,
		/** Find the highest curve value from multiple poses and use that. */
		UseMaxValue,
		/** Find the lowest curve value from multiple poses and use that. */
		UseMinValue,
	};
}

/**
 * Slot node weight information - this is transient data that is used by slot node
 */
struct FSlotNodeWeightInfo
{
	/** Weight of Source Branch. This is the weight of the input pose coming from children.
	This is different than (1.f - SourceWeight) since the Slot can play additive animations, which are overlayed on top of the source pose. */
	float SourceWeight;

	/** Weight of Slot Node. Determined by Montages weight playing on this slot */
	float SlotNodeWeight;

	/** Total Weight of Slot Node. Determined by Montages weight playing on this slot, it can be more than 1 */
	float TotalNodeWeight;

	FSlotNodeWeightInfo()
		: SourceWeight(0.f)
		, SlotNodeWeight(0.f)
		, TotalNodeWeight(0.f)
	{}

	void Reset()
	{
		SourceWeight = 0.f;
		SlotNodeWeight = 0.f;
		TotalNodeWeight = 0.f;
	}
};

USTRUCT()
struct FMarkerSyncData
{
	GENERATED_USTRUCT_BODY()

	/** Authored Sync markers */
	UPROPERTY()
	TArray<FAnimSyncMarker>		AuthoredSyncMarkers;

	/** List of Unique marker names in this animation sequence */
	TArray<FName>				UniqueMarkerNames;

	void GetMarkerIndicesForTime(float CurrentTime, bool bLooping, const TArray<FName>& ValidMarkerNames, FMarkerPair& OutPrevMarker, FMarkerPair& OutNextMarker, float SequenceLength) const;
	
	FMarkerSyncAnimPosition GetMarkerSyncPositionFromMarkerIndicies(int32 PrevMarker, int32 NextMarker, float CurrentTime, float SequenceLength, const UMirrorDataTable* MirrorTable = nullptr) const;
	
	void CollectUniqueNames();
	
	UE_DEPRECATED(5.8, "Use flavor of CollectMarkersInRange() that takes in an UMirrorDataTable")
	void CollectMarkersInRange(float PrevPosition, float NewPosition, TArray<FPassedMarker>& OutMarkersPassedThisTick, float TotalDeltaMove);
	void CollectMarkersInRange(float PrevPosition, float NewPosition, TArray<FPassedMarker>& OutMarkersPassedThisTick, float TotalDeltaMove, const UMirrorDataTable* MirrorDataTable);
};

// Shortcut for the allocator used by animation nodes.
typedef TMemStackAllocator<> FAnimStackAllocator;

/** 
 * Structure for all Animation Weight helper functions.
 */
struct FAnimWeight
{
	/** Helper function to determine if a weight is relevant. */
	static inline bool IsRelevant(float InWeight)
	{
		return (InWeight > ZERO_ANIMWEIGHT_THRESH);
	}

	/** Helper function to determine if a normalized weight is considered full weight. */
	static inline bool IsFullWeight(float InWeight)
	{
		return (InWeight >= (1.f - ZERO_ANIMWEIGHT_THRESH));
	}

	/** Get a small relevant weight for ticking */
	static inline float GetSmallestRelevantWeight()
	{
		return 2.f * ZERO_ANIMWEIGHT_THRESH;
	}
};

/** 
 * Indicates how animation should be evaluated between keys.
 */
UENUM(BlueprintType)
enum class EAnimInterpolationType : uint8
{
	/** Linear interpolation when looking up values between keys. */
	Linear		UMETA(DisplayName="Linear"),

	/** Step interpolation when looking up values between keys. */
	Step		UMETA(DisplayName="Step"),
};


/*
 *  Smart name UID type definition
 * 
 * This is used by FSmartNameMapping/FSmartName to identify by number
 * This defines default type and Max number for the type and used all over animation code to identify curve
 * including bone container containing valid UID list for curves
 */
 namespace SmartName
{
	// ID type, should be used to access SmartNames as fundamental type may change.
	typedef uint16 UID_Type;
	// Max UID used for overflow checking
	inline constexpr UID_Type MaxUID = MAX_uint16;
}
/**
 * Animation Key extraction helper as we have a lot of code that messes up the key length
 */
struct FAnimKeyHelper
{
	FAnimKeyHelper(float InLength, int32 InNumKeys)
		: Length(InLength)
		, NumKeys(InNumKeys)
	{}

	float TimePerKey() const
	{
		return (NumKeys > 1) ? Length / (float)(NumKeys - 1) : MINIMUM_ANIMATION_LENGTH;
	}

	// Returns the FPS for the given length and number of keys
	float KeysPerSecond() const
	{
		return NumKeys > 0 ? (float(NumKeys - 1) / Length) : 0.0f;
	}

	int32 LastKey() const
	{
		return (NumKeys > 1) ? NumKeys - 1 : 0;
	}

	float GetLength() const 
	{
		return Length;
	}

	int32 GetNumKeys() const 
	{
		return NumKeys;
	}

private:
	float Length;
	int32 NumKeys;
};

UENUM()
namespace EAxisOption
{
	enum Type : int
	{
		X,
		Y,
		Z,
		X_Neg,
		Y_Neg,
		Z_Neg,
		Custom
	};
}

struct FAxisOption
{
	static FVector GetAxisVector(const TEnumAsByte<EAxisOption::Type> InAxis, const FVector& CustomAxis)
	{
		switch (InAxis)
		{
		case EAxisOption::X:
			return FVector::ForwardVector;
		case EAxisOption::X_Neg:
			return -FVector::ForwardVector;
		case EAxisOption::Y:
			return FVector::RightVector;
		case EAxisOption::Y_Neg:
			return -FVector::RightVector;
		case EAxisOption::Z:
			return FVector::UpVector;
		case EAxisOption::Z_Neg:
			return -FVector::UpVector;
		case EAxisOption::Custom:
			return CustomAxis;
		}

		return FVector::ForwardVector;
	}

	static FVector GetAxisVector(const TEnumAsByte<EAxisOption::Type> InAxis)
	{
		return GetAxisVector(InAxis, FVector::ForwardVector);
	}
};

// The transform component (attribute) to read from
UENUM()
namespace EComponentType
{
	enum Type : int
	{
		None = 0,
		TranslationX,
		TranslationY,
		TranslationZ,
		RotationX,
		RotationY,
		RotationZ,
		Scale UMETA(DisplayName = "Scale (largest component)"),
		ScaleX,
		ScaleY,
		ScaleZ
	};
}

// @note We have a plan to support skeletal hierarchy. When that happens, we'd like to keep skeleton indexing.
USTRUCT()
struct FTrackToSkeletonMap
{
	GENERATED_USTRUCT_BODY()

	// Index of Skeleton.BoneTree this Track belongs to.
	UPROPERTY()
	int32 BoneTreeIndex;


	FTrackToSkeletonMap()
		: BoneTreeIndex(0)
	{
	}

	FTrackToSkeletonMap(int32 InBoneTreeIndex)
		: BoneTreeIndex(InBoneTreeIndex)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FTrackToSkeletonMap &Item)
	{
		return Ar << Item.BoneTreeIndex;
	}
};

/**
* Raw keyframe data for one track.Each array will contain either NumFrames elements or 1 element.
* One element is used as a simple compression scheme where if all keys are the same, they'll be
* reduced to 1 key that is constant over the entire sequence.
*/
USTRUCT(BlueprintType)
struct FRawAnimSequenceTrack
{
	GENERATED_USTRUCT_BODY()

	/** Position keys. */
	UPROPERTY()
	TArray<FVector3f> PosKeys;

	/** Rotation keys. */
	UPROPERTY()
	TArray<FQuat4f> RotKeys;

	/** Scale keys. */
	UPROPERTY()
	TArray<FVector3f> ScaleKeys;

	// Serializer.
	friend FArchive& operator<<(FArchive& Ar, FRawAnimSequenceTrack& T)
	{
		T.PosKeys.BulkSerialize(Ar);
		T.RotKeys.BulkSerialize(Ar);

		if (Ar.UEVer() >= VER_UE4_ANIM_SUPPORT_NONUNIFORM_SCALE_ANIMATION)
		{
			T.ScaleKeys.BulkSerialize(Ar);
		}

		return Ar;
	}

	bool ContainsNaN() const
	{
		bool bContainsNaN = false;

		auto CheckForNan = [&bContainsNaN](const auto& Keys) -> bool
		{
			if (!bContainsNaN)
			{
				for (const auto& Key : Keys)
				{
					if (Key.ContainsNaN())
						return true;
				}

				return false;
			}
		
			return true;
		};

		bContainsNaN = CheckForNan(PosKeys);
		bContainsNaN = CheckForNan(RotKeys);
		bContainsNaN = CheckForNan(ScaleKeys);

		return bContainsNaN;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID); 

		// Previously generated content was saved without bulk array serialization, so have to rely on UProperty serialization until resaved
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RawAnimSequenceTrackSerializer)
		{
			return false;
		}
 
		Ar << *this;
 
		return true;
	}

	static const uint32 SingleKeySize = sizeof(FVector3f) + sizeof(FQuat4f) + sizeof(FVector3f);
};

template<> struct TStructOpsTypeTraits<FRawAnimSequenceTrack> : public TStructOpsTypeTraitsBase2<FRawAnimSequenceTrack>
{
	enum { WithSerializer = true };
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

UCLASS(MinimalAPI)
class URawAnimSequenceTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Returns the positional keys contained by the FRawAnimSequenceTrack	
	*/
	UFUNCTION(BlueprintPure, Category = Animation, meta=(ScriptMethod))
	static TArray<FVector> GetPositionalKeys(UPARAM(ref)const FRawAnimSequenceTrack& Track)
	{
		TArray<FVector> Keys;
		Algo::Transform(Track.PosKeys, Keys, [](FVector3f FloatKey)
		{
			return FVector(FloatKey);
		});

		return Keys;
	}

	/**
	* Returns the rotational keys contained by the FRawAnimSequenceTrack	
	*/
	UFUNCTION(BlueprintPure, Category = Animation, meta=(ScriptMethod))
	static TArray<FQuat> GetRotationalKeys(UPARAM(ref)const FRawAnimSequenceTrack& Track)
	{
		TArray<FQuat> Keys;
		Algo::Transform(Track.RotKeys, Keys, [](FQuat4f FloatKey)
		{
			return FQuat(FloatKey);
		});

		return Keys;
	}

	/**
	* Returns the scale keys contained by the FRawAnimSequenceTrack	
	*/
	UFUNCTION(BlueprintPure, Category = Animation, meta=(ScriptMethod))
	static TArray<FVector> GetScaleKeys(UPARAM(ref)const FRawAnimSequenceTrack& Track)
	{
		TArray<FVector> Keys;
		Algo::Transform(Track.ScaleKeys, Keys, [](FVector3f FloatKey)
		{
			return FVector(FloatKey);
		});

		return Keys;
	}
};


/**
 * Encapsulates commonly useful data about bones.
 */
class FBoneData
{
public:
	/** The bind pose orientation. */
	FQuat		Orientation;

	/** The bind pose position. */
	FVector3f	Position;

	/** The bind pose scale. */
	FVector3f	Scale;

	/** Bone name. */
	FName		Name;

	/** Direct descendants.  Empty for end effectors. */
	TArray<int32> Children;

	/** List of bone indices from parent up to root. */
	TArray<int32>	BonesToRoot;

	/** List of end effectors for which this bone is an ancestor.  End effectors have only one element in this list, themselves. */
	TArray<int32>	EndEffectors;

	/** If a Socket is attached to that bone */
	bool		bHasSocket;

	/** If matched as a Key end effector */
	bool		bKeyEndEffector;

	/**	@return		Index of parent bone; -1 for the root. */
	int32 GetParent() const
	{
		return GetDepth() ? BonesToRoot[0] : -1;
	}

	/**	@return		Distance to root; 0 for the root. */
	int32 GetDepth() const
	{
		return BonesToRoot.Num();
	}

	/** @return		true if this bone is an end effector (has no children). */
	bool IsEndEffector() const
	{
		return Children.Num() == 0;
	}
};

#if 0
template <typename ArrayType>
FGuid GetArrayGuid(const TArray<ArrayType>& Array)
{
	FSHA1 Sha;
	Sha.Update((uint8*)Array.GetData(), Array.Num() * Array.GetTypeSize());

	Sha.Final();

	uint32 Hash[5];
	Sha.GetHash((uint8*)Hash);
	FGuid Guid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	return Guid;
}

template <typename ArrayType>
void DebugLogArray(const TArray<ArrayType>& Array)
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Array: %i %s\n"), Array.Num(), *GetArrayGuid(Array).ToString());
	for (int32 i = 0; i < Array.Num(); ++i)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\t%i %s\n"), i, *Array[i].ToString());
	}
}

template <>
void DebugLogArray(const TArray<FRawAnimSequenceTrack>& RawData);
#endif
