// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __ANIMATIONCOMPRESSION_ISPH__
#define __ANIMATIONCOMPRESSION_ISPH__

#include "Math/Vector.isph"

static const uniform float Quant16BitDiv = 32767.f;
static const uniform float Quant16BitFactor = 32767.f;
static const uniform int Quant16BitOffs = 32767;

static const uniform float Quant10BitDiv = 511.f;
static const uniform float Quant10BitFactor = 511.f;
static const uniform int Quant10BitOffs = 511;

static const uniform float Quant11BitDiv = 1023.f;
static const uniform float Quant11BitFactor = 1023.f;
static const uniform int Quant11BitOffs = 1023;

static const uniform unsigned int XShift = 21;
static const uniform unsigned int YShift = 10;
static const uniform unsigned int ZShift = 21;
static const uniform unsigned int XMask = 0xffe00000;
static const uniform unsigned int YMask = 0x001ffc00;
static const uniform unsigned int ZMask = 0x000003ff;

static const uniform unsigned int IEEEFloat_MantissaBits = 23;
static const uniform unsigned int IEEEFloat_ExponentBits = 8;
static const uniform unsigned int IEEEFloat_SignShift = 31;
static const uniform unsigned int IEEEFloat_ExponentBias = 127;
static const uniform unsigned int IEEEFloat_MantissaMask = 0x007fffff;
static const uniform unsigned int IEEEFloat_ExponentMask = 0x7f800000;
static const uniform unsigned int IEEEFloat_SignMask = 0x80000000;

static const varying float Zero = 0.0f;

enum AnimationCompressionFormat
{
	ACF_None = 0,
	ACF_Float96NoW,
	ACF_Fixed48NoW,
	ACF_IntervalFixed32NoW,
	ACF_Fixed32NoW,
	ACF_Float32NoW,
	ACF_Identity,
	ACF_MAX
};

enum EAnimInterpolationType 
{
	/** Linear interpolation when looking up values between keys. */
	Linear = 0,

	/** Step interpolation when looking up values between keys. */
	Step,
};

struct BoneTrackPair
{
	int32 AtomIndex;
	int32 TrackIndex;
};

/** Each CompresedTranslationData track's ByteStream will be byte swapped in chunks of this size. */
static const uniform int CompressedTranslationStrides[ACF_MAX] =
{
	4,	// ACF_None					(float X, float Y, float Z)
	4,	// ACF_Float96NoW			(float X, float Y, float Z)
	4,	// ACF_Fixed48NoW			(Illegal value for translation)
	4,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	4,	// ACF_Fixed32NoW			(Illegal value for translation)
	4,	// ACF_Float32NoW			(Illegal value for translation)
	0	// ACF_Identity
};

/** Number of swapped chunks per element. */
static const uniform int CompressedTranslationNum[ACF_MAX] =
{
	3,	// ACF_None					(float X, float Y, float Z)
	3,	// ACF_Float96NoW			(float X, float Y, float Z)
	3,	// ACF_Fixed48NoW			(Illegal value for translation)
	1,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	3,	// ACF_Fixed32NoW			(Illegal value for translation)
	3,	// ACF_Float32NoW			(Illegal value for translation)
	0	// ACF_Identity
};

/** Each CompresedRotationData track's ByteStream will be byte swapped in chunks of this size. */
static const uniform int CompressedRotationStrides[ACF_MAX] =
{
	4,	// ACF_None					(FQuats are serialized per element hence sizeof(float) rather than sizeof(FQuat).
	4,	// ACF_Float96NoW			(FQuats with one component dropped and the remaining three uncompressed at 32bit floating point each
	2,	// ACF_Fixed48NoW			(FQuats with one component dropped and the remaining three compressed to 16-16-16 fixed point.
	4,	// ACF_IntervalFixed32NoW	(FQuats with one component dropped and the remaining three compressed to 11-11-10 per-component interval fixed point.
	4,	// ACF_Fixed32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 fixed point.
	4,	// ACF_Float32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 floating point.
	0	// ACF_Identity
};

/** Number of swapped chunks per element. */
static const uniform int CompressedRotationNum[ACF_MAX] =
{
	4,	// ACF_None					(FQuats are serialized per element hence sizeof(float) rather than sizeof(FQuat).
	3,	// ACF_Float96NoW			(FQuats with one component dropped and the remaining three uncompressed at 32bit floating point each
	3,	// ACF_Fixed48NoW			(FQuats with one component dropped and the remaining three compressed to 16-16-16 fixed point.
	1,	// ACF_IntervalFixed32NoW	(FQuats with one component dropped and the remaining three compressed to 11-11-10 per-component interval fixed point.
	1,	// ACF_Fixed32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 fixed point.
	1,  // ACF_Float32NoW			(FQuats with one component dropped and the remaining three compressed to 11-11-10 floating point.
	0	// ACF_Identity
};

/** Number of swapped chunks per element, split out per component (high 3 bits) and flags (low 3 bits)
  *
  * Note: The entry for ACF_IntervalFixed32NoW is special, and actually indicates how many fixed components there are!
  **/
static const uniform unsigned int8 PerTrackNumComponentTable[ACF_MAX * 8] =
{
	4,4,4,4,4,4,4,4,	// ACF_None
	3,1,1,2,1,2,2,3,	// ACF_Float96NoW (0 is special, as uncompressed rotation gets 'mis'encoded with 0 instead of 7, so it's treated as a 3; a genuine 0 would use ACF_Identity)
	3,1,1,2,1,2,2,3,	// ACF_Fixed48NoW (ditto)
	6,2,2,4,2,4,4,6,	// ACF_IntervalFixed32NoW (special, indicates number of interval pairs stored in the fixed track)
	1,1,1,1,1,1,1,1,	// ACF_Fixed32NoW
	1,1,1,1,1,1,1,1,	// ACF_Float32NoW
	0,0,0,0,0,0,0,0		// ACF_Identity
};

/** Each CompresedScaleData track's ByteStream will be byte swapped in chunks of this size. */
static const uniform int CompressedScaleStrides[ACF_MAX] =
{
	4,	// ACF_None					(float X, float Y, float Z)
	4,	// ACF_Float96NoW			(float X, float Y, float Z)
	4,	// ACF_Fixed48NoW			(Illegal value for Scale)
	4,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	4,	// ACF_Fixed32NoW			(Illegal value for Scale)
	4,	// ACF_Float32NoW			(Illegal value for Scale)
	0	// ACF_Identity
};

/** Number of swapped chunks per element. */
static const uniform int CompressedScaleNum[ACF_MAX] =
{
	3,	// ACF_None					(float X, float Y, float Z)
	3,	// ACF_Float96NoW			(float X, float Y, float Z)
	3,	// ACF_Fixed48NoW			(Illegal value for Scale)
	1,	// ACF_IntervalFixed32NoW	(compressed to 11-11-10 per-component interval fixed point)
	3,	// ACF_Fixed32NoW			(Illegal value for Scale)
	3,	// ACF_Float32NoW			(Illegal value for Scale)
	0	// ACF_Identity
};

inline unsigned int8* Align(const unsigned int8* Val, const uniform int Alignment)
{
	return (unsigned int8*)(((unsigned int64)Val + Alignment - 1) & ~(Alignment - 1));
}

/**
 * Utility function to determine the two key indices to interpolate given a relative position in the animation
 *
 * @param	Seq				The UAnimSequence container.
 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
 * @param	NumKeys			The number of keys present in the track being solved.
 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
 */
inline float TimeToIndex(
	const uniform float SequenceLength,
	const uniform float RelativePos,
	int NumKeys,
	const uniform unsigned int8 Interpolation,
	int &PosIndex0Out,
	int &PosIndex1Out)
{
	float Alpha;

	// Check for before-first-frame case.
	if (NumKeys < 2)
	{
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		return 0.0f;
	}
	// Check for before-first-frame case.
	if( RelativePos <= 0.f )
	{
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		Alpha = 0.0f;
	}
	else
	{
		NumKeys -= 1; // never used without the minus one in this case
		// Check for after-last-frame case.
		if( RelativePos >= 1.0f )
		{
			// If we're not looping, key n-1 is the final key.
			PosIndex0Out = NumKeys;
			PosIndex1Out = NumKeys;
			Alpha = 0.0f;
		}
		else
		{
			// For non-looping animation, the last frame is the ending frame, and has no duration.
			const float KeyPos = RelativePos * (float)NumKeys;
			const float KeyPosFloor = floor(KeyPos);
			PosIndex0Out = min( (int)KeyPosFloor, NumKeys );
			Alpha = (Interpolation == Step) ? 0.0f : KeyPos - KeyPosFloor;
			PosIndex1Out = min( PosIndex0Out + 1, NumKeys );
		}
	}
	return Alpha;
}

/**
 * Utility function to find the key before the specified search value.
 *
 * @param	FrameTable		The frame table, containing on frame index value per key.
 * @param	NumKeys			The total number of keys in the table.
 * @param	SearchFrame		The Frame we are attempting to find.
 * @param	KeyEstimate		An estimate of the best location to search from in the KeyTable.
 * @return	The index of the first key immediately below the specified search frame.
 */
inline int32 FindLowKeyIndexUInt8(
	const unsigned int8* FrameTable, 
	const int NumKeys, 
	const uniform int SearchFrame, 
	const int KeyEstimate)
{
	const int LastKeyIndex = NumKeys-1;
	int LowKeyIndex = KeyEstimate;

	#pragma ignore warning(perf)
	if (FrameTable[KeyEstimate] <= SearchFrame)
	{
		// unless we find something better, we'll default to the last key
		LowKeyIndex = LastKeyIndex;

		// search forward from the estimate for the first value greater than our search parameter
		// if found, this is the high key and we want the one just prior to it
		for (int i = KeyEstimate+1; i <= LastKeyIndex; ++i)
		{
			#pragma ignore warning(perf)
			if (FrameTable[i] > SearchFrame)
			{
				LowKeyIndex= i-1;
				break;
			}
		}
	}
	else
	{
		// unless we find something better, we'll default to the first key
		LowKeyIndex = 0;

		// search backward from the estimate for the first value less than or equal to the search parameter
		// if found, this is the low key we are searching for
		for (int i = KeyEstimate-1; i > 0; --i)
		{
			#pragma ignore warning(perf)
			if (FrameTable[i] <= SearchFrame)
			{
				LowKeyIndex= i;
				break;
			}
		}
	}

	return LowKeyIndex;
}

inline int32 FindLowKeyIndexUInt16(
	const unsigned int16* FrameTable, 
	const int NumKeys, 
	const uniform int SearchFrame, 
	const int KeyEstimate)
{
	const int LastKeyIndex = NumKeys-1;
	int LowKeyIndex = KeyEstimate;

	#pragma ignore warning(perf)
	if (FrameTable[KeyEstimate] <= SearchFrame)
	{
		// unless we find something better, we'll default to the last key
		LowKeyIndex = LastKeyIndex;

		// search forward from the estimate for the first value greater than our search parameter
		// if found, this is the high key and we want the one just prior to it
		for (int i = KeyEstimate+1; i <= LastKeyIndex; ++i)
		{
			#pragma ignore warning(perf)
			if (FrameTable[i] > SearchFrame)
			{
				LowKeyIndex= i-1;
				break;
			}
		}
	}
	else
	{
		// unless we find something better, we'll default to the first key
		LowKeyIndex = 0;

		// search backward from the estimate for the first value less than or equal to the search parameter
		// if found, this is the low key we are searching for
		for (int i = KeyEstimate-1; i > 0; --i)
		{
			#pragma ignore warning(perf)
			if (FrameTable[i] <= SearchFrame)
			{
				LowKeyIndex= i;
				break;
			}
		}
	}

	return LowKeyIndex;
}

/**
 * Utility function to determine the two key indices to interpolate given a relative position in the animation
 *
 * @param	Seq				The UAnimSequence container.
 * @param	FrameTable		The frame table containing a frame index for each key.
 * @param	RelativePos		The relative position to solve in the range [0,1] inclusive.
  * @param	NumKeys			The number of keys present in the track being solved.
 * @param	PosIndex0Out	Output value for the closest key index before the RelativePos specified.
 * @param	PosIndex1Out	Output value for the closest key index after the RelativePos specified.
 * @return	The rate at which to interpolate the two keys returned to obtain the final result.
 */
inline float TimeToIndex(
	const uniform unsigned int8 Interpolation,
	const uniform int NumberOfFrames,
	const unsigned int8* FrameTable,
	const uniform float RelativePos,
	const int NumKeys,
	int &PosIndex0Out,
	int &PosIndex1Out)
{
	float Alpha = 0.0f;
	
	const int LastKey= NumKeys-1;
	
	const uniform int TotalFrames = NumberOfFrames -1;
	const int EndingKey = LastKey;

	if (NumKeys < 2 || RelativePos <= 0.f)
	{
		// return the first key
		PosIndex0Out = 0;
		PosIndex1Out = 0;
		Alpha = 0.0f;
	}
	else if( RelativePos >= 1.0f )
	{
		// return the ending key
		PosIndex0Out = EndingKey;
		PosIndex1Out = EndingKey;
		Alpha = 0.0f;
	}
	else
	{
		// find the proper key range to return
		const uniform int32 LastFrame= TotalFrames-1;
		const float KeyPos = RelativePos * (float)LastKey;
		const uniform float FramePos = RelativePos * (uniform float)TotalFrames;
		const uniform int FramePosFloor = clamp((uniform int)FramePos, 0, LastFrame);
		const int KeyEstimate = clamp((int)KeyPos, 0, LastKey);

		int LowFrame = 0;
		int HighFrame = 0;
		
		// find the pair of keys which surround our target frame index
		if (NumberOfFrames > 0xFF)
		{
			const unsigned int16* Frames= (unsigned int16*)FrameTable;
			PosIndex0Out = FindLowKeyIndexUInt16(Frames, NumKeys, FramePosFloor, KeyEstimate);

			#pragma ignore warning(perf)
			LowFrame = Frames[PosIndex0Out];

			PosIndex1Out = PosIndex0Out + 1;
			if (PosIndex1Out > LastKey)
			{
				PosIndex1Out= EndingKey;
			}

			#pragma ignore warning(perf)
			HighFrame= Frames[PosIndex1Out];
		}
		else
		{
			const unsigned int8* Frames= (unsigned int8*)FrameTable;
			PosIndex0Out = FindLowKeyIndexUInt8(Frames, NumKeys, FramePosFloor, KeyEstimate);

			#pragma ignore warning(perf)
			LowFrame = Frames[PosIndex0Out];

			PosIndex1Out = PosIndex0Out + 1;
			if (PosIndex1Out > LastKey)
			{
				PosIndex1Out= EndingKey;
			}

			#pragma ignore warning(perf)
			HighFrame= Frames[PosIndex1Out];
		}

		// compute the blend parameters for the keys we have found
		int Delta= max(HighFrame - LowFrame, 1);
		const float Remainder = (FramePos - (float)LowFrame);
		Alpha = (Interpolation == Step) ? 0.f : (Remainder / (float)Delta);
	}
	
	return Alpha;
}

inline FVector4f Float96NoWToQuat(const float* Values)
{
	#pragma ignore warning(perf)
	const float X = Values[0];

	#pragma ignore warning(perf)
	const float Y = Values[1];

	#pragma ignore warning(perf)
	const float Z = Values[2];

	const float WSquared = 1.f - X*X - Y*Y - Z*Z;

	return SetVector4(X, Y, Z, WSquared > 0.f ? sqrt(WSquared) : 0.f);
}

inline FVector4f Fixed32NoWToQuat(const unsigned int* PackedValue)
{
	#pragma ignore warning(perf)
	const unsigned int Packed = *PackedValue;

	const unsigned int UnpackedX = Packed >> XShift;
	const unsigned int UnpackedY = (Packed & YMask) >> YShift;
	const unsigned int UnpackedZ = (Packed & ZMask);

	const float X = ((int)UnpackedX - Quant11BitOffs) / Quant11BitDiv;
	const float Y = ((int)UnpackedY - Quant11BitOffs) / Quant11BitDiv;
	const float Z = ((int)UnpackedZ - Quant10BitOffs) / Quant10BitDiv;
	const float WSquared = 1.f - X*X - Y*Y - Z*Z;

	return SetVector4(X, Y, Z, WSquared > 0.f ? sqrt(WSquared) : 0.f);
}

inline FVector4f Fixed48NoWToQuat(const unsigned int16* Values)
{
	#pragma ignore warning(perf)
	const unsigned int16 X = Values[0];

	#pragma ignore warning(perf)
	const unsigned int16 Y = Values[1];

	#pragma ignore warning(perf)
	const unsigned int16 Z = Values[2];

	const float FX = ((int)X - Quant16BitOffs) / Quant16BitDiv;
	const float FY = ((int)Y - Quant16BitOffs) / Quant16BitDiv;
	const float FZ = ((int)Z - Quant16BitOffs) / Quant16BitDiv;
	const float WSquared = 1.f - FX*FX - FY*FY - FZ*FZ;

	return SetVector4(FX, FY, FZ, WSquared > 0.f ? sqrt( WSquared ) : 0.f);
}

inline FVector4f IntervalFixed32NoWToQuat(
	const FVector3f& Mins, 
	const FVector3f& Ranges, 
	const unsigned int* PackedValue)
{
	#pragma ignore warning(perf)
	const unsigned int Packed = *PackedValue;

	const unsigned int UnpackedX = Packed >> XShift;
	const unsigned int UnpackedY = (Packed & YMask) >> YShift;
	const unsigned int UnpackedZ = (Packed & ZMask);

	const float X = ((((int)UnpackedX - Quant11BitOffs) / Quant11BitDiv) * Ranges.V[0] + Mins.V[0]);
	const float Y = ((((int)UnpackedY - Quant11BitOffs) / Quant11BitDiv) * Ranges.V[1] + Mins.V[1]);
	const float Z = ((((int)UnpackedZ - Quant10BitOffs) / Quant10BitDiv) * Ranges.V[2] + Mins.V[2]);
	const float WSquared = 1.f - X*X - Y*Y - Z*Z;

	return SetVector4(X, Y, Z, WSquared > 0.f ? sqrt(WSquared) : 0.f);
}

inline float DecodeFloat(const uniform unsigned int NumExponentBits, const uniform unsigned int NumMantissaBits, const uniform bool bRound, const unsigned int Value)
{
	const uniform int NumOutputsBits = NumExponentBits + NumMantissaBits + 1;
	const uniform int MantissaShift = IEEEFloat_MantissaBits - NumMantissaBits;
	const uniform int ExponentBias = (1 << (NumExponentBits-1)) - 1;
	const uniform int SignShift = NumExponentBits + NumMantissaBits;

	const uniform int MantissaMask = (1 << NumMantissaBits) - 1;
	const uniform int ExponentMask = ((1 << NumExponentBits)-1) << NumMantissaBits;
	const uniform int SignMask = 1 << SignShift;

	const uniform int MinExponent = -ExponentBias - 1;
	const uniform int MaxExponent = ExponentBias;
	
	// Extract mantissa, exponent, sign.
	unsigned int Mantissa = Value & MantissaMask;
	int Exponent = (Value & ExponentMask) >> NumMantissaBits;
	const unsigned int Sign = Value >> SignShift;
	
	// Subtract our bias.
	Exponent += MinExponent;
	// Add IEEE's bias.
	Exponent += IEEEFloat_ExponentBias;
	
	Mantissa <<= MantissaShift;

	return floatbits((Sign << SignShift) | (Exponent << NumMantissaBits) | (Mantissa));
}

inline FVector4f Float32NoWToQuat(const unsigned int* PackedValue)
{
	#pragma ignore warning(perf)
	const unsigned int Packed = *PackedValue;

	const unsigned int UnpackedX = Packed >> XShift;
	const unsigned int UnpackedY = (Packed & YMask) >> YShift;
	const unsigned int UnpackedZ = (Packed & ZMask);

	const float X = DecodeFloat(3, 7, true, UnpackedX);
	const float Y = DecodeFloat(3, 7, true, UnpackedY);
	const float Z = DecodeFloat(3, 6, true, UnpackedZ);
	const float WSquared = 1.f - X*X - Y*Y - Z*Z;

	return SetVector4(X, Y, Z, WSquared > 0.f ? sqrt(WSquared) : 0.f);
}

inline FVector3f IntervalFixed32NoWToVector(const FVector3f& Mins, const FVector3f& Ranges, const unsigned int* PackedValue)
{
	#pragma ignore warning(perf)
	const unsigned int Packed = *PackedValue;

	const unsigned int UnpackedZ = Packed >> ZShift;
	const unsigned int UnpackedY = (Packed & YMask) >> YShift;
	const unsigned int UnpackedX = (Packed & ZMask); // Switch XMask and ZMask for this format

	const float X = ((((int)UnpackedX - Quant10BitOffs) / Quant10BitDiv) * Ranges.V[0] + Mins.V[0]);
	const float Y = ((((int)UnpackedY - Quant11BitOffs) / Quant11BitDiv) * Ranges.V[1] + Mins.V[1]);
	const float Z = ((((int)UnpackedZ - Quant11BitOffs) / Quant11BitDiv) * Ranges.V[2] + Mins.V[2]);

	return SetVector(X, Y, Z);
}

inline FVector3f Fixed48NoWToVector(const unsigned int16* Values)
{
	#pragma ignore warning(perf)
	const unsigned int16 X = Values[0];

	#pragma ignore warning(perf)
	const unsigned int16 Y = Values[1];

	#pragma ignore warning(perf)
	const unsigned int16 Z = Values[2];

	const float FX = ((int)X - Quant16BitOffs) / Quant16BitDiv;
	const float FY = ((int)Y - Quant16BitOffs) / Quant16BitDiv;
	const float FZ = ((int)Z - Quant16BitOffs) / Quant16BitDiv;

	return SetVector(FX * 128.0f, FY * 128.0f, FZ * 128.0f);
}

inline float DecompressFixed16(const unsigned int16 Value)
{
	static const uniform int Log2MaxValue = 7;
	static const uniform int QuantOffset = (1 << (15 - Log2MaxValue)) - 1;
	static const uniform float InvQuantFactor = 1.0f / (float)(QuantOffset >> Log2MaxValue);

	return ((int)Value - QuantOffset) * InvQuantFactor;
}

inline void DecompressRotation(FVector4f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData, const uniform int FORMAT)
{
	if ( FORMAT == ACF_None )
	{
		#pragma ignore warning(perf)
		const FVector4f Keys = *((const FVector4f *)KeyData);
		Out = SetVector4(Keys.V[0], Keys.V[1], Keys.V[2], Keys.V[3]);
	}
	else if ( FORMAT == ACF_Float96NoW )
	{
		Out = Float96NoWToQuat((const float *)KeyData);
	}
	else if ( FORMAT == ACF_Fixed32NoW )
	{
		Out = Fixed32NoWToQuat((const unsigned int *)KeyData);
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		Out = Fixed48NoWToQuat((const unsigned int16 *)KeyData);
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const FVector3f* VTopOfStream = (const FVector3f*)TopOfStream;

		#pragma ignore warning(perf)
		const FVector3f Mins = *(VTopOfStream);

		#pragma ignore warning(perf)
		const FVector3f Ranges = *(VTopOfStream+1);

		Out = IntervalFixed32NoWToQuat(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if ( FORMAT == ACF_Float32NoW )
	{
		Out = Float32NoWToQuat((const unsigned int*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = Float0001;
	}
	else
	{
		Out = Float0001;
	}
}

inline void DecompressRotation(const int Format, const int FormatFlags, FVector4f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData)
{
	if (Format == ACF_Fixed48NoW)
	{
		static const uniform float BiasFix48Data = 32767.0f;
		static const uniform float ScaleRotData = 3.0518509475997192297128208258309e-5f;
		const unsigned int16* TypedKeyData = (const unsigned int16*)KeyData;

		#pragma ignore warning(perf)
		const float Xa = (FormatFlags & 1) ? (*TypedKeyData++) : BiasFix48Data;

		#pragma ignore warning(perf)
		const float Ya = (FormatFlags & 2) ? (*TypedKeyData++) : BiasFix48Data;

		#pragma ignore warning(perf)
		const float Za = (FormatFlags & 4) ? (*TypedKeyData++) : BiasFix48Data;

		const float X = (Xa - BiasFix48Data) * ScaleRotData;
		const float XX = X*X;
		const float Y = (Ya - BiasFix48Data) * ScaleRotData;
		const float YY = Y*Y;
		const float Z = (Za - BiasFix48Data) * ScaleRotData;
		const float ZZ = Z*Z;

		const float WSquared = 1.0f - XX - YY - ZZ;

		const float W = select(WSquared >= 0.f, sqrt(WSquared), 0.0f);

		Out = SetVector4(X, Y, Z, W);
	}
	else if (Format == ACF_Float96NoW)
	{
		Out = Float96NoWToQuat((const float *)KeyData);
	}
	else if ( Format == ACF_IntervalFixed32NoW )
	{
		const float* SourceBounds = (float*)TopOfStream;

		FVector3f Mins = FloatZeroVector;
		FVector3f Ranges = FloatZeroVector;

		if (FormatFlags & 1)
		{
			#pragma ignore warning(perf)
			Mins.V[0] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			#pragma ignore warning(perf)
			Mins.V[1] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			#pragma ignore warning(perf)
			Mins.V[2] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[2] = *SourceBounds++;
		}

		Out = IntervalFixed32NoWToQuat(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if ( Format == ACF_Float32NoW )
	{
		Out = Float32NoWToQuat((const unsigned int*)KeyData);
	}
	else if (Format == ACF_Fixed32NoW)
	{
		Out = Fixed32NoWToQuat((const unsigned int *)KeyData);
	}
	else if ( Format == ACF_Identity )
	{
		Out = Float0001;
	}
	else
	{
		Out = Float0001;
	}
}

inline void DecompressTranslation(FVector3f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData, const uniform int FORMAT)
{
	if ( (FORMAT == ACF_None) || (FORMAT == ACF_Float96NoW) )
	{
		#pragma ignore warning(perf)
		const FVector3f Keys = *((const FVector3f *)KeyData);
		Out = SetVector(Keys.V[0], Keys.V[1], Keys.V[2]);
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const FVector3f* VTopOfStream = (const FVector3f*)TopOfStream;

		#pragma ignore warning(perf)
		const FVector3f Mins = *(VTopOfStream);

		#pragma ignore warning(perf)
		const FVector3f Ranges = *(VTopOfStream+1);
		Out = IntervalFixed32NoWToVector(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = FloatZeroVector;
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		Out = Fixed48NoWToVector((const unsigned int16*)KeyData);
	}
	else
	{
		Out = FloatZeroVector;
	}
}

inline void DecompressTranslation(const int Format, const int FormatFlags, FVector3f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData)
{
	if( Format == ACF_Float96NoW )
	{
		// Legacy Format, all components stored
		if( (FormatFlags & 7) == 0 )
		{
			#pragma ignore warning(perf)
			const FVector3f Keys = *((const FVector3f *)KeyData);
			Out = SetVector(Keys.V[0], Keys.V[1], Keys.V[2]);
		}
		// Stored per components
		else
		{
			const float* TypedKeyData = (const float*)KeyData;
			
			#pragma ignore warning(perf)
			const float X = (FormatFlags & 1) ? (*TypedKeyData++) : 0.0f;

			#pragma ignore warning(perf)
			const float Y = (FormatFlags & 2) ? (*TypedKeyData++) : 0.0f;

			#pragma ignore warning(perf)
			const float Z = (FormatFlags & 4) ? (*TypedKeyData++) : 0.0f;

			Out = SetVector(X, Y, Z);
		}
	}
	else if (Format == ACF_IntervalFixed32NoW)
	{
		const float* SourceBounds = (float*)TopOfStream;

		FVector3f Mins = FloatZeroVector;
		FVector3f Ranges = FloatZeroVector;

		if (FormatFlags & 1)
		{
			#pragma ignore warning(perf)
			Mins.V[0] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			#pragma ignore warning(perf)
			Mins.V[1] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			#pragma ignore warning(perf)
			Mins.V[2] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[2] = *SourceBounds++;
		}

		Out = IntervalFixed32NoWToVector(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if (Format == ACF_Fixed48NoW)
	{
		const unsigned int16* TypedKeyData = (const unsigned int16*)KeyData;

		#pragma ignore warning(perf)
		const float X = (FormatFlags & 1) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		#pragma ignore warning(perf)
		const float Y = (FormatFlags & 2) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		#pragma ignore warning(perf)
		const float Z = (FormatFlags & 4) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		Out = SetVector(X, Y, Z);
	}
	else if ( Format == ACF_Identity )
	{
		Out = FloatZeroVector;
	}
	else
	{
		Out = FloatZeroVector;
	}
}

inline void DecompressScale(FVector3f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData, const uniform int FORMAT)
{
	if ( (FORMAT == ACF_None) || (FORMAT == ACF_Float96NoW) )
	{
		#pragma ignore warning(perf)
		const FVector3f Keys = *((const FVector3f *)KeyData);
		Out = SetVector(Keys.V[0], Keys.V[1], Keys.V[2]);
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const FVector3f* VTopOfStream = (const FVector3f*)TopOfStream;

		#pragma ignore warning(perf)
		const FVector3f Mins = *(VTopOfStream);

		#pragma ignore warning(perf)
		const FVector3f Ranges = *(VTopOfStream+1);
		Out = IntervalFixed32NoWToVector(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = FloatZeroVector;
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		Out = Fixed48NoWToVector((const unsigned int16*)KeyData);
	}
	else
	{
		Out = FloatZeroVector;
	}
}

inline void DecompressScale(const int Format, const int FormatFlags, FVector3f& Out, const unsigned int8* TopOfStream, const unsigned int8* KeyData)
{
	if( Format == ACF_Float96NoW )
	{
		// Legacy Format, all components stored
		if( (FormatFlags & 7) == 0 )
		{
			#pragma ignore warning(perf)
			const FVector3f Keys = *((const FVector3f *)KeyData);
			Out = SetVector(Keys.V[0], Keys.V[1], Keys.V[2]);
		}
		// Stored per components
		else
		{
			const float* TypedKeyData = (const float*)KeyData;

			#pragma ignore warning(perf)
			const float X = (FormatFlags & 1) ? (*TypedKeyData++) : 0.0f;

			#pragma ignore warning(perf)
			const float Y = (FormatFlags & 2) ? (*TypedKeyData++) : 0.0f;

			#pragma ignore warning(perf)
			const float Z = (FormatFlags & 4) ? (*TypedKeyData++) : 0.0f;

			Out = SetVector(X, Y, Z);
		}
	}
	else if (Format == ACF_IntervalFixed32NoW)
	{
		const float* SourceBounds = (float*)TopOfStream;

		FVector3f Mins = FloatZeroVector;
		FVector3f Ranges = FloatZeroVector;

		if (FormatFlags & 1)
		{
			#pragma ignore warning(perf)
			Mins.V[0] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[0] = *SourceBounds++;
		}
		if (FormatFlags & 2)
		{
			#pragma ignore warning(perf)
			Mins.V[1] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[1] = *SourceBounds++;
		}
		if (FormatFlags & 4)
		{
			#pragma ignore warning(perf)
			Mins.V[2] = *SourceBounds++;

			#pragma ignore warning(perf)
			Ranges.V[2] = *SourceBounds++;
		}

		Out = IntervalFixed32NoWToVector(Mins, Ranges, (const unsigned int*)KeyData);
	}
	else if (Format == ACF_Fixed48NoW)
	{
		const unsigned int16* TypedKeyData = (const unsigned int16*)KeyData;
		
		#pragma ignore warning(perf)
		const float X = (FormatFlags & 1) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		#pragma ignore warning(perf)
		const float Y = (FormatFlags & 2) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		#pragma ignore warning(perf)
		const float Z = (FormatFlags & 4) ? DecompressFixed16(*TypedKeyData++) : 0.0f;

		Out = SetVector(X, Y, Z);
	}
	else if ( Format == ACF_Identity )
	{
		Out = FloatZeroVector;
	}
	else
	{
		Out = FloatZeroVector;
	}
}

#endif
