// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FCompactPose;
struct FCompressedAnimSequence;
struct FAnimExtractContext;
struct FAnimSequenceDecompressionContext;
struct FRootMotionReset;

namespace UE::Anim::Decompression
{
	ENGINE_API void DecompressPose(FCompactPose& OutPose,
				const FCompressedAnimSequence& CompressedData,
				const FAnimExtractContext& ExtractionContext,
				FAnimSequenceDecompressionContext& DecompressionContext,
				const TArray<FTransform>& RetargetTransforms,
				const FRootMotionReset& RootMotionReset);
	
	ENGINE_API void DecompressPose(FCompactPose& OutPose, 
				const FCompressedAnimSequence& CompressedData,
				const FAnimExtractContext& ExtractionContext,
				FAnimSequenceDecompressionContext& DecompressionContext,
				FName RetargetSource,
				const FRootMotionReset& RootMotionReset);
}