// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNodeMessages.h"

namespace UE::Anim
{
	
// Notify Context Data containing the Instance ID of the asset player that triggered this notify
// This is used to determine if a notify state was triggered by different instances of the same animation 
class FAnimNotifyAssetPlayerInstanceContext : public UE::Anim::IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE_API(FAnimNotifyAssetPlayerInstanceContext, ENGINE_API)
public:
	ENGINE_API FAnimNotifyAssetPlayerInstanceContext(const uint32 InAssetPlayerInstanceID);
	const uint32 AssetPlayerInstanceID;
};
	
}
