// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AnimationStreaming.h: Definitions of classes used for animation streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/IndirectArray.h"
#include "Containers/Queue.h"
#include "Stats/Stats.h"
#include "ContentStreaming.h"
#include "Async/AsyncWork.h"
#include "Async/AsyncFileHandle.h"
#include "Templates/Atomic.h"
class UAnimStreamable;
struct FAnimationStreamingManager;

// 
struct FLoadedAnimationChunk
{
	TAtomic<FCompressedAnimSequence*> CompressedAnimData;

	class IBulkDataIORequest* IORequest;
	double RequestStart;

	uint32	Index;
	bool	bOwnsCompressedData;

	FLoadedAnimationChunk()
		: CompressedAnimData(nullptr)
		, IORequest(nullptr)
		, RequestStart(-1.0)
		, Index(0)
		, bOwnsCompressedData(false)
	{
	}

	~FLoadedAnimationChunk()
	{
		checkf(CompressedAnimData == nullptr, TEXT("Animation chunk compressed data ptr not null (%p), Index: %u"), CompressedAnimData.Load(), Index);
	}

	void CleanUpIORequest();
};

/**
 * Contains everything that will be needed by a Streamable Anim that's streaming in data
 */
struct FStreamingAnimationData final
{
	FStreamingAnimationData();
	~FStreamingAnimationData();

	// Frees streaming animation data resources, blocks pending async IO requests
	void FreeResources();

	/**
	 * Sets up the streaming wave data and loads the first chunk of animation for instant play
	 *
	 * @param Anim	The streamable animation we are managing
	 */
	bool Initialize(UAnimStreamable* InStreamableAnim, FAnimationStreamingManager* InAnimationStreamingManager);

	/**
	 * Updates the streaming status of the animation and performs finalization when appropriate. The function returns
	 * true while there are pending requests in flight and updating needs to continue.
	 *
	 * @return					true if there are requests in flight, false otherwise
	 */
	bool UpdateStreamingStatus();

	/**
	 * Checks whether the requested chunk indices differ from those loaded
	 *
	 * @param IndicesToLoad		List of chunk indices that should be loaded
	 * @param IndicesToFree		List of chunk indices that should be freed
	 * @return Whether any changes to loaded chunks are required
	 */
	bool HasPendingRequests(TArray<uint32>& IndicesToLoad, TArray<uint32>& IndicesToFree) const;

	/**
	 * Kicks off any pending requests
	 */
	void BeginPendingRequests(const TArray<uint32>& IndicesToLoad, const TArray<uint32>& IndicesToFree);

	/**
	* Blocks till all pending requests are fulfilled.
	*
	* @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
	* @return				Return true if there are no requests left in flight, false if the time limit was reached before they were finished.
	*/
	bool BlockTillAllRequestsFinished(float TimeLimit = 0.0f);

	// Return the number of bytes used
	SIZE_T GetMemorySize() const;

private:
	// Don't allow copy construction as it could free shared memory
	FStreamingAnimationData(const FStreamingAnimationData& that);
	FStreamingAnimationData& operator=(FStreamingAnimationData const&);

	// Creates a new chunk, returns the chunk index
	FLoadedAnimationChunk& AddNewLoadedChunk(uint32 ChunkIndex, FCompressedAnimSequence* ExistingData);
	void FreeLoadedChunk(FLoadedAnimationChunk& LoadedChunk);
	void ResetRequestedChunks();

public:
	/** AnimStreamable this streaming data is for */
	UAnimStreamable* StreamableAnim;

	/* Contains pointers to Chunks of animation data that have been streamed in */
	TArray<FLoadedAnimationChunk> LoadedChunks;

	mutable FCriticalSection LoadedChunksCritcalSection;

	/** Indices of chunks that are currently loaded */
	TArray<uint32>	LoadedChunkIndices;

	TArray<uint32> RequestedChunks;

	TArray<uint32> LoadFailedChunks;

	/** Ptr to owning animation streaming manager. */
	FAnimationStreamingManager* AnimationStreamingManager;
};


/**
* Streaming manager dealing with animation.
*/
struct FAnimationStreamingManager : public IAnimationStreamingManager
{
	/** Constructor, initializing all members */
	FAnimationStreamingManager();

	virtual ~FAnimationStreamingManager();

	// IStreamingManager interface
	virtual void UpdateResourceStreaming( float DeltaTime, bool bProcessEverything=false ) override;
	virtual int32 BlockTillAllRequestsFinished( float TimeLimit = 0.0f, bool bLogResults = false ) override;
	virtual void CancelForcedResources() override;
	virtual void NotifyLevelChange() override;
	virtual void SetDisregardWorldResourcesForFrames( int32 NumFrames ) override;
	virtual void AddLevel( class ULevel* Level ) override;
	virtual void RemoveLevel( class ULevel* Level ) override;
	virtual void NotifyLevelOffset( class ULevel* Level, const FVector& Offset ) override;
	// End IStreamingManager interface

	// IAnimationStreamingManager interface
	virtual void AddStreamingAnim(UAnimStreamable* Anim) override;
	virtual bool RemoveStreamingAnim(UAnimStreamable* Anim) override;
	virtual SIZE_T GetMemorySizeForAnim(const UAnimStreamable* Anim) override;
	virtual const FCompressedAnimSequence* GetLoadedChunk(const UAnimStreamable* Anim, uint32 ChunkIndex, bool bTrackAsRequested) const override;
	// End IAnimationStreamingManager interface

	/** Called when an async callback is made on an async loading animation chunk request. */
	void OnAsyncFileCallback(FStreamingAnimationData* StreamingAnimData, int32 ChunkIndex, int64 ReadSize, IBulkDataIORequest* ReadRequest, bool bWasCancelled);

protected:

	/** Sound Waves being managed. */
	TMap<UAnimStreamable*, FStreamingAnimationData*> StreamingAnimations;

	/** Critical section to protect usage of shared gamethread/workerthread members */
	mutable FCriticalSection CriticalSection;
};
