// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE
{
	namespace Anim
	{
		/** Set of type-traits, used by the Animation Attributes system to verify and implement certain behavior */
		template <class AttributeType>
		struct TAttributeTypeTraitsBase
		{
			enum
			{
				/** Determines whether or not the type should be blended, and is supported to do so. True by default */
				IsBlendable = true,
				/** Determines whether or not the type has an associated user-defined implementation of IAttributeBlendOperator. False by default */
				WithCustomBlendOperator = false,
				/** Determines whether or not the type should be step-interpolated rather than linearly. False by default */
				StepInterpolate = false,
				/** Determines whether or not the type should be normalized after any accumulation / multiplication has been applied. False by default */
				RequiresNormalization = false
			};
		};

		template <class AttributeType>
		struct TAttributeTypeTraits : public TAttributeTypeTraitsBase<AttributeType>
		{
		};
	}
}
