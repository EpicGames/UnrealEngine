// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AttributesRuntime.h"
#include "Animation/CustomAttributes.h"
#include "Animation/IAttributeBlendOperator.h"
#include "Animation/AttributeBlendOperator.h"
#include "Animation/AttributeTraits.h"
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"

enum EAdditiveAnimationType : int;

class UAnimationSettings;

namespace UE
{
	namespace Anim
	{
		/** Concept used to verify a user-defined attribute type with its TAttributeTypeTraits::Blendable value set to true */
		struct CBlendableAttribute
		{
			template <typename T>
			auto Requires(T& Val) -> decltype(
				Val.Multiply(.5f),
				Val.Accumulate(Val, 1.f, (EAdditiveAnimationType)0),
				Val.MakeAdditive(Val),
				Val.Interpolate(Val, 0.5f)
			);
		};

		/** Concept used to verify a user-defined attribute type with its TAttributeTypeTraits::RequiresNormalization value set to true */
		struct CNormalizedAttribute
		{
			template <typename T>
			auto Requires(T& Val) -> decltype(
				Val.Normalize()
			);
		};

		struct AttributeTypes
		{
			DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAttributeTypesChanged, const UScriptStruct*, bool /* bIsAdded */ );
			
		protected:
			static ENGINE_API TArray<TWeakObjectPtr<const UScriptStruct>> RegisteredTypes;
			static ENGINE_API TArray<TUniquePtr<IAttributeBlendOperator>> Operators;
			static ENGINE_API TArray<TWeakObjectPtr<const UScriptStruct>> InterpolatableTypes;
			static ENGINE_API FOnAttributeTypesChanged OnAttributeTypesChangedDelegate;

			/** Register user defined structs as non-blendable animation attribute */
			static ENGINE_API bool RegisterNonBlendableType(const UScriptStruct* InScriptStruct);

			/** Unregisters a specific attribute type and deletes its associated blend operator */
			static ENGINE_API void UnregisterType(const UScriptStruct* InScriptStruct);
			
		public:
			UE_DEPRECATED(5.4, "This function is no longer used, the system will initialize built-in and user-defined types at the appropriate times")
			static void LazyInitialize() {}

			static FOnAttributeTypesChanged& GetOnAttributeTypesChanged() { return OnAttributeTypesChangedDelegate; };
			
			/** Used for registering an attribute type for which TAttributeTypeTraits::WithCustomBlendOperator is set to true, use RegisterType() otherwise */
			template<typename AttributeType, typename OperatorType, typename... OperatorArgs>
			static void RegisterTypeWithOperator(OperatorArgs&&... args)
			{
				static_assert(TAttributeTypeTraits<AttributeType>::WithCustomBlendOperator, "Attribute type does not require a custom blend operation");
				UScriptStruct* ScriptStruct = AttributeType::StaticStruct();

				AttributeTypes::RegisteredTypes.Add(ScriptStruct);
				AttributeTypes::Operators.Add(MakeUnique<OperatorType>(Forward<OperatorArgs>(args)...));

				if constexpr (!UE::Anim::TAttributeTypeTraits<AttributeType>::StepInterpolate)
				{
					AttributeTypes::InterpolatableTypes.Add(ScriptStruct);
				}
			}

			/** Used for registering an attribute type for which TAttributeTypeTraits::WithCustomBlendOperator is set to false, use RegisterTypeWithOperator() otherwise */
			template<typename AttributeType>
			static void RegisterType()
			{
				static_assert(!TAttributeTypeTraits<AttributeType>::WithCustomBlendOperator, "Attribute type requires a custom blend operation");

				UScriptStruct* ScriptStruct = AttributeType::StaticStruct();
				AttributeTypes::RegisteredTypes.Add(ScriptStruct);
				
				if constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::IsBlendable)	
				{
					static_assert(TModels_V<CBlendableAttribute, AttributeType>, "Missing function implementations required for Attribute blending");

					if  constexpr (UE::Anim::TAttributeTypeTraits<AttributeType>::RequiresNormalization)
					{
						static_assert(TModels_V<CNormalizedAttribute, AttributeType>, "Missing function implementations required for Attribute normalization");
					}
					
					if constexpr (!UE::Anim::TAttributeTypeTraits<AttributeType>::StepInterpolate)
					{
						AttributeTypes::InterpolatableTypes.Add(ScriptStruct);
					}
				}

				AttributeTypes::Operators.Add(MakeUnique<TAttributeBlendOperator<AttributeType>>());
			}

			/** Unregisters a specific attribute type and deletes its associated blend operator */
			template<typename AttributeType>
			static void UnregisterType()
			{
				if (UObjectInitialized())
				{
					UScriptStruct* ScriptStruct = AttributeType::StaticStruct();
					const int32 Index = AttributeTypes::RegisteredTypes.IndexOfByKey(ScriptStruct);
					if (Index != INDEX_NONE)
					{
						AttributeTypes::RegisteredTypes.RemoveAtSwap(Index);
						AttributeTypes::Operators.RemoveAtSwap(Index);
					}
				}
			}

			/** Returns the blend operator for the provided type, asserts when the type is not registered */
			static const IAttributeBlendOperator* GetTypeOperator(TWeakObjectPtr<const UScriptStruct> WeakStruct)
			{
				const int32 Index = AttributeTypes::RegisteredTypes.IndexOfByKey(WeakStruct);
				ensure(WeakStruct.IsValid());
				checkf(Index != INDEX_NONE, TEXT("Missing operator for attribute, type %s was not registered previously"), *WeakStruct->GetName());
				return AttributeTypes::Operators[Index].Get();
			}

			/** Returns whether or not the provided type can be interpolated, defaults to false when the type is not registered */
			static bool CanInterpolateType(TWeakObjectPtr<const UScriptStruct> WeakStruct)
			{
				return AttributeTypes::InterpolatableTypes.Contains(WeakStruct);
			}

			/** Returns whether or not the type is registered */
			static bool IsTypeRegistered(const UScriptStruct* ScriptStruct)
			{
				return AttributeTypes::RegisteredTypes.Contains(ScriptStruct);
			}

			/** Returns all registered types */
			static TArray<TWeakObjectPtr<const UScriptStruct>>& GetRegisteredTypes()
			{
				return AttributeTypes::RegisteredTypes;
			}
			
			friend class ::UAnimationSettings;
			friend struct FAttributeTypeRegistrar;
		};
	}
}
