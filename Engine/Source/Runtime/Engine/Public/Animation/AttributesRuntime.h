// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Animation/AnimTypes.h"
#include "UObject/NameTypes.h"
#include "Containers/ContainersFwd.h"

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/StructOnScope.h"

#include "Animation/IAttributeBlendOperator.h"

#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "Animation/AttributesContainer.h"
#include "Animation/WrappedAttribute.h"

struct FCompactPoseBoneIndex;
struct FBlendSampleData;
struct FBoneContainer;

enum class ECustomAttributeBlendType : uint8;
enum EAdditiveAnimationType : int;

struct FAnimatedBoneAttribute;
struct FCompactPoseBoneIndex;
struct FAnimExtractContext;

class UMirrorDataTable;

namespace UE
{
	namespace Anim
	{
		struct FStackAttributeContainer : public TAttributeContainer<FCompactPoseBoneIndex, FAnimStackAllocator> {};
		struct FHeapAttributeContainer : public TAttributeContainer<FCompactPoseBoneIndex, FDefaultAllocator> {};
		struct FMeshAttributeContainer : public TAttributeContainer<FMeshPoseBoneIndex, FDefaultAllocator> {};

		/** Accessor for internal data of the TAttributeContainer.  The TAttributeContainer methods are preferred - this accessor should only be used in limited situations */
		template<class BoneIndexType, typename InAllocator>
        struct TAttributeContainerAccessor
		{
			static TArray<TWrappedAttribute<InAllocator>>& GetValues(TAttributeContainer<BoneIndexType, InAllocator>& Attributes,  int32 TypeIndex) {return Attributes.GetValuesInternal(TypeIndex);}
			static TArray<FAttributeId>& GetKeys(TAttributeContainer<BoneIndexType, InAllocator>& Attributes, int32 TypeIndex) {return Attributes.GetKeysInternal(TypeIndex);}
		};

		struct FStackAttributeContainerAccessor : public TAttributeContainerAccessor<FCompactPoseBoneIndex, FAnimStackAllocator> {};
		struct FHeapAttributeContainerAccessor : public TAttributeContainerAccessor<FCompactPoseBoneIndex, FDefaultAllocator> {};
		
		/** Helper functionality for attributes animation runtime */
		struct Attributes
		{
#if WITH_EDITOR
			UE_DEPRECATED(5.1, "GetAttributeValue with signature using FAnimExtractContext is deprecated use other version instead")
			static ENGINE_API void GetAttributeValue(FStackAttributeContainer& OutAttributes, const FCompactPoseBoneIndex& PoseBoneIndex, const FAnimatedBoneAttribute& Attribute, const FAnimExtractContext& ExtractionContext);

			static ENGINE_API void GetAttributeValue(FStackAttributeContainer& OutAttributes, const FCompactPoseBoneIndex& PoseBoneIndex, const FAnimatedBoneAttribute& Attribute, double CurrentTime);
#endif // WITH_EDITOR

			/** Blend custom attribute values from N set of inputs */
			static ENGINE_API void BlendAttributes(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, FStackAttributeContainer& OutAttributes);

			/** Blend custom attribute values from N set of inputs (ptr-values) */
			static ENGINE_API void BlendAttributes(const TArrayView<const FStackAttributeContainer* const> SourceAttributes, const TArrayView<const float> SourceWeights, FStackAttributeContainer& OutAttributes);

			/** Blend custom attribute values from N set of inputs, using input weight remapping */
			static ENGINE_API void BlendAttributes(const TArrayView<const FStackAttributeContainer> SourceAttributes, const TArrayView<const float> SourceWeights, const TArrayView<const int32> SourceWeightsIndices, FStackAttributeContainer& OutAttributes);

			/* Blend custom attribute values from 2 inputs, using per-bone weights */
			static ENGINE_API void BlendAttributesPerBone(const FStackAttributeContainer& SourceAttributes1, const FStackAttributeContainer& SourceAttributes2, const TArrayView<const float> WeightsOfSource2, FStackAttributeContainer& OutAttributes);

			/* Blend custom attribute values from N set of inputs, using N number of blend samples */
			static ENGINE_API void BlendAttributesPerBone(TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> PerBoneInterpolationIndices, TArrayView<const FBlendSampleData> BlendSampleDataCache, FStackAttributeContainer& OutAttributes);

			/* Blend custom attribute values from N set of inputs, using N number of blend samples */
			static ENGINE_API void BlendAttributesPerBone(TArrayView<const FStackAttributeContainer> SourceAttributes, TArrayView<const int32> PerBoneInterpolationIndices, TArrayView<const FBlendSampleData> BlendSampleDataCache, TArrayView<const int32> BlendSampleDataCacheIndices, FStackAttributeContainer& OutAttributes);

			/* Blend custom attribute values from N set of inputs, using bone filter pose weights */
			static ENGINE_API void BlendAttributesPerBoneFilter(const FStackAttributeContainer& BaseAttributes, const TArrayView<const FStackAttributeContainer> BlendAttributes, const TConstArrayView<FPerBoneBlendWeight> BoneBlendWeights, FStackAttributeContainer& OutAttributes);

			/** Add any new or override existing custom attributes */
			static ENGINE_API void OverrideAttributes(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, float Weight);

			/** Add any new or accumulate with existing custom attributes */
			static ENGINE_API void AccumulateAttributes(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, float Weight, EAdditiveAnimationType AdditiveType);
			
			/** Add any new or accumulate with existing custom attributes with optional per attribute weight overrides */
			static ENGINE_API void AccumulateAttributesWithOverrideWeights(const FStackAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, const TMap<FAttributeId, float>& AttributeWeightOverrides, float Weight, EAdditiveAnimationType AdditiveType);

			/** Add (negated) any new or subtract from existing custom attributes */
			static ENGINE_API void ConvertToAdditive(const FStackAttributeContainer& BaseAttributes, FStackAttributeContainer& OutAdditiveAttributes);

			/** Copy attributes from source, and remap the bone indices according to BoneMapToSource */
			static ENGINE_API void CopyAndRemapAttributes(const FMeshAttributeContainer& SourceAttributes, FStackAttributeContainer& OutAttributes, const TMap<int32, int32>& BoneMapToSource, const FBoneContainer& RequiredBones);

			/** Interpolates between two sets of attributes */
			static ENGINE_API void InterpolateAttributes(FMeshAttributeContainer& FromAttributes, const FMeshAttributeContainer& ToAttributes, float Alpha);

			/** Mirror (swap) attributes with the specified MirrorDataTable. Attributes are swapped using the bone mapping such that bones which are mirrored swap attributes */
			static ENGINE_API void MirrorAttributes(FStackAttributeContainer& Attributes, const UMirrorDataTable& MirrorDataTable, const TArray<FCompactPoseBoneIndex>& CompactPoseMirrorBones);

			UE_DEPRECATED(5.2, "MirrorAttributes has been deprecated, use other signature instead")
			static ENGINE_API void MirrorAttributes(FStackAttributeContainer& Attributes, const UMirrorDataTable& MirrorDataTable);
			
			/** Helper functionality to retrieve the correct blend type (from UAnimationSettings) for the provided attribute name */
			static ENGINE_API ECustomAttributeBlendType GetAttributeBlendType(const FName& InName);

			/** Helper functionality to retrieve the correct blend type (from UAnimationSettings) for the provided attribute info */
			static ENGINE_API ECustomAttributeBlendType GetAttributeBlendType(const FAttributeId& Info);
		};
	}
}

struct ENGINE_API UE_DEPRECATED(5.0, "FStackCustomAttributes has been deprecated use UE::Anim::FStackAttributeContainer instead") FStackCustomAttributes : public UE::Anim::FStackAttributeContainer {};
struct ENGINE_API UE_DEPRECATED(5.0, "FStackCustomAttributes has been deprecated use UE::Anim::FStackAttributeContainer instead") FHeapCustomAttributes : public UE::Anim::FHeapAttributeContainer {};

