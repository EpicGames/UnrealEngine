// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"
#include "BoneReference.generated.h"

struct FBoneContainer;
class USkeleton;

USTRUCT()
struct FBoneReference
{
	GENERATED_USTRUCT_BODY()

	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category = BoneReference)
	FName BoneName;

	/** Cached bone index for run time - right now bone index of skeleton **/
	int32 BoneIndex:31;

	/** Change this to Bitfield if we have more than one bool 
	 * This specifies whether or not this indices is mesh or skeleton
	 */
	uint32 bUseSkeletonIndex:1;

	FCompactPoseBoneIndex CachedCompactPoseIndex;

	FBoneReference()
		: BoneName(NAME_None)
		, BoneIndex(INDEX_NONE)
		, bUseSkeletonIndex(false)
		, CachedCompactPoseIndex(INDEX_NONE)
	{
	}

	FBoneReference(const FName& InBoneName)
		: BoneName(InBoneName)
		, BoneIndex(INDEX_NONE)
		, bUseSkeletonIndex(false)
		, CachedCompactPoseIndex(INDEX_NONE)
	{
	}

	bool operator==(const FBoneReference& Other) const
	{
		return BoneName == Other.BoneName;
	}

	bool operator!=(const FBoneReference& Other) const
	{
		return BoneName != Other.BoneName;
	}

	/** Initialize Bone Reference, return TRUE if success, otherwise, return false **/
	ENGINE_API bool Initialize(const FBoneContainer& RequiredBones);

	// only used by blendspace 'PerBoneBlend'. This is skeleton indices since the input is skeleton
	// @note, if you use this function, it won't work with GetCompactPoseIndex, GetMeshPoseIndex;
	// it triggers ensure in those functions
	ENGINE_API bool Initialize(const USkeleton* Skeleton);

	/** Reset this container to default state */
	void Reset()
	{
		BoneName = NAME_None;
		InvalidateCachedBoneIndex();
	}

	/** return true if it has valid set up */
	bool HasValidSetup() const
	{
		return (BoneIndex != INDEX_NONE);
	}

	/** return true if has valid index, and required bones contain it **/
	ENGINE_API bool IsValidToEvaluate(const FBoneContainer& RequiredBones) const;
	/** return true if has valid compact index. This will return invalid if you're using skeleton index */
	bool IsValidToEvaluate() const
	{
		return (!bUseSkeletonIndex && CachedCompactPoseIndex != INDEX_NONE);
	}

	void InvalidateCachedBoneIndex()
	{
		BoneIndex = INDEX_NONE;
		CachedCompactPoseIndex = FCompactPoseBoneIndex(INDEX_NONE);
	}

	ENGINE_API FSkeletonPoseBoneIndex GetSkeletonPoseIndex(const FBoneContainer& RequiredBones) const;
	
	ENGINE_API FMeshPoseBoneIndex GetMeshPoseIndex(const FBoneContainer& RequiredBones) const;

	ENGINE_API FCompactPoseBoneIndex GetCompactPoseIndex(const FBoneContainer& RequiredBones) const;

	// need this because of BoneReference being used in CurveMetaData and that is in SmartName
	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FBoneReference& B);

	ENGINE_API bool Serialize(FArchive& Ar);
};
