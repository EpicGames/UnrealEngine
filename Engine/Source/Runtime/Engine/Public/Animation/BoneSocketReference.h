// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Abstract base class for a skeletal controller.
 *	A SkelControl is a module that can modify the position or orientation of a set of bones in a skeletal mesh in some programmatic way.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "BoneContainer.h"
#include "BoneSocketReference.generated.h"

class USkeletalMeshComponent;
struct FAnimInstanceProxy;

USTRUCT()
struct FSocketReference
{
	GENERATED_USTRUCT_BODY()

private:
	FTransform CachedSocketLocalTransform;

public:
	/** Target socket to look at. Used if LookAtBone is empty. - You can use  LookAtLocation if you need offset from this point. That location will be used in their local space. **/
	UPROPERTY(EditAnywhere, Category = FSocketReference)
	FName SocketName;

private:
	FMeshPoseBoneIndex CachedSocketMeshBoneIndex;
	FCompactPoseBoneIndex CachedSocketCompactBoneIndex;
	
public:
	FSocketReference()
		: CachedSocketMeshBoneIndex(INDEX_NONE)
		, CachedSocketCompactBoneIndex(INDEX_NONE)
	{
	}

	FSocketReference(const FName& InSocketName)
		: SocketName(InSocketName)
		, CachedSocketMeshBoneIndex(INDEX_NONE)
		, CachedSocketCompactBoneIndex(INDEX_NONE)
	{
	}

	ENGINE_API void InitializeSocketInfo(const FAnimInstanceProxy* InAnimInstanceProxy);
	ENGINE_API void InitialzeCompactBoneIndex(const FBoneContainer& RequiredBones);
	/* There are subtle difference between this two IsValid function
	 * First one says the configuration had a valid socket as mesh index is valid
	 * Second one says the current bonecontainer doesn't contain it, meaning the current LOD is missing the joint that is required to evaluate 
	 * Although the expected behavior is ambiguous, I'll still split these two, and use it accordingly */
	bool HasValidSetup() const
	{
		return (CachedSocketMeshBoneIndex.IsValid());
	}

	bool IsValidToEvaluate() const
	{
		return (CachedSocketCompactBoneIndex.IsValid());
	}

	FCompactPoseBoneIndex GetCachedSocketCompactBoneIndex() const
	{
		return CachedSocketCompactBoneIndex;
	}

	template<typename poseType>
	FTransform GetAnimatedSocketTransform(struct FCSPose<poseType>& InPose) const
	{
		// current LOD has valid index (FCompactPoseBoneIndex is valid if current LOD supports)
		if (CachedSocketCompactBoneIndex.IsValid())
		{
			FTransform BoneTransform = InPose.GetComponentSpaceTransform(CachedSocketCompactBoneIndex);
			return CachedSocketLocalTransform * BoneTransform;
		}

		return FTransform::Identity;
	}

	void InvalidateCachedBoneIndex()
	{
		CachedSocketMeshBoneIndex = FMeshPoseBoneIndex(INDEX_NONE);
		CachedSocketCompactBoneIndex = FCompactPoseBoneIndex(INDEX_NONE);
	}
};

USTRUCT()
struct  FBoneSocketTarget
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = FBoneSocketTarget)
	bool bUseSocket;

	UPROPERTY(EditAnywhere, Category = FBoneSocketTarget, meta=(EditCondition = "!bUseSocket"))
	FBoneReference BoneReference;

	UPROPERTY(EditAnywhere, Category = FBoneSocketTarget, meta = (EditCondition = "bUseSocket"))
	FSocketReference SocketReference;

	FBoneSocketTarget(FName InName = NAME_None, bool bInUseSocket = false)
	{
		bUseSocket = bInUseSocket;

		if (bUseSocket)
		{
			SocketReference.SocketName = InName;
		}
		else
		{
			BoneReference.BoneName = InName;
		}
	}

	void InitializeBoneReferences(const FBoneContainer& RequiredBones)
	{
		if (bUseSocket)
		{
			SocketReference.InitialzeCompactBoneIndex(RequiredBones);
			BoneReference.InvalidateCachedBoneIndex();
		}
		else
		{
			BoneReference.Initialize(RequiredBones);
			SocketReference.InvalidateCachedBoneIndex();
		}
	}

	/** Initialize Bone Reference, return TRUE if success, otherwise, return false **/
	void Initialize(const FAnimInstanceProxy* InAnimInstanceProxy)
	{
		if (bUseSocket)
		{
			SocketReference.InitializeSocketInfo(InAnimInstanceProxy);
		}
	}


	/** return true if valid. Otherwise return false **/
	bool HasValidSetup() const
	{
		if (bUseSocket)
		{
			return SocketReference.HasValidSetup();
		}

		return BoneReference.BoneIndex != INDEX_NONE;
	}

	bool HasTargetSetup() const
	{
		if (bUseSocket)
		{
			return (SocketReference.SocketName != NAME_None);
		}

		return (BoneReference.BoneName != NAME_None);
	}

	FName GetTargetSetup() const
	{
		if (bUseSocket)
		{
			return (SocketReference.SocketName);
		}

		return (BoneReference.BoneName);
	}

	/** return true if valid. Otherwise return false **/
	bool IsValidToEvaluate(const FBoneContainer& RequiredBones) const
	{
		if (bUseSocket)
		{
			return SocketReference.IsValidToEvaluate();
		}

		return BoneReference.IsValidToEvaluate(RequiredBones);
	}

	// this will return the compact pose bone index that matters
	// if you're using socket, it will return socket's related joint's compact pose index
	FCompactPoseBoneIndex GetCompactPoseBoneIndex() const 
	{
		if (bUseSocket)
		{
			return SocketReference.GetCachedSocketCompactBoneIndex();
		}

		return BoneReference.CachedCompactPoseIndex;
	}

	/** Get Target Transform from current incoming pose */
	template<typename poseType>
	FTransform GetTargetTransform(const FVector& TargetOffset, FCSPose<poseType>& InPose, const FTransform& InComponentToWorld) const
	{
		FTransform OutTargetTransform;

		auto SetComponentSpaceOffset = [](const FVector& InTargetOffset, const FTransform& LocalInComponentToWorld, FTransform& LocalOutTargetTransform)
		{
			LocalOutTargetTransform.SetIdentity();
			FVector CSTargetOffset = LocalInComponentToWorld.InverseTransformPosition(InTargetOffset);
			LocalOutTargetTransform.SetLocation(CSTargetOffset);
		};

		if (bUseSocket)
		{
			// this has to be done outside
			if (SocketReference.IsValidToEvaluate())
			{
				FTransform SocketTransformInCS = SocketReference.GetAnimatedSocketTransform(InPose);

				FVector CSTargetOffset = SocketTransformInCS.TransformPosition(TargetOffset);
				OutTargetTransform = SocketTransformInCS;
				OutTargetTransform.SetLocation(CSTargetOffset);
			}
			else
			{
				// if none is found, we consider this offset is world offset
				SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
			}
		}
		// if valid data is available
		else if (BoneReference.HasValidSetup())
		{
			if (BoneReference.IsValidToEvaluate() && 
				InPose.GetPose().IsValid() &&
				ensureMsgf(InPose.GetPose().IsValidIndex(BoneReference.CachedCompactPoseIndex), TEXT("Invalid Cached Pose : Name %s(Bone Index (%d), Cached (%d))"), *BoneReference.BoneName.ToString(), BoneReference.BoneIndex, BoneReference.CachedCompactPoseIndex.GetInt()))
			{
				OutTargetTransform = InPose.GetComponentSpaceTransform(BoneReference.CachedCompactPoseIndex);
				FVector CSTargetOffset = OutTargetTransform.TransformPosition(TargetOffset);
				OutTargetTransform.SetLocation(CSTargetOffset);
			}
			else
			{
				// if none is found, we consider this offset is world offset
				SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
			}
		}
		else
		{
			// if none is found, we consider this offset is world offset
			SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
		}

		return OutTargetTransform;
	}

	template<typename poseType>
	FTransform GetTargetTransform(const FTransform& TargetOffset, FCSPose<poseType>& InPose, const FTransform& InComponentToWorld) const
	{
		FTransform OutTargetTransform;

		auto SetComponentSpaceOffset = [](const FTransform& InTargetOffset, const FTransform& LocalInComponentToWorld, FTransform& LocalOutTargetTransform)
		{
			LocalOutTargetTransform = InTargetOffset.GetRelativeTransform(LocalInComponentToWorld);
		};

		if (bUseSocket)
		{
			// this has to be done outside
			if (SocketReference.IsValidToEvaluate())
			{
				OutTargetTransform = TargetOffset * SocketReference.GetAnimatedSocketTransform(InPose);
			}
			else
			{
				SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
			}
		}
		// if valid data is available
		else if (BoneReference.HasValidSetup())
		{
			if (BoneReference.IsValidToEvaluate() && 
				InPose.GetPose().IsValid() &&
				ensureMsgf(InPose.GetPose().IsValidIndex(BoneReference.CachedCompactPoseIndex), TEXT("Invalid Cached Pose : Name %s(Bone Index (%d), Cached (%d))"), *BoneReference.BoneName.ToString(), BoneReference.BoneIndex, BoneReference.CachedCompactPoseIndex.GetInt()))
			{
				OutTargetTransform = TargetOffset * InPose.GetComponentSpaceTransform(BoneReference.CachedCompactPoseIndex);
			}
			else
			{
				SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
			}
		}
		else
		{
			SetComponentSpaceOffset(TargetOffset, InComponentToWorld, OutTargetTransform);
		}

		return OutTargetTransform;
	}
};

