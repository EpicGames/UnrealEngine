// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "CachedAnimData.generated.h"

class UAnimInstance;

/**
 * This file contains a number of helper structures that can be used to process state-machine-
 * related data in C++. This includes relevancy, state weights, animation time etc.
 */

USTRUCT(BlueprintType)
struct FCachedAnimStateData
{
	GENERATED_USTRUCT_BODY()

	FCachedAnimStateData()
		: MachineIndex(INDEX_NONE)
		, StateIndex(INDEX_NONE)
		, bInitialized(false)
	{}

	/** Name of StateMachine State is in */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateMachineName;

	/** Name of State to Cache */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateName;

	/** Did it find a matching StateMachine and State in the AnimGraph? */
	ENGINE_API bool IsValid(UAnimInstance& InAnimInstance) const;
	
	/** Is the State Machine relevant? (Has any weight) */
	ENGINE_API float IsMachineRelevant(UAnimInstance& InAnimInstance) const;
	
	/** Global weight of state in AnimGraph */
	ENGINE_API float GetGlobalWeight(UAnimInstance& InAnimInstance) const;

	/** Local weight of state inside of state machine. */
	ENGINE_API float GetWeight(UAnimInstance& InAnimInstance) const;

	/** Is State Full Weight? */
	ENGINE_API bool IsFullWeight(UAnimInstance& InAnimInstance) const;

	/** Is State relevant? */
	ENGINE_API bool IsRelevant(UAnimInstance& InAnimInstance) const;

	/** Is State active? */
	ENGINE_API bool IsActiveState(class UAnimInstance& InAnimInstance) const;

	ENGINE_API bool WasAnimNotifyStateActive(UAnimInstance& InAnimInstance, TSubclassOf<UAnimNotifyState> AnimNotifyStateType) const;
		
private:
	mutable int32 MachineIndex;
	mutable int32 StateIndex;
	mutable bool bInitialized;
};

USTRUCT(BlueprintType)
struct FCachedAnimStateArray
{
	GENERATED_USTRUCT_BODY()

	FCachedAnimStateArray()
		: bCheckedValidity(false)
		, bCachedIsValid(true)
		, bHasMultipleStateMachineEntries(false)
	{}

	/** Array of states */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	TArray<FCachedAnimStateData> States;

	/** Returns the total local weight of all states. If the definition contains more than on state machine, the result can be larger than 1 */
	ENGINE_API float GetTotalWeight(UAnimInstance& InAnimInstance) const;
	ENGINE_API bool IsFullWeight(UAnimInstance& InAnimInstance) const;
	ENGINE_API bool IsRelevant(UAnimInstance& InAnimInstance) const;

private:
	ENGINE_API bool IsValid(UAnimInstance& InAnimInstance) const;
	mutable bool bCheckedValidity;
	mutable bool bCachedIsValid;
	mutable bool bHasMultipleStateMachineEntries;
};

USTRUCT(BlueprintType)
struct FCachedAnimAssetPlayerData
{
	GENERATED_USTRUCT_BODY()

	FCachedAnimAssetPlayerData()
		: Index(INDEX_NONE)
		, bInitialized(false)
	{}

	/** Name of StateMachine State is in */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateMachineName;

	/** Name of State to Cache */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateName;

	ENGINE_API float GetAssetPlayerTime(UAnimInstance& InAnimInstance) const;

	ENGINE_API float GetAssetPlayerTimeRatio(UAnimInstance& InAnimInstance) const;

private:
	ENGINE_API void CacheIndices(UAnimInstance& InAnimInstance) const;

	mutable int32 Index;
	mutable bool bInitialized;
};

USTRUCT(BlueprintType)
struct FCachedAnimRelevancyData
{
	GENERATED_USTRUCT_BODY()

	FCachedAnimRelevancyData()
		: MachineIndex(INDEX_NONE)
		, StateIndex(INDEX_NONE)
		, bInitialized(false)
	{}

	/** Name of StateMachine State is in */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateMachineName;

	/** Name of State to Cache */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateName;

	ENGINE_API float GetRelevantAnimTime(UAnimInstance& InAnimInstance) const;
	ENGINE_API float GetRelevantAnimTimeFraction(UAnimInstance& InAnimInstance) const;
	ENGINE_API float GetRelevantAnimTimeRemaining(UAnimInstance& InAnimInstance) const;
	ENGINE_API float GetRelevantAnimTimeRemainingFraction(UAnimInstance& InAnimInstance) const;

private:
	ENGINE_API void CacheIndices(UAnimInstance& InAnimInstance) const;

private:
	mutable int32 MachineIndex;
	mutable int32 StateIndex;
	mutable bool bInitialized;
};

USTRUCT(BlueprintType)
struct FCachedAnimTransitionData
{
	GENERATED_USTRUCT_BODY()

	FCachedAnimTransitionData()
		: MachineIndex(INDEX_NONE)
		, TransitionIndex(INDEX_NONE)
		, bInitialized(false)
	{}

	/** Name of StateMachine State is in */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName StateMachineName;

	/** Name of From State to Cache */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName FromStateName;

	/** Name of To State to Cache */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "State Machine")
	FName ToStateName;

	ENGINE_API float GetCrossfadeDuration(UAnimInstance& InAnimInstance) const;
	ENGINE_API float GetTimeElapsed(UAnimInstance& InAnimInstance) const;
	ENGINE_API float GetTimeElapsedFraction(UAnimInstance& InAnimInstance) const;

private:
	ENGINE_API void CacheIndices(UAnimInstance& InAnimInstance) const;

private:
	mutable int32 MachineIndex;
	mutable int32 TransitionIndex;
	mutable bool bInitialized;
};
