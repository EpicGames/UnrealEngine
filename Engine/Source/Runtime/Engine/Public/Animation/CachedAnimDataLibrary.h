// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/CachedAnimData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "CachedAnimDataLibrary.generated.h"

class UAnimInstance;

/**
 *	A library of commonly used functionality from the CachedAnimData family, exposed to blueprint.
 */
UCLASS(meta = (ScriptName = "CachedAnimDataLibrary"), MinimalAPI)
class UCachedAnimDataLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/*** CachedAnimStateData ***/

	// Returns whether a state is relevant (specified in the provided FCachedAnimStateData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API bool StateMachine_IsStateRelevant(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateData& CachedAnimStateData);

	// Returns the weight of a state, relative to its state machine (specified in the provided FCachedAnimStateData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetLocalWeight(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateData& CachedAnimStateData);

	// Returns the weight of a state, relative to the graph (specified in the provided FCachedAnimStateData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetGlobalWeight(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateData& CachedAnimStateData);

	/*** CachedAnimStateArray ***/

	// Returns the summed weight of a state or states, relative to their state machine (specified in the provided FCachedAnimStateArray)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetTotalWeight(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateArray& CachedAnimStateArray);

	// Returns true when the weight of the input state (or summed weight for multiple input states) is 1.0 of greater (specified in the provided FCachedAnimStateArray)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API bool StateMachine_IsFullWeight(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateArray& CachedAnimStateArray);
	
	// Returns true when the input state, or states, have any weight (specified in the provided FCachedAnimStateArray)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API bool StateMachine_IsRelevant(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimStateArray& CachedAnimStateArray);

	/*** CachedAnimAssetPlayerData ***/

	// Gets the accumulated time, in seconds, of the asset player in the specified state. Assumes only one player in the state (specified in the provided FCachedAnimAssetPlayerData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetAssetPlayerTime(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimAssetPlayerData& CachedAnimAssetPlayerData);

	// Gets the accumulated time, as a fraction, of the asset player in the specified state. Assumes only one player in the state (specified in the provided FCachedAnimAssetPlayerData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetAssetPlayerTimeRatio(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimAssetPlayerData& CachedAnimAssetPlayerData);

	/*** CachedAnimRelevancyData ***/

	// Gets the accumulated time, in seconds, of the most relevant asset player in the specified state (specified in the provided FCachedAnimRelevancyData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetRelevantAnimTime(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimRelevancyData& CachedAnimRelevancyData);
		
	// Gets the time to the end of the asset, in seconds, of the most relevant asset player in the specified state (specified in the provided FCachedAnimRelevancyData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetRelevantAnimTimeRemaining(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimRelevancyData& CachedAnimRelevancyData);

	// Gets the time to the end of the asset, as a fraction, of the most relevant asset player in the specified state (specified in the provided FCachedAnimRelevancyData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetRelevantAnimTimeRemainingFraction(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimRelevancyData& CachedAnimRelevancyData);

	/*** CachedAnimTransitionData ***/

	// Gets the crossfade duration of the transition between the two input states. If multiple transition rules exist, the first will be returned (specified in the provided FCachedAnimTransitionData)
	UFUNCTION(BlueprintPure, Category = "Utilities|Animation|StateMachine")
	static ENGINE_API float StateMachine_GetCrossfadeDuration(UAnimInstance* InAnimInstance, UPARAM(ref) const FCachedAnimTransitionData& CachedAnimTransitionData);

};

