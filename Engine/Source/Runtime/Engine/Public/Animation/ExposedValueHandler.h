// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"

#include "ExposedValueHandler.generated.h"

struct FPropertyAccessLibrary;
class UClass;
struct FAnimationBaseContext;
class FCheckFastPathLatentCommand;

UENUM()
enum class EPostCopyOperation : uint8
{
	None,

	LogicalNegateBool,
};

USTRUCT()
struct FExposedValueCopyRecord
{
	GENERATED_BODY()

	FExposedValueCopyRecord() = default;

	FExposedValueCopyRecord(int32 InCopyIndex, EPostCopyOperation InPostCopyOperation, bool bInOnlyUpdateWhenActive)
		: CopyIndex(InCopyIndex)
		, PostCopyOperation(InPostCopyOperation)
		, bOnlyUpdateWhenActive(bInOnlyUpdateWhenActive)
	{
	}

	UPROPERTY()
	int32 CopyIndex = INDEX_NONE;

	UPROPERTY()
	EPostCopyOperation PostCopyOperation = EPostCopyOperation::None;

	UPROPERTY()
	bool bOnlyUpdateWhenActive = false;
};

// An exposed value updater
USTRUCT()
struct FExposedValueHandler
{
	GENERATED_USTRUCT_BODY()

	friend struct FAnimSubsystem_Base;

	FExposedValueHandler() = default;

	// Direct data access to property in anim instance
	UE_DEPRECATED(5.3, "This has been moved to FAnimNodeExposedValueHandler")
	static ENGINE_API TArray<FExposedValueCopyRecord> CopyRecords;

	// function pointer if BoundFunction != NAME_None
	UE_DEPRECATED(5.3, "This has been moved to FAnimNodeExposedValueHandler")
	static ENGINE_API TObjectPtr<UFunction> Function;

	UE_DEPRECATED(5.3, "This has been moved to FAnimNodeExposedValueHandler")
	static ENGINE_API const FPropertyAccessLibrary* PropertyAccessLibrary;

	UE_DEPRECATED(5.3, "This has been moved to FAnimNodeExposedValueHandler")
	static ENGINE_API FName BoundFunction;

	UE_DEPRECATED(5.3, "This function was meant for internal use, but was public. Initialization should now use the internal FAnimNodeExposedValueHandler")
	static ENGINE_API void ClassInitialization(TArray<FExposedValueHandler>& Handlers, UClass* InClass);

	UE_DEPRECATED(5.3, "This function was meant for internal use, but was public. Initialization should now use the internal FAnimNodeExposedValueHandler")
	ENGINE_API void Initialize(UClass* InClass, const FPropertyAccessLibrary& InPropertyAccessLibrary);

	// Execute the handler
	ENGINE_API void Execute(const FAnimationBaseContext& Context) const;

	// Access the handler struct
	const UScriptStruct* GetHandlerStruct() const { return HandlerStruct; }

	// Access the handler
	const FAnimNodeExposedValueHandler* GetHandler() const { return Handler; }

private:
	// The type of the handler
	UScriptStruct* HandlerStruct = nullptr;

	// Ptr to actual handler (property of the anim instance)
	FAnimNodeExposedValueHandler* Handler = nullptr;
};

USTRUCT()
struct FAnimNodeExposedValueHandler
{
	GENERATED_BODY()

	friend struct FAnimSubsystem_Base;

	virtual ~FAnimNodeExposedValueHandler() = default;

	// Set up the handler when the class is loaded
	virtual void Initialize(const UClass* InClass) {}

	// Execute the handler to set specified values in a node
	virtual void Execute(const FAnimationBaseContext& InContext) const {}
};

USTRUCT()
struct FAnimNodeExposedValueHandler_Base : public FAnimNodeExposedValueHandler
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_Base;

	// FAnimNodeExposedValueHandler interface
	ENGINE_API virtual void Initialize(const UClass* InClass) override;
	ENGINE_API virtual void Execute(const FAnimationBaseContext& InContext) const override;

	// function pointer if BoundFunction != NAME_None
	UPROPERTY()
	TObjectPtr<UFunction> Function;

	// The function to call to update associated properties (can be NAME_None)
	UPROPERTY()
	FName BoundFunction;
};

USTRUCT()
struct FAnimNodeExposedValueHandler_PropertyAccess : public FAnimNodeExposedValueHandler_Base
{
	GENERATED_BODY()

	friend class UAnimBlueprintExtension_Base;

	// FAnimNodeExposedValueHandler interface
	virtual void Initialize(const UClass* InClass) override;
	virtual void Execute(const FAnimationBaseContext& InContext) const override;

	// Direct data access to property in anim instance
	UPROPERTY()
	TArray<FExposedValueCopyRecord> CopyRecords;

	// Cached property access library ptr
	const FPropertyAccessLibrary* PropertyAccessLibrary;
};
