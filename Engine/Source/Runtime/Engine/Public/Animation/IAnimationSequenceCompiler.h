// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"

class UAnimSequence;

namespace UE::Anim
{
	class IAnimSequenceCompilingManager
	{
	public:
		/** Ensures that, in case of any, compilation tasks for the provided Animation Sequences has been completed */
		ENGINE_API static void FinishCompilation(TArrayView<UAnimSequence* const> InAnimSequences);
	};
}

#endif // WITH_EDITOR