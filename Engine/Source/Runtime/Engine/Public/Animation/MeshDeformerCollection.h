// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "MeshDeformerCollection.generated.h"

class UMeshDeformer;

/** A simple collection of Mesh Deformers */
UCLASS(MinimalAPI, BlueprintType)
class UMeshDeformerCollection : public UDataAsset
{
	GENERATED_BODY()
public:
	
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Mesh Deformer Collections")
	FString Description;
#endif
	
	UPROPERTY(EditAnywhere, Category = "Mesh Deformer Collections")
	TArray<TSoftObjectPtr<UMeshDeformer>> MeshDeformers;

	UPROPERTY(EditAnywhere, Category = "Mesh Deformer Collections")
	TArray<TObjectPtr<class UMeshDeformerCollection>> MeshDeformerCollections;

	TArray<TSoftObjectPtr<UMeshDeformer>> GetMeshDeformers() const;

private:
	void GetMeshDeformers_Internal(
		TSet<TObjectPtr<const UMeshDeformerCollection>>& InOutVisitedCollection, 
		TArray<TSoftObjectPtr<UMeshDeformer>>& InOutMeshDeformers
		) const;
};