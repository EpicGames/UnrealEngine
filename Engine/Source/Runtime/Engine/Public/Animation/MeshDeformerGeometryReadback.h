// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API ENGINE_API

struct FMeshDescription;

struct FMeshDeformerGeometryReadbackVertexDataArrays
{
	int32 LODIndex = INDEX_NONE;
	TArray<FVector3f> Positions;
	TArray<FVector3f> Normals;
	TArray<FVector3f> Tangents;
	TArray<FVector4f> Colors;
};

struct FMeshDeformerGeometryReadbackRequest
{
	UE_API ~FMeshDeformerGeometryReadbackRequest();

	TFunction<void(const FMeshDescription&)> MeshDescriptionCallback_AnyThread;
	bool bMeshDescriptionHandled = false;
	
	TFunction<void(const FMeshDeformerGeometryReadbackVertexDataArrays&)> VertexDataArraysCallback_AnyThread;
	bool bVertexDataArraysHandled = false;
};

#undef UE_API
