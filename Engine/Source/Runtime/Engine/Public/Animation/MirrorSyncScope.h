// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNodeMessages.h"
#include "Animation/MirrorDataTable.h"

namespace UE { namespace Anim {

class FAnimSyncGroupScope;

class FAnimNotifyMirrorContext : public IAnimNotifyEventContextDataInterface
{
	DECLARE_NOTIFY_CONTEXT_INTERFACE(FAnimNotifyMirrorContext)
	public:
	FAnimNotifyMirrorContext() {} 
	FAnimNotifyMirrorContext(const UMirrorDataTable* MirrorDataTable) : MirrorTable(MirrorDataTable) {bAnimationMirrored = MirrorDataTable != nullptr;} 
	bool bAnimationMirrored = true; 

	TWeakObjectPtr<const UMirrorDataTable> MirrorTable; 
};
	
// Scoped graph message used to synchronize mirroring 
class FMirrorSyncScope : public IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FMirrorSyncScope, ENGINE_API);
public:
	ENGINE_API FMirrorSyncScope(const FAnimationBaseContext& InContext, const UMirrorDataTable* InMirrorDataTable);
	ENGINE_API virtual ~FMirrorSyncScope();
	ENGINE_API int32 GetMirrorScopeDepth() const;
	ENGINE_API virtual TUniquePtr<const IAnimNotifyEventContextDataInterface> MakeUniqueEventContextData() const override;
private:
	const UMirrorDataTable* MirrorDataTable = nullptr;
	int32 MirrorScopeDepth = 1;
	const UMirrorDataTable* OuterScopeMirrorDataTable = nullptr;
	TWeakPtr<FAnimSyncGroupScope> AnimSyncGroupScope;
};

}}	// namespace UE::Anim
