// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "UObject/ObjectMacros.h"
#include "MotionTrajectoryTypes.generated.h"

USTRUCT(BlueprintType, Category="Motion Trajectory")
struct UE_DEPRECATED(5.6, "FTrajectorySample has been deprecated. Instead use FTransformTrajectorySample.") FTrajectorySample
{
	GENERATED_BODY()

	// The relative accumulated time that this sample is associated with
	// Zero value for instantaneous, negative values for the past, and positive values for the future
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Trajectory")
	float AccumulatedSeconds = 0.f;

	// Position relative to the sampled in-motion object
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Trajectory")
	FTransform Transform = FTransform::Identity;

	// Linear velocity relative to the sampled in-motion object
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Trajectory")
	FVector LinearVelocity = FVector::ZeroVector;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Linear interpolation of all parameters of two trajectory samples
	ENGINE_API FTrajectorySample Lerp(const FTrajectorySample& Sample, float Alpha) const;

	// Centripetal Catmull–Rom spline interpolation of all parameters of two trajectory samples
	ENGINE_API FTrajectorySample SmoothInterp(const FTrajectorySample& PrevSample
		, const FTrajectorySample& Sample
		, const FTrajectorySample& NextSample
		, float Alpha) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// Concatenates DeltaTransform before the current transform is applied and shifts the accumulated time by 
	// DeltaSeconds
	ENGINE_API void PrependOffset(const FTransform DeltaTransform, float DeltaSeconds);

	ENGINE_API void TransformReferenceFrame(const FTransform DeltaTransform);

	// Determines if all sample properties are zeroed
	ENGINE_API bool IsZeroSample() const;
};