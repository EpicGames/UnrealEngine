// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/IsSorted.h"

#define ENABLE_ANIM_CURVE_PROFILING 0

#if ENABLE_ANIM_CURVE_PROFILING
#include "Stats/Stats.h"
#endif

#if ENABLE_ANIM_CURVE_PROFILING
#define CURVE_PROFILE_CYCLE_COUNTER(Stat) QUICK_SCOPE_CYCLE_COUNTER(Stat)
#else
#define CURVE_PROFILE_CYCLE_COUNTER(Stat)
#endif

#define DO_ANIM_NAMED_VALUE_SORTING_CHECKS		0
#define DO_ANIM_NAMED_VALUE_DUPLICATE_CHECKS	0

namespace UE::Anim
{

struct FNamedValueArrayUtils;

/**
 * Container of lazily-sorted name/value pairs.
 * Used to perform efficient merge operations.
 * Assumes that InElementType has an accessible member: FName Name. 
 */
template<typename InAllocatorType, typename InElementType>
struct TNamedValueArray
{
	typedef InAllocatorType AllocatorType;
	typedef InElementType ElementType;

	friend struct FNamedValueArrayUtils;

	/**
	 * Add a named element.
	 * Note that this should only really be used when building a fresh value array, as using this during runtime can
	 * introduce duplicate values.
	 * Asserts in debug builds if duplicate values are present
	 */
	template<typename... ArgTypes>
	void Add(ArgTypes&&... Args)
	{
		Elements.Emplace(Forward<ArgTypes>(Args)...);
		bSorted = false;

		CheckDuplicates();
	}

	/**
	 * Add an array of named elements.
	 * Note that this should only really be used when building a fresh array, as using this during runtime can
	 * introduce duplicate values.
	 * Asserts in debug builds if duplicate values are present
	 */
	void AppendNames(TConstArrayView<FName> InNameArray)
	{
		Elements.Reserve(Elements.Num() + InNameArray.Num());
		for(const FName& Name : InNameArray)
		{
			Elements.Emplace(Name);
		}
		bSorted = false;

		CheckDuplicates();
	}

	/**
	 * Add an array of named elements.
	 * Note that this should only really be used when building a fresh array, as using this during runtime can
	 * introduce duplicate values.
	 * Asserts in debug builds if duplicate values are present
	 */
	void AppendNames(std::initializer_list<const FName> InInputArgs)
	{
		Elements.Reserve(Elements.Num() + InInputArgs.size());
		for(const FName& Name : InInputArgs)
		{
			Elements.Emplace(Name);
		}
		bSorted = false;

		CheckDuplicates();
	}

	/** Reset the internal allocations */
	void Empty()
	{
		Elements.Reset();
		bSorted = false;
	}

	/** Reserves memory for InNumElements */
	void Reserve(int32 InNumElements)
	{
		Elements.Reserve(InNumElements);
	}

	/**
	 * Check whether an element is present for the supplied name
	 * Note that this performs a binary search per-call.
	 * @param	InName	the name of the element to check
	 * @return  true if an element with the supplied name is present
	 */
	bool HasElement(FName InName) const
	{
		return Find(InName) != nullptr;
	}

	/**
	 * Iterate over each element calling InPredicate for each.
	 * Predicate: (const ElementType&) -> void
	 */
	template<typename PredicateType>
	void ForEachElement(PredicateType InPredicate) const
	{
		for(const ElementType& Element : Elements)
		{
			InPredicate(Element);
		}
	}
	
	/** @returns the number of elements */
	int32 Num() const
	{
		return Elements.Num();
	}

	/** @returns the max number of elements reserved in the array */
	int32 Max() const
	{
		return Elements.Max();
	}

	/** Compacts the memory for the elements based on what was actually used */
	void Shrink()
	{
		return Elements.Shrink();
	}

protected:
	// Sort by FName - Note: this is not stable across serialization
	struct FElementSortPredicate
	{
		inline bool operator()(const ElementType& InElement0, const ElementType& InElement1) const
		{
			return InElement0.Name.FastLess(InElement1.Name);
		}
	};

	// Sorts the elements if they are not yet sorted
	void SortElementsIfRequired() const
	{
		if(!bSorted)
		{
			CURVE_PROFILE_CYCLE_COUNTER(SortElementsIfRequired);

			Algo::Sort(Elements, FElementSortPredicate());
			bSorted = true;
		}
	}

	// Checks whether the sorting invariant is correct
	void CheckSorted() const
	{
#if DO_ANIM_NAMED_VALUE_SORTING_CHECKS
		if(bSorted)
		{
			check(Algo::IsSorted(Elements, FElementSortPredicate()));
		}
#endif
	}

	// Checks whether the 'no duplicates' invariant is correct
	void CheckDuplicates() const
	{
#if DO_ANIM_NAMED_VALUE_DUPLICATE_CHECKS
		for(int32 Index0 = 0; Index0 < Elements.Num(); ++Index0)
		{
			for(int32 Index1 = 0; Index1 < Elements.Num(); ++Index1)
			{
				if(Index0 != Index1 && Elements[Index0].Name == Elements[Index1].Name)
				{
					checkf(false, TEXT("Duplicate curve entry found: %s"), *Elements[Index0].Name.ToString());
				}
			}
		}
#endif
	}

	/** Finds index of the element with the specified name, disregarding enabled state */
	int32 IndexOf(FName InName) const
	{
		SortElementsIfRequired();

		return Algo::BinarySearchBy(Elements, ElementType(InName), &ElementType::Name, FElementSortPredicate());
	}

	/** Finds the element with the specified name (const) */
	const ElementType* Find(FName InName) const
	{
		const int32 ElementIndex = IndexOf(InName);
		if(ElementIndex != INDEX_NONE)
		{
			return &Elements[ElementIndex]; 
		}
		return nullptr;
	}

	/** Finds the element with the specified name */
	ElementType* Find(FName InName)
	{
		const int32 ElementIndex = IndexOf(InName);
		if(ElementIndex != INDEX_NONE)
		{
			return &Elements[ElementIndex]; 
		}
		return nullptr;
	}

protected:
	// Named elements, sorted by name
	mutable TArray<ElementType, AllocatorType> Elements;

	// Whether the elements are sorted
	mutable bool bSorted = false;
};

// Flags passed during union operations
enum class ENamedValueUnionFlags : uint8
{
	// No flags
	None		= 0,

	// First argument is valid
	ValidArg0	= 0x01,

	// Second argument is valid
	ValidArg1	= 0x02,

	// Both arguments are valid
	BothArgsValid = ValidArg0 | ValidArg1,
};

struct FNamedValueArrayUtils
{
	// Helper function
	// Uses a simple 'tape merge'
	// Performs an operation per-element on the two value arrays. Writes result to InOutValueArray0.
	// InValueArray0 will be the union of the two value arrays after the operation is completed (i.e.
	// new elements in InValueArray1 are added to InValueArray0)
	// InPredicate is called on all elements that are added to or already existing in InOutValueArray0, with
	// appropriate flags.
	template<typename PredicateType, typename AllocatorTypeResult, typename ElementTypeResult, typename AllocatorTypeParam, typename ElementTypeParam>
	static void Union(TNamedValueArray<AllocatorTypeResult, ElementTypeResult>& InOutValueArray0, const TNamedValueArray<AllocatorTypeParam, ElementTypeParam>& InValueArray1, PredicateType InPredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_Union2Params);

		// Check arrays are not overlapping
		check((void*)&InOutValueArray0 != (void*)&InValueArray1);

		const int32 NumElements0 = InOutValueArray0.Num();	// ValueArray1 elements remain constant, but ValueArray0 can have entries added.
		const int32 NumElements1 = InValueArray1.Num();

		// Nothing to merge, just run the predicate on every elements.
		if (NumElements1 == 0)
		{
			InOutValueArray0.SortElementsIfRequired();
			ElementTypeParam DefaultElement;
			for (auto& Element : InOutValueArray0.Elements)
			{
				DefaultElement.Name = Element.Name;
				InPredicate(Element, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);
			}
			return;
		}

		// Sort both input arrays if required
		InOutValueArray0.SortElementsIfRequired();
		InValueArray1.SortElementsIfRequired();

		// Reserve memory for 1.5x combined curve counts.
		// This can overestimate in some circumstances, but it handles the common cases which are:
		// - One input is empty, the other not
		// - Both inputs are non-empty but do not share most elements
		int32 ReserveSize = FMath::Max(NumElements0, NumElements1);
		ReserveSize += ReserveSize / 2;
		InOutValueArray0.Reserve(ReserveSize);

		// Use pointers to iterate as this uses fewer registers and this code is very hot
		ElementTypeResult* RESTRICT ElementPtr0 = InOutValueArray0.Elements.GetData();
		const ElementTypeParam* RESTRICT ElementPtr1 = InValueArray1.Elements.GetData();

		const ElementTypeResult* RESTRICT ElementEndPtr0 = ElementPtr0 + NumElements0;
		const ElementTypeParam* RESTRICT ElementEndPtr1 = ElementPtr1 + NumElements1;

		// A default element we re-use when an element from one of the two inputs is missing
		ElementTypeParam DefaultElement;

		// When we reach the end of either input arrays, we stop the tape merge and copy what remains
		bool bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;

		// Perform dual-iteration on the two sorted arrays
		while (!bIsDone)
		{
			if (ElementPtr0->Name == ElementPtr1->Name)
			{
				// Elements match, run predicate and increment both indices
				InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::BothArgsValid);

				++ElementPtr0;
				++ElementPtr1;

				bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;
			}
			else if (ElementPtr0->Name.FastLess(ElementPtr1->Name))
			{
				// ValueArray0 element is earlier, so run predicate with only ValueArray0 and increment ValueArray0
				DefaultElement.Name = ElementPtr0->Name;

				InPredicate(*ElementPtr0, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);

				++ElementPtr0;

				bIsDone = ElementPtr0 == ElementEndPtr0;
			}
			else
			{
				// ValueArray1 element is earlier, so add to ValueArray0, run predicate with only second and increment ValueArray1
				const int32 ElementIndex0 = UE_PTRDIFF_TO_INT32(ElementPtr0 - InOutValueArray0.Elements.GetData());
				InOutValueArray0.Elements.InsertUninitialized(ElementIndex0);

				// Refresh pointers since they might have changed
				ElementPtr0 = InOutValueArray0.Elements.GetData() + ElementIndex0;
				ElementEndPtr0 = InOutValueArray0.Elements.GetData() + InOutValueArray0.Elements.Num();

				// We use placement new to make sure the constructor is inlined to reduce redundant work
				new(ElementPtr0) ElementTypeResult();
				ElementPtr0->Name = ElementPtr1->Name;

				InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);

				++ElementPtr0;	// Increment this as well since we've inserted
				++ElementPtr1;

				bIsDone = ElementPtr1 == ElementEndPtr1;
			}
		}

		// Tape merge is done, process anything that might be remaining
		if (ElementPtr0 < ElementEndPtr0)
		{
			// Reached end of ValueArray1 with remaining elements in ValueArray0, call predicate on each
			for (; ElementPtr0 < ElementEndPtr0; ++ElementPtr0)
			{
				DefaultElement.Name = ElementPtr0->Name;

				InPredicate(*ElementPtr0, DefaultElement, UE::Anim::ENamedValueUnionFlags::ValidArg0);
			}
		}
		else if (ElementPtr1 < ElementEndPtr1)
		{
			// Reached end of ValueArray0 with remaining in ValueArray1, we can just copy the remainder of ValueArray1
			const int32 NumResults = InOutValueArray0.Elements.Num();
			const int32 NumNewElements = static_cast<int32>(ElementEndPtr1 - ElementPtr1);
			InOutValueArray0.Elements.Reserve(NumResults + NumNewElements);
			InOutValueArray0.Elements.AddUninitialized(NumNewElements);

			// Refresh pointers since they might have changed
			ElementPtr0 = InOutValueArray0.Elements.GetData() + NumResults;
			ElementEndPtr0 = ElementPtr0 + NumNewElements;

			for (; ElementPtr1 < ElementEndPtr1; ++ElementPtr0, ++ElementPtr1)
			{
				// We use placement new to make sure the constructor is inlined to reduce redundant work
				new(ElementPtr0) ElementTypeResult();
				ElementPtr0->Name = ElementPtr1->Name;

				InPredicate(*ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);
			}
		}

		InOutValueArray0.CheckSorted();
	}

	// Helper function
	// Uses a simple 'tape merge'
	// Performs an operation per-element on the two value arrays. Writes result to InOutValueArray0.
	// InValueArray0 will be the union of the two value arrays after the operation is completed (i.e.
	// new elements in InValueArray1 are added to InValueArray0)
	// Performs a simple copy for each element
	template<typename AllocatorTypeResult, typename ElementType, typename AllocatorTypeParam>
	static void Union(TNamedValueArray<AllocatorTypeResult, ElementType>& InOutValueArray0, const TNamedValueArray<AllocatorTypeParam, ElementType>& InValueArray1)
	{
		// Early out if we just want to perform a simple copy
		if(InOutValueArray0.Num() == 0 && InValueArray1.Num() > 0)
		{
			InOutValueArray0.Elements = InValueArray1.Elements;
			InOutValueArray0.bSorted = InValueArray1.bSorted;
			return;
		}

		Union(InOutValueArray0, InValueArray1, [](ElementType& Element0, const ElementType& Element1, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if(EnumHasAnyFlags(InFlags, UE::Anim::ENamedValueUnionFlags::ValidArg1))
			{
				Element0 = Element1;
			}
		});
	}
	
	// Helper function
	// Uses a simple 'tape merge'
	// Performs an operation per-element on the two value arrays. Writes result to OutResultValueArray.
	// OutResultValueArray will be the union of the two value arrays after the operation is completed.
	// InPredicate is called on all elements that are added to OutResultValueArray, with appropriate flags.
	template<typename PredicateType, typename AllocatorTypeResult, typename ElementTypeResult, typename AllocatorType0, typename ElementType0, typename AllocatorType1, typename ElementType1>
	static void Union(
		TNamedValueArray<AllocatorTypeResult, ElementTypeResult>& OutResultValueArray,
		const TNamedValueArray<AllocatorType0, ElementType0>& InValueArray0,
		const TNamedValueArray<AllocatorType1, ElementType1>& InValueArray1,
		PredicateType InPredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_Union3Params);

		// Check arrays are not overlapping
		check((void*)&OutResultValueArray != (void*)&InValueArray0);
		check((void*)&OutResultValueArray != (void*)&InValueArray1);
		check((void*)&InValueArray0 != (void*)&InValueArray1);

		// Make sure result is clear
		OutResultValueArray.Elements.Reset();

		const int32 NumElements0 = InValueArray0.Num();
		const int32 NumElements1 = InValueArray1.Num();

		// Sort both input arrays if required
		InValueArray0.SortElementsIfRequired();
		InValueArray1.SortElementsIfRequired();

		// Reserve memory for 1.5x combined curve counts.
		// This can overestimate in some circumstances, but it handles the common cases which are:
		// - One input is empty, the other not
		// - Both inputs are non-empty but do not share most elements
		int32 ReserveSize = FMath::Max(NumElements0, NumElements1);
		ReserveSize += ReserveSize / 2;
		OutResultValueArray.Reserve(ReserveSize);

		// Use pointers to iterate as this uses fewer registers and this code is very hot
		const ElementType0* RESTRICT ElementPtr0 = InValueArray0.Elements.GetData();
		const ElementType1* RESTRICT ElementPtr1 = InValueArray1.Elements.GetData();

		const ElementType0* RESTRICT ElementEndPtr0 = ElementPtr0 + NumElements0;
		const ElementType1* RESTRICT ElementEndPtr1 = ElementPtr1 + NumElements1;

		// A default element we re-use when an element from one of the two inputs is missing
		ElementType0 DefaultElement0;
		ElementType1 DefaultElement1;

		// When we reach the end of either input arrays, we stop the tape merge and copy what remains
		bool bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;

		// Perform dual-iteration on the two sorted arrays
		while (!bIsDone)
		{
			ElementTypeResult NewResultElement;

			if (ElementPtr0->Name == ElementPtr1->Name)
			{
				// Elements match, run predicate and increment both indices
				NewResultElement.Name = ElementPtr0->Name;

				InPredicate(NewResultElement, *ElementPtr0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::BothArgsValid);

				++ElementPtr0;
				++ElementPtr1;

				bIsDone = ElementPtr0 == ElementEndPtr0 || ElementPtr1 == ElementEndPtr1;
			}
			else if (ElementPtr0->Name.FastLess(ElementPtr1->Name))
			{
				// ValueArray0 element is earlier, so run predicate with only ValueArray0 and increment ValueArray0
				NewResultElement.Name = ElementPtr0->Name;

				// Element 1 is missing, use stub
				DefaultElement1.Name = ElementPtr0->Name;

				InPredicate(NewResultElement, *ElementPtr0, DefaultElement1, UE::Anim::ENamedValueUnionFlags::ValidArg0);

				++ElementPtr0;

				bIsDone = ElementPtr0 == ElementEndPtr0;
			}
			else
			{
				// ValueArray1 element is earlier, so so run predicate with only ValueArray1 and increment ValueArray1
				NewResultElement.Name = ElementPtr1->Name;

				// Element 0 is missing, use stub
				DefaultElement0.Name = ElementPtr1->Name;

				InPredicate(NewResultElement, DefaultElement0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);

				++ElementPtr1;

				bIsDone = ElementPtr1 == ElementEndPtr1;
			}

			OutResultValueArray.Elements.Add(MoveTemp(NewResultElement));
		}

		// Tape merge is done, copy anything that might be remaining
		if (ElementPtr0 < ElementEndPtr0)
		{
			// Reached end of ValueArray1 with remaining elements in ValueArray0, we can just copy the remainder of ValueArray0
			const int32 NumResults = OutResultValueArray.Elements.Num();
			const int32 NumNewElements = (ElementEndPtr0 - ElementPtr0);
			OutResultValueArray.Elements.Reserve(NumResults + NumNewElements);
			OutResultValueArray.Elements.AddUninitialized(NumNewElements);

			ElementTypeResult* RESTRICT ResultElementPtr = OutResultValueArray.Elements.GetData() + NumResults;

			for (; ElementPtr0 < ElementEndPtr0; ++ResultElementPtr, ++ElementPtr0)
			{
				// Element 1 is missing, use stub
				DefaultElement1.Name = ElementPtr0->Name;

				// We use placement new to make sure the constructor is inlined to reduce redundant work
				new(ResultElementPtr) ElementTypeResult();
				ResultElementPtr->Name = ElementPtr0->Name;

				InPredicate(*ResultElementPtr, *ElementPtr0, DefaultElement1, UE::Anim::ENamedValueUnionFlags::ValidArg0);
			}
		}
		else if (ElementPtr1 < ElementEndPtr1)
		{
			// Reached end of ValueArray0 with remaining elements in ValueArray1, we can just copy the remainder of ValueArray1
			const int32 NumResults = OutResultValueArray.Elements.Num();
			const int32 NumNewElements = (ElementEndPtr1 - ElementPtr1);
			OutResultValueArray.Elements.Reserve(NumResults + NumNewElements);
			OutResultValueArray.Elements.AddUninitialized(NumNewElements);

			ElementTypeResult* RESTRICT ResultElementPtr = OutResultValueArray.Elements.GetData() + NumResults;

			for (; ElementPtr1 < ElementEndPtr1; ++ResultElementPtr, ++ElementPtr1)
			{
				// Element 0 is missing, use stub
				DefaultElement0.Name = ElementPtr1->Name;

				// We use placement new to make sure the constructor is inlined to reduce redundant work
				new(ResultElementPtr) ElementTypeResult();
				ResultElementPtr->Name = ElementPtr1->Name;

				InPredicate(*ResultElementPtr, DefaultElement0, *ElementPtr1, UE::Anim::ENamedValueUnionFlags::ValidArg1);
			}
		}

		// Insertion always proceeds in sorted order, so result is sorted by default
		OutResultValueArray.bSorted = true;

		OutResultValueArray.CheckSorted();
	}

	// Helper function
	// Calls predicate on all elements in the two passed-in value arrays.
	template<typename PredicateType, typename AllocatorType0, typename ElementType0, typename AllocatorType1, typename ElementType1>
	static void Union(const TNamedValueArray<AllocatorType0, ElementType0>& InValueArray0, const TNamedValueArray<AllocatorType1, ElementType1>& InValueArray1, PredicateType InPredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_UnionPredicate);

		// Check arrays are not overlapping
		checkSlow((void*)&InValueArray0 != (void*)&InValueArray1);

		// Sort both input arrays if required
		InValueArray0.SortElementsIfRequired();
		InValueArray1.SortElementsIfRequired();
		
		const int32 NumElements0 = InValueArray0.Num();
		const int32 NumElements1 = InValueArray1.Num();

		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

		// Perform dual-iteration on the two sorted arrays
		while(true)
		{
			if(ElementIndex0 == NumElements0 && ElementIndex1 < NumElements1)
			{
				// Reached end of ValueArray0 with remaining in ValueArray1, we can just iterate over the remainder of ValueArray1
				for( ; ElementIndex1 < NumElements1; ++ElementIndex1)
				{
					const ElementType1* RESTRICT Element1 = &InValueArray1.Elements[ElementIndex1];
					ElementType0 DefaultElement0;
					DefaultElement0.Name = Element1->Name;

					InPredicate(DefaultElement0, *Element1, UE::Anim::ENamedValueUnionFlags::ValidArg1);
				}
				break;
			}
			else if(ElementIndex1 == NumElements1 && ElementIndex0 < NumElements0)
			{
				// Reached end of ValueArray1 with remaining in ValueArray0, we can just iterate over the remainder of ValueArray0
				for( ; ElementIndex0 < NumElements0; ++ElementIndex0)
				{
					const ElementType0* RESTRICT Element0 = &InValueArray0.Elements[ElementIndex0];
					ElementType1 DefaultElement1;
					DefaultElement1.Name = Element0->Name;

					InPredicate(*Element0, DefaultElement1, UE::Anim::ENamedValueUnionFlags::ValidArg0);
				}	
				break;
			}
			else if(ElementIndex0 == NumElements0 && ElementIndex1 == NumElements1)
			{
				// Reached end of both, exit
				break;
			}
			
			const ElementType0* RESTRICT Element0 = &InValueArray0.Elements[ElementIndex0];
			const ElementType1* RESTRICT Element1 = &InValueArray1.Elements[ElementIndex1];
			
			if(Element0->Name == Element1->Name)
			{
				// Elements match, run predicate and increment both indices
				InPredicate(*Element0, *Element1, UE::Anim::ENamedValueUnionFlags::BothArgsValid);
				++ElementIndex0;
				++ElementIndex1;
			}
			else if(Element0->Name.FastLess(Element1->Name))
			{
				// ValueArray0 element is earlier, so run predicate with only ValueArray0 and increment ElementIndex0
				ElementType1 DefaultElement1;
				DefaultElement1.Name = Element0->Name;
				InPredicate(*Element0, DefaultElement1, UE::Anim::ENamedValueUnionFlags::ValidArg0);
				++ElementIndex0;
			}
			else
			{
				// ValueArray1 element is earlier, so run predicate with only ValueArray1 and increment ElementIndex1
				ElementType0 DefaultElement0;
				DefaultElement0.Name = Element1->Name;
				InPredicate(DefaultElement0, *Element1, UE::Anim::ENamedValueUnionFlags::ValidArg1);
				++ElementIndex1;
			}
		}
	}
	
	/**
	 * Calls predicate on all matching elements in the two passed-in value arrays.
	 * ValuePredicateType is a function of signature: (const ElementType0& InElement0, const ElementType1& InElement1) -> void
	 **/
	template<typename AllocatorType0, typename ElementType0, typename AllocatorType1, typename ElementType1, typename ValuePredicateType>
	static void Intersection(const TNamedValueArray<AllocatorType0, ElementType0>& InNamedValues0, const TNamedValueArray<AllocatorType1, ElementType1>& InNamedValues1, ValuePredicateType InValuePredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_Intersection);

		// Check arrays are not overlapping
		checkSlow((void*)&InNamedValues0 != (void*)&InNamedValues1);
		
		// Sort both inputs if required
		InNamedValues0.SortElementsIfRequired();
		InNamedValues1.SortElementsIfRequired();

		const int32 NumElements0 = InNamedValues0.Num();
		const int32 NumElements1 = InNamedValues1.Num();
		
		// Perform dual-iteration on the two sorted arrays
		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

		while(true)
		{
			if(ElementIndex0 == NumElements0 && ElementIndex1 < NumElements1)
			{
				// Reached end of array with remaining in bulk, we can just early out
				break;
			}
			else if(ElementIndex1 == NumElements1 && ElementIndex0 < NumElements0)
			{
				// Reached end of bulk with remaining in array, we can just early out
				break;
			}
			else if(ElementIndex0 == NumElements0 && ElementIndex1 == NumElements1)
			{
				// All elements exhausted, exit
				break;
			}

			const ElementType0* RESTRICT Element0 = &InNamedValues0.Elements[ElementIndex0];
			const ElementType1* RESTRICT Element1 = &InNamedValues1.Elements[ElementIndex1];
			
			if(Element0->Name == Element1->Name)
			{
				// Elements match so extract value
				InValuePredicate(*Element0, *Element1);
				++ElementIndex0;
				++ElementIndex1;
			}
			else if(Element0->Name.FastLess(Element1->Name))
			{
				// Element exists only in array, skip
				++ElementIndex0;
			}
			else
			{
				// Element exists only in bulk, skip
				++ElementIndex1;
			}
		}
	}

	/**
	 * Calls predicate on all elements of the first named value array that are not present in the second named value array: Result = (A - B).
	 * ValuePredicateType is a function of signature: (const ElementType0& InElement0) -> void
	 **/
	template<typename AllocatorType0, typename ElementType0, typename AllocatorType1, typename ElementType1, typename ValuePredicateType>
	static void Subtraction(const TNamedValueArray<AllocatorType0, ElementType0>& InNamedValues0, const TNamedValueArray<AllocatorType1, ElementType1>& InNamedValues1, ValuePredicateType InValuePredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_Subtraction);

		// Check arrays are not overlapping
		checkSlow((void*)&InNamedValues0 != (void*)&InNamedValues1);

		// Sort both inputs if required
		InNamedValues0.SortElementsIfRequired();
		InNamedValues1.SortElementsIfRequired();

		const int32 NumElements0 = InNamedValues0.Num();
		const int32 NumElements1 = InNamedValues1.Num();

		// Perform dual-iteration on the two sorted arrays
		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

		while (true)
		{
			if (ElementIndex0 == NumElements0 && ElementIndex1 < NumElements1)
			{
				// Reached end of the first array, we can just early out
				break;
			}
			else if (ElementIndex1 == NumElements1 && ElementIndex0 < NumElements0)
			{
				// Reached end of the second array, we can just early out
				break;
			}
			else if (ElementIndex0 == NumElements0 && ElementIndex1 == NumElements1)
			{
				// All elements exhausted, exit
				break;
			}

			const ElementType0* RESTRICT Element0 = &InNamedValues0.Elements[ElementIndex0];
			const ElementType1* RESTRICT Element1 = &InNamedValues1.Elements[ElementIndex1];

			if (Element0->Name == Element1->Name)
			{
				// Element exists in both arrays
				++ElementIndex0;
				++ElementIndex1;
			}
			else if (Element0->Name.FastLess(Element1->Name))
			{
				// Element exists only in first array
				InValuePredicate(*Element0);
				++ElementIndex0;
			}
			else
			{
				// Element exists only in second array
				++ElementIndex1;
			}
		}

		// Drain any remaining elements from our first array
		while (ElementIndex0 < NumElements0)
		{
			InValuePredicate(InNamedValues0.Elements[ElementIndex0]);
			++ElementIndex0;
		}
	}

	/**
	 * Removes elements in InOutValueArray0 that match InValueArray1 if predicate returns true
	 **/
	template<typename AllocatorType0, typename ElementType0, typename AllocatorType1, typename ElementType1, typename PredicateType>
	static void RemoveByPredicate(TNamedValueArray<AllocatorType0, ElementType0>& InOutValueArray0, const TNamedValueArray<AllocatorType1, ElementType1>& InValueArray1, PredicateType InPredicate)
	{
		CURVE_PROFILE_CYCLE_COUNTER(FNamedValueArrayUtils_RemoveByPredicate);

		checkSlow((void*)&InOutValueArray0 != (void*)&InValueArray1);

		// Sort both input arrays if required
		InOutValueArray0.SortElementsIfRequired();
		InValueArray1.SortElementsIfRequired();

		// Perform dual-iteration on the two sorted arrays
		int32 ElementIndex0 = 0;
		int32 ElementIndex1 = 0;

		while(ElementIndex0 < InOutValueArray0.Num() && ElementIndex1 < InValueArray1.Num())
		{
			const ElementType0* RESTRICT Element0 = &InOutValueArray0.Elements[ElementIndex0];
			const ElementType1* RESTRICT Element1 = &InValueArray1.Elements[ElementIndex1];

			if(Element0->Name == Element1->Name)
			{
				// Elements match so check filter flags to see if it should be removed from InOutValueArray0
				if(InPredicate(*Element0, *Element1))
				{
					InOutValueArray0.Elements.RemoveAt(ElementIndex0, EAllowShrinking::No);
					++ElementIndex1;
				}
				else
				{
					++ElementIndex0;
					++ElementIndex1;
				}
			}
			else if(Element0->Name.FastLess(Element1->Name))
			{
				++ElementIndex0;
			}
			else
			{
				++ElementIndex1;
			}
		}

		InOutValueArray0.CheckSorted();
	}
};

}

ENUM_CLASS_FLAGS(UE::Anim::ENamedValueUnionFlags);