// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "Animation/NodeMappingProviderInterface.h"
#include "NodeMappingContainer.generated.h"

/* Node Mapping Container Class
 * This saves source items, and target items, and mapping between
 * Used by Retargeting, Control Rig mapping. Will need to improve interface better
 */
UCLASS(hidecategories = Object, ClassGroup = "Animation", BlueprintType, Experimental, MinimalAPI)
class UNodeMappingContainer : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(VisibleAnywhere, Category = Mapping)
	TMap<FName, FNodeItem> SourceItems;

	UPROPERTY(VisibleAnywhere, Category = Mapping)
	TMap<FName, FNodeItem> TargetItems;

	UPROPERTY(EditAnywhere, Category = Mapping)
	TMap<FName, FName>	SourceToTarget;

	// source asset that is used to create source
	// should be UNodeMappingProviderInterface
	UPROPERTY(EditAnywhere, Category = Mapping)
	TSoftObjectPtr<UObject>	SourceAsset; 

	// source asset that is used to create target
	// should be UNodeMappingProviderInterface
	UPROPERTY(EditAnywhere, Category = Mapping)
	TSoftObjectPtr<UObject>	TargetAsset;

public:
	// soft object reference
	const TSoftObjectPtr<UObject>& GetSourceAssetSoftObjectPtr() const { return SourceAsset; }
	const TSoftObjectPtr<UObject>& GetTargetAssetSoftObjectPtr() const { return TargetAsset; }

#if WITH_EDITOR
	ENGINE_API FString GetDisplayName() const;

	// Item getters
	const TMap<FName, FNodeItem>& GetSourceItems() const { return SourceItems; }
	const TMap<FName, FNodeItem>& GetTargetItems() const { return TargetItems; }

	// update data from assets
	ENGINE_API void RefreshDataFromAssets();

	// Asset setters
	ENGINE_API void SetSourceAsset(UObject* InSourceAsset);
	ENGINE_API void SetTargetAsset(UObject* InTargetAsset);

	// Asset getters
	ENGINE_API UObject* GetSourceAsset();
	ENGINE_API UObject* GetTargetAsset();

	// Add/delete mapping
	ENGINE_API void AddMapping(const FName& InSourceNode, const FName& InTargetNode);
	ENGINE_API void DeleteMapping(const FName& InSourceNode);

	// getting node mapping table { source, target }
	const TMap<FName, FName>& GetNodeMappingTable() const { return SourceToTarget;  }

	// this just maps between source to target by name if same
	// note this will override source setting if exists before
	ENGINE_API void AddDefaultMapping();

#endif// WITH_EDITOR

	// get reverse node mapping table { target, source }
	ENGINE_API void GetTargetToSourceMappingTable(TMap<FName, FName>& OutMappingTable) const;

	// return true if source name is mapped
	bool DoesContainMapping(const FName& SourceNode) const
	{
		return SourceToTarget.Contains(SourceNode);
	}

	// this function is not fast, if you want to do this for every frame, cache it somewhere
	FTransform GetSourceToTargetTransform(const FName& SourceNode) const 
	{
		const FName* TargetNode = SourceToTarget.Find(SourceNode);
		if (TargetNode)
		{
			const FNodeItem* Target = TargetItems.Find(*TargetNode);
			const FNodeItem* Source = SourceItems.Find(SourceNode);
			if (Target && Source)
			{
				FTransform Result = Target->Transform.GetRelativeTransform(Source->Transform);
				Result.NormalizeRotation();
				return Result;
			}
		}

		return FTransform::Identity;
	}

private:
#if WITH_EDITOR
	// internal utility function set OutItems by InAsset
	ENGINE_API void SetAsset(UObject* InAsset, TMap<FName, FNodeItem>& OutItems);
	// Validate SourceToTarget mapping is still valid with SourceItems and TargetItems
	ENGINE_API void ValidateMapping();
#endif // WITH_EDITOR
};
