// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "NodeMappingProviderInterface.generated.h"

/* Node Information to save with */
USTRUCT()
struct FNodeItem
{
	GENERATED_USTRUCT_BODY()

    /* Parent Name. If NAME_None, it will consider no parent */
	UPROPERTY()
	FName ParentName;

	/* Space transform (Not based on parent). Used by control rig system */
	UPROPERTY()
	FTransform Transform;

	FNodeItem(FName InParentName = NAME_None, const FTransform& InTransform = FTransform::Identity)
		: ParentName(InParentName)
		, Transform(InTransform)
	{
	}
};

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UNodeMappingProviderInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};


/* This is interface class for providing node inforation 
 *
 * This is used by UNodeMappingContainer to query data to map between 
 * Implemented by Rig, ControlRig, SkeletalMesh (not Skeleton yet because Skeleton doesn't have retarget base pose)
 */

class INodeMappingProviderInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Returns nodes that needs for them to map */
	virtual void GetMappableNodeData(TArray<FName>& OutNames, TArray<FNodeItem>& OutNodeItems) const = 0;
};
