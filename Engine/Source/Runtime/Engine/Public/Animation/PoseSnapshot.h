// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseSnapshot.generated.h"

/** A pose for a skeletal mesh */
USTRUCT(BlueprintType)
struct FPoseSnapshot
{
	GENERATED_BODY()

public:
	FPoseSnapshot()
		: bIsValid(false)
	{}

	void Reset()
	{
		LocalTransforms.Reset();
		BoneNames.Reset();
		SkeletalMeshName = NAME_None;
		SnapshotName = NAME_None;
		bIsValid = false;
	}

	/** Array of transforms per-bone */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Snapshot")
	TArray<FTransform> LocalTransforms;

	/** Array of bone names (corresponding to LocalTransforms) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Snapshot")
	TArray<FName> BoneNames;

	/** The name of the skeletal mesh that was used to take this snapshot */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Snapshot")
	FName SkeletalMeshName;

	/** The name for this snapshot*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Snapshot")
	FName SnapshotName;

	/** Whether the pose is valid */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Snapshot")
	bool bIsValid;
};
