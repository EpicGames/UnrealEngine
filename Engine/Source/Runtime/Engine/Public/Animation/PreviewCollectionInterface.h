// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "PreviewCollectionInterface.generated.h"

class UAnimInstance;
class USkeletalMesh;

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UPreviewCollectionInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/** Preview Collection options. If you have native UDataAsset class that implements this, you can preview all in the animation editor using Additional Mesh section  */
class IPreviewCollectionInterface
{
	GENERATED_IINTERFACE_BODY()

	/** If you want this to set base mesh also, please use this interface. If this returns nullptr, it will just use whatever set up right now */
	virtual USkeletalMesh* GetPreviewBaseMesh() const { return nullptr; }
	/** Returns nodes that needs for them to map */
	virtual void GetPreviewSkeletalMeshes(TArray<USkeletalMesh*>& OutList, TArray<TSubclassOf<UAnimInstance>>& OutAnimBP) const = 0;
};
