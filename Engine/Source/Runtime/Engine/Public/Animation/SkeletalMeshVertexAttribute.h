// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/PerPlatformProperties.h"

#include "SkeletalMeshVertexAttribute.generated.h"


class FSkeletalMeshAttributeVertexBuffer;


UENUM()
enum class ESkeletalMeshVertexAttributeDataType
{
	Float UMETA(DisplayName="Float", ToolTip="Store the vertex attribute values as a 32-bit floating point (float)"),
	HalfFloat UMETA(DisplayName="Half-float", ToolTip="Store the vertex attribute values as a 16-bit floating point (half)"),
	NUInt8 UMETA(DisplayName="8-bit Unsigned Normalized", ToolTip="Quantize and store the vertex attribute values as an unsigned normalized 8-bit integer. Values outside the [0.0 - 1.0] range are clamped."),
	// Commented out until we have PixelFormat support for these types.
	// NSInt8 UMETA(DisplayName="8-bit Signed Normalized", ToolTip="Quantize and store the vertex attribute values as a signed normalized 8-bit integer. Values outside the [-1.0 - 1.0] range are clamped."),
	// NUInt16 UMETA(DisplayName="16-bit Unsigned Normalized", ToolTip="Quantize and store the vertex attribute values as an unsigned normalized 16-bit integer. Values outside the [0.0 - 1.0] range are clamped."),
	// NSInt16 UMETA(DisplayName="16-bit Signed Normalized", ToolTip="Quantize and store the vertex attribute values as a signed normalized 16-bit integer. Values outside the [-1.0 - 1.0] range are clamped.")
};


/** A structure to store user-controllable settings for attributes */  
USTRUCT()
struct FSkeletalMeshVertexAttributeInfo
{
	GENERATED_BODY()

	/** Whether this vertex attribute should be included in the render data. Requires a rebuild of the render mesh. */
	UPROPERTY(EditAnywhere, Category="Vertex Attributes")
	FPerPlatformBool EnabledForRender = false;
	
	/** The name of the vertex attribute */
	UPROPERTY(VisibleAnywhere, Category="Vertex Attributes")
	FName Name;

	/** The data type to store the vertex data as for rendering */
	UPROPERTY(EditAnywhere, Category="Vertex Attributes")
	ESkeletalMeshVertexAttributeDataType DataType = ESkeletalMeshVertexAttributeDataType::Float;

	/** Returns true if this attribute is enabled for rendering on this platform */
	ENGINE_API bool IsEnabledForRender() const;
};

#if WITH_EDITORONLY_DATA

/** Editor-only representation, stored in FSkeletalMeshLODModel */
struct FSkeletalMeshModelVertexAttribute
{
	/** How the data should be encoded for rendering */
	ESkeletalMeshVertexAttributeDataType DataType = ESkeletalMeshVertexAttributeDataType::Float;
	
	/** The per-vertex value data, where each vertex has ComponentCount number of values.
	 */
	TArray<float> Values;

	/** The per vertex component count. Only a range of 1-4 is allowed. */
	int32 ComponentCount;

	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshModelVertexAttribute& Attribute);
};

#endif


struct FSkeletalMeshVertexAttributeRenderData
{
	ENGINE_API ~FSkeletalMeshVertexAttributeRenderData();
	
	ENGINE_API bool AddAttribute(
		const FName InName,
		const ESkeletalMeshVertexAttributeDataType InDataType,
		const int32 InNumVertices,
		const int32 InComponentCount,
		const TArray<float>& InValues
		);

	ENGINE_API FSkeletalMeshAttributeVertexBuffer* GetAttributeBuffer(FName InName) const; 

	ENGINE_API void InitResources();
	ENGINE_API void ReleaseResources();
	ENGINE_API void CleanUp();
	ENGINE_API int32 GetResourceSize() const;

	// void SerializeMetaData(FArchive& Ar);
	ENGINE_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FSkeletalMeshVertexAttributeRenderData& RenderData)
	{
		RenderData.Serialize(Ar);
		return Ar;
	}
	
private:
	TMap<FName, FSkeletalMeshAttributeVertexBuffer*> Buffers;
};
