// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTypes.h"

class USkeleton;

struct FSkeletonRemapping
{
	FSkeletonRemapping() = default;
	ENGINE_API FSkeletonRemapping(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton);

	const TWeakObjectPtr<const USkeleton>& GetSourceSkeleton() const { return SourceSkeleton; }
	const TWeakObjectPtr<const USkeleton>& GetTargetSkeleton() const { return TargetSkeleton; }

	/**
	 * Compose this remapping with another remapping in place.  The other remapping's source skeleton must match
	 * this remapping's target skeleton.  The result will map from this remapping's source skeleton to the other
	 * remapping's target skeleton
	 *
	 * @param	OtherSkeletonRemapping		Skeleton remapping to compose into this remapping
	 */
	ENGINE_API void ComposeWith(const FSkeletonRemapping& OtherSkeletonRemapping);

	/**
	 * Get the target skeleton bone index that corresponds to the specified bone on the source skeleton
	 *
	 * @param	SourceSkeletonBoneIndex		Skeleton bone index on the source skeleton
	 * @return								Skeleton bone index on the target skeleton (or INDEX_NONE)
	 */
	int32 GetTargetSkeletonBoneIndex(int32 SourceSkeletonBoneIndex) const
	{
		return (SourceToTargetBoneIndexes.IsValidIndex(SourceSkeletonBoneIndex))
				   ? SourceToTargetBoneIndexes[SourceSkeletonBoneIndex]
				   : INDEX_NONE;
	}

	/**
	 * Get the source skeleton bone index that corresponds to the specified bone on the target skeleton
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @return								Skeleton bone index on the source skeleton (or INDEX_NONE)
	 */
	int32 GetSourceSkeletonBoneIndex(int32 TargetSkeletonBoneIndex) const
	{
		return (TargetToSourceBoneIndexes.IsValidIndex(TargetSkeletonBoneIndex))
				   ? TargetToSourceBoneIndexes[TargetSkeletonBoneIndex]
				   : INDEX_NONE;
	}

	/**
	 * Get the specified bone transform retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceTransform				Bone transform from the corresponding bone on the source skeleton
	 * @return								Transform mapped onto the target skeleton
	 */
	FTransform RetargetBoneTransformToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FTransform& SourceTransform) const
	{
		return FTransform(
			RetargetBoneRotationToTargetSkeleton(TargetSkeletonBoneIndex, SourceTransform.GetRotation()),
			RetargetBoneTranslationToTargetSkeleton(TargetSkeletonBoneIndex, SourceTransform.GetTranslation()),
			SourceTransform.GetScale3D());
	}

	/**
	 * Get the specified bone translation retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceTranslation			Bone translation from the corresponding bone on the source skeleton
	 * @return								Translation mapped onto the target skeleton
	 */
	FVector RetargetBoneTranslationToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FVector& SourceTranslation) const
	{
		// Compute the translation part of FTransform(Q1) * Source * FTransform(Q0)
		const TTuple<FQuat, FQuat>& QQ = RetargetingTable[TargetSkeletonBoneIndex];
		return QQ.Get<0>().RotateVector(SourceTranslation);
	}

	/**
	 * Get the specified bone rotation retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceRotation				Bone rotation from the corresponding bone on the source skeleton
	 * @return								Rotation mapped onto the target skeleton
	 */
	FQuat RetargetBoneRotationToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FQuat& SourceRotation) const
	{
		// Compute the rotation part of FTransform(Q1) * Source * FTransform(Q0)
		const TTuple<FQuat, FQuat>& QQ = RetargetingTable[TargetSkeletonBoneIndex];
		return QQ.Get<0>() * SourceRotation * QQ.Get<1>();
	}

	/**
	 * Get the specified additive transform retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceTransform				Bone transform from the corresponding bone on the source skeleton
	 * @return								Transform mapped onto the target skeleton
	 */
	FTransform RetargetAdditiveTransformToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FTransform& SourceTransform) const
	{
		return FTransform(
			RetargetAdditiveRotationToTargetSkeleton(TargetSkeletonBoneIndex, SourceTransform.GetRotation()),
			RetargetAdditiveTranslationToTargetSkeleton(TargetSkeletonBoneIndex, SourceTransform.GetTranslation()),
			SourceTransform.GetScale3D());
	}

	/**
	 * Get the specified additive translation retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceTranslation			Bone translation from the corresponding bone on the source skeleton
	 * @return								Translation mapped onto the target skeleton
	 */
	FVector RetargetAdditiveTranslationToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FVector& SourceTranslation) const
	{
		// Compute the translation part of FTransform(Q0.Inverse) * Source * FTransform(Q0)
		const TTuple<FQuat, FQuat>& QQ = RetargetingTable[TargetSkeletonBoneIndex];
		return QQ.Get<0>().RotateVector(SourceTranslation);
	}

	/**
	 * Get the specified additive rotation retargeted from the source skeleton onto the target skeleton, corrected
	 * for differences between source and target rest poses
	 *
	 * @param	TargetSkeletonBoneIndex		Skeleton bone index on the target skeleton
	 * @param	SourceRotation				Bone rotation from the corresponding bone on the source skeleton
	 * @return								Rotation mapped onto the target skeleton
	 */
	FQuat RetargetAdditiveRotationToTargetSkeleton(int32 TargetSkeletonBoneIndex, const FQuat& SourceRotation) const
	{
		// Compute the rotation part of FTransform(Q0.Inverse) * Source * FTransform(Q0)
		const TTuple<FQuat, FQuat>& QQ = RetargetingTable[TargetSkeletonBoneIndex];
		return QQ.Get<0>() * SourceRotation * QQ.Get<0>().Inverse();
	}

	UE_DEPRECATED(5.3, "This function is no longer used")
	ENGINE_API const TArray<SmartName::UID_Type>& GetSourceToTargetCurveMapping() const;

	/** Refreshes the mapping. Empties mapping arrays and regenerates them against the stored skeletons. */
	ENGINE_API void RegenerateMapping();
	
	UE_DEPRECATED(5.3, "This function is no longer used")
	void GenerateCurveMapping() {}

	/** Check to see if a reference pose retarget is required between the source & target hierarchies */
	bool RequiresReferencePoseRetarget() const
	{
		return RetargetingTable.Num() > 0;
	}
	
	/** Check whether this mapping is valid - whether the source and target hierarchies still exist */
	bool IsValid() const
	{
		return SourceSkeleton.IsValid() && TargetSkeleton.IsValid();
	}

	/**
	 * Get the retargeting quaternion pair for a target skeleton bone
	 *
	 * @param   TargetSkeletonBoneIndex     Skeleton bone index on the target skeleton
	 * @return                              Tuple of (Q0, Q1) where RetargetedRotation = Q0 * SourceRotation * Q1
	 */
	const TTuple<FQuat, FQuat>& GetRetargetingQuaternions(int32 TargetSkeletonBoneIndex) const
	{
		return RetargetingTable[TargetSkeletonBoneIndex];
	}

private:
	/** Internal helper function - performs mapping generation */
	ENGINE_API void GenerateMapping();
	
private:
	TWeakObjectPtr<const USkeleton> SourceSkeleton;
	TWeakObjectPtr<const USkeleton> TargetSkeleton;

	// Table of target skeleton bone indexes (indexed by source skeleton bone index)
	TArray<int32> SourceToTargetBoneIndexes;

	// Table of source skeleton bone indexes (indexed by target skeleton bone index)
	TArray<int32> TargetToSourceBoneIndexes;

	// Table of precalculated constants for retargeting from source to target (indexed by target skeleton bone index)
	TArray<TTuple<FQuat, FQuat>> RetargetingTable;
};
