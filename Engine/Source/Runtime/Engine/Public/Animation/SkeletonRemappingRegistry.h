// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/WeakObjectPtr.h"

struct FSkeletonRemapping;
class USkeleton;
class FEngineModule;

namespace UE::Anim
{

struct FSkeletonRemappingRegistryPrivate;

// Global registry of skeleton remappings
// Remappings are created on-demand when calling GetRemapping.
// Calling public functions from multiple threads is expected. Data races are guarded by a FRWLock.
class FSkeletonRemappingRegistry
{
public:
	// Access the global registry
	static ENGINE_API FSkeletonRemappingRegistry& Get();

	// Get a mapping between a source and target skeleton, built on demand
	// Remapping is only valid locally to this call and could be free'd later (they are stored as shared ptrs)
	// Do not assume ownership of the mapping after this call. 
	ENGINE_API const FSkeletonRemapping& GetRemapping(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton);

	// Refresh any existing mappings that the supplied skeleton is used with
	ENGINE_API void RefreshMappings(const USkeleton* InSkeleton);

private:
	friend FSkeletonRemappingRegistryPrivate;
	friend ::FEngineModule;
	
private:
	// Initialize the global registry
	static ENGINE_API void Init();

	// Shutdown the global registry
	static ENGINE_API void Destroy();
	
	// Mappings map is guarded with a RW lock
	FTransactionallySafeRWLock MappingsLock;

	using FWeakSkeletonPair = TPair<TWeakObjectPtr<const USkeleton>, TWeakObjectPtr<const USkeleton>>;
	
	// All mappings
	TMap<FWeakSkeletonPair, TSharedPtr<FSkeletonRemapping>> Mappings;
	TMultiMap<TWeakObjectPtr<const USkeleton>, TSharedPtr<FSkeletonRemapping>> PerSkeletonMappings;
};

}
