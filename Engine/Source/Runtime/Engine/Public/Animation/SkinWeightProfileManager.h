// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkinnedMeshComponent.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Tickable.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Rendering/SkinWeightProfile.h"

#include "SkinWeightProfileManager.generated.h"

class FSkinWeightProfileManager;
class USkinnedAsset;
class UWorld;

typedef TFunction<void(TWeakObjectPtr<USkeletalMesh> WeakMesh, FSkinWeightProfileStack ProfileStack)> FRequestFinished;

/** Describes a single skin weight profile request */
struct FSetProfileRequest
{
	/** Name of the skin weight profile stack to be loaded. Must be normalized, see FSkinWeightProfileStack::Normalized */
	FSkinWeightProfileStack ProfileStack;
	/** LOD Indices to load the profile for */
	TArray<int32> LODIndices;
	/** Called when the profile request has finished and data is ready (called from GT only) */
	FRequestFinished Callback;

	/** Weak UObject that is responsible for this request */
	TWeakObjectPtr<UObject> IdentifyingObject;
	/** Weak skeletal mesh for which the skin weight profile is to be loaded */
	TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh;

	/** Tells how much LODs need to be streamed in so that we can continue processing the request. If INDEX_NONE then no waiting is required */
	int32 ExpectedResidentLODs = INDEX_NONE;

	friend bool operator==(const FSetProfileRequest& A, const FSetProfileRequest& B)
	{
		return A.ProfileStack == B.ProfileStack && A.WeakSkeletalMesh == B.WeakSkeletalMesh && A.IdentifyingObject == B.IdentifyingObject;
	}

	friend uint32 GetTypeHash(FSetProfileRequest A)
	{
		return HashCombine(GetTypeHash(A.ProfileStack), GetTypeHash(A.WeakSkeletalMesh));
	}
};

/** Async task handling the skin weight buffer generation */
class FSkinWeightProfileManagerAsyncTask
{
	FSkinWeightProfileManager* Owner;

public:
	FSkinWeightProfileManagerAsyncTask(FSkinWeightProfileManager* InOwner)
		: Owner(InOwner)
	{

	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSkinWeightProfileManagerAsyncTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread(); 
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
};

USTRUCT()
struct FSkinWeightProfileManagerTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	//~ FTickFunction Interface
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed)  override;
	//~ FTickFunction Interface

	FSkinWeightProfileManager* Owner;
};

template<>
struct TStructOpsTypeTraits<FSkinWeightProfileManagerTickFunction> : public TStructOpsTypeTraitsBase2<FSkinWeightProfileManagerTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

class FSkinWeightProfileManager : public FTickableGameObject
{
protected: 
	friend FSkinWeightProfileManagerAsyncTask;

	static TMap<UWorld*, FSkinWeightProfileManager*> WorldManagers;
	static void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	static void OnPreWorldFinishDestroy(UWorld* World);
	static void OnWorldBeginTearDown(UWorld* World);

public: 
	static void OnStartup();
	static void OnShutdown();
	static ENGINE_API FSkinWeightProfileManager* Get(UWorld* World);
	static bool AllowCPU();
	static bool HandleDelayedLoads();

	FSkinWeightProfileManager(UWorld* InWorld);

	void ENGINE_API RequestSkinWeightProfileStack(FSkinWeightProfileStack InProfileStack, USkinnedAsset* SkinnedAsset, UObject* Requester, FRequestFinished& Callback, int32 LODIndex = INDEX_NONE);
	void CancelSkinWeightProfileRequest(UObject* Requester);
	
	void DoTick(float DeltaTime, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

	/** Track an asset for which TryRegisterStreamOutProtectedAsset succeeded.
	 *  Safe to call from the async task */
	void ENGINE_API RegisterStreamOutProtection_AsyncSafe(USkinnedAsset* SkinnedAsset);
protected:
	void CleanupRequest(const FSetProfileRequest& Request);
	bool ShouldIgnoreLOD(const FSkinWeightProfilesData& SkinWeightProfilesData, const USkeletalMesh& SkeletalMesh, int32 NumResidentLODsRequired) const;
	bool ShouldSkipTick() const;
	int32 ConvertLODIndexToCount(const USkeletalMesh& SkeletalMesh, int32 Index) const;

	/** Walk ProtectedMeshes and unregister any whose readback is past the danger window. */
	void TickReleaseStreamOutProtection_GameThread();
protected:
	TArray<FSetProfileRequest, TInlineAllocator<4>> CanceledRequest;
	TArray<FSetProfileRequest> PendingSetProfileRequests;
	TMap<TWeakObjectPtr<USkeletalMesh>, int32> PendingMeshes;
	FSkinWeightProfileManagerTickFunction TickFunction;
	int32 LastGamethreadProfileIndex;
	bool WaitingForStreaming = false;

	TWeakObjectPtr<UWorld> WeakWorld;

	FGraphEventRef AsyncTask;

	/** Skinned assets for which we hold a stream-out protection registration on the streaming manager. */
	TSet<TWeakObjectPtr<USkinnedAsset>> ProtectedSkinnedAssets;
	FCriticalSection ProtectedSkinnedAssetsCS;
public:
	virtual bool IsTickableWhenPaused() const override;
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;

};
