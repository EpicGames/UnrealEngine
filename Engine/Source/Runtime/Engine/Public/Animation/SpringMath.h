// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.8, "SpringMath has moved from Engine to Core. Use #include \"Math/SpringMath.h\" instead of \"Animation/SpringMath.h\".")

#include "Math/SpringMath.h"
