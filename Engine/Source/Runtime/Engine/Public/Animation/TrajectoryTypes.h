// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "TrajectoryTypes.generated.h"

#define UE_API ENGINE_API

struct FAnimInstanceProxy;

/**
 * A trajectory sample of conformed of a location, orientation, and time. *Scale is disregarded*.
 */
USTRUCT(BlueprintType, Category="Trajectory")
struct FTransformTrajectorySample
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trajectory")
	FQuat Facing = FQuat::Identity;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trajectory")
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trajectory")
	float TimeInSeconds = 0.f;

	UE_API FTransformTrajectorySample Lerp(const FTransformTrajectorySample& Other, float Alpha) const;
	UE_API void SetTransform(const FTransform& Transform);
	FTransform GetTransform() const { return FTransform(Facing, Position); }
};
ENGINE_API FArchive& operator<<(FArchive& Ar, FTransformTrajectorySample& Sample);

/**
 * A trajectory defined by a range of samples.
 *
 * The sample range is conformed of history samples, a current sample and future predicted samples.
 * - History samples have time < 0. Zero or more can be present.
 * - Current sample has a time of 0. Assumes only one is ever present.
 * - Predicted samples have a time of > 0. Zero or more can be present.
 */
USTRUCT(BlueprintType, Category = "Trajectory")
struct FTransformTrajectory
{
	GENERATED_BODY()
	
	// This contains zero or more history samples, a current sample, and zero or more future predicted samples.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Trajectory")
	TArray<FTransformTrajectorySample> Samples;

	UE_API FTransformTrajectorySample GetSampleAtTime(float Time, bool bExtrapolate = false) const;
};
ENGINE_API FArchive& operator<<(FArchive& Ar, FTransformTrajectory& Trajectory);

/**
 * A function library of utilities for working with trajectories.
 */
UCLASS(MinimalAPI)
class UTransformTrajectoryBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	UFUNCTION(BlueprintCallable, Category = "Animation|Trajectory|Debug", meta = (BlueprintThreadSafe))
	static ENGINE_API void DebugDrawTrajectory(UPARAM(ref) const FTransformTrajectory& Trajectory, const UWorld* World, const float DebugThickness = 0.f, float HeightOffset = 0.f);

#if ENABLE_ANIM_DEBUG
	ENGINE_API static void DebugDrawTrajectory(const FTransformTrajectory& Trajectory, FAnimInstanceProxy& AnimInstanceProxy, const float DebugThickness = 0.f, float HeightOffset = 0.f, int MaxHistorySamples = -1, int MaxPredictionSamples = -1);
	ENGINE_API static void DebugDrawTrajectory(const FTransformTrajectory& Trajectory, const UObject* Owner, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity, const float DebugThickness = 0.f, float HeightOffset = 0.f, bool bDisplayVelocity = false);
#endif
};

#undef UE_API
