// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"

#include "TransformProviderData.generated.h"

struct FRenderBounds;
struct FSkinnedMeshInstanceData;
class FInstancedSkinningSceneExtensionProxy;
class FSceneInterface;
class FRHICommandListBase;
class ITargetPlatform;

class FTransformProviderRenderProxy
{
public:
	virtual ~FTransformProviderRenderProxy() = default;

	virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) = 0;

	virtual void DestroyRenderThreadResources() = 0;
};

UCLASS(Abstract, config=Engine, hidecategories=Object, MinimalAPI, BlueprintType)
class UTransformProviderData : public UObject
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const
	{
		return true;
	}

	/**
	 * Returns true when this provider's per-frame evaluation runs entirely on the render thread
	 * (through FTransformProviderRenderProxy) and does not require a game-thread CPU tick on the
	 * owning component. Used by streaming systems (e.g. FastGeo) to decide whether the provider
	 * can be hosted on components that won't receive a per-frame game-thread tick.
	 */
	virtual bool IsGpuOnly() const
	{
		return false;
	}

	virtual const FGuid& GetTransformProviderID() const
	{
		static FGuid InvalidID(0, 0, 0, 0);
		return InvalidID;
	}

	virtual uint32 GetUniqueAnimationCount() const
	{
		return 1u;
	}

	virtual bool UsesSkeletonBatching() const
	{
		return false;
	}

	virtual bool HasAnimationBounds() const
	{
		return false;
	}

	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const
	{
		return false;
	}

	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
	{
		return 0u;
	}

	virtual bool IsCompiling() const
	{
		return false;
	}

	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionSceneProxy) const
	{
		return nullptr;
	}

	virtual void SubmitChanges()
	{
	}

	virtual UTransformProviderData* GetRootTransformProvider()
	{
		return this;
	}

#if WITH_EDITOR
	void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
	{
	}

	bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
	{
		return true;
	}
#endif
};
