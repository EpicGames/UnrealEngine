// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCompression.h: Skeletal mesh animation compression.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "Math/FloatPacker.h"
#include "Animation/AnimEnums.h"
#include "EngineLogs.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimationCompression, Display, All);

// Thresholds
#define TRANSLATION_ZEROING_THRESHOLD (0.0001f)
#define QUATERNION_ZEROING_THRESHOLD (0.0003f)
#define SCALE_ZEROING_THRESHOLD (0.000001f)

/** Size of Dummy bone used, when measuring error at an end effector which has a socket attached to it */
#define END_EFFECTOR_DUMMY_BONE_LENGTH_SOCKET	(50.f)
/** Dummy bone added to end effectors to make sure rotation doesn't get too aggressively compressed. */
#define END_EFFECTOR_DUMMY_BONE_LENGTH	(5.f)

#define Quant16BitDiv     (32767.f)
#define Quant16BitFactor  (32767.f)
#define Quant16BitOffs    (32767)

#define Quant10BitDiv     (511.f)
#define Quant10BitFactor  (511.f)
#define Quant10BitOffs    (511)

#define Quant11BitDiv     (1023.f)
#define Quant11BitFactor  (1023.f)
#define Quant11BitOffs    (1023)

namespace AnimationCompressionUtils
{
	//@TODO: Explore different scales
	inline static constexpr int32 LogScale = 7;

	template<typename ValueType>
	inline ValueType UnalignedRead(const void* Ptr)
	{
#if PLATFORM_SUPPORTS_UNALIGNED_LOADS
		return *reinterpret_cast<const ValueType*>(Ptr);
#else
		// TODO: On ARM devices this will be slower than it needs to be.
		// To make it fast, __packed keyword must be used with the reinterpret cast.
		// Ideally this code would be moved into FPlatformMisc to handle this per platform properly.
		ValueType Result;
		memcpy(&Result, Ptr, sizeof(ValueType));
		return Result;
#endif
	}

	/**
	* Helper template function to interpolate between two data types.
	* Used in the FilterLinearKeysTemplate function below
	*/
	template <typename T>
	inline T Interpolate(const T& A, const T& B, float Alpha)
	{
		// only the custom instantiations below are valid
		check(0);
		return 0;
	}

	/** custom instantiation of Interpolate for FVectors */
	template <> inline FVector3f Interpolate<FVector3f>(const FVector3f& A, const FVector3f& B, float Alpha)
	{
		return FMath::Lerp(A, B, Alpha);
	}
	template <> inline FVector3d Interpolate<FVector3d>(const FVector3d& A, const FVector3d& B, float Alpha)	// LWC_TODO: double Alpha?
	{
		return FMath::Lerp(A, B, Alpha);
	}

	/** custom instantiation of Interpolate for FQuats */
	template <> inline FQuat4d Interpolate<FQuat4d>(const FQuat4d& A, const FQuat4d& B, float Alpha)
	{
		FQuat4d result = FQuat4d::FastLerp(A, B, Alpha);
		result.Normalize();

		return result;
	}
    template <> inline FQuat4f Interpolate<FQuat4f>(const FQuat4f& A, const FQuat4f& B, float Alpha)
    {
    	FQuat4f result = FQuat4f::FastLerp(A, B, Alpha);
    	result.Normalize();
    	return result;
    }
}

class FQuatFixed48NoW
{
public:
	uint16 X;
	uint16 Y;
	uint16 Z;

	FQuatFixed48NoW()
	{}

	explicit FQuatFixed48NoW(const FQuat4f& Quat)
	{
		FromQuat( Quat );
	}

	void FromQuat(const FQuat4f& Quat)
	{
		FQuat4f Temp( Quat );
		if ( Temp.W < 0.f )
		{
			Temp.X = -Temp.X;
			Temp.Y = -Temp.Y;
			Temp.Z = -Temp.Z;
			Temp.W = -Temp.W;
		}
		Temp.Normalize();

		X = (uint16)((int32)(Temp.X * Quant16BitFactor) + Quant16BitOffs);
		Y = (uint16)((int32)(Temp.Y * Quant16BitFactor) + Quant16BitOffs);
		Z = (uint16)((int32)(Temp.Z * Quant16BitFactor) + Quant16BitOffs);
	}

	void ToQuat(FQuat4f& Out) const
	{
		const float FX = ((int32)X - (int32)Quant16BitOffs) / Quant16BitDiv;
		const float FY = ((int32)Y - (int32)Quant16BitOffs) / Quant16BitDiv;
		const float FZ = ((int32)Z - (int32)Quant16BitOffs) / Quant16BitDiv;
		const float WSquared = 1.f - FX*FX - FY*FY - FZ*FZ;

		Out.X = FX;
		Out.Y = FY;
		Out.Z = FZ;
		Out.W = WSquared > 0.f ? FMath::Sqrt( WSquared ) : 0.f;
	}

	friend FArchive& operator<<(FArchive& Ar, FQuatFixed48NoW& Quat)
	{
		Ar << Quat.X;
		Ar << Quat.Y;
		Ar << Quat.Z;
		return Ar;
	}
};

class FQuatFixed32NoW
{
public:
	uint32 Packed;

	FQuatFixed32NoW()
	{}

	explicit FQuatFixed32NoW(const FQuat4f& Quat)
	{
		FromQuat( Quat );
	}

	void FromQuat(const FQuat4f& Quat)
	{
		FQuat4f Temp( Quat );
		if ( Temp.W < 0.f )
		{
			Temp.X = -Temp.X;
			Temp.Y = -Temp.Y;
			Temp.Z = -Temp.Z;
			Temp.W = -Temp.W;
		}
		Temp.Normalize();

		const uint32 PackedX = (int32)(Temp.X * Quant11BitFactor) + Quant11BitOffs;
		const uint32 PackedY = (int32)(Temp.Y * Quant11BitFactor) + Quant11BitOffs;
		const uint32 PackedZ = (int32)(Temp.Z * Quant10BitFactor) + Quant10BitOffs;

		// 21-31 X, 10-20 Y, 0-9 Z.
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		Packed = (PackedX << XShift) | (PackedY << YShift) | (PackedZ);
	}

	template<bool bIsDataAligned>
	static FQuat4f ToQuat(const uint32* PackedValue)
	{
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		const uint32 ZMask = 0x000003ff;
		const uint32 YMask = 0x001ffc00;
		const uint32 XMask = 0xffe00000;

		const uint32 Packed = bIsDataAligned ? *PackedValue : AnimationCompressionUtils::UnalignedRead<uint32>(PackedValue);
		const uint32 UnpackedX = Packed >> XShift;
		const uint32 UnpackedY = (Packed & YMask) >> YShift;
		const uint32 UnpackedZ = (Packed & ZMask);

		const float X = ((int32)UnpackedX - (int32)Quant11BitOffs) / Quant11BitDiv;
		const float Y = ((int32)UnpackedY - (int32)Quant11BitOffs) / Quant11BitDiv;
		const float Z = ((int32)UnpackedZ - (int32)Quant10BitOffs) / Quant10BitDiv;
		const float WSquared = 1.f - X*X - Y*Y - Z*Z;

		return FQuat4f(X, Y, Z, WSquared > 0.f ? FMath::Sqrt(WSquared) : 0.f);
	}

	void ToQuat(FQuat4f& Out) const
	{
		Out = ToQuat<true>(&Packed);
	}

	friend FArchive& operator<<(FArchive& Ar, FQuatFixed32NoW& Quat)
	{
		Ar << Quat.Packed;
		return Ar;
	}
};

class FQuatFloat96NoW
{
public:
	union
	{
		struct
		{
			float X;
			float Y;
			float Z;
		};

		UE_DEPRECATED(all, "For internal use only")
		float XYZ[3];
	};

	FQuatFloat96NoW()
	{}

	explicit FQuatFloat96NoW(const FQuat4f& Quat)
	{
		FromQuat( Quat );
	}

	FQuatFloat96NoW(float InX, float InY, float InZ)
		:	X( InX )
		,	Y( InY )
		,	Z( InZ )
	{}

	void FromQuat(const FQuat4f& Quat)
	{
		FQuat4f Temp( Quat );
		if ( Temp.W < 0.f )
		{
			Temp.X = -Temp.X;
			Temp.Y = -Temp.Y;
			Temp.Z = -Temp.Z;
			Temp.W = -Temp.W;
		}
		Temp.Normalize();
		X = UE_REAL_TO_FLOAT(Temp.X);
		Y = UE_REAL_TO_FLOAT(Temp.Y);
		Z = UE_REAL_TO_FLOAT(Temp.Z);
	}

	template<bool bIsDataAligned = true>
	static FQuat4f ToQuat(const float* Values)
	{
		const float X = bIsDataAligned ? *Values++ : AnimationCompressionUtils::UnalignedRead<float>(Values++);
		const float Y = bIsDataAligned ? *Values++ : AnimationCompressionUtils::UnalignedRead<float>(Values++);
		const float Z = bIsDataAligned ? *Values++ : AnimationCompressionUtils::UnalignedRead<float>(Values++);
		const float WSquared = 1.f - X*X - Y*Y - Z*Z;

		return FQuat4f(X, Y, Z, WSquared > 0.f ? FMath::Sqrt(WSquared) : 0.f);
	}

	void ToQuat(FQuat4f& Out) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Out = ToQuat<true>(XYZ);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	friend FArchive& operator<<(FArchive& Ar, FQuatFloat96NoW& Quat)
	{
		Ar << Quat.X;
		Ar << Quat.Y;
		Ar << Quat.Z;
		return Ar;
	}
};



class FVectorFixed48
{
public:
	uint16 X;
	uint16 Y;
	uint16 Z;

	FVectorFixed48()
	{}

	explicit FVectorFixed48(const FVector3f& Vec)
	{
		FromVector( Vec );
	}

	void FromVector(const FVector3f& Vec)
	{
		FVector3f Temp( Vec / 128.0f );

		X = (uint16)((int32)(Temp.X * Quant16BitFactor) + Quant16BitOffs);
		Y = (uint16)((int32)(Temp.Y * Quant16BitFactor) + Quant16BitOffs);
		Z = (uint16)((int32)(Temp.Z * Quant16BitFactor) + Quant16BitOffs);
	}

	void ToVector(FVector3f& Out) const
	{
		const float FX = ((int32)X - (int32)Quant16BitOffs) / Quant16BitDiv;
		const float FY = ((int32)Y - (int32)Quant16BitOffs) / Quant16BitDiv;
		const float FZ = ((int32)Z - (int32)Quant16BitOffs) / Quant16BitDiv;

		Out.X = FX * 128.0f;
		Out.Y = FY * 128.0f;
		Out.Z = FZ * 128.0f;
	}

	friend FArchive& operator<<(FArchive& Ar, FVectorFixed48& Vec)
	{
		Ar << Vec.X;
		Ar << Vec.Y;
		Ar << Vec.Z;
		return Ar;
	}
};

class FVectorIntervalFixed32NoW
{
public:
	uint32 Packed;

	FVectorIntervalFixed32NoW()
	{}

	explicit FVectorIntervalFixed32NoW(const FVector3f& Value, const float* Mins, const float *Ranges)
	{
		FromVector( Value, Mins, Ranges );
	}

	void FromVector(const FVector3f& Value, const float* Mins, const float *Ranges)
	{
		FVector3f Temp( Value );

		Temp.X -= Mins[0];
		Temp.Y -= Mins[1];
		Temp.Z -= Mins[2];

		const uint32 PackedX = (int32)((Temp.X / Ranges[0]) * Quant10BitFactor ) + Quant10BitOffs;
		const uint32 PackedY = (int32)((Temp.Y / Ranges[1]) * Quant11BitFactor ) + Quant11BitOffs;
		const uint32 PackedZ = (int32)((Temp.Z / Ranges[2]) * Quant11BitFactor ) + Quant11BitOffs;

		// 21-31 Z, 10-20 Y, 0-9 X.
		const uint32 ZShift = 21;
		const uint32 YShift = 10;
		Packed = (PackedZ << ZShift) | (PackedY << YShift) | (PackedX);
	}

	template<bool bIsDataAligned = true>
	static FVector3f ToVector(const float* Mins, const float *Ranges, const uint32* PackedValue)
	{
		const uint32 ZShift = 21;
		const uint32 YShift = 10;
		const uint32 XMask = 0x000003ff;
		const uint32 YMask = 0x001ffc00;
		const uint32 ZMask = 0xffe00000;

		const uint32 Packed = bIsDataAligned ? *PackedValue : AnimationCompressionUtils::UnalignedRead<uint32>(PackedValue);
		const uint32 UnpackedZ = Packed >> ZShift;
		const uint32 UnpackedY = (Packed & YMask) >> YShift;
		const uint32 UnpackedX = (Packed & XMask);

		const float X = ((((int32)UnpackedX - (int32)Quant10BitOffs) / Quant10BitDiv) * Ranges[0] + Mins[0]);
		const float Y = ((((int32)UnpackedY - (int32)Quant11BitOffs) / Quant11BitDiv) * Ranges[1] + Mins[1]);
		const float Z = ((((int32)UnpackedZ - (int32)Quant11BitOffs) / Quant11BitDiv) * Ranges[2] + Mins[2]);

		return FVector3f(X, Y, Z);
	}

	void ToVector(FVector3f& Out, const float* Mins, const float *Ranges) const
	{
		Out = ToVector<true>(Mins, Ranges, &Packed);
	}

	friend FArchive& operator<<(FArchive& Ar, FVectorIntervalFixed32NoW& Value)
	{
		Ar << Value.Packed;
		return Ar;
	}
};


class FQuatIntervalFixed32NoW
{
public:
	uint32 Packed;

	FQuatIntervalFixed32NoW()
	{}

	explicit FQuatIntervalFixed32NoW(const FQuat4f& Quat, const float* Mins, const float *Ranges)
	{
		FromQuat( Quat, Mins, Ranges );
	}

	void FromQuat(const FQuat4f& Quat, const float* Mins, const float *Ranges)
	{
		FQuat4f Temp( Quat );
		if ( Temp.W < 0.f )
		{
			Temp.X = -Temp.X;
			Temp.Y = -Temp.Y;
			Temp.Z = -Temp.Z;
			Temp.W = -Temp.W;
		}
		Temp.Normalize();

		Temp.X -= Mins[0];
		Temp.Y -= Mins[1];
		Temp.Z -= Mins[2];

		const uint32 PackedX = (int32)((Temp.X / Ranges[0]) * Quant11BitFactor ) + Quant11BitOffs;
		const uint32 PackedY = (int32)((Temp.Y / Ranges[1]) * Quant11BitFactor ) + Quant11BitOffs;
		const uint32 PackedZ = (int32)((Temp.Z / Ranges[2]) * Quant10BitFactor ) + Quant10BitOffs;

		// 21-31 X, 10-20 Y, 0-9 Z.
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		Packed = (PackedX << XShift) | (PackedY << YShift) | (PackedZ);
	}

	template<bool bIsDataAligned>
	static FQuat4f ToQuat(const float* Mins, const float *Ranges, const uint32* PackedValue)
	{
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		const uint32 ZMask = 0x000003ff;
		const uint32 YMask = 0x001ffc00;
		const uint32 XMask = 0xffe00000;

		const uint32 Packed = bIsDataAligned ? *PackedValue : AnimationCompressionUtils::UnalignedRead<uint32>(PackedValue);
		const uint32 UnpackedX = Packed >> XShift;
		const uint32 UnpackedY = (Packed & YMask) >> YShift;
		const uint32 UnpackedZ = (Packed & ZMask);

		const float X = ((((int32)UnpackedX - (int32)Quant11BitOffs) / Quant11BitDiv) * Ranges[0] + Mins[0]);
		const float Y = ((((int32)UnpackedY - (int32)Quant11BitOffs) / Quant11BitDiv) * Ranges[1] + Mins[1]);
		const float Z = ((((int32)UnpackedZ - (int32)Quant10BitOffs) / Quant10BitDiv) * Ranges[2] + Mins[2]);
		const float WSquared = 1.f - X*X - Y*Y - Z*Z;

		return FQuat4f(X, Y, Z, WSquared > 0.f ? FMath::Sqrt(WSquared) : 0.f);
	}

	void ToQuat(FQuat4f& Out, const float* Mins, const float *Ranges) const
	{
		Out = ToQuat<true>(Mins, Ranges, &Packed);
	}

	friend FArchive& operator<<(FArchive& Ar, FQuatIntervalFixed32NoW& Quat)
	{
		Ar << Quat.Packed;
		return Ar;
	}
};

class FQuatFloat32NoW
{
public:
	uint32 Packed;

	FQuatFloat32NoW()
	{}

	explicit FQuatFloat32NoW(const FQuat4f& Quat)
	{
		FromQuat( Quat );
	}

	void FromQuat(const FQuat4f& Quat)
	{
		FQuat4f Temp( Quat );
		if ( Temp.W < 0.f )
		{
			Temp.X = -Temp.X;
			Temp.Y = -Temp.Y;
			Temp.Z = -Temp.Z;
			Temp.W = -Temp.W;
		}
		Temp.Normalize();

		TFloatPacker<3, 7, true> Packer7e3;
		TFloatPacker<3, 6, true> Packer6e3;

		const uint32 PackedX = Packer7e3.Encode( Temp.X );
		const uint32 PackedY = Packer7e3.Encode( Temp.Y );
		const uint32 PackedZ = Packer6e3.Encode( Temp.Z );

		// 21-31 X, 10-20 Y, 0-9 Z.
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		Packed = (PackedX << XShift) | (PackedY << YShift) | (PackedZ);
	}

	template<bool bIsDataAligned>
	static FQuat4f ToQuat(const uint32* PackedValue)
	{
		const uint32 XShift = 21;
		const uint32 YShift = 10;
		const uint32 ZMask = 0x000003ff;
		const uint32 YMask = 0x001ffc00;
		const uint32 XMask = 0xffe00000;

		const uint32 Packed = bIsDataAligned ? *PackedValue : AnimationCompressionUtils::UnalignedRead<uint32>(PackedValue);
		const uint32 UnpackedX = Packed >> XShift;
		const uint32 UnpackedY = (Packed & YMask) >> YShift;
		const uint32 UnpackedZ = (Packed & ZMask);

		TFloatPacker<3, 7, true> Packer7e3;
		TFloatPacker<3, 6, true> Packer6e3;

		const float X = Packer7e3.Decode(UnpackedX);
		const float Y = Packer7e3.Decode(UnpackedY);
		const float Z = Packer6e3.Decode(UnpackedZ);
		const float WSquared = 1.f - X*X - Y*Y - Z*Z;

		return FQuat4f(X, Y, Z, WSquared > 0.f ? FMath::Sqrt(WSquared) : 0.f);
	}

	void ToQuat(FQuat4f& Out) const
	{
		Out = ToQuat<true>(&Packed);
	}

	friend FArchive& operator<<(FArchive& Ar, FQuatFloat32NoW& Quat)
	{
		Ar << Quat.Packed;
		return Ar;
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Handy Template Decompressors
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Templated Rotation Decompressor. Generates a unique decompressor per known quantization format
 *
 * @param	Out				The FQuat to fill in.
 * @param	TopOfStream		The start of the compressed rotation stream data.
 * @param	KeyData			The compressed rotation data to decompress.
 * @return	None. 
 */
template <int32 FORMAT, bool bIsDataAligned = true>
inline void DecompressRotation(FQuat4f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	// this if-else stack gets compiled away to a single result based on the template parameter
	if ( FORMAT == ACF_None )
	{
		// due to alignment issue, this crahses accessing non aligned 
		// we don't think this is common case, so this is slower. 
		const float* Keys = (const float*)KeyData;
		const float X = bIsDataAligned ? Keys[0] : AnimationCompressionUtils::UnalignedRead<float>(&Keys[0]);
		const float Y = bIsDataAligned ? Keys[1] : AnimationCompressionUtils::UnalignedRead<float>(&Keys[1]);
		const float Z = bIsDataAligned ? Keys[2] : AnimationCompressionUtils::UnalignedRead<float>(&Keys[2]);
		const float W = bIsDataAligned ? Keys[3] : AnimationCompressionUtils::UnalignedRead<float>(&Keys[3]);
		Out = FQuat4f(X, Y, Z, W);
	}
	else if ( FORMAT == ACF_Float96NoW )
	{
		Out = FQuatFloat96NoW::ToQuat<bIsDataAligned>((const float*)KeyData);
	}
	else if ( FORMAT == ACF_Fixed32NoW )
	{
		Out = FQuatFixed32NoW::ToQuat<bIsDataAligned>((const uint32*)KeyData);
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		((FQuatFixed48NoW*)KeyData)->ToQuat( Out );
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const float* RESTRICT Mins = (float*)TopOfStream;
		const float* RESTRICT Ranges = (float*)(TopOfStream+sizeof(float)*3);
		Out = FQuatIntervalFixed32NoW::ToQuat<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
	}
	else if ( FORMAT == ACF_Float32NoW )
	{
		Out = FQuatFloat32NoW::ToQuat<bIsDataAligned>((const uint32*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = FQuat4f::Identity;
	}
	else
	{
		UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)FORMAT );
		Out = FQuat4f::Identity;
	}
}

/**
 * Templated Translation Decompressor. Generates a unique decompressor per known quantization format
 *
 * @param	Out				The FVector to fill in.
 * @param	TopOfStream		The start of the compressed translation stream data.
 * @param	KeyData			The compressed translation data to decompress.
 * @return	None. 
 */
template <int32 FORMAT, bool bIsDataAligned = true>
inline void DecompressTranslation(FVector3f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	if ( (FORMAT == ACF_None) || (FORMAT == ACF_Float96NoW) )
	{
		Out = bIsDataAligned ? *((FVector3f*)KeyData) : AnimationCompressionUtils::UnalignedRead<FVector3f>(KeyData);
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const float* RESTRICT Mins = (float*)TopOfStream;
		const float* RESTRICT Ranges = (float*)(TopOfStream+sizeof(float)*3);
		Out = FVectorIntervalFixed32NoW::ToVector<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = FVector3f::ZeroVector;
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		((FVectorFixed48*)KeyData)->ToVector( Out );
	}
	else
	{
		UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)FORMAT );
		// Silence compilers warning about a value potentially not being assigned.
		Out = FVector3f::ZeroVector;
	}
}

/**
 * Templated Scale Decompressor. Generates a unique decompressor per known quantization format
 *
 * @param	Out				The FVector to fill in.
 * @param	TopOfStream		The start of the compressed Scale stream data.
 * @param	KeyData			The compressed Scale data to decompress.
 * @return	None. 
 */
template <int32 FORMAT, bool bIsDataAligned = true>
inline void DecompressScale(FVector3f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
{
	if ( (FORMAT == ACF_None) || (FORMAT == ACF_Float96NoW) )
	{
		Out = bIsDataAligned ? *((FVector3f*)KeyData) : AnimationCompressionUtils::UnalignedRead<FVector3f>(KeyData);
	}
	else if ( FORMAT == ACF_IntervalFixed32NoW )
	{
		const float* RESTRICT Mins = (float*)TopOfStream;
		const float* RESTRICT Ranges = (float*)(TopOfStream+sizeof(float)*3);
		Out = FVectorIntervalFixed32NoW::ToVector<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
	}
	else if ( FORMAT == ACF_Identity )
	{
		Out = FVector3f::ZeroVector;
	}
	else if ( FORMAT == ACF_Fixed48NoW )
	{
		((FVectorFixed48*)KeyData)->ToVector( Out );
	}
	else
	{
		UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)FORMAT );
		// Silence compilers warning about a value potentially not being assigned.
		Out = FVector3f::ZeroVector;
	}
}



/**
 * This class contains helper methods for dealing with animations compressed with the per-track codec
 */
class FAnimationCompression_PerTrackUtils
{
public:
	// Log2MaxValue of 0 => -1..1
	// Log2MaxValue of 7 => -128..128
	// Can be 0..15
	/**
	 * Compresses a float into a signed fixed point number, which can range from the symmetrical values of
	 * -2^Log2MaxValue .. 2^Log2MaxValue.  No clamping is done, values that don't fit will overflow.
	 *
	 * For example, a Log2MaxValue of 0 can encode -1..1, and 7 can encode -128..128.
	 *
	 * @param Value			Value to encode
	 * @param Log2MaxValue	Encoding range (can be 0..15)
	 *
	 * @return The quantized value
	 */
	static uint16 CompressFixed16(float Value, int32 Log2MaxValue = 0)
	{
		const int32 QuantOffset = (1 << (15 - Log2MaxValue)) - 1;
		const float QuantFactor = (float)(QuantOffset >> Log2MaxValue);
		return (uint16)((int32)(Value * QuantFactor) + QuantOffset);
	}

	/**
	 * Decompresses a fixed point number encoded by ComrpessFixed16
	 *
	 * @param Value			Value to decode
	 * @param Log2MaxValue	Encoding range (can be 0..15)
	 *
	 * @return The decompressed value
	 */
	template <int32 Log2MaxValue>
	static float DecompressFixed16(uint16 Value)
	{
		const int32 QuantOffset = (1 << (15 - Log2MaxValue)) - 1;
		const float InvQuantFactor = 1.0f / (float)(QuantOffset >> Log2MaxValue);

		return ((int32)Value - QuantOffset) * InvQuantFactor;
	}

	/**
	 * Creates a header integer with four fields:
	 *   NumKeys can be no more than 24 bits (positions 0..23)
	 *   KeyFlags can be no more than 3 bits (positions 24..27)
	 *   bReallyNeedsFrameTable is a single bit (position 27)
	 *   KeyFormat can be no more than 4 bits (positions 31..28)
	 */
	static int32 MakeHeader(const int32 NumKeys, const int32 KeyFormat, const int32 KeyFlags, bool bReallyNeedsFrameTable)
	{
		return (NumKeys & 0x00FFFFFF) | ((KeyFormat & 0xF) << 28) | ((KeyFlags & 0x7) << 24) | ((bReallyNeedsFrameTable & 0x1) << 27);
	}

	/**
	 * Extracts the number of keys from a header created by MakeHeader
	 *
	 * @param Header	Header to extract the number of keys from
	 * @return			The number of keys encoded in the header
	 */
	static int32 GetKeyCountFromHeader(int32 Header)
	{
		return Header & 0x00FFFFFF;
	}

	/**
	 * Figures out the size of various parts of a compressed track from the format and format flags combo
	 *   @param KeyFormat		The encoding format used for each key
	 *   @param FormatFlags		Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
	 *
	 */
	static void GetAllSizesFromFormat(int32 KeyFormat, int32 FormatFlags,
		int32& KeyComponentCount, int32& KeyComponentSize,
		int32& FixedComponentCount, int32& FixedComponentSize)
	{
		extern ENGINE_API const int32 CompressedRotationStrides[ACF_MAX];
		extern ENGINE_API const uint8 PerTrackNumComponentTable[ACF_MAX * 8];

		// Note: this method can be used for translation too, but animation sequences compressed with this codec will
		// use ACF_Float96NoW for uncompressed translation, so using the rotation table is still valid
		KeyComponentSize = CompressedRotationStrides[KeyFormat];
		FixedComponentSize = sizeof(float);

		const int32 ComponentLookup = PerTrackNumComponentTable[(FormatFlags & 0x7) | (KeyFormat << 3)];

		if (KeyFormat != ACF_IntervalFixed32NoW)
		{
			FixedComponentCount = 0;
			KeyComponentCount = ComponentLookup;
		}
		else
		{
			// Min/Range floats for all non-zero channels
			FixedComponentCount = ComponentLookup;
			KeyComponentCount = 1;
		}
	}

	static inline void GetByteSizesFromFormat(int32 KeyFormat, int32 FormatFlags, int32& BytesPerKey, int32& FixedBytes)
	{
		int32 FixedComponentSize = 0;
		int32 FixedComponentCount = 0;
		int32 KeyComponentSize = 0;
		int32 KeyComponentCount = 0;

		GetAllSizesFromFormat(KeyFormat, FormatFlags, /*OUT*/ KeyComponentCount, /*OUT*/ KeyComponentSize, /*OUT*/ FixedComponentCount, /*OUT*/ FixedComponentSize);

		BytesPerKey = KeyComponentCount * KeyComponentSize;
		FixedBytes = FixedComponentCount * FixedComponentSize;
	}

	/**
	 * Decomposes a header created with MakeHeader into three/four fields (two are still left packed into FormatFlags):
	 *   @param Header				The header to decompose
	 *   @param KeyFormat [OUT]		The encoding format used for each key
	 *   @param	NumKeys	[OUT]		The number of keys in this track
	 *   @param FormatFlags [OUT]	Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
	 */
	static inline void DecomposeHeader(int32 Header, int32& KeyFormat, int32& NumKeys, int32& FormatFlags)
	{
		NumKeys = Header & 0x00FFFFFF;
		FormatFlags = (Header >> 24) & 0x0F;
		KeyFormat = (Header >> 28) & 0x0F;
	}

	/**
	 * Decomposes a header created with MakeHeader into three/four fields (two are still left packed into FormatFlags):
	 *   @param Header				The header to decompose
	 *   @param KeyFormat [OUT]		The encoding format used for each key
	 *   @param	NumKeys	[OUT]		The number of keys in this track
	 *   @param FormatFlags [OUT]	Three bits of format-specific information and a single bit to indicate if a key->frame table follows the keys
	 *
	 * And some derived values:
	 *   @param	BytesPerKey [OUT]	The number of bytes each key takes up
	 *   @param	FixedBytes [OUT]	The number of fixed bytes at the head of the track stream
	 */
	static inline void DecomposeHeader(int32 Header, int32& KeyFormat, int32& NumKeys, int32& FormatFlags, int32& BytesPerKey, int32& FixedBytes)
	{
		NumKeys = Header & 0x00FFFFFF;
		FormatFlags = (Header >> 24) & 0x0F;
		KeyFormat = (Header >> 28) & 0x0F;

		// Figure out the component sizes / counts (they can be changed per-format)
		GetByteSizesFromFormat(KeyFormat, FormatFlags, /*OUT*/ BytesPerKey, /*OUT*/ FixedBytes);
	}

	/** Decompress a single translation key from a single track that was compressed with the PerTrack codec (scalar) */
	template<bool bIsDataAligned = true>
	static inline void DecompressTranslation(int32 Format, int32 FormatFlags, FVector3f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
	{
		if( Format == ACF_Float96NoW )
		{
			// Legacy Format, all components stored
			if( (FormatFlags & 7) == 0 )
			{
				Out = bIsDataAligned ? *((FVector3f*)KeyData) : AnimationCompressionUtils::UnalignedRead<FVector3f>(KeyData);
			}
			// Stored per components
			else
			{
				const float* RESTRICT TypedKeyData = (const float*)KeyData;
				if (FormatFlags & 1)
				{
					Out.X = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.X = 0.0f;
				}

				if (FormatFlags & 2)
				{
					Out.Y = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.Y = 0.0f;
				}

				if (FormatFlags & 4)
				{
					Out.Z = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.Z = 0.0f;
				}
			}
		}
		else if (Format == ACF_IntervalFixed32NoW)
		{
			const float* RESTRICT SourceBounds = (float*)TopOfStream;

			float Mins[3] = {0.0f, 0.0f, 0.0f};
			float Ranges[3] = {0.0f, 0.0f, 0.0f};

			if (FormatFlags & 1)
			{
				Mins[0] = *SourceBounds++;
				Ranges[0] = *SourceBounds++;
			}
			if (FormatFlags & 2)
			{
				Mins[1] = *SourceBounds++;
				Ranges[1] = *SourceBounds++;
			}
			if (FormatFlags & 4)
			{
				Mins[2] = *SourceBounds++;
				Ranges[2] = *SourceBounds++;
			}

			Out = FVectorIntervalFixed32NoW::ToVector<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
		}
		else if (Format == ACF_Fixed48NoW)
		{
			const uint16* RESTRICT TypedKeyData = (const uint16*)KeyData;
			if (FormatFlags & 1)
			{
				Out.X = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.X = 0.0f;
			}

			if (FormatFlags & 2)
			{
				Out.Y = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.Y = 0.0f;
			}

			if (FormatFlags & 4)
			{
				Out.Z = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.Z = 0.0f;
			}
		}
		else if ( Format == ACF_Identity )
		{
			Out = FVector3f::ZeroVector;
		}
		else
		{
			UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)Format );
			// Silence compilers warning about a value potentially not being assigned.
			Out = FVector3f::ZeroVector;
		}
	}

	/** Decompress a single rotation key from a single track that was compressed with the PerTrack codec (scalar) */
	template<bool bIsDataAligned = true>
	static inline void DecompressRotation(int32 Format, int32 FormatFlags, FQuat4f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
	{
		if (Format == ACF_Fixed48NoW)
		{
			const uint16* RESTRICT TypedKeyData = (const uint16*)KeyData;

			const float Xa = (FormatFlags & 1) ? (float)(bIsDataAligned ? (*TypedKeyData++) : AnimationCompressionUtils::UnalignedRead<uint16>(TypedKeyData++)) : 32767.0f;
			const float Ya = (FormatFlags & 2) ? (float)(bIsDataAligned ? (*TypedKeyData++) : AnimationCompressionUtils::UnalignedRead<uint16>(TypedKeyData++)) : 32767.0f;
			const float Za = (FormatFlags & 4) ? (float)(bIsDataAligned ? (*TypedKeyData++) : AnimationCompressionUtils::UnalignedRead<uint16>(TypedKeyData++)) : 32767.0f;

			const float X = (Xa - 32767.0f) * 3.0518509475997192297128208258309e-5f;
			const float XX = X*X;
			const float Y = (Ya - 32767.0f) * 3.0518509475997192297128208258309e-5f;
			const float YY = Y*Y;
			const float Z = (Za - 32767.0f) * 3.0518509475997192297128208258309e-5f;
			const float ZZ = Z*Z;

			const float WSquared = 1.0f - XX - YY - ZZ;

			const float W = FMath::FloatSelect(WSquared, FMath::Sqrt(WSquared), 0.0f);

			Out = FQuat4f(X, Y, Z, W);
		}
		else if (Format == ACF_Float96NoW)
		{
			Out = FQuatFloat96NoW::ToQuat<bIsDataAligned>((const float*)KeyData);
		}
		else if ( Format == ACF_IntervalFixed32NoW )
		{
			const float* RESTRICT SourceBounds = (float*)TopOfStream;

			float Mins[3] = {0.0f, 0.0f, 0.0f};
			float Ranges[3] = {0.0f, 0.0f, 0.0f};

			if (FormatFlags & 1)
			{
				Mins[0] = *SourceBounds++;
				Ranges[0] = *SourceBounds++;
			}
			if (FormatFlags & 2)
			{
				Mins[1] = *SourceBounds++;
				Ranges[1] = *SourceBounds++;
			}
			if (FormatFlags & 4)
			{
				Mins[2] = *SourceBounds++;
				Ranges[2] = *SourceBounds++;
			}

			Out = FQuatIntervalFixed32NoW::ToQuat<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
		}
		else if ( Format == ACF_Float32NoW )
		{
			Out = FQuatFloat32NoW::ToQuat<bIsDataAligned>((const uint32*)KeyData);
		}
		else if (Format == ACF_Fixed32NoW)
		{
			Out = FQuatFixed32NoW::ToQuat<bIsDataAligned>((const uint32*)KeyData);
		}
		else if ( Format == ACF_Identity )
		{
			Out = FQuat4f::Identity;
		}
		else
		{
			UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)Format );
			Out = FQuat4f::Identity;
		}
	}

	/** Decompress a single Scale key from a single track that was compressed with the PerTrack codec (scalar) */
	template<bool bIsDataAligned = true>
	static inline void DecompressScale(int32 Format, int32 FormatFlags, FVector3f& Out, const uint8* RESTRICT TopOfStream, const uint8* RESTRICT KeyData)
	{
		if( Format == ACF_Float96NoW )
		{
			// Legacy Format, all components stored
			if( (FormatFlags & 7) == 0 )
			{
				Out = bIsDataAligned ? *((FVector3f*)KeyData) : AnimationCompressionUtils::UnalignedRead<FVector3f>(KeyData);
			}
			// Stored per components
			else
			{
				const float* RESTRICT TypedKeyData = (const float*)KeyData;
				if (FormatFlags & 1)
				{
					Out.X = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.X = 0.0f;
				}

				if (FormatFlags & 2)
				{
					Out.Y = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.Y = 0.0f;
				}

				if (FormatFlags & 4)
				{
					Out.Z = bIsDataAligned ? *TypedKeyData++ : AnimationCompressionUtils::UnalignedRead<float>(TypedKeyData++);
				}
				else
				{
					Out.Z = 0.0f;
				}
			}
		}
		else if (Format == ACF_IntervalFixed32NoW)
		{
			const float* RESTRICT SourceBounds = (float*)TopOfStream;

			float Mins[3] = {0.0f, 0.0f, 0.0f};
			float Ranges[3] = {0.0f, 0.0f, 0.0f};

			if (FormatFlags & 1)
			{
				Mins[0] = *SourceBounds++;
				Ranges[0] = *SourceBounds++;
			}
			if (FormatFlags & 2)
			{
				Mins[1] = *SourceBounds++;
				Ranges[1] = *SourceBounds++;
			}
			if (FormatFlags & 4)
			{
				Mins[2] = *SourceBounds++;
				Ranges[2] = *SourceBounds++;
			}

			Out = FVectorIntervalFixed32NoW::ToVector<bIsDataAligned>(Mins, Ranges, (const uint32*)KeyData);
		}
		else if (Format == ACF_Fixed48NoW)
		{
			const uint16* RESTRICT TypedKeyData = (const uint16*)KeyData;
			if (FormatFlags & 1)
			{
				Out.X = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.X = 0.0f;
			}

			if (FormatFlags & 2)
			{
				Out.Y = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.Y = 0.0f;
			}

			if (FormatFlags & 4)
			{
				Out.Z = FAnimationCompression_PerTrackUtils::DecompressFixed16<AnimationCompressionUtils::LogScale>(*TypedKeyData++);
			}
			else
			{
				Out.Z = 0.0f;
			}
		}
		else if ( Format == ACF_Identity )
		{
			Out = FVector3f::ZeroVector;
		}
		else
		{
			UE_LOGF(LogAnimation, Fatal, "%i: unknown or unsupported animation compression format", (int32)Format );
			// Silence compilers warning about a value potentially not being assigned.
			Out = FVector3f::ZeroVector;
		}
	}
};

template <>
inline float FAnimationCompression_PerTrackUtils::DecompressFixed16<0>(uint16 Value)
{
	return ((int32)Value - 32767) * 3.0518509475997192297128208258309e-5f;
}
