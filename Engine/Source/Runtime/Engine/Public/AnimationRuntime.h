// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationRuntime.h: Skeletal mesh animation utilities
	Should only contain functions needed for runtime animation, no tools.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "BoneIndices.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/SkeletonRemapping.h"
#include "BonePose.h"
#include "Containers/ArrayView.h"

class UBlendSpace;
class USkeletalMeshComponent;
class UMirrorDataTable;
class UMorphTarget;
struct FA2CSPose;
struct FA2Pose;
struct FInputBlendPose;
struct FAnimationPoseData;
using FMorphTargetWeightMap = TMap<const UMorphTarget*, int32>;

namespace UE { namespace Anim { struct FStackAttributeContainer; }}

typedef TArray<FTransform> FTransformArrayA2;

/////////////////////////////////////////////////////////
// Templated Transform Blend Functionality

namespace ETransformBlendMode
{
	enum Type
	{
		Overwrite,
		Accumulate
	};
}

template<int32>
ENGINE_API void BlendTransform(const FTransform& Source, FTransform& Dest, const float BlendWeight);

template<>
inline void BlendTransform<ETransformBlendMode::Overwrite>(const FTransform& Source, FTransform& Dest, const float BlendWeight)
{
	const ScalarRegister VBlendWeight(BlendWeight);
	Dest = Source * VBlendWeight;
}

template<>
inline void BlendTransform<ETransformBlendMode::Accumulate>(const FTransform& Source, FTransform& Dest, const float BlendWeight)
{
	const ScalarRegister VBlendWeight(BlendWeight);
	Dest.AccumulateWithShortestRotation(Source, VBlendWeight);
}

ENGINE_API void BlendCurves(const TArrayView<const FBlendedCurve> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve);
void ENGINE_API BlendCurves(const TArrayView<const FBlendedCurve* const> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve);
ENGINE_API void BlendCurves(const TArrayView<const FBlendedCurve* const> SourceCurves, const TArrayView<const float> SourceWeights, FBlendedCurve& OutCurve, ECurveBlendOption::Type BlendOption);


/////////////////////////////////////////////////////////
/** 
  * Interface used to provide interpolation indices for per bone blends
  */
class IInterpolationIndexProvider
{
public:
	~IInterpolationIndexProvider() = default;
	
	struct FPerBoneInterpolationData
	{
		virtual ~FPerBoneInterpolationData() = default;
	};

	// There may be times when the implementation can pre-calculate data needed for GetPerBoneInterpolationIndex, as the
	// latter is often called multiple times whilst iterating over a skeleton.
	virtual TSharedPtr<FPerBoneInterpolationData> GetPerBoneInterpolationData(const USkeleton* Skeleton) const { return nullptr; }

	UE_DEPRECATED(5.0, "Please use the overload that takes a FCompactPoseBoneIndex")
	ENGINE_API virtual int32 GetPerBoneInterpolationIndex(
		int32 BoneIndex, const FBoneContainer& RequiredBones, const FPerBoneInterpolationData* Data) const;

	// Implementation should return the index into the PerBoneBlendData array that would be required when looking
	// up/blending BoneIndex. This call will be passed the results of GetPerBoneInterpolationData, so the two functions
	// should be matched.
	virtual int32 GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& RequiredBones, const FPerBoneInterpolationData* Data) const = 0;

	// Implementation should return the index into the PerBoneBlendData array that would be required when looking
	// up/blending BoneIndex. This call will be passed the results of GetPerBoneInterpolationData, so the two functions
	// should be matched.
	virtual int32 GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const = 0;
};

/**
 * Intermediary Interface class to be implemented to access FAnimationRuntime::RetargetBoneTransform.
 * This is specialized in FBoneContainerRetargetBoneTransformContext, for example.
 */
class FAnimationRuntimeRetargetBoneTransformContext
{
public:

	FAnimationRuntimeRetargetBoneTransformContext() {}
	virtual ~FAnimationRuntimeRetargetBoneTransformContext() {}

	/** Returns the skeleton of this context. Typically maps to the desired target skeleton for the retarget */
	virtual const USkeleton* GetSkeleton(bool bEvenIfUnreachable = false) const = 0;

	/**
	 * Returns the skeleton bone index for a context bone index.
	 * Context bone index in this case is index within the wrapped context - for example the pose.
	 */
	virtual int32 GetSkeletonBoneIndex(const int32& InContextBoneIndex) const = 0;

	/**
	 * Returns the number of bones within this context - for example the number of bones within a pose
	 */
	virtual int32 GetNumBonesInContext() const = 0;
	
	/**
	 * Returns true if retargeting for this context is disabled.
	 */
	virtual bool IsRetargetingDisabled() const = 0;

	/**
	 * Returns the reference pose transform for a given context bone index. The context bone index refers 
	 * to the index of the bone within the context - for example a wrapped pose.
	 */
	virtual const FTransform& GetRefPoseTransform(const int32& InContextBoneIndex) const = 0;

	/**
	 * Returns the cached source data for retargeting within this context.
	 * This is current implemented only against the FBoneContainer.
	 */
	virtual const FRetargetSourceCachedData& GetRetargetSourceCachedData(const FName& InSourceName, const FSkeletonRemapping& InRemapping, const TArray<FTransform>& InRetargetTransforms) const = 0;
};

/**
 * Specialization of the FAnimationRuntimeRetargetBoneTransformContext to map against a FBoneContainer.
 * This allows to Retarget Bone Transforms for all elements in an FBoneContainer.
 */ 
class FBoneContainerRetargetBoneTransformContext : public FAnimationRuntimeRetargetBoneTransformContext
{
public:

	FBoneContainerRetargetBoneTransformContext(const FBoneContainer& InBoneContainer)
		: BoneContainer(InBoneContainer)
	{
	}

	virtual const USkeleton* GetSkeleton(bool bEvenIfUnreachable = false) const override
	{
		return BoneContainer.GetSkeletonAsset(bEvenIfUnreachable);
	}
	
	virtual int32 GetSkeletonBoneIndex(const int32& InContextBoneIndex) const override
	{
		return BoneContainer.GetSkeletonIndex(FCompactPoseBoneIndex(InContextBoneIndex));
	}
	
	virtual int32 GetNumBonesInContext() const override
	{
		return BoneContainer.GetCompactPoseNumBones();
	}
	
	virtual bool IsRetargetingDisabled() const override
	{
		return BoneContainer.GetDisableRetargeting();
	}
	
	virtual const FTransform& GetRefPoseTransform(const int32& InContextBoneIndex) const override
	{
		return BoneContainer.GetRefPoseTransform(FCompactPoseBoneIndex(InContextBoneIndex));
	}
	
	virtual const FRetargetSourceCachedData& GetRetargetSourceCachedData(const FName& InSourceName, const FSkeletonRemapping& InRemapping, const TArray<FTransform>& InRetargetTransforms) const override
	{
		return BoneContainer.GetRetargetSourceCachedData(InSourceName, InRemapping, InRetargetTransforms);
	}

private:

	const FBoneContainer& BoneContainer;
};

/** In AnimationRunTime Library, we extract animation data based on Skeleton hierarchy, not ref pose hierarchy. 
	Ref pose will need to be re-mapped later **/
class FAnimationRuntime
{
public:
	static ENGINE_API void NormalizeRotations(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms);
	static ENGINE_API void NormalizeRotations(FTransformArrayA2& Atoms);

	static ENGINE_API void InitializeTransform(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms);
#if DO_GUARD_SLOW
	static ENGINE_API bool ContainsNaN(TArray<FBoneIndexType>& RequiredBoneIndices, FA2Pose& Pose);
#endif

	/**
	* Blends together a set of poses, each with a given weight.
	* This function is lightweight, it does not cull out nearly zero weights or check to make sure weights sum to 1.0, the caller should take care of that if needed.
	*
	* The blend is done by taking a weighted sum of each atom, and re-normalizing the quaternion part at the end, not using SLERP.
	* This allows n-way blends, and makes the code much faster, though the angular velocity will not be constant across the blend.
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogether with other signature")
	static ENGINE_API void BlendPosesTogether(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const float> SourceWeights,
		/*out*/ FCompactPose& ResultPose, 
		/*out*/ FBlendedCurve& ResultCurve);

	static ENGINE_API void BlendPosesTogether(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes,
		TArrayView<const float> SourceWeights,
		FAnimationPoseData& OutAnimationPoseData);

	/**
	* Blends together a set of poses, each with a given weight.
	* This function is lightweight, it does not cull out nearly zero weights or check to make sure weights sum to 1.0, the caller should take care of that if needed.
	*
	* The blend is done by taking a weighted sum of each atom, and re-normalizing the quaternion part at the end, not using SLERP.
	* This allows n-way blends, and makes the code much faster, though the angular velocity will not be constant across the blend.
	*
	* SourceWeightsIndices is used to index SourceWeights, to prevent caller having to supply an ordered weights array 
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogether with other signature")
	static ENGINE_API void BlendPosesTogether(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const float> SourceWeights,
		TArrayView<const int32> SourceWeightsIndices,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);

	static ENGINE_API void BlendPosesTogether(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes,
		TArrayView<const float> SourceWeights,
		TArrayView<const int32> SourceWeightsIndices,
		/*out*/ FAnimationPoseData& OutPoseData
		);

	/**
	* Blends together a set of poses, each with a given weight.
	* This function is lightweight, it does not cull out nearly zero weights or check to make sure weights sum to 1.0, the caller should take care of that if needed.
	*
	* The blend is done by taking a weighted sum of each atom, and re-normalizing the quaternion part at the end, not using SLERP.
	* This allows n-way blends, and makes the code much faster, though the angular velocity will not be constant across the blend.
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogetherIndirect with other signature")
	static ENGINE_API void BlendPosesTogetherIndirect(
		TArrayView<const FCompactPose* const> SourcePoses,
		TArrayView<const FBlendedCurve* const> SourceCurves,
		TArrayView<const float> SourceWeights,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);

	static ENGINE_API void BlendPosesTogetherIndirect(
		TArrayView<const FCompactPose* const> SourcePoses,
		TArrayView<const FBlendedCurve* const> SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer* const> SourceAttributes,
		TArrayView<const float> SourceWeights,
		FAnimationPoseData& OutPoseData);

	/**
	* Blends together two poses.
	* This function is lightweight
	*
	* The blend is done by taking a weighted sum of each atom, and re-normalizing the quaternion part at the end, not using SLERP.
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendTwoPosesTogether with other signature")
	static ENGINE_API void BlendTwoPosesTogether(
		const FCompactPose& SourcePose1,
		const FCompactPose& SourcePose2,
		const FBlendedCurve& SourceCurve1,
		const FBlendedCurve& SourceCurve2,
		const float WeightOfPose1,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);
			   
	static ENGINE_API void BlendTwoPosesTogether(
		const FAnimationPoseData& SourcePoseOneData,
		const FAnimationPoseData& SourcePoseTwoData,
		const float WeightOfPoseOne,
		/*out*/ FAnimationPoseData& OutAnimationPoseData);

	/**
	 * Blends two poses together where the first pose is already in the output buffer (in-place).
	 * The first pose is scaled by WeightOfPoseOne, then the second pose is accumulated with (1 - WeightOfPoseOne).
	 * This avoids one pose allocation compared to BlendTwoPosesTogether.
	 */
	static ENGINE_API void BlendTwoPosesTogetherInPlace(
		FAnimationPoseData& InOutPoseOneData,
		const FAnimationPoseData& SourcePoseTwoData,
		const float WeightOfPoseOne);

	/**
	* Blends together a set of poses, each with a given weight.
	* This function is for BlendSpace per bone blending. BlendSampleDataCache contains the weight information
	*
	* This blends in local space
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendTwoPosesTogether with other signature")
	static ENGINE_API void BlendTwoPosesTogetherPerBone(
		const FCompactPose& SourcePose1,
		const FCompactPose& SourcePose2,
		const FBlendedCurve& SourceCurve1,
		const FBlendedCurve& SourceCurve2,
		const TArray<float>& WeightsOfSource2,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);
		
	static ENGINE_API void BlendTwoPosesTogetherPerBone(
		const FAnimationPoseData& SourcePoseOneData,
		const FAnimationPoseData& SourcePoseTwoData,
		const TArray<float>& WeightsOfSource2,
		/*out*/ FAnimationPoseData& OutAnimationPoseData);

	static ENGINE_API void BlendTwoPosesTogetherPerBone(
		const FCompactPose& SourcePoseOneData,
		const FCompactPose& SourcePoseTwoData,
		const TArray<float>& WeightsOfSource2,
		/*out*/ FCompactPose& OutAnimationPoseData);
		
	/**
	* Blends together a set of poses, each with a given weight.
	* This function is for BlendSpace per bone blending. BlendSampleDataCache contains the weight information
	*
	* This blends in local space
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogetherPerBone with other signature")
	static ENGINE_API void BlendPosesTogetherPerBone(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		const IInterpolationIndexProvider* InterpolationIndexProvider,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);

	static ENGINE_API void BlendPosesTogetherPerBone(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes,
		const IInterpolationIndexProvider* InterpolationIndexProvider,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		/*out*/ FAnimationPoseData& OutAnimationPoseData);

	static ENGINE_API void BlendPosesTogetherPerBoneRemapped(
		TArrayView<const FCompactPose> SourcePoses, 
		TArrayView<const FBlendedCurve> SourceCurves, 
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes, 
		const IInterpolationIndexProvider* InterpolationIndexProvider,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		TArrayView<const int32> BlendSampleDataCacheIndices, 
		const FSkeletonRemapping& SkeletonRemapping,
		/*out*/ FAnimationPoseData& OutAnimationPoseData);


	/**
	* Blends together a set of poses, each with a given weight.
	* This function is for BlendSpace per bone blending. BlendSampleDataCache contains the weight information 
	* and is indexed using BlendSampleDataCacheIndices, to prevent caller having to supply an ordered array 
	*
	* This blends in local space
	*
	* @param	ResultPose		Output pose of relative bone transforms.
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogetherPerBone with other signature")
	static ENGINE_API void BlendPosesTogetherPerBone(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		const IInterpolationIndexProvider* InterpolationIndexProvider,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		TArrayView<const int32> BlendSampleDataCacheIndices,
		/*out*/ FCompactPose& ResultPose,
		/*out*/ FBlendedCurve& ResultCurve);

	static ENGINE_API void BlendPosesTogetherPerBone(
		TArrayView<const FCompactPose> SourcePoses,
		TArrayView<const FBlendedCurve> SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes,
		const IInterpolationIndexProvider* InterpolationIndexProvider,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		TArrayView<const int32> BlendSampleDataCacheIndices,
		/*out*/ FAnimationPoseData& OutAnimationPoseData);

	/**
	* Blends together a set of local space (not mesh space) poses, each with a given weight.
	* This function is for BlendSpace per bone blending. BlendSampleDataCache contains the
	* weight information
	*
	* This blends the rotations in mesh space, so is performance intensive as it requires a
	* conversion from joint rotations into mesh space, then the blend, then a conversion back.
	*
	* @param OutResultPose Output blended pose of relative bone transforms.
	* @param OutResultCurve Output blended curves
	*/
	UE_DEPRECATED(4.26, "Use BlendPosesTogetherPerBone with other signature")
	static ENGINE_API void BlendPosesTogetherPerBoneInMeshSpace(
		TArrayView<FCompactPose>           SourcePoses,
		TArrayView<const FBlendedCurve>    SourceCurves,
		const UBlendSpace*                 BlendSpace,
		TArrayView<const FBlendSampleData> BlendSampleDataCache,
		FCompactPose&                      OutResultPose,
		FBlendedCurve&                     OutResultCurve);

	/**
	* Blends together a set of local space (not mesh space) poses, each with a given weight.
	* This function is for BlendSpace per bone blending. BlendSampleDataCache contains the 
	* weight information
	*
	* This blends the rotations in mesh space, so is performance intensive as it requires a 
	* conversion from joint rotations into mesh space, then the blend, then a conversion back.
	*
	* @param OutAnimationPoseData Output pose, curves and attributes.
	*/
	static ENGINE_API void BlendPosesTogetherPerBoneInMeshSpace(
		TArrayView<FCompactPose>                             SourcePoses,
		TArrayView<const FBlendedCurve>                      SourceCurves,
		TArrayView<const UE::Anim::FStackAttributeContainer> SourceAttributes,	
		const UBlendSpace*                                   BlendSpace,
		TArrayView<const FBlendSampleData>                   BlendSampleDataCache,
		FAnimationPoseData&                                  OutAnimationPoseData);

	/** Blending flags for BlendPosesPerBoneFilter */
	enum class EBlendPosesPerBoneFilterFlags : uint32
	{
		None = 0,
		MeshSpaceRotation = (1 << 0),	//! Blend bone rotations in mesh space instead of local space
		MeshSpaceScale    = (1 << 1),	//! Blend bone scales in mesh space instead of local space
		RootSpaceRotation = (1 << 2),	//! Blend bone rotations in root space instead of local space
	};

	/**
	* Blend Poses per bone weights : The BasePoses + BlendPoses(SourceIndex) * Blend Weights(BoneIndex)
	* Please note BlendWeights are array, so you can define per bone base
	* This supports multi per bone blending, but only one pose as blend at a time per track
	* PerBoneBlendWeights.Num() == Atoms.Num()
	*
	* @note : This optionally blends rotation and/or scale in mesh space. Translation is always in local space.
	*
	* I had multiple debates about having PerBoneBlendWeights array(for memory reasons),
	* but it so much simplifies multiple purpose for this function instead of searching bonenames or
	* having multiple bone names with multiple weights, and filtering through which one is correct one
	* I assume all those things should be determined before coming here and this only cares about weights
	**/
	static ENGINE_API void BlendPosesPerBoneFilter(
		FCompactPose& BasePose,
		const TConstArrayView<FCompactPose> BlendPoses,
		FBlendedCurve& BaseCurve,
		const TConstArrayView<FBlendedCurve> BlendedCurves,
		UE::Anim::FStackAttributeContainer& BaseAttributes,
		const TArrayView<UE::Anim::FStackAttributeContainer> BlendAttributes,
		FAnimationPoseData& OutAnimationPoseData,
		const TArrayView<FPerBoneBlendWeight> BoneBlendWeights,
		EBlendPosesPerBoneFilterFlags BlendFlags,
		enum ECurveBlendOption::Type CurveBlendOption);

	static ENGINE_API void UpdateDesiredBoneWeight(
		const TArrayView<FPerBoneBlendWeight> SrcBoneBlendWeights,
		TArrayView<FPerBoneBlendWeight> TargetBoneBlendWeights,
		const TArrayView<float> BlendWeights);

	/**
	 *	Create Mask Weight for skeleton joints, not per mesh or per required bones
	 *  You'll have to filter properly with correct mesh joint or required bones
	 *  The depth should not change based on LOD or mesh or skeleton
	 *	They still should contain same depth
	 */
	static ENGINE_API void CreateMaskWeights(
			TArray<FPerBoneBlendWeight>& BoneBlendWeights,
			const TArray<FInputBlendPose>& BlendFilters, 
			const USkeleton* Skeleton);

	/**
	 *	Create Mask Weight for skeleton joints, not per mesh or per required bones
	 *  Individual alphas are read from a BlendProfile using a BlendMask mode
	 */
	static ENGINE_API void CreateMaskWeights(
		TArray<FPerBoneBlendWeight>& BoneBlendWeights,
		const TArray<class UBlendProfile*>& BlendMasks,
		const USkeleton* Skeleton);

	static ENGINE_API void CombineWithAdditiveAnimations(
		int32 NumAdditivePoses,
		const FTransformArrayA2** SourceAdditivePoses,
		const float* SourceAdditiveWeights,
		const FBoneContainer& RequiredBones,
		/*inout*/ FTransformArrayA2& Atoms);

	/** Get Reference Component Space Transform */
	static ENGINE_API FTransform GetComponentSpaceRefPose(const FCompactPoseBoneIndex& CompactPoseBoneIndex, const FBoneContainer& BoneContainer);

	/** Fill ref pose **/
	static ENGINE_API void FillWithRefPose(TArray<FTransform>& OutAtoms, const FBoneContainer& RequiredBones);

	/** Convert LocalTransforms into MeshSpaceTransforms over RequiredBones. */
	static ENGINE_API void ConvertPoseToMeshSpace(const TArray<FTransform>& LocalTransforms, TArray<FTransform>& MeshSpaceTransforms, const FBoneContainer& RequiredBones);

	/** Convert TargetPose into an AdditivePose, by doing TargetPose = TargetPose - BasePose */
	static ENGINE_API void ConvertPoseToAdditive(FCompactPose& TargetPose, const FCompactPose& BasePose);

	/** convert transform to additive */
	static ENGINE_API void ConvertTransformToAdditive(FTransform& TargetTrasnform, const FTransform& BaseTransform);

	/** Convert LocalPose into MeshSpaceRotations. Rotations are NOT normalized. */
	static ENGINE_API void ConvertPoseToMeshRotation(FCompactPose& LocalPose);

	/** Convert a MeshSpaceRotation pose to Local Space. Rotations are NOT normalized. */
	static ENGINE_API void ConvertMeshRotationPoseToLocalSpace(FCompactPose& Pose);

	/** Accumulate Additive Pose based on AdditiveType*/
	UE_DEPRECATED(4.26, "Use AccumulateAdditivePose with other signature")
	static ENGINE_API void AccumulateAdditivePose(FCompactPose& BasePose, const FCompactPose& AdditivePose, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight, enum EAdditiveAnimationType AdditiveType);
	
	static ENGINE_API void AccumulateAdditivePose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& AdditiveAnimationPoseData, float Weight, enum EAdditiveAnimationType AdditiveType);

private:
	/** Accumulates weighted AdditivePose to BasePose. Rotations are NOT normalized. */
	static ENGINE_API void AccumulateLocalSpaceAdditivePoseInternal(FCompactPose& BasePose, const FCompactPose& AdditivePose, float Weight);

	/** Accumulate a MeshSpaceRotation Additive pose to a local pose. Rotations are NOT normalized */
	static ENGINE_API void AccumulateMeshSpaceRotationAdditiveToLocalPoseInternal(FCompactPose& BasePose, const FCompactPose& MeshSpaceRotationAdditive, float Weight);

	/** Accumulate a RootSpaceRotation Additive pose to a local pose. Rotations are NOT normalized */
	static ENGINE_API void AccumulateRootSpaceAdditivePoseInternal(FCompactPose& BasePose, const FCompactPose& RootSpaceAdditivePose, float Weight);

public:

	/** Accumulates weighted AdditivePose to BasePose. Rotations are NOT normalized. */
	UE_DEPRECATED(4.26, "Use AccumulateAdditivePose with other signature")
	static ENGINE_API void AccumulateLocalSpaceAdditivePose(FCompactPose& BasePose, const FCompactPose& AdditivePose, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight);

	static ENGINE_API void AccumulateLocalSpaceAdditivePose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& AdditiveAnimationPoseData, float Weight);

	/** Accumulate a MeshSpaceRotation Additive pose to a local pose. Rotations are NOT normalized */
	UE_DEPRECATED(4.26, "Use AccumulateAdditivePose with other signature")
	static ENGINE_API void AccumulateMeshSpaceRotationAdditiveToLocalPose(FCompactPose& BasePose, const FCompactPose& MeshSpaceRotationAdditive, FBlendedCurve& BaseCurve, const FBlendedCurve& AdditiveCurve, float Weight);

	static ENGINE_API void AccumulateMeshSpaceRotationAdditiveToLocalPose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& MeshSpaceRotationAdditiveAnimationPoseData, float Weight);

	static ENGINE_API void AccumulateRootSpaceRotationAdditiveToLocalPose(FAnimationPoseData& BaseAnimationPoseData, const FAnimationPoseData& MeshSpaceRotationAdditiveAnimationPoseData, float Weight);

	/** Lerp for FCompactPose. Stores results in PoseA. Performs PoseA = Lerp(PoseA, PoseB, Alpha); */
	static ENGINE_API void LerpPoses(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha);

	/** Lerp for FCompactPose. Stores results in PoseA. 
	 * For each bone performs BoneA[i] = Lerp(BoneA[i], BoneB[i], Alpha * PerBoneWeights[i]);
	 */
	static ENGINE_API void LerpPosesPerBone(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha, const TArray<float>& PerBoneWeights);

	/** Lerp for FCompactPose. Stores results in PoseA. Performs PoseA = Lerp(PoseA, PoseB, Alpha);
	 * on reduced set of bones defined in BoneIndices list.
	 */
	static ENGINE_API void LerpPosesWithBoneIndexList(FCompactPose& PoseA, const FCompactPose& PoseB, FBlendedCurve& CurveA, const FBlendedCurve& CurveB, float Alpha, const TArray<FCompactPoseBoneIndex>& BoneIndices);

	/** Lerp for BoneTransforms. Stores results in A. Performs A = Lerp(A, B, Alpha);
	 * @param A : In/Out transform array.
	 * @param B : B In transform array.
	 * @param Alpha : Alpha.
	 * @param RequiredBonesArray : Array of bone indices.
	 */
	static ENGINE_API void LerpBoneTransforms(TArray<FTransform>& A, const TArray<FTransform>& B, float Alpha, const TArray<FBoneIndexType>& RequiredBonesArray);

	/** 
	 * Blend Array of Transforms by weight
	 *
	 * @param OutTransform : result
	 */
	UE_DEPRECATED(4.26.0, "Please use BlendTransform() for weighted blending")
	static ENGINE_API void BlendTransformsByWeight(FTransform& OutTransform, const TArray<FTransform>& Transforms, const TArray<float>& Weights);

	/**
	 * Mirror (swap) curves with the specified MirrorDataTable.
	 * Partial mirroring is supported,  and curves with two entries (Right->Left and Left->Right) will be swapped while 
	 * curves with a single entry will be overwritten
	 *
	 * @param	Curves			The Curves which are swapped
	 * @param	MirrorDataTable	A UMirrorDataTable specifying which curves to swap
	 */
	static ENGINE_API void MirrorCurves(FBlendedCurve& Curves, const UMirrorDataTable& MirrorDataTable);

	/**
	 * Mirror a vector across the specified mirror axis 
	 * @param	V			The vector to mirror
	 * @param	MirrorAxis	The axis to mirror across
	 * @return				The vector mirrored across the specified axis
	 */
	static ENGINE_API FVector MirrorVector(const FVector& V, EAxis::Type MirrorAxis);

	/** 
	 * Mirror a quaternion across the specified mirror axis 
	 * @param	Q			The quaternion to mirror
	 * @param	MirrorAxis	The axis to mirror across
	 * @return				The quaternion mirrored across the specified axis
	 */
	static ENGINE_API FQuat MirrorQuat(const FQuat& Q, EAxis::Type MirrorAxis);

	/** 
	 * Mirror a pose with the specified MirrorDataTable.  
	 * This method computes the required compact mirror pose and component space reference rotations each call
	 * and should not be used for repeated calculations
	 * 
	 * @param	Pose			The pose which is mirrored in place
	 * @param	MirrorDataTable	A UMirrorDataTable for the same Skeleton as the Pose 
	 */
	static ENGINE_API void MirrorPose(FCompactPose& Pose, const UMirrorDataTable& MirrorDataTable);

	/** Mirror Pose using cached mirror bones and components space arrays.   
	 * 
	 * @param	Pose						The pose which is mirrored in place
	 * @param	MirrorAxis					The axis that all bones are mirrored across 
	 * @param	CompactPoseMirrorBones		Compact array of bone indices. Each index contains the bone to mirror or -1 to indicate mirroring should not apply to that bone.
	 * @param	ComponentSpaceRefRotations	Compoenent space rotations of the reference pose for each bone. 
	 */
	static ENGINE_API void MirrorPose(FCompactPose& Pose, EAxis::Type MirrorAxis, const TArray<FCompactPoseBoneIndex>& CompactPoseMirrorBones, const TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex>& ComponentSpaceRefRotations);

	/** 
	 * Advance CurrentTime to CurrentTime + MoveDelta. 
	 * It will handle wrapping if bAllowLooping is true
	 *
	 * return ETypeAdvanceAnim type
	 */
	static ENGINE_API ETypeAdvanceAnim AdvanceTime(const bool bAllowLooping, const float MoveDelta, float& InOutTime, const float EndTime);

	static ENGINE_API void TickBlendWeight(float DeltaTime, float DesiredWeight, float& Weight, float& BlendTime);
	/** 
	 * Apply Weight to the Transform 
	 * Atoms = Weight * Atoms at the end
	 */
	static ENGINE_API void ApplyWeightToTransform(const FBoneContainer& RequiredBones, /*inout*/ FTransformArrayA2& Atoms, float Weight);

	/** 
	 * Get Key Indices (start/end with alpha from start) with input parameter Time, NumKeys
	 * from % from StartKeyIndex, meaning (CurrentKeyIndex(float)-StartKeyIndex)/(EndKeyIndex-StartKeyIndex)
	 * by this Start-End, it will be between 0-(NumKeys-1), not number of Pos/Rot key tracks 
	 * The FramesPerSecond parameter must be the sample rate of the animation data, for example 30.
	 * If the FramesPerSecond parameter is set to 0 or negative, it will automatically calculate the FramesPerSecond based on the sequence length and number of frames.
	 * The reason why you can provide a FramesPerSecond value is because this can be slightly more accurate than calculating it, in case super high precision is needed.
	 **/
	static ENGINE_API void GetKeyIndicesFromTime(int32& OutKeyIndex1, int32& OutKeyIndex2, float& OutAlpha, const double Time, const int32 NumKeys, const double SequenceLength, double FramesPerSecond=-1.0);
	
	/** Get KeyIndices using FFrameRate::AsFrameTime to calculate the keys and alpha value **/
	static ENGINE_API void GetKeyIndicesFromTime(int32& OutKeyIndex1, int32& OutKeyIndex2, float& OutAlpha, const double Time, const FFrameRate& FrameRate, const int32 NumberOfKeys);

	/** 
	 *	Utility for taking an array of bone indices and ensuring that all parents are present 
	 *	(ie. all bones between those in the array and the root are present). 
	 *	Note that this must ensure the invariant that parent occur before children in BoneIndices.
	 */
	static ENGINE_API void EnsureParentsPresent(TArray<FBoneIndexType>& BoneIndices, const FReferenceSkeleton& RefSkeleton);

	static ENGINE_API void ExcludeBonesWithNoParents(const TArray<int32>& BoneIndices, const FReferenceSkeleton& RefSkeleton, TArray<int32>& FilteredRequiredBones);

	/** 
	 * Convert a ComponentSpace FTransform to specified bone space. 
	 * @param	ComponentTransform	The transform of the component. Only used if Space == BCS_WorldSpace
	 * @param	MeshBases			The pose to use when transforming
	 * @param	InOutCSBoneTM		The component space transform to convert
	 * @param	BoneIndex			The bone index of the transform
	 * @param	Space				The space to convert the input transform into.
	 */
	static ENGINE_API void ConvertCSTransformToBoneSpace(const FTransform& ComponentTransform, FCSPose<FCompactPose>& MeshBases, FTransform& InOutCSBoneTM, FCompactPoseBoneIndex BoneIndex, EBoneControlSpace Space);

	/** 
	 * Convert a FTransform in a specified bone space to ComponentSpace.
	 * @param	ComponentTransform	The transform of the component. Only used if Space == BCS_WorldSpace
	 * @param	MeshBases			The pose to use when transforming
	 * @param	InOutBoneSpaceTM	The bone transform to convert
	 * @param	BoneIndex			The bone index of the transform
	 * @param	Space				The space that the transform is in.
	 */
	static ENGINE_API void ConvertBoneSpaceTransformToCS(const FTransform& ComponentTransform, FCSPose<FCompactPose>& MeshBases, FTransform& InOutBoneSpaceTM, FCompactPoseBoneIndex BoneIndex, EBoneControlSpace Space);

	// FA2Pose/FA2CSPose Interfaces for template functions
	static ENGINE_API FTransform GetSpaceTransform(FA2Pose& Pose, int32 Index);
	static ENGINE_API FTransform GetSpaceTransform(FA2CSPose& Pose, int32 Index);
	static ENGINE_API void SetSpaceTransform(FA2Pose& Pose, int32 Index, FTransform& NewTransform);
	static ENGINE_API void SetSpaceTransform(FA2CSPose& Pose, int32 Index, FTransform& NewTransform);
	// space bases
	static ENGINE_API FTransform GetComponentSpaceTransformRefPose(const FReferenceSkeleton& RefSkeleton, int32 BoneIndex);
	static ENGINE_API FTransform GetComponentSpaceTransform(const FReferenceSkeleton& RefSkeleton, const TArrayView<const FTransform> &BoneSpaceTransforms, int32 BoneIndex);
	static ENGINE_API void FillUpComponentSpaceTransforms(const FReferenceSkeleton& RefSkeleton, const TArrayView<const FTransform> &BoneSpaceTransforms, TArray<FTransform> &ComponentSpaceTransforms);
	static ENGINE_API void MakeSkeletonRefPoseFromMesh(const USkeletalMesh* InMesh, const USkeleton* InSkeleton, TArray<FTransform>& OutBoneBuffer);

	/**
	 * Get the translational retargeting mode for a given bone on the target skeleton when sampling a given source animation.
	 * If bUseSourceRetargetModes is set to true, and SourceSkeletonBoneIndex is INDEX_NONE, then the target bone and target skeleton are used instead.
	 * If bUseSourceRetargetModes is set to false, the SourceSkeleton and SourceSkeletonBoneIndex values are ignored and we get the retarget mode directly from the target skeleton using the target bone index.
	 * 
	 * @param	bUseSourceRetargetModes		Set to true when you wish to extract the retarget mode from the source skeleton instead of the target skeleton.
	 * @param	SourceSkeletonBoneIndex		The skeleton bone index of the bone on the source skeleton to get the retarget mode from.
	 * @param	TargetSkeletonBoneIndex		The target bone index of the bone on the target skeleton to get the retarget mode from.
	 * @param	SourceSkeleton				A pointer to the skeleton of the source (the animation's skeleton for example).
	 * @param	TargetSkeleton				A pointer to the target skeleton, which is the skeleton to apply transforms to.
	 * @param	bDisableRetargeting			Is translational retargeting disabled?
	 * @return	The retargeting mode for the specific bone.
	 */
	static ENGINE_API EBoneTranslationRetargetingMode::Type GetBoneTranslationRetargetingMode(const bool bUseSourceRetargetModes, const int32 SourceSkeletonBoneIndex, const int32 TargetSkeletonBoneIndex, const USkeleton* SourceSkeleton, const USkeleton* TargetSkeleton, const bool bDisableRetargeting);

	/**
	 * Calculate the component-space bone transform for the specified bone. This is similar to GetComponentSpaceTransform(), but uses a cached array of 
	 * transforms to prevent re-calculations when requesting transforms for multiple bones with shared parents. This is only likely to be more efficient than
	 * FillUpComponentSpaceTransforms if you only need a relatively small number of bone transforms.
	 * @param	InRefSkeleton			The bone hierarchy
	 * @param	InBoneSpaceTransforms	Bone-space transforms for all bones in the hierarchy
	 * @param	BoneIndex				We calculate the component-space transform of this bone
	 * @param	CachedTransforms		An array of transforms which holds the transforms of any bones whose transforms have been previously calculated. Should be initialized with CachedTransforms.SetNumUninitialized(BoneSpaceTransforms.Num()) before first use.
	 * @param	CachedTransformReady	An array of flags indicating which bone transforms have been cached. Should be initialized with CachedTransformReady.SetNumZeroed(BoneSpaceTransforms.Num()) before first use.
	 */
	static ENGINE_API const FTransform& GetComponentSpaceTransformWithCache(const FReferenceSkeleton& InRefSkeleton, const TArray<FTransform> &InBoneSpaceTransforms, int32 BoneIndex, TArray<FTransform>& CachedTransforms, TArray<bool>& CachedTransformReady);

#if WITH_EDITOR
	static ENGINE_API void FillUpComponentSpaceTransformsRefPose(const USkeleton* Skeleton, TArray<FTransform> &ComponentSpaceTransforms);
#endif

	/* Weight utility functions */
	static inline bool IsFullWeight(float Weight) { return FAnimWeight::IsFullWeight(Weight); }
	static inline bool HasWeight(float Weight) { return FAnimWeight::IsRelevant(Weight); }
	
	/**
	* Combine CurveKeys (that reference morph targets by name) and ActiveAnims (that reference morphs by reference) into the ActiveMorphTargets array.
	*/
 	static ENGINE_API void AppendActiveMorphTargets(const USkeletalMesh* InSkeletalMesh, const TMap<FName, float>& InMorphCurveAnims, FMorphTargetWeightMap& InOutActiveMorphTargets, TArray<float>& InOutMorphTargetWeights);

	/**
	* Retarget a single bone transform, to apply right after extraction.
	*
	* @param	SourceSkeleton		Skeleton from which this is retargeting
	* @param	RetargetSource		Retarget Source for the retargeting
	* @param	BoneTransform		BoneTransform to read/write from
	* @param	SkeletonBoneIndex	Source Bone Index in SourceSkeleton
	* @param	BoneIndex			Target Bone Index in Bone Transform array
	* @param	RequiredBones		BoneContainer to which this is retargeting
	*/
	static ENGINE_API void RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& RetargetSource, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive);
	static ENGINE_API void RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& SourceName, const TArray<FTransform>& RetargetTransforms, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FBoneContainer& RequiredBones, const bool bIsBakedAdditive);
	static ENGINE_API void RetargetBoneTransform(const USkeleton* SourceSkeleton, const FName& SourceName, const TArray<FTransform>& RetargetTransforms, FTransform& BoneTransform, const int32 SkeletonBoneIndex, const FCompactPoseBoneIndex& BoneIndex, const FAnimationRuntimeRetargetBoneTransformContext* InContext, const bool bIsBakedAdditive);
	
	/** 
	 * Calculate distance how close two strings are. 
	 * By close, it calculates how many operations to transform First to Second 
	 * The return value is [0-MaxLengthString(First, Second)]
	 * 0 means it's identical, Max means it's completely different
	 */
	static ENGINE_API int32 GetStringDistance(const FString& First, const FString& Second);
};

ENUM_CLASS_FLAGS(FAnimationRuntime::EBlendPosesPerBoneFilterFlags);

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Components/SkinnedMeshComponent.h"
#endif
