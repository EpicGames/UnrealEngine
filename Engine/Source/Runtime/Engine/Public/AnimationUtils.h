// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationUtils.h: Skeletal mesh animation utilities.
=============================================================================*/ 

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimSequence.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimNotify, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimMontage, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimMarkerSync, Warning, All);

// Forward declarations.
class UAnimSet;
class UVariableFrameStrippingSettings;

/**
 * Error statistics for an animation generated by TallyErrorsFromPerturbation, representing errors in translation or rotation due to a small incremental translation or rotation
 */
struct FAnimPerturbationError
{
	float MaxErrorInTransDueToTrans;
	float MaxErrorInRotDueToTrans;
	//float MaxErrorInScaleDueToTrans; // Not used TODO: Either use or remove
	float MaxErrorInTransDueToRot;
	float MaxErrorInRotDueToRot;
	//float MaxErrorInScaleDueToRot; // Not used TODO: Either use or remove
	float MaxErrorInTransDueToScale; 
	float MaxErrorInRotDueToScale;
	//float MaxErrorInScaleDueToScale; // Not used TODO: Either use or remove
};

using AnimationErrorStats = FAnimationErrorStats;

/** Different ways to create an additive animation */
enum EConvertToAdditive
{
	/** Delta against SkelMesh Reference pose */
	CTA_RefPose,
	/** Delta against the first frame of another animation */
	CTA_AnimFirstFrame,
	/** Delta against another animation. Other animation scaled to match first anim's length */
	CTA_AnimScaled,
	CTA_MAX	// Always last one
};

/**
 * A collection of useful functions for skeletal mesh animation.
 */
class FAnimationUtils
{
public:


	/**
	* Builds the local-to-component matrix for the specified bone.
	*/
	static void BuildComponentSpaceTransform(FTransform& OutTransform,
												int32 BoneIndex,
												const TArray<FTransform>& BoneSpaceTransforms,
												const TArray<FBoneData>& BoneData);

	ENGINE_API static void BuildSkeletonMetaData(USkeleton* Skeleton, TArray<FBoneData>& OutBoneData);

	static int32 GetAnimTrackIndexForSkeletonBone(const int32 InSkeletonBoneIndex, const TArray<FTrackToSkeletonMap>& TrackToSkelMap);

#if WITH_EDITOR
	/**
	 * Utility function to measure the accuracy of a compressed animation. Each end-effector is checked for 
	 * world-space movement as a result of compression.
	 *
	 * @param	AnimSet		The animset to calculate compression error for.
	 * @param	BoneData	BoneData array describing the hierarchy of the animated skeleton
	 * @param	ErrorStats	Output structure containing the final compression error values
	 * @return				None.
	 */
	ENGINE_API static void ComputeCompressionError(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedData, AnimationErrorStats& ErrorStats);
#endif
	
	/**
	 * Determines the current setting for recompressing all animations upon load. The default value 
	 * is False, but may be overridden by an optional field in the base engine INI file. 
	 *
	 * @return				true if the engine settings request that all animations be recompiled
	 */
	static bool GetForcedRecompressionSetting();

	/** Get default Outer for AnimSequences contained in this AnimSet.
	 *  The intent is to use that when Constructing new AnimSequences to put into that set.
	 *  The Outer will be Package.<AnimSetName>_Group.
	 *  @param bCreateIfNotFound if true, Group will be created. This is only in the editor.
	 */
	static UObject* GetDefaultAnimSequenceOuter(UAnimSet* InAnimSet, bool bCreateIfNotFound);

	/**
	 * Converts an animation codec format into a human readable string
	 *
	 * @param	InFormat	The format to convert into a string
	 * @return				The format as a string
	 */
	static FString GetAnimationKeyFormatString(enum AnimationKeyFormat InFormat);

	/**
	 * Converts an animation compression type into a human readable string
	 *
	 * @param	InFormat	The compression format to convert into a string
	 * @return				The format as a string
	 */
	ENGINE_API static FString GetAnimationCompressionFormatString(enum AnimationCompressionFormat InFormat);

	/**
	 * Computes the 'height' of each track, relative to a given animation linkup.
	 *
	 * The track height is defined as the minimal number of bones away from an end effector (end effectors are 0, their parents are 1, etc...)
	 *
	 * @param AnimLinkup			The animation linkup
	 * @param NumTracks				The number of tracks
	 * @param TrackHeights [OUT]	The computed track heights
	 *
	 */
#if WITH_EDITOR
	static void CalculateTrackHeights(const FCompressibleAnimData& CompressibleAnimData, int NumTracks, TArray<int32>& TrackHeights);
#endif // WITH_EDITOR

	/**
	 * Checks a set of key times to see if the spacing is uniform or non-uniform.
	 * Note: If there are as many times as frames, they are automatically assumed to be uniformly spaced.
	 * Note: If there are two or fewer times, they are automatically assumed to be uniformly spaced.
	 *
	 * @param NumFrames		The number of frames in the source animation data
	 * @param Times			The array of key times
	 *
	 * @return				true if the keys are uniformly spaced (or one of the trivial conditions is detected).  false if any key spacing is greater than 1e-4 off.
	 */
	static bool HasUniformKeySpacing(int32 NumFrames, const TArray<float>& Times);

	/**
	 * Perturbs the bone(s) associated with each track in turn, measuring the maximum error introduced in end effectors as a result
	 */
	
#if WITH_EDITOR
	static void TallyErrorsFromPerturbation(
		const FCompressibleAnimData& CompressibleAnimData,
		int32 NumTracks,
		const FVector& PositionNudge,
		const FQuat& RotationNudge,
		const FVector& ScaleNudge,
		TArray<FAnimPerturbationError>& InducedErrors);

	/** Preload the bone, curve and recorder compression settings */
	ENGINE_API static void PreloadCompressionSettings();
#endif
	
	/** Returns the default animation bone compression settings, can never by null. */
	ENGINE_API static UAnimBoneCompressionSettings* GetDefaultAnimationBoneCompressionSettings();

	/** Returns the default animation bone compression settings for the FAnimRecorder, can never by null. */
	ENGINE_API static UAnimBoneCompressionSettings* GetDefaultAnimationRecorderBoneCompressionSettings();

	/** Returns the default animation curve compression settings, can never by null. */
	ENGINE_API static UAnimCurveCompressionSettings* GetDefaultAnimationCurveCompressionSettings();

	/** Returns the default animation variable frame stripping settings, can never by null. */
	ENGINE_API static UVariableFrameStrippingSettings* GetDefaultVariableFrameStrippingSettings();

	/** Makes sure that the animation sequence and all its dependencies required for compression are loaded. */
	ENGINE_API static void EnsureAnimSequenceLoaded(UAnimSequence& AnimSeq);

	// Extract specific frame from raw track and place in OutAtom (return identity if track is malformed)
	ENGINE_API static void ExtractTransformForFrameFromTrackSafe(const FRawAnimSequenceTrack& RawTrack, int32 Frame, FTransform& OutAtom);

	// Extract specific frame from raw track and place in OutAtom
	ENGINE_API static void ExtractTransformForFrameFromTrack(const FRawAnimSequenceTrack& RawTrack, int32 Frame, FTransform& OutAtom);

	UE_DEPRECATED(5.1, "ExtractTransformFromTrack has been deprecated, use different signature")
	ENGINE_API static void ExtractTransformFromTrack(float Time, int32 NumFrames, float SequenceLength, const struct FRawAnimSequenceTrack& RawTrack, EAnimInterpolationType Interpolation, FTransform &OutAtom);

	ENGINE_API static void ExtractTransformFromTrack(const struct FRawAnimSequenceTrack& RawTrack, double Time, int32 NumFrames, double SequenceLength, EAnimInterpolationType Interpolation, FTransform &OutAtom);

#if WITH_EDITOR
	UE_DEPRECATED(5.1, "ExtractTransformFromCompressionData has been deprecated, use different signature")
	static void ExtractTransformFromCompressionData(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedAnimData, float Time, int32 TrackIndex, bool bUseRawData, FTransform& OutBoneTransform);
	
	static void ExtractTransformFromCompressionData(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& CompressedAnimData, double Time, int32 TrackIndex, bool bUseRawData, FTransform& OutBoneTransform);

	/**
	 * Compresses the animation bones within a sequence with the chosen settings.
	 * Note: This modifies the sequence.
	 *
	 * @return		Returns true on success, false it we fail to compress
	 */
	ENGINE_API static bool CompressAnimBones(FCompressibleAnimData& AnimSeq, FCompressibleAnimDataResult& Target);

	/**
	 * Compresses the animation curves within a sequence with the chosen settings.
	 * Note: This modifies the sequence.
	 *
	 * @return		Returns true on success, false it we fail to compress
	 */
	ENGINE_API static bool CompressAnimCurves(FCompressibleAnimData& AnimSeq, FCompressedAnimSequence& Target);
#endif
};
