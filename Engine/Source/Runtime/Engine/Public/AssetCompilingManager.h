// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#if WITH_EDITOR
#include "Application/ThrottleManager.h"
#endif
#include "AssetCompilingManager.generated.h"

class FQueuedThreadPool;
struct IAssetCompilingManager;

namespace AssetCompilation
{
	struct FProcessAsyncTaskParams;
}

USTRUCT(BlueprintType)
struct FAssetCompileData
{
	GENERATED_BODY();

	/** Object being compiled */
	UPROPERTY(BlueprintReadWrite, Category = AssetCompileData)
	TWeakObjectPtr<UObject> Asset;

	FAssetCompileData()
	{}

	FAssetCompileData(const TWeakObjectPtr<UObject>& InAsset)
		: Asset(InAsset)
	{
	}
};

DECLARE_TS_MULTICAST_DELEGATE_OneParam(FAssetPostCompileEvent, const TArray<FAssetCompileData>&);

class FAssetCompilingManager
{
public:
	ENGINE_API static FAssetCompilingManager& Get();

	/**
	 * Register an asset compiling manager.
	 */
	ENGINE_API bool RegisterManager(IAssetCompilingManager* InAssetCompilingManager);

	/**
	 * Unregister an asset compiling manager.
	 */
	ENGINE_API bool UnregisterManager(IAssetCompilingManager* InAssetCompilingManager);

	/**
	 * Register the list of registered managers.
	 */
	ENGINE_API TArrayView<IAssetCompilingManager* const> GetRegisteredManagers() const;

	/** 
	 * Returns the number of remaining compilations.
	 */
	ENGINE_API int32 GetNumRemainingAssets() const;

	/** 
	 * Blocks until completion of all assets.
	 */
	ENGINE_API void FinishAllCompilation();

	/**
	 * Finish compilation of the requested objects.
	 */
	ENGINE_API void FinishCompilationForObjects(TArrayView<UObject* const> InObjects);

	/**
	 * Mark compilation of the provided objects as canceled.
	 * This is a hint that the compilations are no longer needed and may leave the objects in a bad state.
	 * There is no guarantee that anything will be canceled, but if supported, it might remove heavy CPU work earlier.
	 * If you need to make sure that any activity is done on these objects, a call to FinishCompilationForObjects is still required.
	 */
	ENGINE_API void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects);

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	ENGINE_API void Shutdown();

	/**
	 * Returns the thread-pool where asset compilation should be scheduled.
	 */
	ENGINE_API FQueuedThreadPool* GetThreadPool() const;

	/** 
	 * Called once per frame, fetches completed tasks and applies them to the scene. 
	 */
	ENGINE_API void ProcessAsyncTasks(bool bLimitExecutionTime = false);

	/**
	 * Called once per frame, fetches completed tasks and applies them to the scene.
	 */
	ENGINE_API void ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params);

	/**
	 * Event called after an asset finishes compilation.
	 */
	FAssetPostCompileEvent& OnAssetPostCompileEvent() { return AssetPostCompileEvent; }

	/**
	 * Event called before and after FinishAllCompilation or ProcessAsyncTasks run operations on a specific package,
	 * used for subscribers to associate low-level actions with that Package (e.g. TObjectPtr reads).
	 * void OnPackageScopeEvent(UPackage* Package, bool bEntering)
	 * Called with bEntering=true followed by actions for Package followed by call with bEntering=false.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPackageScopeEvent, UPackage*, bool);
	FPackageScopeEvent& OnPackageScopeEvent() { return PackageScopeEvent; }

private:
	FAssetCompilingManager();
	~FAssetCompilingManager();
	friend struct IAssetCompilingManager;

	/** Take some action whenever the number of remaining asset changes. */
	void UpdateNumRemainingAssets();

	bool bHasShutdown = false;
	int32 LastNumRemainingAssets = 0;

	TArray<IAssetCompilingManager*> AssetCompilingManagers;
	TArray<IAssetCompilingManager*> AssetCompilingManagersWithValidDependencies;

	/** Event issued at the end of the compile process */
	FAssetPostCompileEvent AssetPostCompileEvent;

	FPackageScopeEvent PackageScopeEvent;

#if WITH_EDITOR
	/** Held while there are pending assets to prevent Slate from throttling the editor tick rate during compilation. */
	FThrottleRequest ResponsiveModeHandle;
#endif
};
