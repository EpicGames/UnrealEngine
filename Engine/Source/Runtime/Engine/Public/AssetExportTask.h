// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetExportTask.generated.h"

/**
 * Contains data for a group of assets to export
 */ 
UCLASS(Transient, BlueprintType, MinimalAPI)
class UAssetExportTask : public UObject
{
	GENERATED_BODY()

public:
	/** Asset to export */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TObjectPtr<UObject> Object;

	/** Optional exporter, otherwise it will be determined automatically */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TObjectPtr<class UExporter> Exporter;

	/** File to export as */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	FString Filename;

	/** Export selected only */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bSelected;

	/** Replace identical files */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bReplaceIdentical;

	/** Allow dialog prompts */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bPrompt;

	/** Unattended export */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bAutomated;

	/** Save to a file archive */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bUseFileArchive;

	/** Write even if file empty */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bWriteEmptyFiles;

	/** Array of objects to ignore exporting */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TArray<TObjectPtr<UObject>> IgnoreObjectList;

	/** Exporter specific options */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TObjectPtr<UObject> Options;

	/** Array of error messages encountered during exporter */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TArray<FString> Errors;
};
