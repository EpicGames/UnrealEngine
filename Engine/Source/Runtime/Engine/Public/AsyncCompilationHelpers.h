// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/QueuedThreadPool.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAsyncCompilation, Log, All);

class FAsyncCompilationNotification
{
public:
	FAsyncCompilationNotification(FTextFormat InAssetNameFormat)
		: AssetNameFormat(InAssetNameFormat)
	{
	}

	ENGINE_API void Update(int32 NumJobs);
private:
	FProgressNotificationHandle NotificationHandle;
	/** The number of jobs when the notification began */
	int32 StartNumJobs;
	FTextFormat AssetNameFormat;
};

class FQueuedThreadPoolWrapper;

namespace AsyncCompilationHelpers
{
	class ICompilable
	{
	public:
		inline virtual ~ICompilable() = default;

		/*
		* Reschedules any async tasks to the given thread pool at the given priority.
		*/
		virtual void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) = 0;

		/*
		* Returns true once async tasks are complete, false if timing out.
		* 
		* Use a TimeLimitSeconds of 0 to poll for completion.
		*/
		virtual bool WaitCompletionWithTimeout(float TimeLimitSeconds) = 0;
		virtual FName GetName() = 0;
	};

	template <typename AsyncTaskType>
	class TCompilableAsyncTask : public ICompilable
	{
		virtual AsyncTaskType* GetAsyncTask() = 0;

		void Reschedule(FQueuedThreadPool* InThreadPool, EQueuedWorkPriority InPriority) override
		{
			if (AsyncTaskType* AsyncTask = GetAsyncTask())
			{
				AsyncTask->Reschedule(InThreadPool, InPriority);
			}
		}
		
		bool WaitCompletionWithTimeout(float TimeLimitSeconds) override
		{
			if (AsyncTaskType* AsyncTask = GetAsyncTask())
			{
				return AsyncTask->WaitCompletionWithTimeout(TimeLimitSeconds);
			}
			return true;
		}
	};

	ENGINE_API void FinishCompilation(
		TFunctionRef<ICompilable& (int32 Index)> Getter,
		int32 Num,
#if WITH_EDITOR
		const FText& AssetType,
		const FLogCategoryBase& LogCategory,
#endif
		TFunctionRef<void(ICompilable*)> PostCompileSingle
	);

#if WITH_EDITOR

	class FAsyncCompilationStandardCVars
	{
	public:
		TAutoConsoleVariable<int32> AsyncCompilation;
		TAutoConsoleVariable<int32> AsyncCompilationMaxConcurrency;
		FAutoConsoleCommand         AsyncCompilationFinishAll;
		TAutoConsoleVariable<int32> AsyncCompilationResume;

		ENGINE_API FAsyncCompilationStandardCVars(const TCHAR* AssetType, const TCHAR* AssetTypePluralLowerCase, const FConsoleCommandDelegate& FinishAllCommand);
	};

	ENGINE_API void EnsureInitializedCVars(
		const TCHAR* InName,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilation,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationMaxConcurrency,
		FName ExperimentalSettingsName = NAME_None);

	ENGINE_API void BindThreadPoolToCVar(
		FQueuedThreadPoolWrapper* InThreadPoolWrapper,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilation,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationResume,
		TAutoConsoleVariable<int32>& InCVarAsyncCompilationMaxConcurrency
		);

	ENGINE_API void SaveStallStack(uint64 Cycles);
	ENGINE_API void DumpStallStacks();

#endif // WITH_EDITOR
}
