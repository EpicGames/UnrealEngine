// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Audio.h: Unreal base audio.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "AudioDefines.h"
#include "Audio/AudioSpeakers.h"
#include "Stats/Stats.h"
#include "HAL/ThreadSafeBool.h"
#include "Sound/AudioOutputTarget.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/SoundModulationDestination.h"
#include "Sound/SoundSubmixSend.h"
#include "Sound/SoundSourceBusSend.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudio, Display, All);

// Special log category used for temporary programmer debugging code of audio
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioDebug, Display, All);

/**
 * Audio stats
 */
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Active Sounds"), STAT_ActiveSounds, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Audio Evaluate Concurrency"), STAT_AudioEvaluateConcurrency, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Audio Sources"), STAT_AudioSources, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Wave Instances"), STAT_WaveInstances, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Wave Instances Dropped"), STAT_WavesDroppedDueToPriority, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Virtualized Loops"), STAT_AudioVirtualLoops, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Audible Wave Instances Dropped"), STAT_AudibleWavesDroppedDueToPriority, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Max Channels"), STAT_AudioMaxChannels, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Max Stopping Sources"), STAT_AudioMaxStoppingSources, STATGROUP_Audio, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Finished delegates called"), STAT_AudioFinishedDelegatesCalled, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Finished delegates time"), STAT_AudioFinishedDelegates, STATGROUP_Audio, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Audio Memory Used"), STAT_AudioMemorySize, STATGROUP_Audio, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Audio Buffer Time"), STAT_AudioBufferTime, STATGROUP_Audio, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Audio Buffer Time (w/ Channels)"), STAT_AudioBufferTimeChannels, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering WaveInstances"), STAT_AudioGatherWaveInstances, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Processing Sources"), STAT_AudioStartSources, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Updating Sources"), STAT_AudioUpdateSources, STATGROUP_Audio, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Updating Effects"), STAT_AudioUpdateEffects, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Init"), STAT_AudioSourceInitTime, STATGROUP_Audio, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Create"), STAT_AudioSourceCreateTime, STATGROUP_Audio, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submit Buffers"), STAT_AudioSubmitBuffersTime, STATGROUP_Audio, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Decompress Audio"), STAT_AudioDecompressTime, STATGROUP_Audio, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Prepare Audio Decompression"), STAT_AudioPrepareDecompressionTime, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Finding Nearest Location"), STAT_AudioFindNearestLocation, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Decompress Streamed"), STAT_AudioStreamedDecompressTime, STATGROUP_Audio, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Buffer Creation"), STAT_AudioResourceCreationTime, STATGROUP_Audio, );


class FAudioDevice;
class USoundNode;
class USoundWave;
class USoundClass;
class USoundSubmix;
class USoundSourceBus;
class UAudioLinkSettingsAbstract;
struct FActiveSound;
struct FWaveInstance;
struct FSoundSourceBusSendInfo;

// Forward declarations.
class UAudioComponent;
class USoundNode;
struct FWaveInstance;
struct FReverbSettings;
struct FSampleLoop;
struct FSoundWaveTimecodeInfo;
enum ELoopingMode
{
	/** One shot sound */
	LOOP_Never,
	/** Call the user callback on each loop for dynamic control */
	LOOP_WithNotification,
	/** Loop the sound forever */
	LOOP_Forever
};

struct FNotifyBufferFinishedHooks
{
	ENGINE_API void AddNotify(USoundNode* NotifyNode, UPTRINT WaveInstanceHash);
	ENGINE_API UPTRINT GetHashForNode(USoundNode* NotifyNode) const;
	ENGINE_API void AddReferencedObjects( FReferenceCollector& Collector );
	ENGINE_API void DispatchNotifies(FWaveInstance* WaveInstance, const bool bStopped);

	friend FArchive& operator<<( FArchive& Ar, FNotifyBufferFinishedHooks& WaveInstance );

private:

	struct FNotifyBufferDetails
	{
		TObjectPtr<USoundNode> NotifyNode;
		UPTRINT NotifyNodeWaveInstanceHash;

		FNotifyBufferDetails()
			: NotifyNode(nullptr)
			, NotifyNodeWaveInstanceHash(0)
		{
		}

		FNotifyBufferDetails(USoundNode* InNotifyNode, UPTRINT InHash)
			: NotifyNode(InNotifyNode)
			, NotifyNodeWaveInstanceHash(InHash)
		{
		}
	};


	TArray<FNotifyBufferDetails> Notifies;
};

/** Queries if a plugin of the given type is enabled. */
ENGINE_API bool IsAudioPluginEnabled(EAudioPlugin PluginType);
ENGINE_API UClass* GetAudioPluginCustomSettingsClass(EAudioPlugin PluginType);

/** accessor for our Spatialization enabled CVar. */
ENGINE_API bool IsSpatializationCVarEnabled();

/**
 * Interface for listening to source buffers being rendered.
 */
class ISourceBufferListener
{
public:
	virtual ~ISourceBufferListener() = default;

	struct FOnNewBufferParams
	{
		const float* AudioData = nullptr;
		int32 SourceId = INDEX_NONE;
		int32 NumSamples = 0;
		int32 NumChannels = 0;
		int32 SampleRate = 0;
	};
	virtual void OnNewBuffer(const FOnNewBufferParams&) = 0;
	virtual void OnSourceReleased(const int32 InSourceId) = 0;
};
using FSharedISourceBufferListenerPtr = TSharedPtr<ISourceBufferListener, ESPMode::ThreadSafe>;

/** Bus send types */
enum class EBusSendType : uint8
{
	PreEffect,
	PostEffect,
	PostAttenuation,
	Count
};

/**
 * Structure encapsulating all information required to play a USoundWave on a channel/source. This is required
 * as a single USoundWave object can be used in multiple active cues or multiple times in the same cue.
 */
struct FWaveInstance
{
private:
	/** Static helper to create good unique type hashes */
	static ENGINE_API uint32 PlayOrderCounter;

public:
	/** Wave data */
	TObjectPtr<USoundWave> WaveData;

	/** Sound class */
	TObjectPtr<USoundClass> SoundClass;

	/** Sound submix object to send audio to for mixing in audio mixer.  */
	USoundSubmixBase* SoundSubmix;

	/** Sound submix sends */
	TArray<FSoundSubmixSendInfo> SoundSubmixSends;

	/** The source bus and/or audio bus sends. */
	TArray<FSoundSourceBusSendInfo> BusSends[(int32)EBusSendType::Count];

	/** Sound effect chain */
	USoundEffectSourcePresetChain* SourceEffectChain;

	/** Sound nodes to notify when the current audio buffer finishes */
	FNotifyBufferFinishedHooks NotifyBufferFinishedHooks;

	/** Active Sound this wave instance belongs to */
	FActiveSound* ActiveSound;

	/** Quantized Request data */
	TUniquePtr<Audio::FQuartzQuantizedRequestData> QuantizedRequestData;

	/** Source Buffer listener */
	FSharedISourceBufferListenerPtr SourceBufferListener;
	bool bShouldSourceBufferListenerZeroBuffer = false;

	/** AudioLink Opt in */
	bool bShouldUseAudioLink = true;
	UAudioLinkSettingsAbstract* AudioLinkSettingsOverride = nullptr;

private:

	/** Current volume */
	float Volume;

	/** Volume attenuation due to distance. */
	float DistanceAttenuation;

	/** Volume attenuation due to occlusion. */
	float OcclusionAttenuation;

	/** Current volume multiplier - used to zero the volume without stopping the source */
	float VolumeMultiplier;

	/** The current envelope value of the wave instance. */
	float EnvelopValue;

	/** The estimated relative render cost of the wave instance. 1.0 is cost of a single decoding sound source. Used for limited the overall voice count. */
	float RelativeRenderCost;

public:
	/** The envelope follower attack time in milliseconds. */
	int32 EnvelopeFollowerAttackTime;

	/** The envelope follower release time in milliseconds. */
	int32 EnvelopeFollowerReleaseTime;

	/** An audio component priority value that scales with volume (post all gain stages) and is used to determine voice playback priority. */
	float Priority;

	/** Voice center channel volume */
	float VoiceCenterChannelVolume;

	/** Volume of the radio filter effect */
	float RadioFilterVolume;

	/** The volume at which the radio filter kicks in */
	float RadioFilterVolumeThreshold;

	/** The amount of a sound to bleed to the LFE channel */
	float LFEBleed;

	/** Looping mode - None, loop with notification, forever */
	ELoopingMode LoopingMode;

	/** An offset/seek time to play this wave instance. */
	float StartTime;

	/** Whether or not to enable sending this audio's output to buses.*/
	uint32 bEnableBusSends : 1;

	/** Whether or not to render to the main submix */
	uint32 bEnableBaseSubmix : 1;

	/** Whether or not to enable Submix Sends in addition to the Main Submix*/
	uint32 bEnableSubmixSends : 1;

	/** Whether or not to use source data overrides */
	uint32 bEnableSourceDataOverride : 1;

	/** Set to true if the sound nodes state that the radio filter should be applied */
	uint32 bApplyRadioFilter:1;

	/** Whether wave instanced has been started */
	uint32 bIsStarted:1;

	/** Whether wave instanced is finished */
	uint32 bIsFinished:1;

	/** Whether the notify finished hook has been called since the last update/parsenodes */
	uint32 bAlreadyNotifiedHook:1;

	/** Whether the spatialization method is an external send */
	uint32 bSpatializationIsExternalSend : 1;

private:
	/** Whether to use spatialization */
	uint32 bUseSpatialization:1;

public:
	/** Whether or not to enable the low pass filter */
	uint32 bEnableLowPassFilter:1;
	
	/** Whether or not to enable the high pass filter */
	uint32 bEnableHighPassFilter:1;

	/** Whether or not the sound is occluded. */
	uint32 bIsOccluded:1;

	/** Whether or not this sound plays when the game is paused in the UI */
	uint32 bIsUISound:1;

	/** Whether or not this wave is music */
	uint32 bIsMusic:1;

	/** Whether or not this wave has reverb applied */
	uint32 bReverb:1;

	/** Whether or not this sound class forces sounds to the center channel */
	uint32 bCenterChannelOnly:1;

	/** Whether or not this sound is manually paused */
	uint32 bIsPaused:1;

	/** Prevent spamming of spatialization of surround sounds by tracking if the warning has already been emitted */
	uint32 bReportedSpatializationWarning:1;

	/** Whether or not this wave instance is ambisonics. */
	uint32 bIsAmbisonics:1;

	/** Whether or not this wave instance is stopping. */
	uint32 bIsStopping:1;

	/** Is this or any of the submixes above it dynamic */
	uint32 bIsDynamic:1;

	/** Which spatialization method to use to spatialize 3d sounds. */
	ESoundSpatializationAlgorithm SpatializationMethod;

	/** The occlusion plugin settings to use for the wave instance. */
	USpatializationPluginSourceSettingsBase* SpatializationPluginSettings;

	/** The occlusion plugin settings to use for the wave instance. */
	UOcclusionPluginSourceSettingsBase* OcclusionPluginSettings;

	/** The occlusion plugin settings to use for the wave instance. */
	UReverbPluginSourceSettingsBase* ReverbPluginSettings;

	/** The source data override plugin settings to use for the wave instance. */
	USourceDataOverridePluginSourceSettingsBase* SourceDataOverridePluginSettings;

	/** Which output target the sound should play on. */
	EAudioOutputTarget::Type OutputTarget;

	/** The low pass filter frequency to use */
	float LowPassFilterFrequency;
	
	/** The high pass filter frequency to use */
	float HighPassFilterFrequency;

	/** The low pass filter frequency to use from sound class. */
	float SoundClassFilterFrequency;

	/** The low pass filter frequency to use if the sound is occluded. */
	float OcclusionFilterFrequency;

	/** The low pass filter frequency to use due to ambient zones. */
	float AmbientZoneFilterFrequency;

	/** The low pass filter frequency to use due to distance attenuation. */
	float AttenuationLowpassFilterFrequency;

	/** The high pass filter frequency to use due to distance attenuation. (using in audio mixer only) */
	float AttenuationHighpassFilterFrequency;

	/** Current pitch scale. */
	float Pitch;

	/** Current location */
	FVector Location;

	/** At what distance we start transforming into non-spatialized soundsource */
	float NonSpatializedRadiusStart;

	/** At what distance we are fully non-spatialized*/
	float NonSpatializedRadiusEnd;

	/** How we are doing the non-spatialized radius feature. */
	ENonSpatializedRadiusSpeakerMapMode NonSpatializedRadiusMode;

	/** Amount of spread for 3d multi-channel asset spatialization */
	float StereoSpread;

	/** Distance over which the sound is attenuated. */
	float AttenuationDistance;

	/** The distance from this wave instance to the closest listener. */
	float ListenerToSoundDistance;

	/** The distance from this wave instance to the closest listener. (ignoring attenuation override) */
	float ListenerToSoundDistanceForPanning;

	/** The absolute position of the wave instance relative to forward vector of listener. */
	float AbsoluteAzimuth; 

	/** The playback time of the wave instance. Updated from active sound. */
	float PlaybackTime;

	/** The linear percentage of the Wave which has been played based on its duration. Updated from Mixer Source. */
	float PlaybackPercent;
	
	/** The output reverb send level to use for tje wave instance. */
	float ReverbSendLevel;

	/** TODO remove */
	float ManualReverbSendLevel;
	
	/** The submix send settings to use. */
	TArray<FAttenuationSubmixSendSettings> AttenuationSubmixSends;

private:
	/** Cached play order */
	uint32 PlayOrder;

public:
	/** Hash value for finding the wave instance based on the path through the cue to get to it */
	UPTRINT WaveInstanceHash;

	/** User / Controller index that owns the sound */
	uint8 UserIndex;

	/** Constructor, initializing all member variables. */
	ENGINE_API FWaveInstance(const UPTRINT InWaveInstanceHash, FActiveSound& ActiveSound);

	ENGINE_API FWaveInstance(FWaveInstance&&);
	ENGINE_API FWaveInstance& operator=(FWaveInstance&&);

	/** Stops the wave instance without notifying NotifyWaveInstanceFinishedHook. */
	ENGINE_API void StopWithoutNotification();

	/** Notifies the wave instance that the current playback buffer has finished. */
	ENGINE_API void NotifyFinished(const bool bStopped = false);

	/** Friend archive function used for serialization. */
	friend FArchive& operator<<(FArchive& Ar, FWaveInstance* WaveInstance);

	/** Function used by the GC. */
	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/** Returns the actual volume the wave instance will play at */
	ENGINE_API bool ShouldStopDueToMaxConcurrency() const;

	/** Setters for various values on wave instances. */
	void SetVolume(const float InVolume) { Volume = InVolume; }
	void SetDistanceAttenuation(const float InDistanceAttenuation) { DistanceAttenuation = InDistanceAttenuation; }
	void SetOcclusionAttenuation(const float InOcclusionAttenuation) { OcclusionAttenuation = InOcclusionAttenuation; }
	void SetPitch(const float InPitch) { Pitch = InPitch; }
	void SetVolumeMultiplier(const float InVolumeMultiplier) { VolumeMultiplier = InVolumeMultiplier; }

	void SetStopping(const bool bInIsStopping) { bIsStopping = bInIsStopping; }
	bool IsStopping() const { return bIsStopping; }

	/** Returns whether or not the WaveInstance is actively playing sound or set to
	  * play when silent.
	  */
	ENGINE_API bool IsPlaying() const;

	/** Returns the volume multiplier on the wave instance. */
	float GetVolumeMultiplier() const { return VolumeMultiplier; }

	/** Returns the actual volume the wave instance will play at, including all gain stages. */
	ENGINE_API float GetActualVolume() const;

	/** Returns the volume of the sound including distance attenuation. */
	ENGINE_API float GetVolumeWithDistanceAndOcclusionAttenuation() const;

	/** Returns the combined distance and occlusion attenuation of the source voice. */
	ENGINE_API float GetDistanceAndOcclusionAttenuation() const;

	/** Returns the distance attenuation of the source voice */
	ENGINE_API float GetDistanceAttenuation() const;

	/** Returns the occlusion attenuation of the source voice */
	ENGINE_API float GetOcclusionAttenuation() const;

	/** Returns the dynamic volume of the sound */
	ENGINE_API float GetDynamicVolume() const;

	/** Returns the pitch of the wave instance */
	ENGINE_API float GetPitch() const;

	/** Returns the volume of the wave instance (ignoring application muting) */
	ENGINE_API float GetVolume() const;

	/** Returns the weighted priority of the wave instance. */
	ENGINE_API float GetVolumeWeightedPriority() const;

	ENGINE_API bool IsSeekable() const;

	/** Checks whether wave is streaming and streaming is supported */
	ENGINE_API bool IsStreaming() const;

	/** Returns the name of the contained USoundWave */
	ENGINE_API FString GetName() const;

	/** Sets the envelope value of the wave instance. Only set if the wave instance is actually generating real audio with a source voice. Only implemented in the audio mixer. */
	void SetEnvelopeValue(const float InEnvelopeValue) { EnvelopValue = InEnvelopeValue; }

	/** Gets the envelope value of the waveinstance. Only returns non-zero values if it's a real voice. Only implemented in the audio mixer. */
	float GetEnvelopeValue() const { return EnvelopValue; }

	/** Whether to use spatialization, which controls 3D effects like panning */
	void SetUseSpatialization(const bool InUseSpatialization) { bUseSpatialization = InUseSpatialization; }

	/** Whether this wave will be spatialized, which controls 3D effects like panning */
	ENGINE_API bool GetUseSpatialization() const;

	/** Whether spatialization is an external send */
	void SetSpatializationIsExternalSend(const bool InSpatializationIsExternalSend) { bSpatializationIsExternalSend = InSpatializationIsExternalSend; }

	/** Whether spatialization is an external send */
	bool GetSpatializationIsExternalSend() const {	return bSpatializationIsExternalSend;}

	uint32 GetPlayOrder() const { return PlayOrder; }

	friend inline uint32 GetTypeHash(FWaveInstance* A) { return A->PlayOrder; }

	/** Sets the relative render cost of the wave instance. */
	void SetRelativeRenderCost(float InRelativeRenderCost) { RelativeRenderCost = InRelativeRenderCost; }

	/** Retrieves the relative render cost of wave instance. */
	float GetRelativeRenderCost() const { return RelativeRenderCost; }

	ENGINE_API float GetEffectiveModulationValue(const EModulationDestination InDestination) const;

	ENGINE_API FSoundModulationDestinationSettings GetEffectiveModulationSettings(const EModulationDestination InDestination) const;

private:
	static Audio::FModulationParameter GetParameterBasedOnDestination(const EModulationDestination InDestination);

	/** This function calculates the effective modulation settings and current value, based on the currently configured modulation routing settings
	 * (e.g. it will use inherited modulators if that destination's routing is set to Inherit). */
	void CalculateInheritedModulationSettingsAndValue(const EModulationDestination InDestination, FSoundModulationDestinationSettings& InOutSettings, float& OutValue) const;
};

/*-----------------------------------------------------------------------------
	FSoundBuffer.
-----------------------------------------------------------------------------*/

class FSoundBuffer
{
public:
	FSoundBuffer(class FAudioDevice * InAudioDevice)
		: ResourceID(0)
		, NumChannels(0)
		, bAllocationInPermanentPool(false)
		, AudioDevice(InAudioDevice)
	{

	}

	ENGINE_API virtual ~FSoundBuffer();

	virtual int32 GetSize() PURE_VIRTUAL(FSoundBuffer::GetSize,return 0;);

	/**
	 * Describe the buffer (platform can override to add to the description, but should call the base class version)
	 * 
	 * @param bUseLongNames If TRUE, this will print out the full path of the sound resource, otherwise, it will show just the object name
	 */
	ENGINE_API virtual FString Describe(bool bUseLongName);

	/**
	 * Return the name of the sound class for this buffer 
	 */
	FName GetSoundClassName();

	/**
	 * Turn the number of channels into a string description
	 */
	FString GetChannelsDesc();

	/**
	 * Reads the compressed info of the given sound wave. Not implemented on all platforms.
	 */
	virtual bool ReadCompressedInfo(USoundWave* SoundWave) { return true; }

	/** Reads the next compressed data chunk */
	virtual bool ReadCompressedData(uint8* Destination, int32 NumFramesToDecode, bool bLooping) { return true; }
	
	/** Seeks the buffer to the given seek time */
	virtual void Seek(const float SeekTime) {}

	/**
	 * Gets the chunk index that was last read from (for Streaming Manager requests)
	 */
	virtual int32 GetCurrentChunkIndex() const {return -1;}

	/**
	 * Gets the offset into the chunk that was last read to (for Streaming Manager priority)
	 */
	virtual int32 GetCurrentChunkOffset() const {return -1;}

	/** Returns whether or not a real-time decoding buffer is ready for playback */
	virtual bool IsRealTimeSourceReady() { return true; }

	/** Forces any pending async realtime source tasks to finish for the buffer */
	virtual void EnsureRealtimeTaskCompletion() { }

	/** Unique ID that ties this buffer to a USoundWave */
	int32	ResourceID;
	/** Cumulative channels from all streams */
	int32	NumChannels;
	/** Human readable name of resource, most likely name of UObject associated during caching.	*/
	FString	ResourceName;
	/** Whether memory for this buffer has been allocated from permanent pool. */
	bool	bAllocationInPermanentPool;
	/** Parent Audio Device used when creating the sound buffer. Used to remove tracking info on this sound buffer when its done. */
	class FAudioDevice * AudioDevice;
};

/*-----------------------------------------------------------------------------
FSoundSource.
-----------------------------------------------------------------------------*/

class FSoundSource
{
public:
	/** Constructor */
	FSoundSource(FAudioDevice* InAudioDevice)
		: AudioDevice(InAudioDevice)
		, WaveInstance(nullptr)
		, LFEBleed(0.5f)
		, LPFFrequency(MAX_FILTER_FREQUENCY)
		, HPFFrequency(MIN_FILTER_FREQUENCY)
		, PlaybackTime(0.0f)
		, Pitch(1.0f)
		, LastUpdate(0)
		, LastHeardUpdate(0)
		, TickCount(0)
		, LeftChannelSourceLocation(0)
		, RightChannelSourceLocation(0)
		, NumFramesPlayed(0)
		, NumTotalFrames(1)
		, StartFrame(0)
		, VoiceId(-1)
		, Playing(false)
		, bReverbApplied(false)
		, bIsPausedByGame(false)
		, bIsManuallyPaused(false)
		, Paused(false)
		, bInitialized(true) // Note: this is defaulted to true since not all platforms need to deal with async initialization.
		, bIsPreviewSound(false)
		, bIsVirtual(false)
	{
	}

	/** Destructor */
	ENGINE_API virtual ~FSoundSource();

	/* Prepares the source voice for initialization. This may parse a compressed asset header on some platforms */
	virtual bool PrepareForInitialization(FWaveInstance* InWaveInstance) { return true; }

	/** Returns if the source voice is prepared to initialize. */
	virtual bool IsPreparedToInit() { return true; }

	/** Initializes the sound source. */
	virtual bool Init(FWaveInstance* InWaveInstance) = 0;

	/** Returns whether or not the sound source has initialized. */
	virtual bool IsInitialized() const { return bInitialized; };

	/** Updates the sound source. */
	virtual void Update() = 0;

	/** Plays the sound source. */
	virtual void Play() = 0;

	/** Stops the sound source. */
	ENGINE_API virtual void Stop();

	virtual void StopNow() { Stop(); };

	/** Whether or not the source is stopping. Only implemented in audio mixer. */
	virtual bool IsStopping() { return false; }

	/** Returns true if the sound source has finished playing. */
	virtual	bool IsFinished() = 0;
	
	/** Pause the source from game pause */
	void SetPauseByGame(bool bInIsPauseByGame);

	/** Pause the source manually */
	void SetPauseManually(bool bInIsPauseManually);

	/** Returns a string describing the source (subclass can override, but it should call the base and append). */
	ENGINE_API virtual FString Describe(bool bUseLongName);

	/** Returns source is an in-game only. Will pause when in UI. */
	bool IsGameOnly() const;

	/** Returns the wave instance of the sound source. */
	const FWaveInstance* GetWaveInstance() const { return WaveInstance; }

	/** Returns whether or not the sound source is playing. */
	bool IsPlaying() const { return Playing; }

	/**  Returns true if the sound is paused. */
	bool IsPaused() const { return Paused; }
	
	/**  Returns true if the sound is paused. */
	bool IsPausedByGame() const { return bIsPausedByGame; }

	bool IsPausedManually() const { return bIsManuallyPaused; }

	/** Returns true if reverb should be applied. */
	bool IsReverbApplied() const { return bReverbApplied; }

	/** Set the bReverbApplied variable. */
	ENGINE_API bool SetReverbApplied(bool bHardwareAvailable);

	/** Updates and sets the LFEBleed variable. */
	ENGINE_API float SetLFEBleed();

	/** Updates the FilterFrequency value. */
	ENGINE_API void SetFilterFrequency();

	/** Updates the stereo emitter positions of this voice. */
	ENGINE_API void UpdateStereoEmitterPositions();

	/** Gets parameters necessary for computing 3d spatialization of sources. */
	ENGINE_API FSpatializationParams GetSpatializationParams();

	/** Returns the contained sound buffer object. */
	UE_DEPRECATED(5.5, "The Buffer member no longer exists.  Use GetNumChannels() to qeury channel count.")
	virtual const FSoundBuffer* GetBuffer() const { return nullptr; }

	int32 GetNumChannels() const { return NumChannels; }

	/** Initializes any source effects for this sound source. */
	virtual void InitializeSourceEffects(uint32 InEffectVoiceId)
	{
	}

	/** Sets if this voice is virtual. */
	void SetVirtual()
	{
		bIsVirtual = true;
	}

	/** Returns the source's playback percent. */
	ENGINE_API virtual float GetPlaybackPercent() const;

	/** Returns the sample (frame) rate of the audio played by the sound source. */
	ENGINE_API virtual float GetSourceSampleRate() const;

	/** Returns the number of frames (Samples / NumChannels) played by the sound source. */
	ENGINE_API virtual int64 GetNumFramesPlayed() const;

	/** Returns the total number of frames of audio for the sound wave. */
	ENGINE_API virtual int32 GetNumTotalFrames() const;

	/** Returns the frame index on which the sound source began playback. */
	ENGINE_API virtual int32 GetStartFrame() const;

	/** Returns the source's envelope at the callback block rate. Only implemented in audio mixer. */
	virtual float GetEnvelopeValue() const { return 0.0f; };

	/** Returns the source's estimated relative render cost (relative to a single decoded sound). Used for debug information and to constrain overall CPU usage. */
	virtual float GetRelativeRenderCost() const { return 1.0f; }

	ENGINE_API void GetChannelLocations(FVector& Left, FVector&Right) const;

	void NotifyPlaybackData();

protected:

	/** Initializes common data for all sound source types. */
	ENGINE_API void InitCommon();

	/** Updates common data for all sound source types. */
	ENGINE_API void UpdateCommon();

	/** Pauses the sound source. */
	virtual void Pause() = 0;

	/** Updates this source's pause state */
	void UpdatePause();

	/** Returns the volume of the sound source after evaluating debug commands */
	ENGINE_API float GetDebugVolume(const float InVolume);

	/** Owning audio device. */
	FAudioDevice* AudioDevice;

	/** Contained wave instance. */
	FWaveInstance* WaveInstance;

	/** Cached sound buffer associated with currently bound wave instance. */
	UE_DEPRECATED(5.5, "This is no longer used and should not be accessed directly.")
	FSoundBuffer* Buffer;

	/** The amount of a sound to bleed to the LFE speaker */
	float LFEBleed;

	/** What frequency to set the LPF filter to. Note this could be caused by occlusion, manual LPF application, or LPF distance attenuation. */
	float LPFFrequency;

	/** What frequency to set the HPF filter to. Note this could be caused by HPF distance attenuation. */
	float HPFFrequency;

	/** The last LPF frequency set. Used to avoid making API calls when parameter doesn't changing. */	
	UE_DEPRECATED(5.6, "This Variable duplicates cache behavior used at source voice level")
	float LastLPFFrequency=MAX_FILTER_FREQUENCY;

	/** The last HPF frequency set. Used to avoid making API calls when parameter doesn't changing. */
	UE_DEPRECATED(5.6, "This Variable duplicates cache behavior used at source voice level")
	float LastHPFFrequency=MIN_FILTER_FREQUENCY;

	/** The virtual current playback time. Used to trigger notifications when finished. */
	float PlaybackTime;

	/** The pitch of the sound source. */
	float Pitch;

	/** Last tick when this source was active */
	int32 LastUpdate;

	/** Last tick when this source was active *and* had a hearable volume */
	int32 LastHeardUpdate;

	/** Update tick count. Used to stop oldest stopping sound source. */
	int32 TickCount;

	/** The location of the left-channel source for stereo spatialization. */
	FVector LeftChannelSourceLocation;

	/** The location of the right-channel source for stereo spatialization. */
	FVector RightChannelSourceLocation;

	/** The number of frames (Samples / NumChannels) played by the sound source. */
	int32 NumFramesPlayed;

	/** The total number of frames of audio for the sound wave */
	int32 NumTotalFrames;

	/** The frame we started on. */
	int32 StartFrame;

	/** the number of channels */
	int32 NumChannels = 0;

	/** Effect ID of this sound source in the audio device sound source array. */
	uint32 VoiceId;

	/** Whether we are playing or not. */
	FThreadSafeBool Playing;

	/** Cached sound mode value used to detect when to switch outputs. */
	uint8 bReverbApplied : 1;

	/** Whether we are paused by game state or not. */
	uint8 bIsPausedByGame : 1;

	/** Whether or not we were paused manually. */
	uint8 bIsManuallyPaused : 1;

	/** Whether or not we are actually paused. */
	uint8 Paused : 1;

	/** Whether or not the sound source is initialized. */
	uint8 bInitialized : 1;

	/** Whether or not the sound is a preview sound. */
	uint8 bIsPreviewSound : 1;

	/** True if this isn't a real hardware voice */
	uint32 bIsVirtual : 1;

	friend class FAudioDevice;
	friend struct FActiveSound;

#if ENABLE_AUDIO_DEBUG
public:

	/** Struct containing the debug state of a SoundSource */
	struct FDebugInfo
	{
		/** True if this sound has been soloed. */
		bool bIsSoloed = false;

		/** True if this sound has been muted . */
		bool bIsMuted = false;

		/** Reason why this sound is mute/soloed. */
		FString MuteSoloReason;

		/** Fraction of a single CPU core used to render audio. */
		double CPUCoreUtilization = 0;

		/** Relative cost to render wave. */
		float RelativeRenderCost = 1.f;

		/** Max relative render cost already logged for this sound 
		(used to limit log spam from RelativeRenderCostLogThresholdCVar) */
		float MaxLoggedRelativeRenderCost = 0.f;

		/** Basic CS so we can pass this around safely. */
		FCriticalSection CS;
	};
	
	TSharedPtr<FDebugInfo, ESPMode::ThreadSafe> DebugInfo;
	friend struct FDebugInfo;
#endif //ENABLE_AUDIO_DEBUG
};

// Data representing a cue in a wave file
struct FWaveCue
{
	// Unique identifying gvalue for the cue
	uint32 CuePointID = 0;
	// Sample offset associated with the cue point
	uint32 Position = 0;
	// Cue label
	FString Label;
	// If this is a region, it will have a duration (sample length)
	uint32 SampleLength = 0;
};

// data representing a sample loop in a wave file
struct FWaveSampleLoop
{
	uint32 LoopID = 0;
	uint32 StartFrame = 0;
	uint32 EndFrame = 0;
};

//
// Structure for in-memory interpretation and modification of WAVE sound structures.
//
class FWaveModInfo
{
public:

	// Format specifiers
	static constexpr uint16 WAVE_INFO_FORMAT_PCM = 0x0001;
	static constexpr uint16 WAVE_INFO_FORMAT_ADPCM = 0x0002;
	static constexpr uint16 WAVE_INFO_FORMAT_IEEE_FLOAT = 0x0003;
	static constexpr uint16 WAVE_INFO_FORMAT_DVI_ADPCM = 0x0011;
	static constexpr uint16 WAVE_INFO_FORMAT_OODLE_WAVE = 0xFFFF;

	// Pointers to variables in the in-memory WAVE file.
	const uint32* pSamplesPerSec;
	const uint32* pAvgBytesPerSec;
	const uint16* pBlockAlign;
	const uint16* pBitsPerSample;
	const uint16* pChannels;
	uint16* pFormatTag;

	const uint32* pWaveDataSize;
	const uint32* pMasterSize;
	const uint8*  SampleDataStart;
	const uint8*  SampleDataEnd;
	uint32  SampleDataSize;
	const uint8*  WaveDataEnd;

	uint32  NewDataSize;

	// List of cues parsed from the wave file
	TArray<FWaveCue> WaveCues;

	// List of sample loops parsed from the wave file
	TArray<FWaveSampleLoop> WaveSampleLoops;

	// Timecode data if it was found on import.
	TPimplPtr<FSoundWaveTimecodeInfo, EPimplPtrMode::DeepCopy> TimecodeInfo;

	// Constructor.
	FWaveModInfo()
	{
	}
	
	// 16-bit padding.
	static uint32 Pad16Bit( uint32 InDW )
	{
		return ((InDW + 1)& ~1);
	}

	/** Wave Chunk Id utils */
	ENGINE_API static const TArray<uint32>& GetRequiredWaveChunkIds();
	ENGINE_API static const TArray<uint32>& GetOptionalWaveChunkIds();
	ENGINE_API static const uint32 GetChunkId(const char ChunkID[4]);

	// Read headers and load all info pointers in WaveModInfo. 
	// Returns 0 if invalid data encountered.
	ENGINE_API bool ReadWaveInfo(const uint8* WaveData, int32 WaveDataSize, FString* ErrorMessage = NULL, bool InHeaderDataOnly = false, void** OutFormatHeader = NULL );
	
	/**
	 * Read a wave file header from bulkdata
	 */
	ENGINE_API bool ReadWaveHeader(const uint8* RawWaveData, int32 Size, int32 Offset );

	ENGINE_API void ReportImportFailure() const;

	/** Return total number of samples */
	ENGINE_API uint32 GetNumSamples() const;

	/** Return whether file format is supported for import */
	ENGINE_API bool IsFormatSupported() const;
	/** Return whether file format contains uncompressed PCM data */
	ENGINE_API bool IsFormatUncompressed() const;

};

/** Utility to serialize raw PCM data into a wave file. */
ENGINE_API void SerializeWaveFile(TArray<uint8>& OutWaveFileData, const uint8* InPCMData, const int32 NumBytes, const int32 NumChannels, const int32 SampleRate);

/**
 * Brings loaded sounds up to date for the given platforms (or all platforms), and also sets persistent variables to cover any newly loaded ones.
 *
 * @param	Platform				Name of platform to cook for, or NULL if all platforms
 */
ENGINE_API void SetCompressedAudioFormatsToBuild(const TCHAR* Platform = NULL);

/**
 * Brings loaded sounds up to date for the given platforms (or all platforms), and also sets persistent variables to cover any newly loaded ones.
 *
 * @param	Platform				Name of platform to cook for, or NULL if all platforms
 */
ENGINE_API const TArray<FName>& GetCompressedAudioFormatsToBuild();


