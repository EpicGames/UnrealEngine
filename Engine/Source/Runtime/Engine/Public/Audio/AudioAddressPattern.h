// Copyright Epic Games, Inc. All Rights Reserved.
// Pattern-Matching that fulfills the OSC 1.0 Protocol on address pattern matching
#pragma once

#include "CoreMinimal.h"

class FAudioAddressPattern
{
	static int32 FindPatternTerminatorIndex(const FString& Pattern, int32 PatternIter, TCHAR Terminator);

	static const TArray<TCHAR>& GetInvalidChars();
	static const TArray<TCHAR>& GetPatternChars();
	
public:
	static ENGINE_API bool BracePatternMatches(const FString& Pattern, int32 PatternStartIndex, int32 PatternEndIndex, const FString& Part, int32& PartIter);
	static ENGINE_API bool BracketPatternMatches(const FString& Pattern, int32 PatternStartIndex, int32 PatternEndIndex, TCHAR MatchChar);
	static ENGINE_API bool IsValidPatternPart(const FString& Part);
	static ENGINE_API bool IsValidPattern(const TArray<FString>& InContainers, const FString& InMethod);
	static ENGINE_API bool IsValidPath(const FString& Path, bool bInvalidateSeparator);
	static ENGINE_API bool PartsMatch(const FString& Pattern, const FString& Part);
	static ENGINE_API bool WildPatternMatches(const FString& Pattern, int32& PatternIter, const FString& Part, int32& PartIter);
};
