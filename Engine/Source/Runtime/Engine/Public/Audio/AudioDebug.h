// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioDefines.h"		// For ENABLE_AUDIO_DEBUG
#include "Audio.h"

namespace Audio
{
	// enum shortener for logging.
	inline const TCHAR* ShortEnumString(const TCHAR* InString)
	{
		if (const TCHAR* DoubleColon = FCString::Strstr(InString, TEXT("::")))
		{
			return DoubleColon + 2;
		}
		return InString;
	}
	
#if UE_BUILD_SHIPPING
	inline bool MatchesLogFilter(const TCHAR* InName) { return false; } // Off for shipping.
#else
	ENGINE_API bool MatchesLogFilter(const TCHAR* InName);
#endif // UE_BUILD_SHIPPING
	
	inline bool MatchesLogFilter(const FName& InName)
	{
#if UE_BUILD_SHIPPING
		return false;		// Off for shipping.
#else 		
		return MatchesLogFilter(*InName.ToString());
#endif //		
	}
}

#if ENABLE_AUDIO_DEBUG

 // Forward Declarations
struct FActiveSound;
struct FAudioVirtualLoop;
struct FListener;
struct FWaveInstance;

class FSoundSource;
class FViewportClient;
class USoundSubmix;
class USoundWave;
class UWorld;


namespace Audio
{
	namespace DebugStatNames
	{
		ENGINE_API extern const FName SoundWaves;
		ENGINE_API extern const FName SoundCues;
		ENGINE_API extern const FName Sounds;
		ENGINE_API extern const FName SoundMixes;
		ENGINE_API extern const FName SoundModulation;
		ENGINE_API extern const FName SoundReverb;
		ENGINE_API extern const FName AudioStreaming;

		// TODO: Move to console variables
		ENGINE_API extern const FName DebugSounds;
		ENGINE_API extern const FName LongSoundNames;
	}

	class FAudioDebugger
	{
	public:
		ENGINE_API FAudioDebugger();

		/** Struct which contains debug names for run-time debugging of sounds. */
		struct FDebugNames
		{
			TArray<FName> SoloSoundClass;
			TArray<FName> SoloSoundWave;
			TArray<FName> SoloSoundCue;
			TArray<FName> SoloSubmix;
			TArray<FName> MuteSoundClass;
			TArray<FName> MuteSoundWave;
			TArray<FName> MuteSoundCue;
			TArray<FName> MuteSubmix;

			FString DebugAudioMixerSoundName;
			FString DebugSoundName;
			bool bDebugSoundName;

			FDebugNames()
				: bDebugSoundName(false)
			{}
		};

		static ENGINE_API void DrawDebugInfo(const FSoundSource& SoundSource);
		static ENGINE_API void DrawDebugInfo(const FActiveSound& ActiveSound, const TArray<FWaveInstance*>& ThisSoundsWaveInstances, const float DeltaTime);
		static ENGINE_API void DrawDebugInfo(UWorld& World, const TArray<FListener>& Listeners);
		static ENGINE_API void DrawDebugInfo(const FAudioVirtualLoop& VirtualLoop);
		static ENGINE_API int32 DrawDebugStats(UWorld& World, FViewport* Viewport, FCanvas* Canvas, int32 Y);
		static ENGINE_API bool DrawDebugStatsEnabled();
		static ENGINE_API bool PostStatModulatorHelp(UWorld* World, FCommonViewportClient* ViewportClient, const TCHAR* Stream);
		static ENGINE_API int32 RenderStatCues(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y);
		static ENGINE_API int32 RenderStatMixes(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y);
		static ENGINE_API int32 RenderStatModulators(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);
		static ENGINE_API int32 RenderStatReverb(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y);
		static ENGINE_API int32 RenderStatSounds(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y);
		static ENGINE_API int32 RenderStatWaves(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y);
		static ENGINE_API int32 RenderStatStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation);
		static ENGINE_API void RemoveDevice(const FAudioDevice& AudioDevice);
		static ENGINE_API void ResolveDesiredStats(FViewportClient* ViewportClient);
		static ENGINE_API void SendUpdateResultsToGameThread(const FAudioDevice& AudioDevice, const int32 FirstActiveIndex);
		static ENGINE_API void UpdateAudibleInactiveSounds(const uint32 FirstIndex, const TArray<FWaveInstance*>& WaveInstances);
		static ENGINE_API void LogSubtitle(const TCHAR* InCmd, USoundWave& InSoundWave);
		static ENGINE_API void ClearStats(const FName StatsToToggle, UWorld* InWorld);
		static ENGINE_API void SetStats(const TSet<FName>& StatsToToggle, UWorld* InWorld);

		static ENGINE_API bool IsVirtualLoopVisualizeEnabled();

		ENGINE_API void ClearMutesAndSolos();
		ENGINE_API bool IsMuteOrSoloActive() const;
		ENGINE_API bool IsSubmixMuteOrSoloActive() const;
		ENGINE_API void DumpActiveSounds() const;

		ENGINE_API bool IsVisualizeDebug3dEnabled() const;
		ENGINE_API void ToggleVisualizeDebug3dEnabled();

#if WITH_EDITOR
		static ENGINE_API void OnBeginPIE();
		static ENGINE_API void OnEndPIE();
#endif // WITH_EDITOR

		// Evaluate Mute/Solos
		ENGINE_API void QuerySoloMuteSoundClass(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const;
		ENGINE_API void QuerySoloMuteSoundWave(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const;
		ENGINE_API void QuerySoloMuteSoundCue(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const;
		ENGINE_API void QuerySoloMuteSubmix(const FString& Name, bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const;

		// Is Mute/Solos. (only audio thread).
		bool IsSoloSoundClass(FName InName) const { return DebugNames.SoloSoundClass.Contains(InName); }
		bool IsSoloSoundWave(FName InName) const { return DebugNames.SoloSoundWave.Contains(InName); }
		bool IsSoloSoundCue(FName InName) const { return DebugNames.SoloSoundCue.Contains(InName); }
		bool IsSoloSubmix(FName InName) const { return DebugNames.SoloSubmix.Contains(InName); }
		bool IsMuteSoundClass(FName InName) const { return DebugNames.MuteSoundClass.Contains(InName); }
		bool IsMuteSoundWave(FName InName) const { return DebugNames.MuteSoundWave.Contains(InName); }
		bool IsMuteSoundCue(FName InName) const { return DebugNames.MuteSoundCue.Contains(InName); }
		bool IsMuteSubmix(FName InName) const { return DebugNames.MuteSubmix.Contains(InName); }

		// Mute/Solos toggles. (any thread). (If exclusive, toggle-on will clear everything first, and toggle-off will clear all).
		void ToggleSoloSoundClass(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.SoloSoundClass, bExclusive); }
		void ToggleSoloSoundWave(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.SoloSoundWave, bExclusive); }
		void ToggleSoloSoundCue(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.SoloSoundCue, bExclusive); }
		void ToggleMuteSoundClass(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.MuteSoundClass, bExclusive); }
		void ToggleMuteSoundWave(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.MuteSoundWave, bExclusive); }
		void ToggleMuteSoundCue(FName InName, bool bExclusive = false) { ToggleNameArray(InName, DebugNames.MuteSoundCue, bExclusive); }

		// Set Mute/Solo. (any thread).
		void SetMuteSoundCue(FName InName, bool bInOnOff) { SetNameArray(InName, DebugNames.MuteSoundCue, bInOnOff); }
		void SetMuteSoundWave(FName InName, bool bInOnOff) { SetNameArray(InName, DebugNames.MuteSoundWave, bInOnOff); }
		void SetSoloSoundCue(FName InName, bool bInOnOff) { SetNameArray(InName, DebugNames.SoloSoundCue, bInOnOff); }
		void SetSoloSoundWave(FName InName, bool bInOnOff) { SetNameArray(InName, DebugNames.SoloSoundWave, bInOnOff); }

		// Submix chain solo/mute. (game thread)
		// Recomputes the solo chain by walking the USoundSubmix hierarchy from explicitly soloed submixes.
		// Submixes linked to explicitly soloed submixes in the USoundSubmix hierarchy are implicitly soloed.
		// Explicitly-muted submixes are excluded from the implicit-solo chain.
		ENGINE_API void ToggleSoloSubmixChain(const FName InSubmixName);
		ENGINE_API void SetSoloSubmixChain(const FName InSubmixName, const bool bInSolo);

		ENGINE_API void ToggleMuteSubmix(const FName InSubmixName);
		ENGINE_API void SetMuteSubmix(const FName InSubmixName, const bool bInMute);

		ENGINE_API void ClearSubmixMuteSoloState();
		ENGINE_API bool IsSubmixExplicitlySoloed(const FName InSubmixName) const;
		ENGINE_API bool IsSubmixExplicitlyMuted(const FName InSubmixName) const;

		ENGINE_API void SetAudioMixerDebugSound(const TCHAR* SoundName);
		ENGINE_API void SetAudioDebugSound(const TCHAR* SoundName);

		ENGINE_API const FString& GetAudioMixerDebugSoundName() const;
		ENGINE_API bool GetAudioDebugSound(FString& OutDebugSound);

	private:
		static ENGINE_API int32 DrawDebugStatsInternal(UWorld& World, FViewport& Viewport, FCanvas* Canvas, int32 InY);
		ENGINE_API void SetNameArray(FName InName, TArray<FName>& InNameArray, bool bOnOff);
		ENGINE_API void ToggleNameArray(FName InName, TArray<FName>& NameArray, bool bExclusive);
		ENGINE_API void ExecuteCmdOnAudioThread(TFunction<void()> Cmd);

		ENGINE_API void GetDebugSoloMuteStateX(
			const FString& Name, const TArray<FName>& Solos, const TArray<FName>& Mutes,
			bool& bOutIsSoloed, bool& bOutIsMuted, FString& OutReason) const;

		ENGINE_API void ClearStats(FDeviceId DeviceId, FName StatsToClear);

		ENGINE_API void SetStats(FDeviceId DeviceId, const TSet<FName>& StatsToSet);

		static ENGINE_API bool ToggleStats(UWorld* World, const TSet<FName>& StatToToggle);
		ENGINE_API void ToggleStats(FDeviceId DeviceId, const TSet<FName>& StatsToToggle);

		void RefreshComputedSubmixMuteSoloChain();
		void ProcessExplicitlySoloedSubmixChains(const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames);
		void ProcessExplicitlySoloedMainSubmixChains(const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames);
		void AddImplicitSolosFromHierarchy(const TObjectPtr<USoundSubmix>& InSubmix, const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutComputedSoloNames);
		void CollectSubmixHierarchyNames(const TObjectPtr<USoundSubmix>& InSubmix, const TObjectPtr<USoundSubmix>& InMainSubmix, const TObjectPtr<USoundSubmix>& InBaseDefaultSubmix, TSet<FName>& OutNames) const;

		FDelegateHandle WorldRegisteredWithDeviceHandle;

		/** Instance of the debug names struct. */
		FDebugNames DebugNames;

		/** Explicit = submixes directly soloed/muted by the user. Other submixes may be soloed implicitly. */
		TSet<FName> ExplicitlySoloedSubmixes;
		TSet<FName> ExplicitlyMutedSubmixes;

		/** Whether or not 3d debug visualization is enabled. */
		uint8 bVisualize3dDebug : 1;
	};
} // namespace Audio
#endif // ENABLE_AUDIO_DEBUG
