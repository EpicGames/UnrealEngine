// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "Engine/Font.h"
#include "Engine/Engine.h"

namespace Audio
{
	class FTabularRenderHelper
	{
	public:
		FTabularRenderHelper() = default;
		
		void Draw(FCanvas* Canvas, int32 X, int32 Y) const
		{
			DrawHeaders(Canvas,X,Y, HeaderColor);
			Y+=Font->GetMaxCharHeight();
			DrawRows(Canvas,X,Y, RowColor);
		}

		struct FItem { FString Text; FLinearColor Color=FColor::Green; };
		void AddRow(const TArray<FItem>& Items)
		{
			for(int32 i = 0; i < Items.Num(); ++i)
			{
				if (Cols.IsValidIndex(i))
				{
					Cols[i].Items.Add(Items[i].Text);
					Cols[i].Colors.Add(Items[i].Color);
				}
			}
		}
		void AddCol(FString Text, int32 WidthInChars=0)
		{		
			WidthInChars = !WidthInChars ? Text.Len() : WidthInChars;
			Text=Text.LeftPad(WidthInChars);
			int32 X,Y;
			StringSize(Font, X, Y, *Text);
			Cols.Add({Text, X});
		}
	private:
		struct FCol
		{
			FString Header;
			int32 Width = 32;
			TArray<FString> Items;
			TArray<FLinearColor> Colors;
		};
		TArray<FCol> Cols;
		FColor HeaderColor = FColor::White;
		FColor RowColor = FColor::Green;
		const UFont* Font = UEngine::GetTinyFont();

		void RightJustify(FCanvas* Canvas, const int32 X, const int32 Y, const int32 InterColumnOffset, TCHAR const* Text, FLinearColor const& Color) const
		{
			int32 SizeX, SizeY;
			StringSize(Font, SizeX, SizeY, Text);
			Canvas->DrawShadowedString(X + InterColumnOffset - SizeX, Y, Text, Font, Color);
		}
		void DrawHeaders(FCanvas* Canvas, int32 X, int32 Y,const FLinearColor& Color) const
		{
			for (const FCol& i : Cols)
			{
				RightJustify(Canvas,X,Y,i.Width,*i.Header, Color);
				X += i.Width;
			}
		}
		void DrawRows(FCanvas* Canvas, const int32 InX, const int32 InY,const FLinearColor& Color) const
		{
			int32 Line=0;
			const int32 MaxY = Canvas->GetParentCanvasSize().Y;
			for (int32 Y=InY; Y < MaxY; Y+=Font->GetMaxCharHeight(), ++Line)
			{
				// Draw row.
				int32 X=InX;
				for (const FCol& i : Cols)
				{
					const FLinearColor C = i.Colors.IsValidIndex(Line) ? i.Colors[Line] : Color;
					if (i.Items.IsValidIndex(Line))
					{
						RightJustify(Canvas,X,Y,i.Width,*i.Items[Line], C);
					}
					X += i.Width;
				}
			}
		}
	};
}