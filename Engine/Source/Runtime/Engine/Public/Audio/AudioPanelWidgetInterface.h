// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AudioPanelWidgetInterface.generated.h"


UINTERFACE(MinimalAPI)
class UAudioPanelWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

class IAudioPanelWidgetInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API FText GetEditorName();

	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API FName GetIconBrushName();
};
