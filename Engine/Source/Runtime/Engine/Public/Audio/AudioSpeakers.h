// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum EAudioSpeakers  : int
{							//	4.0	5.1	6.1	7.1
	SPEAKER_FrontLeft,		//	*	*	*	*
	SPEAKER_FrontRight,		//	*	*	*	*
	SPEAKER_FrontCenter,	//		*	*	*
	SPEAKER_LowFrequency,	//		*	*	*
	SPEAKER_LeftSurround,	//	*	*	*	*
	SPEAKER_RightSurround,	//	*	*	*	*
	SPEAKER_LeftBack,		//			*	*		If there is no BackRight channel, this is the BackCenter channel
	SPEAKER_RightBack,		//				*
	SPEAKER_Count
};