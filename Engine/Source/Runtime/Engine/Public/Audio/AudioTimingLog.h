// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"

// Log AudioTiming. For verbose detailed timings diagnostics and debugging.
#if UE_BUILD_SHIPPING
// Dial back the compiled verbosity in shipping
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioTiming, Error, Error);
#else // UE_BUILD_SHIPPING
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioTiming, Log, All);
#endif // UE_BUILD_SHIPPING

