// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDevice.h"
#include "AudioMixerTrace.h"
#include "CoreMinimal.h"

#if UE_AUDIO_PROFILERTRACE_ENABLED

struct FActiveSound;

namespace Audio::Trace
{
	namespace Util
	{
		ENGINE_API FString GetSoundBaseAssetName(const TObjectPtr<UClass> SoundClass);
		ENGINE_API FString GetAudioComponentName(const FActiveSound& InActiveSound);
		ENGINE_API FString GetOwnerActorLabel(const FActiveSound& InActiveSound);
		ENGINE_API FName GetOwnerActorIconName(const FActiveSound& InActiveSound);
		ENGINE_API TObjectPtr<UObject> GetSoundObjectPointer(const FActiveSound& InActiveSound);
	} // namespace Util

	namespace EventLog
	{
		// Add an entry to the Audio Insights Event Log that related to an Active Sound
		// Event may be customized to post custom event types to the Event Log
		ENGINE_API void SendActiveSoundEvent(const FActiveSound& InActiveSound, const FString& Event);

		// Add an entry to the Audio Insights Event Log
		ENGINE_API void SendEvent(const Audio::FDeviceId AudioDeviceID
								, const FString& Event
								, const uint32 AudioObjectID = INDEX_NONE
								, const FString& AssetPath = FString()
								, const FString& ActorLabel = FString()
								, const FString& ActorIconName = FString()
								, const FString& SoundClassName = FString()
								, const FString& AudioComponentName = FString());

	} // namespace EventLog

} // namespace Audio::Trace

#endif // #if UE_AUDIO_PROFILERTRACE_ENABLED