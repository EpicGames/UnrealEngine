// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AssetRegistry/AssetData.h"
#include "Audio/SoundEffectPresetWidgetInterface.h"
#include "Audio/SoundSubmixWidgetInterface.h"
#include "Blueprint/UserWidget.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundSubmix.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AudioWidgetSubsystem.generated.h"


// Forward Declarations
class UWorld;

UCLASS(MinimalAPI)
class UAudioWidgetSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

private:
	static const TArray<FAssetData> GetBlueprintAssetData(UClass* InInterfaceClass);
	static bool ImplementsInterface(const FAssetData& InAssetData, UClass* InInterfaceClass);

public:
	// Returns user widgets that implement a widget interface of the given subclass and optionally, filters and returns only those which pass the provided filter function.
	ENGINE_API TArray<UUserWidget*> CreateUserWidgets(
		UWorld& InWorld,
		TSubclassOf<UInterface> InWidgetInterface,
		TFunction<bool(UUserWidget*)> FilterFunction = TFunction<bool(UUserWidget*)>(),
		TFunction<bool(UBlueprint*)> BlueprintFilterFunction = TFunction<bool(UBlueprint*)>()) const;
};
