// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio.h"
#include "Audio/ISoundHandleSystem.h"

/**
 * An interface to be used for any classes that we want to respond to SoundHandle updates
 * TODO: Add a child of this that uses World?
 */
class ISoundHandleOwner
{
public:
	virtual ~ISoundHandleOwner() = default;
	
	Audio::EResult CreateSoundHandle(USoundBase* Sound, Audio::FSoundHandleID& OutSoundHandleID)
	{
		if(Sound)
		{
			for(ISoundHandleSystem* Interface : ISoundHandleSystem::GetRegisteredInterfaces())
			{
				OutSoundHandleID = Interface->CreateSoundHandle(Sound, this);
				return Audio::EResult::Success;
			}
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Couldn't create Sound Handle because we didn't pass through a valid sound!");
		}
		
		UE_LOGF(LogAudio, Warning, "Couldn't create Sound Handle because we couldn't find any ISoundHandleSystems!");
		return Audio::EResult::Failure;
	}
	
	/**
	 * Called when the sound handle is being removed from a ISoundHandle implementer,
	 * i.e. when the active sound it handles is notified for delete
	 */
	virtual void OnSoundHandleRemoved() {}
};
