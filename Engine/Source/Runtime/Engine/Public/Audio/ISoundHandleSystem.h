// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"

class ISoundHandleOwner;
class USoundBase;

namespace Audio
{
	using FSoundHandleID = uint32;

	enum class EResult : uint8
	{
		Success = 0,
		Failure = 1,
		NotImplemented = 2
	};
}

/**
 * This interface should be used with systems aiming to create Sound Handles: gameplay thread representations of sounds
 */
class ISoundHandleSystem : public IModularFeature
{
public:
	/**	
	 * Get the name of all Sound Handle implementations in the Modular Features registry.
	 * @return "SoundHandles"
	 */
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("SoundHandles");
		return FeatureName;
	}

	virtual ~ISoundHandleSystem() = default;

	/**
	 * @brief This should create a sound and we should map it to an identifier.
	 * Then return the identifier to the owner to keep track of
	 * 
	 * @param Sound
	 * @param Owner 
	 * @return 
	 */
	virtual Audio::FSoundHandleID CreateSoundHandle(USoundBase* Sound, ISoundHandleOwner* Owner) { return INDEX_NONE; }

	/**
	 * @brief Set the transform on the sound that is represented by the sound handle with the following ID
	 * 
	 * @param ID 
	 * @param Transform 
	 */
	virtual void SetTransform(Audio::FSoundHandleID ID, const FTransform& Transform) {}

	/**
	 * @brief Play the sound that is represented by the sound handle with the following ID
	 *
	 * @param ID 
	 */
	virtual Audio::EResult Play(Audio::FSoundHandleID ID) { return Audio::EResult::NotImplemented; }

	/**
	 * @brief Stop the sound that is represented by the sound handle with the following ID
	 *
	 * @param ID 
	 */
	virtual void Stop(Audio::FSoundHandleID ID) {}

	static TArray<ISoundHandleSystem*> GetRegisteredInterfaces()
	{
		// Get all IAudioHandleInterface implementations
		IModularFeatures::Get().LockModularFeatureList();
		TArray<ISoundHandleSystem*> RegisteredInterfaces = IModularFeatures::Get().GetModularFeatureImplementations<ISoundHandleSystem>(GetModularFeatureName());
		IModularFeatures::Get().UnlockModularFeatureList();
		return RegisteredInterfaces;
	}
};

