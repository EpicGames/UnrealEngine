// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "AudioPanelWidgetInterface.h"
#include "Sound/SoundEffectPreset.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"

#include "SoundEffectPresetWidgetInterface.generated.h"


UINTERFACE(Blueprintable, MinimalAPI)
class USoundEffectPresetWidgetInterface : public UAudioPanelWidgetInterface
{
	GENERATED_BODY()
};

class ISoundEffectPresetWidgetInterface : public IAudioPanelWidgetInterface
{
	GENERATED_BODY()

public:
	// Returns the class of Preset the widget supports
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API TSubclassOf<USoundEffectPreset> GetClass();

	// Called when the preset widget is constructed
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On SoundEffectPreset Widget Constructed"))
	ENGINE_API void OnConstructed(USoundEffectPreset* Preset);

	// Called when the preset object is changed
	UFUNCTION(BlueprintImplementableEvent)
	ENGINE_API void OnPropertyChanged(USoundEffectPreset* Preset, FName PropertyName);
};
