// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameterControllerInterface.h"
#include "UObject/Object.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "SoundParameterControllerInterface.generated.h"


// Forward Declarations
class FAudioDevice;
class USoundBase;
struct FActiveSound;

UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class USoundParameterControllerInterface : public UAudioParameterControllerInterface
{
	GENERATED_UINTERFACE_BODY()
};

// UObject interface for all object types that are controlling parameter values sent
// to sound instances (i.e. sources)
class ISoundParameterControllerInterface : public IAudioParameterControllerInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	// IAudioParameterControllerInterface
	ENGINE_API void ResetParameters() override;

	ENGINE_API void SetTriggerParameter(FName InName) override;
	ENGINE_API void SetBoolParameter(FName InName, bool InBool) override;
	ENGINE_API void SetBoolArrayParameter(FName InName, const TArray<bool>& InValue) override;
	ENGINE_API void SetIntParameter(FName InName, int32 InInt) override;
	ENGINE_API void SetIntArrayParameter(FName InName, const TArray<int32>& InValue) override;
	ENGINE_API void SetFloatParameter(FName InName, float InFloat) override;
	ENGINE_API void SetFloatArrayParameter(FName InName, const TArray<float>& InValue) override;
	ENGINE_API void SetStringParameter(FName InName, const FString& InValue) override;
	ENGINE_API void SetStringArrayParameter(FName InName, const TArray<FString>& InValue) override;
	ENGINE_API void SetObjectParameter(FName InName, UObject* InValue) override;
	ENGINE_API void SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue) override;

	ENGINE_API void SetParameter(FAudioParameter&& InValue) override;
	ENGINE_API void SetParameters(TArray<FAudioParameter>&& InValues) override;
	ENGINE_API void SetParameters_Blueprint(const TArray<FAudioParameter>& InValues) override;

	/** Returns the active audio device to use for this component based on whether or not the component is playing in a world. */
	virtual FAudioDevice* GetAudioDevice() const = 0;

	/** Returns the id of the sound owner's instance associated with the parameter interface. */
	virtual uint64 GetInstanceOwnerID() const = 0;

	/** Returns the Game Thread copy of parameters to modify in place. */
	virtual TArray<FAudioParameter>& GetInstanceParameters() = 0;

	/** Returns the USoundBase used to initialize instance parameters to update. */
	virtual USoundBase* GetSound() = 0;

	virtual bool IsPlaying() const = 0;

	virtual bool GetDisableParameterUpdatesWhilePlaying() const = 0;
};

UCLASS(MinimalAPI)
class UAudioParameterConversionStatics : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter BooleanToAudioParameter(FName Name, bool Bool);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter FloatToAudioParameter(FName Name, float Float);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter IntegerToAudioParameter(FName Name, int32 Integer);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter StringToAudioParameter(FName Name, FString String);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter ObjectToAudioParameter(FName Name, UObject* Object);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter BooleanArrayToAudioParameter(FName Name, TArray<bool> Bools);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter FloatArrayToAudioParameter(FName Name, TArray<float> Floats);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter IntegerArrayToAudioParameter(FName Name, TArray<int32> Integers);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter StringArrayToAudioParameter(FName Name, TArray<FString> Strings);

	UFUNCTION(BlueprintPure, Category = "Audio|Parameter", meta = (Keywords = "make construct convert create"))
	static UPARAM(DisplayName = "Parameter") FAudioParameter ObjectArrayToAudioParameter(FName Name, TArray<UObject*> Objects);
};
