// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SoundSubmixWidgetInterface.generated.h"


UINTERFACE(Blueprintable, MinimalAPI)
class USoundSubmixWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

class ISoundSubmixWidgetInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "On SoundSubmix Widget Constructed"))
	ENGINE_API void OnConstructed(USoundSubmixBase* SoundSubmix);
};
