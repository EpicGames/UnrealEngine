// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EngineAnalytics.h"
#include "HAL/Platform.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_AUDIO_ENABLE_ANALYTICS 1


namespace Audio
{
	namespace Analytics
	{
		inline void RecordEvent_Usage(FString&& InEventName, const TArray<FAnalyticsEventAttribute>& InEventAttributes = TArray<FAnalyticsEventAttribute>())
		{
#ifdef UE_AUDIO_ENABLE_ANALYTICS
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Audio.Usage.") + MoveTemp(InEventName), InEventAttributes);
			}
#endif // UE_AUDIO_ENABLE_ANALYTICS
		}
	}
}
