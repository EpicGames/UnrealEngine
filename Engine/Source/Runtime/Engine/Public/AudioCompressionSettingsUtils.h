// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCompressionSettings.h"

struct FCachedAudioStreamingManagerParams;

class FPlatformCompressionUtilities
{
public:
	// Returns the Duration Threshold for the current platform if it is overridden, -1.0f otherwise.
	static ENGINE_API float GetCompressionDurationForCurrentPlatform();

	// Returns the sample rate for a given platform,
	static ENGINE_API float GetTargetSampleRateForPlatform(ESoundwaveSampleRateSettings InSampleRateLevel = ESoundwaveSampleRateSettings::High);

	static ENGINE_API int32 GetMaxPreloadedBranchesForCurrentPlatform();

	static ENGINE_API int32 GetQualityIndexOverrideForCurrentPlatform();

	static ENGINE_API void RecacheCookOverrides();

	// null platformname means to use current platform
	static ENGINE_API const FPlatformAudioCookOverrides* GetCookOverrides(const TCHAR* PlatformName=nullptr, bool bForceRecache = false);

	UE_DEPRECATED(5.5, "this always returns true, no longer needed.")
	static bool IsCurrentPlatformUsingStreamCaching() { return true; };

	// null platformname means to use current platform
	static ENGINE_API const FAudioStreamCachingSettings& GetStreamCachingSettingsForCurrentPlatform();

	/** This is used at runtime to initialize FCachedAudioStreamingManager. */
	static ENGINE_API FCachedAudioStreamingManagerParams BuildCachedStreamingManagerParams();

	/** This is used at runtime in BuildCachedStreamingManagerParams, as well as cooktime in FStreamedAudioCacheDerivedDataWorker::BuildStreamedAudio to split compressed audio.  */
	static ENGINE_API uint32 GetMaxChunkSizeForCookOverrides(const FPlatformAudioCookOverrides* InCompressionOverrides);

	template<typename HashType>
	static void AppendHash(FString& OutString, const TCHAR* InName, const HashType& InValueToHash)
	{
		OutString += FString::Printf(TEXT("%s_%s_"), InName, ToCStr(LexToString(InValueToHash)));
	}

private:
	static ENGINE_API const FPlatformRuntimeAudioCompressionOverrides* GetRuntimeCompressionOverridesForCurrentPlatform();
	
};
