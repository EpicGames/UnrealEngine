// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioDecompress.h: Unreal audio vorbis decompression interface object.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Sound/SoundWave.h"
#include "Misc/ScopeLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "ContentStreaming.h"

// 100ms of 48KHz data
// 108ms of 44.1KHz data
// 218ms of 22KHz data
constexpr int32 MONO_PCM_BUFFER_SAMPLES = 4800;
constexpr uint32 MONO_PCM_SAMPLE_SIZE = sizeof(int16);
constexpr uint32 MONO_PCM_BUFFER_SIZE = MONO_PCM_BUFFER_SAMPLES * MONO_PCM_SAMPLE_SIZE;

struct FSoundQualityInfo;
class FStreamedAudioChunkSeekTable;
class IAudioDecoderOverride;

/**
 * Interface class to decompress various types of audio data
 */
class ICompressedAudioInfo
{
public:
	ENGINE_API ICompressedAudioInfo();

	/**
	* Virtual destructor.
	*/
	ENGINE_API virtual ~ICompressedAudioInfo();

	/**
	* Reads the header information of a compressed format
	*
	* @param	InSrcBufferData		Source compressed data
	* @param	InSrcBufferDataSize	Size of compressed data
	* @param	QualityInfo			Quality Info (to be filled out). This can be null in the case of most implementations of FSoundBuffer::ReadCompressedInfo
	*/
	virtual bool ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo) = 0;

	/**
	* Decompresses data to raw PCM data.
	*
	* @param	Destination	where to place the decompressed sound
	* @param	bLooping	whether to loop the sound by seeking to the start, or pad the buffer with zeroes
	* @param	BufferSize	number of bytes of PCM data to create
	*
	* @return	bool		true if the end of the data was reached (for both single shot and looping sounds)
	*/
	virtual bool ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize) = 0;

	/**
	 * Seeks to time (Some formats might not be seekable)
	 */
	virtual void SeekToTime(const float SeekTime) = 0;

	/**
	* Seeks to specific frame in the audio (Some formats might not be seekable)
	*/
	virtual void SeekToFrame(const uint32 Frame) = 0;

	/**
	* Decompress an entire data file to a TArray
	*/
	virtual void ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo) = 0;

	/**
	* Sets decode to half-rate
	*
	* @param	HalfRate	Whether Half rate is enabled
	*/
	virtual void EnableHalfRate(bool HalfRate) = 0;

	/**
	 * Gets the size of the source buffer originally passed to the info class (bytes)
	 */
	virtual uint32 GetSourceBufferSize() const = 0;

	/**
	 * Whether the decompressed audio will be arranged using Vorbis' channel ordering
	 * See http://www.xiph.org/vorbis/doc/Vorbis_I_spec.html#x1-800004.3.9 for details
	 */
	virtual bool UsesVorbisChannelOrdering() const = 0;

	/**
	* Gets the preferred size for a streaming buffer for this decompression scheme
	*/
	virtual int GetStreamBufferSize() const = 0;

	/**
	* Returns whether this instance can be cast to a IStreamedCompressedInfo. Surprisingly
	* SupportsStreaming doesn't work for this.
	*/
	virtual bool IsStreamedCompressedInfo() const { return false; }

	////////////////////////////////////////////////////////////////
	// Following functions are optional if streaming is supported //
	////////////////////////////////////////////////////////////////

	/**
	 * Whether this decompression class supports streaming decompression
	 */
	virtual bool SupportsStreaming() const {return false;}

	/**
	 * This can be called to explicitly release this decoder's reference to a chunk of compressed audio
	 * without destroying the decoder itself.
	 * @param bBlockUntilReleased when set to true will cause this call to block if the decoder is currently using the chunk.
	 * @returns true if the chunk was released, false otherwise.
	 */
	virtual bool ReleaseStreamChunk(bool bBlockUntilReleased)
	{
		// If we hit this check, ReleaseStreamChunk needs to be implemented for this codec.
		checkNoEntry();
		return false;
	}

	/**
	* Streams the header information of a compressed format
	*
	* @param	Wave			Wave that will be read from to retrieve necessary chunk
	* @param	QualityInfo		Quality Info (to be filled out)
	*/
	ENGINE_API bool StreamCompressedInfo(USoundWave* Wave, struct FSoundQualityInfo* QualityInfo);
	ENGINE_API bool StreamCompressedInfo(const TSharedRef<const FSoundWaveData>& Wave, struct FSoundQualityInfo* QualityInfo);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	ENGINE_API bool StreamCompressedInfo(const FSoundWaveProxyPtr& Wave, struct FSoundQualityInfo* QualityInfo);

	/**
	*  Returns true if a non-recoverable error has occurred.
	*/
	ENGINE_API virtual bool HasError() const;

protected:
	/** Internal override implemented by subclasses. */
	ENGINE_API virtual bool StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, struct FSoundQualityInfo* QualityInfo);
	
	UE_DEPRECATED(5.8, "Please use the const TSharedRef<const FSoundWaveData>& overload instead")
	virtual bool StreamCompressedInfoInternal(const TSharedPtr<const FSoundWaveProxy>& InWaveProxy, struct FSoundQualityInfo* QualityInfo) { return false; }
	
	

public:

	/**
	* Decompresses streamed data to raw PCM data.
	*
	* @param	Destination	where to place the decompressed sound
	* @param	bLooping	whether to loop the sound by seeking to the start, or pad the buffer with zeros
	* @param	BufferSize	number of bytes of PCM data to create
	*
	* @return	bool		true if the end of the data was reached (for both single shot and looping sounds)
	*/
	virtual bool StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed) { OutNumBytesStreamed = -1; return false; }

	/**
	 * Gets the chunk index that was last read from (for Streaming Manager requests)
	 */
	virtual int32 GetCurrentChunkIndex() const {return -1;}

	/**
	 * Gets the offset into the chunk that was last read to (for Streaming Manager priority)
	 */
	virtual int32 GetCurrentChunkOffset() const {return -1;}

	/** Return the streaming sound wave data used by this decoder. Returns nullptr if there is not a streaming sound wave. */
	virtual const TSharedPtr<const FSoundWaveData>& GetStreamingSoundWaveData() const { return StreamingSoundWaveData; }
	
	UE_DEPRECATED(5.8, "Use the GetStreamingSoundWaveData() instead, return value points to nullptr")
	virtual const FSoundWaveProxyPtr& GetStreamingSoundWave() const { static const FSoundWaveProxyPtr EmptyProxy; return EmptyProxy; }

	/** Return a conformed decode size that best matches all codecs. */
	static int32 ConformDecodeSize(int32 InMaxDesiredDecodeSizeInFrames);

protected:
	mutable bool bHasError = false;

#if !UE_BUILD_SHIPPING
	bool bWantsFilteredLogging = false;
#endif //UE_BUILD_SHIPPING
	bool WantsFilteredLogging() const 
	{
#if UE_BUILD_SHIPPING
		return false;
#else
		return bWantsFilteredLogging;
#endif //UE_BUILD_SHIPPING		
	}
	
	UE_DEPRECATED(5.8, "Use StreamingSoundWaveData instead")
	FSoundWaveProxyPtr StreamingSoundWave;
	
	TSharedPtr<const FSoundWaveData> StreamingSoundWaveData;
};

/** Struct used to store the results of a decode operation. **/
struct FDecodeResult
{
	// Number of bytes of compressed data consumed
	int32 NumCompressedBytesConsumed;
	// Number of bytes produced
	int32 NumPcmBytesProduced;
	// Number of frames produced.
	int32 NumAudioFramesProduced;

	FDecodeResult()
		: NumCompressedBytesConsumed(INDEX_NONE)
		, NumPcmBytesProduced(INDEX_NONE)
		, NumAudioFramesProduced(INDEX_NONE)
	{}
};

/** 
 * Default implementation of a streamed compressed audio format.
 * Can be subclassed to support streaming of a specific asset format. Handles all 
 * the platform independent aspects of file format streaming for you (dealing with UE streamed assets)
 */
class IStreamedCompressedInfo : public ICompressedAudioInfo
{
public:
	ENGINE_API IStreamedCompressedInfo();
	ENGINE_API virtual ~IStreamedCompressedInfo();

	//~ Begin ICompressedInfo Interface
	ENGINE_API virtual bool ReadCompressedInfo(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) override;
	ENGINE_API virtual bool ReadCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize) override;
	ENGINE_API virtual void SeekToTime(const float SeekTime) override;
	ENGINE_API virtual void SeekToFrame(const uint32 SeekFrame) override;
	ENGINE_API virtual void ExpandFile(uint8* DstBuffer, struct FSoundQualityInfo* QualityInfo) override;
	virtual void EnableHalfRate(bool HalfRate) override {};
	virtual uint32 GetSourceBufferSize() const override { return SrcBufferDataSize; }
	virtual bool UsesVorbisChannelOrdering() const override { return false; }
	virtual int GetStreamBufferSize() const override { return  MONO_PCM_BUFFER_SIZE; }
	virtual bool SupportsStreaming() const override { return true; }
	ENGINE_API virtual bool StreamCompressedInfoInternal(const TSharedRef<const FSoundWaveData>& InWaveData, FSoundQualityInfo* QualityInfo) override;
	ENGINE_API virtual bool StreamCompressedData(uint8* Destination, bool bLooping, uint32 BufferSize, int32& OutNumBytesStreamed) override;
	virtual int32 GetCurrentChunkIndex() const override { return CurrentChunkIndex; }
	virtual int32 GetCurrentChunkOffset() const override { return SrcBufferOffset; }
	virtual bool IsStreamedCompressedInfo() const override { return true; }
	//~ End ICompressedInfo Interface

	/** Parse the header information from the input source buffer data. This is dependent on compression format. */
	virtual bool ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) = 0;

	/** Create the compression format dependent decoder object. */
	virtual bool CreateDecoder() = 0;

	/** Decode the input compressed frame data into output PCMData buffer. */
	virtual FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) = 0;

	/** Optional method to allow decoder to prepare to loop. */
	virtual void PrepareToLoop() {}

	/** Return the size of the current compression frame */
	virtual int32 GetFrameSize() = 0;

	/** The size of the decode PCM buffer size. */
	virtual uint32 GetMaxFrameSizeSamples() const = 0;

	int32 GetStreamSeekBlockIndex() const { return StreamSeekBlockIndex; }
	int32 GetStreamSeekBlockOffset() const { return StreamSeekBlockOffset; }
protected:

	/** Reads from the internal source audio buffer stream of the given data size. */
	ENGINE_API uint32	Read(void *Outbuffer, uint32 DataSize);

	/**
	* Decompresses a frame of data to PCM buffer
	*
	* @param FrameSize Size of the frame in bytes
	* @return The number of audio frames that were decompressed (< 0 indicates error)
	*/
	int32 DecompressToPCMBuffer(uint16 FrameSize);

	/**
	* Adds to the count of samples that have currently been decoded
	*
	* @param NewSamples	How many samples have been decoded
	* @return How many samples were actually part of the true sample count
	*/
	uint32 IncrementCurrentSampleCount(uint32 NewSamples);

	/**
	* Writes data from decoded PCM buffer, taking into account whether some PCM has been written before
	*
	* @param Destination	Where to place the decoded sound
	* @param BufferSize	Size of the destination buffer in bytes
	* @return				How many bytes were written
	*/
	uint32	WriteFromDecodedPCM(uint8* Destination, uint32 BufferSize);

	/**
	* Zeroes the contents of a buffer
	*
	* @param Destination	Buffer to zero
	* @param BufferSize	Size of the destination buffer in bytes
	* @return				How many bytes were zeroed
	*/
	uint32	ZeroBuffer(uint8* Destination, uint32 BufferSize);

	/**
	 * Helper function for getting a chunk of compressed audio.
	 * @param InSoundWave Pointer to the soundwave to get compressed audio from.
	 * @param ChunkIndex the index of the chunk to get from InSoundWave.
	 * @param[out] OutChunkSize the size of the chunk.
	 * @return a pointer to the chunk if it's loaded, nullptr otherwise.
	 */
	ENGINE_API const uint8* GetLoadedChunk(const TSharedRef<const FSoundWaveData>& InSoundWaveData, uint32 ChunkIndex, uint32& OutChunkSize);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	ENGINE_API const uint8* GetLoadedChunk(const FSoundWaveProxyPtr& InSoundWave, uint32 ChunkIndex, uint32& OutChunkSize);

	void RetainNextChunkInSequence(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const uint32 InChunkIndex);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	void RetainNextChunkInSequence(const FSoundWaveProxyPtr& InSoundWave,const uint32 InChunkIndex);

	/**
	 * Gets the current chunks seektable instance (or creates one)
	 * @return the currently loaded chunk seektable
	 */
	const FStreamedAudioChunkSeekTable& GetCurrentSeekTable() const;
	FStreamedAudioChunkSeekTable& GetCurrentSeekTable();

	/** bool set before ParseHeader. Whether we are streaming a file or not. */
	bool bIsStreaming;
	/** Ptr to the current streamed chunk. */
	const uint8* SrcBufferData;
	/** Size of the current streamed chunk. */
	uint32 SrcBufferDataSize;
	/** What byte we're currently reading in the streamed chunk. */
	uint32 SrcBufferOffset;
	/** Where the actual audio data starts in the current streamed chunk. Accounts for header offset. */
	uint32 AudioDataOffset;
	/** The chunk index where the actual audio data starts. */
	uint32 AudioDataChunkIndex;
	/** The total sample count of the source file. */
	uint32 TrueSampleCount;
	/** How many samples we've currently read in the source file. */
	uint32 CurrentSampleCount;
	/** Number of channels (left/right) in the source file. */
	uint8 NumChannels;
	/** The maximum number of samples per decode frame. */
	uint32 MaxFrameSizeSamples;
	/** The number of bytes per interleaved sample (NumChannels * sizeof(int16)). */
	uint32 SampleStride;
	/** The decoded PCM byte array from the last decoded frame. */
	TArray<uint8> LastDecodedPCM;
	/** The amount of PCM data in bytes was decoded last. */
	uint32 LastPCMByteSize;
	/** The current offset in the last decoded PCM buffer. */
	uint32 LastPCMOffset;
	/** If we're currently reading the final buffer. */
	bool bStoringEndOfFile;
	/** The current chunk index in the streamed chunks. */
	int32 CurrentChunkIndex;
	/** Number of bytes of padding used, overridden in some implementations. Defaults to 0. */
	uint32 SrcBufferPadding;
	/** Chunk Handle to ensure that this chunk of streamed audio is not deleted while we are using it. */
	FAudioChunkHandle CurCompressedChunkHandle;
	/** Handle to the next chunk in sequence. Added once we're playing, to prevent it being dropped before it's played.  */
	FAudioChunkHandle NextPlayingChunkHandle;
	/** If there's a chunked seek-table present, this contains the current chunks portion. */
	TPimplPtr<FStreamedAudioChunkSeekTable> CurrentChunkSeekTable;
	/** 
		When a streaming seek request comes down, this is the block we are going to. INDEX_NONE means no seek pending.
		When using the legacy streaming system this is read on a thread other than the decompression thread
		to prime the correct chunk, so it needs an atomic. It's only ever _set_ on one thread, and the read
		has no timing restrictions, so no lock is necessary. Also the legacy streamer isn't used anymore anyway.

		This and StreamSeekBlockOffset are expected to be set by the codec's SeekToTime() & SeekToFrame() functions.
	*/
	std::atomic<int32> StreamSeekBlockIndex;
	/** When a streaming seek request comes down, this is the offset in to the block we want to start decoding from. */
	int32 StreamSeekBlockOffset;

	/** If using the Chunked seek-tables, we request the seek in samples so we can resolve after the chunk table loads. */
	uint32 StreamSeekToAudioFrames = INDEX_NONE;	

private:
	/** Attempts a seek to the frame by consuming decoded PCM data. Returns true
	 * if future seeks should no longer be attempted. */
	bool SeekToDecodedPCMFrame(uint32 InTargetAudioFrame);
};

struct IAudioInfoFactory 
{	
	virtual ~IAudioInfoFactory() = default;
	virtual ICompressedAudioInfo* Create() = 0;

	/**
	 * Simple decode capabilities struct, filled out by the decoder factories.
	 * NOTE: By Default support is expected unless we opt out.
	 */
	struct FCapabilities
	{
		bool bSupportsSeeking = true;
		bool bSupportsSeekableStreaming = true;
	};
	/**
	* 
	 * Queries if the factory's decoder implementation supports seeking
	 * @param bIsStreaming If the media is streaming or not.
	 * @return true if seeking is supported, false otherwise.
	 */
	virtual bool IsSeekingSupported(const bool bIsStreaming) const = 0;
};
struct IAudioInfoFactoryRegistry
{
	static ENGINE_API IAudioInfoFactoryRegistry& Get();
	virtual ~IAudioInfoFactoryRegistry() = default;
	virtual void Register(IAudioInfoFactory*, FName) = 0;
	virtual void Unregister(IAudioInfoFactory*, FName) = 0;
	virtual IAudioInfoFactory* Find(FName InName) const = 0;
	virtual ICompressedAudioInfo* Create(FName InName, FName InAssetName = NAME_None) const = 0;

	UE_EXPERIMENTAL(5.8, "Decoder overriding is experimental and the API may be changed or removed")
	virtual void RegisterOverride(IAudioDecoderOverride* Override) = 0;
	UE_EXPERIMENTAL(5.8, "Decoder overriding is experimental and the API may be changed or removed")
	virtual void UnregisterOverride(IAudioDecoderOverride* Override) = 0;
};

class FSimpleAudioInfoFactory : public IAudioInfoFactory
{
public:
	/**
	 * Constructs a simple AudioInfo factory.
	 * @param InLambda Lambda to Create a ICompressionAudioInfo object
	 * @param InFormatName The factory name of the Format. "ADPCM" etc.
	 * @param InCaps The Capabilities of the decoder implementation.
	 */
	FSimpleAudioInfoFactory(const TFunction<ICompressedAudioInfo*()>& InLambda, const FName& InFormatName, const FCapabilities InCaps = {})
	:  CreateLambda(InLambda), FormatName(InFormatName), Capabilities(InCaps)
	{
		IAudioInfoFactoryRegistry::Get().Register(this, FormatName);
	}
	virtual ~FSimpleAudioInfoFactory() override 
	{
		IAudioInfoFactoryRegistry::Get().Unregister(this, FormatName);
	}
	virtual ICompressedAudioInfo* Create() override { return CreateLambda(); }

	/**
	 * Quries if seeking is supported by the decoder.
	 * @param bIsStreaming If the media is to be streamed or not.
	 * @return True for supported, false otherwise.
	 */
	virtual bool IsSeekingSupported(const bool bIsStreaming) const override
	{
		return bIsStreaming ?
			Capabilities.bSupportsSeekableStreaming :
			Capabilities.bSupportsSeeking;
	}
	
private:
	TFunction<ICompressedAudioInfo* ()> CreateLambda;
	FName FormatName;
	FCapabilities Capabilities;
};

/**
 * Asynchronous audio decompression
 */
class FAsyncAudioDecompressWorker : public FNonAbandonableTask
{
protected:
	USoundWave* Wave;
	ICompressedAudioInfo* AudioInfo;
	int32 NumPrecacheFrames;

public:
	/**
	 * Async decompression of audio data
	 *
	 * @param	InWave		Wave data to decompress
	 */
	ENGINE_API FAsyncAudioDecompressWorker(USoundWave* InWave, int32 InNumPrecacheFrames, FAudioDevice* InAudioDevice = nullptr);

	/**
	 * Performs the async audio decompression
	 */
	ENGINE_API void DoWork();

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncAudioDecompressWorker, STATGROUP_ThreadPoolAsyncTasks);
	}
};

typedef FAsyncTask<FAsyncAudioDecompressWorker> FAsyncAudioDecompress;

enum class ERealtimeAudioTaskType : uint8
{
	/** Parses the wave compressed asset header file info */
	CompressedInfo,

	/** Decompresses a chunk */
	Decompress,

	/** Processes a procedural buffer to generate more audio */
	Procedural
};

template<class T>
class FAsyncRealtimeAudioTaskWorker : public FNonAbandonableTask
{
protected:
	T* AudioBuffer;
	USoundWave* WaveData;
	uint8* AudioData;
	int32 NumPrecacheFrames;
	int32 MaxSamples;
	int32 BytesWritten;
	ERealtimeAudioTaskType TaskType;
	uint32 bSkipFirstBuffer:1;
	uint32 bLoopingMode:1;
	uint32 bLooped:1;

public:
	FAsyncRealtimeAudioTaskWorker(T* InAudioBuffer, USoundWave* InWaveData)
		: AudioBuffer(InAudioBuffer)
		, WaveData(InWaveData)
		, AudioData(nullptr)
		, NumPrecacheFrames(0)
		, MaxSamples(0)
		, BytesWritten(0)
		, TaskType(ERealtimeAudioTaskType::CompressedInfo)
		, bSkipFirstBuffer(false)
		, bLoopingMode(false)
		, bLooped(false)
	{
		check(AudioBuffer);
		check(WaveData);
	}

	FAsyncRealtimeAudioTaskWorker(T* InAudioBuffer, uint8* InAudioData, int32 InNumPrecacheFrames, bool bInLoopingMode, bool bInSkipFirstBuffer)
		: AudioBuffer(InAudioBuffer)
		, AudioData(InAudioData)
		, NumPrecacheFrames(InNumPrecacheFrames)
		, TaskType(ERealtimeAudioTaskType::Decompress)
		, bSkipFirstBuffer(bInSkipFirstBuffer)
		, bLoopingMode(bInLoopingMode)
		, bLooped(false)
	{
		check(AudioBuffer);
		check(AudioData);
	}

	FAsyncRealtimeAudioTaskWorker(USoundWave* InWaveData, uint8* InAudioData, int32 InMaxSamples)
		: WaveData(InWaveData)
		, AudioData(InAudioData)
		, NumPrecacheFrames(0)
		, MaxSamples(InMaxSamples)
		, BytesWritten(0)
		, TaskType(ERealtimeAudioTaskType::Procedural)
	{
		check(WaveData);
		check(AudioData);
	}

	void DoWork()
	{
		LLM_SCOPE(ELLMTag::AudioMisc);

		switch(TaskType)
		{
		case ERealtimeAudioTaskType::CompressedInfo:
			AudioBuffer->ReadCompressedInfo(WaveData);
			break;

		case ERealtimeAudioTaskType::Decompress:
			if (bSkipFirstBuffer)
			{
#if PLATFORM_ANDROID
				// Only skip one buffer on Android
				AudioBuffer->ReadCompressedData((uint8*)AudioData, NumPrecacheFrames, bLoopingMode );
#else
				// If we're using cached data we need to skip the first two reads from the data
				AudioBuffer->ReadCompressedData((uint8*)AudioData, NumPrecacheFrames, bLoopingMode);
				AudioBuffer->ReadCompressedData((uint8*)AudioData, NumPrecacheFrames, bLoopingMode);
#endif
			}
			bLooped = AudioBuffer->ReadCompressedData((uint8*)AudioData, MONO_PCM_BUFFER_SAMPLES, bLoopingMode);
			break;

		case ERealtimeAudioTaskType::Procedural:
			BytesWritten = WaveData->GeneratePCMData((uint8*)AudioData, MaxSamples);
			break;

		default:
			check(false);
		}
	}

	inline TStatId GetStatId() const
	{
		if (TaskType == ERealtimeAudioTaskType::Procedural)
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncRealtimeAudioProceduralWorker, STATGROUP_ThreadPoolAsyncTasks);
		}
		else
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncRealtimeAudioDecompressWorker, STATGROUP_ThreadPoolAsyncTasks);
		}
	}

	ERealtimeAudioTaskType GetTaskType() const
	{
		return TaskType;
	}

	bool GetBufferLooped() const
	{ 
		check(TaskType == ERealtimeAudioTaskType::Decompress);
		return bLooped;
	}

	int32 GetBytesWritten() const
	{ 
		check(TaskType == ERealtimeAudioTaskType::Procedural);
		return BytesWritten;
	}
};

ENGINE_API bool ShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask();

template<class T>
class FAsyncRealtimeAudioTaskProxy
{
public:
	FAsyncRealtimeAudioTaskProxy(T* InAudioBuffer, USoundWave* InWaveData)
	{
		Task = new FAsyncTask<FAsyncRealtimeAudioTaskWorker<T>>(InAudioBuffer, InWaveData);
	}

	FAsyncRealtimeAudioTaskProxy(T* InAudioBuffer, uint8* InAudioData, int32 InNumFramesToDecode, bool bInLoopingMode, bool bInSkipFirstBuffer)
	{
		Task = new FAsyncTask<FAsyncRealtimeAudioTaskWorker<T>>(InAudioBuffer, InAudioData, InNumFramesToDecode, bInLoopingMode, bInSkipFirstBuffer);
	}
	FAsyncRealtimeAudioTaskProxy(USoundWave* InWaveData, uint8* InAudioData, int32 InMaxSamples)
	{
		Task = new FAsyncTask<FAsyncRealtimeAudioTaskWorker<T>>(InWaveData, InAudioData, InMaxSamples);
	}

	~FAsyncRealtimeAudioTaskProxy()
	{
		check(IsDone());
		delete Task;
	}

	bool IsDone()
	{
		FScopeLock Lock(&CritSect);
		return Task->IsDone();
	}

	void EnsureCompletion(bool bDoWorkOnThisThreadIfNotStarted = true)
	{
		FScopeLock Lock(&CritSect);

		if (!bDoWorkOnThisThreadIfNotStarted)
		{
			Task->Cancel();
		}
		Task->EnsureCompletion(bDoWorkOnThisThreadIfNotStarted);
	}

	void StartBackgroundTask()
	{
		FScopeLock Lock(&CritSect);
		const bool bUseBackground = ShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask() && (Task->GetTask().GetTaskType() != ERealtimeAudioTaskType::Procedural);
		Task->StartBackgroundTask(bUseBackground ? GBackgroundPriorityThreadPool : GThreadPool);
	}

	FAsyncRealtimeAudioTaskWorker<T>& GetTask()
	{
		FScopeLock Lock(&CritSect);
		return Task->GetTask();
	}

private:
	FCriticalSection CritSect;
	FAsyncTask<FAsyncRealtimeAudioTaskWorker<T>>* Task;
};
