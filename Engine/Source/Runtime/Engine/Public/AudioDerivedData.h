// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DerivedDataPluginInterface.h"

#include "AudioDerivedData.generated.h"

class IAudioFormat;
class USoundWave;
struct FPlatformAudioCookOverrides;
struct FAudioCookInputs;


/**
 * Struct that's serialized into the DDC record for "Audio" DDC task.
 * Filled in by the encoders, additional relevant state is passed back here.
 * Like decisions make during encoding, like resampling, transformations etc.
 */
USTRUCT()
struct FAudioCookOutputs
{
	GENERATED_BODY()

	/**
	 * Get the expected ID of this struct. uint32 so we can peek at memory.
	 * @return uint32 Id.
	 */
	static uint32 GetId()
	{
		// Magic value is "ACO"
		return ((uint32)'O' << 16 | (uint32)'C' << 8 | (uint32)'A');
	}

	/**
	 * Gets the current Version expected. Any other versions would be an error.
	 * @return int32 Version.
	 */
	static int32 GetVersion();

	/**
	 * Serializer (read/write)
	 * @param Ar Reader/Writer Archive.
	 * @return True if successfully read/written, false otherwise.
	 */
	bool Serialize(FArchive& Ar);

	UPROPERTY()
	uint32 Id = GetId();
	
	UPROPERTY()
	int32 Version = GetVersion();

	/**
	 * Final channel count that was encoded by the IAudioFormat
	 */
	UPROPERTY()
	int32 NumChannels = 0;

	/**
	 * Final Sample rate that was encoded by IAudioFormat 
	 */
	UPROPERTY()
	int32 SampleRate = 0;

	/**
	 * Number of frames in the encoded data.
	 */
	UPROPERTY()
	uint32 NumFrames = 0;

	/**
	 * The Binary output of the IAudioFormat. 
	 */
	UPROPERTY()
	TArray<uint8> EncodedData;
};

class FDerivedAudioDataCompressor : public FDerivedDataPluginInterface
{
private:
	TUniquePtr<FAudioCookInputs> CookInputs;

public:
	ENGINE_API FDerivedAudioDataCompressor(USoundWave* InSoundNode, FName InBaseFormat, FName InHashedFormat, const FPlatformAudioCookOverrides* InCompressionOverrides, const ITargetPlatform* InTargetPlatform=nullptr);

	virtual const TCHAR* GetPluginName() const override
	{
		return TEXT("Audio");
	}

	virtual const TCHAR* GetVersionString() const override;

	virtual FString GetPluginSpecificCacheKeySuffix() const override;
	
	virtual bool IsBuildThreadsafe() const override;

	virtual bool Build(TArray<uint8>& OutData) override;

	virtual FString GetDebugContextString() const override;
};
