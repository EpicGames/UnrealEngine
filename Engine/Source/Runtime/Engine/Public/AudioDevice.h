// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceHandle.h"
#include "AudioDynamicParameter.h"
#include "AudioMixerTrace.h"
#include "AudioVirtualLoop.h"
#include "Components/AudioComponent.h"
#include "HAL/LowLevelMemStats.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Sound/AudioVolume.h"
#include "Sound/SoundClass.h"
#include "Sound/SoundMix.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "UObject/StrongObjectPtr.h"


/**
 * Forward declares
 */

class FArchive;
class FAudioDevice;
class FAudioEffectsManager;
class FCanvas;
class FOutputDevice;
class FReferenceCollector;
class FSoundBuffer;
class FViewport;
class FViewportClient;
class IAudioSpatialization;
class ICompressedAudioInfo;
class ISubmixBufferListener;
class UReverbEffect;
class USoundAttenuation;
class USoundBase;
class USoundClass;
class USoundConcurrency;
class USoundEffectSourcePreset;
class USoundEffectSubmixPreset;
class USoundMix;
class USoundModulatorBase;
class USoundSubmixBase;
class USoundSourceBus;
class USoundWave;
class UWorld;

struct FActiveSound;
struct FAttenuationFocusData;
struct FAudioComponentParam;
struct FAudioQualitySettings;
struct FAudioVirtualLoop;
struct FSourceEffectChainEntry;
struct FSoundClassDynamicProperties;
struct FSoundClassProperties;
struct FSoundGroup;
struct FSoundParseParameters;
struct FSoundSpectrumAnalyzerDelegateSettings;
struct FSoundSpectrumAnalyzerSettings;
struct FWaveInstance;

namespace Audio
{
	class FPatchInput;
	struct FPatchOutput;
	typedef TSharedPtr<FPatchOutput, ESPMode::ThreadSafe> FPatchOutputStrongPtr;
}

namespace Audio
{
	class FAudioDebugger;

	/** Returns a decoder for the sound assets's given compression type. Will return nullptr if the compression type is platform-dependent. */
	ENGINE_API ICompressedAudioInfo* CreateSoundAssetDecoder(const FName& InRuntimeFormat);

	/** Creates an ID for use by Parameter Transmitters that can differentiate between multiple voices playing on the same Audio Component. */
	ENGINE_API uint64 GetTransmitterID(uint64 ComponentID, UPTRINT WaveInstanceHash, uint32 PlayOrder);
}

/**
 * Debug state of the audio system
 */
enum EDebugState
{
	// No debug sounds
	DEBUGSTATE_None,
	// No reverb sounds
	DEBUGSTATE_IsolateDryAudio,
	// Only reverb sounds
	DEBUGSTATE_IsolateReverb,
	// Force LPF on all sources
	DEBUGSTATE_TestLPF,
	// Force LPF on all sources
	DEBUGSTATE_TestHPF,
	// Bleed all sounds to the LFE speaker
	DEBUGSTATE_TestLFEBleed,
	// Disable any LPF filter effects
	DEBUGSTATE_DisableLPF,
	// Disable any LPF filter effects
	DEBUGSTATE_DisableHPF,
	// Disable any radio filter effects
	DEBUGSTATE_DisableRadio,
	DEBUGSTATE_MAX,
};

/**
 * Current state of a SoundMix
 */
namespace ESoundMixState
{
	enum Type
	{
		// Waiting to fade in
		Inactive,
		// Fading in
		FadingIn,
		// Fully active
		Active,
		// Fading out
		FadingOut,
		// Time elapsed, just about to be removed
		AwaitingRemoval,
	};

	static const TCHAR* GetString(ESoundMixState::Type InType)
	{
		switch (InType)
		{
			case ESoundMixState::Inactive: return TEXT("Inactive");
			case ESoundMixState::FadingIn: return TEXT("FadingIn");
			case ESoundMixState::Active: return TEXT("Active");
			case ESoundMixState::FadingOut: return TEXT("FadingOut");
			case ESoundMixState::AwaitingRemoval: return TEXT("AwaitingRemoval");
			default: return TEXT("unknown");
		}
	}
}

namespace ESortedActiveWaveGetType
{
	enum Type
	{
		FullUpdate,
		PausedUpdate,
		QueryOnly,
	};
}

/**
 * Defines the properties of the listener
 */
struct FListener
{
	FTransform Transform;
	FVector Velocity;

	/** An attenuation override to use for distance and attenuation calculations */
	FVector AttenuationOverride;

	/** Is our attenuation override active */
	uint32 bUseAttenuationOverride:1;

	struct FInteriorSettings InteriorSettings;

	/** The ID of the volume the listener resides in */
	uint32 AudioVolumeID;

	/** The ID of the world the listener resides in */
	uint32 WorldID;

	/** Index of this listener inside the AudioDevice's listener array */
	int32 ListenerIndex;

	/** The times of interior volumes fading in and out */
	double InteriorStartTime;
	double InteriorEndTime;
	double ExteriorEndTime;
	double InteriorLPFEndTime;
	double ExteriorLPFEndTime;
	float InteriorVolumeInterp;
	float InteriorLPFInterp;
	float ExteriorVolumeInterp;
	float ExteriorLPFInterp;
	FAudioDevice* AudioDevice;

	FVector GetUp() const		{ return Transform.GetUnitAxis(EAxis::Z); }
	FVector GetFront() const	{ return Transform.GetUnitAxis(EAxis::Y); }
	FVector GetRight() const	{ return Transform.GetUnitAxis(EAxis::X); }

	/**
	 * Gets the position of the listener
	 */
	FVector GetPosition(bool bAllowOverride) const;

	/**
	 * Works out the interp value between source and end
	 */
	float Interpolate(const double EndTime);

	/**
	 * Gets the current state of the interior settings for the listener
	 */
	void UpdateCurrentInteriorSettings();

	/**
	 * Apply the interior settings to ambient sounds
	 */
	void ApplyInteriorSettings(uint32 AudioVolumeID, const FInteriorSettings& Settings);

	FListener(FAudioDevice* InAudioDevice)
		: Transform(FTransform::Identity)
		, Velocity(ForceInit)
		, AttenuationOverride(ForceInit)
		, bUseAttenuationOverride(false)
		, AudioVolumeID(0)
		, ListenerIndex(0)
		, InteriorStartTime(0.0)
		, InteriorEndTime(0.0)
		, ExteriorEndTime(0.0)
		, InteriorLPFEndTime(0.0)
		, ExteriorLPFEndTime(0.0)
		, InteriorVolumeInterp(0.f)
		, InteriorLPFInterp(0.f)
		, ExteriorVolumeInterp(0.f)
		, ExteriorLPFInterp(0.f)
		, AudioDevice(InAudioDevice)
	{
	}

private:
	FListener();
};

/**
 * Game thread representation of a listener
 */
struct FListenerProxy
{
	FTransform Transform;

	/** An attenuation override to use for distance and attenuation calculations */
	FVector AttenuationOverride;

	/** Is our attenuation override active */
	uint32 bUseAttenuationOverride :1;

	/** The ID of the world the listener resides in */
	uint32 WorldID = INDEX_NONE;

	/**
	 * Gets the position of the listener proxy
	 */
	FVector GetPosition(bool bAllowOverride) const;

	FListenerProxy()
		: bUseAttenuationOverride(false)
	{
	}

	FListenerProxy(const FListener& Listener)
		: Transform(Listener.Transform)
		, AttenuationOverride(Listener.AttenuationOverride)
		, bUseAttenuationOverride(Listener.bUseAttenuationOverride)
		, WorldID(Listener.WorldID)
	{
	}
};

/**
 * Structure for collating info about sound classes
 */
struct FAudioClassInfo
{
	int32 NumResident;
	int32 SizeResident;
	int32 NumRealTime;
	int32 SizeRealTime;

	FAudioClassInfo()
		: NumResident(0)
		, SizeResident(0)
		, NumRealTime(0)
		, SizeRealTime(0)
	{
	}
};

struct FSoundMixState
{
	bool IsBaseSoundMix;
	uint32 ActiveRefCount;
	uint32 PassiveRefCount;
	double StartTime;
	double FadeInStartTime;
	double FadeInEndTime;
	double FadeOutStartTime;
	double EndTime;
	float InterpValue;
	ESoundMixState::Type CurrentState;
};

struct FSoundMixClassOverride
{
	FSoundClassAdjuster SoundClassAdjustor;
	FDynamicParameter VolumeOverride;
	FDynamicParameter PitchOverride;
	float FadeInTime;
	uint8 bOverrideApplied : 1;
	uint8 bOverrideChanged : 1;
	uint8 bIsClearing : 1;
	uint8 bIsCleared : 1;

	FSoundMixClassOverride()
		: VolumeOverride(1.0f)
		, PitchOverride(1.0f)
		, FadeInTime(0.0f)
		, bOverrideApplied(false)
		, bOverrideChanged(false)
		, bIsClearing(false)
		, bIsCleared(false)
	{
	}
};

typedef TMap<USoundClass*, FSoundMixClassOverride> FSoundMixClassOverrideMap;
typedef TWeakObjectPtr<UAudioComponent> FAudioComponentPtr;

struct FActivatedReverb
{
	FReverbSettings ReverbSettings;
	float Priority;

	FActivatedReverb()
		: Priority(0.f)
	{
	}
};

/** Struct used to cache listener attenuation vector math results */
struct FAttenuationListenerData
{
	FVector ListenerToSoundDir;
	FVector::FReal AttenuationDistance;
	FVector::FReal ListenerToSoundDistance;

	// (AudioMixer only)
	// Non-attenuation distance for calculating surround sound speaker maps for sources w/ spread
	FVector::FReal ListenerToSoundDistanceForPanning;

	FTransform ListenerTransform;
	const FTransform SoundTransform;
	const FSoundAttenuationSettings* AttenuationSettings;

	/** Computes and returns some geometry related to the listener and the given sound transform. */
	static FAttenuationListenerData Create(const FAudioDevice& AudioDevice, int32 ListenerIndex, const FTransform& InSoundTransform, const FSoundAttenuationSettings& InAttenuationSettings);

private:
	FAttenuationListenerData(const FTransform& InListenerTransform, const FTransform& InSoundTransform, const FSoundAttenuationSettings& InAttenuationSettings)
		: ListenerToSoundDir(FVector::ZeroVector)
		, AttenuationDistance(0.0f)
		, ListenerToSoundDistance(0.0f)
		, ListenerToSoundDistanceForPanning(0.0f)
		, ListenerTransform(InListenerTransform)
		, SoundTransform(InSoundTransform)
		, AttenuationSettings(&InAttenuationSettings)
	{
	}
};

/*
* Setting for global focus scaling
*/
struct FGlobalFocusSettings
{
	float FocusAzimuthScale;
	float NonFocusAzimuthScale;
	float FocusDistanceScale;
	float NonFocusDistanceScale;
	float FocusVolumeScale;
	float NonFocusVolumeScale;
	float FocusPriorityScale;
	float NonFocusPriorityScale;

	FGlobalFocusSettings()
		: FocusAzimuthScale(1.0f)
		, NonFocusAzimuthScale(1.0f)
		, FocusDistanceScale(1.0f)
		, NonFocusDistanceScale(1.0f)
		, FocusVolumeScale(1.0f)
		, NonFocusVolumeScale(1.0f)
		, FocusPriorityScale(1.0f)
		, NonFocusPriorityScale(1.0f)
	{
	}
};

/** Interface to register a device changed listener to respond to audio device changes. */
class IDeviceChangedListener
{
public:
	virtual void OnDeviceRemoved(FString DeviceID) = 0;
	virtual void OnDefaultDeviceChanged() = 0;
};

struct FAudioDeviceRenderInfo
{
	int32 NumFrames = 0;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioDevicePreRender, const FAudioDeviceRenderInfo&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioDevicePostRender, const FAudioDeviceRenderInfo&);

class FAudioDevice : public FExec
{
public:

	//Begin FExec Interface
#if UE_ALLOW_EXEC_COMMANDS
	ENGINE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar = *GLog) override;
#endif
	//End FExec Interface

#if !UE_BUILD_SHIPPING
	ENGINE_API UAudioComponent* GetTestComponent(UWorld* InWorld);
	ENGINE_API void StopTestComponent();

private:
	/**
	 * Exec command handlers
	 */
	bool HandleDumpSoundInfoCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	/**
	 * Lists all the playing waveinstances and their associated source
	 */
	bool HandleListWavesCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	/**
	 * Lists a summary of loaded sound collated by class
	 */
	bool HandleListSoundClassesCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	/**
	 * shows sound class hierarchy
	 */
	bool HandleShowSoundClassHierarchyCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleListSoundClassVolumesCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleListAudioComponentsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleListSoundDurationsCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSoundTemplateInfoCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSetBaseSoundMixCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleIsolateDryAudioCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleIsolateReverbCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleTestLPFCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleTestHPFCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleTestLFEBleedCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleDisableLPFCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleDisableHPFCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleDisableRadioCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleEnableRadioCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleResetSoundStateCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleToggleSpatializationExtensionCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleEnableHRTFForAllCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSoloCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleClearSoloCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandlePlayAllPIEAudioCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudio3dVisualizeCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioMemoryInfo(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioSoloSoundClass(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioSoloSoundWave(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioSoloSoundCue(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioMixerDebugSound(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSoundClassFixup(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleAudioDebugSound(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleResetAllDynamicSoundVolumesCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleResetDynamicSoundVolumeCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleGetDynamicSoundVolumeCommand(const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSetDynamicSoundCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Handles all argument parsing for the solo commands in one place */
	using FToggleSoloPtr = void (Audio::FAudioDebugger::*)(FName InName, bool bExclusive);
	void HandleAudioSoloCommon(const TCHAR* Cmd, FOutputDevice& Ar, FToggleSoloPtr Funct);

	/**
	* Lists a summary of loaded sound collated by class
	*/
	void ShowSoundClassHierarchy(FOutputDevice& Ar, USoundClass* SoundClass = nullptr, int32 Indent = 0) const;

	/**
	* Gets a summary of loaded sound collated by class
	*/
	void GetSoundClassInfo(TMap<FName, FAudioClassInfo>& AudioClassInfos);
#endif

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

public:

	/**
	 * Constructor
	 */
	ENGINE_API FAudioDevice();

	ENGINE_API virtual ~FAudioDevice();

	/** Returns an array of available audio devices names for the platform */
	virtual void GetAudioDeviceList(TArray<FString>& OutAudioDeviceNames) const
	{
	}

	/** Returns the quality settings used by the default audio settings. */
	ENGINE_API static const FAudioQualitySettings& GetQualityLevelSettings();

	/**
	 * Basic initialization of the platform agnostic layer of the audio system
	 */
	ENGINE_API bool Init(Audio::FDeviceId InDeviceID, int32 InMaxSources, int32 BufferSizeOverride = INDEX_NONE, int32 NumBuffersOverride = INDEX_NONE);

	/**
	 * Called after FAudioDevice creation and init
	 */
	ENGINE_API void OnDeviceCreated(Audio::FDeviceId InDeviceID);

	/**
	 * Called before FAudioDevice teardown
	 */
	ENGINE_API void OnDeviceDestroyed(Audio::FDeviceId InDeviceID);

	/**
	 * Tears down the audio device
	 */
	ENGINE_API void Teardown();

	/**
	 * De-initialization step that occurs after device destroyed broadcast, but before removal from the device manager
	 */
	ENGINE_API void Deinitialize();

	/**
	 * The audio system's main "Tick" function
	 */
	ENGINE_API void Update(bool bGameTicking);

	/** Update called on game thread. */
	virtual void UpdateGameThread() {}

	/**
	 * Suspend/resume all sounds (global pause for device suspend/resume, etc.)
	 *
	 * @param bGameTicking Whether the game is still ticking at the time of suspend
	 */
	ENGINE_API void Suspend(bool bGameTicking);

	/**
	 * Counts the bytes for the structures used in this class
	 */
	ENGINE_API virtual void CountBytes(FArchive& Ar);

	/**
	 * Track references to UObjects
	 */
	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/**
	 * Iterate over the active AudioComponents for wave instances that could be playing.
	 *
	 * @return Index of first wave instance that can have a source attached
	 */
	ENGINE_API int32 GetSortedActiveWaveInstances(TArray<FWaveInstance*>& WaveInstances, const ESortedActiveWaveGetType::Type GetType);

	/** Update the active sound playback time. This is done here to do after all audio is updated. */
	ENGINE_API void UpdateActiveSoundPlaybackTime(bool bIsTimeTicking);

	/** Optional fadeout and fade in of audio to avoid clicks when closing or opening/reusing audio device. */
	virtual void FadeOut() {}
	virtual void FadeIn() {}

	/**
	 * Stop all the audio components and sources attached to the world. nullptr world means all components.
	 */
	ENGINE_API void Flush(UWorld* WorldToFlush, bool bClearActivatedReverb = true);

	/*
	* Derived classes can override this method to do their own cleanup. Called at the end of FAudioDevice::Flush()
	*/
	ENGINE_API virtual void FlushExtended(UWorld* WorldToFlush, bool bClearActivatedReverb);

	/**
	 * Allows audio rendering command queue to flush during audio device flush.
	 * @param bPumpSynchronously must be called in situations where the audio render thread is not being called.
	 * @param bRenderThreadStopped if true, skips waiting on the render thread event and pumps commands inline immediately.
	 */
	virtual void FlushAudioRenderingCommands(bool bPumpSynchronously = false, bool bRenderThreadStopped = false) {}

	ENGINE_API void OnPreGarbageCollect();

	/**
	 * Stop any playing sounds that are using a particular SoundWave
	 *
	 * @param SoundWave					Resource to stop any sounds that are using it
	 * @param[out] StoppedComponents	List of Audio Components that were stopped
	 */
	ENGINE_API void StopSoundsUsingResource(USoundWave* SoundWave, TArray<UAudioComponent*>* StoppedComponents = nullptr);

	ENGINE_API static bool LegacyReverbDisabled();

#if WITH_EDITOR
	/** Deals with anything audio related that should happen when PIE starts */
	ENGINE_API void OnBeginPIE(const bool bIsSimulating);

	/** Deals with anything audio related that should happen when PIE ends */
	ENGINE_API void OnEndPIE(const bool bIsSimulating);
#endif

	/**
	 * Precaches the passed in sound node wave object.
	 *
	 * @param	SoundWave		Resource to be precached.
	 * @param	bSynchronous	If true, this function will block until a vorbis decompression is complete
	 * @param	bTrackMemory	If true, the audio mem stats will be updated
	 * @param   bForceFullDecompression If true, the sound wave will be fully decompressed regardless of size.
	 */
	ENGINE_API virtual void Precache(USoundWave* SoundWave, bool bSynchronous = false, bool bTrackMemory = false, bool bForceFullDecompression = false);

	UE_DEPRECATED(5.7, "With PCM audio caching removed, this hueristic is no longer used.")
	ENGINE_API float GetCompressionDurationThreshold(const FSoundGroup &SoundGroup);

	/**
	 * Returns true if a sound wave should be decompressed.
	 */
	UE_DEPRECATED(5.7, "Most of these arguments are no longer used, replace calls with the caller's value for bForceFullDecompression")
	ENGINE_API bool ShouldUseRealtimeDecompression(bool bForceFullDecompression, const FSoundGroup &SoundGroup, USoundWave* SoundWave, float CompressedDurationThreshold) const;

	/**
	 * Precaches all existing sounds. Called when audio setup is complete
	 */
	ENGINE_API void PrecacheStartupSounds();

	/**
	 * Sets the maximum number of channels dynamically. Can't raise the cap over the initial value but can lower it
	 */
	ENGINE_API void SetMaxChannels(int32 InMaxChannels);

	/**
	 * Sets the maximum number of channels dynamically by scaled percentage.
	 */
	ENGINE_API void SetMaxChannelsScaled(float InScaledChannelCount);

	/** Returns the max channels used by the audio device. */
	ENGINE_API int32 GetMaxChannels() const;

	/** Returns the maximum sources used by the audio device set on initialization,
	  * including the number of stopping voices reserved. */
	ENGINE_API int32 GetMaxSources() const;

	/**
	 * Returns global pitch range
	 */
	ENGINE_API TRange<float> GetGlobalPitchRange() const;

	/**
	* Stops any sound sources which are using the given buffer.
	*
	* @param	FSoundBuffer	Buffer to check against
	*/
	UE_DEPRECATED(5.5, "No longer needed for resource managment, existing calls may be deleted")
	ENGINE_API void StopSourcesUsingBuffer(FSoundBuffer * SoundBuffer);

	/**
	 * Stops all game sounds (and possibly UI) sounds
	 *
	 * @param bShouldStopUISounds If true, this function will stop UI sounds as well
	 */
	ENGINE_API virtual void StopAllSounds(bool bShouldStopUISounds = false);

	/**
	 * Sets the details about the listener
	 * @param	World				The world the listener is being set on.
	 * @param   ListenerIndex		The index of the listener
	 * @param   ListenerTransform   The listener's world transform
	 * @param   DeltaSeconds		The amount of time over which velocity should be calculated.  If 0, then velocity will not be calculated.
	 */
	ENGINE_API void SetListener(UWorld* World, int32 InListenerIndex, const FTransform& ListenerTransform, float InDeltaSeconds);

	/** Sets an override position for the specified listener to do attenuation calculations. */
	ENGINE_API void SetListenerAttenuationOverride(int32 ListenerIndex, const FVector AttenuationPosition);

	/** Removes a listener attenuation override for the specified listener. */
	ENGINE_API void ClearListenerAttenuationOverride(int32 ListenerIndex);

	const TArray<FListener>& GetListeners() const { check(IsInAudioThread()); return Listeners; }

	/**
	 * Returns the currently applied reverb effect if there is one.
	 */
	UReverbEffect* GetCurrentReverbEffect() const
	{
		check(IsInGameThread());
		return CurrentReverbEffect;
	}

	struct FCreateComponentParams
	{
		ENGINE_API FCreateComponentParams();
		ENGINE_API FCreateComponentParams(UWorld* World, AActor* Actor = nullptr);
		ENGINE_API FCreateComponentParams(AActor* Actor);
		ENGINE_API FCreateComponentParams(FAudioDevice* AudioDevice);

		USoundAttenuation* AttenuationSettings;
		TSubclassOf<UAudioComponent> AudioComponentClass = UAudioComponent::StaticClass();
		TSet<USoundConcurrency*> ConcurrencySet;
		bool bAutoDestroy;
		bool bPlay;
		bool bStopWhenOwnerDestroyed;

		ENGINE_API void SetLocation(FVector Location);
		ENGINE_API bool ShouldUseAttenuation() const;

	private:
		UWorld* World;
		AActor* Actor;
		FAudioDevice* AudioDevice;

		bool bLocationSet;
		FVector Location;

		ENGINE_API void CommonInit();

		friend FAudioDevice;
	};

	/**
	 * Creates an audio component to handle playing a sound.
	 * @param   Sound				The USoundBase to play at the location.
	 * @param   Params				The parameter block of properties to create the component based on
	 * @return	The created audio component if the function successfully created one or a nullptr if not successful. Note: if audio is disabled or if there were no hardware audio devices available, this will return nullptr.
	 */
	ENGINE_API static UAudioComponent* CreateComponent(USoundBase* Sound, const FCreateComponentParams& Params = FCreateComponentParams());

	/**
	 * Plays a sound at the given location without creating an audio component.
	 * @param   Sound				The USoundBase to play at the location.
	 * @param   World				The world this sound is playing in.
	 * @param   VolumeMultiplier	The volume multiplier to set on the sound.
	 * @param   PitchMultiplier		The pitch multiplier to set on the sound.
	 * @param	StartTime			The initial time offset for the sound.
	 * @param	Location			The sound's position.
	 * @param	Rotation			The sound's rotation.
	 * @param	AttenuationSettings	The sound's attenuation settings to use (optional). Will default to the USoundBase's AttenuationSettings if not specified.
	 * @param	USoundConcurrency	The sound's sound concurrency settings to use (optional). Will use the USoundBase's USoundConcurrency if not specified.
	 * @param	Params				An optional list of audio component params to immediately apply to a sound.
	 */
	ENGINE_API void PlaySoundAtLocation(USoundBase* Sound, UWorld* World, float VolumeMultiplier, float PitchMultiplier, float StartTime, const FVector& Location, const FRotator& Rotation, USoundAttenuation* AttenuationSettings = nullptr, USoundConcurrency* ConcurrencySettings = nullptr, const TArray<FAudioParameter>* Params = nullptr, const AActor* OwningActor = nullptr);
	/** As above, but supporting move semantics to reduce copying of Params */
	ENGINE_API void PlaySoundAtLocation(USoundBase* Sound, UWorld* World, float VolumeMultiplier, float PitchMultiplier, float StartTime, const FVector& Location, const FRotator& Rotation, USoundAttenuation* AttenuationSettings, USoundConcurrency* Concurrency, TArray<FAudioParameter>&& Params, const AActor* OwningActor = nullptr);


	/**
	 * Adds an active sound to the audio device
	 */
	ENGINE_API void AddNewActiveSound(const FActiveSound& ActiveSound, const TArray<FAudioParameter>* InDefaultParams = nullptr, const FString* EventLogID = nullptr);

	/**
	 * Adds an active sound to the audio device
	*/
	ENGINE_API void AddNewActiveSound(TSharedRef<FActiveSound> ActiveSound, TArray<FAudioParameter>&& InDefaultParams, const FString* EventLogID = nullptr);

	/**
	 * Adds an active sound to the audio device
	 */
	UE_DEPRECATED(5.7, "Use AddNewActiveSound which accepts a TSharedRef<>")
	ENGINE_API void AddNewActiveSound(const FActiveSound& ActiveSound, TArray<FAudioParameter>&& InDefaultParams);

	/**
	 * Attempts to retrigger a provided loop
	 */
	ENGINE_API void RetriggerVirtualLoop(FAudioVirtualLoop& VirtualLoop);

	/**
	 * Removes the active sound for the specified audio component
	 */
	ENGINE_API void StopActiveSound(uint64 AudioComponentID, const FString* EventLogID = nullptr);

	/**
	* (Deprecated in favor of AddSoundToStop). Stops the active sound
	*/
	ENGINE_API void StopActiveSound(FActiveSound* ActiveSound, const FString* EventLogID = nullptr);

	/**
	* Pauses the active sound for the specified audio component
	*/
	ENGINE_API void PauseActiveSound(uint64 AudioComponentID, const bool bInIsPaused);

	/** Notify that a pending async occlusion trace finished on the active sound. */
	ENGINE_API void NotifyActiveSoundOcclusionTraceDone(FActiveSound* ActiveSound, bool bIsOccluded);

	/**
	* Finds the active sound for the specified audio component ID.
	* This call cannot be called from the GameThread & AudioComponents can now execute 
	* multiple ActiveSound instances at once.  Use 'SendCommandToActiveSounds' instead.
	*/
	UE_DEPRECATED(5.0, "This call cannot be called from the GameThread & AudioComponents can now execute multiple ActiveSound instances at once.  Use 'SendCommandToActiveSounds' instead.")
	ENGINE_API FActiveSound* FindActiveSound(uint64 AudioComponentID);

	/**
	 * Whether a given Audio Component ID should be allowed to have multiple
	 * associated Active Sounds
	 */
	ENGINE_API bool CanHaveMultipleActiveSounds(uint64 AudioComponentID) const;

	/**
	 * Set whether a given Audio Component ID should be allowed to have multiple
	 * associated Active Sounds
	 */
	ENGINE_API void SetCanHaveMultipleActiveSounds(uint64 AudioComponentID, bool InCanHaveMultipleActiveSounds);

	/**
	 * Removes an active sound from the active sounds array
	 */
	ENGINE_API void RemoveActiveSound(FActiveSound* ActiveSound);

	ENGINE_API void AddAudioVolumeProxy(const FAudioVolumeProxy& Proxy);
	ENGINE_API void RemoveAudioVolumeProxy(uint32 AudioVolumeID);
	ENGINE_API void UpdateAudioVolumeProxy(const FAudioVolumeProxy& Proxy);

	struct FAudioVolumeSettings
	{
		uint32 AudioVolumeID = uint32(INDEX_NONE);
		float Priority = 0.0f;
		FReverbSettings ReverbSettings;
		FInteriorSettings InteriorSettings;
		TArray<FAudioVolumeSubmixSendSettings> SubmixSendSettings;
		TArray<FAudioVolumeSubmixOverrideSettings> SubmixOverrideSettings;
		bool bChanged = false;
	};

	ENGINE_API void GetAudioVolumeSettings(const uint32 WorldID, const FVector& Location, FAudioVolumeSettings& OutSettings) const;
	ENGINE_API void ResetAudioVolumeProxyChangedState();

	/**
	 * Gathers data about interior volumes affecting the active sound (called on audio thread)
	 */
	ENGINE_API void GatherInteriorData(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const;

	/**
	 * Applies interior settings from affecting volumes to the active sound (called on audio thread)
	 */
	ENGINE_API void ApplyInteriorSettings(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const;

	/**
	 * Allows subsystems to modify ParseParams before Sound.Parse begins
	 */
	ENGINE_API void SubsystemsAdjustParseParams(FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) const;

	/**
	 * Notifies subsystems an active sound is about to be deleted (called on audio thread) - Deprecated, see NotifySubsystemsActiveSoundDeleting
	 */
	UE_DEPRECATED(5.3, "NotifyPending is deprecated in public scope. Use IActiveSoundUpdateInterface::NotifyActiveSoundDeleting instead.")
	void NotifyPendingDelete(FActiveSound& ActiveSound) const {}

protected:

	/**
	 * Notifies subsystems an active sound has just been added (called on audio thread).
	 * Called both for brand new sounds and for virtualized sounds that have just become active.
	 */
	ENGINE_API void NotifySubsystemsActiveSoundCreated(FActiveSound& ActiveSound) const;

	/**
	 * Notifies subsystems an active sound is about to be deleted (called on audio thread).
	 * Called when a sound is either stopped or virtualized. In either case,
	 * the referenced ActiveSound object no longer exists after this call; any pointers to it should be discarded.
	 */
	ENGINE_API void NotifySubsystemsActiveSoundDeleting(FActiveSound& ActiveSound) const;

	/**
	 * Notifies subsystems a virtualized sound has just been added (called on audio thread).
	 * Called both for brand new sounds and for active sounds that have just become virtualized.
	 */
	ENGINE_API void NotifySubsystemsVirtualizedSoundCreated(FActiveSound& ActiveSound) const;

	/**
	 * Notifies subsystems a virtualized sound is about to be deleted (called on audio thread).
	 * Called when a sound is either stopped or re-triggered. In either case,
	 * the referenced ActiveSound object no longer exists after this call; any pointers to it should be discarded.
	 */
	ENGINE_API void NotifySubsystemsVirtualizedSoundDeleting(FActiveSound& ActiveSound) const;

	UE_DEPRECATED(5.5, "NotifyAddActiveSound is deprecated. Use NotifySubsystemsActiveSoundCreated instead.")
	void NotifyAddActiveSound(FActiveSound& ActiveSound) const
	{
		NotifySubsystemsActiveSoundCreated(ActiveSound);
	}

	UE_DEPRECATED(5.5, "NotifyPendingDeleteInternal is deprecated. Use NotifySubsystemsActiveSoundDeleting and/or NotifySubsystemsVirtualizedSoundDeleting instead.")
	void NotifyPendingDeleteInternal(FActiveSound& ActiveSound) const
	{
		NotifySubsystemsActiveSoundDeleting(ActiveSound);
	}

	UE_DEPRECATED(5.7, "NotifySubsystemsSetOwner is deprecated. Use ActiveSound.GetOwnerKey instead.")
	ENGINE_API void NotifySubsystemsSetOwner(FActiveSound& ActiveSound, const UObject* Owner) const;

public:

	/**
	 * Gets the default reverb and interior settings for the provided world.  Returns true if settings with WorldID were located
	 */
	ENGINE_API bool GetDefaultAudioSettings(uint32 WorldID, FReverbSettings& OutReverbSettings, FInteriorSettings& OutInteriorSettings) const;

	/**
	 * Sets the default reverb and interior settings for the provided world
	 */
	ENGINE_API void SetDefaultAudioSettings(UWorld* World, const FReverbSettings& DefaultReverbSettings, const FInteriorSettings& DefaultInteriorSettings);

	/**
	 * Gets the current audio debug state
	 */
	EDebugState GetMixDebugState() const { return((EDebugState)DebugState); }

	ENGINE_API void SetMixDebugState(EDebugState DebugState);

	/**
	 * Set up the sound class hierarchy
	 */
	ENGINE_API void InitSoundClasses();

protected:
	/**
	 * Set up the initial sound sources
	 * Allows us to initialize sound source early on, allowing for render callback hookups for iOS Audio.
	 */
	ENGINE_API void InitSoundSources();

	/** Create our subsystem collection root object and initialize subsystems */
	ENGINE_API void InitializeSubsystemCollection();

	// Handle for our device created delegate
	FDelegateHandle DeviceCreatedHandle;

	// Handle for our device destroyed delegate
	FDelegateHandle DeviceDestroyedHandle;

	FCriticalSection RenderStateCallbackListCritSec;

	// Callback as audio device is about to render a buffer
	FOnAudioDevicePreRender OnAudioDevicePreRender;

	// Callback as audio device has just finished rendering a buffer
	FOnAudioDevicePostRender OnAudioDevicePostRender;

public:
	/**
	 * Registers a sound class with the audio device
	 *
	 * @param	SoundClassName	name of sound class to retrieve
	 * @return	sound class properties if it exists
	 */
	ENGINE_API void RegisterSoundClass(USoundClass* InSoundClass);

	/**
	* Unregisters a sound class
	*/
	ENGINE_API void UnregisterSoundClass(USoundClass* SoundClass);

	/** Initializes sound submixes. */
	virtual void InitSoundSubmixes() {}

	/** Registers the sound submix */
	virtual void RegisterSoundSubmix(USoundSubmixBase* SoundSubmix, bool bInit) {}

	/** Unregisters the sound submix */
	virtual void UnregisterSoundSubmix(const USoundSubmixBase* SoundSubmix, const bool bReparentChildren) {}

	ENGINE_API virtual USoundSubmix& GetMainSubmixObject() const;

	UE_DEPRECATED(5.4, "Use RegisterSubmixBufferListener version that requires a shared reference to a listener and provide explicit reference to a submix: use GetMainSubmixObject to register with the Main Output Submix (rather than nullptr for safety), and instantiate buffer listener via the shared pointer API.")
	ENGINE_API virtual void RegisterSubmixBufferListener(ISubmixBufferListener* InSubmixBufferListener, USoundSubmix* SoundSubmix = nullptr);

	UE_DEPRECATED(5.4, "Use UnregisterSubmixBufferListener version that requires a shared reference to a listener and provide explicit reference to a submix: use GetMainSubmixObject to unregister from the Main Output Submix (rather than nullptr for safety), and instantiate buffer listener via the shared pointer API.")
	ENGINE_API virtual void UnregisterSubmixBufferListener(ISubmixBufferListener* InSubmixBufferListener, USoundSubmix* SoundSubmix = nullptr);

	/**
	 * Registers the provided submix buffer listener with the given submix.
	*/
	ENGINE_API virtual void RegisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& SoundSubmix);

	/**
	 * Unregisters the provided submix buffer listener with the given submix.
	*/
	ENGINE_API virtual void UnregisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& SoundSubmix);


	ENGINE_API virtual Audio::FPatchOutputStrongPtr AddPatchForSubmix(uint32 InObjectId, float InPatchGain);

	ENGINE_API virtual Audio::FPatchInput AddPatchInputForAudioBus(uint32 InAudioBusId, int32 InFrames, int32 InChannels, float InGain = 1.f);

	ENGINE_API virtual Audio::FPatchOutputStrongPtr AddPatchOutputForAudioBus(uint32 InAudioBusId, int32 InFrames, int32 InChannels, float InGain = 1.f);

	/**
	* Gets the current properties of a sound class, if the sound class hasn't been registered, then it returns nullptr
	*
	* @param	SoundClassName	name of sound class to retrieve
	* @return	sound class properties if it exists
	*/
	ENGINE_API FSoundClassProperties* GetSoundClassCurrentProperties(USoundClass* InSoundClass);

	/** 
	* Returns the parameters which are dynamic from the given sound class. 
	*/
	ENGINE_API FSoundClassDynamicProperties* GetSoundClassDynamicProperties(USoundClass* InSoundClass);

	/**
	* Checks to see if a coordinate is within a distance of any listener
	*/
	ENGINE_API bool LocationIsAudible(const FVector& Location, const float MaxDistance) const;

	/**
	* Checks to see if a coordinate is within a distance of a specific listener
	*/
	ENGINE_API bool LocationIsAudible(const FVector& Location, int32 ListenerIndex, const float MaxDistance) const;

	/**
	* Returns the distance to the nearest listener from the given location
	*/
	ENGINE_API float GetDistanceToNearestListener(const FVector& Location) const;

	/**
	* Sets OutSqDistance to the distance from location to the appropriate listener representation, depending on calling thread.
	* Returns true if listener position is valid, false if not (in which case, OutSqDistance is undefined).
	*/
	ENGINE_API bool GetDistanceSquaredToListener(const FVector& Location, int32 ListenerIndex, float& OutSqDistance) const;

	/**
	* Sets OutSqDistance to the distance from location the closest listener, depending on calling thread.
	* Returns true if listener position is valid, false if not (in which case, OutSqDistance is undefined).
	*/
	ENGINE_API bool GetDistanceSquaredToNearestListener(const FVector& Location, float& OutSqDistance) const;
		
	/**
	* Returns the global maximum distance used in the audio engine.
	*/
	ENGINE_API static float GetMaxWorldDistance();

	/**
	* Returns a position from the appropriate listener representation, depending on calling thread.
	*
	* @param	ListenerIndex	index of the listener or proxy
	* @param	OutPosition		filled in position of the listener or proxy
	* @param	bAllowOverride	if true we will use the attenuation override for position, if set
	* @return	true if successful
	*/
	ENGINE_API bool GetListenerPosition(int32 ListenerIndex, FVector& OutPosition, bool bAllowOverride) const;

	/**
	* Returns the transform of the appropriate listener representation, depending on calling thread
	*/
	ENGINE_API bool GetListenerTransform(int32 ListenerIndex, FTransform& OutTransform) const;

	/**
	 * Returns the WorldID of the appropriate listener representation, depending on calling thread
	 */
	ENGINE_API bool GetListenerWorldID(int32 ListenerIndex, uint32& OutWorldID) const;

	/**
	 * Sets the Sound Mix that should be active by default
	 */
	ENGINE_API void SetDefaultBaseSoundMix(USoundMix* SoundMix);

	/**
	 * Removes a sound mix - called when SoundMix is unloaded
	 */
	ENGINE_API void RemoveSoundMix(USoundMix* SoundMix);

	/**
	 * Resets all interpolating values to defaults.
	 */
	ENGINE_API void ResetInterpolation();

	/** Enables or Disables the radio effect. */
	ENGINE_API void EnableRadioEffect(bool bEnable = false);

	friend class FAudioEffectsManager;
	/**
	 * Sets a new sound mix and applies it to all appropriate sound classes
	 */
	ENGINE_API void SetBaseSoundMix(USoundMix* SoundMix);

	/**
	 * Push a SoundMix onto the Audio Device's list.
	 *
	 * @param SoundMix The SoundMix to push.
	 * @param bIsPassive Whether this is a passive push from a playing sound.
	 */
	ENGINE_API void PushSoundMixModifier(USoundMix* SoundMix, bool bIsPassive = false, bool bIsRetrigger = false);

	/**
	 * Sets a sound class override in the given sound mix.
	 */
	ENGINE_API void SetSoundMixClassOverride(USoundMix* InSoundMix, USoundClass* InSoundClass, float Volume, float Pitch, float FadeInTime, bool bApplyToChildren);

	/**
	* Clears a sound class override in the given sound mix.
	*/
	ENGINE_API void ClearSoundMixClassOverride(USoundMix* InSoundMix, USoundClass* InSoundClass, float FadeOutTime);

	/**
	 * Pop a SoundMix from the Audio Device's list.
	 *
	 * @param SoundMix The SoundMix to pop.
	 * @param bIsPassive Whether this is a passive pop from a sound finishing.
	 */
	ENGINE_API void PopSoundMixModifier(USoundMix* SoundMix, bool bIsPassive = false);

	/**
	 * Clear the effect of one SoundMix completely.
	 *
	 * @param SoundMix The SoundMix to clear.
	 */
	ENGINE_API void ClearSoundMixModifier(USoundMix* SoundMix);

	/**
	 * Clear the effect of all SoundMix modifiers.
	 */
	ENGINE_API void ClearSoundMixModifiers();

	/** Activates a Reverb Effect without the need for a volume
	 * @param ReverbEffect Reverb Effect to use
	 * @param TagName Tag to associate with Reverb Effect
	 * @param Priority Priority of the Reverb Effect
	 * @param Volume Volume level of Reverb Effect
	 * @param FadeTime Time before Reverb Effect is fully active
	 */
	ENGINE_API void ActivateReverbEffect(UReverbEffect* ReverbEffect, FName TagName, float Priority, float Volume, float FadeTime);

	/**
	 * Deactivates a Reverb Effect not applied by a volume
	 *
	 * @param TagName Tag associated with Reverb Effect to remove
	 */
	ENGINE_API void DeactivateReverbEffect(FName TagName);

	/** Whether this SoundWave has an associated info class to decompress it */
	virtual bool HasCompressedAudioInfoClass(USoundWave* SoundWave) { return false; }

	/** Whether this device supports realtime decompression of sound waves (i.e. DTYPE_RealTime) */
	UE_DEPRECATED(5.7, "Realtime decpmpression is available on all platforms")
	virtual bool SupportsRealtimeDecompression() const
	{
		return true;
	}

	/** Whether or not the platform disables caching of decompressed PCM data (i.e. to save memory on fixed memory platforms */
	UE_DEPRECATED(5.7, "PCM Audio Caching is a leagacy feature that is now disabled for all platforms")
	virtual bool DisablePCMAudioCaching() const
	{
		return true;
	}

	/**
	 * Check for errors and output a human readable string
	 */
	virtual bool ValidateAPICall(const TCHAR* Function, uint32 ErrorCode)
	{
		return true;
	}

	const TArray<FActiveSound*>& GetActiveSounds() const
	{
		check(IsInAudioThread());
		return ActiveSounds;
	}

	/** When the set of Audio volumes have changed invalidate the cached values of active sounds */
	ENGINE_API void InvalidateCachedInteriorVolumes() const;

	/** Suspend any context related objects */
	virtual void SuspendContext() {}

	/** Resume any context related objects */
	virtual void ResumeContext() {}

	/** Check if any background music or sound is playing through the audio device */
	virtual bool IsExernalBackgroundSoundActive() { return false; }

	/** Whether or not HRTF spatialization is enabled for all. */
	ENGINE_API bool IsHRTFEnabledForAll() const;

	void SetHRTFEnabledForAll(bool InbHRTFEnabledForAll)
	{
		const bool bNewHRTFEnabledForAll = InbHRTFEnabledForAll;

		bHRTFEnabledForAll_OnGameThread = bNewHRTFEnabledForAll;

		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetHRTFEnabledForAll"), STAT_SetHRTFEnabledForAll, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, bNewHRTFEnabledForAll]()
		{
			AudioDevice->bHRTFEnabledForAll = bNewHRTFEnabledForAll;

		}, GET_STATID(STAT_SetHRTFEnabledForAll));
	}

	/** Whether or not HRTF is disabled. */
	ENGINE_API bool IsHRTFDisabled() const;

	void SetHRTFDisabled(bool InIsHRTFDisabled)
	{
		const bool bNewHRTFDisabled = InIsHRTFDisabled;

		bHRTFDisabled_OnGameThread = bNewHRTFDisabled;

		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SetHRTFDisabled"), STAT_SetHRTFDisabled, STATGROUP_AudioThreadCommands);

		FAudioDevice* AudioDevice = this;
		FAudioThread::RunCommandOnAudioThread([AudioDevice, bNewHRTFDisabled]()
		{
			AudioDevice->bHRTFDisabled = bNewHRTFDisabled;

		}, GET_STATID(STAT_SetHRTFDisabled));
	}

	void SetSpatializationInterfaceEnabled(bool InbSpatializationInterfaceEnabled)
	{
		FAudioThread::SuspendAudioThread();

		bSpatializationInterfaceEnabled = InbSpatializationInterfaceEnabled;

		FAudioThread::ResumeAudioThread();
	}

	/** Registers a third party listener-observer to this audio device. */
	ENGINE_API void RegisterPluginListener(const TAudioPluginListenerPtr PluginListener);

	/** Unregisters a third party listener-observer to this audio device. */
	ENGINE_API void UnregisterPluginListener(const TAudioPluginListenerPtr PluginListener);

	ENGINE_API bool IsAudioDeviceMuted() const;

	ENGINE_API void SetDeviceMuted(bool bMuted);

	/** Returns the azimuth angle of the sound relative to the sound's nearest listener. Used for 3d audio calculations. */
	ENGINE_API void GetAzimuth(const FAttenuationListenerData& OutListenerData, float& OutAzimuth, float& AbsoluteAzimuth) const;

	/** Returns the focus factor of a sound based on its position and listener data. */
	ENGINE_API float GetFocusFactor(const float Azimuth, const FSoundAttenuationSettings& AttenuationSettings) const;

	/** Gets the max distance and focus factor of a sound. */
	ENGINE_API void GetMaxDistanceAndFocusFactor(USoundBase* Sound, const UWorld* World, const FVector& Location, const FSoundAttenuationSettings* AttenuationSettingsToApply, float& OutMaxDistance, float& OutFocusFactor);

	/**
	* Checks if the given sound would be audible.
	* @param Sound					The sound to check if it would be audible
	* @param World					The world the sound is playing in
	* @param Location				The location the sound is playing in the world
	* @param AttenuationSettings	The (optional) attenuation settings the sound is using
	* @param MaxDistance			The computed max distance of the sound.
	* @param FocusFactor			The focus factor of the sound.
	*
	* @return Returns true if the sound is audible, false otherwise.
	*/
	ENGINE_API bool SoundIsAudible(USoundBase* Sound, const UWorld* World, const FVector& Location, const FSoundAttenuationSettings* AttenuationSettingsToApply, float MaxDistance, float FocusFactor) const;

	/** Returns the index of the listener closest to the given sound transform */
	ENGINE_API static int32 FindClosestListenerIndex(const FTransform& SoundTransform, const TArray<FListener>& InListeners);

	/** Returns the index of the listener closest to the given sound transform */
	ENGINE_API int32 FindClosestListenerIndex(const FTransform& SoundTransform) const;
	ENGINE_API int32 FindClosestListenerIndex(const FVector& Position, float& OutSqDistance, bool AllowAttenuationOverrides) const;

	/** Disables ActiveSound from responding to calls from its associated AudioComponent. */
	ENGINE_API void UnlinkActiveSoundFromComponent(const FActiveSound& InActiveSound);

	/** Return the audio stream time */
	virtual double GetAudioTime() const
	{
		return 0.0;
	}

	/** Enables the audio device to output debug audio to test audio device output. */
	virtual void EnableDebugAudioOutput()
	{
	}

	/** Returns the main audio device of the engine */
	ENGINE_API static FAudioDeviceHandle GetMainAudioDevice();

	/** Returns the audio device manager */
	ENGINE_API static FAudioDeviceManager* GetAudioDeviceManager();

	/** Low pass filter OneOverQ value */
	ENGINE_API float GetLowPassFilterResonance() const;

	/** Returns the number of active sound sources */
	virtual int32 GetNumActiveSources() const { return 0; }

	/** Returns the number of free sources. */
	int32 GetNumFreeSources() const { return Sources.Num(); }

	/** Returns the sample rate used by the audio device. */
	float GetSampleRate() const { return (float)(SampleRate); }

	/** Returns the buffer length of the audio device. */
	int32 GetBufferLength() const { return PlatformSettings.CallbackBufferFrameSize; }

	/** Returns the number of buffers the audio device keeps queued. */
	int32 GetNumBuffers() const { return PlatformSettings.NumBuffers; }

	/** Whether or not there's a spatialization plugin enabled. */
	bool IsSpatializationPluginEnabled() const
	{
		return bSpatializationInterfaceEnabled;
	}

	/** Set and initialize the current Spatialization plugin (by name) */
	ENGINE_API bool SetCurrentSpatializationPlugin(FName PluginName);

	/** Get the current list of available spatialization plugins */
	ENGINE_API TArray<FName> GetAvailableSpatializationPluginNames() const;

	/** Return the spatialization plugin interface. */
	TAudioSpatializationPtr GetSpatializationPluginInterface()
	{
		return GetCurrentSpatializationPluginInterfaceInfo().SpatializationPlugin;
	}

	/** Return the spatialization plugin interface info (requested by name). */
	struct FAudioSpatializationInterfaceInfo;
	ENGINE_API FAudioSpatializationInterfaceInfo GetCurrentSpatializationPluginInterfaceInfo();

	ENGINE_API bool SpatializationPluginInterfacesAvailable();

	/** Whether or not there's a modulation plugin enabled. */
	bool IsModulationPluginEnabled() const
	{
		return bModulationInterfaceEnabled;
	}

	/** Whether or not there's an occlusion plugin enabled. */
	bool IsOcclusionPluginEnabled() const
	{
		return bOcclusionInterfaceEnabled;
	}

	ENGINE_API static bool IsOcclusionPluginLoaded();

	/** Whether or not there's a reverb plugin enabled. */
	bool IsReverbPluginEnabled() const
	{
		return bReverbInterfaceEnabled;
	}

	ENGINE_API static bool IsReverbPluginLoaded();

	bool IsSourceDataOverridePluginEnabled() const
	{
		return bSourceDataOverrideInterfaceEnabled;
	}

	ENGINE_API static bool IsSourceDataOverridePluginLoaded();

	/** Returns if stopping voices is enabled. */
	bool IsStoppingVoicesEnabled() const
	{
		return bIsStoppingVoicesEnabled;
	}

	/** Returns if baked analysis querying is enabled. */
	bool IsBakedAnalaysisQueryingEnabled() const
	{
		return bIsBakedAnalysisEnabled;
	}

	/** Performs an operation on all active sounds requested to execute by an audio component */
	ENGINE_API void SendCommandToActiveSounds(uint64 InAudioComponentID, TUniqueFunction<void(FActiveSound&)> InFunc, const TStatId InStatId = TStatId());

	virtual bool IsNonRealtime() const
	{
		return false;
	}

	/** Updates the source effect chain. Only implemented in audio mixer. */
	virtual void UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails) {}

	/** Returns the current source effect chain entries set dynamically from BP or elsewhere. */
	virtual bool GetCurrentSourceEffectChain(const uint32 SourceEffectChainId, TArray<FSourceEffectChainEntry>& OutCurrentSourceEffectChainEntries) { return false; }

	/** Updates the submix properties of any playing submix instances. Allows editor to make changes to submix properties and hear them propagate live.*/
	virtual void UpdateSubmixProperties(USoundSubmixBase* InSubmix)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** This is called by a USoundSubmix to start recording a submix instance on this device. */
	virtual void StartRecording(USoundSubmix* InSubmix, float ExpectedRecordingDuration)
	{
		UE_LOGF(LogAudio, Error, "Submix recording only works with the audio mixer. Please run using -audiomixer to or set INI file use submix recording.");
	}

	/** This is called by a USoundSubmix when we stop recording a submix on this device. */
	virtual Audio::FAlignedFloatBuffer& StopRecording(USoundSubmix* InSubmix, float& OutNumChannels, float& OutSampleRate)
	{
		UE_LOGF(LogAudio, Error, "Submix recording only works with the audio mixer. Please run using -audiomixer to or set INI file use submix recording.");

		static Audio::FAlignedFloatBuffer InvalidBuffer;
		return InvalidBuffer;
	}

	/** This is called by a USoundSubmix to start envelope following on a submix isntance on this device. */
	virtual void StartEnvelopeFollowing(USoundSubmix* InSubmix)
	{
		UE_LOGF(LogAudio, Error, "Envelope following submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
	}

	/** This is called by a USoundSubmix when we stop envelope following a submix instance on this device. */
	virtual void StopEnvelopeFollowing(USoundSubmix* InSubmix)
	{
		UE_LOGF(LogAudio, Error, "Envelope following submixes only works with the audio mixer. Please run using -audiomixer or set INI file to use submix recording.");
	}

	/** Set the wet-dry level of the given submix */
	virtual void SetSubmixWetDryLevel(USoundSubmix* InSoundSubmix, float InOutputVolume, float InWetLevel, float InDryLevel)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Set whether or not a submix is auto-disabled. */
	virtual void SetSubmixAutoDisable(USoundSubmix* InSoundSubmix, bool bInAutoDisable)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Set what the auto-disable time is. */
	virtual void SetSubmixAutoDisableTime(USoundSubmix* InSoundSubmix, float InDisableTime)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	UE_DEPRECATED(5.1, "UpdateSubmixModulationSettings taking single modulators is deprecated.  Use the overload that allows for modulator sets")
	virtual void UpdateSubmixModulationSettings(USoundSubmix* InSoundSubmix, USoundModulatorBase* InOutputModulation, USoundModulatorBase* InWetLevelModulation, USoundModulatorBase* InDryLevelModulation)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer & 'UpdateSubmixModulationSettings' function taking single modulators no longer supported.");
	}

	virtual void UpdateSubmixModulationSettings(USoundSubmix* InSoundSubmix, const TSet<TObjectPtr<USoundModulatorBase>>& InOutputModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InWetLevelModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InDryLevelModulation)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	virtual void SetSubmixModulationBaseLevels(USoundSubmix* InSoundSubmix, float InVolumeModBase, float InWetModBase, float InDryModBase)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Set the wet-dry level of the given submix */
	virtual void SetSubmixOutputVolume(USoundSubmix* InSoundSubmix, float InOutputVolume)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Set the wet-dry level of the given submix */
	virtual void SetSubmixWetLevel(USoundSubmix* InSoundSubmix, float InWetLevel)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Set the wet-dry level of the given submix */
	virtual void SetSubmixDryLevel(USoundSubmix* InSoundSubmix, float InDryLevel)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Sets a submix effect chain override for the given submix */
	virtual void SetSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, const TArray<FSoundEffectSubmixPtr>& InSubmixEffectChain, float InCrossfadeTime)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Clears all submix effect chain overrides from the submix. */
	virtual void ClearSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, float InCrossfadeTime)
	{
		UE_LOGF(LogAudio, Error, "Submixes are only supported in audio mixer.");
	}

	/** Adds an envelope follower delegate to the submix for this audio device. */
	ENGINE_API virtual void AddEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);
	ENGINE_API virtual void RemoveEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);

	ENGINE_API virtual void StartSpectrumAnalysis(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerSettings& InSettings);
	ENGINE_API virtual void StopSpectrumAnalysis(USoundSubmix* InSubmix);
	ENGINE_API virtual void GetMagnitudesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes);
	ENGINE_API virtual void GetPhasesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutPhases);
	ENGINE_API virtual void AddSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP);
	ENGINE_API virtual void RemoveSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP);


protected:
	friend class FSoundSource;

	/**
	 * Handle pausing/unpausing of sources when entering or leaving pause mode, or global pause (like device suspend)
	 */
	ENGINE_API void HandlePause(bool bGameTicking, bool bGlobalPause = false);

	/**
	 * Stop sources that need to be stopped, and touch the ones that need to be kept alive
	 * Stop sounds that are too low in priority to be played
	 */
	ENGINE_API void StopSources(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex);

	/**
	 * Start and/or update any sources that have a high enough priority to play
	 */
	ENGINE_API void StartSources(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex, bool bGameTicking);

	/**
	 * This is overridden in Audio::FMixerDevice to propogate listener information to the audio thread.
	 */
	virtual void OnListenerUpdated(const TArray<FListener>& InListeners) {};

	ENGINE_API void NotifyAudioDevicePreRender(const FAudioDeviceRenderInfo& InInfo);

	ENGINE_API void NotifyAudioDevicePostRender(const FAudioDeviceRenderInfo& InInfo);

private:

	/**
	 * Adds an active sound to the audio device. Can be a new active sound or one provided by the re-triggering
	 * loop system.
	 */
	void AddNewActiveSoundInternal(TSharedRef<FActiveSound> InActiveSound, TArray<FAudioParameter>&& InDefaultParams, FAudioVirtualLoop* InVirtualLoopToRetrigger = nullptr);


	/**
	 * Adds an active sound to the audio device. Can be a new active sound or one provided by the re-triggering
	 * loop system.
	*/
	void AddNewActiveSoundInternal(const FActiveSound& InActiveSound, TArray<FAudioParameter>&& InDefaultParams, FAudioVirtualLoop* InVirtualLoopToRetrigger = nullptr);

	/**
	 * Initializes the parameters on the active sound.
	 */
	void InitSoundParams(FActiveSound& InOutActiveSound, TArray<FAudioParameter> InDefaultParams, bool bInIsVirtualLoopRealzing) const;

	/**
	 * Reports if a sound fails to start when attempting to create a new active sound.
	 */
	void ReportSoundFailedToStart(const uint64 AudioComponentID, FAudioVirtualLoop* VirtualLoop);

	/**
	* Initializes all plugin listeners belonging to this audio device.
	* Called in the game thread.
	*
	* @param World: Pointer to the UWorld the listener is in.
	*/
	void InitializePluginListeners(UWorld* World);

	/**
	* Notifies all plugin listeners belonging to this audio device that
	* the world changed. Called in the game thread.
	*
	* @param World: Pointer to the UWorld the listener is in.
	*/
	void NotifyPluginListenersWorldChanged(UWorld* World);

	/**
	 * Parses the sound classes and propagates multiplicative properties down the tree.
	 */
	void ParseSoundClasses(float InDeltaTime);

	/** Stops quiet/low priority sounds due to being evaluated as not fulfilling concurrency requirements
	 */
	void UpdateConcurrency(TArray<FWaveInstance*>& WaveInstances, TArray<FActiveSound*>& ActiveSoundsCopy);

	/**
	 * Checks if the given sound would be audible.
	 * @param NewActiveSound	The ActiveSound attempting to be created
	 * @return True if the sound is audible, false otherwise.
	 */
	bool SoundIsAudible(const FActiveSound& NewActiveSound);


	/**
	 * Set the mix for altering sound class properties
	 *
	 * @param NewMix The SoundMix to apply
	 * @param SoundMixState The State associated with this SoundMix
	 */
	bool ApplySoundMix(USoundMix* NewMix, FSoundMixState* SoundMixState);

	/**
	 * Updates the state of a sound mix if it is pushed more than once.
	 *
	 * @param SoundMix The SoundMix we are updating
	 * @param SoundMixState The State associated with this SoundMix
	 */
	void UpdateSoundMix(USoundMix* SoundMix, FSoundMixState* SoundMixState);

	/**
	 * Updates list of SoundMixes that are applied passively, pushing and popping those that change
	 *
	 * @param WaveInstances Sorted list of active wave instances
	 * @param FirstActiveIndex Index of first wave instance that will be played.
	 */
	void UpdatePassiveSoundMixModifiers(TArray<FWaveInstance*>& WaveInstances, int32 FirstActiveIndex);

	/**
	 * Attempt to clear the effect of a particular SoundMix
	 *
	 * @param SoundMix The SoundMix we're attempting to clear
	 * @param SoundMixState The current state of this SoundMix
	 *
	 * @return Whether this SoundMix could be cleared (only true when both ref counts are zero).
	 */
	bool TryClearingSoundMix(USoundMix* SoundMix, FSoundMixState* SoundMixState);

	/**
	 * Attempt to remove this SoundMix's EQ effect - it may not currently be active
	 *
	 * @param SoundMix The SoundMix we're attempting to clear
	 *
	 * @return Whether the effect of this SoundMix was cleared
	 */
	bool TryClearingEQSoundMix(USoundMix* SoundMix);

	/**
	 * Find the SoundMix with the next highest EQ priority to the one passed in
	 *
	 * @param SoundMix The highest priority SoundMix, which will be ignored
	 *
	 * @return The next highest priority SoundMix or nullptr if one cannot be found
	 */
	USoundMix* FindNextHighestEQPrioritySoundMix(USoundMix* IgnoredSoundMix);

	/**
	 * Clear the effect of a SoundMix completely - only called after checking it's safe to
	 */
	void ClearSoundMix(USoundMix* SoundMix);

	/**
	 * Sets the sound class adjusters from a SoundMix.
	 *
	 * @param SoundMix		The SoundMix to apply adjusters from
	 * @param InterpValue	Proportion of adjuster to apply
	 * @param DeltaTime 	The current frame delta time. Used to interpolate sound class adjusters.
	 */
	void ApplyClassAdjusters(USoundMix* SoundMix, float InterpValue, float DeltaTime);

	/**
	* Construct the CurrentSoundClassProperties map
	* @param DeltaTime The current frame delta. Used to interpolate sound class adjustments.
	*
	* This contains the original sound class properties propagated properly, and all adjustments due to the sound mixes
	*/
	void UpdateSoundClassProperties(float DeltaTime);

	void VirtualizeInactiveLoops();

	/**
	 * Recursively apply an adjuster to the passed in sound class and all children of the sound class
	 *
	 * @param InAdjuster		The adjuster to apply
	 * @param InSoundClassName	The name of the sound class to apply the adjuster to.  Also applies to all children of this class
	 */
	void RecursiveApplyAdjuster(const FSoundClassAdjuster& InAdjuster, USoundClass* InSoundClass);

	/**
	 * Takes an adjuster value and modifies it by the proportion that is currently in effect
	 */
	float InterpolateAdjuster(const float Adjuster, const float InterpValue) const
	{
		return Adjuster * InterpValue + 1.0f - InterpValue;
	}

	/** Retrieve the filter frequency to use. Takes into account logarithmic nature of frequency. */
	float GetInterpolatedFrequency(const float InFrequency, const float InterpValue) const;

	/** Allow platforms to optionally specify low-level audio platform settings. */
	virtual FAudioPlatformSettings GetPlatformSettings() const { return FAudioPlatformSettings(); }

	/** Updates audio volume effects. */
	void UpdateAudioVolumeEffects();

	/** Updates audio engine subsystems on this device. */
	void UpdateAudioEngineSubsystems();

#if UE_AUDIO_PROFILERTRACE_ENABLED
	/** Notify Audio Insights that an active sound has started playing. */
	void SendSoundStartedTraceMessage(const FActiveSound& InActiveSound) const;

	/** Notify Audio Insights that an active sound is still alive and playing. */
	void SendSoundIsAlivePingTraceMessage(const FActiveSound& InActiveSound) const;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

public:

	/**
	 * Platform dependent call to init effect data on a sound source
	 */
	ENGINE_API void* InitEffect(FSoundSource* Source);

	/**
	 * Platform dependent call to update the sound output with new parameters
	 * The audio system's main "Tick" function
	 */
	ENGINE_API void* UpdateEffect(FSoundSource* Source);

	/**
	 * Platform dependent call to destroy any effect related data
	 */
	ENGINE_API void DestroyEffect(FSoundSource* Source);

	/**
	 * Return the pointer to the sound effects handler
	 */
	FAudioEffectsManager* GetEffects()
	{
		check(IsInAudioThread());
		return Effects;
	}

	/**
	 * Return the pointer to the sound effects handler
	 */
	const FAudioEffectsManager* GetEffects() const
	{
		check(IsInAudioThread());
		return Effects;
	}

	const TMap<USoundMix*, FSoundMixState>& GetSoundMixModifiers() const
	{
		return ObjectPtrDecay(SoundMixModifiers);
	}

	const TArray<USoundMix*>& GetPrevPassiveSoundMixModifiers() const
	{
		return ObjectPtrDecay(PrevPassiveSoundMixModifiers);
	}

	USoundMix* GetDefaultBaseSoundMixModifier()
	{
		return DefaultBaseSoundMix;
	}

	void SetSoundMixModifiers(const TMap<USoundMix*, FSoundMixState>& InSoundMixModifiers, const TArray<USoundMix*>& InPrevPassiveSoundMixModifiers, USoundMix* InDefaultBaseSoundMix)
	{
		SoundMixModifiers = ObjectPtrWrap(InSoundMixModifiers);
		PrevPassiveSoundMixModifiers = ObjectPtrWrap(InPrevPassiveSoundMixModifiers);
		DefaultBaseSoundMix = InDefaultBaseSoundMix;
	}

	ENGINE_API FDelegateHandle AddPreRenderDelegate(const FOnAudioDevicePreRender::FDelegate& InDelegate);

	ENGINE_API bool RemovePreRenderDelegate(const FDelegateHandle& InHandle);

	ENGINE_API FDelegateHandle AddPostRenderDelegate(const FOnAudioDevicePostRender::FDelegate& InDelegate);

	ENGINE_API bool RemovePostRenderDelegate(const FDelegateHandle& InHandle);

private:
	/**
	 * Internal helper function used by ParseSoundClasses to traverse the tree.
	 *
	 * @param CurrentClass			Subtree to deal with
	 * @param ParentProperties		Propagated properties of parent node
	 */
	void RecurseIntoSoundClasses(USoundClass* CurrentClass, FSoundClassProperties& ParentProperties);

	/**
	 * Find the current highest priority reverb after a change to the list of active ones.
	 */
	void UpdateHighestPriorityReverb();

	void SendUpdateResultsToGameThread(int32 FirstActiveIndex);

public:

// If we make FAudioDevice not be subclassable, then all the functions following would move to IAudioDeviceModule

	/** Starts up any platform specific hardware/APIs */
	virtual bool InitializeHardware()
	{
		return true;
	}

	/** Shuts down any platform specific hardware/APIs */
	virtual void TeardownHardware()
	{
	}

	/** Updates timing information for hardware. */
	virtual void UpdateHardwareTiming()
	{
	}

	/** Lets the platform any tick actions */
	virtual void UpdateHardware()
	{
	}

	/** Creates a new platform specific sound source */
	ENGINE_API virtual FAudioEffectsManager* CreateEffectsManager();

	/** Creates a new platform specific sound source */
	virtual FSoundSource* CreateSoundSource() = 0;

	/**
	 * Marks a sound to be stopped.  Returns true if added to stop,
	 * false if already pending stop.
	 */
	ENGINE_API void AddSoundToStop(FActiveSound* SoundToStop);

	/**
	 * Whether the provided ActiveSound is currently pending to stop
	 */
	ENGINE_API bool IsPendingStop(FActiveSound* ActiveSound);

	/**
	* Gets the direction of the given position vector transformed relative to listener.
	* @param Position				Input position vector to transform relative to listener
	* @param OutDistance			Optional output of distance from position to listener
	* @return The input position relative to the listener.
	*/
	ENGINE_API FVector GetListenerTransformedDirection(const FVector& Position, float* OutDistance);

	/** Returns the current audio device update delta time. */
	ENGINE_API float GetDeviceDeltaTime() const;

	/** Returns the game's delta time */
	ENGINE_API float GetGameDeltaTime() const;

	/** Returns if the specific listener is using an attenuation override position. */
	ENGINE_API bool IsUsingListenerAttenuationOverride(int32 ListenerIndex) const;

	/** Returns the listener attenuation override for the specified listener */
	ENGINE_API const FVector& GetListenerAttenuationOverride(int32 ListenerIndex) const;

	ENGINE_API void UpdateVirtualLoops(bool bForceUpdate);

	/** Sets the update delta time for the audio frame */
	virtual void UpdateDeviceDeltaTime()
	{
		const double CurrTime = FPlatformTime::Seconds();
		DeviceDeltaTime = UE_REAL_TO_FLOAT(CurrTime - LastUpdateTime);
		LastUpdateTime = CurrTime;
	}

private:
	/** Processes the set of pending sounds that need to be stopped */
	void ProcessingPendingActiveSoundStops(bool bForceDelete = false);

	/** Stops oldest sound source. */
	void StopOldestStoppingSource();

	/** Check whether we should use attenuation settings */
	bool ShouldUseAttenuation(const UWorld* World) const;

	/** Returns the number of frames to use per precache buffer. */
	int32 GetNumPrecacheFrames() const;

	/** Called by StartSources when sounds are not prepared to Init for whatever reason */
	void UpdateUnpreparedSound(FWaveInstance* WaveInstance, bool bGameTicking) const;

	/** Adjusts the active sound duration to make up for decode latency */
	void UpdateSoundDuration(FWaveInstance* WaveInstance, bool bGameTicking) const;

	bool RemoveVirtualLoop(FActiveSound& ActiveSound);
public:

	/** Query if the editor is in VR Preview for the current play world. Returns false for non-editor builds */
	ENGINE_API static bool CanUseVRAudioDevice();

	/** Returns the audio clock of the audio device. Not supported on all platforms. */
	double GetAudioClock() const { return AudioClock; }

	/** Returns the audio clock interploated between audio device callbacks to provide a smoothed value.
	 *  Default implementation does not interpolate.
	 */
	virtual double GetInterpolatedAudioClock() const { return GetAudioClock(); }

	ENGINE_API void AddVirtualLoop(const FAudioVirtualLoop& InVirtualLoop);

	bool AreStartupSoundsPreCached() const { return bStartupSoundsPreCached; }

	UE_DEPRECATED(5.1, "GetTransientMasterVolume has been deprecated. Please use GetTransientPrimaryVolume instead.")
	float GetTransientMasterVolume() const { check(IsInAudioThread()); return TransientPrimaryVolume; }

	UE_DEPRECATED(5.1, "SetTransientMasterVolume has been deprecated. Please use SetTransientPrimaryVolume instead.")
	ENGINE_API void SetTransientMasterVolume(float TransientPrimaryVolume);

	UE_DEPRECATED(5.1, "GetMasterVolume has been deprecated. Please use GetPrimaryVolume instead.")
	float GetMasterVolume() const { return PrimaryVolume; }

	float GetTransientPrimaryVolume() const { check(IsInAudioThread()); return TransientPrimaryVolume; }
	ENGINE_API void SetTransientPrimaryVolume(float TransientPrimaryVolume);

	/** Returns the volume that combines transient master volume and the FApp::GetVolumeMultiplier() value */
	float GetPrimaryVolume() const { return PrimaryVolume; }

	ENGINE_API FSoundSource* GetSoundSource(FWaveInstance* WaveInstance) const;

	ENGINE_API const FGlobalFocusSettings& GetGlobalFocusSettings() const;
	ENGINE_API void SetGlobalFocusSettings(const FGlobalFocusSettings& NewFocusSettings);

	const FDynamicParameter& GetGlobalPitchScale() const { check(IsInAudioThread()); return GlobalPitchScale; }
	ENGINE_API void SetGlobalPitchModulation(float PitchScale, float TimeSec);
	ENGINE_API float ClampPitch(float InPitchScale) const;

	/** Overrides the attenuation scale used on a sound class. */
	ENGINE_API void SetSoundClassDistanceScale(USoundClass* InSoundClass, float DistanceScale, float TimeSec);

	float GetPlatformAudioHeadroom() const { check(IsInAudioThread()); return PlatformAudioHeadroom; }
	ENGINE_API void SetPlatformAudioHeadroom(float PlatformHeadRoom);

	ENGINE_API const TMap<FName, FActivatedReverb>& GetActiveReverb() const;

	/** Whether play when silent is enabled for all sounds associated with this audio device*/
	bool PlayWhenSilentEnabled() const { return bAllowPlayWhenSilent; }

	ENGINE_API bool IsMainAudioDevice() const;

	/** Set whether or not we force the use of attenuation for non-game worlds (as by default we only care about game worlds) */
	void SetUseAttenuationForNonGameWorlds(bool bInUseAttenuationForNonGameWorlds)
	{
		bUseAttenuationForNonGameWorlds = bInUseAttenuationForNonGameWorlds;
	}

	ENGINE_API const TArray<FWaveInstance*>& GetActiveWaveInstances() const;

	/** Returns the default reverb send level used for sources which have reverb applied but no attenuation settings. */
	float GetDefaultReverbSendLevel() const { return DefaultReverbSendLevel; }

	ENGINE_API const TMap<FWaveInstance*, FSoundSource*>& GetWaveInstanceSourceMap() const;

	ENGINE_API FName GetAudioStateProperty(const FName& PropertyName) const;
	ENGINE_API void SetAudioStateProperty(const FName& PropertyName, const FName& PropertyValue);

	/** Get a Subsystem of specified type */
	UAudioEngineSubsystem* GetSubsystemBase(TSubclassOf<UAudioEngineSubsystem> SubsystemClass) const
	{
		return SubsystemCollection.GetSubsystem<UAudioEngineSubsystem>(SubsystemClass);
	}

	/** Get a Subsystem of specified type */
	template <typename TSubsystemClass>
	TSubsystemClass* GetSubsystem() const
	{
		return SubsystemCollection.GetSubsystem<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Get a Subsystem of specified type from the provided AudioDeviceHandle
	 * returns nullptr if the Subsystem cannot be found or the AudioDeviceHandle is invalid
	 */
	template <typename TSubsystemClass>
	static inline TSubsystemClass* GetSubsystem(const FAudioDeviceHandle& InHandle)
	{
		if (InHandle.IsValid())
		{
			return InHandle->GetSubsystem<TSubsystemClass>();
		}
		return nullptr;
	}

	/**
	 * Gets all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetSubsystemArrayCopy() const
	{
		return SubsystemCollection.GetSubsystemArrayCopy<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Performs the given operation on all subsystems of the given class. It's safe to create new subsystems during this operation, but not to remove subsystems.
	 */
	template <typename TSubsystemClass>
	void ForEachSubsystem(TFunctionRef<void(TSubsystemClass*)> Operation) const
	{
		((FSubsystemCollection<UAudioEngineSubsystem>&)SubsystemCollection).ForEachSubsystem(
			[Operation=MoveTemp(Operation)](UAudioEngineSubsystem* Subsystem){
				Operation(CastChecked<TSubsystemClass>(Subsystem));
			}, TSubsystemClass::StaticClass());
	}

public:

	/** The number of sources to reserve for stopping sounds. */
	int32 NumStoppingSources;

	/** The sample rate of all the audio devices */
	int32 SampleRate;

	/** The platform specific audio settings. */
	FAudioPlatformSettings PlatformSettings;

	/** The number of frames to precache. */
	int32 NumPrecacheFrames;

	/** The handle for this audio device used in the audio device manager. */
	Audio::FDeviceId DeviceID;

	struct FAudioSpatializationInterfaceInfo
	{
		// ctors
		FAudioSpatializationInterfaceInfo() = default;
		ENGINE_API FAudioSpatializationInterfaceInfo(FName InPluginName, FAudioDevice* InAudioDevice, IAudioSpatializationFactory* InAudioSpatializationFactoryPtr);

		ENGINE_API bool IsValid() const;

		FName PluginName;
		TAudioSpatializationPtr SpatializationPlugin = nullptr;
		int32 MaxChannelsSupportedBySpatializationPlugin = 1;
		uint8 bSpatializationIsExternalSend:1;
		uint8 bIsInitialized:1;
		uint8 bReturnsToSubmixGraph:1;
	};

	UE_DEPRECATED(5.1, "Do not access this member directly, it is not used. Call GetSpatializationPluginInterface() instead.")
	TAudioSpatializationPtr SpatializationPluginInterface = nullptr;

protected:
	/** 3rd party audio spatialization interface. */
	FName CurrentSpatializationPluginInterfaceName;
	TArray<FAudioSpatializationInterfaceInfo> SpatializationInterfaces;
	FAudioSpatializationInterfaceInfo* CurrentSpatializationInterfaceInfoPtr = nullptr;

	/** Cached parameters passed to the initialization of various audio plugins */
	FAudioPluginInitializationParams PluginInitializationParams;

public:
	/** 3rd party source data override interface. */
	TAudioSourceDataOverridePtr SourceDataOverridePluginInterface;

	/** 3rd party reverb interface. */
	TAudioReverbPtr ReverbPluginInterface;

	/** 3rd party occlusion interface. */
	TAudioOcclusionPtr OcclusionInterface;

	/** 3rd party modulation interface */
	TAudioModulationPtr ModulationInterface;

	/** 3rd party listener observers registered to this audio device. */
	TArray<TAudioPluginListenerPtr> PluginListeners;

	// Game thread cache of listener transforms
	TArray<FListenerProxy> ListenerProxies;

private:
	/** The maximum number of sources.  Value cannot change after initialization. */
	int32 MaxSources;

	/** The maximum number of concurrent audible sounds. Value cannot exceed MaxSources. */
	int32 MaxChannels;
	int32 MaxChannels_GameThread;

	/** Normalized (0.0f - 1.0f) scalar multiplier on max channels. */
	float MaxChannelsScale;
	float MaxChannelsScale_GameThread;

	uint64 CurrentTick;

	/** An AudioComponent to play test sounds on */
	TStrongObjectPtr<UAudioComponent> TestAudioComponent;

	/** The debug state of the audio device */
	TEnumAsByte<enum EDebugState> DebugState;

	/** transient master volume multiplier that can be modified at runtime without affecting user settings automatically reset to 1.0 on level change */
	float TransientPrimaryVolume;

	/** The master volume of the game combines the FApp::GetVolumeMultipler() value and the TransientPrimaryVolume. */
	float PrimaryVolume;

	/** Global dynamic pitch scale parameter */
	FDynamicParameter GlobalPitchScale;

	/** The global focus settings */
	FGlobalFocusSettings GlobalFocusSettings;
	FGlobalFocusSettings GlobalFocusSettings_GameThread;

	/** Timestamp of the last update */
	double LastUpdateTime;

	/** Next resource ID to assign out to a wave/buffer */
	int32 NextResourceID;

protected:
	// Audio thread representation of listeners
	TArray<FListener> Listeners;

	TArray<FSoundSource*> Sources;
	TArray<FSoundSource*> FreeSources;

private:

	/** Anchor used to connect UAudioEngineSubsystems to FAudioDevice */
	TStrongObjectPtr<UAudioSubsystemCollectionRoot> SubsystemCollectionRoot;

	/** Subsystems tied to this device's lifecycle */
	FAudioSubsystemCollection SubsystemCollection;

	/** Set of sources used to play sounds (platform will subclass these) */
	TMap<FWaveInstance*, FSoundSource*>	WaveInstanceSourceMap;

	/** Current properties of all sound classes */
	TMap<USoundClass*, FSoundClassProperties> SoundClasses;
	TMap<USoundClass*, FSoundClassDynamicProperties> DynamicSoundClassProperties;

	/** The Base SoundMix that's currently active */
	USoundMix* BaseSoundMix;

	/** The Base SoundMix that should be applied by default */
	TObjectPtr<USoundMix> DefaultBaseSoundMix;

	/** Map of sound mixes currently affecting audio properties */
	TMap<TObjectPtr<USoundMix>, FSoundMixState> SoundMixModifiers;

	/** Map of sound mix sound class overrides. Will override any sound class effects for any sound mixes */
	TMap<USoundMix*, FSoundMixClassOverrideMap> SoundMixClassEffectOverrides;

protected:
	/** Interface to audio effects processing */
	FAudioEffectsManager* Effects;

private:
	UReverbEffect* CurrentReverbEffect;

	/** A volume headroom to apply to specific platforms to achieve better platform consistency. */
	float PlatformAudioHeadroom;

	/** The default reverb send level to use for sources which have reverb applied but don't have an attenuation settings. */
	float DefaultReverbSendLevel;

	/** Reverb Effects activated without volumes - Game Thread owned */
	TMap<FName, FActivatedReverb> ActivatedReverbs;

	/** Most recently activated in-flight reverb going from the GT to the AT */
	TSharedPtr<FActivatedReverb> PendingActivatedReverb;

	/** The activated reverb that currently has the highest priority - Audio Thread owned */
	FActivatedReverb HighestPriorityActivatedReverb_AudioThread;

	/** The current audio volume settings the listener is in. */
	FAudioVolumeSettings CurrentAudioVolumeSettings;

	/** Gamethread representation of whether HRTF is enabled for all 3d sounds. (not bitpacked to avoid thread issues) */
	bool bHRTFEnabledForAll_OnGameThread;

	/** Gamethread representation of whether HRTF is disabbled for all 3d sounds. */
	bool bHRTFDisabled_OnGameThread;

	uint8 bGameWasTicking:1;

public:
	/** HACK: Temporarily disable audio caching.  This will be done better by changing the decompression pool size in the future */
	UE_DEPRECATED(5.7, "bDisableAudioCaching has no effect and will be removed in 5.8")
	uint8 bDisableAudioCaching:1 = false;

	/** Whether or not the lower-level audio device hardware initialized. */
	uint8 bIsAudioDeviceHardwareInitialized : 1;

	uint8 bIsStoppingVoicesEnabled : 1;

	/** If baked analysis querying is enabled. */
	uint8 bIsBakedAnalysisEnabled : 1;

	/** Whether or not the audio mixer module is being used by this device. */
	uint8 bAudioMixerModuleLoaded : 1;

	/** Whether or not various audio plugin interfaces are external sends. */
	uint8 bOcclusionIsExternalSend:1;
	uint8 bReverbIsExternalSend:1;

// deprecate these as they have been moved to the info struct.

	UE_DEPRECATED(5.1, "This member is no longer  in use. Use the return value of GetCurrentSpatializationPluginInterfaceInfo()")
	uint8 bSpatializationIsExternalSend:1;

	/** Max amount of channels a source can be to be spatialized by our active spatialization plugin. */
	UE_DEPRECATED(5.1, "This member is no longer  in use. Use the return value of GetCurrentSpatializationPluginInterfaceInfo()")
	int32 MaxChannelsSupportedBySpatializationPlugin;



private:
	/** True once the startup sounds have been precached */
	uint8 bStartupSoundsPreCached:1;

	/** Whether or not various audio plugin interfaces are enabled. */
	uint8 bSpatializationInterfaceEnabled:1;
	uint8 bOcclusionInterfaceEnabled:1;
	uint8 bReverbInterfaceEnabled:1;
	uint8 bSourceDataOverrideInterfaceEnabled:1;
	uint8 bModulationInterfaceEnabled:1;

	/** Whether or not we've initialized plugin listeners array. */
	uint8 bPluginListenersInitialized:1;

	/** Whether HRTF is enabled for all 3d sounds. This will automatically make all 3d mono sounds HRTF spatialized. */
	uint8 bHRTFEnabledForAll:1;

	/** Whether or not HRTF is disabled. This will make any sounds which are set to HRTF spatialize to spatialize with panning. */
	uint8 bHRTFDisabled:1;

	/** Whether the audio device is active (current audio device in-focus in PIE) */
	uint8 bIsDeviceMuted:1;

	/** Whether the audio device has been initialized */
	uint8 bIsInitialized:1;

protected:

	/** The audio clock from the audio hardware. Not supported on all platforms. */
	double AudioClock;

	/** Whether or not we allow center channel panning (audio mixer only feature.) */
	uint8 bAllowCenterChannel3DPanning : 1;

	float DeviceDeltaTime;

#if UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR
	static constexpr float SoundIsAlivePingWaitTimeSec = 1.0f;
	float SoundIsAlivePingTimeSec = 0.0f;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED && WITH_EDITOR

	/** Whether the device was initialized. */
	inline bool IsInitialized() const { return bIsInitialized; }

private:

	/** Whether the value in HighestPriorityActivatedReverb should be used - Audio Thread owned */
	uint8 bHasActivatedReverb:1;

	/** Whether or not we're supporting zero volume wave instances */
	uint8 bAllowPlayWhenSilent:1;

	/** Whether or not we force the use of attenuation for non-game worlds (as by default we only care about game worlds) */
	uint8 bUseAttenuationForNonGameWorlds:1;

	/** The audio thread update delta time for this audio thread update tick. */


	TArray<FActiveSound*> ActiveSounds;
	/** Array of sound waves to add references to avoid GC until guaranteed to be done with precache or decodes. */
	TArray<TObjectPtr<USoundWave>> ReferencedSoundWaves;

	void UpdateReferencedSoundWaves();
	TArray<USoundWave*> ReferencedSoundWaves_AudioThread;
	FCriticalSection AudioThreadReferenceCritSec;

	TArray<USoundWave*> PrecachingSoundWaves;

	TArray<FWaveInstance*> ActiveWaveInstances;

	/** Array of dormant loops stopped due to proximity/applicable concurrency rules
	  * that can be retriggered.
	  */
	TMap<FActiveSound*, FAudioVirtualLoop> VirtualLoops;

	/** Cached copy of sound class adjusters array. Cached to avoid allocating every frame. */
	TArray<FSoundClassAdjuster> SoundClassAdjustersCopy;

	/** Set of sounds which will be stopped next audio frame update */
	TSet<FActiveSound*> PendingSoundsToStop;

	/** Pending active sounds waiting to be added. */
	TQueue<FActiveSound*> PendingAddedActiveSounds;

	/** Max number of active sounds to add per frame. */
	int32 MaxActiveSoundsAddedPerFrame;

	/** A set of sounds which need to be deleted but weren't able to be deleted due to pending async operations */
	TArray<FActiveSound*> PendingSoundsToDelete;

	TMap<uint64, TArray<FActiveSound*>> AudioComponentIDToActiveSoundMap;

	/** Used to determine if a given Audio Component should be allowed to have multiple simultaneous associated active sounds */
	TMap<uint64, bool> AudioComponentIDToCanHaveMultipleActiveSoundsMap;

	TMap<uint32, FAudioVolumeProxy> AudioVolumeProxies;

	TMap<uint32, TPair<FReverbSettings,FInteriorSettings>> WorldIDToDefaultAudioVolumeSettingsMap;

	/** List of passive SoundMixes active last frame */
	TArray<TObjectPtr<USoundMix>> PrevPassiveSoundMixModifiers;

	/** A generic mapping of FNames, used to store and retrieve tokens across the engine boundary. */
	TMap<FName, FName> AudioStateProperties;

	friend class FSoundConcurrencyManager;
	FSoundConcurrencyManager ConcurrencyManager;

	/** Inverse listener transformations, used for spatialization */
	TArray<FMatrix> InverseListenerTransforms;

	/** A count of the number of one-shot active sounds. */
	uint32 OneShotCount;

	// Global min and max pitch scale, derived from audio settings
	float GlobalMinPitch;
	float GlobalMaxPitch;
};

