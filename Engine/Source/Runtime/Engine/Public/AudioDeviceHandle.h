// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"

// Set this to one if you'd like to check who owns
// handles to an audio device.
#ifndef INSTRUMENT_AUDIODEVICE_HANDLES
#define INSTRUMENT_AUDIODEVICE_HANDLES 0
#endif

namespace Audio
{
	class FMixerDevice;

	/**
	 * Typed identifier for Audio Device Id
	 */
	using FDeviceId = uint32;
}

class FAudioDevice;
class UWorld;


// Strong handle to an audio device. Guarantees that the audio device it references will stay alive while it is in scope.
class FAudioDeviceHandle
{
public:
	ENGINE_API FAudioDeviceHandle();
	ENGINE_API FAudioDeviceHandle(const FAudioDeviceHandle& Other);
	ENGINE_API FAudioDeviceHandle(FAudioDeviceHandle&& Other);

	ENGINE_API FAudioDeviceHandle& operator=(const FAudioDeviceHandle& Other);
	ENGINE_API FAudioDeviceHandle& operator=(FAudioDeviceHandle&& Other);


	ENGINE_API ~FAudioDeviceHandle();

	// gets a pointer to the audio device.
	ENGINE_API FAudioDevice* GetAudioDevice() const;

	// Returns the device ID for the audio device referenced by this handle.
	ENGINE_API Audio::FDeviceId GetDeviceID() const;

	// Returns world set when handle was created
	ENGINE_API TWeakObjectPtr<UWorld> GetWorld() const;

	// Checks whether this points to a valid AudioDevice.
	ENGINE_API bool IsValid() const;

	// Resets handle to invalid state
	ENGINE_API void Reset();

private:
	// This constructor should only be called by FAudioDeviceManager.
	ENGINE_API FAudioDeviceHandle(FAudioDevice* InDevice, Audio::FDeviceId InID, UWorld* InWorld);

	// The world that this FAudioDeviceHandle was requested with.
	// Null if this device handle wasn't generated by RequestAudioDevice.
	TWeakObjectPtr<UWorld> World;

	FAudioDevice* Device;
	Audio::FDeviceId DeviceId;

#if INSTRUMENT_AUDIODEVICE_HANDLES
	uint32 StackWalkID;
	ENGINE_API void AddStackDumpToAudioDeviceContainer();
#endif

	friend class FAudioDeviceManager;

public:
	// These are convenience operators to use an FAudioDeviceHandle like an FAudioDevice* or an Audio::FDeviceId.
	// For safety, we still require explicit casting to an FAudioDevice* to ensure ownership isn't lost due to programmer error.
	const FAudioDevice& operator*() const
	{
		check(IsValid());
		return *Device;
	}

	FAudioDevice& operator*()
	{
		check(IsValid());
		return *Device;
	}

	const FAudioDevice* operator->() const
	{
		check(IsValid());
		return Device;
	}

	FAudioDevice* operator->()
	{
		check(IsValid());
		return Device;
	}

	inline explicit operator bool() const
	{
		return IsValid();
	}

	inline bool operator==(const FAudioDeviceHandle& Other) const
	{
		return DeviceId == Other.DeviceId;
	}

	inline bool operator==(const FAudioDevice*& Other) const
	{
		return Device == Other;
	}

	inline bool operator==(FAudioDevice*& Other) const
	{
		return Device == Other;
	}

	inline bool operator==(const Audio::FDeviceId& Other) const
	{
		return DeviceId == Other;
	}

	explicit operator FAudioDevice*() const
	{
		check(IsValid());
		return Device;
	}

	explicit operator Audio::FDeviceId() const
	{
		return DeviceId;
	}
};
