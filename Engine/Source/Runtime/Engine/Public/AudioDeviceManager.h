// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioDeviceHandle.h"
#include "AudioThread.h"
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"


#if ENABLE_AUDIO_DEBUG
class FAudioDebugger;
#endif // ENABLE_AUDIO_DEBUG

class FReferenceCollector;
class FSoundBuffer;
class IAudioDeviceModule;
class UAudioComponent;
class USoundClass;
class USoundMix;
class USoundSubmixBase;
class USoundWave;
class UWorld;
class FAudioDevice;
struct FSourceEffectChainEntry;
class FSimpleAudioInfoFactory;

namespace Audio
{
	class FMixerDevice;
	class FAudioDebugger;
	class FAudioFormatSettings;
}

enum class ESoundType : uint8
{
	Class,
	Cue,
	Wave
};

// This enum is used in FAudioDeviceManager::RequestAudioDevice to map a given UWorld to an audio device.
enum class EAudioDeviceScope : uint8
{
	// Default to the behavior specified by the editor preferences.
	Default,
	// Use an audio device that can be shared by multiple worlds.
	Shared,
	// Create a new audio device specifically for this handle.
	Unique
};

// Parameters passed into FAudioDeviceManager::RequestAudioDevice.
struct FAudioDeviceParams
{
	// Optional world parameter. This allows tools to surface information about which worlds are being rendered through which audio devices.
	UWorld* AssociatedWorld = nullptr;
	// This should be set to EAudioDeviceScope::Unique if you'd like to force a new device to be created from scratch, or use EAudioDeviceScope::Shared to use an existing device if possible.
	EAudioDeviceScope Scope = EAudioDeviceScope::Default;
	// Set this to true to get a handle to a non realtime audio renderer.
	bool bIsNonRealtime = false;
	// Use this to force this audio device to use a specific audio module. If nullptr, uses the default audio module.
	IAudioDeviceModule* AudioModule = nullptr;
	// Buffer size override
	int32 BufferSizeOverride = INDEX_NONE;
	// Num buffers override
	int32 NumBuffersOverride = INDEX_NONE;
};

// List of delegates for the audio device manager.
class FAudioDeviceManagerDelegates
{
public:
	// This delegate is called whenever an entirely new audio device is created.
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnAudioDeviceCreated, Audio::FDeviceId /* AudioDeviceId*/);
	static ENGINE_API FOnAudioDeviceCreated OnAudioDeviceCreated;

	// This delegate is called just before an audio device is destroyed,
	// allowing listeners to flush any async work that could deadlock during OnAudioDeviceDestroyed.
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnPreAudioDeviceDestroyed, Audio::FDeviceId /* AudioDeviceId*/);
	static ENGINE_API FOnPreAudioDeviceDestroyed OnPreAudioDeviceDestroyed;

	// This delegate is called whenever an audio device is destroyed.
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnAudioDeviceDestroyed, Audio::FDeviceId /* AudioDeviceId*/);
	static ENGINE_API FOnAudioDeviceDestroyed OnAudioDeviceDestroyed;
};

/**
* Class for managing multiple audio devices.
*/
class FAudioDeviceManager
{
public:

	/**
	* Constructor
	*/
	ENGINE_API FAudioDeviceManager();

	/**
	* Destructor
	*/
	ENGINE_API ~FAudioDeviceManager();

	/** Returns the handle to the main audio device. */
	const FAudioDeviceHandle & GetMainAudioDeviceHandle() const { return MainAudioDeviceHandle; }
	FAudioDevice* GetMainAudioDeviceRaw() const { return MainAudioDeviceHandle.GetAudioDevice(); }
	Audio::FDeviceId GetMainAudioDeviceID() const { return MainAudioDeviceHandle.GetDeviceID(); }

	static ENGINE_API FAudioDevice* GetAudioDeviceFromWorldContext(const UObject* WorldContextObject);
	static ENGINE_API Audio::FMixerDevice* GetAudioMixerDeviceFromWorldContext(const UObject* WorldContextObject);

	/**
	 * returns the currently used audio device module for this platform.
	 * returns nullptr if Initialize() has not been called yet.
	 */
	ENGINE_API IAudioDeviceModule* GetAudioDeviceModule();

	ENGINE_API FAudioDeviceParams GetDefaultParamsForNewWorld();

	/**
	* Creates or requests an audio device instance internally and returns a
	* handle to the audio device.
	* This audio device is guaranteed to be alive as long as the returned handle is in scope.
	*/
	ENGINE_API FAudioDeviceHandle RequestAudioDevice(const FAudioDeviceParams& InParams);

	/**
	* Returns whether the audio device handle is valid (i.e. points to
	* an actual audio device instance)
	*/
	ENGINE_API bool IsValidAudioDevice(Audio::FDeviceId DeviceID) const;

	/**
	 * Returns a strong handle to the audio device associated with the given device ID.
	 * if the device ID is invalid returns an invalid, zeroed handle.
	 */
	ENGINE_API FAudioDeviceHandle GetAudioDevice(Audio::FDeviceId InDeviceID);

	/**
	 * Returns a raw ptr to the audio device associated with the handle. If the
	 * handle is invalid then a NULL device ptr will be returned.
	 */
	ENGINE_API FAudioDevice* GetAudioDeviceRaw(Audio::FDeviceId InDeviceID);
	ENGINE_API const FAudioDevice* GetAudioDeviceRaw(Audio::FDeviceId InDeviceID) const;

	/**
	  Sets the device associated with the given world.
	  */
	ENGINE_API void SetAudioDevice(UWorld& InWorld, Audio::FDeviceId InDeviceID);

	/**
	 * Pre-initializes the Manager.
	 * This starts up and initializes the manager early without need for the Engine to be available, so that 
	 * we can load essential services ahead of Engine Startup.
	 * @return True if success, false otherwise
	 */
	static ENGINE_API bool PreInitialize();

	/**
	 * Initializes the Manager.
	 * NOTE: If the manager hasn't been PreInitialized, that stage will be performed first.
	 * @return True if success, false otherwise
	 */
	static ENGINE_API bool Initialize();

	/**
	 * Returns the AudioDeviceManager Singleton
	 * @return The Instance instance or nullptr if the device has not been *fully* initialized.
	 */
	static ENGINE_API FAudioDeviceManager* Get();

	/**
	 * Shuts down the AudioDeviceManager and all AudioDevices.
	 */
	static ENGINE_API void Shutdown();

	/** Creates the main audio device. */
	ENGINE_API bool CreateMainAudioDevice();

	/**
	* Returns a ptr to the active audio device. If there is no active
	* device then it will return the main audio device.
	*/
	ENGINE_API FAudioDeviceHandle GetActiveAudioDevice();

	/** Returns the current number of active audio devices. */
	ENGINE_API uint8 GetNumActiveAudioDevices() const;

	/** Returns the number of worlds (e.g. PIE viewports) using the main audio device. */
	ENGINE_API uint8 GetNumMainAudioDeviceWorlds() const;

	/** Updates all active audio devices */
	ENGINE_API void UpdateActiveAudioDevices(bool bGameTicking);

	/** Iterates over all managed audio devices */
	ENGINE_API void IterateOverAllDevices(TUniqueFunction<void(Audio::FDeviceId, FAudioDevice*)> ForEachDevice);
	ENGINE_API void IterateOverAllDevices(TUniqueFunction<void(Audio::FDeviceId, const FAudioDevice*)> ForEachDevice) const;

	/** Tracks objects in the active audio devices. */
	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/** Stops sounds using the given resource on all audio devices. */
	ENGINE_API void StopSoundsUsingResource(class USoundWave* InSoundWave, TArray<UAudioComponent*>* StoppedComponents = nullptr);

	/** Registers the Sound Class for all active devices. */
	ENGINE_API void RegisterSoundClass(USoundClass* SoundClass);

	/** Unregisters the Sound Class for all active devices. */
	ENGINE_API void UnregisterSoundClass(USoundClass* SoundClass);

	/** Registers the world with the provided device Id */
	ENGINE_API void RegisterWorld(UWorld* InWorld, Audio::FDeviceId DeviceId);

	/** Unregisters the world from the provided device Id */
	ENGINE_API void UnregisterWorld(UWorld* InWorld, Audio::FDeviceId DeviceId);

	/** Initializes the sound class for all active devices. */
	ENGINE_API void InitSoundClasses();

	/** Registers the Sound Mix for all active devices. */
	ENGINE_API void RegisterSoundSubmix(USoundSubmixBase* SoundSubmix);

	/** Registers the Sound Mix for all active devices. */
	ENGINE_API void UnregisterSoundSubmix(const USoundSubmixBase* SoundSubmix);

	/** Initializes the sound mixes for all active devices. */
	ENGINE_API void InitSoundSubmixes();

	/** Initialize all sound effect presets. */
	UE_DEPRECATED(5.4, "Will be removed in upcoming versions of this code")
	ENGINE_API void InitSoundEffectPresets();

	/** Updates source effect chain on all sources currently using the source effect chain. */
	ENGINE_API void UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails);

	/** Updates this submix for any changes made. Broadcasts to all submix instances. */
	ENGINE_API void UpdateSubmix(USoundSubmixBase* SoundSubmix);

	/** Sets which audio device is the active audio device. */
	ENGINE_API void SetActiveDevice(uint32 InAudioDeviceHandle);

	/** Sets an audio device to be solo'd */
	ENGINE_API void SetSoloDevice(Audio::FDeviceId InAudioDeviceHandle);

	UE_DEPRECATED(5.5, "This is no longer used.  Existing calls may be deleted")
	ENGINE_API void TrackResource(USoundWave* SoundWave, FSoundBuffer* Buffer);
	/** Frees the given sound wave resource from the device manager */
	ENGINE_API void FreeResource(USoundWave* SoundWave);

	UE_DEPRECATED(5.5, "This is no longer used.  Existing calls may be deleted")
	ENGINE_API void FreeBufferResource(FSoundBuffer* SoundBuffer);

	UE_DEPRECATED(5.5, "This is no longer used.  Existing calls may be deleted")
	ENGINE_API void StopSourcesUsingBuffer(FSoundBuffer* Buffer);

	UE_DEPRECATED(5.5, "This is no longer used.  Existing calls may be deleted")
	ENGINE_API FSoundBuffer* GetSoundBufferForResourceID(uint32 ResourceID);

	/** Removes the sound buffer for the given resource id */
	UE_DEPRECATED(5.5, "This is no longer used.  Existing calls may be deleted")
	ENGINE_API void RemoveSoundBufferForResourceID(uint32 ResourceID);

	/** Removes sound mix from all audio devices */
	ENGINE_API void RemoveSoundMix(USoundMix* SoundMix);

	/** Toggles playing audio for all active PIE sessions (and all devices). */
	ENGINE_API void TogglePlayAllDeviceAudio();

	/** Gets whether or not all devices should play their audio. */
	bool IsPlayAllDeviceAudio() const { return bPlayAllDeviceAudio; }

	/** Gets whether or not non-realtime devices should play their audio. */
	ENGINE_API bool IsAlwaysPlayNonRealtimeDeviceAudio() const;

	/** Is debug visualization of 3d sounds enabled */
	ENGINE_API bool IsVisualizeDebug3dEnabled() const;

	/** Toggles 3d visualization of 3d sounds on/off */
	ENGINE_API void ToggleVisualize3dDebug();

	/** Reset all sound cue trims */
	ENGINE_API void ResetAllDynamicSoundVolumes();

	/** Get, reset, or set a sound cue trim */
	ENGINE_API float GetDynamicSoundVolume(ESoundType SoundType,  const FName& SoundName) const;
	ENGINE_API void ResetDynamicSoundVolume(ESoundType SoundType, const FName& SoundName);
	ENGINE_API void SetDynamicSoundVolume(ESoundType SoundType, const FName& SoundName, float Volume);

#if ENABLE_AUDIO_DEBUG
	/** Get the audio debugger instance */
	ENGINE_API Audio::FAudioDebugger& GetDebugger();
	ENGINE_API const Audio::FAudioDebugger& GetDebugger() const;
#endif // ENABLE_AUDIO_DEBUG

public:

	/** Array of all created buffers */
	UE_DEPRECATED(5.5, "This is no longer used and should not be accessed directly.")
	TArray<FSoundBuffer*>			Buffers;
	
	/** Look up associating a USoundWave's resource ID with sound buffers	*/
	UE_DEPRECATED(5.5, "This is no longer used and should not be accessed directly.")
	TMap<int32, FSoundBuffer*>	WaveBufferMap;

	/** Returns all the audio devices managed by device manager. */
	ENGINE_API TArray<FAudioDevice*> GetAudioDevices() const;

	ENGINE_API TArray<UWorld*> GetWorldsUsingAudioDevice(const Audio::FDeviceId& InID) const;

#if INSTRUMENT_AUDIODEVICE_HANDLES
	ENGINE_API void AddStackWalkForContainer(Audio::FDeviceId InId, uint32 StackWalkID, FString&& InStackWalk);
	ENGINE_API void RemoveStackWalkForContainer(Audio::FDeviceId InId, uint32 StackWalkID);
#endif

	ENGINE_API void LogListOfAudioDevices();

	ENGINE_API Audio::FAudioFormatSettings& GetAudioFormatSettings() const;

	ENGINE_API static void EnableAggregateDeviceSupport(const bool bInEnableAggregateDevice);
	ENGINE_API static bool IsAggregateDeviceSupportEnabled();

private:

#if ENABLE_AUDIO_DEBUG
	/** Instance of audio debugger shared across audio devices */ 
	TUniquePtr<Audio::FAudioDebugger> AudioDebugger;
#endif // ENABLE_AUDIO_DEBUG

	TPimplPtr<Audio::FAudioFormatSettings> AudioFormatSettings;
	TArray<TPimplPtr<FSimpleAudioInfoFactory>> EngineFormats;

	/** True if aggregate devices are supported. Static so it can be set prior to initialization of AudioDeviceManager singleton. */
	static bool bEnableAggregateDeviceSupport;

	/**
	 * Conditionally Creates the AudioDeviceManager
	 * @return New Instance or Existing Singleton, or nullptr on failure (which can happen if audio is disabled).
	 */
	static FAudioDeviceManager* GetOrCreate();

	/** Creates a handle given the index and a generation value. */
	ENGINE_API uint32 GetNewDeviceID();

	/** Load audio device module. */
	ENGINE_API bool LoadDefaultAudioDeviceModule();

	ENGINE_API FAudioDeviceHandle CreateNewDevice(const FAudioDeviceParams& InParams);

	// Called exclusively by the FAudioDeviceHandle copy constructor and assignment operators:
	ENGINE_API void IncrementDevice(Audio::FDeviceId DeviceID);

	// Called exclusively by the FAudioDeviceHandle dtor.
	ENGINE_API void DecrementDevice(Audio::FDeviceId DeviceID, UWorld* InWorld);

	/** Application enters background handler */
	ENGINE_API void AppWillEnterBackground();
	
	ENGINE_API void RegisterAudioInfoFactories();

	/**
	 * Queries if the device manager is initalized.
	 * @return true if initialized, false otherwise.
	 */
	bool IsInitialized() const { return InitPhase == EInitPhase::Initialized; }	

	/**
	 * Pre-initializes the Manager
	 * This starts up and intializes the manager early without need for the Engine to be available, so that 
	 * we can load essential services ahead of Engine Startup.
	 * This is optional in the flow, as InitializeManager will also call this if required.
	 * @return True if success, false otherwise
	 */
	ENGINE_API bool PreInitializeManager();

	/**
	 * Initalizes the Manager.
	 * NOTE: If PreInitializeManager has not been called already, a PreInitializeManager call is made first.
	 * @return True if success, false otherwise
	 */
	ENGINE_API bool InitializeManager();

	/** Audio device module which creates audio devices. */
	IAudioDeviceModule* AudioDeviceModule;

	/** The audio mixer module name. This is the audio mixer module name to use. E.g. AudioMixerXAudio2 */
	FString AudioMixerModuleName;

	/** Handle to the main audio device. */
	FAudioDeviceHandle MainAudioDeviceHandle;

	struct FAudioDeviceContainer
	{
		// Singularly owned device.
		// Could be a TUniquePtr if FAudioDevice was not an incomplete type here.
		FAudioDevice* Device;

		// Ref count of FAudioDeviceHandles referencing this device.
		int32 NumberOfHandlesToThisDevice;

		/** Worlds that have been registered to this device. */
		TArray<UWorld*> WorldsUsingThisDevice;

		/** Whether this device can be shared. */
		EAudioDeviceScope Scope;

		/** Whether this audio device is realtime or not. */
		bool bIsNonRealtime;

		/** Module this was created with. If nullptr, this device was created with the default module. */
		IAudioDeviceModule* SpecifiedModule;

		FAudioDeviceContainer(const FAudioDeviceParams& InParams, Audio::FDeviceId InDeviceID, FAudioDeviceManager* DeviceManager);
		~FAudioDeviceContainer();

		FAudioDeviceContainer(const FAudioDeviceContainer& Other)
		{
			// We explicitly enforce that we invoke the move instructor.
			// If this was hit, you likely need to call either Devices.Emplace(args) or Devices.Add(DeviceID, MoveTemp(Container));
			checkNoEntry();
		}

		FAudioDeviceContainer(FAudioDeviceContainer&& Other);

#if INSTRUMENT_AUDIODEVICE_HANDLES
		TMap<uint32, FString> HandleCreationStackWalks;
#endif

	private:
		FAudioDeviceContainer();
	};

	ENGINE_API FAudioDeviceHandle BuildNewHandle(FAudioDeviceContainer&Container, Audio::FDeviceId DeviceID, const FAudioDeviceParams &InParams);

	/**
	 * This function is used to check if we can use an existing audio device.
	 */
	static ENGINE_API bool CanUseAudioDevice(const FAudioDeviceParams& InParams, const FAudioDeviceContainer& InContainer);

#if INSTRUMENT_AUDIODEVICE_HANDLES
	static ENGINE_API uint32 CreateUniqueStackWalkID();
#endif

	/**
	* Bank of audio devices. Will increase in size as we create new audio devices,
	*/
	TMap<Audio::FDeviceId, FAudioDeviceContainer> Devices;
	mutable FCriticalSection DeviceMapCriticalSection;

	/* Counter used by GetNewDeviceID() to generate a unique ID for a given audio device. */
	uint32 DeviceIDCounter;

	/** Next resource ID to assign out to a wave/buffer */
	int32 NextResourceID;

	/** Which audio device is solo'd */
	Audio::FDeviceId SoloDeviceHandle;

	/** Which audio device is currently active */
	Audio::FDeviceId ActiveAudioDeviceID;

	/** Dynamic volume map */
	TMap<TTuple<ESoundType, FName>, float> DynamicSoundVolumes;

	/** Whether or not to play all audio in all active audio devices. */
	bool bPlayAllDeviceAudio;

	/** Audio Fence to ensure that we don't allow the audio thread to drift never endingly behind. */
	FAudioCommandFence SyncFence;

	/**
	 * Simple state machine of the init state of the manager.
	 */
	enum class EInitPhase : uint32
	{
		Constructed,
		PreInitialized,
		Initialized
	};
	EInitPhase InitPhase = EInitPhase::Constructed;
	
	friend class FAudioDeviceHandle;
	static ENGINE_API FAudioDeviceManager* Singleton;
};
