// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/** Simple class that wraps the math involved with interpolating a parameter
  * over time based on audio device update time.
  */
class FDynamicParameter
{
public:
	ENGINE_API explicit FDynamicParameter(float Value);

	ENGINE_API void Set(float Value, float InDuration);
	ENGINE_API void Update(float DeltaTime);

	bool IsDone() const
	{
		return CurrTimeSec >= DurationSec;
	}
	float GetValue() const
	{
		return CurrValue;
	}
	float GetTargetValue() const
	{
		return TargetValue;
	}

private:
	float CurrValue;
	float StartValue;
	float DeltaValue;
	float CurrTimeSec;
	float DurationSec;
	float LastTime;
	float TargetValue;
};
