// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioEffect.h: Unreal base audio.
=============================================================================*/

#pragma once

//#include "AudioDevice.h"
#include "CoreMinimal.h"
#include "Sound/AudioVolume.h"
#include "Sound/ReverbSettings.h"
#include "Sound/SoundMix.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "AudioEffect.generated.h"

class FAudioDevice;
class FSoundSource;
class UReverbEffect;


USTRUCT()
struct FAudioReverbEffect : public FAudioEffectParameters
{
	GENERATED_BODY()

public:
	/** Sets the default values for a reverb effect */
	ENGINE_API FAudioReverbEffect();

	/** Sets the platform agnostic parameters */
	ENGINE_API FAudioReverbEffect
	(
		float InRoom,
		float InRoomHF,
		float InRoomRolloffFactor,
		float InDecayTime,
		float InDecayHFRatio,
		float InReflections,
		float InReflectionsDelay,
		float InReverb,
		float InReverbDelay,
		float InDiffusion,
		float InDensity,
		float InAirAbsorption,
		bool bInBypassEarlyReflections,
		bool bInBypassLateReflections
	);

	ENGINE_API FAudioReverbEffect& operator=(class UReverbEffect* InReverbEffect);

	/** Interpolates between Start and End reverb effect settings, storing results locally and returning if interpolation is complete */
	ENGINE_API bool Interpolate(const FAudioEffectParameters& InStart, const FAudioEffectParameters& InEnd);

	ENGINE_API void PrintSettings() const override;

	/** Time when this reverb was initiated or completed faded in */
	double		Time;

	/** Overall volume of effect */
	float		Volume;					// 0.0 to 1.0


	/** Platform agnostic parameters that define a reverb effect. Min < Default < Max */
	float		Density;				// 0.0 < 1.0 < 1.0
	float		Diffusion;				// 0.0 < 1.0 < 1.0
	float		Gain;					// 0.0 < 0.32 < 1.0 
	float		GainHF;					// 0.0 < 0.89 < 1.0
	float		DecayTime;				// 0.1 < 1.49 < 20.0	Seconds
	float		DecayHFRatio;			// 0.1 < 0.83 < 2.0
	float		ReflectionsGain;		// 0.0 < 0.05 < 3.16
	float		ReflectionsDelay;		// 0.0 < 0.007 < 0.3	Seconds
	float		LateGain;				// 0.0 < 1.26 < 10.0
	float		LateDelay;				// 0.0 < 0.011 < 0.1	Seconds
	float		AirAbsorptionGainHF;	// 0.892 < 0.994 < 1.0
	float		RoomRolloffFactor;		// 0.0 < 0.0 < 10.0

	bool bBypassEarlyReflections;
	bool bBypassLateReflections;
};

/**
 * Used to store and manipulate parameters related to a radio effect. 
 */
class FAudioRadioEffect : public FAudioEffectParameters
{
public:
	/**
	 * Constructor (default).
	 */
	FAudioRadioEffect(void)
	{
	}
};

/** 
 * Manager class to handle the interface to various audio effects
 */
class FAudioEffectsManager
{
public:
	ENGINE_API FAudioEffectsManager( FAudioDevice* Device );

	virtual ~FAudioEffectsManager( void ) = default; 

	void AddReferencedObjects( FReferenceCollector& Collector );

	/** 
	 * Feed in new settings to the audio effect system
	 */
	void Update( void );

	/** 
	 * Engine hook to handle setting and fading in of reverb effects
	 */
	void SetReverbSettings(const FReverbSettings& ReverbSettings, bool bForce = false);

	/** 
	 * Engine hook to handle setting and fading in of EQ effects and group ducking
	 *
	 * @param Mix The SoundMix we want to switch to.
	 * @param bIgnorePriority Whether EQPriority should be ignored to force mix change.
	 */
	void SetMixSettings(USoundMix* Mix, bool bIgnorePriority = false, bool bForce = false);

	/**
	 * Clears the current SoundMix and any EQ settings it has applied
	 */
	void ClearMixSettings();

	/** 
	 * Calls the platform specific code to set the parameters that define reverb
	 */
	virtual void SetReverbEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
	}

	/** 
	 * Calls the platform specific code to set the parameters that define EQ
	 */
	virtual void SetEQEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
	}

	/** 
	 * Calls the platform-specific code to set the parameters that define a radio effect.
	 * 
	 * @param	InEffectParameters	The new parameters for the radio distortion effect. 
	 */
	virtual void SetRadioEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
	}

	/** 
	 * Platform dependent call to init effect data on a sound source
	 */
	virtual void* InitEffect( FSoundSource* Source )
	{
		return( NULL ); 
	}

	/** 
	 * Platform dependent call to update the sound output with new parameters
	 */
	virtual void* UpdateEffect( class FSoundSource* Source ) 
	{ 
		return( NULL ); 
	}

	/** 
	 * Platform dependent call to destroy any effect related data
	 */
	void DestroyEffect( FSoundSource* Source )
	{
	}

	/**
	 * Converts and volume (0.0f to 1.0f) to a deciBel value
	 */
	ENGINE_API int64 VolumeToDeciBels( float Volume );

	/**
	 * Converts and volume (0.0f to 1.0f) to a MilliBel value (a Hundredth of a deciBel)
	 */
	ENGINE_API int64 VolumeToMilliBels( float Volume, int32 MaxMilliBels );

	/** 
	 * Resets all interpolating values to defaults.
	 */
	void ResetInterpolation( void );

	/**
	 * Gets the SoundMix currently controlling EQ.
	 */
	USoundMix* GetCurrentEQMix() const
	{
		return CurrentEQMix;
	}

	UReverbEffect* GetCurrentReverbEffect() const
	{
		return CurrentReverbAsset;
	}

protected:

	/** 
	 * Sets up default reverb and eq settings
	 */
	void InitAudioEffects( void );

	FAudioDevice*			AudioDevice;
	bool					bEffectsInitialised;

	TObjectPtr<UReverbEffect>			CurrentReverbAsset;

	FAudioReverbEffect	SourceReverbEffect;
	FAudioReverbEffect	CurrentReverbEffect;
	FAudioReverbEffect	PrevReverbEffect;
	FAudioReverbEffect	DestinationReverbEffect;

	FReverbSettings 		CurrentReverbSettings;

	USoundMix*				CurrentEQMix;

	FAudioEQEffect			SourceEQEffect;
	FAudioEQEffect			CurrentEQEffect;
	FAudioEQEffect			DestinationEQEffect;

	bool bReverbActive;
	bool bEQActive;
	bool bReverbChanged;
	bool bEQChanged;
};

// end 
