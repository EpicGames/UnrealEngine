// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "IAudioExtensionPlugin.h"

/************************************************************************/
/* Plugin Utilities                                                     */
/************************************************************************/
struct AudioPluginUtilities
{
	static ENGINE_API const FName& GetDefaultModulationPluginName();

	/*
	 * These functions return a pointer to the plugin factory
	 * that matches the plugin name specified in the target
	 * platform's settings.
	 * 
	 * if no matching plugin is found, nullptr is returned.
	 */
	static ENGINE_API FName GetDesiredSpatializationPluginName();
	static ENGINE_API TArray<IAudioSpatializationFactory*> GetSpatialPluginArray();
	static ENGINE_API IAudioSpatializationFactory* GetDesiredSpatializationPlugin();
	static ENGINE_API IAudioSourceDataOverrideFactory* GetDesiredSourceDataOverridePlugin();
	static ENGINE_API IAudioReverbFactory* GetDesiredReverbPlugin();
	static ENGINE_API IAudioOcclusionFactory* GetDesiredOcclusionPlugin();
	static ENGINE_API IAudioModulationFactory* GetDesiredModulationPlugin();

	/** This function returns the name of the plugin specified in the platform settings. */
	static ENGINE_API FString GetDesiredPluginName(EAudioPlugin PluginType);
};
