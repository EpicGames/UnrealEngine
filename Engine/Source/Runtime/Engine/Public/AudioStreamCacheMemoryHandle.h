// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "HAL/Platform.h"

#define UE_API ENGINE_API

/**
 * Class for utilizing memory in the stream cache budget for an unrelated, temporary audio-based feature.
 * Allows us to borrow from budgeted memory for audio features that would otherwise not fit in the overall memory budget.
 *
 * Usage:
 * - Create an instance of this class on an object or subsystem where want to track memory usage
 * - Memory usage will immediately be taken out of the Audio Stream Cache budget on construction of the object
 * - Update memory usage via this class as necessary.
 * - Deleting the instance will automatically reset the memory usage to 0
*/
class FAudioStreamCacheMemoryHandle : public FNoncopyable
{
public:
	UE_API FAudioStreamCacheMemoryHandle(FName InFeatureName, uint64 InMemoryUseInBytes);
	
	UE_API ~FAudioStreamCacheMemoryHandle();
	
	inline uint64 GetMemoryUseInBytes() const { return MemoryUseInBytes; } 
	inline FName GetFeatureName() const { return FeatureName; }

	UE_API void ResetMemoryUseInBytes(uint64 InMemoryUseInBytes);
	
private:
	const FName FeatureName;
	uint64 MemoryUseInBytes;
};

#undef UE_API
