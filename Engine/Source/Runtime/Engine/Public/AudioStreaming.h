// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AudioStreaming.h: Definitions of classes used for audio streaming.
=============================================================================*/

#pragma once

#include "Async/AsyncWork.h"
#include "CoreMinimal.h"
#include "ContentStreaming.h"
#include "HAL/ThreadSafeCounter.h"

class FSoundWaveProxy;

using FSoundWaveProxyPtr = TSharedPtr<FSoundWaveProxy, ESPMode::ThreadSafe>;

/**
 * Async worker to stream audio chunks from the derived data cache. 
 */
class FAsyncStreamDerivedChunkWorker : public FNonAbandonableTask
{
public:
	/** Initialization constructor. */
	FAsyncStreamDerivedChunkWorker(
		const FString& InDerivedDataKey,
		void* InDestChunkData,
		int32 InChunkSize,
		FThreadSafeCounter* InThreadSafeCounter,
		TFunction<void(bool)> InOnLoadComplete
		);
	
	/**
	 * Retrieves the derived chunk from the derived data cache.
	 */
	void DoWork();

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncStreamDerivedChunkWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	/**
	 * Returns true if the streaming mip request failed.
	 */
	bool DidRequestFail() const
	{
		return bRequestFailed;
	}

private:
	/** Key for retrieving chunk data from the derived data cache. */
	FString DerivedDataKey;
	/** The location to which the chunk data should be copied. */
	void* DestChunkData;
	/** The size of the chunk in bytes. */
	int32 ExpectedChunkSize;
	/** true if the chunk data was not present in the derived data cache. */
	bool bRequestFailed;
	/** Thread-safe counter to decrement when data has been copied. */
	FThreadSafeCounter* ThreadSafeCounter;
	/** This function is called when the load is completed */
	TFunction<void(bool)> OnLoadCompleted;
};

/** Async task to stream chunks from the derived data cache. */
typedef FAsyncTask<FAsyncStreamDerivedChunkWorker> FAsyncStreamDerivedChunkTask;

/** Struct used to store results of an async file load. */
using FStreamingWaveData = void*;
struct UE_DEPRECATED(5.5, "this struct relies on deprecated / deleted types, and should not be used") FASyncAudioChunkLoadResult
{
	// Place to safely copy the ptr of a loaded audio chunk when load result is finished
	uint8* DataResults;

	// Actual storage of the loaded audio chunk, will be filled on audio thread.
	FStreamingWaveData* StreamingWaveData;

	// Loaded audio chunk index
	int32 LoadedAudioChunkIndex;

	FASyncAudioChunkLoadResult()
		: DataResults(nullptr)
		, StreamingWaveData(nullptr)
		, LoadedAudioChunkIndex(INDEX_NONE)
	{}
};
