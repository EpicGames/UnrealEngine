// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AudioStreaming.h: Definitions of classes used for audio streaming.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/IndirectArray.h"
#include "Containers/Queue.h"
#include "Stats/Stats.h"
#include "ContentStreaming.h"
#include "Async/AsyncWork.h"
#include "Async/AsyncFileHandle.h"
#include "HAL/ThreadSafeBool.h"
#include "AudioStreaming.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveLoadingBehavior.h"
#include "Templates/DontCopy.h"
#include "Templates/UniqueObj.h"
#include "UObject/ObjectKey.h"


#define DEBUG_STREAM_CACHE !UE_BUILD_SHIPPING

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioStreamCaching, Display, All);

class FAudioStreamCacheMemoryHandle;

// Basic fixed-size LRU cache for retaining chunks of compressed audio data.
class FAudioChunkCache
{
public:
	struct FChunkKey
	{
	public:
		FChunkKey() = default;

		FChunkKey(const FChunkKey& Other) = default;

		FChunkKey(
			  const FName& InSoundWaveName
			, const FGuid& InSoundWaveGuid
			, uint32 InChunkIndex
#if WITH_EDITOR
			, uint32 InChunkRevision = 0
#endif // #if WITH_EDITOR
		);

		FChunkKey(
			  const TSharedRef<const FSoundWaveData>& InSoundWave
			, uint32 InChunkIndex
#if WITH_EDITOR
			, uint32 InChunkRevision = 0
#endif // #if WITH_EDITOR
		);

		FChunkKey& operator=(const FChunkKey& Other) = default;


	public:
		FName SoundWaveName = FName();
		FGuid ObjectKey = FGuid();
		uint32 ChunkIndex = INDEX_NONE;

#if WITH_EDITOR
		// This is used in the editor to invalidate stale compressed chunks.
		uint32 ChunkRevision = 0;
#endif
		bool operator==(const FChunkKey& Other) const;

		/** Hash function */
		friend uint32 GetTypeHash(const FChunkKey& InChunkKey)
		{
			// We can use HashCombineFast since we're hashing FName and they won't give the same result across different engine restart anyway.
			uint32 Hash = HashCombineFast(HashCombineFast(GetTypeHash(InChunkKey.SoundWaveName), InChunkKey.ChunkIndex), GetTypeHash(InChunkKey.ObjectKey));
#if WITH_EDITOR
			Hash = HashCombineFast(Hash, InChunkKey.ChunkRevision);
#endif
			return Hash;
		}
	};

	struct FCacheMissEntry
	{
	public:
		// ctor
		FCacheMissEntry(FName InSoundWaveName, uint32 InChunkIndex)
			: SoundWaveName(InSoundWaveName)
			, ChunkIndex(InChunkIndex)
		{}

		inline bool operator==(const FCacheMissEntry& Other) const
		{
			return (SoundWaveName == Other.SoundWaveName) && (ChunkIndex == Other.ChunkIndex);
		}

		FName SoundWaveName;

		uint32 ChunkIndex;

		/** Hash function */
		friend uint32 GetTypeHash(const FCacheMissEntry& InCacheMissEntry)
		{
			// We can use HashCombineFast since we're hashing FName and they won't give the same result across different engine restart anyway.
			return HashCombineFast(GetTypeHash(InCacheMissEntry.SoundWaveName), InCacheMissEntry.ChunkIndex);
		}
	};

	FAudioChunkCache(uint32 InMaxChunkSize, uint32 NumChunks, uint64 InMemoryLimitInBytes);
	~FAudioChunkCache();

	// Places chunk in cache, or puts this chunk back at the top of the cache if it's already loaded. Returns the static lookup ID of the chunk in the cache on success,
	// or InvalidAudioStreamCacheLookupID on failiure.
	uint64 AddOrTouchChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWavePtr, TFunction<void(EAudioChunkLoadResult) > OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback);

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	uint64 AddOrTouchChunk(const FChunkKey& InKey, const TSharedPtr<FSoundWaveData>& InSoundWavePtr, TFunction<void(EAudioChunkLoadResult) > OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback);

	// Returns the chunk asked for, or an empty TArrayView if that chunk is not loaded.
	// InOutCacheLookupID can optionally be set as a cache offset to use directly rather than searching the cache for a matching chunk.
	// InOutCacheLookupID will be set to the offset the chunk is in the cache, which can be used for faster lookup in the future.
	TArrayView<uint8> GetChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWavePtr, bool bBlockForLoadCompletion, bool bNeededForPlayback, uint64& InOutCacheLookupID);

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	TArrayView<uint8> GetChunk(const FChunkKey& InKey, const TSharedPtr<FSoundWaveData>& InSoundWavePtr, bool bBlockForLoadCompletion, bool bNeededForPlayback, uint64& InOutCacheLookupID);

	// add an additional reference for a chunk.
	void AddNewReferenceToChunk(const FChunkKey& InKey);
	void RemoveReferenceToChunk(const FChunkKey& InKey);

	// Evict all sounds from the cache. Does not account for force inline sounds
	void ClearCache();

	void AddForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& InSoundWaveData); 
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	void AddForceInlineSoundWave(const FSoundWaveProxyPtr& InSoundWave)
	{
		if (ensure(InSoundWave.IsValid()))
		{
			AddForceInlineSoundWave(InSoundWave->GetSoundWaveDataRef());	
		}
	}

	void RemoveForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& InSoundWaveData);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	void RemoveForceInlineSoundWave(const FSoundWaveProxyPtr& InSoundWave)
	{
		if (ensure (InSoundWave.IsValid()))
		{
			RemoveForceInlineSoundWave(InSoundWave->GetSoundWaveDataRef());
		}
	}

	void AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature);

	void RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature);

	// This function will reclaim memory by freeing as many chunks as needed to free BytesToFree.
	// returns the amount of bytes we were actually able to free.
	// It's important to note that this will block any chunk requests.
	uint64 TrimMemory(uint64 BytesToFree, bool bAllowRetainedChunkTrimming);

	// Returns an array of the USoundwaves retaining the least recently used retained chunks in the cache.
	// This can potentially return soundwaves for chunks that are retained by a currently playing sound, if the cache is thrashed enough.
	TArray<FObjectKey> GetLeastRecentlyUsedRetainedSoundWaves(int32 NumSoundWavesToRetrieve);

	// This function will continue to lock until any async file loads are finished.
	void BlockForAllPendingLoads() const;

	// This function will cancel any in-flight loads and wait for their completion.
	void CancelAllPendingLoads();

	// Reports the size of this cache's memory pool, in bytes.
	uint64 ReportCacheSize();

	// Debug tools:
	// Call this to start enqueing reports on any cache misses to a queue.
	// This queue will continue to grow until FlushCacheMissLog is called.
	void BeginLoggingCacheMisses();

	// This will stop enqueueing reports of cache misses.
	void StopLoggingCacheMisses();

	// When called, flushes the entire queue of cache misses that has accumulated
	// And prints them to a formatted 
	FString FlushCacheMissLog();

	// Static helper function to make sure a chunk is withing the bounds of a USoundWave.
	static bool DoesKeyContainValidChunkIndex(const FChunkKey& InKey, const FSoundWaveData& InSoundWaveData);

	// interface with the cache id lookup map
	uint64 GetCacheLookupIDForChunk(const FChunkKey& InChunkKey) const;
	void SetCacheLookupIDForChunk(const FChunkKey& InChunkKey, uint64 InCacheLookupID);

	const int32 MaxChunkSize;

	// This is for debugging purposes only. Prints the elements in the cache from most recently used to least.
	// Returns the dimensions of this debug log so that multiple caches can be tiled across the screen.
	TPair<int, int> DebugDisplayLegacy(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const;

	// Generate a formatted text file for this cache.
	FString DebugPrint();

	// This is for debugging purposes only. Prints the elements in the cache from most recently used to least.
	// Returns the dimensions of this debug log so that multiple caches can be tiled across the screen.

	enum class EDebugDisplayElementTypes
	{
		NumRetainedAndPlaying = 0,
		NumRetained,
		NumPrimedAndPlaying,
		NumPrimed,
		NumRetainedAndPlayingCacheMiss,
		NumRetainedCacheMiss,
		NumPrimedAndPlayingCacheMiss,
		NumPrimedCacheMiss,
		NumLODAndPlaying,
		NumLOD,
		NumTrimmed,
		NumLoadInProgress,
		NumOther,
		Count
	};

	TPair<int, int> DebugDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const;
	TPair<int, int> DebugVisualDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const;
	TPair<int, int> DebugBirdsEyeDisplay(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) const;

	void HandleStarvation(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	void HandleStarvation(const FSoundWaveProxyPtr& SoundWave, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected)
	{
		if (ensure(SoundWave))
		{
			HandleStarvation(SoundWave->GetSoundWaveDataRef(), ChunkIndex, InBytesProduced, InBytesExpected);
		}
	}

	void IncrementCacheOverflowCounter()
	{
		CacheOverflowCount.Increment();
	}

	int32 GetNumberOfCacheOverflows() const
	{
		return CacheOverflowCount.GetValue();
	}

private:

#if DEBUG_STREAM_CACHE
	// This struct lets us breadcrumb debug information.
	struct FCacheElementDebugInfo
	{
		// The total number of chunks in the sound wave.
		int32 NumTotalChunks;

		// Number of times this chunk was requested during its time in the cache.
		int32 NumTimesTouched;

		uint64 TimeLoadStartedCycles = 0;
		uint64 TimeLoadEndedCycles = 0;
		// Amount of time spent loading the audio file.
		float TimeToLoadMs;

		// This is a cumulative moving average of a chunks location before it was 
		float AverageLocationInCacheWhenNeeded;

		// Note the loading behavior of the sound wave that inserted this element into the cache
		ESoundWaveLoadingBehavior LoadingBehavior;
		bool bLoadingBehaviorExternallyOverriden;

		// cached format.
		FName Format;
		float OffsetInSeconds=0.f;

		// if true, 
		bool bWasCacheMiss;
		bool bWasLoadedFromInlineChunk = false;
		bool bWasInlinedButUnloaded = false;
		bool bWaveDestroyed = false;

		FCacheElementDebugInfo()
			: NumTotalChunks(0)
			, NumTimesTouched(0)
			, TimeLoadStartedCycles(0)
			, TimeToLoadMs(0.0)
			, AverageLocationInCacheWhenNeeded(0.0f)
			, LoadingBehavior(ESoundWaveLoadingBehavior::Uninitialized)
			, bWasCacheMiss(false)
		{
		}

		void Reset()
		{
			TimeLoadEndedCycles = 0;
			NumTotalChunks = 0;
			NumTimesTouched = 0;
			TimeLoadStartedCycles = 0;
			TimeToLoadMs = 0.0f;
			LoadingBehavior = ESoundWaveLoadingBehavior::Uninitialized;
			bWasCacheMiss = false;
			AverageLocationInCacheWhenNeeded = 0.0f;
			bWasLoadedFromInlineChunk = false;
			bWasInlinedButUnloaded = false;
			bWaveDestroyed = false;
			Format = {};
		}

	};
#endif


	// counter for the number of times this cache has overflown
	FThreadSafeCounter CacheOverflowCount;


	// Struct containing a single element in our LRU Cache.  
	struct FCacheElement
	{
		FChunkKey Key;
		uint8* ChunkData;
		uint32 ChunkDataSize;
		FCacheElement* MoreRecentElement;
		FCacheElement* LessRecentElement;
		uint64 CacheLookupID = InvalidAudioStreamCacheLookupID;

		TWeakPtr<const FSoundWaveData, ESPMode::ThreadSafe> SoundWaveWeakPtr;

		FThreadSafeBool bIsLoaded;

#if !UE_BUILD_SHIPPING
		// if the chunk belongs to a sound that wants more logging. This should be under LogAudio so its'
		// with the other extra logging.
		bool bWantsExtraLogging = false;
#endif
		bool WantsFilteredLogging() const
		{
#if !UE_BUILD_SHIPPING		
			return bWantsExtraLogging;
#else
			return false;
#endif //!UE_BUILD_SHIPPING
		}

		// How many disparate consumers have called GetLoadedChunk.
		FThreadSafeCounter NumConsumers;

#if WITH_EDITORONLY_DATA
		TUniquePtr<FAsyncStreamDerivedChunkTask> DDCTask;
#endif

		FBulkDataBatchRequest Read;
		EAsyncIOPriorityAndFlags Priority = AIOP_MIN;

#if DEBUG_STREAM_CACHE
		FCacheElementDebugInfo DebugInfo;
#endif

		FCacheElement(uint32 InCacheIndex)
			: ChunkData(nullptr)
			, ChunkDataSize(0)
			, MoreRecentElement(nullptr)
			, LessRecentElement(nullptr)
			, CacheLookupID(InCacheIndex)
			, bIsLoaded(false)
		{
		}

		void WaitForAsyncLoadCompletion(bool bCancel);
	
		bool IsLoadInProgress() const
		{
			return !bIsLoaded;
		}

		bool IsInUse() const
		{
			return NumConsumers.GetValue() > 0;
		}

#if DEBUG_STREAM_CACHE
		bool IsBeingPlayed() const;
		void UpdateDebugInfoLoadingBehavior();
#endif

		bool CanEvictChunk() const
		{
			return !IsInUse() && !IsLoadInProgress();
		}

		uint32 GetNumChunks() const;
		FStreamedAudioChunk* GetChunk(uint32 InChunkIndex) const;

#if WITH_EDITOR
		bool IsChunkStale();
#endif
		void ReleaseRetainedAudioOnSoundWave();
		bool IsSoundWaveRetainingAudio() const;

		~FCacheElement()
		{
			WaitForAsyncLoadCompletion(true);

			if(NumConsumers.GetValue() != 0)
			{
				/** TODO: This is failing because of `Key.GetLoadingBehavior()`. We could get this from the FSoundWaveData or
				 * store it, or ignore it here. 
				UE_LOGF(LogAudioStreamCaching, Error, "Tried to destroy streaming cache while the cached data was in use! (Sound: %ls - Loading Behavior: %ls - Chunk Index: %i)"
					, *Key.SoundWaveName.ToString(), EnumToString(Key.GetLoadingBehavior()), Key.ChunkIndex);
				*/
			}
			else if (ChunkData)
			{
				FMemory::Free(ChunkData);
				ChunkData = nullptr;
			}

		}
	};

	// Our actual memory pool.
	TArray<FCacheElement> CachePool;
	FCacheElement* MostRecentElement;
	FCacheElement* LeastRecentElement;

	// This is incremented on every call of InsertChunk until we hit CachePool.Num() or MemoryCounterBytes hits MemoryLimitBytes.
	int32 ChunksInUse;

	// This counter is used to start evicting chunks before we hit CachePool.Num().
	TUniqueObj<TAtomic<uint64>> MemoryCounterBytes;
	uint64 MemoryLimitBytes;

	TUniqueObj<TAtomic<uint64>> ForceInlineMemoryCounterBytes;

	TUniqueObj<TAtomic<uint64>> FeatureMemoryCounterBytes;

	// Number of async load operations we have currently in flight.
	FThreadSafeCounter NumberOfLoadsInFlight;

	// Time begin play came in. (so we can validate load times against load screen/vs level loads).
	uint64 TimeOfBeginPlayCycles=0;

	// Critical section: only used when we are modifying element positions in the cache. This only happens in TouchElement, EvictLeastRecentChunk, and TrimMemory.
	// Individual cache elements should be thread safe to access.
	mutable TDontCopy<FCriticalSection> CacheMutationCriticalSection;

	// Map that USoundWaves, FSoundWaveProxys, FChunkKeys, and FAudioChunkHandles can use to
	// quickly lookup where their chunks are currently stored in the cache
	TMap<FChunkKey, uint64> CacheLookupIdMap;

	struct FSoundWaveMemoryTracker
	{
		int32 RefCount = 0;
		int64 MemoryCount = 0;
	};
	// A map to look up the number of times a sound wave has been added for memory tracking
	// as well as the memory added for the "latest" addition
	TMap<const FSoundWaveData*, FSoundWaveMemoryTracker> SoundWaveTracker;
	TDontCopy<FCriticalSection> SoundWaveMemoryTrackerCritSec;

	// This struct is used for logging cache misses.
	struct FCacheMissInfo
	{
		FName SoundWaveName;
		uint32 ChunkIndex;
		uint32 TotalChunksInWave;
		bool bBlockedForLoad;
	};

	// This queue is pushed to anytime GetChunk fails to get the chunk. and bLogCacheMisses is true.
	TUniqueObj<TQueue<FCacheMissInfo>> CacheMissQueue;

	// This is set to true when BeginLoggingCacheMisses is called. 
	bool bLogCacheMisses;

	uint64 GetCurrentMemoryUsageBytes() const { return *MemoryCounterBytes + *ForceInlineMemoryCounterBytes + *FeatureMemoryCounterBytes; }

	// Returns cached element if it exists in our cache, nullptr otherwise.
	// If the index of the element is already known, it can be used here to avoid searching the cache.
	FCacheElement* FindElementForKey(const FChunkKey& InKey);
	FCacheElement* LinearSearchCacheForElement(const FChunkKey& InKey);
	FCacheElement* LinearSearchChunkArrayForElement(const FChunkKey& InKey);

	// Puts this element at the front of the linked list.
	void TouchElement(FCacheElement* InElement);

	// Inserts a new element into the cache, potentially evicting the oldest element in the cache.
	FCacheElement* InsertChunk(const FChunkKey& InKey, const TSharedRef<const FSoundWaveData>& InSoundWaveWeakPtr);

	// This is called once we have more than one chunk in our cache:
	void SetUpLeastRecentChunk();

	// This is called in InsertChunk. it determines whether we should add a new chunk at the tail of the linked list or
	// evict the least recent chunk.
	bool ShouldAddNewChunk() const;

	// Returns the least recent chunk and fixes up the linked list accordingly.
	FCacheElement* EvictLeastRecentChunk(bool bBlockForPendingLoads = false);


	void KickOffAsyncLoad(FCacheElement* CacheElement, const FChunkKey& InKey, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type CallbackThread, bool bNeededForPlayback);
	EAsyncIOPriorityAndFlags GetAsyncPriorityForChunk(const FChunkKey& InKey, bool bNeededForPlayback);

	// Calls OnLoadCompleted on current thread if CallbackThread == ENamedThreads::AnyThread, and dispatchs an async task on a named thread otherwise.
	static void ExecuteOnLoadCompleteCallback(EAudioChunkLoadResult Result, const TFunction<void(EAudioChunkLoadResult)>& OnLoadCompleted, const ENamedThreads::Type& CallbackThread);
};

// This is used to sort the cache array from smallest chunk size to biggest.
inline bool operator< (const FAudioChunkCache& Element1, const FAudioChunkCache& Element2)
{
	return Element1.MaxChunkSize < Element2.MaxChunkSize;
}

struct FCachedAudioStreamingManagerParams
{
	struct FCacheDimensions
	{
		// The max size, in bytes, of a single chunk of compressed audio.
		// During cook, compressed audio assets will be chunked based on this amount.
		int32 MaxChunkSize;

		// The maximum number of elements stored in a single cache before it is evicted.
		// At runtime, this will be clamped to ensure that it is greater than the amount of
		// sources that can be playing simultaneously.
		int32 NumElements;

		// The maximum number of elements stored in a single cache before it is evicted.
		// At runtime, this will be clamped to ensure that it is greater than the amount of
		// sources that can be playing simultaneously.
		uint64 MaxMemoryInBytes;
	};

	// Most use cases will only use a single cache, but applications can optionally
	// use multiple LRU caches to reduce churn for specific types of sounds.
	// For example, an application can have
	// a cache for short sounds with room for many elements, and a separate cache
	// for longer sounds with fewer elements.
	TArray<FCacheDimensions> Caches;
};

/**
 * This implementation of the audio streaming manager uses an internal LRU cache (or in more advanced applications, a bank of parallel LRU caches)
 */
struct FCachedAudioStreamingManager : public IAudioStreamingManager
{
public:
	/** Constructor, initializing all members */
	FCachedAudioStreamingManager(const FCachedAudioStreamingManagerParams& InitParams);

	virtual ~FCachedAudioStreamingManager();

	// IStreamingManager interface
	virtual void UpdateResourceStreaming( float DeltaTime, bool bProcessEverything=false ) override;
	virtual int32 BlockTillAllRequestsFinished( float TimeLimit = 0.0f, bool bLogResults = false ) override;
	virtual void CancelForcedResources() override;
	virtual void NotifyLevelChange() override;
	virtual void SetDisregardWorldResourcesForFrames( int32 NumFrames ) override;
	virtual void AddLevel( class ULevel* Level ) override;
	virtual void RemoveLevel( class ULevel* Level ) override;
	virtual void NotifyLevelOffset( class ULevel* Level, const FVector& Offset ) override;
	// End IStreamingManager interface

	// IAudioStreamingManager interface
	virtual void AddForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWaveData) override;
	virtual void RemoveForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWaveData) override;
	virtual bool RequestChunk(const TSharedRef<const FSoundWaveData>& SoundWaveData, uint32 ChunkIndex, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted, ENamedThreads::Type ThreadToCallOnLoadCompletedOn, bool bForImmediatePlayback = false) override;
	virtual FAudioChunkHandle GetLoadedChunk(const TSharedRef<const FSoundWaveData>& SoundWaveData, uint32 ChunkIndex, bool bBlockForLoad = false, bool bForImmediatePlayback = false) const override;
	virtual uint64 TrimMemory(uint64 NumBytesToFree) override;
	virtual int32 RenderStatAudioStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) override;
	virtual FString GenerateMemoryReport() override;
	virtual void SetProfilingMode(bool bEnabled) override;
	// End IAudioStreamingManager interface

	/**
	 * Returns the next chunk to kick off a load for, or INDEX_NONE if there is only one chunk to cache.
	 * @param InSoundWaveData Wave data
	 * @param InCurrentChunkIndex Current index to move forwards of
	 * @return Chunk Index or INDEX_NONE if there is only one chunk 
	 */
	static int32 GetNextChunkIndex(const TSharedRef<const FSoundWaveData>& InSoundWaveData, const uint32 InCurrentChunkIndex);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	static int32 GetNextChunkIndex(const FSoundWaveProxyPtr& InSoundWave, const uint32 InCurrentChunkIndex)
	{
		if (ensure(InSoundWave.IsValid()))
		{
			return GetNextChunkIndex(InSoundWave->GetSoundWaveDataRef(), InCurrentChunkIndex);
		}
		
		return INDEX_NONE;
	}
	
protected:

	// These are used to reference count consumers of audio chunks.
	virtual void AddReferenceToChunk(const FAudioChunkHandle& InHandle) override;
	virtual void RemoveReferenceToChunk(const FAudioChunkHandle& InHandle) override;

	// These are used to update the memory count of Memory Counted Features
	virtual void AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) override;
	virtual void RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) override;

	virtual void HandleStarvation(const TSharedRef<const FSoundWaveData>& SoundWaveData, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected) override;
	
	/**
	 * Returns which cache this sound wave should be in,
	 * based on the size of this sound wave's chunk,
	 * or nullptr if MemoryLoadOnDemand is disabled.
	 */
	FAudioChunkCache* GetCacheForWave(const TSharedRef<const FSoundWaveData>& InSoundWave) const;
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	FAudioChunkCache* GetCacheForWave(const FSoundWaveProxyPtr& InSoundWave) const
	{
		if (ensure(InSoundWave.IsValid()))
		{
			return GetCacheForWave(InSoundWave->GetSoundWaveDataRef());
		}
		
		return nullptr;
	}
	FAudioChunkCache* GetCacheForChunkSize(uint32 InChunkSize) const;

	/** Audio chunk caches. These are set up on initialization. */
	TArray<FAudioChunkCache> CacheArray;
};
