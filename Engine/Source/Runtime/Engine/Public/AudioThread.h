// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioThread.h: Rendering thread definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

#include "Tasks/Pipe.h"

////////////////////////////////////
// Audio thread API
// (the naming is outdated as the thread was replaced by piped tasks)
////////////////////////////////////

DECLARE_STATS_GROUP(TEXT("Audio Thread Commands"), STATGROUP_AudioThreadCommands, STATCAT_Advanced);

class FAudioThread
{
private:
	/** Whether to run with an audio thread */
	static bool bUseThreadedAudio;

	static TUniquePtr<UE::Tasks::FTaskEvent> ResumeEvent;
	static int32 SuspendCount;  // accessed only from GT

	// GC callback handles
	static FDelegateHandle PreGC;
	static FDelegateHandle PostGC;
	static FDelegateHandle PreGCDestroy;
	static FDelegateHandle PostGCDestroy;

	/** Stat id of the currently executing audio thread command. */
	static TStatId CurrentAudioThreadStatId;
	static TStatId LongestAudioThreadStatId;
	static double LongestAudioThreadTimeMsec;
	static FCriticalSection CurrentAudioThreadStatIdCS;

	/** Sets the current audio thread stat id. */
	static void SetCurrentAudioThreadStatId(TStatId InStatId);
	static void SetLongestTimeAndId(TStatId NewLongestId, double LongestTimeMsec);
	static double GetCurrentLongestTime() { return LongestAudioThreadTimeMsec; }

	// a helper to apply stats and optional additional logging
	static TUniqueFunction<void()> GetCommandWrapper(TUniqueFunction<void()> InFunction, const TStatId InStatId);

public:
	/** Starts the audio thread. */
	static ENGINE_API void StartAudioThread();

	/** Stops the audio thread. */
	static ENGINE_API void StopAudioThread();

	/** Execute a command on the audio thread. If it's safe the command will execute immediately. */
	static ENGINE_API void RunCommandOnAudioThread(TUniqueFunction<void()> InFunction, const TStatId InStatId = TStatId());

	/** Processes all enqueued audio thread commands. */
	static ENGINE_API void ProcessAllCommands();

	/** Execute a (presumably audio) command on the game thread. If GIsAudioThreadRunning is false the command will execute immediately */
	static ENGINE_API void RunCommandOnGameThread(TUniqueFunction<void()> InFunction, const TStatId InStatId = TStatId());

	static ENGINE_API void SetUseThreadedAudio(bool bInUseThreadedAudio);
	static ENGINE_API bool IsUsingThreadedAudio();

	static ENGINE_API void SuspendAudioThread();
	static ENGINE_API void ResumeAudioThread();

	/** Retrieves the current audio thread stat id. Useful for reporting when an audio thread command stalls or deadlocks. */
	static ENGINE_API FString GetCurrentAudioThreadStatId();

	static ENGINE_API void ResetAudioThreadTimers();

	static ENGINE_API void GetLongestTaskInfo(FString& OutLongestTask, double& OutLongestTaskTimeMs);
};


/** Suspends the audio thread for the duration of the lifetime of the object */
struct FAudioThreadSuspendContext
{
	FAudioThreadSuspendContext()
	{
		FAudioThread::SuspendAudioThread();
	}

	~FAudioThreadSuspendContext()
	{
		FAudioThread::ResumeAudioThread();
	}
};

////////////////////////////////////
// Audio fences
////////////////////////////////////

/**
* Used to track pending audio commands from the game thread.
*/
class FAudioCommandFence
{
public:
	ENGINE_API ~FAudioCommandFence();

	/**
	* Adds a fence command to the audio command queue.
	* Conceptually, the pending fence count is incremented to reflect the pending fence command.
	* Once the rendering thread has executed the fence command, it decrements the pending fence count.
	*/
	ENGINE_API void BeginFence();

	/**
	* Waits for pending fence commands to retire.
	* @param bProcessGameThreadTasks, if true we are on a short callstack where it is safe to process arbitrary game thread tasks while we wait
	*/
	ENGINE_API void Wait(bool bProcessGameThreadTasks = false) const;

	// return true if the fence is complete
	ENGINE_API bool IsFenceComplete() const;

private:
	/** The last audio batch task **/
	mutable UE::Tasks::FTask Fence;
};

