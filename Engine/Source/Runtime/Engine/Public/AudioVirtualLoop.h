// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ActiveSound.h"
#include "AudioMixerTrace.h"
#include "Sound/SoundModulationDestination.h"

/**
 * Class that tracks virtualized looping active sounds that are eligible to revive re-trigger
 * as long as no stop request is received from the game thread.
 */
struct FAudioVirtualLoop
{
private:
	float TimeSinceLastUpdate;
	float TimeVirtualized;
	float UpdateInterval;

#if UE_AUDIO_PROFILERTRACE_ENABLED
	float IsVirtualizedPingTimeSec = 0.0;
	static constexpr float IsVirtualizedPingWaitTimeSec = 1.0;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	FActiveSound* ActiveSound;

	/**
	 * Array of Modulation Destinations for virtualized ActiveSounds to use when evaluating volume for concurrency
	 * For sounds with multiple wave instance (e.g. Sound Cues), each WaveInstance needs its own Modulation Destination.
	 */
	TArray<Audio::FModulationDestination> VolumeConcurrencyDestinations;

	struct FWaveInstanceData
	{
		float EffectivePitch;
		Audio::FModulationDestination PitchModulationDestination;
		USoundClass* SoundClass;
		
	};
	
	TSortedMap<UPTRINT, float> EffectivePlaybackTimes;
	TSortedMap<UPTRINT, FWaveInstanceData> WaveInstanceData;

	/**
	  * Check if provided active sound is in audible range.
	  */
	static bool IsInAudibleRange(const FActiveSound& InActiveSound, const FAudioDevice* InAudioDevice = nullptr);

public:
	ENGINE_API FAudioVirtualLoop();

	/**
	 * Check to see if listener move is far enough such that a check for virtual loop realization is necessary
	 */
	static ENGINE_API bool ShouldListenerMoveForceUpdate(const FTransform& LastTransform, const FTransform& CurrentTransform);

	/**
	 * Checks if provided active sound is available to be virtualized.  If so, returns new active sound ready to be
	 * added to virtual loop management by parent audio device.
	 */
	static ENGINE_API bool Virtualize(const FActiveSound& InActiveSound, bool bDoRangeCheck, FAudioVirtualLoop& OutVirtualLoop);
	static ENGINE_API bool Virtualize(const FActiveSound& InActiveSound, FAudioDevice& AudioDevice, bool bDoRangeCheck, FAudioVirtualLoop& OutVirtualLoop);

	/**
	 * Whether the virtual loop system is enabled or not
	 */
	static ENGINE_API bool IsEnabled();

	/**
	 * Returns the internally-managed active sound
	 */
	ENGINE_API FActiveSound& GetActiveSound();

	/**
	 * Returns the time the sound has been virtualized
	 */
	ENGINE_API float GetTimeVirtualized() const;

	/**
	 * Returns the effective start time for each wave instance that was virtualized, factoring in pitch multipliers.
	 * This map will be empty if the sound was not virtualized with Seek Restart.
	 */
	const TSortedMap<UPTRINT, float>& GetEffectivePlaybackTimes();
	
	/**
	 * Returns the wait interval being observed before next update
	 */
	ENGINE_API float GetUpdateInterval() const;

	/**
	 */
	ENGINE_API const FActiveSound& GetActiveSound() const;

	/**
	 * Overrides the update interval to the provided length
	 */
	ENGINE_API void CalculateUpdateInterval();

	/**
	 * Takes aggregate update delta and updates focus so that realization
	 * check can test if ready to play.
	 */
	ENGINE_API void UpdateFocusData(float DeltaTime);

	/**
	  * Updates the loop and checks if ready to play (or 'realize').
	  * Returns whether or not the sound is ready to be realized.
	  */
	ENGINE_API bool Update(float DeltaTime, bool bForceUpdate);

#if UE_AUDIO_PROFILERTRACE_ENABLED
	ENGINE_API void OnTraceStarted();
#endif
};
