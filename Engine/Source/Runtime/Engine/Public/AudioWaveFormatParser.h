// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AudioWaveFormatParser.h: Generic parser function for WAVE formatted files.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

struct FChunkHeader;

/**
* Data used to parse the wave file formats
*/

struct FRiffWaveHeader
{
	uint32 ChunkId;
	uint32 ChunkDataSize;
	uint32 TypeId;
};

struct FChunkHeader
{
	uint32 ChunkId;
	uint32 ChunkDataSize;
};

struct FFormatChunk
{
	// format ID
	uint16 FormatTag;

	// number of channels, 1 = mono, 2 = stereo
	uint16 NumChannels;
	
	// sampling rate
	uint32 SamplesPerSec;
	
	// average bite rate
	uint32 AverageBytesPerSec;
	
	// audio block size. ((mono) 1 = 8bit, 2 = 16bit), ((stereo) 2 = 8bit, 4 = 16bit)
	uint16 BlockAlign;
	
	// Quantization rate, 8, 12, 16
	uint16 BitsPerSample;
	
	// Riff extension size, 34
	uint16 cbSize;
	
	// Number of output samples of audio per block
	uint16 SamplesPerBlock;
	
	// Mapping of channels to spatial location
	uint32 ChannelMask;
	
	// Codec ID (for subformat encoding)
	uint8 SubFormat[16];
	
	// Version information of the format
	uint32 VersionInfo;
	
	// Subformat-specific setting information
	uint8 ConfigData[4];
	
	// Reserved, 0
	uint32 Reserved;
};

struct FFactChunk
{
	uint32 TotalSamples;					// total samples per channel
	uint32 DelaySamplesInputOverlap;		// samples of input and overlap delay
	uint32 DelaySamplesInputOverlapEncoder; // samples of input and overlap and encoder delay
};

struct FSampleLoop
{
	uint32 Identifier;
	uint32 Type;
	uint32 Start;
	uint32 End;
	uint32 Fraction;
	uint32 PlayCount;
};

struct FSampleChunk
{
	uint32 Manufacturer;
	uint32 Product;
	uint32 SamplePeriod;
	uint32 MidiUnityNote;
	uint32 MidiPitchFraction;
	uint32 SmpteFormat;
	uint32 SmpteOffset;
	uint32 SampleLoops;
	uint32 SamplerData;
	FSampleLoop SampleLoop[2];
};

/**
* The header of an wave formatted file
*/
struct FWaveFormatInfo
{
	FRiffWaveHeader RiffWaveHeader;
	FChunkHeader FmtChunkHeader;
	FFormatChunk FmtChunk;
	FChunkHeader FactChunkHeader;
	FFactChunk FactChunk;
	FChunkHeader SampleChunkHeader;
	FSampleChunk SampleChunk;
	FChunkHeader DataChunkHeader;
	uint32 DataStartOffset;

	FWaveFormatInfo()
	{
		FMemory::Memzero(this, sizeof(FWaveFormatInfo));
	}
};

bool ENGINE_API ParseWaveFormatHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FWaveFormatInfo& OutHeader);
