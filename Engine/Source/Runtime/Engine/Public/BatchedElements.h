// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BatchedElements.h: Batched element rendering.
=============================================================================*/

#pragma once

#include "Engine/EngineTypes.h"
#include "Templates/RefCounting.h"
#include "RenderResource.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Math/DoubleFloat.h"

class FGraphicsPipelineStateInitializer;
class FHitProxyId;
class FRHICommandList;
class FSceneView;

struct FBatchedPoint;
struct FMeshPassProcessorRenderState;
struct FDFRelativeViewMatrices;

enum EBlendMode : int;
enum ESimpleElementBlendMode : int;

namespace EBlendModeFilter
{
	enum Type
	{
		None = 0,
		OpaqueAndMasked = 1,
		Translucent = 2,
		All = (OpaqueAndMasked | Translucent)
	};
};

/** The type used to store batched line vertices. */
struct FSimpleElementVertex
{
	// Store LWC-scale positions per-vertex
	// Could potentially optimize this by storing a global batch offset, along with relative position per-vertex, but this would be more complicated
	// Could also pack this structure to save some space, W component of position is currently unused for example
	FDFVector4 Position;
	FVector2f TextureCoordinate;
	FLinearColor Color;
	FColor HitProxyIdColor;

	ENGINE_API FSimpleElementVertex();

	ENGINE_API FSimpleElementVertex(const FVector4f& InPosition, const FVector2f& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor);
	ENGINE_API FSimpleElementVertex(const FVector4f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor);
	ENGINE_API FSimpleElementVertex(const FVector3f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor);
	ENGINE_API FSimpleElementVertex(const FVector4d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor);
	ENGINE_API FSimpleElementVertex(const FVector3d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, const FColor& InHitProxyColor);

	ENGINE_API FSimpleElementVertex(const FVector4f& InPosition, const FVector2f& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId);
	ENGINE_API FSimpleElementVertex(const FVector4f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId);
	ENGINE_API FSimpleElementVertex(const FVector3f& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId);
	ENGINE_API FSimpleElementVertex(const FVector4d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId);
	ENGINE_API FSimpleElementVertex(const FVector3d& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId InHitProxyId);
};

/**
* The simple element vertex declaration resource type.
*/
class FSimpleElementVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	ENGINE_API FSimpleElementVertexDeclaration();
	ENGINE_API FSimpleElementVertexDeclaration(FSimpleElementVertexDeclaration&&);
	ENGINE_API virtual ~FSimpleElementVertexDeclaration();

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
};

/** The simple element vertex declaration. */
extern ENGINE_API TGlobalResource<FSimpleElementVertexDeclaration> GSimpleElementVertexDeclaration;



/** Custom parameters for batched element shaders.  Derive from this class to implement your shader bindings. */
class FBatchedElementParameters
	: public FRefCountedObject
{

public:

	/** Binds vertex and pixel shaders for this element */
	// LWC_TODO - InTransform should be a FMatrix44f, and/or should extend this with a method that takes FRelativeViewMatrices, to allow LWC-aware rendering with customized shaders
	virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) = 0;

};



/** Batched elements for later rendering. */
class FBatchedElements
{
public:
	/**
	 * Constructor 
	 */
	ENGINE_API FBatchedElements();

	/** Adds a line to the batch. Note only SE_BLEND_Opaque will be used for batched line rendering. */
	ENGINE_API void AddLine(const FVector& Start,const FVector& End,const FLinearColor& Color,FHitProxyId HitProxyId, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

	/** Adds a translucent line to the batch. */
	ENGINE_API void AddTranslucentLine(const FVector& Start, const FVector& End, const FLinearColor& Color, FHitProxyId HitProxyId, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false);

	/** Adds a point to the batch. Note only SE_BLEND_Opaque will be used for batched point rendering. */
	ENGINE_API void AddPoint(const FVector& Position,float Size,const FLinearColor& Color,FHitProxyId HitProxyId);

	/** This is for compatibility but should be avoided since it's slower due to conversions. */
	ENGINE_API int32 AddVertex(const FVector4& InPosition, const FVector2D& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId HitProxyId);
	/** Adds a mesh vertex to the batch. */
	ENGINE_API int32 AddVertexf(const FVector4f& InPosition, const FVector2f& InTextureCoordinate, const FLinearColor& InColor, FHitProxyId HitProxyId);

	/** Adds a triangle to the batch. */
	ENGINE_API void AddTriangle(int32 V0,int32 V1,int32 V2,const FTexture* Texture,EBlendMode BlendMode);

	/** Adds a triangle to the batch. */
	ENGINE_API void AddTriangle(int32 V0, int32 V1, int32 V2, const FTexture* Texture, ESimpleElementBlendMode BlendMode, const FDepthFieldGlowInfo& GlowInfo = FDepthFieldGlowInfo());

	/** Adds a triangle to the batch. */
	ENGINE_API void AddTriangle(int32 V0,int32 V1,int32 V2,FBatchedElementParameters* BatchedElementParameters,ESimpleElementBlendMode BlendMode);

	/** 
	* Reserves space in index array for a mesh element for current number plus expected number.
	* 
	* @param NumMeshTriangles - number of triangles to reserve space for
	* @param Texture - used to find the mesh element entry
	* @param BlendMode - used to find the mesh element entry
	*/
	ENGINE_API void AddReserveTriangles(int32 NumMeshTriangles,const FTexture* Texture,ESimpleElementBlendMode BlendMode);

	/** 
	* Reserves space in index array for a mesh element
	* 
	* @param NumMeshTriangles - number of triangles to reserve space for
	* @param Texture - used to find the mesh element entry
	* @param BlendMode - used to find the mesh element entry
	*/
	ENGINE_API void ReserveTriangles(int32 NumMeshTriangles,const FTexture* Texture,ESimpleElementBlendMode BlendMode);
	
	/** 
	* Reserves space in mesh vertex array for current number plus expected number.
	* 
	* @param NumMeshVerts - number of verts to reserve space for
	* @param Texture - used to find the mesh element entry
	* @param BlendMode - used to find the mesh element entry
	*/
	ENGINE_API void AddReserveVertices(int32 NumMeshVerts);

	/** 
	* Reserves space in mesh vertex array for at least this many total verts.
	* 
	* @param NumMeshVerts - number of verts to reserve space for
	* @param Texture - used to find the mesh element entry
	* @param BlendMode - used to find the mesh element entry
	*/
	ENGINE_API void ReserveVertices(int32 NumMeshVerts);

	/** 
	 * Reserves space in line vertex array
	 * 
	 * @param NumLines - number of lines to reserve space for
	 * @param bDepthBiased - whether reserving depth-biased lines or non-biased lines
	 * @param bThickLines - whether reserving regular lines or thick lines
	 */
	ENGINE_API void AddReserveLines(int32 NumLines, bool bDepthBiased = false, bool bThickLines = false);

	/** Adds a sprite to the batch. */
	ENGINE_API void AddSprite(
		const FVector& Position,
		float SizeX,
		float SizeY,
		const FTexture* Texture,
		const FLinearColor& Color,
		FHitProxyId HitProxyId,
		float U,
		float UL,
		float V,
		float VL,
		uint8 BlendMode,
		float OpacityMaskRefVal
		);

	/**
	 * Draws the batch
	 *
	 * @param View			FSceneView for shaders that need access to view constants. Non-optional to also reference its ViewProjectionMatrix and size of the ViewRect
	 * @param bHitTesting	Whether or not we are hit testing
	 * @param Gamma			Optional gamma override
	 * @param DepthTexture	DepthTexture for manual depth testing with editor compositing in the pixel shader
	 */
	ENGINE_API bool Draw(FRHICommandList& RHICmdList, const FMeshPassProcessorRenderState& DrawRenderState, ERHIFeatureLevel::Type FeatureLevel, const FSceneView& View, bool bHitTesting, float Gamma = 1.0f, EBlendModeFilter::Type Filter = EBlendModeFilter::All, uint32 NumInstances = 1) const;

	/**
	 * Creates a proxy FSceneView for operations that are not tied directly to a scene but still require batched elements to be drawn.
	 */
	static ENGINE_API FSceneView CreateProxySceneView(const FMatrix& ProjectionMatrix, const FIntRect& ViewRect);

	inline bool HasPrimsToDraw() const
	{
		return( LineVertices.Num() || Points.Num() || Sprites.Num() || MeshElements.Num() || ThickLines.Num() || WireTris.Num() > 0 );
	}

	/** Adds a triangle to the batch. Extensive version where all parameters can be passed in. */
	ENGINE_API void AddTriangleExtensive(int32 V0,int32 V1,int32 V2,FBatchedElementParameters* BatchedElementParameters,const FTexture* Texture,ESimpleElementBlendMode BlendMode, const FDepthFieldGlowInfo& GlowInfo = FDepthFieldGlowInfo());

	/** Clears any batched elements **/
	ENGINE_API void Clear();

	/** 
	 * Helper function to return the amount of memory allocated by this class 
	 *
	 * @return number of bytes allocated by this container
	 */
	inline uint32 GetAllocatedSize( void ) const
	{
		return sizeof(*this) + Points.GetAllocatedSize() + WireTris.GetAllocatedSize() + WireTriVerts.GetAllocatedSize() + ThickLines.GetAllocatedSize()
			+ Sprites.GetAllocatedSize() + MeshElements.GetAllocatedSize() + MeshVertices.GetAllocatedSize();
	}

	UE_DEPRECATED(5.4, "EnableMobileHDREncoding is no longer supported")
	void EnableMobileHDREncoding(bool bInEnableHDREncoding) {}

	class FAllocationInfo
	{
	public:
		FAllocationInfo() = default;

	private:
		int32 NumLineVertices = 0;
		int32 NumPoints = 0;
		int32 NumWireTris = 0;
		int32 NumWireTriVerts = 0;
		int32 NumThickLines = 0;
		int32 NumSprites = 0;
		int32 NumMeshElements = 0;
		int32 NumMeshVertices = 0;

		friend FBatchedElements;
	};

	/** Accumulates allocation info for use calling Reserve. */
	ENGINE_API void AddAllocationInfo(FAllocationInfo& AllocationInfo) const;

	/** Reserves memory for all containers. */
	ENGINE_API void Reserve(const FAllocationInfo& AllocationInfo);

	/** Appends contents of another batched elements into this one and clears the other one. */
	ENGINE_API void Append(FBatchedElements& Other);

private:

	/**
	 * Draws points
	 *
	 * @param	Transform	Transformation matrix to use
	 * @param	ViewportSizeX	Horizontal viewport size in pixels
	 * @param	ViewportSizeY	Vertical viewport size in pixels
	 * @param	CameraX		Local space normalized view direction X vector
	 * @param	CameraY		Local space normalized view direction Y vector
	 */
	ENGINE_API void DrawPointElements(FRHICommandList& RHICmdList, const FMatrix& Transform, const uint32 ViewportSizeX, const uint32 ViewportSizeY, const FVector& CameraX, const FVector& CameraY, const uint32 NumInstances) const;

	TArray<FSimpleElementVertex> LineVertices;

	struct FBatchedPoint
	{
		FVector Position;
		float Size;
		FColor Color;
		FColor HitProxyColor;
	};
	TArray<FBatchedPoint> Points;

	struct FBatchedWireTris
	{
		float DepthBias;
	};
	TArray<FBatchedWireTris> WireTris;

	mutable TResourceArray<FSimpleElementVertex> WireTriVerts;

	struct FBatchedThickLines
	{
		FVector Start;
		FVector End;
		float Thickness;
		FLinearColor Color;
		FColor HitProxyColor;
		float DepthBias;
		uint32 bScreenSpace;
	};
	TArray<FBatchedThickLines> ThickLines;

	struct FBatchedSprite
	{
		FVector Position;
		float SizeX;
		float SizeY;
		const FTexture* Texture;
		FLinearColor Color;
		FColor HitProxyColor;
		float U;
		float UL;
		float V;
		float VL;
		float OpacityMaskRefVal;
		uint8 BlendMode;
	};
	TArray<FBatchedSprite> Sprites;

	struct FBatchedMeshElement
	{
		/** starting index in vertex buffer for this batch */
		uint32 MinVertex;
		/** largest vertex index used by this batch */
		uint32 MaxVertex;
		/** index buffer for triangles */
		TArray<uint16> Indices;
		/** all triangles in this batch draw with the same texture */
		const FTexture* Texture;
		/** Parameters for this batched element */
		TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;
		/** all triangles in this batch draw with the same blend mode */
		ESimpleElementBlendMode BlendMode;
		/** all triangles in this batch draw with the same depth field glow (depth field blend modes only) */
		FDepthFieldGlowInfo GlowInfo;
	};

	/** Max number of mesh index entries that will fit in a DrawPriUP call */
	int32 MaxMeshIndicesAllowed;
	/** Max number of mesh vertices that will fit in a DrawPriUP call */
	int32 MaxMeshVerticesAllowed;

	TArray<FBatchedMeshElement> MeshElements;
	TArray<FSimpleElementVertex> MeshVertices;

	/**
	 * Sets the appropriate vertex and pixel shader.
	 */
	ENGINE_API void PrepareShaders(
		FRHICommandList& RHICmdList,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit,
		uint32 StencilRef,
		ERHIFeatureLevel::Type FeatureLevel,
		ESimpleElementBlendMode BlendMode,
		const FDFRelativeViewMatrices& ViewMatrices,
		FBatchedElementParameters* BatchedElementParameters,
		const FTexture* Texture,
		bool bHitTesting,
		float Gamma,
		const FDepthFieldGlowInfo* GlowInfo = nullptr,
		const FSceneView* View = nullptr,
		float OpacityMaskRefVal = .5f
		) const;
};

