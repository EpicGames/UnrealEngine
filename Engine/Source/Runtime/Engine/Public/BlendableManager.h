// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// used to blend IBlendableInterface object data, the member act as a header for a following data block
struct FBlendableEntry
{
	// weight 0..1, 0 to disable this entry
	float Weight;

private:
	// to ensure type safety
	FName BlendableType;
	// to be able to jump over data
	uint32 DataSize;
	// to align the data
	uint32 PrePadding;
	 
	// @return pointer to the next object or end (can be compared with container end pointer)
	uint8* GetDataPtr(){ check(this); return ((uint8*)(this + 1)) + PrePadding; }
	// @return next or end of the array
	FBlendableEntry* GetNext() { return (FBlendableEntry*)(GetDataPtr() + DataSize); }

	friend class FBlendableManager;
};

// Manager class which allows blending of FBlendableBase objects, stores a copy (for the render thread)
class FBlendableManager
{
public:

	FBlendableManager()
	{
		// can be increased if needed (to avoid reallocations and copies at the cost of more memory allocation)
		Scratch.Reserve(10 * 1024);
	}

	// @param InWeight 0..1, excluding 0 as this is used to disable entries
	// @param InData is copied with a memcpy
	template <class T>
	T* PushBlendableData(float InWeight, const T& InData)
	{
		FName BlendableType = T::GetFName();

		// at least 4 byte alignment
		uint32 Alignment = FMath::Max((uint32)4, (uint32)alignof(T));

		FBlendableEntry* Entry = PushBlendableDataPtr(InWeight, BlendableType, (const uint8*)&InData, sizeof(T), Alignment);

		T* Ret = (T*)Entry->GetDataPtr();

		return Ret;
	}

	// used to blend multiple blendables of the given type with lerp into one final one
	template <class T>
	T& GetSingleFinalData()
	{
		FBlendableEntry* Iterator = 0;
		T* FinalDataPtr = IterateBlendables<T>(Iterator);

		if(!FinalDataPtr)
		{
			T Base;
			Base.SetBaseValues();

			PushBlendableData<T>(1.0f, Base);
			// can be optimized
			FinalDataPtr = IterateBlendables<T>(Iterator);
		}
		
		return *FinalDataPtr;
	}

	// use to pickup the on final blendable that was generated by blending 0-n blendables of the given type
	template <class T>
	const T& GetSingleFinalDataConst() const
	{
		FBlendableEntry* Iterator = 0;
		const T* FinalDataPtr = IterateBlendables<T>(Iterator);

		if(!FinalDataPtr)
		{
			// could be improved
			static T Base;
			Base.SetBaseValues();

			return Base;
		}
		
		return *FinalDataPtr;
	}

	// find next that has the given type, O(n), n is number of elements after the given one or all if started with 0
	// @param InIterator 0 to start iteration
	// @return 0 if no further one of that type was found
	template <class T>
	T* IterateBlendables(FBlendableEntry*& InIterator) const
	{
		FName BlendableType = T::GetFName();

		do
		{
			InIterator = GetNextBlendableEntryPtr(InIterator);
			
		} while (InIterator && InIterator->BlendableType != BlendableType);		

		if (InIterator)
		{
			return (T*)InIterator->GetDataPtr();
		}

		return 0;
	}

private:

	// stored multiple elements with a FBlendableEntry header and following data of a size specified in the header
	TArray<uint8> Scratch;

	// find next, no restriction on the type O(n), n is number of elements after the given one or all if started with 0
	FBlendableEntry* GetNextBlendableEntryPtr(FBlendableEntry* InIterator = 0) const
	{
		if (!InIterator)
		{
			// start at the beginning
			InIterator = (FBlendableEntry*)Scratch.GetData();
		}
		else
		{
			// go to next
			InIterator = InIterator->GetNext();
		}

		// end reached?
		if ((uint8*)InIterator == Scratch.GetData() + Scratch.Num())
		{
			// no more entries
			return 0;
		}

		return InIterator;
	}

	// @param InWeight 0..1, excluding 0 as this is used to disable entries
	// @param InData is copied
	// @param InDataSize >0
	// @return pointer to the newly added entry
	FBlendableEntry* PushBlendableDataPtr(float InWeight, FName InBlendableType, const uint8* InData, uint32 InDataSize, uint32 Alignment)
	{
		check(InWeight > 0.0f && InWeight <= 1.0f);
		check(InData);
		check(InDataSize);

		uint32 PrePadding;
		{
			uint8* DataStart = Scratch.GetData() + Scratch.Num() + sizeof(FBlendableEntry);
			PrePadding = static_cast<uint32>(Alignment - reinterpret_cast<ptrdiff_t>(DataStart) % Alignment);
			PrePadding = (PrePadding == Alignment) ? 0 : PrePadding;	
		}

		uint32 OldSize = Scratch.AddUninitialized(sizeof(FBlendableEntry) + InDataSize + PrePadding);
		
		FBlendableEntry* Dst = (FBlendableEntry*)&Scratch[OldSize];

		Dst->Weight = InWeight;
		Dst->BlendableType = InBlendableType;
		Dst->DataSize = InDataSize;
		Dst->PrePadding = PrePadding;
		memcpy(Dst->GetDataPtr(), InData, InDataSize);

		return Dst;
	}
};
