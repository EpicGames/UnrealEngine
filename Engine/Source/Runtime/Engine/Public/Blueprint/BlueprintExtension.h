// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"

#include "BlueprintExtension.generated.h"

class UBlueprint;
class FKismetCompilerContext;
class UEdGraph;

/**
 * Per-instance extension object that can be added to UBlueprint::Extensions in order to augment built-in blueprint functionality
 * Ideally this would be an editor-only class, but such classes are not permitted within Engine modules (even inside WITH_EDITORONLY_DATA blocks)
 */
UCLASS(MinimalAPI)
class UBlueprintExtension : public UObject
{
public:

	GENERATED_BODY()

#if WITH_EDITORONLY_DATA

	/** Used to add Key/Value pairs in the FindInBlueprintManager */
	struct FSearchTagDataPair
	{
		FSearchTagDataPair(FText InKey, FText InValue)
			: Key(InKey)
			, Value(InValue)
		{}

		FText Key;
		FText Value;
	};
	struct FSearchArrayData;
	struct FSearchData
	{
		TArray<FSearchTagDataPair> Datas;
		TArray<TUniquePtr<FSearchArrayData>> SearchArrayDatas;
	};
	struct FSearchArrayData
	{
		FText Identifier;
		TArray<FSearchData> SearchSubList;
	};

	/**
	 * Called during compilation (after skeleton class generation) in order to generate dynamic function graphs for this blueprint
	 */
	void GenerateFunctionGraphs(FKismetCompilerContext* CompilerContext)
	{
		HandleGenerateFunctionGraphs(CompilerContext);
	}

	/**
	 * Called before blueprint compilation to ensure that any objects necessary for the specified blueprint's compilation
	 */
	void PreloadObjectsForCompilation(UBlueprint* OwningBlueprint)
	{
		HandlePreloadObjectsForCompilation(OwningBlueprint);
	}
	
	/**
	 * Called when the find in blueprint system gather the search data for the blueprint
	 */
	FSearchData GatherSearchData(const UBlueprint* OwningBlueprint) const
	{
		return HandleGatherSearchData(OwningBlueprint);
	}

	/**
	 * Override this function to inform editor tools of any graphs in this extension (eg. blueprint diff, asset search)
	 */
	virtual void GetAllGraphs(TArray<UEdGraph*>& Graphs) const {}

private:

	/**
	 * Override this function to define custom preload logic into the specified blueprint
	 */
	virtual void HandlePreloadObjectsForCompilation(UBlueprint* OwningBlueprint) {}

	/**
	 * Override this function to define custom function generation logic for the specified blueprint compiler context
	 */
	virtual void HandleGenerateFunctionGraphs(FKismetCompilerContext* CompilerContext) {}

	/**
	 * Override this function to define additional search data for the specified blueprint
	 */
	virtual FSearchData HandleGatherSearchData(const UBlueprint* OwningBlueprint) const { return FSearchData(); }

#endif // WITH_EDITORONLY_DATA
};
