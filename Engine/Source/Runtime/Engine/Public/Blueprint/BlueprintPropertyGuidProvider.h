// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

/**
 * Interface to query the property name<->GUID relationship using either a UBlueprint or a UBlueprintGeneratedClass.
 * This allows cooked and uncooked Blueprints to be queried via the same API.
 */
class IBlueprintPropertyGuidProvider
{
public:
	virtual ~IBlueprintPropertyGuidProvider() = default;

	/**
	 * Returns the property name for the given GUID, if any.
	 * @note Does not recurse into parents.
	 */
	virtual FName FindBlueprintPropertyNameFromGuid(const FGuid& PropertyGuid) const = 0;

	/**
	 * Returns the property GUID for the given name, if any.
	 * @note Does not recurse into parents.
	 */
	virtual FGuid FindBlueprintPropertyGuidFromName(const FName PropertyName) const = 0;
};

