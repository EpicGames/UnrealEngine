// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* Enums to use when grouping the blueprint members in the list panel. The order here will determine the order in the list */
namespace NodeSectionID
{
	enum Type
	{
		NONE = 0,
		GRAPH,					// Graph
		ANIMGRAPH,				// Anim Graph
		ANIMLAYER,				// Anim Layer
		FUNCTION,				// Functions
		FUNCTION_OVERRIDABLE,	// Overridable functions
		INTERFACE,				// Interface
		MACRO,					// Macros
		VARIABLE,				// Variables
		COMPONENT,				// Components
		DELEGATE,				// Delegate/Event
		USER_ENUM,				// User defined enums
		LOCAL_VARIABLE,			// Local variables
		USER_STRUCT,			// User defined structs
		USER_SORTED				// User sorted categories
	};
};
