// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

struct FEdGraphEditAction;
struct FPropertyChangedEvent;

DECLARE_MULTICAST_DELEGATE_OneParam( FOnGraphChanged, const FEdGraphEditAction& );
DECLARE_DELEGATE_OneParam( FSingleNodeEvent, class UEdGraphNode* );
DECLARE_DELEGATE_OneParam( FEdGraphEvent, class UEdGraph* );
/** Delegate for notification when property changed */
DECLARE_MULTICAST_DELEGATE_TwoParams( FOnPropertyChanged, const FPropertyChangedEvent&, const FString& );

DECLARE_STATS_GROUP(TEXT("Kismet Compiler"), STATGROUP_KismetCompiler, STATCAT_Advanced);
