// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "BoneIndices.h"
#include "ReferenceSkeleton.h"
#include "Animation/AnimCurveFilter.h"
#include "Animation/AnimTypes.h"
#include "Animation/BoneReference.h"
#include "Animation/AnimCurveMetadata.h"
#include "Animation/AnimBulkCurves.h"

class USkeletalMesh;
struct FBoneContainer;
struct FSkeletonRemapping;
struct FBlendedCurve;
struct FAnimCurveType;

namespace SmartName
{
typedef uint16 UID_Type;
}

/** Struct used to store per-component ref pose override */
struct FSkelMeshRefPoseOverride
{
	/** Inverse of (component space) ref pose matrices  */
	TArray<FMatrix44f> RefBasesInvMatrix;
	/** Per bone transforms (local space) for new ref pose */
	TArray<FTransform> RefBonePoses;
};

/**
* This is a native transient structure. Used to store virtual bone mappings for compact poses
**/
struct FVirtualBoneCompactPoseData
{
	/** Index of this virtual bone */
	FCompactPoseBoneIndex VBIndex;
	/** Index of source bone */
	FCompactPoseBoneIndex SourceIndex;
	/** Index of target bone */
	FCompactPoseBoneIndex TargetIndex;

	FVirtualBoneCompactPoseData(FCompactPoseBoneIndex InVBIndex, FCompactPoseBoneIndex InSourceIndex, FCompactPoseBoneIndex InTargetIndex)
		: VBIndex(InVBIndex)
		, SourceIndex(InSourceIndex)
		, TargetIndex(InTargetIndex)
	{}
};

// DEPRECATED - please use UE::Anim::FCurveFilterSettings
struct FCurveEvaluationOption
{
	bool bAllowCurveEvaluation;
	const TArray<FName>* DisallowedList;
	int32 LODIndex;

	UE_DEPRECATED(5.3, "Please use UE::Anim::FCurveFilterSettings.")
	FCurveEvaluationOption(bool bInAllowCurveEvaluation = true, const TArray<FName>* InDisallowedList = nullptr, int32 InLODIndex = 0)
		: bAllowCurveEvaluation(bInAllowCurveEvaluation)
		, DisallowedList(InDisallowedList)
		, LODIndex(InLODIndex)
	{
	}
};

namespace UE::Anim
{

/** Options for filtering curves. Used to initialize a bone container with a filter curve. */
struct FCurveFilterSettings
{
	FCurveFilterSettings(UE::Anim::ECurveFilterMode InFilterMode = UE::Anim::ECurveFilterMode::DisallowFiltered, const TArray<FName>* InFilterCurves = nullptr, int32 InLODIndex = 0)
		: FilterCurves(InFilterCurves)
		, FilterMode(InFilterMode)
		, LODIndex(InLODIndex)
	{}

	// Filter curves. Application of these curves is dependent on FilterMode
	const TArray<FName>* FilterCurves = nullptr;
	
	// Filtering mode - allows curves to be disable entirely or allow/deny-listed
	UE::Anim::ECurveFilterMode FilterMode = UE::Anim::ECurveFilterMode::DisallowFiltered;

	// LOD index
	int32 LODIndex = 0;
};

}

/** Stores cached data for Orient and Scale bone translation retargeting */
struct FOrientAndScaleRetargetingCachedData
{
	FQuat TranslationDeltaOrient;
	float TranslationScale;
	FVector SourceTranslation;
	FVector TargetTranslation;

	FOrientAndScaleRetargetingCachedData
	(
		const FQuat& InTranslationDeltaOrient, 
		const float InTranslationScale, 
		const FVector& InSourceTranslation,
		const FVector& InTargetTranslation
	)
		: TranslationDeltaOrient(InTranslationDeltaOrient)
		, TranslationScale(InTranslationScale)
		, SourceTranslation(InSourceTranslation)
		, TargetTranslation(InTargetTranslation)
	{
	}
};

/**
 * An array of cached curve remappings. This is used to remap curves between skeletons.
 * It is used in the FBoneContainer and is generated using a lazy approach. 
 */
struct FCachedSkeletonCurveMapping
{
	TArray<SmartName::UID_Type>	UIDToArrayIndices; /** The mapping table used for mapping curves. This is indexed by UID and returns the curve index, or MAX_uint16 in case its not used. */
	bool bIsDirty = true; /** Specifies whether we need to rebuild this cached data or not. */
};

/** Retargeting cached data for a specific Retarget Source */
struct FRetargetSourceCachedData
{
	/** Orient and Scale cached data. */
	TArray<FOrientAndScaleRetargetingCachedData> OrientAndScaleData;

	/** LUT from CompactPoseIndex to OrientAndScaleIndex */
	TArray<int32> CompactPoseIndexToOrientAndScaleIndex;
};

/** Iterator for compact pose indices */
struct FCompactPoseBoneIndexIterator
{
	int32 Index;

	FCompactPoseBoneIndexIterator(int32 InIndex) : Index(InIndex) {}

	FCompactPoseBoneIndexIterator& operator++() { ++Index; return (*this); }
	bool operator==(FCompactPoseBoneIndexIterator& Rhs) { return Index == Rhs.Index; }
	bool operator!=(FCompactPoseBoneIndexIterator& Rhs) { return Index != Rhs.Index; }
	FCompactPoseBoneIndex operator*() const { return FCompactPoseBoneIndex(Index); }
};

/** Reverse iterator for compact pose indices */
struct FCompactPoseBoneIndexReverseIterator
{
	int32 Index;

	FCompactPoseBoneIndexReverseIterator(int32 InIndex) : Index(InIndex) {}

	FCompactPoseBoneIndexReverseIterator& operator++() { --Index; return (*this); }
	bool operator==(FCompactPoseBoneIndexReverseIterator& Rhs) { return Index == Rhs.Index; }
	bool operator!=(FCompactPoseBoneIndexReverseIterator& Rhs) { return Index != Rhs.Index; }
	FCompactPoseBoneIndex operator*() const { return FCompactPoseBoneIndex(Index); }
};

struct FRetargetSourceCachedDataKey
{
	FTopLevelAssetPath SourceSkeletonPath;
	FName SourceRetargetName;
 
	FRetargetSourceCachedDataKey(const UObject* Skeleton, const FName& InSourceRetargetName)
		: SourceSkeletonPath(Skeleton)
		, SourceRetargetName(InSourceRetargetName)
	{
	}
 
	bool operator==(const FRetargetSourceCachedDataKey& Other) const
	{
		return SourceSkeletonPath == Other.SourceSkeletonPath && SourceRetargetName == Other.SourceRetargetName;
	}
	friend uint32 GetTypeHash(const FRetargetSourceCachedDataKey& Obj)
	{
		return HashCombine(GetTypeHash(Obj.SourceSkeletonPath), GetTypeHash(Obj.SourceRetargetName));
	}
};

/**
* This is a native transient structure.
* Contains:
* - BoneIndicesArray: Array of RequiredBoneIndices for Current Asset. In increasing order. Mapping to current Array of Transforms (Pose).
* - BoneSwitchArray: Size of current Skeleton. true if Bone is contained in RequiredBones array, false otherwise.
**/
struct FBoneContainer
{
private:
	/** Array of RequiredBonesIndices. In increasing order. */
	TArray<FBoneIndexType>	BoneIndicesArray;
	/** Array sized by Current RefPose. true if Bone is contained in RequiredBones array, false otherwise. */
	TBitArray<>				BoneSwitchArray;

	/** Asset BoneIndicesArray was made for. Typically a SkeletalMesh. */
	TWeakObjectPtr<UObject>	Asset;
	/** If Asset is a SkeletalMesh, this will be a pointer to it. Can be NULL if Asset is a USkeleton. */
	TWeakObjectPtr<USkeletalMesh> AssetSkeletalMesh;
	/** If Asset is a Skeleton that will be it. If Asset is a SkeletalMesh, that will be its Skeleton. */
	TWeakObjectPtr<USkeleton> AssetSkeleton;

	/** Pointer to RefSkeleton of Asset. */
	const FReferenceSkeleton* RefSkeleton;

	/** Mapping table between Skeleton Bone Indices and Pose Bone Indices. */
	TArray<int32> SkeletonToPoseBoneIndexArray;

	/** Mapping table between Pose Bone Indices and Skeleton Bone Indices. */
	TArray<int32> PoseToSkeletonBoneIndexArray;

	// Look up from skeleton to compact pose format
	TArray<int32> CompactPoseToSkeletonIndex;

	// Look up from compact pose format to skeleton
	TArray<FCompactPoseBoneIndex> SkeletonToCompactPose;

	// Compact pose format of Parent Bones (to save us converting to mesh space and back)
	TArray<FCompactPoseBoneIndex> CompactPoseParentBones;

	// Array of cached virtual bone data so that animations running from raw data can generate them.
	TArray<FVirtualBoneCompactPoseData> VirtualBoneCompactPoseData;

	/** Curve used for filtering by LOD/bone */
	UE::Anim::FCurveFilter CurveFilter;

	/** Curve flags to apply (built from curve metadata) */
	UE::Anim::FBulkCurveFlags CurveFlags;
	
	TSharedPtr<FSkelMeshRefPoseOverride> RefPoseOverride;
	
	// The serial number of this bone container. This is incremented each time the container is regenerated and can
	// be used to track whether to cache bone data. If this value is zero then the bone container is considered invalid
	// as it has never been regenerated.
	uint16 SerialNumber;

	/** For debugging. */
#if DO_CHECK
	/** The LOD that we calculated required bones when regenerated */
	int32 CalculatedForLOD;
#endif
	/** Disable Retargeting. Extract animation, but do not retarget it. */
	bool bDisableRetargeting;
	/** Disable animation compression, use RAW data instead. */
	bool bUseRAWData;
	/** Use Source Data that is imported that are not compressed. */
	bool bUseSourceData;

public:

	ENGINE_API FBoneContainer();

	ENGINE_API FBoneContainer(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const UE::Anim::FCurveFilterSettings& InCurveFilterSettings, UObject& InAsset);
	
	ENGINE_API void InitializeTo(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const UE::Anim::FCurveFilterSettings& InCurveFilterSettings, const UObject& InAsset);
	
	UE_DEPRECATED(5.3, "Please use the constructor that takes a FCurveFilterSettings.")
	ENGINE_API FBoneContainer(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const FCurveEvaluationOption& CurveEvalOption, UObject& InAsset);

	UE_DEPRECATED(5.3, "Please use InitializeTo that takes a FCurveFilterSettings.")
	ENGINE_API void InitializeTo(const TArrayView<const FBoneIndexType>& InRequiredBoneIndexArray, const FCurveEvaluationOption& CurveEvalOption, const UObject& InAsset);

	/** Resets the container and reclaims all allocated memory but preserve the serial number. */
	ENGINE_API void Reset();

	/** Returns true if FBoneContainer is Valid. Needs an Asset, a RefPoseArray, and a RequiredBonesArray. */
	const bool IsValid() const
	{
		return (Asset.IsValid(
#if WITH_EDITOR
			true, true
#endif
			)
				&& (RefSkeleton != nullptr) && (BoneIndicesArray.Num() > 0));
	}

	/** Get Asset this BoneContainer was made for. Typically a SkeletalMesh, but could also be a USkeleton. */
	UObject* GetAsset() const
	{
		return Asset.Get();
	}

	/** Get SkeletalMesh Asset this BoneContainer was made for. Could be NULL if Asset is a Skeleton. */
	USkeletalMesh* GetSkeletalMeshAsset() const
	{
		return AssetSkeletalMesh.Get();
	}

	/** Get Skeleton Asset. Could either be the SkeletalMesh's Skeleton, or the Skeleton this BoneContainer was made for. Is non NULL is BoneContainer is valid. */
	USkeleton* GetSkeletonAsset(bool bEvenIfUnreachable = false) const
	{
		return
#if WITH_EDITOR
		bEvenIfUnreachable ? AssetSkeleton.GetEvenIfUnreachable() : 
#endif
		AssetSkeleton.Get();
	}

	/** Disable Retargeting for debugging. */
	void SetDisableRetargeting(bool InbDisableRetargeting)
	{
		bDisableRetargeting = InbDisableRetargeting;
	}

	/** True if retargeting is disabled for debugging. */
	bool GetDisableRetargeting() const
	{
		return bDisableRetargeting;
	}

	/** Ignore compressed data and use RAW data instead, for debugging. */
	void SetUseRAWData(bool InbUseRAWData)
	{
		bUseRAWData = InbUseRAWData;
	}

	/** True if we're requesting RAW data instead of compressed data. For debugging. */
	bool ShouldUseRawData() const
	{
		return bUseRAWData;
	}

	/** use Source data instead.*/
	void SetUseSourceData(bool InbUseSourceData)
	{
		bUseSourceData = InbUseSourceData;
	}

	/** True if we're requesting Source data instead of RawAnimationData. For debugging. */
	bool ShouldUseSourceData() const
	{
		return bUseSourceData;
	}

	/**
	 * Returns array of the size of compact pose, mapping to mesh pose index
	 * returns Required Bone Indices Array
	 */
	const TArray<FBoneIndexType>& GetBoneIndicesArray() const
	{
		return BoneIndicesArray;
	}

	/**
	* returns virutal bone cached data
	*/
	const TArray<FVirtualBoneCompactPoseData>& GetVirtualBoneCompactPoseData() const { return VirtualBoneCompactPoseData; }

	/**
	* returns Bone Switch Array. BitMask for RequiredBoneIndex array.
	*/
	const TBitArray<>& GetBoneSwitchArray() const
	{
		return BoneSwitchArray;
	}

	/** Pointer to RefPoseArray for current Asset. */
	const TArray<FTransform>& GetRefPoseArray() const
	{
		return RefSkeleton->GetRefBonePose();
	}

	const TArray<FCompactPoseBoneIndex>& GetCompactPoseParentBoneArray() const
	{
		return CompactPoseParentBones;
	}

	// Fill the supplied buffer with the compact pose reference pose
	template<typename ArrayType>
	void FillWithCompactRefPose(ArrayType& OutTransforms) const
	{
		const int32 CompactPoseBoneCount = GetCompactPoseNumBones();
		OutTransforms.Reset(CompactPoseBoneCount);
		OutTransforms.SetNumUninitialized(CompactPoseBoneCount);
		
		const TArray<FTransform>& RefPoseTransforms = RefPoseOverride.IsValid() ? RefPoseOverride->RefBonePoses : RefSkeleton->GetRefBonePose();
		for (int32 CompactBoneIndex = 0; CompactBoneIndex < CompactPoseBoneCount; ++CompactBoneIndex)
		{
			OutTransforms[CompactBoneIndex] = RefPoseTransforms[BoneIndicesArray[CompactBoneIndex]];
		}
	}
	
	const FTransform& GetRefPoseTransform(const FCompactPoseBoneIndex& BoneIndex) const
	{
		const TArray<FTransform>& RefPoseTransforms = RefPoseOverride.IsValid() ? RefPoseOverride->RefBonePoses : RefSkeleton->GetRefBonePose();
		return RefPoseTransforms[BoneIndicesArray[BoneIndex.GetInt()]];
	}

	/** Override skeleton ref pose. */
	void SetRefPoseOverride(const TSharedPtr<FSkelMeshRefPoseOverride>& InRefPoseOverride)
	{
		if (InRefPoseOverride.Get() != RefPoseOverride.Get())
		{
			RefPoseOverride = InRefPoseOverride;
		}
	}

	/** Access to Asset's RefSkeleton. */
	const FReferenceSkeleton& GetReferenceSkeleton() const
	{
		return *RefSkeleton;
	}

	/** Number of Bones in RefPose for current asset. This is NOT the number of bones in RequiredBonesArray, but the TOTAL number of bones in the RefPose of the current Asset! */
	const int32 GetNumBones() const
	{
		return RefSkeleton->GetNum();
	}

	const int32 GetCompactPoseNumBones() const
	{
		return BoneIndicesArray.Num();
	}

	/** Get BoneIndex for BoneName for current Asset. */
	ENGINE_API int32 GetPoseBoneIndexForBoneName(const FName& BoneName) const;

	/** Get ParentBoneIndex for current Asset. */
	ENGINE_API int32 GetParentBoneIndex(const int32 BoneIndex) const;

	/** Get ParentBoneIndex for current Asset. */
	ENGINE_API FCompactPoseBoneIndex GetParentBoneIndex(const FCompactPoseBoneIndex& BoneIndex) const;

	/** Get Depth between bones for current asset. */
	ENGINE_API int32 GetDepthBetweenBones(const int32 BoneIndex, const int32 ParentBoneIndex) const;

	/** Returns true if bone is child of for current asset. */
	ENGINE_API bool BoneIsChildOf(const int32 BoneIndex, const int32 ParentBoneIndex) const;

	/** Returns true if bone is child of for current asset. */
	ENGINE_API bool BoneIsChildOf(const FCompactPoseBoneIndex& BoneIndex, const FCompactPoseBoneIndex& ParentBoneIndex) const;

	/** Get filter used for filtering by LOD/bone */
	const UE::Anim::FCurveFilter& GetCurveFilter() const
	{
		return CurveFilter;
	}

	/** Get flags to apply to curves (built from metadata) */
	const UE::Anim::FBulkCurveFlags& GetCurveFlags() const
	{
		return CurveFlags;
	}

	UE_DEPRECATED(5.3, "GetUIDToArrayLookupTable is deprecated, it is no longer used.")
	ENGINE_API TArray<uint16> const& GetUIDToArrayLookupTable() const;

	UE_DEPRECATED(5.3, "GetUIDToArrayIndexLookupTableValidCount is deprecated, it is no longer used.")
	ENGINE_API int32 GetUIDToArrayIndexLookupTableValidCount() const;

	UE_DEPRECATED(5.0, "GetUIDToNameLookupTable is deprecated, please access from the SmartNameMapping directly via GetSkeletonAsset()->GetSmartNameContainer(USkeleton::AnimCurveMappingName)")
	TArray<FName> const& GetUIDToNameLookupTable() const
	{
		static TArray<FName> Dummy;
		return Dummy;
	}

	UE_DEPRECATED(5.0, "GetUIDToCurveTypeLookupTable is deprecated, please access from the SmartNameMapping directly via GetSkeletonAsset()->GetSmartNameContainer(USkeleton::AnimCurveMappingName)")
	ENGINE_API TArray<FAnimCurveType> const& GetUIDToCurveTypeLookupTable() const;

	UE_DEPRECATED(5.3, "GetUIDToArrayLookupTableBackup is deprecated, it is no longer used.")
	ENGINE_API TArray<SmartName::UID_Type> const& GetUIDToArrayLookupTableBackup() const;

	/**
	* Serializes the bones
	*
	* @param Ar - The archive to serialize into.
	* @param Rect - The bone container to serialize.
	*
	* @return Reference to the Archive after serialization.
	*/
	friend FArchive& operator<<(FArchive& Ar, FBoneContainer& B)
	{
		Ar
			<< B.BoneIndicesArray
			<< B.BoneSwitchArray
			<< B.Asset
			<< B.AssetSkeletalMesh
			<< B.AssetSkeleton
			<< B.SkeletonToPoseBoneIndexArray
			<< B.PoseToSkeletonBoneIndexArray
			<< B.bDisableRetargeting
			<< B.bUseRAWData
			<< B.bUseSourceData
			;

		return Ar;
	}

	/**
	* Returns true of RequiredBonesArray contains this bone index.
	*/
	bool Contains(FBoneIndexType NewIndex) const
	{
		return BoneSwitchArray[NewIndex];
	}

	/** Const accessor to SkeletonToPoseBoneIndexArray. */
	UE_DEPRECATED(5.0, "Please use GetMeshPoseIndexFromSkeletonPoseIndex")
	TArray<int32> const & GetSkeletonToPoseBoneIndexArray() const
	{
		return SkeletonToPoseBoneIndexArray;
	}

	/** Const accessor to PoseToSkeletonBoneIndexArray. */
	UE_DEPRECATED(5.0, "Please use GetSkeletonPoseIndexFromMeshPoseIndex")
	TArray<int32> const & GetPoseToSkeletonBoneIndexArray() const
	{
		return PoseToSkeletonBoneIndexArray;
	}
	
	template<typename IterType>
	struct FRangedForSupport
	{
		const FBoneContainer& BoneContainer;

		FRangedForSupport(const FBoneContainer& InBoneContainer) : BoneContainer(InBoneContainer) {};
		
		IterType begin() { return BoneContainer.MakeBeginIter(); }
		IterType end() { return BoneContainer.MakeEndIter(); }
	};

	template<typename IterType>
	struct FRangedForReverseSupport
	{
		const FBoneContainer& BoneContainer;

		FRangedForReverseSupport(const FBoneContainer& InBoneContainer) : BoneContainer(InBoneContainer) {};

		IterType begin() { return BoneContainer.MakeBeginIterReverse(); }
		IterType end() { return BoneContainer.MakeEndIterReverse(); }
	};
	
	inline FRangedForSupport<FCompactPoseBoneIndexIterator> ForEachCompactPoseBoneIndex() const
	{
		return FRangedForSupport<FCompactPoseBoneIndexIterator>(*this);
	}
	inline FRangedForReverseSupport<FCompactPoseBoneIndexReverseIterator> ForEachCompactPoseBoneIndexReverse() const
	{
		return FRangedForReverseSupport<FCompactPoseBoneIndexReverseIterator>(*this);
	}
	inline FCompactPoseBoneIndexIterator MakeBeginIter() const
	{
		return FCompactPoseBoneIndexIterator(0);
	}
	inline FCompactPoseBoneIndexIterator MakeEndIter() const
	{
		return FCompactPoseBoneIndexIterator(GetCompactPoseNumBones());
	}
	inline FCompactPoseBoneIndexReverseIterator MakeBeginIterReverse() const
	{
		return FCompactPoseBoneIndexReverseIterator(GetCompactPoseNumBones() - 1);
	}
	inline FCompactPoseBoneIndexReverseIterator MakeEndIterReverse() const
	{
		return FCompactPoseBoneIndexReverseIterator(-1);
	}
	
	/** 
	 * Map skeleton bone index to mesh index
	 * @return	the mesh pose bone index for the specified skeleton pose bone index. Returns an invalid index if the mesh
	 *			does not include the specified skeleton bone.
	 */
	FMeshPoseBoneIndex GetMeshPoseIndexFromSkeletonPoseIndex(const FSkeletonPoseBoneIndex& SkeletonIndex) const
	{
		if (SkeletonToPoseBoneIndexArray.IsValidIndex(SkeletonIndex.GetInt()))
		{
			return FMeshPoseBoneIndex(SkeletonToPoseBoneIndexArray[SkeletonIndex.GetInt()]);
		}
		
		return FMeshPoseBoneIndex(INDEX_NONE);
	}

	/**
	 * Map mesh bone index to skeleton index
	 * @return	the skeleton pose bone index for the specified mesh pose bone index. Ensures and returns an invalid
	 *			index if the skeleton does not include the specified mesh bone.
	 */
	FSkeletonPoseBoneIndex GetSkeletonPoseIndexFromMeshPoseIndex(const FMeshPoseBoneIndex& MeshIndex) const
	{
		if (ensure(PoseToSkeletonBoneIndexArray.IsValidIndex(MeshIndex.GetInt())))
		{
			return FSkeletonPoseBoneIndex(PoseToSkeletonBoneIndexArray[MeshIndex.GetInt()]);
		}
		
		return FSkeletonPoseBoneIndex(INDEX_NONE);
	}

	// DEPRECATED - Ideally should use GetSkeletonPoseIndexFromCompactPoseIndex due to raw int32 here
	int32 GetSkeletonIndex(const FCompactPoseBoneIndex& BoneIndex) const
	{
		return CompactPoseToSkeletonIndex[BoneIndex.GetInt()];
	}

	/**
	 * Map compact bone index to skeleton index
	 * @return	the skeleton pose bone index for the specified compact pose bone index. Ensures and returns an invalid
	 *			index if the skeleton does not include the specified compact pose bone.
	 */	
	FSkeletonPoseBoneIndex GetSkeletonPoseIndexFromCompactPoseIndex(const FCompactPoseBoneIndex& BoneIndex) const
	{
		if (ensure(CompactPoseToSkeletonIndex.IsValidIndex(BoneIndex.GetInt())))
		{
			return FSkeletonPoseBoneIndex(CompactPoseToSkeletonIndex[BoneIndex.GetInt()]);
		}
		
		return FSkeletonPoseBoneIndex(INDEX_NONE);
	}

	// DEPRECATED - Ideally should use GetCompactPoseIndexFromSkeletonPoseIndex due to raw int32 here
	FCompactPoseBoneIndex GetCompactPoseIndexFromSkeletonIndex(const int32 SkeletonIndex) const
	{
		if (ensure(SkeletonToCompactPose.IsValidIndex(SkeletonIndex)))
		{
			return SkeletonToCompactPose[SkeletonIndex];
		}

		return FCompactPoseBoneIndex(INDEX_NONE);
	}

	/** 
	 * Map skeleton bone index to compact pose index
	 * @return	the compact pose bone index for the specified skeleton pose bone index. Returns an invalid index if the
	 *			compact pose does not include the specified skeleton bone.
	 */	
	FCompactPoseBoneIndex GetCompactPoseIndexFromSkeletonPoseIndex(const FSkeletonPoseBoneIndex& SkeletonIndex) const
	{
		if (SkeletonToCompactPose.IsValidIndex(SkeletonIndex.GetInt()))
		{
			return SkeletonToCompactPose[SkeletonIndex.GetInt()];
		}

		return FCompactPoseBoneIndex(INDEX_NONE);
	}

	/**
	 * Returns whether or not the skeleton index is contained in the mapping used to build this container.
	 * Note that even if the skeleton index is valid, it might not contain a valid compact pose index if
	 * that bone isn't used due to LOD or other reasons.
	 */
	bool IsSkeletonPoseIndexValid(const FSkeletonPoseBoneIndex& SkeletonIndex) const
	{
		return SkeletonToCompactPose.IsValidIndex(SkeletonIndex.GetInt());
	}

	FMeshPoseBoneIndex MakeMeshPoseIndex(const FCompactPoseBoneIndex& BoneIndex) const
	{
		return FMeshPoseBoneIndex(GetBoneIndicesArray()[BoneIndex.GetInt()]);
	}

	FCompactPoseBoneIndex MakeCompactPoseIndex(const FMeshPoseBoneIndex& BoneIndex) const
	{
		return FCompactPoseBoneIndex(GetBoneIndicesArray().IndexOfByKey(BoneIndex.GetInt()));
	}

	UE_DEPRECATED(5.3, "Please use CacheRequiredAnimCurves with a FCurveFilterSettings.")
	void CacheRequiredAnimCurveUids(const FCurveEvaluationOption& CurveEvalOption)
	{
		const UE::Anim::FCurveFilterSettings CurveFilterSettings(CurveEvalOption.bAllowCurveEvaluation ? UE::Anim::ECurveFilterMode::None : UE::Anim::ECurveFilterMode::DisallowAll, CurveEvalOption.DisallowedList, CurveEvalOption.LODIndex);
		CacheRequiredAnimCurves(CurveFilterSettings);
	}

	/** Cache required Anim Curves */
	ENGINE_API void CacheRequiredAnimCurves(const UE::Anim::FCurveFilterSettings& InCurveFilterSettings);
	
	ENGINE_API const FRetargetSourceCachedData& GetRetargetSourceCachedData(const FName& InRetargetSource) const;
	ENGINE_API const FRetargetSourceCachedData& GetRetargetSourceCachedData(const FName& InSourceName, const FSkeletonRemapping& InRemapping, const TArray<FTransform>& InRetargetTransforms) const;

#if DO_CHECK
	/** Get the LOD that we calculated required bones when regenerated */
	int32 GetCalculatedForLOD() const { return CalculatedForLOD; }
#endif
	
	// Curve remapping
	ENGINE_API const FCachedSkeletonCurveMapping& GetOrCreateCachedCurveMapping(const FSkeletonRemapping* SkeletonRemapping);
	ENGINE_API void MarkAllCachedCurveMappingsDirty();

	// Get the serial number of this bone container. @see SerialNumber
	uint16 GetSerialNumber() const { return SerialNumber; }

private:
	/** 
	 * Runtime cached data for retargeting from a specific RetargetSource to this current SkelMesh LOD.
	 * @todo: We could also cache this once per skelmesh per lod, rather than creating it at runtime for each skelmesh instance.
	 */
	mutable TMap<FRetargetSourceCachedDataKey, FRetargetSourceCachedData> RetargetSourceCachedDataLUT;

	/** Initialize FBoneContainer. */
	ENGINE_API void Initialize(const UE::Anim::FCurveFilterSettings& CurveFilterSettings);

	/** Cache remapping data if current Asset is a SkeletalMesh, with all compatible Skeletons. */
	ENGINE_API void RemapFromSkelMesh(USkeletalMesh const & SourceSkeletalMesh, USkeleton& TargetSkeleton);

	/** Cache remapping data if current Asset is a Skeleton, with all compatible Skeletons. */
	ENGINE_API void RemapFromSkeleton(USkeleton const & SourceSkeleton);

	// Regenerate the serial number after internal data is updated
	ENGINE_API void RegenerateSerialNumber();
};


