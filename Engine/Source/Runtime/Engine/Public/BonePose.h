// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "BoneIndices.h"
#include "Animation/AnimTypes.h"
#include "CustomBoneIndexArray.h"
#include "Animation/AnimStats.h"
#include "Misc/Base64.h"
#include "Animation/Skeleton.h"
#include "BoneContainer.h"

struct FBoneTransform
{
	/** @todo anim: should be Skeleton bone index in the future, but right now it's CompactBoneIndex **/
	FCompactPoseBoneIndex BoneIndex;

	/** Transform to apply **/
	FTransform Transform;

	FBoneTransform()
		: BoneIndex(INDEX_NONE)
	{}

	FBoneTransform(FCompactPoseBoneIndex InBoneIndex, const FTransform& InTransform)
		: BoneIndex(InBoneIndex)
		, Transform(InTransform)
	{}
};

// Comparison Operator for Sorting.
struct FCompareBoneTransformIndex
{
	inline bool operator()(const FBoneTransform& A, const FBoneTransform& B) const
	{
		return A.BoneIndex < B.BoneIndex;
	}
};


template<class BoneIndexType, typename InAllocator>
struct FBasePose
{
public:
	inline void InitBones(int32 NumBones) { Bones.Reset(NumBones); Bones.AddUninitialized(NumBones); }

	inline int32 GetNumBones() const { return Bones.Num(); }

	inline bool IsValidIndex(const BoneIndexType& BoneIndex) const
	{
		return Bones.IsValidIndex(BoneIndex.GetInt());
	}

	inline FTransform& operator[](const BoneIndexType& BoneIndex)
	{ 
		return Bones[BoneIndex.GetInt()];
	}

	inline const FTransform& operator[] (const BoneIndexType& BoneIndex) const
	{
		return Bones[BoneIndex.GetInt()];
	}

	inline TArrayView<FTransform> GetMutableBones()
	{
		return TArrayView<FTransform>(Bones);
	}

	//Bone Index Iteration
	template<typename PoseType, typename IterType>
	struct FRangedForSupport
	{
		const PoseType& Pose;

		FRangedForSupport(const PoseType& InPose) : Pose(InPose) {};
		
		IterType begin() { return Pose.MakeBeginIter(); }
		IterType end() { return Pose.MakeEndIter(); }
	};

	template<typename PoseType, typename IterType>
	struct FRangedForReverseSupport
	{
		const PoseType& Pose;

		FRangedForReverseSupport(const PoseType& InPose) : Pose(InPose) {};

		IterType begin() { return Pose.MakeBeginIterReverse(); }
		IterType end() { return Pose.MakeEndIterReverse(); }
	};

	const TArray<FTransform, InAllocator>& GetBones() const { return Bones; }

	TArray<FTransform, InAllocator>&& MoveBones() { return MoveTemp(Bones); }

protected:
	TArray<FTransform, InAllocator> Bones;
};

template <typename InAllocator>
struct FBaseCompactPose : FBasePose<FCompactPoseBoneIndex, InAllocator>
{
public:

	FBaseCompactPose()
		: BoneContainer(nullptr)
	{}

	typedef FCompactPoseBoneIndex BoneIndexType;
	typedef InAllocator   Allocator;
	//--------------------------------------------------------------------------
	//Bone Index Iteration
	typedef typename FBasePose<FCompactPoseBoneIndex, Allocator>::template FRangedForSupport<FBaseCompactPose, FCompactPoseBoneIndexIterator> RangedForBoneIndexFwd;
	typedef typename FBasePose<FCompactPoseBoneIndex, Allocator>::template FRangedForReverseSupport<FBaseCompactPose, FCompactPoseBoneIndexReverseIterator> RangedForBoneIndexBwd;

	inline RangedForBoneIndexFwd ForEachBoneIndex() const
	{
		return RangedForBoneIndexFwd(*this);
	}

	inline RangedForBoneIndexBwd ForEachBoneIndexReverse() const
	{
		return RangedForBoneIndexBwd(*this);
	}

	inline FCompactPoseBoneIndexIterator MakeBeginIter() const { return FCompactPoseBoneIndexIterator(0); }

	inline FCompactPoseBoneIndexIterator MakeEndIter() const { return FCompactPoseBoneIndexIterator(this->GetNumBones()); }

	inline FCompactPoseBoneIndexReverseIterator MakeBeginIterReverse() const { return FCompactPoseBoneIndexReverseIterator(this->GetNumBones() - 1); }

	inline FCompactPoseBoneIndexReverseIterator MakeEndIterReverse() const { return FCompactPoseBoneIndexReverseIterator(-1); }
	//--------------------------------------------------------------------------

	const FBoneContainer& GetBoneContainer() const 
	{
		checkSlow(BoneContainer && BoneContainer->IsValid());
		return *BoneContainer;
	}

	FBoneContainer& GetBoneContainer()
	{
		checkSlow(BoneContainer && BoneContainer->IsValid());
		return *const_cast<FBoneContainer*>(BoneContainer);
	}

	void SetBoneContainer(const FBoneContainer* InBoneContainer)
	{
		check(InBoneContainer && InBoneContainer->IsValid());
		BoneContainer = InBoneContainer;
		this->InitBones(BoneContainer->GetBoneIndicesArray().Num());
	}

	void CopyAndAssignBoneContainer(FBoneContainer& NewBoneContainer)
	{
		NewBoneContainer = *BoneContainer;
		BoneContainer = &NewBoneContainer;
	}

	void InitFrom(const FBaseCompactPose& SrcPose)
	{
		SetBoneContainer(SrcPose.BoneContainer);
		this->Bones = SrcPose.Bones;
	}

	void InitFrom(FBaseCompactPose&& SrcPose)
	{
		SetBoneContainer(SrcPose.BoneContainer);
		this->Bones = MoveTemp(SrcPose.Bones);
		SrcPose.BoneContainer = nullptr;
	}

	// Copy bone transform from SrcPose to this
	template <typename OtherAllocator>
	void CopyBonesFrom(const FBaseCompactPose<OtherAllocator>& SrcPose)
	{
		this->Bones = SrcPose.GetBones();
		BoneContainer = &SrcPose.GetBoneContainer();
	}

	void CopyBonesFrom(const FBaseCompactPose<Allocator>& SrcPose)
	{
		if (this != &SrcPose)
		{
			this->Bones = SrcPose.GetBones();
			BoneContainer = &SrcPose.GetBoneContainer();
		}
	}
	
	void MoveBonesFrom(FBaseCompactPose<Allocator>& SrcPose)
	{
		if (this != &SrcPose)
		{
			this->Bones = SrcPose.MoveBones();
			BoneContainer = &SrcPose.GetBoneContainer();
			SrcPose.BoneContainer = nullptr;
		}
	}

	template <typename OtherAllocator>
	void CopyBonesFrom(const TArray<FTransform, OtherAllocator>& SrcPoseBones)
	{
		// only allow if the size is same
		// if size doesn't match, we can't guarantee the bonecontainer would work
		// so we can't accept
		if (this->Bones.Num() == SrcPoseBones.Num())
		{
			this->Bones = SrcPoseBones;
		}
	}

	template <typename OtherAllocator>
	void CopyBonesTo(TArray<FTransform, OtherAllocator>& DestPoseBones) const
	{
		// this won't work if you're copying to FBaseCompactPose without BoneContainer data
		// you'll like to make CopyBonesTo(FBaseCompactPose<OtherAllocator>& DestPose) to fix this properly
		// if you need bone container
		DestPoseBones = this->Bones;
	}

	// Moves transform data out of the supplied InTransforms. InTransform will be left empty
	void MoveBonesFrom(TArray<FTransform, Allocator>&& InTransforms)
	{
		this->Bones = MoveTemp(InTransforms);
	}

	// Moves transform data out of this to the supplied InTransforms. Bones will be left empty
	void MoveBonesTo(TArray<FTransform, Allocator>& InTransforms)
	{
		InTransforms = MoveTemp(this->Bones);
	}

	void Empty()
	{
		BoneContainer = nullptr;
		this->Bones.Empty();
	}

	// Sets this pose to its ref pose
	void ResetToRefPose()
	{
		ResetToRefPose(GetBoneContainer());
	}

	// Sets this pose to the supplied BoneContainers ref pose
	void ResetToRefPose(const FBoneContainer& RequiredBones)
	{
		RequiredBones.FillWithCompactRefPose(this->Bones);

		this->BoneContainer = &RequiredBones;

		// If retargeting is disabled, copy ref pose from Skeleton, rather than mesh.
		// this is only used in editor and for debugging.
		if (RequiredBones.GetDisableRetargeting())
		{
			checkSlow(RequiredBones.IsValid());
			// Only do this if we have a mesh. otherwise we're not retargeting animations.
			if (RequiredBones.GetSkeletalMeshAsset())
			{
				TArray<FTransform> const & SkeletonRefPose = RequiredBones.GetSkeletonAsset()->GetRefLocalPoses();

				for (const FCompactPoseBoneIndex BoneIndex : ForEachBoneIndex())
				{
					const int32 SkeletonBoneIndex = GetBoneContainer().GetSkeletonIndex(BoneIndex);

					// Pose bone index should always exist in Skeleton
					checkSlow(SkeletonBoneIndex != INDEX_NONE);
					this->Bones[BoneIndex.GetInt()] = SkeletonRefPose[SkeletonBoneIndex];
				}
			}
		}
	}

	// Sets every bone transform to Identity
	void ResetToAdditiveIdentity()
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.SetIdentityZeroScale();
		}
	}

	// returns true if all bone rotations are normalized
	bool IsNormalized() const
	{
		for (const FTransform& Bone : this->Bones)
		{
			if (!Bone.IsRotationNormalized())
			{
				return false;
			}
		}

		return true;
	}

	// Returns true if any bone rotation contains NaN or Inf
	bool ContainsNaN() const
	{
		for (const FTransform& Bone : this->Bones)
		{
			if (Bone.ContainsNaN())
			{
				return true;
			}
		}

		return false;
	}


	// Normalizes all rotations in this pose
	void NormalizeRotations()
	{
		for (FTransform& Bone : this->Bones)
		{
			Bone.NormalizeRotation();
		}
	}

	bool IsValid() const
	{
		return (BoneContainer && BoneContainer->IsValid());
	}

	// Returns the bone index for the parent bone
	BoneIndexType GetParentBoneIndex(const BoneIndexType& BoneIndex) const
	{
		return GetBoneContainer().GetParentBoneIndex(BoneIndex);
	}

	// Returns the ref pose for the supplied bone
	const FTransform& GetRefPose(const BoneIndexType& BoneIndex) const
	{
		return GetBoneContainer().GetRefPoseTransform(BoneIndex);
	}

protected:

	//Reference to our BoneContainer
	const FBoneContainer* BoneContainer;
};

struct FCompactPose : public FBaseCompactPose<FAnimStackAllocator>
{
	// Sets every bone transform to Identity
	ENGINE_API void ResetToAdditiveIdentity();

	// Normalizes all rotations in this pose
	ENGINE_API void NormalizeRotations();
};

struct FCompactHeapPose : public FBaseCompactPose<FDefaultAllocator>
{
	// Sets every bone transform to Identity
	ENGINE_API void ResetToAdditiveIdentity();

	// Normalizes all rotations in this pose
	ENGINE_API void NormalizeRotations();
};

struct FMeshPose : public FBasePose<FMeshPoseBoneIndex, FDefaultAllocator>
{
public:
	typedef FMeshPoseBoneIndex BoneIndexType;

	const FBoneContainer& GetBoneContainer() const
	{
		checkSlow(BoneContainer && BoneContainer->IsValid());
		return *BoneContainer;
	}

	void SetBoneContainer(const FBoneContainer* InBoneContainer)
	{
		check(InBoneContainer && InBoneContainer->IsValid());
		BoneContainer = InBoneContainer;
		InitBones(BoneContainer->GetNumBones());
	}

	// Sets this pose to its ref pose
	ENGINE_API void ResetToRefPose();

	// Sets every bone transform to Identity
	void ResetToIdentity();

	// returns true if all bone rotations are normalized
	bool IsNormalized() const;

	// Returns true if any bone rotation contains NaN
	bool ContainsNaN() const;


	inline BoneIndexType GetParentBone(const BoneIndexType& BoneIndex)
	{
		return BoneIndexType(BoneContainer->GetParentBoneIndex(BoneIndex.GetInt()));
	}

protected:

	//Reference to our BoneContainer
	const FBoneContainer* BoneContainer;
};

template<class PoseType>
struct FCSPose
{
	// Set up our index type based on the type of pose we are manipulating
	typedef typename PoseType::BoneIndexType BoneIndexType;
	typedef typename PoseType::Allocator AllocatorType;
	
	// overriding ctor, copy, move to allow deprecations.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCSPose() = default;
	~FCSPose() = default;
	FCSPose(const FCSPose& Other) = default;
	FCSPose(FCSPose&& Other) = default;
	FCSPose& operator=(const FCSPose& Other) = default;
	FCSPose& operator=(FCSPose&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Init Pose
	void InitPose(const FBoneContainer* InBoneContainer)
	{
		Pose.SetBoneContainer(InBoneContainer);
		Pose.ResetToRefPose();
		ComponentSpaceFlags.Empty(Pose.GetNumBones());
		ComponentSpaceFlags.AddZeroed(Pose.GetNumBones());
		ComponentSpaceFlags[0] = 1;
	}

	// Init Pose
	void InitPose(const PoseType& SrcPose)
	{
		Pose.InitFrom(SrcPose);
		ComponentSpaceFlags.Empty(Pose.GetNumBones());
		ComponentSpaceFlags.AddZeroed(Pose.GetNumBones());
		ComponentSpaceFlags[0] = 1;
	}

	void InitPose(PoseType&& SrcPose)
	{
		Pose.InitFrom(MoveTemp(SrcPose));
		ComponentSpaceFlags.Empty(Pose.GetNumBones());
		ComponentSpaceFlags.AddZeroed(Pose.GetNumBones());
		ComponentSpaceFlags[0] = 1;
	}

	// Copy Pose
	template <typename OtherPoseType>
	void CopyPose(const OtherPoseType& SrcPose)
	{
		Pose.CopyBonesFrom(SrcPose.GetPose());
		ComponentSpaceFlags = SrcPose.GetComponentSpaceFlags();
	}

	void CopyAndAssignBoneContainer(FBoneContainer& NewBoneContainer)
	{
		Pose.CopyAndAssignBoneContainer(NewBoneContainer);
	}

	void Empty()
	{
		Pose.Empty();
		ComponentSpaceFlags.Empty();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BonesToConvert.Empty();
		BoneMask.Empty();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const PoseType& GetPose() const { return Pose; }
	const TCustomBoneIndexArray<uint8, BoneIndexType, AllocatorType>& GetComponentSpaceFlags() const { return ComponentSpaceFlags; }

	// Get transform for supplied bone in local space
	FTransform GetLocalSpaceTransform(BoneIndexType BoneIndex);

	// Get Transform for supplied bone in component space
	const FTransform& GetComponentSpaceTransform(BoneIndexType BoneIndex);

	// Set the transform for the supplied bone 
	void SetComponentSpaceTransform(BoneIndexType BoneIndex, const FTransform& NewTransform);

	// Calculate the component space transform for the supplied bone
	void CalculateComponentSpaceTransform(BoneIndexType BoneIndex);

	/**
	* Convert Bone to Local Space.
	*/
	void ConvertBoneToLocalSpace(BoneIndexType BoneIndex);

	/**
	* Set a bunch of Component Space Bone Transforms.
	* Do this safely by insuring that Parents are already in Component Space,
	* and any Component Space children are converted back to Local Space before hand.
	*/
	void SafeSetCSBoneTransforms(TConstArrayView<FBoneTransform> BoneTransforms);

	/**
	* Blends Component Space transforms to MeshPose in Local Space.
	* Used by SkelControls to apply their transforms.
	*
	* The tricky bit is that SkelControls deliver their transforms in Component Space,
	* But the blending is done in Local Space. Also we need to refresh any Children they have
	* that has been previously converted to Component Space.
	*/
	inline void LocalBlendCSBoneTransforms(TConstArrayView<FBoneTransform> BoneTransforms, float Alpha);

	/** This function convert component space to local space to OutPose 
	 *
	 * This has been optimized with an assumption of
	 * all parents are calculated in component space for those who have been converted to component space
	 * After this function, accessing InPose.Pose won't be valid anymore because of Move semantics
	 *
	 * If a part of chain hasn't been converted, it will trigger ensure. 
	 */
	inline static void ConvertComponentPosesToLocalPoses(FCSPose<PoseType>&& InPose, PoseType& OutPose);
	
	/** This function convert component space to local space to OutPose 
	 *
	 * This has been optimized with an assumption of
	 * all parents are calculated in component space for those who have been converted to component space
	 *
	 * If a part of chain hasn't been converted, it will trigger ensure. 
	 */
	inline static void ConvertComponentPosesToLocalPoses(const FCSPose<PoseType>& InPose, PoseType& OutPose);

	/**
	 * This function convert component space to local space to OutPose
	 *
	 * Contrast to ConvertComponentPosesToLocalPoses, 
	 * this allows some parts of chain to stay in local space before conversion
	 * And it will calculate back to component space correctly before converting back to new local space
	 * This is issue when child is in component space, but the parent is not. 
	 * Then we have to convert parents to be in the component space before converting back to local
	 *
	 * However it is more expensive as a result. 
	 */
	inline static void ConvertComponentPosesToLocalPosesSafe(FCSPose<PoseType>& InPose, PoseType& OutPose);
protected:
	PoseType Pose;

	// Flags to track each bones current state (0 means local pose, 1 means component space pose)
	TCustomBoneIndexArray<uint8, BoneIndexType, AllocatorType> ComponentSpaceFlags;

	UE_DEPRECATED(5.7, "Property no longer required. We now allocate it via TMemStackAllocator directly.")
	TCustomBoneIndexArray<uint8, BoneIndexType, AllocatorType> BoneMask;

	UE_DEPRECATED(5.7, "Property no longer required. We now allocate it via TMemStackAllocator directly.")
	TArray<FCompactPoseBoneIndex, AllocatorType> BonesToConvert;
};

template<class PoseType>
FTransform FCSPose<PoseType>::GetLocalSpaceTransform(BoneIndexType BoneIndex)
{
	checkSlow(Pose.IsValid());
	check(Pose.IsValidIndex(BoneIndex));

	// if evaluated, calculate it
	if (ComponentSpaceFlags[BoneIndex])
	{
		const BoneIndexType ParentIndex = Pose.GetParentBoneIndex(BoneIndex);

		if (ParentIndex != INDEX_NONE)
		{
			const FTransform& ParentTransform = GetComponentSpaceTransform(ParentIndex);
			const FTransform& BoneTransform = Pose[BoneIndex];
			// calculate local space
			return BoneTransform.GetRelativeTransform(ParentTransform);
		}
	}

	return Pose[BoneIndex];
}

template<class PoseType>
const FTransform& FCSPose<PoseType>::GetComponentSpaceTransform(BoneIndexType BoneIndex)
{
	checkSlow(Pose.IsValid());
	check(Pose.IsValidIndex(BoneIndex));	// Invalid index supplied. If this came from an FBoneReference make sure you are
											// handling lod changes properly. (for instance: if on an anim node initialize the reference in CacheBones)

	check(!Pose[BoneIndex].ContainsNaN());
	// if not evaluate, calculate it
	if (ComponentSpaceFlags[BoneIndex] == 0)
	{
		CalculateComponentSpaceTransform(BoneIndex);
	}
	check(!Pose[BoneIndex].ContainsNaN());
	return Pose[BoneIndex];
}

template<class PoseType>
void FCSPose<PoseType>::SetComponentSpaceTransform(BoneIndexType BoneIndex, const FTransform& NewTransform)
{
	checkSlow(Pose.IsValid());
	check(Pose.IsValidIndex(BoneIndex));

	// this one forcefully sets component space transform
	Pose[BoneIndex] = NewTransform;
	ComponentSpaceFlags[BoneIndex] = 1;
}

template<class PoseType>
void FCSPose<PoseType>::CalculateComponentSpaceTransform(BoneIndexType BoneIndex)
{
	checkSlow(Pose.IsValid());
	check(ComponentSpaceFlags[BoneIndex] == 0);

	FMemMark Mark(FMemStack::Get());
	TArray<BoneIndexType, TMemStackAllocator<>> BoneIndexStack;
	BoneIndexStack.Reserve(ComponentSpaceFlags.Num());
	BoneIndexStack.Add(BoneIndex);	//Add a dummy index to avoid last element checks in the loop

	do
	{
		// root is already verified, so root should not come here
		// check AllocateLocalPoses
		const BoneIndexType ParentIndex = Pose.GetParentBoneIndex(BoneIndex);

		// if Parent already has been calculated, use it
		if (ComponentSpaceFlags[ParentIndex] == 0)
		{
			BoneIndexStack.Add(BoneIndex);
			BoneIndex = ParentIndex;
			continue;
		}

		// current Bones(Index) should contain LocalPoses.
		FTransform& Bone = Pose[BoneIndex];
		FTransform& ParentBone = Pose[ParentIndex];
		
		checkf(!Bone.ContainsNaN(), TEXT("Pose[%d] contains NaN: %s\nBase64: %s"),
			BoneIndex.GetInt(),
			*Bone.ToString(),
			*FBase64::Encode(reinterpret_cast<const uint8*>(&Bone), sizeof(FTransform))
		);

		checkf(!ParentBone.ContainsNaN(), TEXT("Pose[%d] contains NaN: %s\nBase64: %s"),
			ParentIndex.GetInt(),
			*ParentBone.ToString(),
			*FBase64::Encode(reinterpret_cast<const uint8*>(&ParentBone), sizeof(FTransform))
		);

		FTransform ComponentTransform = Bone * ParentBone;
		if (ComponentTransform.ContainsNaN())
		{
			// We've failed, output as much info as we can....
			// Added for Jira UE-55511
			auto BoolToStr = [](const bool& bValue) { return bValue ? TEXT("true") : TEXT("false"); };

			const TCHAR* BoneHasNaN = BoolToStr(Bone.ContainsNaN());
			const TCHAR* ParentHasNaN = BoolToStr(ParentBone.ContainsNaN());
			FString ErrorMsg = FString("NaN created in during FTransform Multiplication\n");
			ErrorMsg += FString::Format(TEXT("\tBoneIndex {0} : ParentBoneIndex {1} BoneTransformNaN={2} : ParentTransformNaN={3}\n"), { BoneIndex.GetInt(), ParentIndex.GetInt(), BoneHasNaN, ParentHasNaN });
			ErrorMsg += FString::Format(TEXT("\tBone {0}\n"), { Bone.ToString() });
			ErrorMsg += FString::Format(TEXT("\tParent {0}\n"), { ParentBone.ToString() });
			ErrorMsg += FString::Format(TEXT("\tResult {0}\n"), { ComponentTransform.ToString() });
			ErrorMsg += FString::Format(TEXT("\tBone B64 {0}\n"), { FBase64::Encode((uint8*)&Bone, sizeof(FTransform)) });
			ErrorMsg += FString::Format(TEXT("\tParent B64 {0}\n"), { FBase64::Encode((uint8*)&ParentBone, sizeof(FTransform)) });
			ErrorMsg += FString::Format(TEXT("\tResult B64 {0}\n"), { FBase64::Encode((uint8*)&ComponentTransform, sizeof(FTransform)) });
			checkf(false, TEXT("Error during CalculateComponentSpaceTransform\n%s"), *ErrorMsg); // Failed during multiplication
		}
		Bone = ComponentTransform;
		Bone.NormalizeRotation();
		
		checkf(!Bone.ContainsNaN(), TEXT("Normalized ComponentTransform for Pose[%d] contains NaN: %s\nBase64: %s"),
			BoneIndex.GetInt(),
			*Bone.ToString(),
			*FBase64::Encode(reinterpret_cast<const uint8*>(&Bone), sizeof(FTransform))
		);

		ComponentSpaceFlags[BoneIndex] = 1;

		BoneIndex = BoneIndexStack.Pop(EAllowShrinking::No);
	} while (BoneIndexStack.Num());
}

template<class PoseType>
void FCSPose<PoseType>::ConvertBoneToLocalSpace(BoneIndexType BoneIndex)
{
	checkSlow(Pose.IsValid());

	// If BoneTransform is in Component Space, then convert it.
	// Never convert Root to Local Space.
	if (!BoneIndex.IsRootBone() && ComponentSpaceFlags[BoneIndex] == 1)
	{
		const BoneIndexType ParentIndex = Pose.GetParentBoneIndex(BoneIndex);

		// Verify that our Parent is also in Component Space. That should always be the case.
		check(ComponentSpaceFlags[ParentIndex] == 1);

		// Convert to local space.
		Pose[BoneIndex].SetToRelativeTransform(Pose[ParentIndex]);
		ComponentSpaceFlags[BoneIndex] = 0;
	}
}

template<class PoseType>
void FCSPose<PoseType>::SafeSetCSBoneTransforms(TConstArrayView<FBoneTransform> BoneTransforms)
{
	checkSlow(Pose.IsValid());

	FMemMark Mark(FMemStack::Get());
	TArray<FCompactPoseBoneIndex, TMemStackAllocator<>> StackBonesToConvert;
	StackBonesToConvert.Reserve(BoneTransforms.Num());

	// Minimum bone index, we don't need to look at bones prior to this in the pose
	const int32 MinIndex = BoneTransforms[0].BoneIndex.GetInt();

	// Add BoneTransforms indices if they're in component space
	for(const FBoneTransform& Transform : BoneTransforms)
	{
		if(ComponentSpaceFlags[Transform.BoneIndex] == 1)
		{
			StackBonesToConvert.Add(Transform.BoneIndex);
		}
	}

	// Store the beginning of the child transforms, below we don't need to convert any bone added
	// from BoneTransforms because they're about to be overwritten
	const int32 FirstChildTransform = StackBonesToConvert.Num();

	FCompactPoseBoneIndexIterator Iter = FCompactPoseBoneIndexIterator(MinIndex);
	FCompactPoseBoneIndexIterator EndIter = Pose.MakeEndIter();

	// Add child bones if they're in component space
	for(; Iter != EndIter; ++Iter)
	{
		const FCompactPoseBoneIndex BoneIndex = *Iter;
		const FCompactPoseBoneIndex ParentIndex = Pose.GetParentBoneIndex(BoneIndex);

		if(ComponentSpaceFlags[BoneIndex] == 1 && StackBonesToConvert.Contains(ParentIndex))
		{
			StackBonesToConvert.AddUnique(BoneIndex);
		}
	}

	// Convert the bones, we walk backwards to process children first, the pose iteration above is sorted
	// so we know we already have the right order. We also stop when we get to the bones contained in
	// BoneTransforms because we're about to overwrite them anyway
	const int32 NumToConvert = StackBonesToConvert.Num();
	for(int32 Idx = NumToConvert - 1; Idx >= FirstChildTransform; --Idx)
	{
		ConvertBoneToLocalSpace(StackBonesToConvert[Idx]);
	}

	// Finally copy our Component Space transforms
	for (const FBoneTransform& BoneTransform : BoneTransforms)
	{
		const FCompactPoseBoneIndex BoneIndex = BoneTransform.BoneIndex;

		// Make sure our BoneTransforms were in Component Space in the first place, before we overwrite them
		// Only check their parent to do minimal work needed.
		const FCompactPoseBoneIndex ParentBoneIndex = Pose.GetParentBoneIndex(BoneIndex);
		if (ParentBoneIndex != INDEX_NONE && ComponentSpaceFlags[ParentBoneIndex] == 0)
		{
			CalculateComponentSpaceTransform(ParentBoneIndex);
		}

		// Set new Component Space transform.
		SetComponentSpaceTransform(BoneIndex, BoneTransform.Transform);
	}
}

template<class PoseType>
void FCSPose<PoseType>::LocalBlendCSBoneTransforms(TConstArrayView<FBoneTransform> BoneTransforms, float Alpha)
{
	// if Alpha is small enough, skip
	if (Alpha < ZERO_ANIMWEIGHT_THRESH)
	{
		return;
	}

#if DO_CHECK
	if (BoneTransforms.Num() > 0)
	{
		FCompactPoseBoneIndex LastIndex(BoneTransforms[0].BoneIndex);
		// Make sure bones are sorted in "Parents before Children" order.
		for (int32 I = 1; I < BoneTransforms.Num(); ++I)
		{
			check(BoneTransforms[I].BoneIndex >= LastIndex);
			LastIndex = BoneTransforms[I].BoneIndex;
		}
	}
#endif

	// If we are not doing any blending, do a faster path.
	// Set transforms directly in Component Space. But still refresh children.
	if (Alpha >= 1.f - ZERO_ANIMWEIGHT_THRESH)
	{
		SafeSetCSBoneTransforms(BoneTransforms);
	}
	// Slower blending path.
	else
	{
		// Bone Mask to keep track of which bones have to be converted to local space.
		// This is basically BoneTransforms bones and their children.
		FMemMark Mark(FMemStack::Get());
		TCustomBoneIndexArray<uint8, BoneIndexType, TMemStackAllocator<>> StackBoneMask;
		StackBoneMask.AddZeroed(Pose.GetNumBones());

		TArray<FBoneTransform, TMemStackAllocator<>> LocalBoneTransforms;
		LocalBoneTransforms.SetNumUninitialized(BoneTransforms.Num());

		// First, save the current local-space poses for the modified bones
		for (int32 Index = 0; Index < BoneTransforms.Num(); Index++)
		{
			const BoneIndexType BoneIndex = BoneTransforms[Index].BoneIndex;

			// save current local pose - we will blend it back in later
			LocalBoneTransforms[Index].Transform = GetLocalSpaceTransform(BoneIndex);
			LocalBoneTransforms[Index].BoneIndex = BoneIndex;

			StackBoneMask[BoneIndex] = 1;
		}

		// Next, update the pose as if Alpha = 1.0
		SafeSetCSBoneTransforms(BoneTransforms);

		// Then, convert MeshPose Bones from BoneTransforms list, and their children, to local space if they are not already.
		for (const BoneIndexType BoneIndex : Pose.ForEachBoneIndex())
		{
			const BoneIndexType ParentIndex = Pose.GetParentBoneIndex(BoneIndex);
			// Propagate our BoneMask to children.
			if (ParentIndex != INDEX_NONE)
			{
				StackBoneMask[BoneIndex] |= StackBoneMask[ParentIndex];
			}
		}

		for (const BoneIndexType BoneIndex : Pose.ForEachBoneIndexReverse())
		{
			if (!BoneIndex.IsRootBone())
			{
				// If this bone has to be converted to Local Space...
				if (StackBoneMask[BoneIndex] != 0)
				{
					// .. And is not currently in Local Space, then convert it.
					ConvertBoneToLocalSpace(BoneIndex);
				}
			}
		}

		// Lastly, do the blending in local space.
		for (int32 Index = 0; Index < LocalBoneTransforms.Num(); Index++)
		{
			const FCompactPoseBoneIndex BoneIndex = LocalBoneTransforms[Index].BoneIndex;
			// Make sure this transform is in local space, because we are writing a local space one to it.
			// If we are not in local space, this could mean trouble for our children.
			check((ComponentSpaceFlags[BoneIndex] == 0) || (BoneIndex == 0));

			// No need to normalize rotation since BlendWith() does it.
			const float AlphaInv = 1.0f - Alpha;
			Pose[BoneIndex].BlendWith(LocalBoneTransforms[Index].Transform, AlphaInv);
		}
	}
}

template<class PoseType>
void FCSPose<PoseType>::ConvertComponentPosesToLocalPoses(const FCSPose<PoseType>& InPose, PoseType& OutPose)
{
	checkSlow(InPose.Pose.IsValid());
	OutPose = InPose.Pose;

	// now we need to convert back to local bases
	// only convert back that has been converted to mesh base
	// if it was local base, and if it hasn't been modified
	// that is still okay even if parent is changed, 
	// that doesn't mean this local has to change
	// go from child to parent since I need parent inverse to go back to local
	// root is same, so no need to do Index == 0
	const BoneIndexType RootBoneIndex(0);
	if (InPose.ComponentSpaceFlags[RootBoneIndex])
	{
		OutPose[RootBoneIndex] = InPose.Pose[RootBoneIndex];
	}

	const int32 NumBones = InPose.Pose.GetNumBones();
	for (int32 Index = NumBones - 1; Index > 0; Index--)
	{
		const BoneIndexType BoneIndex(Index);
		if (InPose.ComponentSpaceFlags[BoneIndex])
		{
			const BoneIndexType ParentIndex = InPose.Pose.GetParentBoneIndex(BoneIndex);
			// ensure if parent hasn't been calculated, otherwise, this assumption is not correct
			ensureMsgf(InPose.ComponentSpaceFlags[ParentIndex], TEXT("Parent hasn't been calculated. Please use ConvertComponentPosesToLocalPosesSafe instead"));

			OutPose[BoneIndex].SetToRelativeTransform(OutPose[ParentIndex]);
			OutPose[BoneIndex].NormalizeRotation();
		}
	}
}

template<class PoseType>
void FCSPose<PoseType>::ConvertComponentPosesToLocalPoses(FCSPose<PoseType>&& InPose, PoseType& OutPose)
{
	checkSlow(InPose.Pose.IsValid());

	const int32 NumBones = InPose.Pose.GetNumBones();

	// now we need to convert back to local bases
	// only convert back that has been converted to mesh base
	// if it was local base, and if it hasn't been modified
	// that is still okay even if parent is changed, 
	// that doesn't mean this local has to change
	// go from child to parent since I need parent inverse to go back to local
	// root is same, so no need to do Index == 0
	const BoneIndexType RootBoneIndex(0);
	if (InPose.ComponentSpaceFlags[RootBoneIndex])
	{
		OutPose[RootBoneIndex] = InPose.Pose[RootBoneIndex];
	}

	OutPose = MoveTemp(InPose.Pose);
	
	for (int32 Index = NumBones - 1; Index > 0; Index--)
	{
		const BoneIndexType BoneIndex(Index);
		if (InPose.ComponentSpaceFlags[BoneIndex])
		{
			const BoneIndexType ParentIndex = OutPose.GetParentBoneIndex(BoneIndex);
			
			// ensure if parent hasn't been calculated, otherwise, this assumption is not correct
			// Pose has moved, but ComponentSpaceFlags should be safe here
			ensureMsgf(InPose.ComponentSpaceFlags[ParentIndex], TEXT("Parent hasn't been calculated. Please use ConvertComponentPosesToLocalPosesSafe instead"));
			
			OutPose[BoneIndex].SetToRelativeTransform(OutPose[ParentIndex]);
			OutPose[BoneIndex].NormalizeRotation();
		}
	}
}

template<class PoseType>
void FCSPose<PoseType>::ConvertComponentPosesToLocalPosesSafe(FCSPose<PoseType>& InPose, PoseType& OutPose)
{
	checkSlow(InPose.Pose.IsValid());

	const int32 NumBones = InPose.Pose.GetNumBones();

	// now we need to convert back to local bases
	// only convert back that has been converted to mesh base
	// if it was local base, and if it hasn't been modified
	// that is still okay even if parent is changed, 
	// that doesn't mean this local has to change
	// go from child to parent since I need parent inverse to go back to local
	// root is same, so no need to do Index == 0
	const BoneIndexType RootBoneIndex(0);
	if (InPose.ComponentSpaceFlags[RootBoneIndex])
	{
		OutPose[RootBoneIndex] = InPose.Pose[RootBoneIndex];
	}

	
	for (int32 Index = NumBones - 1; Index > 0; Index--)
	{
		const BoneIndexType BoneIndex(Index);
		if (InPose.ComponentSpaceFlags[BoneIndex])
		{
			const BoneIndexType ParentIndex = OutPose.GetParentBoneIndex(BoneIndex);

			// if parent is local space, we have to calculate parent
			if (!InPose.ComponentSpaceFlags[ParentIndex])
			{
				// if I'm calculated, but not parent, update parent
				InPose.CalculateComponentSpaceTransform(ParentIndex);
			}

			OutPose[BoneIndex] = InPose.Pose[BoneIndex];
			OutPose[BoneIndex].SetToRelativeTransform(InPose.Pose[ParentIndex]);
			OutPose[BoneIndex].NormalizeRotation();
		}
	}
}

// Populate InOutPose based on raw animation data. 
UE_DEPRECATED(5.0, "BuildPoseFromRawData has been deprecated, use BuildPoseFromRawData signature with RetargetTransforms parameter")
extern ENGINE_API void BuildPoseFromRawData(
	const TArray<FRawAnimSequenceTrack>& InAnimationData,
	const TArray<struct FTrackToSkeletonMap>& TrackToSkeletonMapTable,
	FCompactPose& InOutPose,
	float InTime,
	EAnimInterpolationType Interpolation,
	int32 NumFrames,
	float SequenceLength,
	FName RetargetSource,
	const TMap<int32, const struct FTransformCurve*>* AdditiveBoneTransformCurves = nullptr
);

UE_DEPRECATED(5.0, "BuildPoseFromRawData has been deprecated, use UE::Anim::BuildPoseFromModel")
extern ENGINE_API void BuildPoseFromRawData(
	const TArray<FRawAnimSequenceTrack>& InAnimationData, 
	const TArray<struct FTrackToSkeletonMap>& TrackToSkeletonMapTable, 
	FCompactPose& InOutPose, 
	float InTime, 
	EAnimInterpolationType Interpolation, 
	int32 NumFrames, 
	float SequenceLength, 
	FName SourceName, 
	const TArray<FTransform>& RetargetTransforms,
	const TMap<int32, const FTransformCurve*>* AdditiveBoneTransformCurves = nullptr
	);