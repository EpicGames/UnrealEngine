// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"

class FBufferVisualizationData
{
public:

	FBufferVisualizationData()
		: bIsInitialized(false)
	{

	}

	/** Initialise the system */
	void Initialize();

	/** Check if system was initialized */
	bool IsInitialized() const { return bIsInitialized; }

	/** Get a named material from the available material map **/
	ENGINE_API UMaterialInterface* GetMaterial(FName InMaterialName) const;

	/** Get whether auto exposure should be applied for the visualization mode **/
	ENGINE_API bool GetMaterialApplyAutoExposure(FName InMaterialName) const;

	/** Get whether this visualization mode should be enabled on mobile **/
	ENGINE_API bool GetMaterialEnableOnMobile(FName InMaterialName) const;

	/** Get the display name of a named material from the available material map **/
	ENGINE_API FText GetMaterialDisplayName(FName InMaterialName) const;

	/** Get the default display name if no material is used and the Overview window is being used **/
	ENGINE_API static FText GetMaterialDefaultDisplayName();

	/** We cache the overview material name list from the console command here, so all dynamically created views can re-use the existing cached list of materials */
	void SetCurrentOverviewMaterialNames(const FString& InNameList);
	bool IsDifferentToCurrentOverviewMaterialNames(const FString& InNameList);

	/** Access the list of materials currently in use by the buffer visualization overview */
	TArray<UMaterialInterface*>& GetOverviewMaterials();

	/** Iterator function for iterating over available materials */
	template<class T> void IterateOverAvailableMaterials(T& Iterator) const
	{
		for (TMaterialMap::TConstIterator It = MaterialMap.CreateConstIterator(); It; ++It)
		{
			const Record& Rec = It.Value();
			Iterator.ProcessValue(Rec.Name, Rec.Material, Rec.DisplayName);
		}
	}

	/** Return the console command name for enabling single buffer visualization */
	static const TCHAR* GetVisualizationTargetConsoleCommandName()
	{
		return TEXT("r.BufferVisualizationTarget");
	}

private:

	/** Describes a single available visualization material */
	struct Record
	{
		FString Name;
		FText DisplayName;
		UMaterialInterface* Material;
		bool bApplyAutoExposure;
		bool bEnableOnMobile;
	};

	/** Mapping of FName (first parameter in ini file material list) to a material record */
	typedef TMultiMap<FName, Record> TMaterialMap;

	/** The name->material mapping table */
	TMaterialMap MaterialMap;

	/** The UMaterial.name->material mapping table */
	TMaterialMap MaterialMapFromMaterialName;

	/** List of material names to use in the buffer visualization overview */
	FString CurrentOverviewMaterialNames;

	/** List of material currently in use by the buffer visualization overview */
	TArray<UMaterialInterface*> OverviewMaterials;

	/** Storage for console variable documentation strings **/
	FString ConsoleDocumentationVisualizationMode;
	FString ConsoleDocumentationOverviewTargets;

	/** Flag indicating if system is initialized **/
	bool bIsInitialized;

	/** Internal helper function for creating the buffer visualization system console commands */
	void ConfigureConsoleCommand();

	/** Internal helper function to get record associated with material */
	const Record* GetRecord(FName InMaterialName) const;
};

ENGINE_API FBufferVisualizationData& GetBufferVisualizationData();
