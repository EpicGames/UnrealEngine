// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CDKey.h: CD Key validation
=============================================================================*/

#pragma once

#include "Containers/UnrealString.h"

/*-----------------------------------------------------------------------------
	Global CD Key functions
-----------------------------------------------------------------------------*/

FString GetCDKeyHash();
FString GetCDKeyResponse( const TCHAR* Challenge );

