// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderGraphDefinitions.h"

/**
 * The output of some geometry caching system.
 * For example this might be the GPUSkinCache, MeshDeformer or any other compute process that generates vertex buffers.
 * The contents can be ingested by some rendering system.
 * For example this might be used to place the hair groom system against a deformed mesh.
 */
struct FCachedGeometry
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCachedGeometry() = default;
	~FCachedGeometry() = default;
	FCachedGeometry(const FCachedGeometry&) = default;
	FCachedGeometry(FCachedGeometry&&) = default;
	FCachedGeometry& operator=(const FCachedGeometry&) = default;
	FCachedGeometry& operator=(FCachedGeometry&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct Section
	{
		FRDGBufferSRVRef RDGPositionBuffer = nullptr;
		FRDGBufferSRVRef RDGPreviousPositionBuffer = nullptr;
		FRDGBufferSRVRef RDGTangentBuffer = nullptr;

		FRHIShaderResourceView* PositionBuffer = nullptr;			// Valid when the input comes from the GPUSkinCache (since it is doesn't use RDG)
		FRHIShaderResourceView* PreviousPositionBuffer = nullptr;	// Valid when the input comes from the GPUSkinCache (since it is doesn't use RDG)
		FRHIShaderResourceView* TangentBuffer = nullptr;			// Valid when the input comes from the GPUSkinCache (since it is doesn't use RDG)
		
		FRHIShaderResourceView* UVsBuffer = nullptr;
		FRHIShaderResourceView* IndexBuffer = nullptr;
		
		uint32 UVsChannelOffset = 0;
		uint32 UVsChannelCount = 0;
		uint32 NumPrimitives = 0;
		uint32 NumVertices = 0;
		uint32 VertexBaseIndex = 0;
		uint32 IndexBaseIndex = 0;
		uint32 TotalVertexCount = 0;
		uint32 TotalIndexCount = 0;
		uint32 SectionIndex = 0;
		int32 LODIndex = -1;
		int32 TangentFormat = 0;  // 0: Interleaved TangentX/TangentZ, 1: TangentZ only
	};

	int32 LODIndex = -1;
	TArray<Section> Sections;

	UE_DEPRECATED(5.8, "Use scene proxy transform instead.")
	FTransform LocalToWorld = FTransform::Identity;
};
