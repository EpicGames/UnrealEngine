// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleInterface.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleManager.h"

class APlayerCameraManager;
class ICameraPhotography;
struct FMinimalViewInfo;
struct FPostProcessSettings;

/**
* The public interface of the CameraPhotographyModule
*/
class ICameraPhotographyModule : public IModuleInterface, public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("CameraPhotography");
		return FeatureName;
	}

	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	/**
	* Singleton-like access to ICameraPhotographyModule
	*
	* @return Returns ICameraPhotographyModule singleton instance, loading the module on demand if needed
	*/
	static inline ICameraPhotographyModule& Get()
	{
		return FModuleManager::LoadModuleChecked< ICameraPhotographyModule >(GetModularFeatureName());
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(GetModularFeatureName());
	}

	/**
	* Attempts to create a new photography interface
	*
	* @return Interface to the photography implementation, if we were able to successfully create one
	*/
	virtual TSharedPtr< class ICameraPhotography > CreateCameraPhotography() = 0;
};

class ICameraPhotography
{
public:
	ICameraPhotography() {};
	virtual ~ICameraPhotography() = default;
	virtual bool UpdateCamera(FMinimalViewInfo& InOutPOV, APlayerCameraManager* PCMgr) = 0;
	virtual void UpdatePostProcessing(FPostProcessSettings& InOutPostProcessingSettings) = 0;
	virtual void StartSession() = 0;
	virtual void StopSession() = 0;
	virtual bool IsSupported() = 0;
	virtual void SetUIControlVisibility(uint8 UIControlTarget, bool bIsVisible) = 0;
	virtual void DefaultConstrainCamera(const FVector NewCameraLocation, const FVector PreviousCameraLocation, const FVector OriginalCameraLocation, FVector& OutCameraLocation, APlayerCameraManager* PCMgr) = 0;
	virtual const TCHAR* const GetProviderName() = 0;	
};
