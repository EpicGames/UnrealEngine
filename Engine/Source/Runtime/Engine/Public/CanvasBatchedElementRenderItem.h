// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Canvas.h: Unreal canvas definition.
=============================================================================*/

#pragma once

#include "CanvasTypes.h"
#include "BatchedElements.h"

/**
* Info needed to render a batched element set
*/
class FCanvasBatchedElementRenderItem : public FCanvasBaseRenderItem
{
public:
	/** 
	* Init constructor 
	*/
	FCanvasBatchedElementRenderItem(
		FBatchedElementParameters* InBatchedElementParameters=NULL,
		const FTexture* InTexture=NULL,
		ESimpleElementBlendMode InBlendMode=SE_BLEND_MAX,
		FCanvas::EElementType InElementType=FCanvas::ET_MAX,
		const FCanvas::FTransformEntry& InTransform=FCanvas::FTransformEntry(FMatrix::Identity),
		const FDepthFieldGlowInfo& InGlowInfo=FDepthFieldGlowInfo() )
		// this data is deleted after rendering has completed
		: Data(new FRenderData(InBatchedElementParameters, InTexture, InBlendMode, InElementType, InTransform, InGlowInfo))
	{}

	/**
	* Destructor to delete data in case nothing rendered
	*/
	virtual ~FCanvasBatchedElementRenderItem()
	{
		delete Data;
	}

	/**
	* FCanvasBatchedElementRenderItem instance accessor
	*
	* @return this instance
	*/
	virtual class FCanvasBatchedElementRenderItem* GetCanvasBatchedElementRenderItem() override
	{ 
		return this; 
	}

	/**
	* Renders the canvas item. 
	* Iterates over all batched elements and draws them with their own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @param RHICmdList - command list to use
	* @return true if anything rendered
	*/
	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas) override;
	
	/**
	* Renders the canvas item.
	* Iterates over all batched elements and draws them with their own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @return true if anything rendered
	*/
	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope) override;

	/**
	* Determine if this is a matching set by comparing texture,blendmode,elementype,transform. All must match
	*
	* @param BatchedElementParameters - parameters for this batched element
	* @param InTexture - texture resource for the item being rendered
	* @param InBlendMode - current alpha blend mode 
	* @param InElementType - type of item being rendered: triangle,line,etc
	* @param InTransform - the transform for the item being rendered
	* @param InGlowInfo - the depth field glow of the item being rendered
	* @return true if the parameters match this render item
	*/
	bool IsMatch(FBatchedElementParameters* BatchedElementParameters, const FTexture* InTexture, ESimpleElementBlendMode InBlendMode, FCanvas::EElementType InElementType, const FCanvas::FTransformEntry& InTransform, const FDepthFieldGlowInfo& InGlowInfo)
	{
		return(	Data->BatchedElementParameters.GetReference() == BatchedElementParameters &&
				Data->Texture == InTexture &&
				Data->BlendMode == InBlendMode &&
				Data->ElementType == InElementType &&
				Data->Transform.GetMatrixCRC() == InTransform.GetMatrixCRC() &&
				Data->GlowInfo == InGlowInfo );
	}

	/**
	* Accessor for the batched elements. This can be used for adding triangles and primitives to the batched elements
	*
	* @return pointer to batched elements struct
	*/
	inline FBatchedElements* GetBatchedElements()
	{
		return &Data->BatchedElements;
	}

private:
	class FRenderData
	{
	public:
		/**
		* Init constructor
		*/
		FRenderData(
			FBatchedElementParameters* InBatchedElementParameters=NULL,
			const FTexture* InTexture=NULL,
			ESimpleElementBlendMode InBlendMode=SE_BLEND_MAX,
			FCanvas::EElementType InElementType=FCanvas::ET_MAX,
			const FCanvas::FTransformEntry& InTransform=FCanvas::FTransformEntry(FMatrix::Identity),
			const FDepthFieldGlowInfo& InGlowInfo=FDepthFieldGlowInfo() )
			:	BatchedElementParameters(InBatchedElementParameters)
			,	Texture(InTexture)
			,	BlendMode(InBlendMode)
			,	ElementType(InElementType)
			,	Transform(InTransform)
			,	GlowInfo(InGlowInfo)
		{}
		/** Current batched elements, destroyed once rendering completes. */
		FBatchedElements BatchedElements;
		/** Batched element parameters */
		TRefCountPtr<FBatchedElementParameters> BatchedElementParameters;
		/** Current texture being used for batching, set to NULL if it hasn't been used yet. */
		const FTexture* Texture;
		/** Current blend mode being used for batching, set to BLEND_MAX if it hasn't been used yet. */
		ESimpleElementBlendMode BlendMode;
		/** Current element type being used for batching, set to ET_MAX if it hasn't been used yet. */
		FCanvas::EElementType ElementType;
		/** Transform used to render including projection */
		FCanvas::FTransformEntry Transform;
		/** info for optional glow effect when using depth field rendering */
		FDepthFieldGlowInfo GlowInfo;
	};
	
	/**
	* Render data which is allocated when a new FCanvasBatchedElementRenderItem is added for rendering.
	* This data is only freed on the rendering thread once the item has finished rendering
	*/
	FRenderData* Data;		
};

