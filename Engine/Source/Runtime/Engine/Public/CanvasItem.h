// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CanvasItem.h: Unreal canvas item definitions
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Fonts/ShapedTextFwd.h"
#include "Engine/EngineTypes.h"
#include "Templates/Casts.h"
#include "Fonts/SlateFontInfo.h"
#include "Engine/Font.h"
#include "SceneTypes.h"

class FBatchedElementParameters;
class FBatchedElements;
class FCanvas;
class FMaterialRenderProxy;
class FTexture;
class UMaterial;

class FCanvasItem
{
public:
	/** 
	 * Basic render item.
	 *
	 * @param	InPosition		Draw position
	 */
	FCanvasItem( const FVector2D& InPosition )
		: Position( InPosition )
		, StereoDepth( 0 )
		, BlendMode( SE_BLEND_Opaque )
		, bFreezeTime( false )
		, BatchedElementParameters( nullptr )
		, Color( FLinearColor::White ) {};

	virtual ~FCanvasItem() = default;

	virtual void Draw( FCanvas* InCanvas ) = 0;

	/** 
	 * Draw this item (this will affect the items position for future draw calls that do no specify a position)
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	InPosition		Draw position - this will not preserve the items position
	 */
	virtual void Draw( FCanvas* InCanvas, const FVector2D& InPosition )
	{
		Position = InPosition;
		Draw( InCanvas );
	}

	/** 
	 * Draw this item (this will affect the items position for future draw calls that do no specify a position)
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	X				X Draw position 
	 * @param	Y				Y Draw position
	 */
	virtual void Draw( FCanvas* InCanvas, float X, float Y )
	{
		Position.X = X;
		Position.Y = Y;
		Draw( InCanvas );
	}
		
	/* Set the Color of the item. */
	virtual void SetColor( const FLinearColor& InColor )
	{
		Color = InColor;
	}
	
	/* The position to draw the item. */
	FVector2D Position;

	/* Stereo projection depth in game units.  Default value 0 draws at canvas property StereoDepth. */
	uint32 StereoDepth;

		/* Blend mode. */
	ESimpleElementBlendMode BlendMode;
		
	bool bFreezeTime;

	/* Used for batch rendering. */
	FBatchedElementParameters* BatchedElementParameters;
protected:
	/* Color of the item. */
	FLinearColor Color;
};


/* 'Tile' item can override size and UV . */
class FCanvasTileItem : public FCanvasItem
{
public:
	/** 
	 * Tile item using size from texture.
	 *
	 * @param	InPosition		Draw position
	 * @param	InTexture		The texture
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FTexture* InTexture, const FLinearColor& InColor );

	/** 
	 * Tile item with texture using given size. 
	 *
	 * @param	InPosition		Draw position
	 * @param	InTexture		The texture
	 * @param	InSize			The size to render
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InSize, const FLinearColor& InColor );

	/** 
	 * Tile item which uses the default white texture using given size. 
	 *
	 * @param	InPosition		Draw position
	 * @param	InSize			The size to render
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FVector2D& InSize, const FLinearColor& InColor );

	/** 
	 * Tile item with texture using size from texture specific UVs. 
	 *
	 * @param	InPosition		Draw position
	 * @param	InTexture		The texture
	 * @param	InUV0			UV coordinates (Normalized Top/Left)
	 * @param	InUV1			UV coordinates (Normalized Bottom/Right)
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InUV0, const FVector2D& InUV1, const FLinearColor& InColor );

	/** 
	 * Tile item with texture using given size and specific UVs.
	 *
	 * @param	InPosition		Draw position
	 * @param	InTexture		The texture
	 * @param	InSize			The size to render
	 * @param	InUV0			UV coordinates (Normalized Top/Left)
	 * @param	InUV1			UV coordinates (Normalized Bottom/Right)
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FTexture* InTexture, const FVector2D& InSize, const FVector2D& InUV0, const FVector2D& InUV1, const FLinearColor& InColor );

	/** 
	 * Tile item with FMaterialRenderProxy using given size. 
	 *
	 * @param	InPosition				Draw position
	 * @param	InMaterialRenderProxy	Material proxy for rendering
	 * @param	InSize					The size to render
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InSize );

	/** 
	 * Tile item with FMaterialRenderProxy using given size and UVs.
	 *
	 * @param	InPosition				Draw position
	 * @param	InMaterialRenderProxy	Material proxy for rendering
	 * @param	InSize					The size to render
	 * @param	InUV0					UV coordinates (Normalized Top/Left)
	 * @param	InUV1					UV coordinates (Normalized Bottom/Right)
	 */
	ENGINE_API FCanvasTileItem( const FVector2D& InPosition, const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InSize, const FVector2D& InUV0, const FVector2D& InUV1 );

	/** 
	 * Draw the item at the given coordinates.
	 *
	 * @param	InPosition		Draw position.
	 */
	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;

	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;

	/* Size of the tile. */
	FVector2D Size;

	/* used to calculate depth. */
	float Z;

	/* UV Coordinates 0 (Left/Top). */
	FVector2D UV0;
	
	/* UV Coordinates 0 (Right/Bottom). */
	FVector2D UV1;

	/* Texture to render. */
	const FTexture* Texture;
	
	/* Material proxy for rendering. */
	const FMaterialRenderProxy* MaterialRenderProxy;

	/* Rotation. */
	FRotator Rotation;
	
	/* Pivot point, as percentage of tile (0-1). */
	FVector2D	PivotPoint;

private:
	/* Render when we have a material proxy. */
	void RenderMaterialTile( FCanvas* InCanvas, const FVector2D& InPosition );
};

/* Resizable 3x3 border item. */
class FCanvasBorderItem : public FCanvasItem
{
public:

	/** 
	 * 3x3 grid border with tiled frame and tiled interior. 
	 *
	 * @param	InPosition		    Draw position
	 * @param	InBorderTexture		The texture to use for border
	 * @param	InBackgroundTexture	The texture to use for border background
	 * @param	InSize			    The size to render
	 * @param	InColor			    Tint of the border
	 */
	FCanvasBorderItem( const FVector2D& InPosition, const FTexture* InBorderTexture, const FTexture* InBackgroundTexture, 
		const FTexture* InBorderLeftTexture, const FTexture* InBorderRightTexture, const FTexture* InBorderTopTexture, const FTexture* InBorderBottomTexture,
		const FVector2D& InSize, const FLinearColor& InColor )
		: FCanvasItem( InPosition )
		, Size( InSize )
		, BorderScale( FVector2D(1.0f,1.0f ))
		, BackgroundScale( FVector2D(1.0f,1.0f ))
		, Z( 1.0f )
		, BorderUV0( 0.0f, 0.0f )
		, BorderUV1( 1.0f, 1.0f )
		, BorderTexture( InBorderTexture )
		, BackgroundTexture( InBackgroundTexture )
		, BorderLeftTexture( InBorderLeftTexture )
		, BorderRightTexture( InBorderRightTexture )
		, BorderTopTexture( InBorderTopTexture )
		, BorderBottomTexture( InBorderBottomTexture )
		, Rotation( ForceInitToZero )
		, PivotPoint( FVector2D::ZeroVector )
	{
		SetColor( InColor );
		// Ensure textures is valid
		check( InBorderTexture );
		check( InBackgroundTexture );
	}

	/** 
	 * Draw the item at the given coordinates.
	 *
	 * @param	InPosition		Draw position.
	 */
	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;

	virtual void SetColor(const FLinearColor& InColor) override final { FCanvasItem::SetColor(InColor); }


	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;

	/* Size of the border. */
	FVector2D Size;

	/** Scale of the border */
	FVector2D BorderScale;

	/** Scale of the border */
	FVector2D BackgroundScale;

	/* used to calculate depth. */
	float Z;

	/* Border UV Coordinates 0 (Left/Top). */
	FVector2D BorderUV0;
	
	/* Border UV Coordinates 1 (Right/Bottom). */
	FVector2D BorderUV1;

	/* Corners texture. */
	const FTexture* BorderTexture;

	/* Background tiling texture. */
	const FTexture* BackgroundTexture;

	/* Border left tiling texture. */
	const FTexture* BorderLeftTexture;

	/* Border right tiling texture. */
	const FTexture* BorderRightTexture;

	/* Border top tiling texture. */
	const FTexture* BorderTopTexture;

	/* Border bottom tiling texture. */
	const FTexture* BorderBottomTexture;
	
	/* Rotation. */
	FRotator Rotation;
	
	/* Pivot point. */
	FVector2D	PivotPoint;

	/** Frame corner size in percent of frame texture (should be < 0.5f) */
	FVector2D   CornerSize;
};

/* Base item used for drawing text */
class FCanvasTextItemBase : public FCanvasItem
{
public:
	ENGINE_API FCanvasTextItemBase( const FVector2D& InPosition, const FLinearColor& InColor );
	ENGINE_API virtual ~FCanvasTextItemBase();

	/** 
	 * Set the shadow offset and color. 
	 *
	 * @param	InColor			Shadow color
	 * @param	InOffset		Shadow offset. Defaults to 1,1. (Passing zero vector will disable the shadow)	 
	 */
	void EnableShadow( const FLinearColor& InColor, const FVector2D& InOffset = FVector2D( 1.0f, 1.0f ) )
	{
		ShadowOffset = InOffset;
		ShadowColor = InColor;
		FontRenderInfo.bEnableShadow = ShadowOffset.SizeSquared() != 0.0f;
	}
	
	/** 
	 * Disable the shadow
	 */
	void DisableShadow() 
	{
		ShadowOffset = FVector2D::ZeroVector;
		FontRenderInfo.bEnableShadow = false;
	}

	/** 
	 * Draw the item at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 */
	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;

	virtual void SetColor(const FLinearColor& InColor) override final { FCanvasItem::SetColor(InColor); }

	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;

	/* Horizontal spacing adjustment. */
	float HorizSpacingAdjust;

	/* Depth sort key. */
	float Depth;

	/* Custom font render information. */
	FFontRenderInfo	FontRenderInfo;	
	
	/* The color of the shadow */
	FLinearColor ShadowColor;

	/* The offset of the shadow. */
	FVector2D ShadowOffset;

	/* The size of the drawn text after the draw call. */
	FVector2D DrawnSize;

	/* Centre the text in the viewport on horizontally. */
	bool	bCentreX;
	
	/* Centre the text in the viewport on vertically. */
	bool	bCentreY;

	/* Draw an outline on the text. */
	bool	bOutlined;

	/* The color of the outline. */
	FLinearColor OutlineColor;

	/* Disables correction of font render issue when using stereoscopic display. */
	bool	bDontCorrectStereoscopic;

	/* The scale of the text */
	FVector2D Scale;

protected:
	/* Background tile used to fixup 3d text issues. */
	FCanvasTileItem	TileItem;

	/**
	 * Does this text item have valid text to draw?
	 */
	virtual bool HasValidText() const = 0;

	/**
	 * Get the blend mode we should use to draw this text
	 */
	virtual ESimpleElementBlendMode GetTextBlendMode( const bool bHasShadow ) const = 0;

	/**
	 * Get the measured size of the text to draw
	 */
	virtual FVector2D GetTextSize(float DPIScale) const = 0;

	struct FTextEffect
	{
		FVector2f Offset;
		FLinearColor Color;
	};

	/** 
	 * Internal string draw
	 *
	 * In a method to make it simpler to do effects like shadow, outline
	 */
	virtual void DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> Offsets) = 0;

	/** 
	 * These are used by the DrawStringInternal function. 
	 */
	/* Used for batching. */
	FBatchedElements* BatchedElements;
};

/*
 * Base item used for drawing simple text strings with misc optional items such as shadow, centering etc.
 * Provides functions which operate on FStringView, which derived classes can use.
 */
class FCanvasSimpleTextItem : public FCanvasTextItemBase
{
public:

	/**
	 * Constructor.
	 *
	 * @param	InPosition		Draw position.
	 * @param	InFont			Font to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	FCanvasSimpleTextItem(const FVector2D& InPosition, const UFont* InFont, const FLinearColor& InColor)
		: FCanvasTextItemBase(InPosition, InColor)
		, Font(InFont)
		, SlateFontInfo()
	{
		BlendMode = SE_BLEND_Translucent;
	}

	/**
	 * Constructor.
	 *
	 * @param	InPosition		Draw position.
	 * @param	InFontInfo		Font info to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	FCanvasSimpleTextItem(const FVector2D& InPosition, const FSlateFontInfo& InFontInfo, const FLinearColor& InColor)
		: FCanvasTextItemBase(InPosition, InColor)
		, Font(Cast<const UFont>(InFontInfo.FontObject))
		, SlateFontInfo(InFontInfo)
	{
		BlendMode = SE_BLEND_TranslucentAlphaOnly;
	}

	virtual ~FCanvasSimpleTextItem() override = default;

	/* Font to draw text with. */
	const UFont* Font;

	/** Font info to draw the text with. */
	TOptional<FSlateFontInfo> SlateFontInfo;

protected:
	/** Get the type of font cache the UFont is using */
	ENGINE_API EFontCacheType GetFontCacheType() const;

	//~ Begin FCanvasTextItemBase interface
	virtual bool HasValidText() const override = 0;
	ENGINE_API virtual ESimpleElementBlendMode GetTextBlendMode(const bool bHasShadow) const override;
	virtual FVector2D GetTextSize(float DPIScale) const override = 0;
	virtual void DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> Offsets) override = 0;
	//~ End FCanvasTextItemBase interface

	ENGINE_API FVector2D GetTextSizeInternal(FStringView Text, float DPIScale) const;
	ENGINE_API void DrawStringInternal(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects);

	/**
	 * Internal string draw
	 *
	 * In a method to make it simpler to do effects like shadow, outline
	 */
	ENGINE_API void DrawStringInternal_OfflineCache(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects);
	ENGINE_API void DrawStringInternal_RuntimeCache(FCanvas* InCanvas, FStringView Text, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects);

};

/*
 * Text item with misc optional items such as shadow, centering etc.
 * This class operates on FText.
 */
class FCanvasTextItem : public FCanvasSimpleTextItem
{
public:

	/** 	 
	 * Constructor.
	 *
	 * @param	InPosition		Draw position.
	 * @param	InText			String to draw.
	 * @param	InFont			Font to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	ENGINE_API FCanvasTextItem(const FVector2D& InPosition, const FText& InText, const UFont* InFont, const FLinearColor& InColor);

	/** 	 
	 * Constructor
	 *
	 * @param	InPosition		Draw position.
	 * @param	InText			String to draw.
	 * @param	InFontInfo		Font info to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	ENGINE_API FCanvasTextItem(const FVector2D& InPosition, const FText& InText, const FSlateFontInfo& InFontInfo, const FLinearColor& InColor);
	
	ENGINE_API virtual ~FCanvasTextItem();

	/* The text to draw. */
	FText Text;

protected:

	//~ Begin FCanvasTextItemBase interface
	ENGINE_API virtual bool HasValidText() const override;
	ENGINE_API virtual FVector2D GetTextSize(float DPIScale) const override;
	ENGINE_API virtual void DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects) override;
	//~ End FCanvasTextItemBase interface
	
	/** 
	 * Internal string draw.
	 *
	 * In a method to make it simpler to do effects like shadow, outline
	 */
	ENGINE_API void DrawStringInternal_OfflineCache(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects);
	ENGINE_API void DrawStringInternal_RuntimeCache(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects);

};

/*
 * Text item with misc optional items such as shadow, centering etc.
 * This class operates on FStringView - ensure that the underlying string is valid for
 * the lifetime of the string view stored in this class.
 */
class FCanvasTextStringViewItem : public FCanvasSimpleTextItem
{
public:

	/**
	 * Constructor.
	 *
	 * @param	InPosition		Draw position.
	 * @param	InText			String to draw.
	 * @param	InFont			Font to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	FCanvasTextStringViewItem(const FVector2D& InPosition, FStringView InText, const UFont* InFont, const FLinearColor& InColor)
		: FCanvasSimpleTextItem(InPosition, InFont, InColor)
		, Text(InText)
	{
	}

	/**
	 * Constructor.
	 *
	 * @param	InPosition		Draw position.
	 * @param	InText			String to draw.
	 * @param	InFontInfo		Font info to draw with.
	 * @param	InColor			Color to draw the text with.
	 */
	FCanvasTextStringViewItem(const FVector2D& InPosition, FStringView InText, const FSlateFontInfo& InFontInfo, const FLinearColor& InColor)
		: FCanvasSimpleTextItem(InPosition, InFontInfo, InColor)
		, Text(InText)
	{
	}

	virtual ~FCanvasTextStringViewItem() override = default;

	/* The text to draw. */
	FStringView Text;

protected:

	//~ Begin FCanvasTextItemBase interface
	ENGINE_API virtual bool HasValidText() const override;
	ENGINE_API virtual FVector2D GetTextSize(float DPIScale) const override;
	ENGINE_API virtual void DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects) override;
	//~ End FCanvasTextItemBase interface

};

/* Text item which can handle complex shaped text. */
class FCanvasShapedTextItem : public FCanvasTextItemBase
{
public:
	/**
	 * Shaped text item for complex script rendering.
	 *
	 * @param InPosition             Draw position
	 * @param InShapedGlyphSequence  Shaped text information to draw
	 * @param InColor                Color to draw the text with
	 */
	FCanvasShapedTextItem( const FVector2D& InPosition, FShapedGlyphSequenceRef InShapedGlyphSequence, const FLinearColor& InColor )
		: FCanvasTextItemBase( InPosition, InColor )
		, ShapedGlyphSequence( InShapedGlyphSequence )
	{
		BlendMode = SE_BLEND_TranslucentAlphaOnly;
	}

	/** The shaped text sequence to draw */
	FShapedGlyphSequencePtr ShapedGlyphSequence;

protected:

	//~ Begin FCanvasTextItemBase interface
	ENGINE_API virtual bool HasValidText() const override;
	ENGINE_API virtual ESimpleElementBlendMode GetTextBlendMode( const bool bHasShadow ) const override;
	ENGINE_API virtual FVector2D GetTextSize(float DPIScale) const override;
	ENGINE_API virtual void DrawStringInternal(FCanvas* InCanvas, const FVector2D& DrawPos, const FLinearColor& DrawColor, TArrayView<FTextEffect> TextEffects) override;
	//~ End FCanvasTextItemBase interface

};

/* Line item. Note blend mode will be disregarded for these - only SE_BLEND_Opaque is currently supported. */
class FCanvasLineItem : public FCanvasItem
{
public:
	FCanvasLineItem()
		: FCanvasItem( FVector2D::ZeroVector )
		, LineThickness( 0.0f )
	{
		Origin.X = 0.0f;
		Origin.Y = 0.0f;
		Origin.Z = 0.0f;
		EndPos.X = 0.0f;
		EndPos.Y = 0.0f;
		EndPos.Z = 0.0f;
	}
	/** 
	 * A Line. 
	 * 
	 * @param	InPosition		Start position
	 * @param	InEndPos		End position 
	 */
	FCanvasLineItem( const FVector2D& InPosition, const FVector2D& InEndPos )
		: FCanvasItem( InPosition )
		, LineThickness( 0.0f )
	{
		Origin.X = InPosition.X;
		Origin.Y = InPosition.Y;
		Origin.Z = 0.0f;
		EndPos.X = InEndPos.X;
		EndPos.Y = InEndPos.Y;
		EndPos.Z = 0.0f;
	}
	
	/** 
	 * A Line. 
	 * 
	 * @param	InPosition		Start position
	 * @param	InEndPos		End position 
	 */
	FCanvasLineItem( const FVector& InPosition, const FVector& InEndPos )
		: FCanvasItem( FVector2D( UE_REAL_TO_FLOAT(InPosition.X), UE_REAL_TO_FLOAT(InPosition.Y) ) )
		, LineThickness( 0.0f )
	{
		Origin = InPosition;
		EndPos = InEndPos;
	}

	/** 
	 * Draw line at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 */
	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;
	
	/** 
	 * Draw line at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	InPosition		Draw Start position
	 */
	virtual void Draw( FCanvas* InCanvas, const FVector2D& InPosition ) override
	{
		Origin.X = InPosition.X;
		Origin.Y = InPosition.Y;
		Draw( InCanvas );
	}
	/** 
	 * Draw line using the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	InStartPos		Line start position
	 * @param	InEndPos		Line end position
	 */
	virtual void Draw( FCanvas* InCanvas, const FVector2D& InStartPos, const FVector2D& InEndPos )
	{
		Origin.X = InStartPos.X;
		Origin.Y = InStartPos.Y;
		EndPos.X = InEndPos.X;
		EndPos.Y = InEndPos.Y;
		Draw( InCanvas );
	}
	/** 
	 * Draw line at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	X				X Draw position 
	 * @param	Y				Y Draw position
	 */
	virtual void Draw( FCanvas* InCanvas, float InX, float InY ) override
	{
		Origin.X = InX;
		Origin.Y = InY;
		Draw( InCanvas );
	}
		
	/** 
	 * Draw line at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	InPosition		Draw position
	 */
	virtual void Draw( FCanvas* InCanvas, const FVector& InPosition )
	{
		Origin = InPosition;
		Draw( InCanvas );
	}

	/** 
	 * Draw line at the given coordinates.
	 *
	 * @param	InCanvas		Canvas on which to draw
	 * @param	X				X Draw position 
	 * @param	Y				Y Draw position
	 * @param	Z				Z Draw position
	 */
	virtual void Draw( FCanvas* InCanvas, float X, float Y, float Z )
	{
		Origin.X = X;
		Origin.Y = Y;
		Origin.Z = Z;
		Draw( InCanvas );
	}

	/** 
	 * Set the line end position.
	 *
	 * @param	InEndPos		End position of the line
	 */
	void SetEndPos( const FVector2D& InEndPos )
	{
		EndPos.X = InEndPos.X;
		EndPos.Y = InEndPos.Y;
	}

	/* The origin of the line. */
	FVector		Origin;

	/* The end position of the line. */
	FVector		EndPos;

	/* The thickness of the line. */
	float		LineThickness;
};

class FCanvasBoxItem : public FCanvasItem
{
public:
	FCanvasBoxItem( const FVector2D& InPosition, const FVector2D& InSize )
		: FCanvasItem( InPosition )
		, Size( InSize )
		, LineThickness( 0.0f )	{};

	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;

	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;

	/* Size of the box. */
	FVector2D Size;

	/* The thickness of the line. */
	float		LineThickness;
private:
	ENGINE_API void SetupBox();
	TArray< FVector > Corners;
};

/**
 * A box drawn with dashed lines that wrap at the corners.
 */
class FCanvasDashedBoxItem : public FCanvasItem
{
public:
	FCanvasDashedBoxItem( const FVector2D& InPosition, const FVector2D& InSize )
		: FCanvasItem( InPosition )
		, Size( InSize )
		, DashThickness( 0.0f )
		, DashLength( 1.0f )
		, DashGap( 1.0f )
		{};
	
	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;
	
	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;
	
	/* Size of the box. */	
	FVector2d Size;
	
	/* The thickness of the dashes. */
	float DashThickness;
	
	/* The length of the dashes. */
	float DashLength;
	
	/* The length of the gap between dashes. */
	float DashGap;
};


class FCanvasTriangleItem : public FCanvasItem
{
public:	
	/** 	 
	 * Triangle item (no texture)
	 *
	 * @param	InPointA		Point A of triangle
	 * @param	InPointB		Point B of triangle
	 * @param	InPointC		Point C of triangle
	 */
	FCanvasTriangleItem( const FVector2D& InPointA, const FVector2D& InPointB, const FVector2D& InPointC, const FTexture* InTexture )
		: FCanvasItem( InPointA )
		, Texture( InTexture )
		, MaterialRenderProxy( nullptr )
		, BatchedElementParameters( nullptr )
	{
		FCanvasUVTri SingleTri;
		SingleTri.V0_Pos = InPointA;
		SingleTri.V1_Pos = InPointB;
		SingleTri.V2_Pos = InPointC;
		
		SingleTri.V0_UV = FVector2D::ZeroVector;
		SingleTri.V1_UV = FVector2D::ZeroVector;
		SingleTri.V2_UV = FVector2D::ZeroVector;

		SingleTri.V0_Color = FLinearColor::White;
		SingleTri.V1_Color = FLinearColor::White;
		SingleTri.V2_Color = FLinearColor::White;

		TriangleList.Add( SingleTri );
	};

	/** 	 
	 * Triangle item
	 *
	 * @param	InPointA		Point A of triangle
	 * @param	InPointB		Point B of triangle
	 * @param	InPointC		Point C of triangle
	 * @param	InPointA		UV of Point A of triangle
	 * @param	InPointB		UV of Point B of triangle
	 * @param	InPointC		UV of Point C of triangle
	 * @param	InTexture		Texture
	 */
	FCanvasTriangleItem( const FVector2D& InPointA, const FVector2D& InPointB, const FVector2D& InPointC, const FVector2D& InTexCoordPointA, const FVector2D& InTexCoordPointB, const FVector2D& InTexCoordPointC, const FTexture* InTexture )
		:FCanvasItem( InPointA )
		, Texture( InTexture )
		, MaterialRenderProxy( nullptr )
		, BatchedElementParameters( nullptr )
	{
		FCanvasUVTri SingleTri;
		SingleTri.V0_Pos = InPointA;
		SingleTri.V1_Pos = InPointB;
		SingleTri.V2_Pos = InPointC;

		SingleTri.V0_UV = InTexCoordPointA;
		SingleTri.V1_UV = InTexCoordPointB;
		SingleTri.V2_UV = InTexCoordPointC;

		SingleTri.V0_Color = FLinearColor::White;
		SingleTri.V1_Color = FLinearColor::White;
		SingleTri.V2_Color = FLinearColor::White;
		TriangleList.Add( SingleTri );
	};

	/** 	 
	 * Triangle item
	 *
	 * @param	InSingleTri		Triangle struct
	 * @param	InTexture		Texture
	 */
	FCanvasTriangleItem( FCanvasUVTri InSingleTri, const FTexture* InTexture )
		:FCanvasItem( InSingleTri.V0_Pos )
		, Texture( InTexture )
		, MaterialRenderProxy( nullptr )
		, BatchedElementParameters( nullptr )
	{
		TriangleList.Add( InSingleTri );
	};

	/** 	 
	 * Triangle item
	 *
	 * @param	InTriangleList	List of triangles
	 * @param	InTexture		Texture
	 */
	FCanvasTriangleItem( const TArray< FCanvasUVTri >&	InTriangleList, const FTexture* InTexture )
		:FCanvasItem( FVector2D::ZeroVector )
		, Texture( InTexture )
		, MaterialRenderProxy( nullptr )
		, BatchedElementParameters( nullptr )
	{
		check( InTriangleList.Num() >= 1 );
		
		TriangleList = InTriangleList;
		Position = TriangleList[ 0 ].V0_Pos;
	};

	virtual ~FCanvasTriangleItem()= default;

	/* Set all 3 points of triangle */
	void SetPoints( const FVector2D& InPointA, const FVector2D& InPointB, const FVector2D& InPointC )
	{
		check( TriangleList.Num() != 0 );
		TriangleList[0].V0_Pos = InPointA;
		TriangleList[0].V1_Pos = InPointB;
		TriangleList[0].V2_Pos = InPointC;
	}

	ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;
		
	/* Expose the functions defined in the base class. */
	using FCanvasItem::Draw;

	/* Set the Color of the item. */
	ENGINE_API virtual void SetColor( const FLinearColor& InColor ) override;

	/* texture to use for triangle(s). */
	const FTexture* Texture;

	/* Material proxy for rendering. */
	const FMaterialRenderProxy* MaterialRenderProxy;

	FBatchedElementParameters* BatchedElementParameters;

	/* List of triangles. */
	TArray< FCanvasUVTri >	TriangleList;
};


class FCanvasNGonItem :  public FCanvasItem
{
public:	
	/** 	 
	 * NGon item Several texture tris with a common central point with a fixed radius. 
	 *
	 * @param	InPosition	List of triangles
	 * @param	InRadius	Size of the object
	 * @param	InNumSides	How many tris/sides the object has	
	 * @param	InTexture	Texture to render
	 * @param	InColor		Color to tint the texture with
	 */
	 FCanvasNGonItem( const FVector2D& InPosition, const FVector2D& InRadius, int32 InNumSides, const FTexture* InTexture, const FLinearColor& InColor )
	  : FCanvasItem( InPosition )
		, TriListItem( nullptr )
		, Texture( InTexture )
	 {
		 Color = InColor;
		 check( InNumSides >= 3 );
		 TriangleList.SetNum( InNumSides );
		 SetupPosition( InPosition, InRadius );
	 }

	 /** 	 
	 * NGon item Several tris with a common central point with a fixed radius. 
	 *
	 * @param	InPosition	List of triangles
	 * @param	InNumSides	How many tris/sides the object has	
	 * @param	InTexture	Texture to render
	 * @param	InColor		Color to tint the texture with
	 */
	 ENGINE_API FCanvasNGonItem( const FVector2D& InPosition, const FVector2D& InRadius, int32 InNumSides, const FLinearColor& InColor );

	 virtual ~FCanvasNGonItem()
	 {
		 delete TriListItem;
	 }
	 ENGINE_API virtual void Draw( FCanvas* InCanvas ) override;

	 /** 	 
	 * Regenerates the tri list for the object with a new central point and radius
	 *
	 * @param	InPosition	List of triangles
	 * @param	InRadius	Size of the object
	 */
	 void SetupPosition( const FVector2D& InPosition, const FVector2D& InRadius )
	 {
		 if( TriListItem )
		 {
			 delete TriListItem;
		 }		 
		 // todo: make this calculate the UVs correctly so if you pass a texture it will work as you would expect
		 float Angle = 0.0f;
		 int32 NumSides = TriangleList.Num();
		 FVector2D LastPoint = InPosition + FVector2D( InRadius.X*FMath::Cos(Angle), InRadius.Y*FMath::Sin(Angle) );
		 for(int32 i=1; i< NumSides+1; i++)
		 {
			 Angle = (2 * (float)UE_PI) * (float)i/(float)NumSides;			 			 
			 TriangleList[ i - 1 ].V0_Pos = InPosition;
			 TriangleList[ i - 1 ].V0_Color = Color;
			 TriangleList[ i - 1 ].V1_Pos = LastPoint;
			 TriangleList[ i - 1 ].V1_Color = Color;
			 LastPoint = InPosition + FVector2D( InRadius.X*FMath::Cos(Angle), InRadius.Y*FMath::Sin(Angle) );
			 TriangleList[ i - 1 ].V2_Pos = LastPoint;
			 TriangleList[ i - 1 ].V2_Color = Color;
		 }
		 TriListItem = new FCanvasTriangleItem( TriangleList, Texture );
	 }

	 /* Set the Color of the item. */
	 ENGINE_API virtual void SetColor( const FLinearColor& InColor ) override final;

private:
	
	TArray< FCanvasUVTri >	TriangleList;
	FCanvasTriangleItem*	TriListItem;
	const FTexture* Texture;	
};


#if WITH_EDITOR
class FCanvasItemTestbed
{
public:
	ENGINE_API FCanvasItemTestbed();
	ENGINE_API void  Draw( class FViewport* Viewport, FCanvas* Canvas );
	struct LineVars
	{
		LineVars()
		{
			LineStart = FVector2D::ZeroVector;
			LineEnd = FVector2D::ZeroVector;
			LineMove = FVector2D::ZeroVector;
			LineMove2 = FVector2D::ZeroVector;
			bTestSet = false;
			Testangle = 0.0f;
		}
		FVector2D LineStart;
		FVector2D LineEnd;
		FVector2D LineMove;
		FVector2D LineMove2;
		bool bTestSet;
		float Testangle;
	};
	static ENGINE_API LineVars TestLine;
	static ENGINE_API bool	bTestState;
	static ENGINE_API bool	bShowTestbed;
	static ENGINE_API bool bShowLines;
	static ENGINE_API bool bShowBoxes;
	static ENGINE_API bool bShowTris;
	static ENGINE_API bool bShowText;
	static ENGINE_API bool bShowTiles;

	UMaterial* TestMaterial;
	
};
#endif // WITH_EDITOR

