// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "RenderGraphBuilder.h"

class FCanvasRenderContext
{
public:
	ENGINE_API FCanvasRenderContext(FRDGBuilder& InGraphBuilder, const FRenderTarget* InRenderTarget, FIntRect InViewportRect, FIntRect InScissorRect, bool bScaledToRenderTarget, int16 InSliceIndex = -1);

	template <typename ExecuteLambdaType, typename ParameterStructType>
	void AddPass(FRDGEventName&& PassName, const ParameterStructType* PassParameters, ExecuteLambdaType&& ExecuteLambda)
	{
		GraphBuilder.AddPass(Forward<FRDGEventName>(PassName), PassParameters, ERDGPassFlags::Raster,
			[LocalScissorRect = ScissorRect, LocalViewportRect = ViewportRect, LocalExecuteLambda = Forward<ExecuteLambdaType&&>(ExecuteLambda)](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(static_cast<float>(LocalViewportRect.Min.X), static_cast<float>(LocalViewportRect.Min.Y), 0.0f, static_cast<float>(LocalViewportRect.Max.X), static_cast<float>(LocalViewportRect.Max.Y), 1.0f);

			if (LocalScissorRect.Area() > 0)
			{
				RHICmdList.SetScissorRect(true, static_cast<uint32>(LocalScissorRect.Min.X), static_cast<uint32>(LocalScissorRect.Min.Y), static_cast<uint32>(LocalScissorRect.Max.X), static_cast<uint32>(LocalScissorRect.Max.Y));
			}
		
			LocalExecuteLambda(RHICmdList);
		});
	}

	template <typename ExecuteLambdaType>
	void AddPass(FRDGEventName&& PassName, ExecuteLambdaType&& ExecuteLambda)
	{
		FRenderTargetParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(RenderTarget, ERenderTargetLoadAction::ELoad, /*MipIndex*/0, SliceIndex);
		AddPass(Forward<FRDGEventName&&>(PassName), PassParameters, Forward<ExecuteLambdaType&&>(ExecuteLambda));
	}

	template <typename T, typename... TArgs>
	T* Alloc(TArgs&&... Args)
	{
		return GraphBuilder.AllocObject<T>(Forward<TArgs&&>(Args)...);
	}

	template <typename T>
	void DeferredRelease(TSharedPtr<T>&& Ptr)
	{
		// Hold the reference until completion of the graph.
		Alloc<TSharedPtr<T>>(Forward<TSharedPtr<T>&&>(Ptr));
	}

	template <typename T>
	void DeferredDelete(const T* Ptr)
	{
		struct FDeleter
		{
			FDeleter(const T* InPtr)
				: Ptr(InPtr)
			{}

			~FDeleter()
			{
				delete Ptr;
			}

			const T* Ptr;
		};

		Alloc<FDeleter>(Ptr);
	}

	FRDGTextureRef GetRenderTarget() const
	{
		return RenderTarget;
	}

	FIntRect GetViewportRect() const
	{
		return ViewportRect;
	}

	FIntRect GetScissorRect() const
	{
		return ScissorRect;
	}

	int16 GetSliceIndex() const
	{
		return SliceIndex;
	}

	FRDGBuilder& GraphBuilder;

private:
	FRDGTextureRef RenderTarget;
	FIntRect ViewportRect;
	FIntRect ScissorRect;
	int16 SliceIndex;
};

class FCanvasRenderThreadScope
{
	using RenderCommandFunction = TFunction<void(FCanvasRenderContext&)>;
	using RenderCommandFunctionArray = TArray<RenderCommandFunction>;
public:
	ENGINE_API FCanvasRenderThreadScope(const FCanvas& Canvas);
	ENGINE_API ~FCanvasRenderThreadScope();

	void EnqueueRenderCommand(RenderCommandFunction&& Lambda)
	{
		RenderCommands->Add(MoveTemp(Lambda));
	}

	template <typename ExecuteLambdaType>
	void AddPass(const TCHAR* PassName, ExecuteLambdaType&& Lambda)
	{
		EnqueueRenderCommand(
			[PassName, InLambda = Forward<ExecuteLambdaType&&>(Lambda)]
		(FCanvasRenderContext& RenderContext) mutable
		{
			RenderContext.AddPass(RDG_EVENT_NAME("%s", PassName), Forward<ExecuteLambdaType&&>(InLambda));
		});
	}

	template <typename T>
	void DeferredDelete(const T* Ptr)
	{
		EnqueueRenderCommand([Ptr](FCanvasRenderContext& RenderContext)
		{
			RenderContext.DeferredDelete(Ptr);
		});
	}

private:
	const FCanvas& Canvas;
	RenderCommandFunctionArray* RenderCommands;
};