// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "LocalVertexFactory.h"
#include "StaticMeshResources.h"

/**
* Info needed to render a single FTileRenderer
*/
class FCanvasTileRendererItem : public FCanvasBaseRenderItem
{
public:
	/**
	* Init constructor
	*/
	FCanvasTileRendererItem(ERHIFeatureLevel::Type InFeatureLevel,
		const FMaterialRenderProxy* InMaterialRenderProxy = NULL,
		const FCanvas::FTransformEntry& InTransform = FCanvas::FTransformEntry(FMatrix::Identity),
		bool bInFreezeTime = false)
		// this data is deleted after rendering has completed
		: Data(MakeShared<FRenderData>(InFeatureLevel, InMaterialRenderProxy, InTransform))
		, bFreezeTime(bInFreezeTime)
	{}

	/**
	* FCanvasTileRendererItem instance accessor
	*
	* @return this instance
	*/
	virtual class FCanvasTileRendererItem* GetCanvasTileRendererItem() override
	{
		return this;
	}

	/**
	* Renders the canvas item.
	* Iterates over each tile to be rendered and draws it with its own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @param RHICmdList - command list to use
	* @return true if anything rendered
	*/
	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas) override;

	/**
	* Renders the canvas item.
	* Iterates over each tile to be rendered and draws it with its own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @return true if anything rendered
	*/
	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope) override;

	/**
	* Determine if this is a matching set by comparing material,transform. All must match
	*
	* @param IInMaterialRenderProxy - material proxy resource for the item being rendered
	* @param InTransform - the transform for the item being rendered
	* @return true if the parameters match this render item
	*/
	bool IsMatch(const FMaterialRenderProxy* InMaterialRenderProxy, const FCanvas::FTransformEntry& InTransform)
	{
		return(Data->MaterialRenderProxy == InMaterialRenderProxy &&
			Data->Transform.GetMatrixCRC() == InTransform.GetMatrixCRC());
	};

	/**
	* Add a new tile to the render data. These tiles all use the same transform and material proxy
	*
	* @param X - tile X offset
	* @param Y - tile Y offset
	* @param SizeX - tile X size
	* @param SizeY - tile Y size
	* @param U - tile U offset
	* @param V - tile V offset
	* @param SizeU - tile U size
	* @param SizeV - tile V size
	* @param return number of tiles added
	*/
	inline int32 AddTile(float X, float Y, float SizeX, float SizeY, float U, float V, float SizeU, float SizeV, FHitProxyId HitProxyId, FColor InColor)
	{
		return Data->AddTile(X, Y, SizeX, SizeY, U, V, SizeU, SizeV, HitProxyId, InColor);
	};

private:
	class FTileVertexFactory : public FLocalVertexFactory
	{
	public:
		FTileVertexFactory(const FStaticMeshVertexBuffers* VertexBuffers, ERHIFeatureLevel::Type InFeatureLevel);
		void InitResource(FRHICommandListBase& RHICmdList) override;

	private:
		const FStaticMeshVertexBuffers* VertexBuffers;
	};

	class FRenderData
	{
	public:
		FRenderData(
			ERHIFeatureLevel::Type InFeatureLevel,
			const FMaterialRenderProxy* InMaterialRenderProxy,
			const FCanvas::FTransformEntry& InTransform);

		~FRenderData();

		void RenderTiles(
			FCanvasRenderContext& RenderContext,
			FMeshPassProcessorRenderState& DrawRenderState,
			const FSceneView& View,
			bool bIsHitTesting,
			bool bUse128bitRT = false);

		const FMaterialRenderProxy* const MaterialRenderProxy;
		const FCanvas::FTransformEntry Transform;

		inline int32 AddTile(float X, float Y, float SizeX, float SizeY, float U, float V, float SizeU, float SizeV, FHitProxyId HitProxyId, FColor InColor)
		{
			FTileInst NewTile = { X,Y,SizeX,SizeY,U,V,SizeU,SizeV,HitProxyId,InColor };
			return Tiles.Add(NewTile);
		};

		uint32 GetNumVertices() const;
		uint32 GetNumIndices() const;

	private:
		FMeshBatch* AllocTileMeshBatch(FCanvasRenderContext& InRenderContext, FHitProxyId InHitProxyId);
		void InitTileMesh(FRHICommandListBase& RHICmdList, const FSceneView& View);
		void ReleaseTileMesh();

		FRawIndexBuffer16or32 IndexBuffer;
		FStaticMeshVertexBuffers StaticMeshVertexBuffers;
		FTileVertexFactory VertexFactory;

		struct FTileInst
		{
			float X, Y;
			float SizeX, SizeY;
			float U, V;
			float SizeU, SizeV;
			FHitProxyId HitProxyId;
			FColor InColor;
		};
		TArray<FTileInst> Tiles;
	};

	/**
	 * Render data which is allocated when a new FCanvasTileRendererItem is added for rendering.
	 * This data is only freed on the rendering thread once the item has finished rendering
	 */
	TSharedPtr<FRenderData> Data;

	const bool bFreezeTime;
};

/**
* Info needed to render a single FTriangleRenderer
*/
class FCanvasTriangleRendererItem : public FCanvasBaseRenderItem
{
public:
	/**
	* Init constructor
	*/
	FCanvasTriangleRendererItem(ERHIFeatureLevel::Type InFeatureLevel,
		const FMaterialRenderProxy* InMaterialRenderProxy = NULL,
		const FCanvas::FTransformEntry& InTransform = FCanvas::FTransformEntry(FMatrix::Identity),
		bool bInFreezeTime = false)
		// this data is deleted after rendering has completed
		: Data(MakeShared<FRenderData>(InFeatureLevel, InMaterialRenderProxy, InTransform))
		, bFreezeTime(bInFreezeTime)
	{}

	/**
	 * FCanvasTriangleRendererItem instance accessor
	 *
	 * @return this instance
	 */
	virtual class FCanvasTriangleRendererItem* GetCanvasTriangleRendererItem() override
	{
		return this;
	}

	/**
	* Renders the canvas item.
	* Iterates over each triangle to be rendered and draws it with its own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @param RHICmdList - command list to use
	* @return true if anything rendered
	*/
	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas) override;

	/**
	* Renders the canvas item.
	* Iterates over each triangle to be rendered and draws it with its own transforms
	*
	* @param Canvas - canvas currently being rendered
	* @return true if anything rendered
	*/
	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope) override;

	/**
	* Determine if this is a matching set by comparing material,transform. All must match
	*
	* @param IInMaterialRenderProxy - material proxy resource for the item being rendered
	* @param InTransform - the transform for the item being rendered
	* @return true if the parameters match this render item
	*/
	bool IsMatch(const FMaterialRenderProxy* InMaterialRenderProxy, const FCanvas::FTransformEntry& InTransform)
	{
		return(Data->MaterialRenderProxy == InMaterialRenderProxy &&
			Data->Transform.GetMatrixCRC() == InTransform.GetMatrixCRC());
	};

	/**
	* Add a new triangle to the render data. These triangles all use the same transform and material proxy
	*
	* @param return number of triangles added
	*/
	inline int32 AddTriangle(const FCanvasUVTri& Tri, FHitProxyId HitProxyId)
	{
		return Data->AddTriangle(Tri, HitProxyId);
	};

	/**
	 * Reserves space in array for NumTriangles new triangles.
	 *
	 * @param NumTriangles Additional number of triangles to reserve space for.
	 */
	inline void AddReserveTriangles(int32 NumTriangles)
	{
		Data->AddReserveTriangles(NumTriangles);
	}

	/**
	* Reserves space in array for at least NumTriangles total triangles.
	*
	* @param NumTriangles Additional number of triangles to reserve space for.
	*/
	inline void ReserveTriangles(int32 NumTriangles)
	{
		Data->ReserveTriangles(NumTriangles);
	}

private:
	class FTriangleVertexFactory : public FLocalVertexFactory
	{
	public:
		FTriangleVertexFactory(const FStaticMeshVertexBuffers* VertexBuffers, ERHIFeatureLevel::Type InFeatureLevel);
		void InitResource(FRHICommandListBase& RHICmdList) override;

	private:
		const FStaticMeshVertexBuffers* VertexBuffers;
	};

	class FRenderData
	{
	public:
		FRenderData(ERHIFeatureLevel::Type InFeatureLevel,
			const FMaterialRenderProxy* InMaterialRenderProxy,
			const FCanvas::FTransformEntry& InTransform)
			: MaterialRenderProxy(InMaterialRenderProxy)
			, Transform(InTransform)
			, VertexFactory(&StaticMeshVertexBuffers, InFeatureLevel)
		{}

		~FRenderData();

		inline int32 AddTriangle(const FCanvasUVTri& Tri, FHitProxyId HitProxyId)
		{
			FTriangleInst NewTri = { Tri, HitProxyId };
			return Triangles.Add(NewTri);
		};

		inline void AddReserveTriangles(int32 NumTriangles)
		{
			Triangles.Reserve(Triangles.Num() + NumTriangles);
		}

		inline void ReserveTriangles(int32 NumTriangles)
		{
			Triangles.Reserve(NumTriangles);
		}

		void RenderTriangles(
			FCanvasRenderContext& RenderContext,
			FMeshPassProcessorRenderState& DrawRenderState,
			const FSceneView& View,
			bool bIsHitTesting);

		const FMaterialRenderProxy* const MaterialRenderProxy;
		const FCanvas::FTransformEntry Transform;

		uint32 GetNumVertices() const;
		uint32 GetNumIndices() const;

	private:
		FMeshBatch* AllocTriangleMeshBatch(FCanvasRenderContext& InRenderContext, FHitProxyId InHitProxyId);
		void InitTriangleMesh(FRHICommandListBase& RHICmdList, const FSceneView& View);
		void ReleaseTriangleMesh();

		FRawIndexBuffer16or32 IndexBuffer;
		FStaticMeshVertexBuffers StaticMeshVertexBuffers;
		FTriangleVertexFactory VertexFactory;

		struct FTriangleInst
		{
			FCanvasUVTri Tri;
			FHitProxyId HitProxyId;
		};
		TArray<FTriangleInst> Triangles;
	};

	/**
	 * Render data which is allocated when a new FCanvasTriangleRendererItem is added for rendering.
	 * This data is only freed on the rendering thread once the item has finished rendering
	 */
	TSharedPtr<FRenderData> Data;

	const bool bFreezeTime;
};
