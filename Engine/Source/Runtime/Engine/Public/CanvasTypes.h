// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Canvas.h: Unreal canvas definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "GameTime.h"
#include "SceneTypes.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "HitProxies.h"
#include "CanvasTypes.generated.h"

class FBatchedElementParameters;
class FBatchedElements;
class FRDGBuilder;
class FRHICommandListImmediate;
class FCanvasBatchedElementRenderItem;
class FCanvasRenderContext;
class FCanvasRenderThreadScope;
class FCanvasItem;
class FTexture;
class FMaterialRenderProxy;
class FRenderTarget;
class FSceneInterface;
class IBreakIterator;
class UFont;
class FRDGTexture;
namespace ERHIFeatureLevel { enum Type : int; }
struct FMeshPassProcessorRenderState;
using FRDGTextureRef = FRDGTexture*;
enum EShaderPlatform : uint16;

/**
 * General purpose data structure for grouping all parameters needed when sizing or wrapping a string
 */
USTRUCT()
struct FTextSizingParameters
{
	GENERATED_USTRUCT_BODY()

	/** a pixel value representing the horizontal screen location to begin rendering the string */
	UPROPERTY()
	float DrawX;

	/** a pixel value representing the vertical screen location to begin rendering the string */
	UPROPERTY()
	float DrawY;

	/** a pixel value representing the width of the area available for rendering the string */
	UPROPERTY()
	float DrawXL;

	/** a pixel value representing the height of the area available for rendering the string */
	UPROPERTY()
	float DrawYL;

	/** A value between 0.0 and 1.0, which represents how much the width/height should be scaled, where 1.0 represents 100% scaling. */
	UPROPERTY()
	FVector2D Scaling;

	/** the font to use for sizing/wrapping the string */
	UPROPERTY()
	TObjectPtr<const UFont> DrawFont;

	/** Horizontal spacing adjustment between characters and vertical spacing adjustment between wrapped lines */
	UPROPERTY()
	FVector2D SpacingAdjust;

	FTextSizingParameters()
		: DrawX(0)
		, DrawY(0)
		, DrawXL(0)
		, DrawYL(0)
		, Scaling(ForceInit)
		, DrawFont(NULL)
		, SpacingAdjust(ForceInit)
	{
	}


		FTextSizingParameters( float inDrawX, float inDrawY, float inDrawXL, float inDrawYL, const UFont* inFont=NULL )
		: DrawX(inDrawX), DrawY(inDrawY), DrawXL(inDrawXL), DrawYL(inDrawYL)
		, Scaling(1.f,1.f), DrawFont(inFont)
		, SpacingAdjust( 0.0f, 0.0f )
		{
		}
		FTextSizingParameters( const UFont* inFont, float ScaleX, float ScaleY)
		: DrawX(0.f), DrawY(0.f), DrawXL(0.f), DrawYL(0.f)
		, Scaling(ScaleX,ScaleY), DrawFont(inFont)
		, SpacingAdjust( 0.0f, 0.0f )
		{
		}
	
};

/**
 * Used by UUIString::WrapString to track information about each line that is generated as the result of wrapping.
 */
USTRUCT()
struct FWrappedStringElement
{
	GENERATED_USTRUCT_BODY()

	/** the string associated with this line */
	UPROPERTY()
	FString Value;

	/** the size (in pixels) that it will take to render this string */
	UPROPERTY()
	FVector2D LineExtent;

	FWrappedStringElement()
		: LineExtent(ForceInit)
	{
	}

	FWrappedStringElement(FString&& InValue, float Width, float Height )
		: Value(MoveTemp(InValue)), LineExtent(Width,Height)
	{}

	FWrappedStringElement(const FString& InValue, float Width, float Height)
		: Value(CopyTemp(InValue)), LineExtent(Width, Height)
	{}
	
};

class FCanvasWordWrapper
{
public:
	/** Array of indices where the wrapped lines begin and end in the source string */
	typedef TArray< TPair<int32, int32> > FWrappedLineData;

private:
	struct FWrappingState
	{
		FWrappingState(	FStringView InString,
						const FTextSizingParameters& InParameters,
						TArray<FWrappedStringElement>& InResults,
						FWrappedLineData* const InWrappedLineData)
			: String(InString)
			, Parameters(InParameters)
			, StartIndex(0)
			, Results(InResults)
			, WrappedLineData(InWrappedLineData)
		{}

		FStringView String;
		const FTextSizingParameters& Parameters;
		int32 StartIndex;
		TArray<FWrappedStringElement>& Results;
		FWrappedLineData* const WrappedLineData;
	};

public:
	ENGINE_API FCanvasWordWrapper();

	/**
	* Used to generate multi-line/wrapped text.
	*
	* @param InString The unwrapped text.
	* @param InFontInfo The font used to render the text.
	* @param InWrapWidth The width available.
	* @param OutWrappedLineData An optional array to fill with the indices from the source string marking the begin and end points of the wrapped lines
	*/
	ENGINE_API void Execute(FStringView InString, const FTextSizingParameters& InParameters, TArray<FWrappedStringElement>& OutStrings, FWrappedLineData* const OutWrappedLineData);

private:
	/**
		* Processes the string using a word wrapping algorithm, resulting in up to a single line.
		*
		* @return	True if a new line could be processed, false otherwise such as having reached the end of the string.
		*/
	bool ProcessLine(FWrappingState& WrappingState);

	/**
		* Stub method that should measure the substring of the range [StartIndex, EndIndex).
		*
		* @return	True if the substring fits the desired width.
		*/
	bool DoesSubstringFit(FWrappingState& WrappingState, const int32 EndIndex);

	/**
		* Stub method that should measure the substring starting from StartIndex until the wrap width is found or no more indices remain.
		*
		* @return	The index of the character that is at or after the desired width.
		*/
	int32 FindIndexAtOrAfterWrapWidth(FWrappingState& WrappingState);

	/**
		* Stub method for processing a single produced substring of the range [StartIndex, EndIndex) as a new line.
		*/
	void AddLine(FWrappingState& WrappingState, const int32 EndIndex);

	int32 FindFirstMandatoryBreakBetween(FWrappingState& WrappingState, const int32 WrapIndex);
	int32 FindLastBreakCandidateBetween(const int32 StartIndex, const int32 WrapIndex);
	int32 FindEndOfLastWholeGraphemeCluster(const int32 StartIndex, const int32 WrapIndex);

private:
	TSharedPtr<IBreakIterator> GraphemeBreakIterator;
	TSharedPtr<IBreakIterator> LineBreakIterator;
};

/**
 * Encapsulates the canvas state.
 */
class FCanvas
{
public:

	/**
	 * Enum that describes what type of element we are currently batching.
	 */
	enum EElementType
	{
		ET_Line,
		ET_Triangle,
		ET_MAX
	};

	/**
	 * Enum for canvas features that are allowed
	 **/
	enum ECanvasAllowModes
	{
		// flushing and rendering
		Allow_Flush = 1 << 0,
		// delete the render batches when rendering
		Allow_DeleteOnRender = 1 << 1
	};

	enum ECanvasDrawMode
	{
		CDM_DeferDrawing,
		CDM_ImmediateDrawing
	};

	ENGINE_API static FCanvas* Create(FRDGBuilder& GraphBuilder, FRDGTextureRef InRenderTarget, FHitProxyConsumer* InHitProxyConsumer, const FGameTime& Time, ERHIFeatureLevel::Type InFeatureLevel, float InDPIScale = 1.0f);

	/**
	* Constructor.
	*/
	ENGINE_API FCanvas(FRenderTarget* InRenderTarget, FHitProxyConsumer* InHitProxyConsumer, UWorld* InWorld, ERHIFeatureLevel::Type InFeatureLevel, ECanvasDrawMode DrawMode = CDM_DeferDrawing, float InDPIScale = 1.0f);

	/**
	* Constructor. For situations where a world is not available, but time information is
	*/
	ENGINE_API FCanvas(FRenderTarget* InRenderTarget, FHitProxyConsumer* InHitProxyConsumer, const FGameTime& Time, ERHIFeatureLevel::Type InFeatureLevel, float InDPIScale = 1.0f);
	
	/**
	* Destructor.
	*/
	ENGINE_API ~FCanvas();


	ENGINE_API static ESimpleElementBlendMode BlendToSimpleElementBlend(EBlendMode BlendMode);
	/**
	* Returns a FBatchedElements pointer to be used for adding vertices and primitives for rendering.
	* Adds a new render item to the sort element entry based on the current sort key.
	*
	* @param InElementType - Type of element we are going to draw.
	* @param InBatchedElementParameters - Parameters for this element
	* @param InTexture - New texture that will be set.
	* @param InBlendMode - New blendmode that will be set.
	* @param GlowInfo - info for optional glow effect when using depth field rendering
	* @return Returns a pointer to a FBatchedElements object.
	*/
	ENGINE_API FBatchedElements* GetBatchedElements(EElementType InElementType, FBatchedElementParameters* InBatchedElementParameters = NULL, const FTexture* Texture = NULL, ESimpleElementBlendMode BlendMode = SE_BLEND_MAX, const FDepthFieldGlowInfo& GlowInfo = FDepthFieldGlowInfo(), bool bApplyDPIScale = true);

	/**
	* Generates a new FCanvasTileRendererItem for the current sortkey and adds it to the sortelement list of items to render
	*/
	ENGINE_API void AddTileRenderItem(float X, float Y, float SizeX, float SizeY, float U, float V, float SizeU, float SizeV, const FMaterialRenderProxy* MaterialRenderProxy, FHitProxyId HitProxyId, bool bFreezeTime, FColor InColor);

	/**
	* Generates a new FCanvasTriangleRendererItem for the current sortkey and adds it to the sortelement list of items to render
	*/
	ENGINE_API void AddTriangleRenderItem(const FCanvasUVTri& Tri, const FMaterialRenderProxy* MaterialRenderProxy, FHitProxyId HitProxyId, bool bFreezeTime);

	/**
	* Sends a message to the rendering thread to draw the batched elements.
	* @param RHICmdList - command list to use
	* @param bForce - force the flush even if Allow_Flush is not enabled
	* @param bInsideRenderPass - Set to true if flushing inside a render pass (e.g. Render Graph pass).
	*	This will skip creating a render pass internally, and assert if the command list is not in a render pass.
	*/
	ENGINE_API void Flush_RenderThread(FRHICommandListImmediate& RHICmdList, bool bForce = false);
	ENGINE_API void Flush_RenderThread(FRDGBuilder& GraphBuilder, bool bForce = false);

	/**
	* Sends a message to the rendering thread to draw the batched elements.
	* @param bForce - force the flush even if Allow_Flush is not enabled
	*/
	ENGINE_API void Flush_GameThread(bool bForce = false);

	/**
	 * Pushes a transform onto the canvas's transform stack, multiplying it with the current top of the stack.
	 * @param Transform - The transform to push onto the stack.
	 */
	ENGINE_API void PushRelativeTransform(const FMatrix& Transform);

	/**
	 * Pushes a transform onto the canvas's transform stack.
	 * @param Transform - The transform to push onto the stack.
	 */
	ENGINE_API void PushAbsoluteTransform(const FMatrix& Transform);

	/**
	 * Removes the top transform from the canvas's transform stack.
	 */
	ENGINE_API void PopTransform();

	/**
	* Replace the base (ie. TransformStack(0)) transform for the canvas with the given matrix
	*
	* @param Transform - The transform to use for the base
	*/
	ENGINE_API void SetBaseTransform(const FMatrix& Transform);

	/**
	* Generate a 2D projection for the canvas. Use this if you only want to transform in 2D on the XY plane
	*
	* @param ViewSizeX - Viewport width
	* @param ViewSizeY - Viewport height
	* @return Matrix for canvas projection
	*/
	ENGINE_API static FMatrix CalcBaseTransform2D(uint32 ViewSizeX, uint32 ViewSizeY);

	/**
	* Generate a 3D projection for the canvas. Use this if you want to transform in 3D
	*
	* @param ViewSizeX - Viewport width
	* @param ViewSizeY - Viewport height
	* @param fFOV - Field of view for the projection
	* @param NearPlane - Distance to the near clip plane
	* @return Matrix for canvas projection
	*/
	ENGINE_API static FMatrix CalcBaseTransform3D(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV, float NearPlane);

	/**
	* Generate a view matrix for the canvas. Used for CalcBaseTransform3D
	*
	* @param ViewSizeX - Viewport width
	* @param ViewSizeY - Viewport height
	* @param fFOV - Field of view for the projection
	* @return Matrix for canvas view orientation
	*/
	ENGINE_API static FMatrix CalcViewMatrix(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV);

	/**
	* Generate a projection matrix for the canvas. Used for CalcBaseTransform3D
	*
	* @param ViewSizeX - Viewport width
	* @param ViewSizeY - Viewport height
	* @param fFOV - Field of view for the projection
	* @param NearPlane - Distance to the near clip plane
	* @return Matrix for canvas projection
	*/
	ENGINE_API static FMatrix CalcProjectionMatrix(uint32 ViewSizeX, uint32 ViewSizeY, float fFOV, float NearPlane);

	/**
	* Get the current top-most transform entry without the canvas projection
	* @return matrix from transform stack.
	*/
	FMatrix GetTransform() const
	{
		return TransformStack.Top().GetMatrix() * TransformStack[0].GetMatrix().InverseFast();
	}

	/**
	* Get the bottom-most element of the transform stack.
	* @return matrix from transform stack.
	*/
	const FMatrix& GetBottomTransform() const
	{
		return TransformStack[0].GetMatrix();
	}

	/**
	* Get the current top-most transform entry
	* @return matrix from transform stack.
	*/
	const FMatrix& GetFullTransform() const
	{
		return TransformStack.Top().GetMatrix();
	}

	/**
	 * Copy the conents of the TransformStack from an existing canvas
	 *
	 * @param Copy	canvas to copy from
	 **/
	ENGINE_API void CopyTransformStack(const FCanvas& Copy);

	/**
	 * Sets the render target which will be used for subsequent canvas primitives.
	 */
	ENGINE_API void SetRenderTarget_GameThread(FRenderTarget* NewRenderTarget);

	/**
	* Get the current render target for the canvas
	*/
	inline FRenderTarget* GetRenderTarget() const
	{
		return RenderTarget;
	}

	/**
	 * Sets a rect that should be used to offset rendering into the viewport render target
	 * If not set the canvas will render to the full target
	 *
	 * @param ViewRect The rect to use
	 */
	ENGINE_API void SetRenderTargetRect(const FIntRect& ViewRect);

	/**
	 * The clipping rectangle used when rendering this canvas
	 * @param ScissorRect The rect to use
	 */
	ENGINE_API void SetRenderTargetScissorRect( const FIntRect& ScissorRect );

	/**
	* Marks render target as dirty so that it will be resolved to texture
	*/
	void SetRenderTargetDirty(bool bDirty)
	{
		bRenderTargetDirty = bDirty;
	}

	/**
	* Sets the hit proxy which will be used for subsequent canvas primitives.
	*/
	ENGINE_API void SetHitProxy(HHitProxy* HitProxy);

	// HitProxy Accessors.	

	FHitProxyId GetHitProxyId() const { return CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId(); }
	FHitProxyConsumer* GetHitProxyConsumer() const { return HitProxyConsumer; }
	bool IsHitTesting() const { return HitProxyConsumer != NULL; }

	FSceneInterface* GetScene() const { return Scene; }

	/**
	* Push sort key onto the stack. Rendering is done with the current sort key stack entry.
	*
	* @param InSortKey - key value to push onto the stack
	*/
	void PushDepthSortKey(int32 InSortKey)
	{
		DepthSortKeyStack.Push(InSortKey);
	};

	/**
	* Pop sort key off of the stack.
	*
	* @return top entry of the sort key stack
	*/
	int32 PopDepthSortKey()
	{
		int32 Result = 0;
		if (DepthSortKeyStack.Num() > 0)
		{
			Result = DepthSortKeyStack.Pop();
		}
		else
		{
			// should always have one entry
			PushDepthSortKey(0);
		}
		return Result;
	};

	/**
	* Return top sort key of the stack.
	*
	* @return top entry of the sort key stack
	*/
	int32 TopDepthSortKey()
	{
		checkSlow(DepthSortKeyStack.Num() > 0);
		return DepthSortKeyStack.Top();
	}

	/**
	 * Toggle allowed canvas modes
	 *
	 * @param InAllowedModes	New modes to set
	 */
	void SetAllowedModes(uint32 InAllowedModes)
	{
		AllowedModes = InAllowedModes;
	}
	/**
	 * Accessor for allowed canvas modes
	 *
	 * @return current allowed modes
	 */
	uint32 GetAllowedModes() const
	{
		return AllowedModes;
	}

	/**
	 * Determine if the canvas has dirty batches that need to be rendered
	 *
	 * @return true if the canvas has any element to render
	 */
	bool HasBatchesToRender() const;

	/**
	 * Clear the list of dirty batches that need to be rendered by the canvas
	 */
	void ClearBatchesToRender();

	/**
	* Access current feature level
	*
	* @return feature level that this canvas is rendering at
	*/
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	/**
	* Access current shader platform
	*
	* @return shader platform that this canvas is rendering at
	*/
	ENGINE_API EShaderPlatform GetShaderPlatform() const;

public:
	float AlphaModulate;

	/** Entry for the transform stack which stores a matrix and its CRC for faster comparisons */
	class FTransformEntry
	{
	public:
		FTransformEntry(const FMatrix& InMatrix)
			: Matrix(InMatrix)
		{
			MatrixCRC = FCrc::MemCrc_DEPRECATED(&Matrix, sizeof(FMatrix));
		}
		inline void SetMatrix(const FMatrix& InMatrix)
		{
			Matrix = InMatrix;
			MatrixCRC = FCrc::MemCrc_DEPRECATED(&Matrix, sizeof(FMatrix));
		}
		inline const FMatrix& GetMatrix() const
		{
			return Matrix;
		}
		inline uint32 GetMatrixCRC() const
		{
			return MatrixCRC;
		}
	private:
		FMatrix Matrix;
		uint32 MatrixCRC;
	};

	/** returns the transform stack */
	inline const TArray<FTransformEntry>& GetTransformStack() const
	{
		return TransformStack;
	}
	inline const FIntRect& GetViewRect() const
	{
		return ViewRect;
	}

	inline void SetScaledToRenderTarget(bool bScale = true)
	{
		bScaledToRenderTarget = bScale;
	}
	inline bool IsScaledToRenderTarget() const { return bScaledToRenderTarget; }

	inline void SetStereoRendering(bool bStereo = true)
	{
		bStereoRendering = bStereo;
	}
	inline bool IsStereoRendering() const { return bStereoRendering; }

	inline void SetUseInternalTexture(const bool bInUseInternalTexture)
	{
		bUseInternalTexture = bInUseInternalTexture;
	}

	inline bool IsUsingInternalTexture() const { return bUseInternalTexture; }

	inline void SetWriteDestinationAlpha(const bool bInForceWriteAlpha)
	{
		bWriteDestinationAlpha = bInForceWriteAlpha;
	}

	inline bool IsWriteDestinationAlphaSet() const { return bWriteDestinationAlpha; }

	inline void SetParentCanvasSize(FIntPoint InParentSize)
	{
		ParentSize = InParentSize;
	}

	inline FIntPoint GetParentCanvasSize() const { return ParentSize; }

	float GetDPIScale() const { return  bStereoRendering ? 1.0f : DPIScale; }

	int16 GetSliceIndex() const
	{
		return SliceIndex;
	}

	ENGINE_API void SetSliceIndex(int16 InSliceIndex);

public:
	/** Private class for handling word wrapping behavior. */
	TSharedPtr<FCanvasWordWrapper> WordWrapper;

private:

	/** Stack of SortKeys. All rendering is done using the top most sort key */
	TArray<int32> DepthSortKeyStack;	
	/** Stack of matrices. Bottom most entry is the canvas projection */
	TArray<FTransformEntry> TransformStack;	
	/** View rect for the render target */
	FIntRect ViewRect;
	/** Scissor rect for the render target */
	FIntRect ScissorRect;
	/** Current render target used by the canvas */
	FRenderTarget* RenderTarget;
	/** Current hit proxy consumer */
	FHitProxyConsumer* HitProxyConsumer;
	/** Current hit proxy object */
	TRefCountPtr<HHitProxy> CurrentHitProxy;
	/* Optional scene for rendering. */
	FSceneInterface* Scene;
	/** Toggles for various canvas rendering functionality **/
	uint32 AllowedModes;
	/** true if the render target has been rendered to since last calling SetRenderTarget() */
	bool bRenderTargetDirty;	
	/** Current gameplay time */
	FGameTime Time;
	/** true, if Canvas should be scaled to whole render target */
	bool bScaledToRenderTarget;
	/** Feature level that we are currently rendering with */
	ERHIFeatureLevel::Type FeatureLevel;

	/** true, if Canvas should be rendered in stereo */
	bool bStereoRendering;

	/** true, if Canvas is being rendered in its own texture */
	bool bUseInternalTexture;

	/** true, if the destination alpha channel must be written */
	bool bWriteDestinationAlpha;

	FIntPoint ParentSize;

	ECanvasDrawMode DrawMode;

	float DPIScale;

	int16 SliceIndex = -1;

	/** 
	* Shared construction function
	*/
	void Construct();

public:	

	/**
	 * Access gameplay time
	 */
	const FGameTime& GetTime() const
	{
		return Time;
	}
	
	/** 
	 * Draw a CanvasItem
	 *
	 * @param Item			Item to draw
	 */
	ENGINE_API void DrawItem(FCanvasItem& Item);

	/**
	 * Draw a CanvasItem at the given coordinates
	 *
	 * @param Item			Item to draw
	 * @param InPosition	Position to draw item
	 */
	ENGINE_API void DrawItem(FCanvasItem& Item, const FVector2D& InPosition);

	/** 
	 * Draw a CanvasItem at the given coordinates
	 *
	 * @param Item			Item to draw
	 * @param X				X Position to draw item
	 * @param Y				Y Position to draw item
	 */
	ENGINE_API void DrawItem(FCanvasItem& Item, float X, float Y);

	/**
	* Clear the canvas
	*
	* @param	Color		Color to clear with.
	*/
	ENGINE_API void Clear(const FLinearColor& Color);



	/** 
	* Draw arbitrary aligned rectangle.
	*
	* @param X - X position to draw tile at
	* @param Y - Y position to draw tile at
	* @param SizeX - Width of tile
	* @param SizeY - Height of tile
	* @param U - Horizontal position of the upper left corner of the portion of the texture to be shown(texels)
	* @param V - Vertical position of the upper left corner of the portion of the texture to be shown(texels)
	* @param SizeU - The width of the portion of the texture to be drawn. This value is in texels. 
	* @param SizeV - The height of the portion of the texture to be drawn. This value is in texels. 
	* @param Color - tint applied to tile
	* @param Texture - Texture to draw
	* @param AlphaBlend - true to alphablend
	*/
	ENGINE_API void DrawTile(double X, double Y, double SizeX, double SizeY, float U, float V, float SizeU, float SizeV, const FLinearColor& Color, const FTexture* Texture = NULL, bool AlphaBlend = true);
	ENGINE_API void DrawTile(double X, double Y, double SizeX, double SizeY, float U, float V, float SizeU, float SizeV, const FLinearColor& Color, const FTexture* Texture, ESimpleElementBlendMode BlendMode);

	/** 
	* Draw an string centered on given location. 
	* This function is being deprecated. a FCanvasTextItem should be used instead.
	* 
	* @param StartX - X point
	* @param StartY - Y point
	* @param Text - Text to draw
	* @param Font - Font to use
	* @param Color - Color of the text
	* @param ShadowColor - Shadow color to draw underneath the text (ignored for distance field fonts)
	* @return total size in pixels of text drawn
	*/
	ENGINE_API int32 DrawShadowedString(double StartX, double StartY, FStringView Text, const UFont* Font, const FLinearColor& Color, const FLinearColor& ShadowColor = FLinearColor::Black );
	
	ENGINE_API int32 DrawShadowedText(double StartX, double StartY, const FText& Text, const UFont* Font, const FLinearColor& Color, const FLinearColor& ShadowColor = FLinearColor::Black );

	ENGINE_API void WrapString( FTextSizingParameters& Parameters, const float InCurX, FStringView Text, TArray<FWrappedStringElement>& out_Lines, FCanvasWordWrapper::FWrappedLineData* const OutWrappedLineData = nullptr);

	ENGINE_API void DrawNGon(const FVector2D& Center, const FColor& Color, int32 NumSides, float Radius);

	/** 
	* Contains all of the batched elements that need to be rendered at a certain depth sort key
	*/
	class FCanvasSortElement
	{
	public:
		/** 
		* Init constructor 
		*/
		FCanvasSortElement(int32 InDepthSortKey=0)
			:	DepthSortKey(InDepthSortKey)
		{}

		/** 
		* Equality is based on sort key 
		*
		* @param Other - instance to compare against
		* @return true if equal
		*/
		bool operator==(const FCanvasSortElement& Other) const
		{
			return DepthSortKey == Other.DepthSortKey;
		}

		/** sort key for this set of render batch elements */
		int32 DepthSortKey;
		/** list of batches that should be rendered at this sort key level */
		TArray<class FCanvasBaseRenderItem*> RenderBatchArray;
	};
	
	/** Batched canvas elements to be sorted for rendering. Sort order is back-to-front */
	TArray<FCanvasSortElement> SortedElements;
	/** Map from sortkey to array index of SortedElements for faster lookup of existing entries */
	TMap<int32,int32> SortedElementLookupMap;

	/** Store index of last Element off to avoid semi expensive Find() */
	int32 LastElementIndex;
	
	/**
	* Get the sort element for the given sort key. Allocates a new entry if one does not exist
	*
	* @param DepthSortKey - the key used to find the sort element entry
	* @return sort element entry
	*/
	ENGINE_API FCanvasSortElement& GetSortElement(int32 DepthSortKey);

	friend class FCanvasRenderContext;
	friend class FCanvasRenderThreadScope;
};

/**
* Base interface for canvas items which can be batched for rendering
*/
class FCanvasBaseRenderItem
{
public:
	virtual ~FCanvasBaseRenderItem() = default;

	/**
	* Renders the canvas item
	*
	* @param RenderContext - the canvas render context to submit render passes to.
	* @param Canvas - canvas currently being rendered
	* @return true if anything rendered
	*/
	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas) = 0;
	
	/**
	* Renders the canvas item
	*
	* @param Canvas - canvas currently being rendered
	* @return true if anything rendered
	*/
	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope) = 0;
	
	/**
	* FCanvasBatchedElementRenderItem instance accessor
	*
	* @return FCanvasBatchedElementRenderItem instance
	*/
	virtual class FCanvasBatchedElementRenderItem* GetCanvasBatchedElementRenderItem() { return NULL; }
	
	/**
	* FCanvasTileRendererItem instance accessor
	*
	* @return FCanvasTileRendererItem instance
	*/
	virtual class FCanvasTileRendererItem* GetCanvasTileRendererItem() { return NULL; }

	/**
	* FCanvasTriangleRendererItem instance accessor
	*
	* @return FCanvasTriangleRendererItem instance
	*/
	virtual class FCanvasTriangleRendererItem* GetCanvasTriangleRendererItem() { return NULL; }
};


/**
* Render string using both a font and a material. The material should have a font exposed as a 
* parameter so that the correct font page can be set based on the character being drawn.
*
* @param Font - font containing texture pages of character glyphs
* @param XL - out width
* @param YL - out height
* @param Text - string of text to be measured
*/
extern ENGINE_API void StringSize( const UFont* Font, int32& XL, int32& YL, FStringView Text);

/**
 * Helper class to write a line of texts on screen
 */
struct FScreenMessageWriter
{
	FScreenMessageWriter(FCanvas& InCanvas, int32 InY)
		: Canvas(InCanvas)
		, Y(InY)
	{}

	inline void EmptyLine()
	{
		Y += 14;
	}

	ENGINE_API void DrawLine(const FText& Message, int32 X = 10, const FLinearColor& Color = FLinearColor(1.0f, 0.05f, 0.05f, 1.0f));

	FCanvas& Canvas;
	int32 Y;
};
