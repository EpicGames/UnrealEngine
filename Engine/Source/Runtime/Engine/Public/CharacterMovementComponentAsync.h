// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GameFramework/RootMotionSource.h"
#include "Chaos/SimCallbackObject.h"
#include "CollisionQueryParams.h"
#include "CollisionShape.h"
#include "Components/SceneComponent.h"
#include "Engine/OverlapInfo.h"
#include "CharacterMovementComponentAsync.generated.h"

struct FCharacterMovementComponentAsyncOutput;
struct FCharacterMovementComponentAsyncInput;

// TODO move these common structures to separate header?

// Enum used to control GetPawnCapsuleExtent behavior
enum EShrinkCapsuleExtent
{
	SHRINK_None,			// Don't change the size of the capsule
	SHRINK_RadiusCustom,	// Change only the radius, based on a supplied param
	SHRINK_HeightCustom,	// Change only the height, based on a supplied param
	SHRINK_AllCustom,		// Change both radius and height, based on a supplied param
};

/** Data about the floor for walking movement, used by CharacterMovementComponent. */
USTRUCT(BlueprintType)
struct FFindFloorResult
{
	GENERATED_USTRUCT_BODY()

		/**
		* True if there was a blocking hit in the floor test that was NOT in initial penetration.
		* The HitResult can give more info about other circumstances.
		*/
		UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		uint32 bBlockingHit : 1;

	/** True if the hit found a valid walkable floor. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		uint32 bWalkableFloor : 1;

	/** True if the hit found a valid walkable floor using a line trace (rather than a sweep test, which happens when the sweep test fails to yield a walkable surface). */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		uint32 bLineTrace : 1;

	/** The distance to the floor, computed from the swept capsule trace. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		float FloorDist;

	/** The distance to the floor, computed from the trace. Only valid if bLineTrace is true. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		float LineDist;

	/** Hit result of the test that found a floor. Includes more specific data about the point of impact and surface normal at that point. */
	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Category = CharacterFloor)
		FHitResult HitResult;

public:

	FFindFloorResult()
		: bBlockingHit(false)
		, bWalkableFloor(false)
		, bLineTrace(false)
		, FloorDist(0.f)
		, LineDist(0.f)
		, HitResult(1.f)
	{
	}

	/** Returns true if the floor result hit a walkable surface. */
	bool IsWalkableFloor() const
	{
		return bBlockingHit && bWalkableFloor;
	}

	void Clear()
	{
		bBlockingHit = false;
		bWalkableFloor = false;
		bLineTrace = false;
		FloorDist = 0.f;
		LineDist = 0.f;
		HitResult.Reset(1.f, false);
	}

	/** Gets the distance to floor, either LineDist or FloorDist. */
	float GetDistanceToFloor() const
	{
		// When the floor distance is set using SetFromSweep, the LineDist value will be reset.
		// However, when SetLineFromTrace is used, there's no guarantee that FloorDist is set.
		return bLineTrace ? LineDist : FloorDist;
	}

	ENGINE_API void SetFromSweep(const FHitResult& InHit, const float InSweepFloorDist, const bool bIsWalkableFloor);
	ENGINE_API void SetFromLineTrace(const FHitResult& InHit, const float InSweepFloorDist, const float InLineDist, const bool bIsWalkableFloor);
};


/** Struct updated by StepUp() to return result of final step down, if applicable. */
struct FStepDownResult
{
	uint32 bComputedFloor : 1;		// True if the floor was computed as a result of the step down.
	FFindFloorResult FloorResult;	// The result of the floor test if the floor was updated.

	FStepDownResult()
		: bComputedFloor(false)
	{
	}
};

struct FCharacterAsyncOutput
{
	virtual ~FCharacterAsyncOutput() = default;

	// Character Owner Data
	FRotator Rotation;
	int32 JumpCurrentCountPreJump;
	int32 JumpCurrentCount;
	float JumpForceTimeRemaining;
	float JumpKeyHoldTime;
	bool bWasJumping;
	bool bPressedJump;
	bool bClearJumpInput; // If true when applying output we will clear bPressedJump on game thread.
};

struct FUpdatedComponentAsyncOutput
{
	// TODO Overlaps: When overlapping, if UpdatedComponent enables overlaps, we cannot read from
	// overlapped component to determine if it needs overlap event, because we are not on game thread.
	// We cache overlap regardless if other component enables Overlap events,
	// and will have to cull them on game thread when applying output.
	// TODO see ShouldIngoreOverlapResult, check WorldSEttings and ActorInitialized condition.
	TArray<FOverlapInfo> SpeculativeOverlaps;

	// stolen from prim component TODO 
	int32 IndexOfOverlap(const FOverlapInfo& SearchItem)
	{
		return SpeculativeOverlaps.IndexOfByPredicate(FFastOverlapInfoCompare(SearchItem));
	}

	// stolen from prim component TODO dedupe
	// Helper for adding an FOverlapInfo uniquely to an Array, using IndexOfOverlapFast and knowing that at least one overlap is valid (non-null
	void AddUniqueSpeculativeOverlap(const FOverlapInfo& NewOverlap)
	{
		if (IndexOfOverlap(NewOverlap) == INDEX_NONE)
		{
			SpeculativeOverlaps.Add(NewOverlap);
		}
	}
};


struct FCharacterMovementComponentAsyncOutput : public Chaos::FSimCallbackOutput
{
	using FCharacterOutput = FCharacterAsyncOutput;

	FCharacterMovementComponentAsyncOutput()
		: FSimCallbackOutput()
		, bIsValid(false)
	{
		CharacterOutput = MakeUnique<FCharacterOutput>();
	}

	FCharacterMovementComponentAsyncOutput(TUniquePtr<FCharacterOutput>&& InCharacterOutput)
		: FSimCallbackOutput()
		, CharacterOutput(MoveTemp(InCharacterOutput))
		, bIsValid(false)
	{
	}

	virtual ~FCharacterMovementComponentAsyncOutput() = default;

public:
	void Reset() { bIsValid = false; }

	ENGINE_API void Copy(const FCharacterMovementComponentAsyncOutput& Value);
	
	bool IsValid() const { return bIsValid; }

	static FRotator GetDeltaRotation(const FRotator& InRotationRate, float InDeltaTime);
	static float GetAxisDeltaRotation(float InAxisRotationRate, float InDeltaTime);

	bool bWasSimulatingRootMotion;
	EMovementMode MovementMode;
	EMovementMode GroundMovementMode;
	uint8 CustomMovementMode;
	FVector Acceleration;
	float AnalogInputModifier;
	FVector LastUpdateLocation;
	FQuat LastUpdateRotation;
	FVector LastUpdateVelocity;
	bool bForceNextFloorCheck;
	FVector Velocity;
	FVector LastPreAdditiveVelocity;
	bool bIsAdditiveVelocityApplied;
	bool bDeferUpdateBasedMovement;
	EMoveComponentFlags MoveComponentFlags;
	FVector PendingForceToApply;
	FVector PendingImpulseToApply;
	FVector PendingLaunchVelocity;
	bool bCrouchMaintainsBaseLocation;
	bool bJustTeleported;
	float ScaledCapsuleRadius;
	float ScaledCapsuleHalfHeight;
	bool bIsCrouched;
	bool bWantsToCrouch;
	bool bMovementInProgress;
	FFindFloorResult CurrentFloor;
	bool bHasRequestedVelocity;
	bool bRequestedMoveWithMaxSpeed;
	FVector RequestedVelocity;
	FVector LastUpdateRequestedVelocity;
	int32 NumJumpApexAttempts;
	FVector AnimRootMotionVelocity;
	bool bShouldApplyDeltaToMeshPhysicsTransforms; // See UpdateBasedMovement
	FVector DeltaPosition;
	FQuat DeltaQuat;
	float DeltaTime;
	FVector OldVelocity; // Cached for CallMovementUpdateDelegate
	FVector OldLocation;

	// Used to override the rotation rate in the presence of a velocity-based turn curve.
	FRotator ModifiedRotationRate;
	bool bUsingModifiedRotationRate = false;

	// See MaybeUpdateBasedMovement
	// TODO MovementBase, handle tick group changes properly
	bool bShouldDisablePostPhysicsTick;
	bool bShouldEnablePostPhysicsTick;
	bool bShouldAddMovementBaseTickDependency;
	bool bShouldRemoveMovementBaseTickDependency;
	UPrimitiveComponent* NewMovementBase; // call SetBase
	AActor* NewMovementBaseOwner; // make sure this is set whenever base component is. TODO

	FUpdatedComponentAsyncOutput UpdatedComponentOutput;
	TUniquePtr<FCharacterAsyncOutput> CharacterOutput;

	bool bIsValid;
};

// Don't read into this part too much it needs to be changed.
struct FCachedMovementBaseAsyncData
{
	// TODO: Convert this to use new FMovementBaseInterfaceData struct
	// data derived from movement base
	UPrimitiveComponent* CachedMovementBase; // Do not access, this was input movement base, only here so I could ensure when it changed.
	
	// Invalid if movement base changes.
	bool bMovementBaseUsesRelativeLocationCached;
	bool bMovementBaseIsSimulatedCached;
	bool bMovementBaseIsValidCached;
	bool bMovementBaseOwnerIsValidCached;
	bool bMovementBaseIsDynamicCached;

	bool bIsBaseTransformValid;
	FQuat BaseQuat;
	FVector BaseLocation;
	FQuat OldBaseQuat;
	FVector OldBaseLocation;

	// Calling this before reading movement base data, as if it changed during tick, we are using stale data,
	// can't read from game thread. Need to think about this more.
	void Validate(const FCharacterMovementComponentAsyncOutput& Output) const
	{
		ensure(Output.NewMovementBase == CachedMovementBase);
	}
};

struct FRootMotionAsyncData
{
	bool bHasAnimRootMotion;
	bool bHasOverrideRootMotion;
	bool bHasOverrideWithIgnoreZAccumulate;
	bool bHasAdditiveRootMotion;
	bool bUseSensitiveLiftoff;
	FVector AdditiveVelocity;
	FVector OverrideVelocity;
	FQuat OverrideRotation;
	FTransform AnimTransform;
	float TimeAccumulated;

	void Clear()
	{
		TimeAccumulated = 0.f;
		bHasAnimRootMotion = false;
		bHasOverrideRootMotion = false;
		bHasOverrideWithIgnoreZAccumulate = false;
		bHasAdditiveRootMotion = false;
		bUseSensitiveLiftoff = false;
		AdditiveVelocity = FVector::ZeroVector;
		OverrideVelocity = FVector::ZeroVector;
		OverrideRotation = FQuat::Identity;
		AnimTransform = FTransform::Identity;
	}

	FRootMotionAsyncData()
	{
		Clear();
	}
};

// Data and implementation that lives on movement component's character owner
struct FCharacterAsyncInput
{
	virtual ~FCharacterAsyncInput() = default;

	// Character owner input
	float JumpMaxHoldTime;
	int32 JumpMaxCount;
	ENetRole  LocalRole;
	ENetRole RemoteRole;
	bool bIsLocallyControlled;
	bool bIsPlayingNetworkedRootMontage;
	bool bUseControllerRotationPitch;
	bool bUseControllerRotationYaw;
	bool bUseControllerRotationRoll;
	FRotator ControllerDesiredRotation;

	ENGINE_API virtual void FaceRotation(FRotator NewControlRotation, float DeltaTime, const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void CheckJumpInput(float DeltaSeconds, const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void ClearJumpInput(float DeltaSeconds, const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CanJump(const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void ResetJumpState(const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void OnMovementModeChanged(EMovementMode PrevMovementMode, const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output, uint8 PreviousCustomMode = 0);
};

// Represents the UpdatedComponent's state and implementation
struct FUpdatedComponentAsyncInput
{
	virtual ~FUpdatedComponentAsyncInput() = default;

	// base Implementation from PrimitiveComponent, this will be wrong if UpdatedComponent is SceneComponent.
	ENGINE_API virtual bool MoveComponent(const FVector& Delta, const FQuat& NewRotationQuat, bool bSweep, FHitResult* OutHit, EMoveComponentFlags MoveFlags, ETeleportType Teleport, const FCharacterMovementComponentAsyncInput& Input, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool AreSymmetricRotations(const FQuat& A, const FQuat& B, const FVector& Scale3D) const;

	// TODO Dedupe with PrimitiveComponent where possible
	static ENGINE_API void PullBackHit(FHitResult& Hit, const FVector& Start, const FVector& End, const float Dist);
	static ENGINE_API bool ShouldCheckOverlapFlagToQueueOverlaps(const UPrimitiveComponent& ThisComponent);
	static ENGINE_API bool ShouldIgnoreHitResult(const UWorld* InWorld, FHitResult const& TestHit, FVector const& MovementDirDenormalized, const AActor* MovingActor, EMoveComponentFlags MoveFlags);
	static ENGINE_API bool ShouldIgnoreOverlapResult(const UWorld* World, const AActor* ThisActor, const UPrimitiveComponent& ThisComponent, const AActor* OtherActor, const UPrimitiveComponent& OtherComponent);

	FVector GetForwardVector() const { return GetRotation().GetAxisX(); }
	FVector GetRightVector() const { return GetRotation().GetAxisY(); }
	FVector GetUpVector() const { return GetRotation().GetAxisZ(); }

	// Async API, physics thread only.
	ENGINE_API void SetPosition(const FVector& Position) const;
	ENGINE_API FVector GetPosition() const;
	ENGINE_API void SetRotation(const FQuat& Rotation) const;
	ENGINE_API FQuat GetRotation() const;

	bool bIsQueryCollisionEnabled;
	bool bIsSimulatingPhysics;

	// PrimComponent->InitSweepCollisionParams + modified IgnoreTouches and trace tag.
	FComponentQueryParams MoveComponentQueryParams;
	FCollisionResponseParams MoveComponentCollisionResponseParams;

	UPrimitiveComponent* UpdatedComponent; // TODO Threadsafe make sure we aren't accessing this anywhere.
	FPhysicsActorHandle PhysicsHandle;

	FCollisionShape CollisionShape;
	bool bForceGatherOverlaps; // !ShouldCheckOverlapFlagToQueueOverlaps(*PrimitiveComponent);
	bool bGatherOverlaps; // GetGenerateOverlapEvents() || bForceGatherOverlaps
	FVector Scale;
};


// This contains inputs from GT that are applied to async sim state before simulation.
struct FCharacterMovementGTInputs
{
	bool bWantsToCrouch;
	
	bool bValidMovementMode; // Should we actually copy movement mode?
	EMovementMode MovementMode;

	// Pawn inputs
	bool bPressedJump;

	void UpdateOutput(FCharacterMovementComponentAsyncOutput& Output) const
	{
		Output.bWantsToCrouch = bWantsToCrouch;
		if (bValidMovementMode)
		{
			Output.MovementMode = MovementMode;
		}
		Output.CharacterOutput->bPressedJump = bPressedJump;
	}
};

/*
* Contains all input and implementation required to run async character movement.
* Base implementation is from CharacterMovementComponent.
* Contains 'CharacterInput' and 'UpdatedComponentInput' represent data/impl of Character and our UpdatedComponent.
* All input is const, non-const data goes in output. 'AsyncSimState' points to non-const sim state.
*/
struct FCharacterMovementComponentAsyncInput : public Chaos::FSimCallbackInput
{
	using FCharacterInput = FCharacterAsyncInput;
	using FUpdatedComponentInput = FUpdatedComponentAsyncInput;

	// Has this been filled out?
	bool bInitialized = false;

	FVector InputVector;
	ENetworkSmoothingMode NetworkSmoothingMode;
	bool bIsNetModeClient; // shared state, TODO remove.
	bool bWasSimulatingRootMotion;
	bool bRunPhysicsWithNoController;
	bool bForceMaxAccel;
	float MaxAcceleration;
	float MinAnalogWalkSpeed;
	bool bIgnoreBaseRotation;
	bool bOrientRotationToMovement;
	bool bUseControllerDesiredRotation;
	bool bConstrainToPlane;
	FVector PlaneConstraintOrigin;
	FVector PlaneConstraintNormal;
	bool bHasValidData; // TODO look into if this can become invalid during sim
	float MaxStepHeight;
	bool bAlwaysCheckFloor;
	float WalkableFloorZ;
	bool bUseFlatBaseForFloorChecks;
	float GravityZ;
	bool bCanEverCrouch;
	int32 MaxSimulationIterations;
	float MaxSimulationTimeStep;
	bool bMaintainHorizontalGroundVelocity;
	bool bUseSeparateBrakingFriction;
	float GroundFriction;
	float BrakingFrictionFactor;
	float BrakingFriction;
	float BrakingSubStepTime;
	float BrakingDecelerationWalking;
	float BrakingDecelerationFalling;
	float BrakingDecelerationSwimming;
	float BrakingDecelerationFlying;
	float MaxDepenetrationWithGeometryAsProxy;
	float MaxDepenetrationWithGeometry;
	float MaxDepenetrationWithPawn;
	float MaxDepenetrationWithPawnAsProxy;
	bool bCanWalkOffLedgesWhenCrouching;
	bool bCanWalkOffLedges;
	float LedgeCheckThreshold;
	float PerchRadiusThreshold;
	float AirControl;
	float AirControlBoostMultiplier;
	float AirControlBoostVelocityThreshold;
	bool bDontFallBelowJumpZVelocityDuringJump;
	bool bApplyGravityWhileJumping;
	float PhysicsVolumeTerminalVelocity;
	int32 MaxJumpApexAttemptsPerSimulation;
	EMovementMode DefaultLandMovementMode;
	float FallingLateralFriction;
	mutable float JumpZVelocity;
	bool bAllowPhysicsRotationDuringAnimRootMotion;
	bool bDeferUpdateMoveComponent;
	bool bRequestedMoveUseAcceleration;
	float PerchAdditionalHeight;
	bool bNavAgentPropsCanJump; // from NavAgentProps.bCanJump
	bool bMovementStateCanJump; // from MovementState.bCanJump
	float MaxWalkSpeedCrouched;
	float MaxWalkSpeed;
	float MaxSwimSpeed;
	float MaxFlySpeed;
	float MaxCustomMovementSpeed;
	FRotator RotationRate;
	FRootMotionAsyncData RootMotion;

	FCachedMovementBaseAsyncData MovementBaseAsyncData;
	TUniquePtr<FUpdatedComponentAsyncInput> UpdatedComponentInput;
	TUniquePtr<FCharacterAsyncInput> CharacterInput;
	

	UWorld* World; // Remove once we have physics thread scene query API
	//AActor* Owner; // TODO Threadsafe make sure this isn't accessed.

	// primitive component InitSweepCollisionParams
	FComponentQueryParams QueryParams;
	FCollisionResponseParams CollisionResponseParams;
	ECollisionChannel CollisionChannel;
	FCollisionQueryParams CapsuleParams;
	FRandomStream RandomStream;

	
	// This is the latest simulated state of this movement component.
	TSharedPtr<FCharacterMovementComponentAsyncOutput, ESPMode::ThreadSafe> AsyncSimState;

	FCharacterMovementGTInputs GTInputs;

	virtual ~FCharacterMovementComponentAsyncInput() = default;
	
	template <typename FCharacterInput, typename FUpdatedComponentInput> 
	void Initialize()
	{
		CharacterInput = MakeUnique<FCharacterInput>();
		UpdatedComponentInput = MakeUnique<FUpdatedComponentInput>();
	}

	void Reset()
	{
		/* TODO Should actually implement this.*/
		bInitialized = false;
		CharacterInput.Reset();
		UpdatedComponentInput.Reset();
		AsyncSimState.Reset();
	}

	void UpdateAsyncStateFromGTInputs_Internal() const
	{
		GTInputs.UpdateOutput(*AsyncSimState);
	}

	// Entry point of async tick
	ENGINE_API void Simulate(const float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;

	// TODO organize these
	ENGINE_API virtual void ControlledCharacterMove(const float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void PerformMovement(float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void MaybeUpdateBasedMovement(float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void UpdateBasedMovement(float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void StartNewPhysics(float deltaTime, int32 Iterations, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void PhysWalking(float deltaTime, int32 Iterations, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void PhysFalling(float deltaTime, int32 Iterations, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void PhysicsRotation(float DeltaTime, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void MoveAlongFloor(const FVector& InVelocity, float DeltaSeconds, FStepDownResult* OutStepDownResult, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector ComputeGroundMovementDelta(const FVector& Delta, const FHitResult& RampHit, const bool bHitFromLineTrace, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CanCrouchInCurrentState(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector ConstrainInputAcceleration(FVector InputAcceleration, const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector ScaleInputAcceleration(FVector InputAcceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual float ComputeAnalogInputModifier(FVector Acceleration) const;
	ENGINE_API virtual FVector ConstrainLocationToPlane(FVector Location) const;
	ENGINE_API virtual FVector ConstrainDirectionToPlane(FVector Direction) const;
	ENGINE_API virtual FVector ConstrainNormalToPlane(FVector Normal) const;
	ENGINE_API virtual void MaintainHorizontalGroundVelocity(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool MoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FCharacterMovementComponentAsyncOutput& Output, FHitResult* OutHitResult = nullptr, ETeleportType TeleportType = ETeleportType::None) const;
	ENGINE_API virtual bool SafeMoveUpdatedComponent(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult& OutHit, FCharacterMovementComponentAsyncOutput& Output, ETeleportType Teleport = ETeleportType::None) const;
	ENGINE_API virtual void ApplyAccumulatedForces(float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void ClearAccumulatedForces(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void SetMovementMode(EMovementMode NewMovementMode, FCharacterMovementComponentAsyncOutput& Output, uint8 NewCustomMode = 0) const;
	ENGINE_API virtual void OnMovementModeChanged(EMovementMode PreviousMovementMode, uint8 PreviousCustomMode, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void FindFloor(const FVector& CapsuleLocation, FFindFloorResult& OutFloorResult, bool bCanUseCachedLocation, FCharacterMovementComponentAsyncOutput& Output, const FHitResult* DownwardSweepResult = nullptr) const;
	ENGINE_API virtual void ComputeFloorDist(const FVector& CapsuleLocation, float LineDistance, float SweepDistance, FFindFloorResult& OutFloorResult, float SweepRadius, FCharacterMovementComponentAsyncOutput& Output, const FHitResult* DownwardSweepResult = nullptr) const;
	ENGINE_API virtual bool FloorSweepTest(struct FHitResult& OutHit, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const struct FCollisionShape& CollisionShape, const struct FCollisionQueryParams& Params, const struct FCollisionResponseParams& ResponseParam, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsWithinEdgeTolerance(const FVector& CapsuleLocation, const FVector& TestImpactPoint, const float CapsuleRadius) const;
	ENGINE_API virtual bool IsWalkable(const FHitResult& Hit) const;
	ENGINE_API virtual void UpdateCharacterStateAfterMovement(float DeltaSeconds, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual float GetSimulationTimeStep(float RemainingTime, int32 Iterations) const;
	ENGINE_API virtual void CalcVelocity(float DeltaTime, float Friction, bool bFluid, float BrakingDeceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ApplyRequestedMove(float DeltaTime, float MaxAccel, float MaxSpeed, float Friction, float BrakingDeceleration, FVector& OutAcceleration, float& OutRequestedSpeed, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ShouldComputeAccelerationToReachRequestedVelocity(const float RequestedSpeed, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual float GetMinAnalogSpeed(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual float GetMaxBrakingDeceleration(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void ApplyVelocityBraking(float DeltaTime, float Friction, float BrakingDeceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector GetPenetrationAdjustment(FHitResult& HitResult) const;
	ENGINE_API virtual bool ResolvePenetration(const FVector& Adjustment, const FHitResult& Hit, const FQuat& NewRotation, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void HandleImpact(const FHitResult& Impact, FCharacterMovementComponentAsyncOutput& Output, float TimeSlice = 0.0f, const FVector& MoveDelta = FVector::ZeroVector) const;
	ENGINE_API virtual float SlideAlongSurface(const FVector& Delta, float Time, const FVector& InNormal, FHitResult& Hit, bool bHandleImpact, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector HandleSlopeBoosting(const FVector& SlideResult, const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void OnCharacterStuckInGeometry(const FHitResult* Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CanStepUp(const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool StepUp(const FVector& GravDir, const FVector& Delta, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output, FStepDownResult* OutStepDownResult = nullptr) const;
	ENGINE_API virtual bool CanWalkOffLedges(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector GetLedgeMove(const FVector& OldLocation, const FVector& Delta, const FVector& GravDir, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CheckLedgeDirection(const FVector& OldLocation, const FVector& SideStep, const FVector& GravDir, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API FVector GetPawnCapsuleExtent(const EShrinkCapsuleExtent ShrinkMode, const float CustomShrinkAmount, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API FCollisionShape GetPawnCapsuleCollisionShape(const EShrinkCapsuleExtent ShrinkMode, FCharacterMovementComponentAsyncOutput& Output, const float CustomShrinkAmount = 0.0f) const;
	ENGINE_API void TwoWallAdjust(FVector& OutDelta, const FHitResult& Hit, const FVector& OldHitNormal, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API void RevertMove(const FVector& OldLocation, UPrimitiveComponent* OldBase, const FVector& PreviousBaseLocation, const FFindFloorResult& OldFloor, bool bFailMove, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API ETeleportType GetTeleportType(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void HandleWalkingOffLedge(const FVector& PreviousFloorImpactNormal, const FVector& PreviousFloorContactNormal, const FVector& PreviousLocation, float TimeDelta) const;
	ENGINE_API virtual bool ShouldCatchAir(const FFindFloorResult& OldFloor, const FFindFloorResult& NewFloor) const;
	ENGINE_API virtual void StartFalling(int32 Iterations, float remainingTime, float timeTick, const FVector& Delta, const FVector& subLoc, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void AdjustFloorHeight(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API void SetBaseFromFloor(const FFindFloorResult& FloorResult, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ShouldComputePerchResult(const FHitResult& InHit, FCharacterMovementComponentAsyncOutput& Output, bool bCheckRadius = true) const;
	ENGINE_API virtual bool ComputePerchResult(const float TestRadius, const FHitResult& InHit, const float InMaxFloorDist, FFindFloorResult& OutPerchFloorResult, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API float GetPerchRadiusThreshold() const;
	ENGINE_API virtual float GetValidPerchRadius(const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CheckFall(const FFindFloorResult& OldFloor, const FHitResult& Hit, const FVector& Delta, const FVector& OldLocation, float remainingTime, float timeTick, int32 Iterations, bool bMustJump, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector GetFallingLateralAcceleration(float DeltaTime, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector GetAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual float BoostAirControl(float DeltaTime, float TickAirControl, const FVector& FallAcceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ShouldLimitAirControl(float DeltaTime, const FVector& FallAcceleration, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector LimitAirControl(float DeltaTime, const FVector& FallAcceleration, const FHitResult& HitResult, bool bCheckForValidLandingSpot, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API void RestorePreAdditiveRootMotionVelocity(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API void ApplyRootMotionToVelocity(float deltaTime, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FVector NewFallVelocity(const FVector& InitialVelocity, const FVector& Gravity, float DeltaTime, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsValidLandingSpot(const FVector& CapsuleLocation, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void ProcessLanded(const FHitResult& Hit, float remainingTime, int32 Iterations, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void SetPostLandedPhysics(const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual void SetDefaultMovementMode(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ShouldCheckForValidLandingSpot(float DeltaTime, const FVector& Delta, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual FRotator ComputeOrientToMovementRotation(const FRotator& CurrentRotation, float DeltaTime, FRotator& DeltaRotation, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool ShouldRemainVertical(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool CanAttemptJump(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool DoJump(bool bReplayingMoves, FCharacterMovementComponentAsyncOutput& Output) const;

	// UNavMovementComponent (super class) impl
	ENGINE_API bool IsJumpAllowed() const;

	ENGINE_API virtual float GetMaxSpeed(FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsCrouching(const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsFalling(const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsFlying(const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsMovingOnGround(const FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API virtual bool IsExceedingMaxSpeed(float MaxSpeed, const FCharacterMovementComponentAsyncOutput& Output) const;

	// More from UMovementComponent, this is not impl ported from CharacterMovementComponent, but super class, so we can call Super:: in CMC impl.
	// TODO rename or re-organize this.
	ENGINE_API FVector MoveComponent_GetPenetrationAdjustment(FHitResult& HitResult) const;
	ENGINE_API float MoveComponent_SlideAlongSurface(const FVector& Delta, float Time, const FVector& Normal, FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output, bool bHandleImpact = false) const;
	ENGINE_API FVector MoveComponent_ComputeSlideVector(const FVector& Delta, const float Time, const FVector& Normal, const FHitResult& Hit, FCharacterMovementComponentAsyncOutput& Output) const;

	// Root Motion Stuff
	ENGINE_API FVector ConstrainAnimRootMotionVelocity(const FVector& RootMotionVelocity, const FVector& CurrentVelocity, FCharacterMovementComponentAsyncOutput& Output) const;
	ENGINE_API FVector CalcAnimRootMotionVelocity(const FVector& RootMotionDeltaMove, float DeltaSeconds, const FVector& CurrentVelocity) const;

	const FRotator& GetRotationRate(const FCharacterMovementComponentAsyncOutput& Output) const { return Output.bUsingModifiedRotationRate ? Output.ModifiedRotationRate : RotationRate; }
};

class FCharacterMovementComponentAsyncCallback : public Chaos::TSimCallbackObject<FCharacterMovementComponentAsyncInput, FCharacterMovementComponentAsyncOutput>
{
public:
	virtual FName GetFNameForStatId() const override;
private:
	virtual void OnPreSimulate_Internal() override;
};


template <typename FAsyncCallbackInput, typename FAsyncCallbackOutput, typename FAsyncCallback>
void PreSimulateImpl(FAsyncCallback& Callback)
{
	const FAsyncCallbackInput* Input = Callback.GetConsumerInput_Internal();
	if (Input && Input->bInitialized)
	{
		// Update sim state from game thread inputs
		Input->UpdateAsyncStateFromGTInputs_Internal();

		// Ensure that if we reset jump recently we do not process stale inputs enqueued from game thread.
		if (Input->AsyncSimState->CharacterOutput->bClearJumpInput)
		{
			if (Input->GTInputs.bPressedJump == false)
			{
				// Game thread has consumed output clearing jump input, reset our flag so we accept next jump input.
				Input->AsyncSimState->CharacterOutput->bClearJumpInput = false;
			}
			else
			{
				// Game thread has not consumed output clearing jump input yet, this is stale input that we do not want
				Input->AsyncSimState->CharacterOutput->bPressedJump = false;
			}
		}

		Input->Simulate(Callback.GetDeltaTime_Internal(), static_cast<FAsyncCallbackOutput&>(*Input->AsyncSimState));

		//  Copy sim state to callback output that will be pushed to game thread
		FAsyncCallbackOutput& Output = Callback.GetProducerOutputData_Internal();
		Output.Copy(static_cast<FAsyncCallbackOutput&>(*Input->AsyncSimState));
	}
}
