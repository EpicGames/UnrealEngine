// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * ChartCreation
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/Histogram.h"
#include "Scalability.h"
#include "Delegates/IDelegateInstance.h"

enum class EDeviceThermalState : int32;

//////////////////////////////////////////////////////////////////////

// What kind of hitch was detected (if any)
enum class EFrameHitchType : uint8
{
	// We didn't hitch
	NoHitch,

	// We hitched but couldn't isolate which unit caused it
	UnknownUnit,

	// Hitched and it was likely caused by the game thread
	GameThread,

	// Hitched and it was likely caused by the render thread
	RenderThread,

	// Hitched and it was likely caused by the RHI thread
	RHIThread,

	// Hitched and it was likely caused by the GPU
	GPU
};


//////////////////////////////////////////////////////////////////////
// IPerformanceDataConsumer

// This is an interface for any consumer of per-frame performance data
// such as FPS charts, PerfCounters, analytics, etc...

class IPerformanceDataConsumer
{
public:
	struct FFrameData
	{
		// Estimate of how long the last frame was (this is either TrueDeltaSeconds or TrueDeltaSeconds - IdleSeconds, depending on the cvar t.FPSChart.ExcludeIdleTime)
		double DeltaSeconds;

		// Time elapsed since the last time the performance tracking system ran
		double TrueDeltaSeconds;

		// How long did we burn idling until this frame (FApp::GetIdleTime) (e.g., when running faster than a frame rate target on a dedicated server)
		double IdleSeconds;

		// How long did we burn idling beyond what we requested this frame (FApp::GetIdleTimeOvershoot). WaitTime we requested for the frame = IdleSeconds - IdleOvershootSeconds.
		double IdleOvershootSeconds;

		// Duration of each of the major functional units (GPU time is frequently inferred rather than actual)
		double GameThreadTimeSeconds;
		double RenderThreadTimeSeconds;
		double RHIThreadTimeSeconds;
		double GPUTimeSeconds;
		/** Duration of the primary networking portion of the frame (that is, communication between server and clients). Currently happens on the game thread on both client and server. */
		double GameDriverTickFlushTimeSeconds;
		/** Duration of the replay networking portion of the frame. Can happen in a separate thread (not on servers). */
		double DemoDriverTickFlushTimeSeconds;

		/** Dynamic resolution screen percentage. 0.0 if dynamic resolution is disabled. */
		double DynamicResolutionScreenPercentage;

		/** Total time spent flushing async loading on the game thread this frame. */
		double FlushAsyncLoadingTime;
		/** Total number of times FlushAsyncLoading() was called this frame. */
		uint32 FlushAsyncLoadingCount;
		/** The number of sync loads performed in this frame. */
		uint32 SyncLoadCount;

		/** Number of async packages in the loading queue */
		int32 NumAsyncPackages;

		// Should this frame be considered for histogram generation (controlled by t.FPSChart.MaxFrameDeltaSecsBeforeDiscarding)
		bool bBinThisFrame;

		// Was this frame bound in one of the major functional units (only set if bBinThisFrame is true and the frame was longer than FEnginePerformanceTargets::GetTargetFrameTimeThresholdMS) 
		bool bGameThreadBound;
		bool bRenderThreadBound;
		bool bRHIThreadBound;
		bool bGPUBound;

		// Did we hitch?
		EFrameHitchType HitchStatus;

		// If a hitch, how was it bound
		//@TODO: This uses different logic to the three bools above but probably shouldn't (though it also ignores the MaxFrameDeltaSecsBeforeDiscarding)

		FFrameData()
			: DeltaSeconds(0.0)
			, TrueDeltaSeconds(0.0)
			, IdleSeconds(0.0)
			, IdleOvershootSeconds(0.0)
			, GameThreadTimeSeconds(0.0)
			, RenderThreadTimeSeconds(0.0)
			, RHIThreadTimeSeconds(0.0)
			, GPUTimeSeconds(0.0)
			, GameDriverTickFlushTimeSeconds(0.0)
			, DemoDriverTickFlushTimeSeconds(0.0)
			, DynamicResolutionScreenPercentage(0.0)
			, FlushAsyncLoadingTime(0.0)
			, FlushAsyncLoadingCount(0)
			, SyncLoadCount(0)
			, NumAsyncPackages(0)
			, bBinThisFrame(false)
			, bGameThreadBound(false)
			, bRenderThreadBound(false)
			, bRHIThreadBound(false)
			, bGPUBound(false)
			, HitchStatus(EFrameHitchType::NoHitch)
		{
		}
	};

	virtual void StartCharting()=0;
	virtual void ProcessFrame(const FFrameData& FrameData)=0;
	virtual void StopCharting()=0;
	virtual ~IPerformanceDataConsumer() = default;
};

//////////////////////////////////////////////////////////////////////
// FPerformanceTrackingChart

/**
 * Chart for a single portion of gameplay (e.g., gameplay or in-game-shop or settings menu open)
 * WARNING: If you add members here, you MUST also update AccumulateWith() as it accumulates each measure from another chart. 
 */
class FPerformanceTrackingChart : public IPerformanceDataConsumer
{
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPerformanceTrackingChart(const FPerformanceTrackingChart&) = default;
	FPerformanceTrackingChart& operator=(const FPerformanceTrackingChart&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// The mode being tracked by this chart
	FString ChartLabel;

	// Frame time histogram (in seconds)
	FHistogram FrametimeHistogram;

	// Hitch time histogram (in seconds)
	FHistogram HitchTimeHistogram;

	// Hitch time histogram (in seconds)
	FHistogram DynamicResHistogram;

	// Track size of the async loader package queue (I/O)
	FHistogram AsyncPackageQueueHistogram;

	/** Number of frames for each time of <boundtype> **/
	uint32 NumFramesBound_GameThread;
	uint32 NumFramesBound_RenderThread;
	uint32 NumFramesBound_RHIThread;
	uint32 NumFramesBound_GPU;

	/** Time spent bound on each kind of thing (in seconds) */
	double TotalFramesBoundTime_GameThread;
	double TotalFramesBoundTime_RenderThread;
	double TotalFramesBoundTime_RHIThread;
	double TotalFramesBoundTime_GPU;

	/** Total time spent on each thread (in seconds) */
	double TotalFrameTime_GameThread;
	double TotalFrameTime_RenderThread;
	double TotalFrameTime_RHIThread;
	double TotalFrameTime_GPU;

	/** Total time spent flushing async loading (in seconds), and call count */
	double TotalFlushAsyncLoadingTime;
	int32  TotalFlushAsyncLoadingCalls;
	double MaxFlushAsyncLoadingTime;

	/** Total number of sync loads */
	uint32 TotalSyncLoadCount;

	/** Total number of hitches bound by each kind of thing */
	int32 TotalGameThreadBoundHitchCount;
	int32 TotalRenderThreadBoundHitchCount;
	int32 TotalRHIThreadBoundHitchCount;
	int32 TotalGPUBoundHitchCount;

	/** Total number of draw calls made */
	int32 MaxDrawCalls;
	int32 MinDrawCalls;
	int32 TotalDrawCalls;

	/** 
	 * Total number of player ticks (note: it is up to the game to populate this field) 
	 * WARNING: Legacy fields. These should more properly be handled by deriving the class and providing game-specific data as necessary.
	 */
	int32 MaxPlayerTicks;
	int32 MinPlayerTicks;
	int32 TotalPlayerTicks;

	/**
	 * Total number of vehicle ticks (note: it is up to the game to populate this field)
	 * WARNING: Legacy field. These should more properly be handled by deriving the class and providing game-specific data as necessary.
	 */
	int32 MaxVehicleTicks;
	int32 TotalVehicleTicks;

	/** Total number of primitives drawn */
	int32 MaxDrawnPrimitives;
	int32 MinDrawnPrimitives;
	int64 TotalDrawnPrimitives;

	/** Start time of the capture */
	FDateTime CaptureStartTime;

	/** Total accumulated raw (including idle) time spent with the chart registered */
	double AccumulatedChartTime;

	/** Total time (sec) that were disregarded because the frame was too long. This is probably a clock glitch or an app returning from background, suspend, etc, and can really skew the data. */
	double TimeDisregarded;
	/** Number of frames that were disregarded because the frame was too long. This is probably a clock glitch or an app returning from background, suspend, etc, and can really skew the data. */
	int FramesDisregarded;

	UE_DEPRECATED(5.8, "StartTemperatureLevel has been superseded by StartTemperature and/or StartThermalState because it's behavior was poorly defined (on android it was temperature, on ios it was a thermal status enum value).")
	float StartTemperatureLevel;
	UE_DEPRECATED(5.8, "StopTemperatureLevel has been superseded by StopTemperature and/or StopThermalState because it's behavior was poorly defined (on android it was temperature, on ios it was a thermal status enum value).")
	float StopTemperatureLevel;
	float StartTemperature;
	float StopTemperature;
	EDeviceThermalState StartThermalState;
	EDeviceThermalState StopThermalState;

	int StartBatteryLevel;
	int StopBatteryLevel;

	/** WARNING: This value could technically change while data collection is ongoing. Hanlding such changes is not handled by this code. */
	FString DeviceProfileName;

	bool bIsChartingPaused;
	
	// memory stats
	uint32 NumFramesAtCriticalMemoryPressure;
	uint64 MaxPhysicalMemory;
	uint64 MaxVirtualMemory;
	uint64 MinPhysicalMemory;
	uint64 MinVirtualMemory;
	uint64 MinAvailablePhysicalMemory;
	uint64 TotalPhysicalMemoryUsed;
	uint64 TotalVirtualMemoryUsed;

public:
	ENGINE_API FPerformanceTrackingChart();
	ENGINE_API FPerformanceTrackingChart(const FDateTime& InStartTime, const FString& InChartLabel);
	ENGINE_API virtual ~FPerformanceTrackingChart();

	// Discard all accumulated data
	ENGINE_API void Reset(const FDateTime& InStartTime);

	ENGINE_API void AccumulateWith(const FPerformanceTrackingChart& Chart);

	double GetTotalTime() const
	{
		return FrametimeHistogram.GetSumOfAllMeasures();
	}

	int64 GetNumFrames() const
	{
		return FrametimeHistogram.GetNumMeasurements();
	}

	double GetAverageFramerate() const
	{
		return GetNumFrames() / GetTotalTime();
	}

	int64 GetNumHitches() const
	{
		return HitchTimeHistogram.GetNumMeasurements();
	}

	/**
	 * Sum of all recorded hitch lengths (in seconds)
	 * @param bSubtractHitchThreshold Bias towards larger hitches by removing "acceptable" frame time
	 **/
	ENGINE_API double GetTotalHitchFrameTime(bool bSubtractHitchThreshold = true) const;

	double GetPercentHitchTime(bool bSubtractHitchThreshold = true) const
	{
		const double TotalTime = GetTotalTime();
		const double TotalHitchTime = GetTotalHitchFrameTime(bSubtractHitchThreshold);
		return (TotalTime > 0.0) ? ((TotalHitchTime * 100.0) / TotalTime) : 0.0;
	}

	double GetPercentMissedVSync(int32 TargetFPS) const
	{
		const int64 TotalTargetFrames = static_cast<int64>(TargetFPS * GetTotalTime());
		const int64 MissedFrames = FMath::Max<int64>(TotalTargetFrames - GetNumFrames(), 0);
		return ((MissedFrames * 100.0) / (double)TotalTargetFrames);
	}

	double GetAvgHitchesPerMinute() const
	{
		const double TotalTime = GetTotalTime();
		const int32 TotalHitchCount = (int32)GetNumHitches();

		return (TotalTime > 0.0) ? (TotalHitchCount / (TotalTime / 60.0f)) : 0.0;
	}

	double GetAvgHitchFrameLength() const
	{
		return HitchTimeHistogram.GetAverageOfAllMeasures();
	}

	ENGINE_API double GetPercentFramesIOBusy(int32 StartingBin=0) const;

	void ChangeLabel(const FString& NewLabel)
	{
		ChartLabel = NewLabel;
	}

	ENGINE_API void DumpFPSChart(const FString& InMapName);

	// Dumps the FPS chart information to an analytic event param array.
	ENGINE_API void DumpChartToAnalyticsParams(const FString& InMapName, TArray<struct FAnalyticsEventAttribute>& InParamArray, bool bIncludeClientHWInfo, bool bIncludeHistograms = true) const;


	// Dumps the FPS chart information to the log.
	ENGINE_API void DumpChartsToOutputLog(double WallClockElapsed, const TArray<const FPerformanceTrackingChart*>& Charts, const FString& InMapName);

#if ALLOW_DEBUG_FILES
	// Dumps the FPS chart information to HTML.
	ENGINE_API void DumpChartsToHTML(double WallClockElapsed, const TArray<const FPerformanceTrackingChart*>& Charts, const FString& InMapName, const FString& HTMLFilename);

	// Dumps the FPS chart information to the special stats log file.
	ENGINE_API void DumpChartsToLogFile(double WallClockElapsed, const TArray<const FPerformanceTrackingChart*>& Charts, const FString& InMapName, const FString& LogFileName);
#endif

	// IPerformanceDataConsumer interface
	ENGINE_API virtual void StartCharting() override;
	ENGINE_API virtual void ProcessFrame(const FFrameData& FrameData) override;
	ENGINE_API virtual void StopCharting() override;
	// End of IPerformanceDataConsumer interface

	ENGINE_API void PauseCharting();
	ENGINE_API void ResumeCharting();
};

//////////////////////////////////////////////////////////////////////
// FFineGrainedPerformanceTracker

#if ALLOW_DEBUG_FILES

// Fine-grained tracking (records the frame time of each frame rather than just a histogram)
class FFineGrainedPerformanceTracker : public IPerformanceDataConsumer
{
public:
	ENGINE_API FFineGrainedPerformanceTracker(const FDateTime& InStartTime);

	/** Resets the fine-grained tracker, allocating enough memory to hold NumFrames frames (it can track more, but this avoids extra allocations when the length is short enough) */
	ENGINE_API void Presize(int32 NumFrames);

	/**
	 * Dumps the timings for each frame to a .csv
	 */
	ENGINE_API void DumpFrameTimesToStatsLog(const FString& FrameTimeFilename);

	/**
	 * Finds a percentile value in an array.
	 *
	 * @param Array array of values to look for (needs to be writable, will be modified)
	 * @param Percentile number between 0-99
	 * @return percentile value or -1 if no samples
	 */
	static ENGINE_API float GetPercentileValue(TArray<float>& Samples, int32 Percentile);

	// IPerformanceDataConsumer interface
	ENGINE_API virtual void StartCharting() override;
	ENGINE_API virtual void ProcessFrame(const FFrameData& FrameData) override;
	ENGINE_API virtual void StopCharting() override;
	// End of IPerformanceDataConsumer interface

public:
	/** Arrays of render/game/GPU and total frame times. Captured and written out if FPS charting is enabled and bFPSChartRecordPerFrameTimes is true */
	TArray<float> RenderThreadFrameTimes;
	TArray<float> GameThreadFrameTimes;
	TArray<float> GPUFrameTimes;
	TArray<float> FrameTimes;
	TArray<int32> ActiveModes;
	TArray<float> DynamicResolutionScreenPercentages;

	/** Start time of the capture */
	const FDateTime CaptureStartTime;

	/** Current context (user-specified integer stored per frame, could be used to signal game mode changes without doing discrete captures */
	int32 CurrentModeContext;
};

#endif

//////////////////////////////////////////////////////////////////////
// FPerformanceTrackingSystem

// Overall state of the built-in performance tracking
struct FPerformanceTrackingSystem
{
public:
	ENGINE_API FPerformanceTrackingSystem();

	ENGINE_API IPerformanceDataConsumer::FFrameData AnalyzeFrame(float DeltaSeconds);
	ENGINE_API void StartCharting();
	ENGINE_API void StopCharting();

	/**
	 * This will create the file name for the file we are saving out.
	 *
	 * @param ChartType	the type of chart we are making 
	 * @param InMapName	the name of the map
	 * @param FileExtension	the filename extension to append
	 **/
	static ENGINE_API FString CreateFileNameForChart(const FString& ChartType, const FString& InMapName, const FString& FileExtension);

	/** This will create the folder name for the output directory for FPS charts (and actually create the directory). */
	static ENGINE_API FString CreateOutputDirectory(const FDateTime& CaptureStartTime);

	// Should we subtract off idle time spent waiting (due to running above target framerate) before thresholding into bins?
	// (controlled by t.FPSChart.ExcludeIdleTime)
	static ENGINE_API bool ShouldExcludeIdleTimeFromCharts();

private:
	/** Start time of current FPS chart */
	double FPSChartStartTime;

	/** Stop time of current FPS chart */
	double FPSChartStopTime;

	/** We can't trust delta seconds if frame time clamping is enabled or if we're benchmarking so we simply calculate it ourselves. */
	double LastTimeChartCreationTicked;

	/** Keep track of our previous frame's statistics */
	float LastDeltaSeconds;

	/** Keep track of the last time we saw a hitch (used to suppress knock on hitches for a short period) */
	double LastHitchTime;
};

//////////////////////////////////////////////////////////////////////

// Prints the FPS chart summary to an endpoint
struct FDumpFPSChartToEndpoint
{
protected:
	const FPerformanceTrackingChart& Chart;

public:
	/**
	* Dumps a chart, allowing subclasses to format the data in their own way via various protected virtuals to be overridden
	*/
	ENGINE_API void DumpChart(double InWallClockTimeFromStartOfCharting, FString InMapName, FString InDeviceProfileName = FString("Unknown"));

	FDumpFPSChartToEndpoint(const FPerformanceTrackingChart& InChart)
		: Chart(InChart)
	{
	}

	virtual ~FDumpFPSChartToEndpoint() = default;

protected:
	double WallClockTimeFromStartOfCharting; // This can be much larger than TotalTime if the chart was paused or long frames were omitted
	FString MapName;
	FString DeviceProfileName;

	float AvgGPUFrameTime;
	float AvgRenderThreadFrameTime;
	float AvgGameThreadFrameTime;

	double TotalFlushAsyncLoadingTimeInMS;
	int32  TotalFlushAsyncLoadingCalls;
	double MaxFlushAsyncLoadingTimeInMS;
	double AvgFlushAsyncLoadingTimeInMS;

	int32 TotalSyncLoadCount;

	float BoundGameThreadPct;
	float BoundRenderThreadPct;
	float BoundGPUPct;

	Scalability::FQualityLevels ScalabilityQuality;
	FIntPoint GameResolution;
	FString WindowMode;

	FString OSMajor;
	FString OSMinor;

	FString CPUVendor;
	FString CPUBrand;

	// The primary GPU for the desktop (may not be the one we ended up using, e.g., in an optimus laptop)
	FString DesktopGPUBrand;

	// The actual GPU adapter we initialized
	FString ActualGPUBrand;

protected:
	virtual void PrintToEndpoint(const FString& Text) = 0;

	ENGINE_API virtual void FillOutMemberStats();
	ENGINE_API virtual void HandleHitchBucket(const FHistogram& HitchHistogram, int32 BucketIndex);
	ENGINE_API virtual void HandleHitchSummary(int32 TotalHitchCount, double TotalTimeSpentInHitchBuckets);
	ENGINE_API virtual void HandleFPSThreshold(int32 TargetFPS, float PctMissedFrames);
	ENGINE_API virtual void HandleDynamicResThreshold(int32 TargetScreenPercentage, float PctTimeAbove);
	ENGINE_API virtual void HandleBasicStats();
};
