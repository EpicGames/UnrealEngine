// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothCollisionPrim.h"
#include "Components/SkeletalMeshComponent.h"

/** Helper struct used to store info about a cloth collision source */
struct FClothCollisionSource
{
	FClothCollisionSource(USkeletalMeshComponent* InSourceComponent, UPhysicsAsset* InSourcePhysicsAsset, const FOnBoneTransformsFinalizedMultiCast::FDelegate& InOnBoneTransformsFinalizedDelegate);
	ENGINE_API ~FClothCollisionSource();

	/** Extract collisions for cloth from the source component given a target component we want to apply the data to. */
	void ExtractCollisions(const USkeletalMeshComponent* DestClothComponent, FClothCollisionData& OutCollisions);

	/** Component that collision data will be copied from */
	TWeakObjectPtr<USkeletalMeshComponent> SourceComponent;

	/** Physics asset to use to generate collision against the source component */
	TWeakObjectPtr<UPhysicsAsset> SourcePhysicsAsset;

	/** Callback used to remove the cloth transform updates delegate */
	FDelegateHandle OnBoneTransformsFinalizedHandle;

	/** Cached skeletal mesh used to invalidate the cache if the skeletal mesh has changed */
	TWeakObjectPtr<USkeletalMesh> CachedSkeletalMesh;

	/** Cached spheres from physics asset */
	TArray<FClothCollisionPrim_Sphere> CachedSpheres;

	/** Cached sphere connections from physics asset */
	TArray<FClothCollisionPrim_SphereConnection> CachedSphereConnections;

	/** Flag whether the cache is valid */
	bool bCached;
};
