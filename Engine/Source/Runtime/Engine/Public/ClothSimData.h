// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** data for updating cloth section from the results of clothing simulation */
//struct FClothSimulData
//{
//	TArray<FVector4> Positions;
//	TArray<FVector4> Normals;
//};
