// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CollectionManagerScriptingTypes.generated.h"

UENUM(DisplayName="Collection Share Type", meta=(ScriptName="CollectionShareType"))
enum class ECollectionScriptingShareType : uint8
{
	/** This collection is only visible to you and is not in source control. */
	Local,
	/** This collection is only visible to you. */
	Private,
	/** This collection is visible to everyone. */
	Shared,
};
