// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Collision.h: Common collision code.
=============================================================================*/

#pragma once

#include "Math/Vector.h"
#include "Stats/Stats.h"
#include "EngineDefines.h"

struct FHitResult;

/**
 * Collision stats
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("SceneQueryTotal"), STAT_Collision_SceneQueryTotal, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RaycastAny"),STAT_Collision_RaycastAny,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RaycastSingle"),STAT_Collision_RaycastSingle,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RaycastMultiple"),STAT_Collision_RaycastMultiple,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomSweepAny"),STAT_Collision_GeomSweepAny,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomSweepSingle"),STAT_Collision_GeomSweepSingle,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomSweepMultiple"),STAT_Collision_GeomSweepMultiple,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomOverlapMultiple"),STAT_Collision_GeomOverlapMultiple,STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomOverlapBlocking"), STAT_Collision_GeomOverlapBlocking, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomOverlapAny"), STAT_Collision_GeomOverlapAny, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BodyInstanceOverlapMulti"), STAT_Collision_FBodyInstance_OverlapMulti, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BodyInstanceOverlapTest"), STAT_Collision_FBodyInstance_OverlapTest, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BodyInstanceLineTrace"), STAT_Collision_FBodyInstance_LineTrace, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PreFilter"), STAT_Collision_PreFilter, STATGROUP_CollisionVerbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SpherecastMultipleInternal"), STAT_Collision_SpherecastMultiple_Internal, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GeomSweepMultipleInternal"), STAT_Collision_GeomSweepMultiple_Internal, STATGROUP_Collision, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RaycastMultipleInternal"), STAT_Collision_RaycastMultiple_Internal, STATGROUP_Collision, );

/** Enable collision analyzer support */
#if (1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITOR && WITH_UNREAL_DEVELOPER_TOOLS)
	#define ENABLE_COLLISION_ANALYZER 1
#else
	#define ENABLE_COLLISION_ANALYZER 0
#endif

//
//	FSeparatingAxisPointCheck - Checks for intersection between an AABB and a convex polygon.
//

class FSeparatingAxisPointCheck
{
public:
	/** The normal of the separating axis that the bounding box is penetrating the least */
	FVector HitNormal;

	/** The amount that the bounding box is penetrating the axis defined by HitNormal */
	float BestDist;

	/** Whether the bounding box intersects with the polygon */
	bool bHit;

	/**
	 *	Creates an object representing the intersection between an axis-aligned bounding box and a convex polygon.
	 *
	 *	@param	InPolyVertices			Array of points forming a convex polygon.
	 *	@param	InBoxCenter				The center of the axis-aligned bounding box being checked.
	 *	@param	InBoxExtent				The extents of the axis-aligned bounding box being checked.
	 *	@param	bInCalcLeastPenetration	Whether the axis and amount of penetration of the bounding box into the polygon should be calculated.
	 */
	FSeparatingAxisPointCheck(
		const TArray<FVector>& InPolyVertices,
		const FVector& InBoxCenter,
		const FVector& InBoxExtent,
		bool bInCalcLeastPenetration = true
		)
		: HitNormal(FVector::ZeroVector),
		  BestDist(TNumericLimits<float>::Max()),
		  PolyVertices(InPolyVertices),
		  BoxCenter(InBoxCenter),
		  BoxExtent(InBoxExtent),
		  bCalcLeastPenetration(bInCalcLeastPenetration)
	{
		// Optimization: if the poly is a triangle, use a more optimized code path
		bHit = (PolyVertices.Num() == 3) ? FindSeparatingAxisTriangle() : FindSeparatingAxisGeneric();
	}

private:

	/**
	 *	Given a separating axis, and a minimum and maximum point projected onto it, determines whether the bounding box
	 *	encroaches on the polygon along that axis, according to the separating axis theorem.
	 *	Optionally sets the HitNormal and BestDist members according to the least penetration of the bounding box into the polygon.
	 *
	 *	@param	Axis				The separating axis.
	 *	@param	ProjectedPolyMin	The minimum polygon point, projected onto the separating axis.
	 *	@param	ProjectedPolyMax	The maximum polygon point, projected onto the separating axis.
	 */
	ENGINE_API bool TestSeparatingAxisCommon(const FVector& Axis, float ProjectedPolyMin, float ProjectedPolyMax);

	/**
	 *	Determines whether the bounding box encroaches on the triangle along the given axis.
	 *	
	 *	@param	Axis				The separating axis.
	 *
	 *	@return	True if the bounding box encroaches on the triangle along the given axis.
	 */
	ENGINE_API bool TestSeparatingAxisTriangle(const FVector& Axis);

	/**
	 *	Determines whether the bounding box encroaches on the convex polygon along the given axis.
	 *
	 *	@param	Axis				The separating axis.
	 *
	 *	@return	True if the bounding box encroaches on the triangle along the given axis.
	 */
	ENGINE_API bool TestSeparatingAxisGeneric(const FVector& Axis);

	/**
	 *	Determines whether the bounding box encroaches on the triangle, checking all relevant axes.
	 *
	 *	@return	True if the bounding box encroaches on the triangle.
	 */
	ENGINE_API bool FindSeparatingAxisTriangle();

	/**
	 *	Determines whether the bounding box encroaches on the convex polygon, checking all relevant axes.
	 *
	 *	@return	True if the bounding box encroaches on the convex polygon.
	 */
	ENGINE_API bool FindSeparatingAxisGeneric();

	/** Array of vertices defining the convex polygon being checked. */
	const TArray<FVector>& PolyVertices;

	/** Center of the axis-aligned bounding box being checked. */
	FVector BoxCenter;

	/** Extents of the axis-aligned bounding box being checked. */
	FVector BoxExtent;

	/** Flag specifying whether the least penetration should be calculated. */
	bool bCalcLeastPenetration;
};

/**
 *	Line Check With Triangle
 *	Algorithm based on "Fast, Minimum Storage Ray/Triangle Intersection"
 *	Returns true if the line segment does hit the triangle
 */
ENGINE_API bool LineCheckWithTriangle(FHitResult& Result,const FVector& V1,const FVector& V2,const FVector& V3,const FVector& Start,const FVector& End,const FVector& Direction);

