// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

struct FHitResult;

/** Draw line trace results */
ENGINE_API void DrawLineTraces(const UWorld* InWorld, const FVector& Start, const FVector& End, const TArray<FHitResult>& Hits, float Lifetime);
/** Draw sphere sweep results */
ENGINE_API void DrawSphereSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const float Radius, const TArray<FHitResult>& Hits, float Lifetime);
/** Draw box sweep results */
ENGINE_API void DrawBoxSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, const FVector& Extent, const FQuat& Rot, const TArray<FHitResult>& Hits, float Lifetime);
/** Draw capsule sweep results */
ENGINE_API void DrawCapsuleSweeps(const UWorld* InWorld, const FVector& Start, const FVector& End, float HalfHeight, float Radius, const FQuat& Rotation, const TArray<FHitResult>& Hits, float Lifetime);

/** Draw box overlap results */
ENGINE_API void DrawBoxOverlap(const UWorld* InWorld, const FVector& Pos, const FVector& Extent, const FQuat& Rot, TArray<struct FOverlapResult>& Overlaps, float Lifetime);
/** Draw sphere overlap results */
ENGINE_API void DrawSphereOverlap(const UWorld* InWorld, const FVector& Pos, const float Radius, TArray<struct FOverlapResult>& Overlaps, float Lifetime);
/** Draw capsule overlap results */
ENGINE_API void DrawCapsuleOverlap(const UWorld* InWorld, const FVector& Pos, const float HalfHeight, const float Radius, const FQuat& Rot, TArray<struct FOverlapResult>& Overlaps, float Lifetime);
