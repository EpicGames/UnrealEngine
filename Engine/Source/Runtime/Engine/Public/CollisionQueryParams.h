// Copyright Epic Games, Inc. All Rights Reserved.

// Structs used for passing parameters to scene query functions

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Chaos/ChaosEngineInterface.h"
#include "UObject/RemoteObjectTransfer.h"

/** Macro to convert ECollisionChannels to bit flag **/
#define ECC_TO_BITFIELD(x)	(1LLU<<(x))
/** Macro to convert from CollisionResponseContainer to bit flag **/
#define CRC_TO_BITFIELD(x)	(1LLU<<(x))

class AActor;
class UPrimitiveComponent;

enum class EQueryMobilityType
{
	Any,
	Static,	//Any shape that is considered static by physx (static mobility)
	Dynamic	//Any shape that is considered dynamic by physx (movable/stationary mobility)
};

/** Set to 1 so the compiler can find all QueryParams that don't take in a stat id.
  * Note this will not include any queries taking a default SceneQuery param
  */
#define FIND_UNKNOWN_SCENE_QUERIES 0

#if ENABLE_STATNAMEDEVENTS
	#define SCENE_QUERY_STAT_ONLY(QueryName) TStatId(ANSI_TO_PROFILING(#QueryName))
#else
	#define SCENE_QUERY_STAT_ONLY(QueryName) QUICK_USE_CYCLE_STAT(QueryName, STATGROUP_CollisionTags)
#endif

#define SCENE_QUERY_STAT_NAME_ONLY(QueryName) [](){ static FName StaticName(#QueryName); return StaticName;}()
#define SCENE_QUERY_STAT(QueryName) SCENE_QUERY_STAT_NAME_ONLY(QueryName), SCENE_QUERY_STAT_ONLY(QueryName)

/** Structure that defines parameters passed into collision function */
struct FCollisionQueryParams
{
	/** Tag used to provide extra information or filtering for debugging of the trace (e.g. Collision Analyzer) */
	FName TraceTag;

	/** Tag used to indicate an owner for this trace */
	FName OwnerTag;

	/** Whether we should trace against complex collision */
	bool bTraceComplex;

	/** Whether we want to find out initial overlap or not. If true, it will return if this was initial overlap. */
	bool bFindInitialOverlaps;

	/** Whether we want to return the triangle face index for complex static mesh traces */
	bool bReturnFaceIndex;

	/** Whether we want to include the physical material in the results. */
	bool bReturnPhysicalMaterial;

	/** Whether to ignore blocking results. */
	bool bIgnoreBlocks;

	/** Whether to ignore touch/overlap results. */
	bool bIgnoreTouches;

	/** Whether to skip narrow phase checks (only for overlaps). */
	bool bSkipNarrowPhase;

	/** Whether to ignore traces to the cluster union and trace against its children instead. */
	bool bTraceIntoSubComponents;

	/** If bTraceIntoSubComponents is true, whether to replace the hit of the cluster union with its children instead. */
	bool bReplaceHitWithSubComponents;

	/** Filters query by mobility types (static vs stationary/movable)*/
	EQueryMobilityType MobilityType;

	/** TArray typedef of components to ignore. */
	typedef TArray<uint32, TInlineAllocator<8>> IgnoreComponentsArrayType;

	/** TArray typedef of actors to ignore. */
	typedef TArray<uint32, TInlineAllocator<4>> IgnoreSourceObjectsArrayType;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	typedef TArray<FRemoteObjectReference, TInlineAllocator<4>> IgnoreSourceObjectReferencesArrayType;
	typedef TArray<FRemoteObjectReference, TInlineAllocator<8>> IgnoreComponentReferencesArrayType;
#endif

	typedef IgnoreSourceObjectsArrayType IgnoreActorsArrayType;

	/** Extra filtering done on the query. See declaration for filtering logic */
	FMaskFilter IgnoreMask;

	/** StatId used for profiling individual expensive scene queries */
	TStatId StatId;

	static inline TStatId GetUnknownStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(UnknownSceneQuery, STATGROUP_Collision);
	}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	bool bDebugQuery;
#endif

private:

	/** Tracks whether the IgnoreComponents list is verified unique. */
	mutable bool bComponentListUnique;

	/** Set of components to ignore during the trace */
	mutable IgnoreComponentsArrayType IgnoreComponents;

	/** Set of source objects to ignore during the trace. In actor workflows, these are actors. */
	IgnoreSourceObjectsArrayType IgnoreSourceObjects;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** 
	 * When running with auto-rtfm transactions, these reference arrays are the source of truth for all ignore sources.
	 * The pre-existing ignore lists will only contain the local objects. 
	 * It should always be possible to reconstruct the ignore object lists from the ignore references lists.
	 */
	mutable IgnoreComponentReferencesArrayType IgnoreComponentReferences;
	IgnoreSourceObjectReferencesArrayType IgnoreSourceObjectReferences;
#endif

	void Internal_AddIgnoredSourceObject(const FWeakObjectPtr& WeakPtr);

	ENGINE_API void Internal_AddIgnoredComponent(const UPrimitiveComponent* InIgnoreComponent);

public:

	/** Returns set of unique components to ignore during the trace. Elements are guaranteed to be unique (they are made so internally if they are not already). */
	ENGINE_API const IgnoreComponentsArrayType& GetIgnoredComponents() const;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	const IgnoreComponentReferencesArrayType& GetIgnoredComponentReferences() const
	{
		return IgnoreComponentReferences;
	}
#endif

	UE_DEPRECATED(5.5, "Use GetIgnoredSourceObjects instead.")
	/** Returns set of source objects (including actors) to ignore during the trace. Note that elements are NOT guaranteed to be unique. This is less important for actors since it's less likely that duplicates are added.*/
	const IgnoreActorsArrayType& GetIgnoredActors() const
	{
		return IgnoreSourceObjects;
	}

	/**
	 * Returns the set of source objects (such as actors) to ignore during the trace. Note that elements are
	 * NOT guaranteed to be unique. This is less important for source objects than components since it's
	 * less likely that duplicates are added.
	 */
	const IgnoreActorsArrayType& GetIgnoredSourceObjects() const
	{
		return IgnoreSourceObjects;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	const IgnoreSourceObjectReferencesArrayType& GetIgnoredSourceObjectReferences() const
	{
		return IgnoreSourceObjectReferences;
	}
#endif

	/** Clears the set of components to ignore during the trace. */
	void ClearIgnoredComponents()
	{
		IgnoreComponents.Reset();
		bComponentListUnique = true;

#if UE_WITH_REMOTE_OBJECT_HANDLE
		IgnoreComponentReferences.Reset();
#endif
	}

	UE_DEPRECATED(5.5, "Use ClearIgnoredSourceObjects instead.")
	/** Clears the set of actors to ignore during the trace. */
	void ClearIgnoredActors()
	{
		IgnoreSourceObjects.Reset();

#if UE_WITH_REMOTE_OBJECT_HANDLE
		IgnoreSourceObjectReferences.Reset();
#endif
	}

	/** Clears the set of source objects (such as actors) to ignore during the trace. */
	void ClearIgnoredSourceObjects()
	{
		IgnoreSourceObjects.Reset();

#if UE_WITH_REMOTE_OBJECT_HANDLE
		IgnoreSourceObjectReferences.Reset();
#endif
	}

	/**
	 * Set the number of ignored components in the list. Uniqueness is not changed, it operates on the current state (unique or not).
	 * Useful for temporarily adding some, then restoring to a previous size. NewNum must be <= number of current components for there to be any effect.
	 */
	ENGINE_API void SetNumIgnoredComponents(int32 NewNum);

	// Constructors
#if !FIND_UNKNOWN_SCENE_QUERIES

	FCollisionQueryParams()
	{
		bTraceComplex = false;
		MobilityType = EQueryMobilityType::Any;
		TraceTag = NAME_None;
		bFindInitialOverlaps = true;
		bReturnFaceIndex = false;
		bReturnPhysicalMaterial = false;
		bComponentListUnique = true;
		IgnoreMask = 0;
		bIgnoreBlocks = false;
		bIgnoreTouches = false;
		bSkipNarrowPhase = false;
		StatId = GetUnknownStatId();
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		bDebugQuery = false;
#endif
		bTraceIntoSubComponents = true;
		bReplaceHitWithSubComponents = true;
	}

	FCollisionQueryParams(FName InTraceTag, bool bInTraceComplex=false, const AActor* InIgnoreActor=NULL)
	: FCollisionQueryParams(InTraceTag, GetUnknownStatId(), bInTraceComplex, InIgnoreActor)
	{

	}
#endif

	ENGINE_API FCollisionQueryParams(FName InTraceTag, const TStatId& InStatId, bool bInTraceComplex = false, const AActor* InIgnoreActor = NULL);

	// Utils

	/** Add an actor for this trace to ignore. Equivalent to calling AddIgnoredSourceObject. */
	ENGINE_API void AddIgnoredActor(const AActor* InIgnoreActor);

	/** Add an actor by ID for this trace to ignore. Equivalent to calling AddIgnoredSourceObject. */
	ENGINE_API void AddIgnoredActor(const uint32 InIgnoreActorID);

	/** Add a source object for this trace to ignore */
	ENGINE_API void AddIgnoredSourceObject(const UObject* InIgnoreActor);
	ENGINE_API void AddIgnoredSourceObject(const TWeakObjectPtr<const UObject>& InIgnoreObject);
	ENGINE_API void AddIgnoredSourceObject(const FRemoteObjectReference& ObjectRef);

	/** Add a source object for this trace to ignore */
	ENGINE_API void AddIgnoredSourceObject(const uint32 InIgnoreActorID);

	/** Add a collection of actors for this trace to ignore. Equivalent to calling AddIgnoredSourceObjects */
	ENGINE_API void AddIgnoredActors(const TArray<AActor*>& InIgnoreActors);
	ENGINE_API void AddIgnoredActors(const TArray<const AActor*>& InIgnoreActors);
	ENGINE_API void AddIgnoredActors(const TArray<TWeakObjectPtr<const AActor> >& InIgnoreActors);

	/** Add a collection of source objects for this trace to ignore */
	ENGINE_API void AddIgnoredSourceObjects(const TArray<UObject*>& InIgnoreObjects);
	ENGINE_API void AddIgnoredSourceObjects(const TArray<const UObject*>& InIgnoreObjects);
	ENGINE_API void AddIgnoredSourceObjects(const TArray<TWeakObjectPtr<const UObject> >& InIgnoreObjects);

	/** Add a component for this trace to ignore */
	ENGINE_API void AddIgnoredComponent(const UPrimitiveComponent* InIgnoreComponent);
	ENGINE_API void AddIgnoredComponent(const TWeakObjectPtr<UPrimitiveComponent>& InIgnoreComponent);
	ENGINE_API void AddIgnoredComponent(const FRemoteObjectReference& InComponentRef);

	/** Add a collection of components for this trace to ignore */
	ENGINE_API void AddIgnoredComponents(const TArray<UPrimitiveComponent*>& InIgnoreComponents);
	
	/** Variant that uses an array of TWeakObjectPtrs */
	ENGINE_API void AddIgnoredComponents(const TArray<TWeakObjectPtr<UPrimitiveComponent>>& InIgnoreComponents);

	/**
	 * Special variant that hints that we are likely adding a duplicate of the root component or first ignored component.
	 * Helps avoid invalidating the potential uniquess of the IgnoreComponents array.
	 */
	ENGINE_API void AddIgnoredComponent_LikelyDuplicatedRoot(const UPrimitiveComponent* InIgnoreComponent);

	/** Add a component (by id) for this trace to ignore. Internal method meant for copying data in/out*/
	UE_INTERNAL ENGINE_API void Internal_AddIgnoredComponent(const uint32 ComponentID);

	FString ToString() const
	{
		return FString::Printf(TEXT("[%s:%s] TraceComplex(%d)"), *OwnerTag.ToString(), *TraceTag.ToString(), bTraceComplex );
	}

	/** static variable for default data to be used without reconstructing everytime **/
	static ENGINE_API FCollisionQueryParams DefaultQueryParam;
};

/** Structure when performing a collision query using a component's geometry */
struct FComponentQueryParams : public FCollisionQueryParams
{
#if !FIND_UNKNOWN_SCENE_QUERIES
	FComponentQueryParams()
	: FCollisionQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(),false)
	{
	}

	FComponentQueryParams(FName InTraceTag, const AActor* InIgnoreActor=NULL, const FCollisionEnabledMask InShapeCollisionMask = 0)
	: FComponentQueryParams(InTraceTag, GetUnknownStatId(), InIgnoreActor, InShapeCollisionMask)
	{
	}
#endif

	FComponentQueryParams(FName InTraceTag, const TStatId& InStatId, const AActor* InIgnoreActor = NULL, const FCollisionEnabledMask InShapeCollisionMask = 0)
		: FCollisionQueryParams(InTraceTag, InStatId, false, InIgnoreActor), ShapeCollisionMask(InShapeCollisionMask)
	{
	}

	/** Only use query shapes which remain unmasked by this collision mask (if mask is nonzero) **/
	FCollisionEnabledMask ShapeCollisionMask;

	/** static variable for default data to be used without reconstructing everytime **/
	static ENGINE_API FComponentQueryParams DefaultComponentQueryParams;
};

/** Structure that defines response container for the query. Advanced option. */
struct FCollisionResponseParams
{
	/** 
	 *	Collision Response container for trace filtering. If you'd like to ignore certain channel for this trace, use this struct.
	 *	By default, every channel will be blocked
	 */
	struct FCollisionResponseContainer CollisionResponse;

	FCollisionResponseParams(ECollisionResponse DefaultResponse = ECR_Block)
	{
		CollisionResponse.SetAllChannels(DefaultResponse);
	}

	FCollisionResponseParams(const FCollisionResponseContainer& ResponseContainer)
	{
		CollisionResponse = ResponseContainer;
	}
	/** static variable for default data to be used without reconstructing everytime **/
	static ENGINE_API FCollisionResponseParams DefaultResponseParam;
};

// If ECollisionChannel entry has metadata of "TraceType = 1", they will be excluded by Collision Profile
// Any custom channel with bTraceType=true also will be excluded
// By default everything is object type
struct FCollisionQueryFlag
{
private:
	static constexpr uint32 MaxCollisionChannelCount = ECC_MAX - 1;
	int64 AllObjectQueryFlag;
	int64 AllStaticObjectQueryFlag;

	static int64 ToBitMask(ECollisionChannel QueryChannel)
	{
		return ECC_TO_BITFIELD(QueryChannel);
	}

	FCollisionQueryFlag()
	{
		AllObjectQueryFlag = std::numeric_limits<uint64>::max();
		AllStaticObjectQueryFlag = ToBitMask(ECC_WorldStatic);
	}

public:
	static ENGINE_API FCollisionQueryFlag& Get();

	UE_DEPRECATED(5.8, "Use GetAllObjectsQueryFlag64")
	int32 GetAllObjectsQueryFlag()
	{
		return (int32)GetAllObjectsQueryFlag64();
	}

	int64 GetAllObjectsQueryFlag64() const
	{
		// this doesn't really verify trace queries to come this way
		return AllObjectQueryFlag;
	}
	
	UE_DEPRECATED(5.8, "Use GetAllStaticObjectsQueryFlag64")
	int32 GetAllStaticObjectsQueryFlag()
	{
		return (int32)GetAllStaticObjectsQueryFlag64();
	}

	int64 GetAllStaticObjectsQueryFlag64() const
	{
		return AllStaticObjectQueryFlag;
	}
	
	UE_DEPRECATED(5.8, "Use GetAllDynamicObjectsQueryFlag64")
	int32 GetAllDynamicObjectsQueryFlag()
	{
		return (int32)GetAllDynamicObjectsQueryFlag64();
	}

	int64 GetAllDynamicObjectsQueryFlag64() const
	{
		return (AllObjectQueryFlag & ~AllStaticObjectQueryFlag);
	}

	void AddToAllObjectsQueryFlag(ECollisionChannel NewChannel)
	{
		if (ensure ((int32)NewChannel < MaxCollisionChannelCount))
		{
			SetAllObjectsQueryFlag (AllObjectQueryFlag |= ToBitMask(NewChannel));
		}
	}

	void AddToAllStaticObjectsQueryFlag(ECollisionChannel NewChannel)
	{
		if (ensure ((int32)NewChannel < MaxCollisionChannelCount))
		{
			SetAllStaticObjectsQueryFlag (AllStaticObjectQueryFlag |= ToBitMask(NewChannel));
		}
	}

	void RemoveFromAllObjectsQueryFlag(ECollisionChannel NewChannel)
	{
		if (ensure ((int32)NewChannel < MaxCollisionChannelCount))
		{
			SetAllObjectsQueryFlag (AllObjectQueryFlag &= ~ToBitMask(NewChannel));
		}
	}

	void RemoveFromAllStaticObjectsQueryFlag(ECollisionChannel NewChannel)
	{
		if (ensure ((int32)NewChannel < MaxCollisionChannelCount))
		{
			SetAllStaticObjectsQueryFlag (AllStaticObjectQueryFlag &= ~ToBitMask(NewChannel));
		}
	}

	void SetAllObjectsQueryFlag(int64 NewQueryFlag)
	{
		// if all object query has changed, make sure to apply to static object query
		AllObjectQueryFlag = NewQueryFlag;
		AllStaticObjectQueryFlag = AllObjectQueryFlag & AllStaticObjectQueryFlag;
	}

	void SetAllStaticObjectsQueryFlag(int64 NewQueryFlag)
	{
		AllStaticObjectQueryFlag = NewQueryFlag;
	}

	void SetAllDynamicObjectsQueryFlag(int64 NewQueryFlag)
	{
		AllStaticObjectQueryFlag = AllObjectQueryFlag & ~NewQueryFlag;
	}
};

/** Structure that contains list of object types the query is intersted in.  */
struct FCollisionObjectQueryParams
{
private:
	int64 ObjectTypesToQuery64 = 0;

	void UpdateDeprecatedObjectTypesToQuery()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ObjectTypesToQuery = (int32)ObjectTypesToQuery64;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static int64 ToBitMask(ECollisionChannel QueryChannel)
	{
		return ECC_TO_BITFIELD(QueryChannel);
	}
public:

	enum InitType
	{
		AllObjects,
		AllStaticObjects,
		AllDynamicObjects
	};

	int64 GetObjectTypesToQuery() const
	{
		return ObjectTypesToQuery64;
	}

	void SetObjectTypesToQuery(int64 InObjectTypesToQuery)
	{
		ObjectTypesToQuery64 = InObjectTypesToQuery;
		UpdateDeprecatedObjectTypesToQuery();
	}

	/** Set of object type queries that it is interested in **/
	UE_DEPRECATED(5.8, "Use Get/Set ObjectTypesToQuery instead.")
	int32 ObjectTypesToQuery = 0;

	/** Extra filtering done during object query. See declaration for filtering logic */
	FMaskFilter IgnoreMask;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCollisionObjectQueryParams(const FCollisionObjectQueryParams&) = default;
	FCollisionObjectQueryParams(FCollisionObjectQueryParams&&) = default;
	FCollisionObjectQueryParams& operator=(const FCollisionObjectQueryParams&) = default;
	FCollisionObjectQueryParams& operator=(FCollisionObjectQueryParams&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FCollisionObjectQueryParams()
		: ObjectTypesToQuery64(0)
		, IgnoreMask(0)
	{
	}

	FCollisionObjectQueryParams(ECollisionChannel QueryChannel)
	{
		ObjectTypesToQuery64 = ToBitMask(QueryChannel);
		IgnoreMask = 0;
		UpdateDeprecatedObjectTypesToQuery();
	}

	FCollisionObjectQueryParams(const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes)
	{
		ObjectTypesToQuery64 = 0;

		for (auto Iter = ObjectTypes.CreateConstIterator(); Iter; ++Iter)
		{
			AddObjectTypesToQuery(UEngineTypes::ConvertToCollisionChannel((*Iter).GetValue()));
		}

		IgnoreMask = 0;
	}

	FCollisionObjectQueryParams(enum FCollisionObjectQueryParams::InitType QueryType)
	{
		switch (QueryType)
		{
		case AllObjects:
			ObjectTypesToQuery64 = FCollisionQueryFlag::Get().GetAllObjectsQueryFlag64();
			break;
		case AllStaticObjects:
			ObjectTypesToQuery64 = FCollisionQueryFlag::Get().GetAllStaticObjectsQueryFlag64();
			break;
		case AllDynamicObjects:
			ObjectTypesToQuery64 = FCollisionQueryFlag::Get().GetAllDynamicObjectsQueryFlag64();
			break;
		}

		IgnoreMask = 0;
		UpdateDeprecatedObjectTypesToQuery();
	};

	// to do this, use ECC_TO_BITFIELD to convert to bit field
	// i.e. FCollisionObjectQueryParams ( ECC_TO_BITFIELD(ECC_WorldStatic) | ECC_TO_BITFIELD(ECC_WorldDynamic) )
	FCollisionObjectQueryParams(int64 InObjectTypesToQuery)
	{
		ObjectTypesToQuery64 = InObjectTypesToQuery;
		UpdateDeprecatedObjectTypesToQuery();
		IgnoreMask = 0;
		DoVerify();
	}

	void AddObjectTypesToQuery(ECollisionChannel QueryChannel)
	{
		ObjectTypesToQuery64 |= ToBitMask(QueryChannel);
		UpdateDeprecatedObjectTypesToQuery();
		DoVerify();
	}

	void RemoveObjectTypesToQuery(ECollisionChannel QueryChannel)
	{
		ObjectTypesToQuery64 &= ~ToBitMask(QueryChannel);
		UpdateDeprecatedObjectTypesToQuery();
		DoVerify();
	}

	UE_DEPRECATED(5.8, "Use GetQueryBitfield64")
	int32 GetQueryBitfield() const
	{
		return (int32)GetQueryBitfield64();
	}

	int64 GetQueryBitfield64() const
	{
		checkSlow(IsValid());

		return ObjectTypesToQuery64;
	}

	bool IsValid() const
	{ 
		return (ObjectTypesToQuery64 != 0); 
	}

	static bool IsValidObjectQuery(ECollisionChannel QueryChannel) 
	{
		const uint64 QueryChannelBit = ToBitMask(QueryChannel);
		// return true if this belong to object query type
		return (QueryChannelBit & FCollisionQueryFlag::Get().GetAllObjectsQueryFlag64()) != 0;
	}

	void DoVerify() const
	{
		// you shouldn't use Trace Types to be Object Type Query parameter. This is not technical limitation, but verification process. 
		checkSlow((ObjectTypesToQuery64 & FCollisionQueryFlag::Get().GetAllObjectsQueryFlag64()) == ObjectTypesToQuery64);
	}

	/** Internal. */
	static inline FCollisionObjectQueryParams::InitType GetCollisionChannelFromOverlapFilter(EOverlapFilterOption Filter)
	{
		static FCollisionObjectQueryParams::InitType ConvertMap[3] = { FCollisionObjectQueryParams::InitType::AllObjects, FCollisionObjectQueryParams::InitType::AllDynamicObjects, FCollisionObjectQueryParams::InitType::AllStaticObjects };
		return ConvertMap[Filter];
	}
	static ENGINE_API FCollisionObjectQueryParams DefaultObjectQueryParam;
};

