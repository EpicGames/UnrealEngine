// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace CommandletHelpers
{
	ENGINE_API FString BuildCommandletProcessArguments(const TCHAR* const CommandletName, const TCHAR* const ProjectPath, const TCHAR* const AdditionalArguments);
}
