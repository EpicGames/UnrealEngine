// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/StructOnScope.h"
#include "Engine/EngineTypes.h"
#include "ComponentInstanceDataCache.generated.h"

class AActor;
class UActorComponent;
class USceneComponent;
enum class EComponentCreationMethod : uint8;

/** At what point in the rerun construction script process is ApplyToActor being called for */
enum class ECacheApplyPhase
{
	PostSimpleConstructionScript,	// After the simple construction script has been run
	PostUserConstructionScript,		// After the user construction script has been run
	NonConstructionScript			// Not called during the construction script process
};

UENUM()
enum class EComponentCreationMethod : uint8
{
	/** A component that is part of a native class. */
	Native,
	/** A component that is created from a template defined in the Components section of the Blueprint. */
	SimpleConstructionScript,
	/**A dynamically created component, either from the UserConstructionScript or from a Add Component node in a Blueprint event graph. */
	UserConstructionScript,
	/** A component added to a single Actor instance via the Component section of the Actor's details panel. */
	Instance,
};

UE_DEPRECATED(5.1, "FActorComponentDuplicatedObjectData has been renamed to FDataCacheDuplicatedObjectData")
typedef FDataCacheDuplicatedObjectData FActorComponentDuplicatedObjectData;

USTRUCT()
struct FDataCacheDuplicatedObjectData
{
	GENERATED_BODY()

	FDataCacheDuplicatedObjectData(UObject* InObject = nullptr);

	bool Serialize(FArchive& Ar);

	// The duplicated object
	UObject* DuplicatedObject;

	// Object Outer Depth so we can sort creation order
	int32 ObjectPathDepth;
};

// Trait to signal ActorCompomentInstanceData duplicated objects uses a serialize function
template<>
struct TStructOpsTypeTraits<FDataCacheDuplicatedObjectData> : public TStructOpsTypeTraitsBase2<FDataCacheDuplicatedObjectData>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Base class for instance cached data of a particular type. */
USTRUCT()
struct FInstanceCacheDataBase
{
	GENERATED_BODY()

	ENGINE_API virtual ~FInstanceCacheDataBase();

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);

	/** Get (or create) the unique transient outer for the duplicated objects created for this object */
	ENGINE_API UObject* GetUniqueTransientObject(UClass* Class);

	const TArray<FDataCacheDuplicatedObjectData>& GetDuplicatedObjects() const { return DuplicatedObjects; }
	const TArray<TObjectPtr<UObject>>& GetReferencedObjects() const { return ReferencedObjects; }
	const TArray<uint8>& GetSavedProperties() const { return SavedProperties; }

protected:
	UPROPERTY()
	TArray<uint8> SavedProperties;

private:
	friend class FDataCachePropertyWriter;
	friend class FDataCachePropertyReader;

	/** 
	 * A unique outer created in the transient package to act as outer for this object's duplicated objects 
	 * to avoid name conflicts of objects that already exist in the transient package
	 */
	UPROPERTY()
	FDataCacheDuplicatedObjectData UniqueTransientPackage;

	// Duplicated objects created when saving instance properties
	UPROPERTY()
	TArray<FDataCacheDuplicatedObjectData> DuplicatedObjects;

	// Referenced objects in instance saved properties
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ReferencedObjects;

	// Referenced names in instance saved properties
	UPROPERTY()
	TArray<FName> ReferencedNames;
};

USTRUCT()
struct FActorComponentInstanceSourceInfo
{
	GENERATED_BODY()

public:
	FActorComponentInstanceSourceInfo() = default;
	ENGINE_API explicit FActorComponentInstanceSourceInfo(const UActorComponent* SourceComponent);
	ENGINE_API FActorComponentInstanceSourceInfo(TObjectPtr<const UObject> InSourceComponentTemplate, EComponentCreationMethod InSourceComponentCreationMethod, int32 InSourceComponentTypeSerializedIndex);

	/** Determines whether this component instance data matches the component */
	ENGINE_API bool MatchesComponent(const UActorComponent* Component) const;
	ENGINE_API bool MatchesComponent(const UActorComponent* Component, const UObject* ComponentTemplate) const;

	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

private:
	/** The template used to create the source component */
	UPROPERTY()
	TObjectPtr<const UObject> SourceComponentTemplate = nullptr;

	/** The method that was used to create the source component */
	UPROPERTY()
	EComponentCreationMethod SourceComponentCreationMethod = EComponentCreationMethod::Native;

	/** The index of the source component in its owner's serialized array
	when filtered to just that component type */
	UPROPERTY()
	int32 SourceComponentTypeSerializedIndex = INDEX_NONE;
};

/** Base class for component instance cached data of a particular type. */
USTRUCT()
struct FActorComponentInstanceData : public FInstanceCacheDataBase
{
	GENERATED_BODY()
public:
	ENGINE_API FActorComponentInstanceData();
	ENGINE_API FActorComponentInstanceData(const UActorComponent* SourceComponent);

	/** Determines whether this component instance data matches the component */
	ENGINE_API bool MatchesComponent(const UActorComponent* Component, const UObject* ComponentTemplate) const;

	/** Determines if any instance data was actually saved. */
	virtual bool ContainsData() const { return SavedProperties.Num() > 0; }

	/** Applies this component instance data to the supplied component */
	ENGINE_API virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase);

	/** Replaces any references to old instances during Actor reinstancing */
	virtual void FindAndReplaceInstances(const TMap<UObject*, UObject*>& OldToNewInstanceMap) { };

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	const UClass* GetComponentClass() const { return SourceComponentTemplate ? SourceComponentTemplate->GetClass() : nullptr; }

	const UObject* GetComponentTemplate() const { return SourceComponentTemplate; }

protected:

	/** The template used to create the source component */
	UPROPERTY()
	TObjectPtr<const UObject> SourceComponentTemplate;

	/** The method that was used to create the source component */
	UPROPERTY() 
	EComponentCreationMethod SourceComponentCreationMethod;

	/** The index of the source component in its owner's serialized array
	when filtered to just that component type */
	UPROPERTY()
	int32 SourceComponentTypeSerializedIndex;
};

/** Per instance data to be persisted for a given actor */
USTRUCT()
struct FActorInstanceData : public FInstanceCacheDataBase
{
	GENERATED_BODY()
public:
	FActorInstanceData() = default;
	ENGINE_API FActorInstanceData(const AActor* SourceActor);

	ENGINE_API const UClass* GetActorClass() const;

	bool HasInstanceData() const { return GetSavedProperties().Num() > 0; }

	/** Iterates over an Actor's components and applies the stored component instance data to each */
	ENGINE_API void ApplyToActor(AActor* Actor, const ECacheApplyPhase CacheApplyPhase);

protected:

	/** The class of the actor that the instance data is for */
	UPROPERTY()
	TSubclassOf<AActor> ActorClass;
};

/** 
 *	Cache for component instance data.
 *	Note, does not collect references for GC, so is not safe to GC if the cache is only reference to a UObject.
 */
class FComponentInstanceDataCache
{
public:

	FComponentInstanceDataCache() = default;

	/** Constructor that also populates cache from Actor */
	ENGINE_API FComponentInstanceDataCache(const AActor* InActor);

	~FComponentInstanceDataCache() = default;

	/** Non-copyable */
	FComponentInstanceDataCache(const FComponentInstanceDataCache&) = delete;
	FComponentInstanceDataCache& operator=(const FComponentInstanceDataCache&) = delete;

	/** Movable */
	FComponentInstanceDataCache(FComponentInstanceDataCache&&) = default;
	FComponentInstanceDataCache& operator=(FComponentInstanceDataCache&&) = default;

	/** Serialize Instance data for persistence or transmission. */
	ENGINE_API void Serialize(FArchive& Ar);

	/** Iterates over an Actor's components and applies the stored component instance data to each */
	ENGINE_API void ApplyToActor(AActor* Actor, const ECacheApplyPhase CacheApplyPhase) const;

	/** Iterates over components and replaces any object references with the reinstanced information */
	ENGINE_API void FindAndReplaceInstances(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	bool HasInstanceData() const { return ComponentsInstanceData.Num() > 0; }

	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	static ENGINE_API void GetComponentHierarchy(const AActor* Actor, TArray<UActorComponent*, TInlineAllocator<NumInlinedActorComponents>>& OutComponentHierarchy);

private:
	// called during de-serialization to copy serialized properties over existing component instance data and keep non UPROPERTY data intact
	ENGINE_API void CopySerializableProperties(TArray<TStructOnScope<FActorComponentInstanceData>> InComponentsInstanceData);

	/** Map of component instance data struct (template -> instance data) */
	TArray<TStructOnScope<FActorComponentInstanceData>> ComponentsInstanceData;

	/** Map of the actor instanced scene component to their transform relative to the root. */
	TMap< TObjectPtr<USceneComponent>, FTransform > InstanceComponentTransformToRootMap;
};
