// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Components/ActorComponent.h"
#include "Components/ComponentInterfaces.h"
#include "SceneInterface.h"

/** Destroys render state for a component and then recreates it when this object is destroyed */
class FComponentRecreateRenderStateContext
{
private:
	/** Pointer to component we are recreating render state for */	
	UActorComponent* Component = nullptr;
	IPrimitiveComponent* ComponentInterface = nullptr;

	TSet<FSceneInterface*>* ScenesToUpdateAllPrimitiveSceneInfos = nullptr;

public:
	FComponentRecreateRenderStateContext(IPrimitiveComponent* InComponentInterface, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfos = nullptr)		
			: ScenesToUpdateAllPrimitiveSceneInfos(InScenesToUpdateAllPrimitiveSceneInfos)
	{
		check(InComponentInterface);
		checkf(!InComponentInterface->IsUnreachable(), TEXT("%s"), *InComponentInterface->GetFullName());

		if (InComponentInterface->IsRegistered() && InComponentInterface->IsRenderStateCreated())
		{
			InComponentInterface->DestroyRenderState();
			ComponentInterface = InComponentInterface;

			UpdateAllPrimitiveSceneInfosForSingleComponentInterface(InComponentInterface, ScenesToUpdateAllPrimitiveSceneInfos);
		}
	}

	FComponentRecreateRenderStateContext(UActorComponent* InComponent, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfos = nullptr)
		: ScenesToUpdateAllPrimitiveSceneInfos(InScenesToUpdateAllPrimitiveSceneInfos)
	{
		check(InComponent);
		checkf(!InComponent->IsUnreachable(), TEXT("%s"), *InComponent->GetFullName());

		if (InComponent->IsRegistered() && InComponent->IsRenderStateCreated())
		{
			InComponent->DestroyRenderState_Concurrent();
			Component = InComponent;

			UpdateAllPrimitiveSceneInfosForSingleComponent(InComponent, ScenesToUpdateAllPrimitiveSceneInfos);
		}
	}

	FComponentRecreateRenderStateContext(const FComponentRecreateRenderStateContext&) = delete;
	FComponentRecreateRenderStateContext& operator=(const FComponentRecreateRenderStateContext&) = delete;
	
	FComponentRecreateRenderStateContext(FComponentRecreateRenderStateContext&& Other)
		: Component(Other.Component)
		, ComponentInterface(Other.ComponentInterface)
		, ScenesToUpdateAllPrimitiveSceneInfos(Other.ScenesToUpdateAllPrimitiveSceneInfos)
	{
		Other.Component = nullptr;
		Other.ComponentInterface = nullptr;
		Other.ScenesToUpdateAllPrimitiveSceneInfos = nullptr;
	}

	FComponentRecreateRenderStateContext& operator=(FComponentRecreateRenderStateContext&& Other)
	{
		Component = Other.Component;
		ComponentInterface = Other.ComponentInterface ;
		ScenesToUpdateAllPrimitiveSceneInfos = Other.ScenesToUpdateAllPrimitiveSceneInfos;
		Other.Component = nullptr;
		Other.ComponentInterface = nullptr;
		Other.ScenesToUpdateAllPrimitiveSceneInfos = nullptr;
		return *this;
	}

	ENGINE_API ~FComponentRecreateRenderStateContext();
};

/** Destroys render states for all components or for a provided list of components and then recreates them when this object is destroyed */
class FGlobalComponentRecreateRenderStateContext
{
public:
	/** 
	* Initialization constructor. 
	*/
	ENGINE_API FGlobalComponentRecreateRenderStateContext();

	/** 
	* Initialization constructor for a provided component list. 
	*/
	ENGINE_API FGlobalComponentRecreateRenderStateContext(const TArray<UActorComponent*>& InComponents);


	/** Destructor */
	ENGINE_API ~FGlobalComponentRecreateRenderStateContext();

	/** Indicates that a FGlobalComponentRecreateRenderStateContext is currently active */
	static int32 ActiveGlobalRecreateRenderStateContextCount;

private:
	/** The recreate contexts for the individual components. */
	TArray<FComponentRecreateRenderStateContext> ComponentContexts;

	TSet<FSceneInterface*> ScenesToUpdateAllPrimitiveSceneInfos;

	void UpdateAllPrimitiveSceneInfos();
};
