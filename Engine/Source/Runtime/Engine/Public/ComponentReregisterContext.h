// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "RenderingThread.h"
#include "UObject/UObjectIterator.h"
#include "Engine/World.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogActorComponent, Log, All);

/**
 * Base class for Component Reregister objects, provides helper functions for performing the UnRegister
 * and ReRegister
 */
class FComponentReregisterContextBase
{
protected:
	TSet<FSceneInterface*>* ScenesToUpdateAllPrimitiveSceneInfos = nullptr;

	//Unregisters the Component and returns the world it was registered to.
	ENGINE_API UWorld* UnRegister(UActorComponent* InComponent);

	//Reregisters the given component on the given scene
	ENGINE_API void ReRegister(UActorComponent* InComponent, UWorld* InWorld);
};

/**
 * Unregisters a component for the lifetime of this class.
 *
 * Typically used by constructing the class on the stack:
 * {
 *		FComponentReregisterContext ReregisterContext(this);
 *		// The component is unregistered with the world here as ReregisterContext is constructed.
 *		...
 * }	// The component is registered with the world here as ReregisterContext is destructed.
 */

class FComponentReregisterContext : public FComponentReregisterContextBase
{
private:
	/** Pointer to component we are unregistering */
	TWeakObjectPtr<UActorComponent> Component;
	/** Cache pointer to world from which we were removed */
	TWeakObjectPtr<UWorld> World;
public:
	FComponentReregisterContext(UActorComponent* InComponent, TSet<FSceneInterface*>* InScenesToUpdateAllPrimitiveSceneInfos = nullptr)
		: World(nullptr)
	{
		ScenesToUpdateAllPrimitiveSceneInfos = InScenesToUpdateAllPrimitiveSceneInfos;

		World = UnRegister(InComponent);
		// If we didn't get a scene back NULL the component so we dont try to
		// process it on destruction
		Component = World.IsValid() ? InComponent : nullptr;
	}

	~FComponentReregisterContext()
	{
		if( Component.IsValid() && World.IsValid() )
		{
			ReRegister(Component.Get(), World.Get());
		}
	}
};


/* Pairing of UActorComponent and its UWorld. Used only by FMultiComponentReregisterContext
 * for tracking purposes 
 */
struct FMultiComponentReregisterPair
{
	/** Pointer to component we are unregistering */
	UActorComponent* Component;
	/** Cache pointer to world from which we were removed */
	UWorld* World;

	FMultiComponentReregisterPair(UActorComponent* _Component, UWorld* _World) : Component(_Component), World(_World) {}
};


/**
 * Unregisters multiple components for the lifetime of this class.
 *
 * Typically used by constructing the class on the stack:
 * {
 *		FMultiComponentReregisterContext ReregisterContext(arrayOfComponents);
 *		// The components are unregistered with the world here as ReregisterContext is constructed.
 *		...
 * }	// The components are registered with the world here as ReregisterContext is destructed.
 */

class FMultiComponentReregisterContext : public FComponentReregisterContextBase
{
private:
	/** Component pairs that need to be re registered */
	TArray<FMultiComponentReregisterPair> ComponentsPair;
	
public:
	FMultiComponentReregisterContext(const TArray<UActorComponent*>& InComponents)
	{
		// Unregister each component and cache resulting scene
		for (UActorComponent* Component : InComponents)
		{
			UWorld* World = UnRegister(Component);
			if(World)
			{
				ComponentsPair.Push( FMultiComponentReregisterPair(Component, World) );
			}
		}
	}

	~FMultiComponentReregisterContext()
	{
		//Re-register each valid component pair that we unregistered in our constructor
		for(auto Iter = ComponentsPair.CreateIterator(); Iter; ++Iter)
		{
			FMultiComponentReregisterPair& pair = (*Iter);
			if(pair.Component)
			{
				ReRegister(pair.Component, pair.World);
			}
		}
	}
};

/** Removes all components from their scenes for the lifetime of the class. */
class FGlobalComponentReregisterContext
{
public:
	/** 
	* Initialization constructor. 
	*/
	ENGINE_API FGlobalComponentReregisterContext();
	
	/** 
	* Initialization constructor. 
	*
	* @param ExcludeComponents - Component types to exclude when reregistering 
	*/
	FGlobalComponentReregisterContext(const TArray<UClass*>& ExcludeComponents);

	/** Destructor */
	ENGINE_API ~FGlobalComponentReregisterContext();

	/** Indicates that a FGlobalComponentReregisterContext is currently active */
	static int32 ActiveGlobalReregisterContextCount;

	DECLARE_EVENT(FGlobalComponentReregisterContext, FOnGlobalComponentReregisterContextCreated);
	static FOnGlobalComponentReregisterContextCreated& OnGlobalComponentReregisterContextCreated() { return GlobalComponentReregisterContextCreatedEvent; }

	DECLARE_EVENT(FGlobalComponentReregisterContext, FOnGlobalComponentReregisterContextDestroyed);
	static FOnGlobalComponentReregisterContextDestroyed& OnGlobalComponentReregisterContextDestroyed() { return GlobalComponentReregisterContextDestroyedEvent; }

private:
	/** The recreate contexts for the individual components. */
	TIndirectArray<FComponentReregisterContext> ComponentContexts;

	TSet<FSceneInterface*> ScenesToUpdateAllPrimitiveSceneInfos;

	/** Static delegate broadcasted when the first active FGlobalComponentReregisterContext is created. */
	ENGINE_API static FOnGlobalComponentReregisterContextCreated GlobalComponentReregisterContextCreatedEvent;

	/** Static delegate broadcasted when the last active FGlobalComponentReregisterContext is destroyed. */
	ENGINE_API static FOnGlobalComponentReregisterContextDestroyed GlobalComponentReregisterContextDestroyedEvent;

	void UpdateAllPrimitiveSceneInfos();
};

/** Removes all components of the templated type from their scenes for the lifetime of the class. */
template<class ComponentType>
class TComponentReregisterContext
{
public:
	/** Initialization constructor. */
	TComponentReregisterContext()
	{
		// wait until resources are released
		FlushRenderingCommands();

		// Reregister all components of the templated type.
		for(TObjectIterator<ComponentType> ComponentIt;ComponentIt;++ComponentIt)
		{
			ComponentContexts.Add(new FComponentReregisterContext(*ComponentIt, &ScenesToUpdateAllPrimitiveSceneInfos));
		}

		UpdateAllPrimitiveSceneInfos();
	}

	~TComponentReregisterContext()
	{
		UpdateAllPrimitiveSceneInfos();
	}

private:
	/** The recreate contexts for the individual components. */
	TIndirectArray<FComponentReregisterContext> ComponentContexts;

	TSet<FSceneInterface*> ScenesToUpdateAllPrimitiveSceneInfos;

	void UpdateAllPrimitiveSceneInfos()
	{
		UpdateAllPrimitiveSceneInfosForScenes(MoveTemp(ScenesToUpdateAllPrimitiveSceneInfos));

		check(ScenesToUpdateAllPrimitiveSceneInfos.Num() == 0);
	}
};

