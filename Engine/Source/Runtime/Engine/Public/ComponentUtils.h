// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Components/SceneComponent.h"

class USCS_Node;
class USimpleConstructionScript;

DECLARE_STATS_GROUP(TEXT("Component"), STATGROUP_Component, STATCAT_Advanced);

namespace ComponentUtils
{
	/**
	 * A helper for retrieving the simple-construction-script that this component belongs in.
	 * 
	 * @param  Component	The component you want the SCS for.
	 * @return The component's blueprint SCS (NULL if one wasn't found).
	 */
	ENGINE_API USimpleConstructionScript const* GetSimpleConstructionScript(USceneComponent const* Component);

	/**
	 * A static helper function for retrieving the simple-construction-script node that
	 * corresponds to the specified scene component template.
	 * 
	 * @param  ComponentObj	The component you want to find a USCS_Node for
	 * @return A USCS_Node pointer corresponding to the specified component (NULL if we didn't find one)
	 */
	ENGINE_API USCS_Node* FindCorrespondingSCSNode(USceneComponent const* ComponentObj);

	/**
	 * A static helper function used to retrieve a component's scene parent
	 * 
	 * @param  SceneComponentObject	The component you want the attached parent for
	 * @return A pointer to the component's scene parent (NULL if there was not one)
	 */
	ENGINE_API USceneComponent* GetAttachedParent(USceneComponent const* SceneComponentObject);

};
