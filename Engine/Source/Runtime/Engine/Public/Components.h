// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Containers/StridedView.h"
#include "MeshUVChannelInfo.h"
#include "VertexStreamComponent.h"

/*=============================================================================
	Components.h: Forward declarations of object components of actors
=============================================================================*/

class FRHIShaderResourceView;


// Constants.
enum { MAX_STATIC_TEXCOORDS = 8 };

/** The information used to build a static-mesh vertex. */
struct FStaticMeshBuildVertex
{
	FVector3f Position;

	FVector3f TangentX;
	FVector3f TangentY;
	FVector3f TangentZ;

	float CurveRadius;

	FVector2f UVs[MAX_STATIC_TEXCOORDS];
	FColor Color;
};

struct FStaticMeshDataType
{
	/** The stream to read the vertex position from. */
	FVertexStreamComponent PositionComponent;

	/** The streams to read the tangent basis from. */
	FVertexStreamComponent TangentBasisComponents[2];

	/** The streams to read the texture coordinates from. */
	TArray<FVertexStreamComponent, TFixedAllocator<MAX_STATIC_TEXCOORDS / 2> > TextureCoordinates;

	/** The stream to read the shadow map texture coordinates from. */
	FVertexStreamComponent LightMapCoordinateComponent;

	/** The stream to read the vertex color from. */
	FVertexStreamComponent ColorComponent;

	FRHIShaderResourceView* PositionComponentSRV = nullptr;

	FRHIShaderResourceView* TangentsSRV = nullptr;

	/** A SRV to manually bind and load TextureCoordinates in the vertex shader. */
	FRHIShaderResourceView* TextureCoordinatesSRV = nullptr;

	/** A SRV to manually bind and load Colors in the vertex shader. */
	FRHIShaderResourceView* ColorComponentsSRV = nullptr;

	uint32 ColorIndexMask = ~0u;
	int8 LightMapCoordinateIndex = -1;
	uint8 NumTexCoords = 0;
	uint8 LODLightmapDataIndex = 0;
};

/** The information used to build a mesh. */
struct FConstMeshBuildVertexView
{
	TStridedView<const FVector3f> Position;
	TStridedView<const FVector3f> TangentX;
	TStridedView<const FVector3f> TangentY;
	TStridedView<const FVector3f> TangentZ;
	TStridedView<const float> CurveRadius;
	TArray<TStridedView<const uint16>> BoneIndices;
	TArray<TStridedView<const uint16>> BoneWeights;
	TArray<TStridedView<const FVector2f>, TInlineAllocator<1>> UVs;
	TStridedView<const FColor> Color;
};

struct FMeshBuildVertexView
{
	TStridedView<FVector3f> Position;
	TStridedView<FVector3f> TangentX;
	TStridedView<FVector3f> TangentY;
	TStridedView<FVector3f> TangentZ;
	TStridedView<float> CurveRadius;
	TArray<TStridedView<uint16>> BoneIndices;
	TArray<TStridedView<uint16>> BoneWeights;
	TArray<TStridedView<FVector2f>, TInlineAllocator<1>> UVs;
	TStridedView<FColor> Color;
};

struct FMeshBuildVertexData
{
	TArray<FVector3f> Position;
	TArray<FVector3f> TangentX;
	TArray<FVector3f> TangentY;
	TArray<FVector3f> TangentZ;
	TArray<float> CurveRadius;
	TArray<TArray<uint16>> BoneIndices;
	TArray<TArray<uint16>> BoneWeights;
	TArray<TArray<FVector2f>, TInlineAllocator<1>> UVs;
	TArray<FColor> Color;

	inline void Empty(int32 Slack = 0, int32 NumTexCoords = 0, int32 NumBoneInfluences = 0)
	{
		Position.Empty(Slack);
		TangentX.Empty(Slack);
		TangentY.Empty(Slack);
		TangentZ.Empty(Slack);
		CurveRadius.Empty(Slack);
		Color.Empty(Slack);

		BoneIndices.SetNum(NumBoneInfluences);
		BoneWeights.SetNum(NumBoneInfluences);
		for (int32 Influence = 0; Influence < NumBoneInfluences; ++Influence)
		{
			BoneIndices[Influence].Empty(Slack);
			BoneWeights[Influence].Empty(Slack);
		}

		UVs.SetNum(NumTexCoords);
		for (int32 TexCoord = 0; TexCoord < NumTexCoords; ++TexCoord)
		{
			UVs[TexCoord].Empty(Slack);
		}
	}
};

inline void RemoveInvalidVertexColor(FMeshBuildVertexView& VertexView)
{
	if (VertexView.Color.Num() > 0)
	{
		// Don't trust any input. We only have color if it isn't all white.
		uint32 Channel = 255;

		for (const FColor& Color : VertexView.Color)
		{
			Channel &= Color.R;
			Channel &= Color.G;
			Channel &= Color.B;
			Channel &= Color.A;

			if (Channel != 255)
			{
				break;
			}
		}

		if (Channel == 255)
		{
			VertexView.Color = {};
		}
	}
}

inline void RemoveInvalidVertexColor(FConstMeshBuildVertexView& VertexView)
{
	if (VertexView.Color.Num() > 0)
	{
		// Don't trust any input. We only have color if it isn't all white.
		uint32 Channel = 255;

		for (const FColor& Color : VertexView.Color)
		{
			Channel &= Color.R;
			Channel &= Color.G;
			Channel &= Color.B;
			Channel &= Color.A;

			if (Channel != 255)
			{
				break;
			}
		}

		if (Channel == 255)
		{
			VertexView.Color = {};
		}
	}
}

/** Make a strided mesh build vertex view from FStaticMeshBuildVertex. */
inline FMeshBuildVertexView MakeMeshBuildVertexView(TArray<FStaticMeshBuildVertex>& InVertices)
{
	FMeshBuildVertexView View{};
	if (InVertices.Num() > 0)
	{
		View.Position = MakeStridedView(InVertices, &FStaticMeshBuildVertex::Position);
		View.TangentX = MakeStridedView(InVertices, &FStaticMeshBuildVertex::TangentX);
		View.TangentY = MakeStridedView(InVertices, &FStaticMeshBuildVertex::TangentY);
		View.TangentZ = MakeStridedView(InVertices, &FStaticMeshBuildVertex::TangentZ);
		View.CurveRadius = MakeStridedView(InVertices, &FStaticMeshBuildVertex::CurveRadius);
		for (int32 UVCoord = 0; UVCoord < MAX_STATIC_TEXCOORDS; ++UVCoord)
		{
			View.UVs.Add(MakeStridedView(sizeof(FStaticMeshBuildVertex), &InVertices[0].UVs[UVCoord], InVertices.Num()));
		}
		View.Color = MakeStridedView(InVertices, &FStaticMeshBuildVertex::Color);
		RemoveInvalidVertexColor(View);
	}

	return View;
}

inline FConstMeshBuildVertexView MakeConstMeshBuildVertexView(const TConstArrayView<FStaticMeshBuildVertex>& InVertices)
{
	FConstMeshBuildVertexView View{};
	if (InVertices.Num() > 0)
	{
		View.Position = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::Position);
		View.TangentX = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::TangentX);
		View.TangentY = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::TangentY);
		View.TangentZ = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::TangentZ);
		View.CurveRadius = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::CurveRadius);
		for (int32 UVCoord = 0; UVCoord < MAX_STATIC_TEXCOORDS; ++UVCoord)
		{
			View.UVs.Add(MakeConstStridedView(sizeof(FStaticMeshBuildVertex), &InVertices[0].UVs[UVCoord], InVertices.Num()));
		}
		View.Color = MakeConstStridedView(InVertices, &FStaticMeshBuildVertex::Color);
		RemoveInvalidVertexColor(View);
	}

	return View;
}

/** Make a strided mesh build vertex view from FMeshBuildVertexData. */
inline FMeshBuildVertexView MakeMeshBuildVertexView(FMeshBuildVertexData& InVertexData)
{
	FMeshBuildVertexView View{};
	{
		View.Position	= MakeStridedView(InVertexData.Position);
		View.TangentX	= MakeStridedView(InVertexData.TangentX);
		View.TangentY	= MakeStridedView(InVertexData.TangentY);
		View.TangentZ	= MakeStridedView(InVertexData.TangentZ);
		View.CurveRadius= MakeStridedView(InVertexData.CurveRadius);
		View.Color		= MakeStridedView(InVertexData.Color);

		View.BoneIndices.Reserve(InVertexData.BoneIndices.Num());
		for (int32 Influence = 0; Influence < InVertexData.BoneIndices.Num(); ++Influence)
		{
			if (InVertexData.BoneIndices[Influence].Num() > 0)
			{
				View.BoneIndices.Add(MakeStridedView(sizeof(uint16), &InVertexData.BoneIndices[Influence][0], InVertexData.BoneIndices[Influence].Num()));
			}
			else
			{
				View.BoneIndices.Add({});
			}
		}

		View.BoneWeights.Reserve(InVertexData.BoneWeights.Num());
		for (int32 Influence = 0; Influence < InVertexData.BoneWeights.Num(); ++Influence)
		{
			if (InVertexData.BoneWeights[Influence].Num() > 0)
			{
				View.BoneWeights.Add(MakeStridedView(sizeof(uint16), &InVertexData.BoneWeights[Influence][0], InVertexData.BoneWeights[Influence].Num()));
			}
			else
			{
				View.BoneWeights.Add({});
			}
		}

		View.UVs.Reserve(InVertexData.UVs.Num());
		for (int32 UVCoord = 0; UVCoord < InVertexData.UVs.Num(); ++UVCoord)
		{
			if (InVertexData.UVs[UVCoord].Num() > 0)
			{
				View.UVs.Add(MakeStridedView(sizeof(FVector2f), &InVertexData.UVs[UVCoord][0], InVertexData.UVs[UVCoord].Num()));
			}
			else
			{
				View.UVs.Add({});
			}
		}

		RemoveInvalidVertexColor(View);
	}
	return View;
}

inline FConstMeshBuildVertexView MakeConstMeshBuildVertexView(const FMeshBuildVertexData& InVertexData)
{
	FConstMeshBuildVertexView View{};
	{
		View.Position	= MakeStridedView(InVertexData.Position);
		View.TangentX	= MakeStridedView(InVertexData.TangentX);
		View.TangentY	= MakeStridedView(InVertexData.TangentY);
		View.TangentZ	= MakeStridedView(InVertexData.TangentZ);
		View.CurveRadius= MakeStridedView(InVertexData.CurveRadius);
		View.Color		= MakeStridedView(InVertexData.Color);

		View.BoneIndices.Reserve(InVertexData.BoneIndices.Num());
		for (int32 Influence = 0; Influence < InVertexData.BoneIndices.Num(); ++Influence)
		{
			if (InVertexData.BoneIndices[Influence].Num() > 0)
			{
				View.BoneIndices.Add(MakeStridedView(sizeof(uint16), &InVertexData.BoneIndices[Influence][0], InVertexData.BoneIndices[Influence].Num()));
			}
			else
			{
				View.BoneIndices.Add({});
			}
		}

		View.BoneWeights.Reserve(InVertexData.BoneWeights.Num());
		for (int32 Influence = 0; Influence < InVertexData.BoneWeights.Num(); ++Influence)
		{
			if (InVertexData.BoneWeights[Influence].Num() > 0)
			{
				View.BoneWeights.Add(MakeStridedView(sizeof(uint16), &InVertexData.BoneWeights[Influence][0], InVertexData.BoneWeights[Influence].Num()));
			}
			else
			{
				View.BoneWeights.Add({});
			}
		}
		
		View.UVs.Reserve(InVertexData.UVs.Num());
		for (int32 UVCoord = 0; UVCoord < InVertexData.UVs.Num(); ++UVCoord)
		{
			if (InVertexData.UVs[UVCoord].Num() > 0)
			{
				View.UVs.Add(MakeStridedView(sizeof(FVector2f), &InVertexData.UVs[UVCoord][0], InVertexData.UVs[UVCoord].Num()));
			}
			else
			{
				View.UVs.Add({});
			}
		}

		RemoveInvalidVertexColor(View);
	}
	return View;
}


inline FStaticMeshBuildVertex MakeStaticMeshVertex(const FMeshBuildVertexView& View, int32 Index)
{
	FStaticMeshBuildVertex Vertex;

	Vertex.Position = View.Position[Index];
	Vertex.TangentX = View.TangentX[Index];
	Vertex.TangentY = View.TangentZ[Index];
	Vertex.TangentZ = View.TangentY[Index];
	Vertex.CurveRadius = View.CurveRadius.Num() > 0 ? View.CurveRadius[Index] : 0.f;

	for (int32 UVCoord = 0; UVCoord < View.UVs.Num(); ++UVCoord)
	{
		Vertex.UVs[UVCoord] = View.UVs[UVCoord][Index];
	}

	for (int32 UVCoord = View.UVs.Num(); UVCoord < MAX_STATIC_TEXCOORDS; ++UVCoord)
	{
		Vertex.UVs[UVCoord] = FVector2f::ZeroVector;
	}

	if (View.Color.Num() > 0)
	{
		Vertex.Color = View.Color[Index];
	}
	else
	{
		Vertex.Color = FColor::White;
	}

	return Vertex;
}

inline FStaticMeshBuildVertex MakeStaticMeshVertex(const FMeshBuildVertexData& InVertexData, int32 Index)
{
	FStaticMeshBuildVertex Output;
	Output.Position = InVertexData.Position[Index];
	Output.TangentX = InVertexData.TangentX[Index];
	Output.TangentY = InVertexData.TangentY[Index];
	Output.TangentZ = InVertexData.TangentZ[Index];
	Output.CurveRadius = InVertexData.CurveRadius.Num() ? InVertexData.CurveRadius[Index] : 0.f;
	Output.Color = InVertexData.Color.Num() ? InVertexData.Color[Index] : FColor::White;
	
	for (int32 UVCoord = 0; UVCoord < InVertexData.UVs.Num(); ++UVCoord)
	{
		Output.UVs[UVCoord] = InVertexData.UVs[UVCoord][Index];
	}

	for (int32 UVCoord = InVertexData.UVs.Num(); UVCoord < MAX_STATIC_TEXCOORDS; ++UVCoord)
	{
		Output.UVs[UVCoord] = FVector2f::ZeroVector;
	}

	return Output;
}
