// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGBuilder;
class FSceneInterface;
class FSceneView;
namespace ERHIFeatureLevel { enum Type : int; }

struct FComputeContext
{
	FRDGBuilder& GraphBuilder;
	FName ExecutionGroupName;
	ERHIFeatureLevel::Type FeatureLevel = static_cast<ERHIFeatureLevel::Type>(0);
	const FSceneInterface* Scene = nullptr;
	const FSceneView* View = nullptr; // Null if executed outside of view rendering.

	int32 GraphKernelIndex = INDEX_NONE;
};

/** 
 * Interface for a compute task worker.
 * Implementations will queue and schedule work per scene before the renderer submits at fixed points in the frame.
 */
class IComputeTaskWorker
{
public:
	virtual ~IComputeTaskWorker() = default;

	/** Returns true if there is any scheduled work. */
	virtual bool HasWork(FName InExecutionGroupName) const = 0;

	/** Add any scheduled work to an RDGBuilder ready for execution. */
	virtual void SubmitWork(FComputeContext& Context) = 0;

	UE_DEPRECATED(5.7, "Use version that takes FComputeContext instead")
	ENGINE_API virtual void SubmitWork(FRDGBuilder& GraphBuilder, FName InExecutionGroupName, ERHIFeatureLevel::Type InFeatureLevel);
};

/** Core execution group names for use in IComputeTaskWorker::SubmitWork(). */
struct ComputeTaskExecutionGroup
{
	static ENGINE_API FName Immediate;
	static ENGINE_API FName EndOfFrameUpdate;
	static ENGINE_API FName BeginInitViews;
	static ENGINE_API FName PostTLASBuild;
};
