// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ContentStreaming.h: Definitions of classes used for content streaming.
=============================================================================*/

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "RenderedTextureStats.h"
#include "Serialization/BulkData.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FSoundSource;
class IPrimitiveComponent;
class UPrimitiveComponent;
class FCanvas;
class FViewport;
class FSoundWaveData;
class FSoundWaveProxy;
class ICompressedAudioInfo;
class UTexture2D;
struct FRenderAssetStreamingManager;
struct FWaveInstance;
class UAnimStreamable;
enum class EStreamableRenderAssetType : uint8;
struct FCompressedAnimSequence;
class UStreamableSparseVolumeTexture;

using FSoundWavePtr = TSharedPtr<FSoundWaveData, ESPMode::ThreadSafe>;
using FSoundWaveProxyPtr = TSharedPtr<FSoundWaveProxy, ESPMode::ThreadSafe>;

/*-----------------------------------------------------------------------------
	Stats.
-----------------------------------------------------------------------------*/

// Forward declarations
class UPrimitiveComponent;
class AActor;
class UTexture2D;
class UStaticMesh;
class USkeletalMesh;
class UStreamableRenderAsset;
class FRenderAssetUpdate;
class FSoundSource;
class FAudioStreamCacheMemoryHandle;
struct FWaveInstance;
struct FRenderAssetStreamingManager;

namespace Nanite
{
	class FCoarseMeshStreamingManager;
}

/** Helper function to flush resource streaming. */
void FlushResourceStreaming();

/*-----------------------------------------------------------------------------
	Base streaming classes.
-----------------------------------------------------------------------------*/

enum ERemoveStreamingViews
{
	/** Removes normal views, but leaves override views. */
	RemoveStreamingViews_Normal,
	/** Removes all views. */
	RemoveStreamingViews_All
};

/**
 * Helper structure containing all relevant information for streaming.
 */
struct FStreamingViewInfo
{
	FStreamingViewInfo( const FVector& InViewOrigin, float InScreenSize, float InFOVScreenSize, float InBoostFactor, bool bInOverrideLocation, float InDuration, TWeakObjectPtr<AActor> InActorToBoost, TWeakObjectPtr<UWorld> InWorld)
	:	ViewOrigin( InViewOrigin )
	,	ScreenSize( InScreenSize )
	,	FOVScreenSize( InFOVScreenSize )
	,	BoostFactor( InBoostFactor )
	,	Duration( InDuration )
	,	bOverrideLocation( bInOverrideLocation )
	,	ActorToBoost( InActorToBoost )
	,	World( InWorld )
	{
	}
	/** View origin */
	FVector ViewOrigin;
	/** Screen size, not taking FOV into account */
	float	ScreenSize;
	/** Screen size, taking FOV into account */
	float	FOVScreenSize;
	/** A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa. */
	float	BoostFactor;
	/** How long the streaming system should keep checking this location, in seconds. 0 means just for the next Tick. */
	float	Duration;
	/** Whether this is an override location, which forces the streaming system to ignore all other regular locations */
	bool	bOverrideLocation;
	/** Optional pointer to an actor who's textures should have their streaming priority boosted */
	TWeakObjectPtr<AActor> ActorToBoost;
	/** Optional pointer to the world the view is located into */
	TWeakObjectPtr<UWorld> World;
};

/**
 * This structure allows audio chunk data to be accessed, and guarantees that the chunk in question will not be deleted
 * during it's lifecycle.
 */
class FAudioChunkHandle
{
public:
	ENGINE_API FAudioChunkHandle();
	ENGINE_API FAudioChunkHandle(const FAudioChunkHandle& Other);
	ENGINE_API FAudioChunkHandle(FAudioChunkHandle&& Other);

	ENGINE_API FAudioChunkHandle& operator=(const FAudioChunkHandle& Other);
	ENGINE_API FAudioChunkHandle& operator=(FAudioChunkHandle&& Other);

	ENGINE_API ~FAudioChunkHandle();

	// gets a pointer to the compressed chunk.
	ENGINE_API const uint8* GetData() const;

	// Returns the num bytes pointed to by GetData().
	ENGINE_API uint32 Num() const;

	// Checks whether this points to a valid compressed chunk.
	ENGINE_API bool IsValid() const;

#if WITH_EDITOR
	// If the soundwave has been recompressed, the compressed audio retained by this handle will not be up to date, and this will return true. 
	ENGINE_API bool IsStale() const;
#endif

private:
	// This constructor should only be called by an implementation of IAudioStreamingManager.
	ENGINE_API FAudioChunkHandle(const uint8* InData, uint32 NumBytes, const TSharedRef<const FSoundWaveData>& InSoundWave, const FName& SoundWaveName, uint32 InChunkIndex, uint64 InCacheLookupID);

	const uint8*  CachedData;
	int32 CachedDataNumBytes;

	FName CorrespondingWaveName;
	FGuid CorrespondingWaveGuid;

	// The index of this chunk in the sound wave's full set of chunks of compressed audio.
	int32 ChunkIndex;

#if WITH_EDITOR
	TWeakPtr<const FSoundWaveData, ESPMode::ThreadSafe> CorrespondingWave;
	uint32 ChunkRevision;
#endif

	friend struct IAudioStreamingManager;
	friend struct FCachedAudioStreamingManager;
};

/**
 * Pure virtual base class of a streaming manager.
 */
struct IStreamingManager
{
	IStreamingManager()
		: NumWantingResources(0)
		, NumWantingResourcesCounter(0)
	{
	}

	/** Virtual destructor */
	virtual ~IStreamingManager() = default;

	ENGINE_API static struct FStreamingManagerCollection& Get();

	/** Same as get but could fail if state not allocated or shutdown. */
	ENGINE_API static struct FStreamingManagerCollection* Get_Concurrent();

	ENGINE_API static void Shutdown();

	/** Checks if the streaming manager has already been shut down. **/
	ENGINE_API static bool HasShutdown();

	/**
	 * Calls UpdateResourceStreaming(), and does per-frame cleaning. Call once per frame.
	 *
	 * @param DeltaTime				Time since last call in seconds
	 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
	 */
	ENGINE_API virtual void Tick(float DeltaTime, bool bProcessEverything = false);

	/**
	 * Updates streaming, taking into account all current view infos. Can be called multiple times per frame.
	 *
	 * @param DeltaTime				Time since last call in seconds
	 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
	 */
	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) = 0;

	/**
	 * Streams in/out all resources that wants to and blocks until it's done.
	 *
	 * @param TimeLimit					Maximum number of seconds to wait for streaming I/O. If zero, uses .ini setting
	 * @return							Number of streaming requests still in flight, if the time limit was reached before they were finished.
	 */
	ENGINE_API virtual int32 StreamAllResources(float TimeLimit = 0.0f);
	
	/**
	 * Blocks till all pending requests are fulfilled.
	 *
	 * @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
	 * @param bLogResults	Whether to dump the results to the log.
	 * @return				Number of streaming requests still in flight, if the time limit was reached before they were finished.
	 */
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) = 0;

	/**
	 * Cancels the timed Forced resources (i.e used the Kismet action "Stream In Textures").
	 */
	virtual void CancelForcedResources() = 0;

	/**
	 * Notifies manager of "level" change.
	 */
	virtual void NotifyLevelChange() = 0;

	/**
	 * Removes streaming views from the streaming manager. This is also called by Tick().
	 *
	 * @param RemovalType	What types of views to remove (all or just the normal views)
	 */
	void RemoveStreamingViews(ERemoveStreamingViews RemovalType);

	/**
	 * Adds the passed in view information to the static array.
	 *
	 * @param ScreenSize			Screen size
	 * @param FOVScreenSize			Screen size taking FOV into account
	 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
	 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other regular locations
	 * @param Duration				How long the streaming system should keep checking this location, in seconds. 0 means just for the next Tick.
	 * @param InActorToBoost		Optional pointer to an actor who's textures should have their streaming priority boosted
	 * @param InWorld				The world this view is located in
	 */
	ENGINE_API void AddViewInformation(const FVector& ViewOrigin, float ScreenSize, float FOVScreenSize, float BoostFactor = 1.0f, bool bOverrideLocation = false, float Duration = 0.0f, TWeakObjectPtr<AActor> InActorToBoost = NULL, TWeakObjectPtr<UWorld> InWorld = NULL);

	/**
	 * Queue up view locations to the streaming system. These locations will be added properly at the next call to AddViewInformation,
	 * re-using the screensize and FOV settings.
	 *
	 * @param Location				World-space view origin
	 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
	 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other locations
	 * @param Duration				How long the streaming system should keep checking this location, in seconds. 0 means just for the next Tick.
	 */
	ENGINE_API void AddViewLocation(const FVector& Location, float BoostFactor = 1.0f, bool bOverrideLocation = false, float Duration = 0.0f);

	/** Don't stream world resources for the next NumFrames. */
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) = 0;

	/**
	 * Allows the streaming manager to process exec commands.
	 *
	 * @param InWorld World context
	 * @param Cmd	Exec command
	 * @param Ar	Output device for feedback
	 * @return		true if the command was handled
	 */
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		return false;
	}

	/** Adds a ULevel that has already prepared StreamingData to the streaming manager. */
	virtual void AddLevel(class ULevel* Level) = 0;

	/** Removes a ULevel from the streaming manager. */
	virtual void RemoveLevel(class ULevel* Level) = 0;

	/**
	 * Notifies manager that level primitives were shifted
	 */
	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) = 0;

	/** Called when a spawned actor is destroyed. */
	virtual void NotifyActorDestroyed(AActor* Actor)
	{
	}

	/** Called when a primitive is detached from an actor or another component. */
	virtual void NotifyPrimitiveDetached(const IPrimitiveComponent* Primitive)
	{
	}

	/** Called when a primitive streaming data needs to be updated. */
	virtual void NotifyPrimitiveUpdated(const IPrimitiveComponent* Primitive)
	{
	}

	/**  Called when a primitive streaming data needs to be updated in the last stage of the frame. */
	virtual void NotifyPrimitiveUpdated_Concurrent(const IPrimitiveComponent* Primitive)
	{
	}

	ENGINE_API void NotifyPrimitiveDetached(const UPrimitiveComponent* Primitive);
	ENGINE_API void NotifyPrimitiveUpdated(const UPrimitiveComponent* Primitive);
	ENGINE_API void NotifyPrimitiveUpdated_Concurrent(const UPrimitiveComponent* Primitive);

	/** Returns the number of view infos. */
	int32 GetNumViews() const
	{
		return CurrentViewInfos.Num();
	}

	/** Returns the view info by the specified index. */
	const FStreamingViewInfo& GetViewInformation(int32 ViewIndex) const
	{
		return CurrentViewInfos[ViewIndex];
	}

	/** Returns the number of resources that currently wants to be streamed in. */
	virtual int32 GetNumWantingResources() const
	{
		return NumWantingResources;
	}

	/**
	 * Returns the current ID for GetNumWantingResources().
	 * The ID is incremented every time NumWantingResources is updated by the streaming system (every few frames).
	 * Can be used to verify that any changes have been fully examined, by comparing current ID with
	 * what it was when the changes were made.
	 */
	virtual int32 GetNumWantingResourcesID() const
	{
		return NumWantingResourcesCounter;
	}

	/** Propagates a change to the active lighting scenario. */
	virtual void PropagateLightingScenarioChange()
	{
	}

#if WITH_EDITOR
	virtual void OnAudioStreamingParamsChanged() {};
#endif

protected:

	/**
	 * Sets up the CurrentViewInfos array based on PendingViewInfos, LastingViewInfos and SecondaryLocations.
	 * Removes out-dated LastingViewInfos.
	 *
	 * @param DeltaTime		Time since last call in seconds
	 */
	void SetupViewInfos( float DeltaTime );

	/**
	 * Adds the passed in view information to the static array.
	 *
	 * @param ViewInfos				Array to add the view to
	 * @param ViewOrigin			View origin
	 * @param ScreenSize			Screen size
	 * @param FOVScreenSize			Screen size taking FOV into account
	 * @param BoostFactor			A factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa.
	 * @param bOverrideLocation		Whether this is an override location, which forces the streaming system to ignore all other regular locations
	 * @param Duration				How long the streaming system should keep checking this location (in seconds). 0 means just for the next Tick.
	 * @param InActorToBoost		Optional pointer to an actor who's textures should have their streaming priority boosted
	 * @param InWorld				The world this view is located in
	 */
	static void AddViewInfoToArray( TArray<FStreamingViewInfo> &ViewInfos, const FVector& ViewOrigin, float ScreenSize, float FOVScreenSize, float BoostFactor, bool bOverrideLocation, float Duration, TWeakObjectPtr<AActor> InActorToBoost, TWeakObjectPtr<UWorld> InWorld);

	/**
	 * Remove view infos with the same location from the given array.
	 *
	 * @param ViewInfos				[in/out] Array to remove the view from
	 * @param ViewOrigin			View origin
	 */
	static void RemoveViewInfoFromArray( TArray<FStreamingViewInfo> &ViewInfos, const FVector& ViewOrigin );

	struct FSecondaryLocation
	{
		FSecondaryLocation( const FVector& InLocation, float InBoostFactor, bool bInOverrideLocation, float InDuration )
		:	Location( InLocation )
		,	BoostFactor( InBoostFactor )
		,	Duration( InDuration )
		,	bOverrideLocation( bInOverrideLocation )
		{
		}
		/** A location to use for distance-based heuristics next Tick(). */
		FVector		Location;
		/** A boost factor that affects all streaming distances for this location. 1.0f is default. Higher means higher-resolution textures and vice versa. */
		float		BoostFactor;
		/** How long the streaming system should keep checking this location (in seconds). 0 means just for the next Tick. */
		float		Duration;
		/** Whether this is an override location, which forces the streaming system to ignore all other locations */
		bool		bOverrideLocation;
	};

	/** Current collection of views that need to be taken into account for streaming. Emptied every frame. */
	ENGINE_API static TArray<FStreamingViewInfo> CurrentViewInfos;

	/** Pending views. Emptied every frame. */
	static TArray<FStreamingViewInfo> PendingViewInfos;

	/** Views that stick around for a while. Override views are ignored if no movie is playing. */
	static TArray<FStreamingViewInfo> LastingViewInfos;

	/** Collection of view locations that will be added at the next call to AddViewInformation. */
	static TArray<FSecondaryLocation> SecondaryLocations;

	/** Set when Tick() has been called. The first time a new view is added, it will clear out all old views. */
	static bool bPendingRemoveViews;

	/** Number of resources that currently wants to be streamed in. */
	int32		NumWantingResources;

	/**
	 * The current counter for NumWantingResources.
	 * This counter is bumped every time NumWantingResources is updated by the streaming system (every few frames).
	 * Can be used to verify that any changes have been fully examined, by comparing current counter with
	 * what it was when the changes were made.
	 */
	int32		NumWantingResourcesCounter;
};

/**
 * Interface to add functions specifically related to texture/mesh streaming
 */
struct IRenderAssetStreamingManager : public IStreamingManager
{
	/**
	* Updates streaming for an individual texture/mesh, taking into account all view infos.
	*
	* @param RenderAsset		Texture or mesh to update
	*/
	virtual void UpdateIndividualRenderAsset(UStreamableRenderAsset* RenderAsset) = 0;

	/** Stream in non-resident mips for an asset ASAP. Returns true if streaming request will be successful. */
	virtual bool FastForceFullyResident(UStreamableRenderAsset* RenderAsset) = 0;

	/**
	* Temporarily boosts the streaming distance factor by the specified number.
	* This factor is automatically reset to 1.0 after it's been used for mip-calculations.
	*/
	virtual void BoostTextures(AActor* Actor, float BoostFactor) = 0;

	/**
	*	Try to stream out texture/mesh mip-levels to free up more memory.
	*	@param RequiredMemorySize	- Required minimum available texture memory
	*	@return						- Whether it succeeded or not
	**/
	virtual bool StreamOutRenderAssetData(int64 RequiredMemorySize) = 0;

	/** Adds a new texture/mesh to the streaming manager. */
	virtual void AddStreamingRenderAsset(UStreamableRenderAsset* RenderAsset) = 0;

	/** Removes a texture/mesh from the streaming manager. */
	virtual void RemoveStreamingRenderAsset(UStreamableRenderAsset* RenderAsset) = 0;

	/** Check whether all runtime-allowed LODs have been loaded. */
	virtual bool IsFullyStreamedIn(UStreamableRenderAsset* RenderAsset) = 0;

	/**
	 * Try to register an asset as protected from stream-out. Succeeds only if the asset
	 * currently has no pending streaming operation. While registered, the streaming
	 * manager will not initiate a StreamOut on this asset. StreamIn is NOT gated.
	 *
	 * Caller must call UnregisterStreamOutProtectedAsset(), the streaming manager also
	 * drops any registration when the asset is removed from streaming (BeginDestroy path).
	 */
	virtual bool TryRegisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset) = 0;
	virtual void UnregisterStreamOutProtectedAsset(UStreamableRenderAsset* Asset) = 0;

	virtual int64 GetMemoryOverBudget() const = 0;

	/** Pool size for streaming. */
	virtual int64 GetPoolSize() const = 0;

	/** Estimated memory in bytes the streamer would use if there was no limit */
	virtual int64 GetRequiredPoolSize() const = 0;

	/** Max required textures/meshes ever seen in bytes. */
	virtual int64 GetMaxEverRequired() const = 0;

	/** Amount of memory cached in pool */
	virtual float GetCachedMips() const = 0;

	/** Resets the max ever required textures/meshes.  For possibly when changing resolutions or screen pct. */
	virtual void ResetMaxEverRequired() = 0;

	/** Set current pause state for texture/mesh streaming */
	virtual void PauseRenderAssetStreaming(bool bInShouldPause) = 0;

	/** Return all bounds related to the ref object */
	virtual void GetObjectReferenceBounds(const UObject* RefObject, TArray<FBox>& AssetBoxes) = 0;

	constexpr static auto ShouldChooseDefault = [](const IPrimitiveComponent*) { return true; };

	/** Return all components referencing the asset */
	virtual void GetAssetComponents(
		const UStreamableRenderAsset* RenderAsset,
		TArray<const IPrimitiveComponent*>& OutComps,
		TFunction<bool(const IPrimitiveComponent*)> ShouldChoose = +ShouldChooseDefault) = 0;

	//BEGIN: APIs for backward compatibility
	ENGINE_API void UpdateIndividualTexture(UTexture2D* Texture);
	ENGINE_API bool StreamOutTextureData(int64 RequiredMemorySize);
	ENGINE_API void AddStreamingTexture(UTexture2D* Texture);
	ENGINE_API void RemoveStreamingTexture(UTexture2D* Texture);
	ENGINE_API void PauseTextureStreaming(bool bInShouldPause);
	//END: APIs for backward compatibility

	/** Notify the streamer that the mounted state of a file needs to be re-evaluated. */
	virtual void MarkMountedStateDirty(FIoFilenameHash FilenameHash) = 0;

	virtual void AddRenderedTextureStats(TMap<FString, FRenderedTextureStats>& InOutRenderedTextureAssets) = 0;

	virtual void OnRenderAssetUpdateAbandoned(UStreamableRenderAsset* OwningRenderAsset, TRefCountPtr<FRenderAssetUpdate>& RenderAssetUpdate) = 0;

	/* Schedules file recache */
	virtual void InvalidateFileCache() = 0;
};

enum class EAudioChunkLoadResult : uint8
{
	Completed,
	AlreadyLoaded,
	Interrupted,
	ChunkOutOfBounds,
	CacheBlown
};

/**
 * Interface to add functions specifically related to audio streaming
 */
struct IAudioStreamingManager : public IStreamingManager
{
	/** Adds the memory usage of the force inline sound to the streaming cache budget */
	virtual void AddForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWaveData) { };
	
	/** Removes the memory usage of the force inline sound from the streaming cache budget */
	virtual void RemoveForceInlineSoundWave(const TSharedRef<const FSoundWaveData>& SoundWaveData) { };
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	virtual void AddForceInlineSoundWave(const TSharedPtr<FSoundWaveProxy>& SoundWave) { };

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	virtual void RemoveForceInlineSoundWave(const TSharedPtr<FSoundWaveProxy>& SoundWave) { };
	
	/** 
	 * Manually prepare a chunk to start playing back. This should only be used when the Load On Demand feature is enabled, and returns false on failure. t
	 * @param SoundWave SoundWave we would like to request a chunk of.
	 * @param ChunkIndex the index of that soundwave we'd like to request a chunk of.
	 * @param OnLoadCompleted optional callback when the load completes.
	 * @param ThreadToCallOnLoadCompleteOn. Optional specifier for which thread OnLoadCompleted should be called on.
	 * @param bForImmediatePlayback if true, this will optionally reprioritize this chunk's load request.
	 */
	constexpr static auto LoadCompletedDefault = [](EAudioChunkLoadResult) {};
	virtual bool RequestChunk(const TSharedRef<const FSoundWaveData>& SoundWave, uint32 ChunkIndex, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted = LoadCompletedDefault, ENamedThreads::Type ThreadToCallOnLoadCompletedOn = ENamedThreads::AnyThread, bool bForImmediatePlayback = false);
	
	UE_DEPRECATED(5.8, "Please use the const TSharedRef<const FSoundWaveData>& overload instead, that will become pure virtual when this function is deleted")
	virtual bool RequestChunk(const TSharedPtr<const FSoundWaveProxy>& SoundWave, uint32 ChunkIndex, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted = LoadCompletedDefault, ENamedThreads::Type ThreadToCallOnLoadCompletedOn = ENamedThreads::AnyThread, bool bForImmediatePlayback = false) { return false; }
	

	/**
	 * Gets a pointer to a chunk of audio data
	 *
	 * @param SoundWave		SoundWave we want a chunk from
	 * @param ChunkIndex	Index of the chunk we want
	 * @param bBlockForLoad if true, will block this thread until we finish loading this chunk.
	 * @param bForImmediatePlayback if true, will optionally reprioritize this chunk's load request. See au.streamcaching.PlaybackRequestPriority.
	 * @return a handle to the loaded chunk. Can return a default constructed FAudioChunkHandle if the chunk is not loaded yet.
	 */
	virtual FAudioChunkHandle GetLoadedChunk(const TSharedRef<const FSoundWaveData>& SoundWave, uint32 ChunkIndex,  bool bBlockForLoad = false, bool bForImmediatePlayback = false) const;
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData> overload instead")
	virtual FAudioChunkHandle GetLoadedChunk(const TSharedPtr<const FSoundWaveProxy>&  SoundWave, uint32 ChunkIndex,  bool bBlockForLoad = false, bool bForImmediatePlayback = false) const { return {}; }

	/**
	 * This will start evicting elements from the cache until either hit our target of bytes or run out of chunks we can free.
	 *
	 * @param NumBytesToFree	The amount of memory we would like to free, in bytes.
	 * @return the amount of bytes we managed to free.
	 */
	virtual uint64 TrimMemory(uint64 NumBytesToFree) = 0;

	/**
	 * Used for rendering debug info:
	 */
	virtual int32 RenderStatAudioStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) = 0;

	/**
	 * Generate a memory report as a formatted string for this streaming manager.
	 */
	virtual FString GenerateMemoryReport() = 0;

	/**
	 * Whether to toggle a performance intensive profiling mode the streaming manager.
	 */
	virtual void SetProfilingMode(bool bEnabled) = 0;

	/**
	 * In the event that the decoder playing back the chunks is in a starvation state due
	 * to an outstanding chunk load, handle that here.
	 */
	virtual void HandleStarvation(const TSharedRef<const FSoundWaveData>& SoundWaveData, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected ) {}

	UE_DEPRECATED(5.8, "Please use the const TSharedRef<const FSoundWaveData>& overload instead")
	virtual void HandleStarvation(const TSharedPtr<FSoundWaveProxy>& SoundWave, const uint32 ChunkIndex, const uint32 InBytesProduced, const uint32 InBytesExpected );

protected:
	friend FAudioChunkHandle;
	friend FAudioStreamCacheMemoryHandle;

	/** This can be called by implementers of IAudioStreamingManager to construct an FAudioChunkHandle using an otherwise inaccessible constructor. */
	static FAudioChunkHandle BuildChunkHandle(const uint8* InData, uint32 NumBytes, const TSharedRef<const FSoundWaveData>& InSoundWaveData, const FName& SoundWaveName, uint32 InChunkIndex, uint64 CacheLookupID);
	
	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveData>& overload instead")
	static FAudioChunkHandle BuildChunkHandle(const uint8* InData, uint32 NumBytes, const TSharedPtr<FSoundWaveProxy>& InSoundWave, const FName& SoundWaveName, uint32 InChunkIndex, uint64 CacheLookupID);

	/**
	 * This can be used to increment reference counted handles to audio chunks. Called by the copy constructor of FAudioChunkHandle.
	 */
	virtual void AddReferenceToChunk(const FAudioChunkHandle& InHandle) = 0;

	/**
	 * This can be used to decrement reference counted handles to audio chunks. Called by the destructor of FAudioChunkHandle.
	 */
	virtual void RemoveReferenceToChunk(const FAudioChunkHandle& InHandle) = 0;

	/**
     * This can be used to increase the memory count for external features. Called by FAudioStreamCacheMemoryHandle.
     * The pattern for _changing_ the amount of memory of an already added feature is to first remove and then add again with the new number
     */
	virtual void AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) { };

	/**
	* This can be used to decrease the memory count for external features. Called by FAudioStreamCacheMemoryHandle.
	*/
	virtual void RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) { };

// DEPRECATED VIRTUAL INTERFACE
private:
	static void LogWarning();
};

/**
 * Dummy audio streaming manager used on the servers and whenever we cannot render audio
 */
struct FDummyAudioStreamingManager final : public IAudioStreamingManager
{
	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override {}
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override { return 0; }
	virtual void CancelForcedResources() override {}
	virtual void NotifyLevelChange() override {}
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) {}
	virtual void AddLevel(class ULevel* Level) {}
	virtual void RemoveLevel(class ULevel* Level) {}
	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) {}

	virtual void AddMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) override {}
	virtual void RemoveMemoryCountedFeature(const FAudioStreamCacheMemoryHandle& Feature) override {}
	virtual bool RequestChunk(const TSharedRef<const FSoundWaveData>& SoundWaveData, uint32 ChunkIndex, TFunction<void(EAudioChunkLoadResult)> OnLoadCompleted = LoadCompletedDefault, ENamedThreads::Type ThreadToCallOnLoadCompletedOn = ENamedThreads::AnyThread, bool bForImmediatePlayback = false) override { return false; }
	virtual FAudioChunkHandle GetLoadedChunk(const TSharedRef<const FSoundWaveData>& SoundWave, uint32 ChunkIndex,  bool bBlockForLoad = false, bool bForImmediatePlayback = false) const override { return FAudioChunkHandle(); }
	virtual uint64 TrimMemory(uint64 NumBytesToFree) override { return 0; }
	virtual int32 RenderStatAudioStreaming(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation) override { return 0; }
	virtual FString GenerateMemoryReport() override { return TEXT(""); }
	virtual void SetProfilingMode(bool bEnabled) override {}

protected:
	virtual void AddReferenceToChunk(const FAudioChunkHandle& InHandle) override {}
	virtual void RemoveReferenceToChunk(const FAudioChunkHandle& InHandle) override {}
};

/**
 * Interface to add functions specifically related to animation streaming
 */
struct IAnimationStreamingManager : public IStreamingManager
{
	/** Adds a new Streamable Anim to the streaming manager. */
	virtual void AddStreamingAnim(UAnimStreamable* Anim) = 0;

	/** Removes a Streamable Anim from the streaming manager. */
	virtual bool RemoveStreamingAnim(UAnimStreamable* Anim) = 0;

	/** Returns the memory a Streamable Anim is currently using */
	virtual SIZE_T GetMemorySizeForAnim(const UAnimStreamable* Anim) = 0;

	/**
	 * Gets a pointer to a chunk of animation data
	 *
	 * @param Anim			AnimStreamable we want a chunk from
	 * @param ChunkIndex	Index of the chunk we want
	 * @return Either the desired chunk or NULL if it's not loaded
	 */
	virtual const FCompressedAnimSequence* GetLoadedChunk(const UAnimStreamable* Anim, uint32 ChunkIndex, bool bRequestNextChunk) const = 0;
};

/**
 * Streaming manager collection, routing function calls to streaming managers that have been added
 * via AddStreamingManager.
 */
struct FStreamingManagerCollection : public IStreamingManager
{
	/** Default constructor, initializing all member variables. */
	ENGINE_API FStreamingManagerCollection();
	ENGINE_API ~FStreamingManagerCollection();

	/**
	 * Calls UpdateResourceStreaming(), and does per-frame cleaning. Call once per frame.
	 *
	 * @param DeltaTime				Time since last call in seconds
	 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
	 */
	ENGINE_API virtual void Tick( float DeltaTime, bool bProcessEverything=false ) override;

	/**
	 * Updates streaming, taking into account all current view infos. Can be called multiple times per frame.
	 *
	 * @param DeltaTime				Time since last call in seconds
	 * @param bProcessEverything	[opt] If true, process all resources with no throttling limits
	 */
	virtual void UpdateResourceStreaming( float DeltaTime, bool bProcessEverything=false ) override;

	/**
	 * Streams in/out all resources that wants to and blocks until it's done.
	 *
	 * @param TimeLimit					Maximum number of seconds to wait for streaming I/O. If zero, uses .ini setting
	 * @return							Number of streaming requests still in flight, if the time limit was reached before they were finished.
	 */
	virtual int32 StreamAllResources( float TimeLimit=0.0f ) override;

	/**
	 * Blocks till all pending requests are fulfilled.
	 *
	 * @param TimeLimit		Optional time limit for processing, in seconds. Specifying 0 means infinite time limit.
	 * @param bLogResults	Whether to dump the results to the log.
	 * @return				Number of streaming requests still in flight, if the time limit was reached before they were finished.
	 */
	virtual int32 BlockTillAllRequestsFinished( float TimeLimit = 0.0f, bool bLogResults = false ) override;

	/** Returns the number of resources that currently wants to be streamed in. */
	virtual int32 GetNumWantingResources() const override;

	/**
	 * Returns the current ID for GetNumWantingResources().
	 * The ID is bumped every time NumWantingResources is updated by the streaming system (every few frames).
	 * Can be used to verify that any changes have been fully examined, by comparing current ID with
	 * what it was when the changes were made.
	 */
	virtual int32 GetNumWantingResourcesID() const override;

	/**
	 * Cancels the timed Forced resources (i.e used the Kismet action "Stream In Textures").
	 */
	virtual void CancelForcedResources() override;

	/**
	 * Notifies manager of "level" change.
	 */
	virtual void NotifyLevelChange() override;

	/**
	 * Checks whether any kind of streaming is active
	 */
	ENGINE_API bool IsStreamingEnabled() const;

	/**
	 * Checks whether texture streaming is enabled. 
	 */
	ENGINE_API bool IsTextureStreamingEnabled() const;

	/**
	 * Checks whether texture/mesh streaming is enabled
	 */
	ENGINE_API bool IsRenderAssetStreamingEnabled(EStreamableRenderAssetType FilteredAssetType) const;

	/**
	 * Gets a reference to the Texture Streaming Manager interface
	 */
	ENGINE_API IRenderAssetStreamingManager& GetTextureStreamingManager() const;

	/**
	 * Get the streaming manager for textures and meshes
	 */
	ENGINE_API IRenderAssetStreamingManager& GetRenderAssetStreamingManager() const;

	/**
	 * Gets a reference to the Audio Streaming Manager interface
	 */
	ENGINE_API IAudioStreamingManager& GetAudioStreamingManager() const;

	/**
	 * Gets a reference to the Animation Streaming Manager interface
	 */
	ENGINE_API IAnimationStreamingManager& GetAnimationStreamingManager() const;

	/**
	 * Gets a reference to the Virtual Texture Streaming Manager
	*/
	ENGINE_API struct FVirtualTextureChunkStreamingManager& GetVirtualTextureStreamingManager() const;

	/**
	 * Gets a reference to the Nanite Coarse Mesh Streaming Manager
	*/
	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated")
	ENGINE_API Nanite::FCoarseMeshStreamingManager* GetNaniteCoarseMeshStreamingManager() const;

	/**
	 * Adds a streaming manager to the array of managers to route function calls to.
	 *
	 * @param StreamingManager	Streaming manager to add
	 */
	ENGINE_API void AddStreamingManager( IStreamingManager* StreamingManager );

	/**
	 * Removes a streaming manager from the array of managers to route function calls to.
	 *
	 * @param StreamingManager	Streaming manager to remove
	 */
	ENGINE_API void RemoveStreamingManager( IStreamingManager* StreamingManager );

	/**
	 * Sets the number of iterations to use for the next time UpdateResourceStreaming is being called. This 
	 * is being reset to 1 afterwards.
	 *
	 * @param NumIterations	Number of iterations to perform the next time UpdateResourceStreaming is being called.
	 */
	void SetNumIterationsForNextFrame( int32 NumIterations );

	/** Don't stream world resources for the next NumFrames. */
	virtual void SetDisregardWorldResourcesForFrames( int32 NumFrames ) override;

	/**
	 * Disables resource streaming. Enable with EnableResourceStreaming. Disable/enable can be called multiple times nested
	 */
	void DisableResourceStreaming();

	/**
	 * Enables resource streaming, previously disabled with enableResourceStreaming. Disable/enable can be called multiple times nested
	 * (this will only actually enable when all disables are matched with enables)
	 */
	void EnableResourceStreaming();

	/**
	 * Allows the streaming manager to process exec commands.
	 *
	 * @param InWorld World context
	 * @param Cmd	Exec command
	 * @param Ar	Output device for feedback
	 * @return		true if the command was handled
	 */
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override;

	/** Adds a ULevel to the streaming manager. */
	virtual void AddLevel( class ULevel* Level ) override;

	/** Removes a ULevel from the streaming manager. */
	virtual void RemoveLevel( class ULevel* Level ) override;
	
	/* Notifies manager that level primitives were shifted. */
	virtual void NotifyLevelOffset( class ULevel* Level, const FVector& Offset ) override;

	/** Called when a spawned actor is destroyed. */
	virtual void NotifyActorDestroyed( AActor* Actor ) override;

	/** Called when a primitive is detached from an actor or another component. */
	virtual void NotifyPrimitiveDetached( const IPrimitiveComponent* Primitive ) override;

	/** Called when a primitive streaming data needs to be updated. */
	virtual void NotifyPrimitiveUpdated( const IPrimitiveComponent* Primitive ) override;

	/**  Called when a primitive streaming data needs to be updated in the last stage of the frame. */
	virtual void NotifyPrimitiveUpdated_Concurrent( const IPrimitiveComponent* Primitive ) override;

	using IStreamingManager::NotifyPrimitiveDetached;
	using IStreamingManager::NotifyPrimitiveUpdated;
	using IStreamingManager::NotifyPrimitiveUpdated_Concurrent;

	/** Propagates a change to the active lighting scenario. */
	void PropagateLightingScenarioChange() override;

#if WITH_EDITOR
	virtual void OnAudioStreamingParamsChanged() override;
#endif

protected:

	virtual void AddOrRemoveTextureStreamingManagerIfNeeded(bool bIsInit=false);

	/** Array of streaming managers to route function calls to */
	TArray<IStreamingManager*> StreamingManagers;
	/** Number of iterations to perform. Gets reset to 1 each frame. */
	int32 NumIterations;

	/** Count of how many nested DisableResourceStreaming's were called - will enable when this is 0 */
	int32 DisableResourceStreamingCount;

	/** Maximum number of seconds to block in StreamAllResources(), by default (.ini setting). */
	float LoadMapTimeLimit;

	/** The currently added texture streaming manager. Can be NULL*/
	FRenderAssetStreamingManager* RenderAssetStreamingManager;

	/** The audio streaming manager, should always exist */
	IAudioStreamingManager* AudioStreamingManager;

	/** The animation streaming manager, should always exist */
	IAnimationStreamingManager* AnimationStreamingManager;

	/** The virtual texture streaming manager, should always exist */
	FVirtualTextureChunkStreamingManager* VirtualTextureStreamingManager;

	/** The nanite coarse mesh streaming manager, should always exist */
	UE_DEPRECATED(5.8, "Coarse Mesh Streaming is deprecated")
	Nanite::FCoarseMeshStreamingManager* NaniteCoarseMeshStreamingManager;

#if WITH_EDITOR
	// Locks out any audio streaming manager call when we are re-initializing the audio streaming manager.
	mutable FCriticalSection AudioStreamingManagerCriticalSection;
#endif
};
