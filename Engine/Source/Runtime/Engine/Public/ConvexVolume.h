// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/*=============================================================================
	ConvexVolume.h: Convex volume definitions.
=============================================================================*/

/**
 * Encapsulates the inside and/or outside state of an intersection test.
 */
struct FOutcode
{
private:
	bool bInside;
	bool bOutside;

public:

	// Constructor.

	FOutcode():
		bInside(false), bOutside(false)
	{}
	FOutcode(bool bInInside,bool bInOutside):
		bInside(bInInside), bOutside(bInOutside)
	{}

	// Accessors.

	void SetInside(bool bNewInside) { bInside = bNewInside; }
	void SetOutside(bool bNewOutside) { bOutside = bNewOutside; }
	bool GetInside() const { return bInside; }
	bool GetOutside() const { return bOutside; }
};

//
//	FConvexVolume
//

struct FConvexVolume
{
public:

	typedef TArray<FPlane,TInlineAllocator<6> > FPlaneArray;
	typedef TArray<FPlane,TInlineAllocator<8> > FPermutedPlaneArray;

	FPlaneArray Planes;
	/** This is the set of planes pre-permuted to SSE/Altivec form */
	FPermutedPlaneArray PermutedPlanes;

	FConvexVolume()
	{
//		int32 N = 5;
	}

	/**
	 * Builds the set of planes used to clip against. Also, puts the planes
	 * into a form more readily used by SSE/Altivec so 4 planes can be
	 * clipped against at once.
	 */
	FConvexVolume(const TArray<FPlane, TInlineAllocator<6>>& InPlanes) :
		Planes( InPlanes )
	{
		Init();
	}

	/**
	 * Builds the permuted planes for SSE/Altivec fast clipping
	 */
	ENGINE_API void Init(void);

	/**
	 * Clips a polygon to the volume.
	 *
	 * @param	Polygon - The polygon to be clipped.  If the true is returned, contains the
	 *			clipped polygon.
	 *
	 * @return	Returns false if the polygon is entirely outside the volume and true otherwise.
	 */
	ENGINE_API bool ClipPolygon(class FPoly& Polygon) const;

	// Intersection tests.

	ENGINE_API FOutcode GetBoxIntersectionOutcode(const FVector& Origin,const FVector& Extent) const;

    /**
     * Intersection test with a translated axis-aligned box.
     * @param Origin of the box.
     * @param Translation -to apply to the box.
     * @param Extent of the box along each axis.
     * @returns true if this convex volume intersects the given translated box.
     */
	ENGINE_API bool IntersectBox(const FVector& Origin,const FVector& Extent) const;

    /**
     * Intersection test with a translated axis-aligned box.
     * @param Origin of the box.
     * @param Translation -to apply to the box.
     * @param Extent of the box along each axis.
	 * param bOutFullyContained to know if the box was fully contained 
     * @returns true if this convex volume intersects the given translated box.
     */
	ENGINE_API bool IntersectBox(const FVector& Origin,const FVector& Extent, bool& bOutFullyContained) const;

	/**
     * Intersection test with a sphere
     * @param Origin of the sphere.
     * @param Radius of the sphere.
     * @returns true if this convex volume intersects the given sphere (the result is conservative at the corners)
     */
	ENGINE_API bool IntersectSphere(const FVector& Origin,const float& Radius) const;

	/**
     * Intersection test with a sphere
     * @param Origin of the sphere.
     * @param Radius of the sphere.
	 * param bOutFullyContained to know if the sphere was fully contained 
     * @returns true if this convex volume intersects the given sphere (the result is conservative at the corners)
     */
	ENGINE_API bool IntersectSphere(const FVector& Origin,const float& Radius, bool& bOutFullyContained) const;

	/**
     * Intersection test with a triangle
	 * param bOutFullyContained to know if the triangle was fully contained 
     * @returns true if this convex volume intersects the given triangle.
     */
	ENGINE_API bool IntersectTriangle(const FVector& PointA, const FVector& PointB, const FVector& PointC, bool& bOutFullyContained) const;

	/**
     * Intersection test with line segment
     * @param Start of the segment.
     * @param End of the segment.
	 * param bOutFullyContained to know if the sphere was fully contained 
     * @returns true if this convex volume intersects the given line segment
     */
	ENGINE_API bool IntersectLineSegment(const FVector& Start, const FVector& End) const;

	/**
	 * Calculates the maximum perpendicular distance of a point to the plains of the convex volume.
	 * The distance can be used to see if the sphere is touching the volume
	 * @param 	Point to calculate the distance to.
	 * @return 	Returns the maximum perpendicular distance to then plains (the distance is conservative at the corners)
	 */
	ENGINE_API float DistanceTo(const FVector& Point) const;

	/**
	 * Intersection test with a translated axis-aligned box.
	 * @param Origin - Origin of the box.
	 * @param Translation - Translation to apply to the box.
	 * @param Extent - Extent of the box along each axis.
	 * @returns true if this convex volume intersects the given translated box.
	 */
	ENGINE_API bool IntersectBox(const FVector& Origin,const FVector& Translation,const FVector& Extent) const;

	/** 
	 * Determines whether the given point lies inside the convex volume
	 * @param Point to test against.
	 * @returns true if the point is inside the convex volume.
	 */
	bool IntersectPoint(const FVector& Point) const
	{
		return IntersectSphere(Point, 0.0f);
	}

	/**
	 * Serializer
	 *
	 * @param	Ar				Archive to serialize data to
	 * @param	ConvexVolume	Convex volumes to serialize to archive
	 *
	 * @return passed in archive
	 */
	friend ENGINE_API FArchive& operator<<(FArchive& Ar,FConvexVolume& ConvexVolume);
};

/**
 * Creates a convex volume bounding the view frustum for a view-projection matrix.
 *
 * @param [out]	The FConvexVolume which contains the view frustum bounds.
 * @param		ViewProjectionMatrix - The view-projection matrix which defines the view frustum.
 * @param		bUseNearPlane - True if the convex volume should be bounded by the view frustum's near clipping plane.
 */
extern ENGINE_API void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, bool bUseNearPlane);

/**
 * Creates a convex volume bounding the view frustum for a view-projection matrix.
 *
 * @param [out]	The FConvexVolume which contains the view frustum bounds.
 * @param		ViewProjectionMatrix - The view-projection matrix which defines the view frustum.
 * @param		bUseNearPlane - True if the convex volume should be bounded by the view frustum's near clipping plane.
 * @param		bUseNearPlane - True if the convex volume should be bounded by the view frustum's far clipping plane.
 */
extern ENGINE_API void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, bool bUseNearPlane, bool bUseFarPlane);

/**
 * Creates a convex volume bounding the view frustum for a view-projection matrix, with an optional far plane override.
 *
 * @param [out]	The FConvexVolume which contains the view frustum bounds.
 * @param		ViewProjectionMatrix - The view-projection matrix which defines the view frustum.
 * @param		InFarPlane - Plane to use if bOverrideFarPlane is true.
 * @param		bOverrideFarPlane - Whether to override the far plane.
 * @param		bUseNearPlane - True if the convex volume should be bounded by the view frustum's near clipping plane.
 */
extern ENGINE_API void GetViewFrustumBounds(FConvexVolume& OutResult, const FMatrix& ViewProjectionMatrix, const FPlane& InFarPlane, bool bOverrideFarPlane, bool bUseNearPlane);
