// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CubemapUnwapUtils.h: Pixel and Vertex shader to render a cube map as 2D texture
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "BatchedElements.h"
#include "GlobalShader.h"

class UTextureCube;
class UTextureRenderTargetCube;

namespace CubemapHelpers
{
	/**
	* Creates an unwrapped 2D image of the cube map ( longitude/latitude )
	* @param	CubeTexture	Source UTextureCube object.
	* @param	BitsOUT	Raw bits of the 2D image bitmap.
	* @param	SizeXOUT	Filled with the X dimension of the output bitmap.
	* @param	SizeYOUT	Filled with the Y dimension of the output bitmap.
	* @param	FormatOUT	Filled with the pixel format of the output bitmap.
	* @return	true on success.
	*/
	ENGINE_API bool GenerateLongLatUnwrap(const UTextureCube* CubeTexture, TArray64<uint8>& BitsOUT, FIntPoint& SizeOUT, EPixelFormat& FormatOUT);

	/**
	* Creates an unwrapped 2D image of the cube map ( longitude/latitude )
	* @param	CubeTarget	Source UTextureRenderTargetCube object.
	* @param	BitsOUT	Raw bits of the 2D image bitmap.
	* @param	SizeXOUT	Filled with the X dimension of the output bitmap.
	* @param	SizeYOUT	Filled with the Y dimension of the output bitmap.
	* @param	FormatOUT	Filled with the pixel format of the output bitmap.
	* @return	true on success.
	*/
	ENGINE_API bool GenerateLongLatUnwrap(const UTextureRenderTargetCube* CubeTarget, TArray64<uint8>& BitsOUT, FIntPoint& SizeOUT, EPixelFormat& FormatOUT);
}

/**
 * A vertex shader for rendering a texture on a simple element.
 */
class FCubemapTexturePropertiesVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FCubemapTexturePropertiesVS,Global);
public:
	FCubemapTexturePropertiesVS();
	FCubemapTexturePropertiesVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix& TransformValue);

private:
	LAYOUT_FIELD(FShaderParameter, Transform);
};

/**
 * Simple pixel shader reads from a cube map texture and unwraps it in the LongitudeLatitude form.
 */
class FCubemapTexturePropertiesPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FCubemapTexturePropertiesPS,Global);

	class FHDROutput : SHADER_PERMUTATION_BOOL("HDR_OUTPUT");
	class FCubeArray : SHADER_PERMUTATION_BOOL("TEXTURECUBE_ARRAY");
	using FPermutationDomain = TShaderPermutationDomain<FHDROutput, FCubeArray>;

public:
	FCubemapTexturePropertiesPS();
	FCubemapTexturePropertiesPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* InTexture, const FMatrix& InColorWeightsValue, float InMipLevel, float InSliceIndex, bool bInIsTextureCubeArray, const FMatrix44f& InViewMatrix, bool bInShowLongLatUnwrap, float InGammaValue, bool bInUsePointSampling);

private:
	LAYOUT_FIELD(FShaderResourceParameter, CubeTexture);
	LAYOUT_FIELD(FShaderResourceParameter, CubeTextureSampler);
	LAYOUT_FIELD(FShaderParameter, PackedProperties0);
	LAYOUT_FIELD(FShaderParameter, ColorWeights);
	LAYOUT_FIELD(FShaderParameter, Gamma);
	LAYOUT_FIELD(FShaderParameter, NumSlices);
	LAYOUT_FIELD(FShaderParameter, SliceIndex);
	LAYOUT_FIELD(FShaderParameter, ViewMatrix);
};


class FMipLevelBatchedElementParameters : public FBatchedElementParameters
{
public:
	FMipLevelBatchedElementParameters(float InMipLevel, float InSliceIndex, bool bInIsTextureCubeArray, const FMatrix44f& InViewMatrix, bool bInShowLongLatUnwrap, bool bInHDROutput, bool bInUsePointSampling)
		: bHDROutput(bInHDROutput)
		, MipLevel(InMipLevel)
		, SliceIndex(InSliceIndex)
		, ViewMatrix(InViewMatrix)
		, bShowLongLatUnwrap(bInShowLongLatUnwrap)
		, bIsTextureCubeArray(bInIsTextureCubeArray)
		, bUsePointSampling(bInUsePointSampling)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	ENGINE_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	bool bHDROutput;

	/** Parameters that need to be passed to the shader */
	float MipLevel;
	float SliceIndex;
	FMatrix44f ViewMatrix;
	bool bShowLongLatUnwrap;

	/** Parameters that are used to select a shader permutation */
	bool bIsTextureCubeArray;

	/** Whether to use nearest-point sampling when rendering the cubemap */
	bool bUsePointSampling;
};


/**
 * Simple pixel shader that renders a IES light profile for the purposes of visualization.
 */
class FIESLightProfilePS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FIESLightProfilePS,Global);
public:
	FIESLightProfilePS();
	FIESLightProfilePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_IES_PROFILE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_IES_STANDALONE_TEXTURE"), 1);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, float InBrightnessInLumens);

private:
	/** The texture to sample. */
	LAYOUT_FIELD(FShaderResourceParameter, IESTexture);
	LAYOUT_FIELD(FShaderResourceParameter, IESTextureSampler);
	LAYOUT_FIELD(FShaderParameter, BrightnessInLumens);
};

class FIESLightProfileBatchedElementParameters : public FBatchedElementParameters
{
public:
	FIESLightProfileBatchedElementParameters(float InBrightnessInLumens) : BrightnessInLumens(InBrightnessInLumens)
	{
	}

	/** Binds vertex and pixel shaders for this element */
	ENGINE_API virtual void BindShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, ERHIFeatureLevel::Type InFeatureLevel, const FMatrix& InTransform, const float InGamma, const FMatrix& ColorWeights, const FTexture* Texture) override;

private:
	float BrightnessInLumens;
};
