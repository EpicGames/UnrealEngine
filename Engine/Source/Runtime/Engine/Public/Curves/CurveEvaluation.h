// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"

namespace UE
{
	namespace Curves
	{
		/* Solve Cubic Euqation using Cardano's forumla
		* Adopted from Graphic Gems 1
		* https://github.com/erich666/GraphicsGems/blob/master/gems/Roots3And4.c
		*  Solve cubic of form
		*
		* @param Coeff Coefficient parameters of form  Coeff[0] + Coeff[1]*x + Coeff[2]*x^2 + Coeff[3]*x^3 + Coeff[4]*x^4 = 0
		* @param Solution Up to 3 real solutions. We don't include imaginary solutions, would need a complex number objecct
		* @return Returns the number of real solutions returned in the Solution array.
		*/
		ENGINE_API int32 SolveCubic(double Coeff[4], double Solution[3]);

		/**  Assuming that P0, P1, P2 and P3 are sequential control points of an N=4 Bezier curve, returns
		 * the interpolated value for interpolation constant Alpha in [0, 1]
		 */
		template<typename CurveValueType>
		CurveValueType BezierInterp(CurveValueType P0, CurveValueType P1, CurveValueType P2, CurveValueType P3, float Alpha)
		{
			const CurveValueType P01 = FMath::Lerp(P0, P1, Alpha);
			const CurveValueType P12 = FMath::Lerp(P1, P2, Alpha);
			const CurveValueType P23 = FMath::Lerp(P2, P3, Alpha);
			const CurveValueType P012 = FMath::Lerp(P01, P12, Alpha);
			const CurveValueType P123 = FMath::Lerp(P12, P23, Alpha);
			const CurveValueType P0123 = FMath::Lerp(P012, P123, Alpha);

			return P0123;
		}

		/*
		*   Convert the control values for a polynomial defined in the Bezier
		*		basis to a polynomial defined in the power basis (t^3 t^2 t 1).
		*/
		ENGINE_API void BezierToPower(double A1, double B1, double C1, double D1, double* A2, double* B2, double* C2, double* D2);
		
		float WeightedEvalForTwoKeys(float Key1Value, float Key1Time, float Key1LeaveTangent, float Key1LeaveTangentWeight, ERichCurveTangentWeightMode Key1TangentWeightMode, float Key2Value, float Key2Time, float Key2ArriveTangent, float Key2ArriveTangentWeight,  ERichCurveTangentWeightMode Key2TangentWeightMode, float InTime);

		/** Returns whether the interpolation between Key1 and Key2 does NOT require using weighted tangent data */
		bool IsItNotWeighted(const FRichCurveKey& Key1, const FRichCurveKey& Key2);

		/** Returns whether  the interpolation between Key1 and Key2 requires using weighted tangent data */
		bool IsWeighted(const FRichCurveKey& Key1, const FRichCurveKey& Key2);

		/** Evaluates and interpolates specific point, determined by InTime, between Key1 and Key2 */
		ENGINE_API float EvalForTwoKeys(const FRichCurveKey& Key1, const FRichCurveKey& Key2, const float InTime);
	}
}