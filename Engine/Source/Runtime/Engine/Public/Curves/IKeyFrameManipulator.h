// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"

template<typename TimeType> class TKeyFrameManipulator;
template<typename TimeType> struct TKeyTimeIterator;

/**
 * Templated key frame manipulator that knows how to add, remove and (re)arrange key times.
 * Should guarantee that any manipulation of a previously sorted time array, will remain sorted.
 */
template<typename TimeType = float>
class IKeyFrameManipulator
{
public:

	// Pass by value/ref parameter type
	typedef typename TCallTraits<TimeType>::ParamType TimeTypeRef;

	/**
	 * Add a new key time to the data structure
	 *
	 * @param InTime		The value of the time to add
	 * @return A key handle for the new key
	 */
	inline FKeyHandle AddKey(TimeTypeRef InTime)
	{
		return AddKeyImpl(InTime);
	}

	/**
	 * Set the time for a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to set the time for
	 * @param NewTime			The time to assign to this key
	 */
	inline void SetKeyTime(FKeyHandle KeyHandle, TimeTypeRef NewTime)
	{
		SetKeyTimeImpl(KeyHandle, NewTime);
	}


	/**
	 * Remove a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to remove
	 */
	inline void RemoveKey(FKeyHandle KeyHandle)
	{
		RemoveKeyImpl(KeyHandle);
	}

	/**
	 * Get the time that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to get the time for.
	 * @return The time that this key exists at, or an empty container if it was not found
	 */
	inline TOptional<TimeType> GetKeyTime(FKeyHandle KeyHandle) const
	{
		return GetKeyTimeImpl(KeyHandle);
	}

	/**
	 * Attempt to find a key using a custom predicate
	 *
	 * @param InPredicate		Predicate function to use when searching
	 * @return The key's handle if found, or an empty container if not
	 */
	inline TOptional<FKeyHandle> FindKey(TimeTypeRef KeyTime, TimeTypeRef KeyTimeTolerance) const
	{
		auto Predicate = [=](TimeTypeRef InTime) { return FMath::IsNearlyEqual(KeyTime, KeyTimeTolerance); };
		return FindKeyImpl(Predicate);
	}

private:

	/**
	 * Add a new key time to the data structure
	 *
	 * @param InTime		The value of the time to add
	 * @return A key handle for the new key
	 */
	virtual FKeyHandle AddKeyImpl(TimeTypeRef InTime) = 0;

	/**
	 * Set the time for a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to set the time for
	 * @param NewTime			The time to assign to this key
	 */
	virtual void SetKeyTimeImpl(FKeyHandle KeyHandle, TimeTypeRef NewTime) = 0;

	/**
	 * Remove a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to remove
	 */
	virtual void RemoveKeyImpl(FKeyHandle KeyHandle) = 0;

	/**
	 * Get the time that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to get the time for.
	 * @return The time that this key exists at, or an empty container if it was not found
	 */
	virtual TOptional<TimeType> GetKeyTimeImpl(FKeyHandle KeyHandle) const = 0;

	/**
	 * Attempt to find a key using a custom predicate
	 *
	 * @param InPredicate		Predicate function to use when searching
	 * @return The key's handle if found, or an empty container if not
	 */
	virtual TOptional<FKeyHandle> FindKeyImpl(const TFunctionRef<bool(TimeTypeRef)>& InPredicate) const = 0;

	/**
	 * Iterate the times stored in the external data structure
	 *
	 * @return An iterator that iterates the keys in order, and can optionally supply handles
	 */
	virtual TKeyTimeIterator<TimeType> IterateKeysImpl() const = 0;
};

template<typename> class TKeyFrameManipulator;

/**
 * Key time iterator for TKeyFrameManipulator
 */
template<typename TimeType>
struct TKeyTimeIterator
{
	TKeyTimeIterator(const TKeyFrameManipulator<TimeType>& InManipulator)
		: Manipulator(InManipulator)
		, Index(0)
	{}

	inline TKeyTimeIterator& operator++()
	{
		++Index;
		return *this;
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	inline explicit operator bool() const
	{ 
		return Index < Manipulator.KeyTimes->Num();
	}
	/** inverse of the "bool" operator */
	inline bool operator !() const 
	{
		return !(bool)*this;
	}

	inline bool operator==(const TKeyTimeIterator& RHS) const
	{
		return Manipulator.KeyTimes == RHS.Manipulator.KeyTimes && Index == RHS.Index;
	}

	inline friend bool operator!=(const TKeyTimeIterator& LHS, const TKeyTimeIterator& RHS)
	{
		return !(LHS == RHS);
	}

	inline TimeType operator*() const
	{
		return (*Manipulator.KeyTimes)[Index];
	}

	FKeyHandle GetKeyHandle()
	{
		return Manipulator.KeyHandleLUT->FindOrAddKeyHandle(Index);
	}

	int32 GetStartIndex() const
	{
		return 0;
	}

	int32 GetEndIndex() const
	{
		return Manipulator.KeyTimes->Num();
	}

	inline friend TKeyTimeIterator<TimeType> begin(const TKeyTimeIterator<TimeType>& Iter)
	{
		TKeyTimeIterator<TimeType> NewIter = Iter;
		NewIter.Index = Iter.GetStartIndex();
		return NewIter;
	}
	inline friend TKeyTimeIterator<TimeType> end  (const TKeyTimeIterator<TimeType>& Iter)
	{
		TKeyTimeIterator<TimeType> NewIter = Iter;
		NewIter.Index = NewIter.GetEndIndex();
		return NewIter;
	}

private:
	const TKeyFrameManipulator<TimeType>& Manipulator;
	int32 Index;
};
