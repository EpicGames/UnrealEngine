// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "Curves/IKeyFrameManipulator.h"

template<typename TimeType> struct TKeyTimeIterator;

/**
 * Templated key frame manipulator that knows how to add, remove and (re)arrange key times.
 * Guarantees that any manipulation of a previously sorted time array, will remain sorted.
 */
template<typename TimeType>
class TKeyFrameManipulator : IKeyFrameManipulator<TimeType>
{
public:
	// Pass by value/ref parameter type
	typedef typename TCallTraits<TimeType>::ParamType TimeTypeRef;

	/**
	 * Construction from an externally owned array of times
	 */
	TKeyFrameManipulator(TArray<TimeType>* KeyTimesParam, FKeyHandleLookupTable* ExternalKeyHandleLUT = nullptr)
		: KeyTimes(KeyTimesParam)
		, KeyHandleLUT(ExternalKeyHandleLUT)
	{
		check(KeyTimes);

		if (!KeyHandleLUT)
		{
			KeyHandleLUT = &TemporaryKeyHandleLUT;
			for (int32 TimeIndex = 0; TimeIndex < KeyTimes->Num(); ++TimeIndex)
			{
				FKeyHandle Handle = KeyHandleLUT->AllocateHandle(TimeIndex);
			}
		}
	}

	TKeyFrameManipulator(const TKeyFrameManipulator& Rhs)
		: KeyTimes(Rhs.KeyTimes)
		, KeyHandleLUT(Rhs.KeyHandleLUT == &Rhs.TemporaryKeyHandleLUT ? &TemporaryKeyHandleLUT : Rhs.KeyHandleLUT)
		, TemporaryKeyHandleLUT(Rhs.TemporaryKeyHandleLUT)
	{}

	TKeyFrameManipulator& operator=(const TKeyFrameManipulator& Rhs)
	{
		if (&Rhs != this)
		{
			KeyTimes = Rhs.KeyTimes;
			KeyHandleLUT = Rhs.KeyHandleLUT == &Rhs.TemporaryKeyHandleLUT ? &TemporaryKeyHandleLUT : Rhs.KeyHandleLUT;
			TemporaryKeyHandleLUT = Rhs.TemporaryKeyHandleLUT;
		}
		return *this;
	}

	TKeyFrameManipulator(TKeyFrameManipulator&& Rhs)
		: KeyTimes(Rhs.KeyTimes)
		, KeyHandleLUT(Rhs.KeyHandleLUT == &Rhs.TemporaryKeyHandleLUT ? &TemporaryKeyHandleLUT : Rhs.KeyHandleLUT)
		, TemporaryKeyHandleLUT(MoveTemp(Rhs.TemporaryKeyHandleLUT))
	{}

	TKeyFrameManipulator& operator=(TKeyFrameManipulator&& Rhs)
	{
		if (&Rhs != this)
		{
			KeyTimes = Rhs.KeyTimes;
			KeyHandleLUT = Rhs.KeyHandleLUT == &Rhs.TemporaryKeyHandleLUT ? &TemporaryKeyHandleLUT : Rhs.KeyHandleLUT;
			TemporaryKeyHandleLUT = MoveTemp(Rhs.TemporaryKeyHandleLUT);
		}
		return *this;
	}

	virtual ~TKeyFrameManipulator() = default;

private:

	/** Called when a new key time has been added */
	virtual void OnKeyAdded(int32 Index) {}
	/** Called when a key time has been moved in the array */
	virtual void OnKeyRelocated(int32 OldIndex, int32 NewIndex) {}
	/** Called when a key time has been removed from the array */
	virtual void OnKeyRemoved(int32 Index) {}
	/** Called when all key times have been removed */
	virtual void OnReset() {}

private:

	virtual FKeyHandle AddKeyImpl(TimeTypeRef InTime) override { return AddKey(InTime); }
	virtual void SetKeyTimeImpl(FKeyHandle KeyHandle, TimeTypeRef NewTime) override { SetKeyTime(KeyHandle, NewTime); }
	virtual void RemoveKeyImpl(FKeyHandle KeyHandle) override { RemoveKey(KeyHandle); }
	virtual TOptional<TimeType> GetKeyTimeImpl(FKeyHandle KeyHandle) const override { return GetKeyTime(KeyHandle); }
	virtual TOptional<FKeyHandle> FindKeyImpl(const TFunctionRef<bool(TimeTypeRef)>& InPredicate) const override { return FindKey(InPredicate); }
	virtual TKeyTimeIterator<TimeType> IterateKeysImpl() const override { return IterateKeys(); }

public:

	/**
	 * Add a new key time to the data structure
	 *
	 * @param InTime		The value of the time to add
	 * @return A key handle for the new key
	 */
	FKeyHandle AddKey(TimeTypeRef InTime)
	{
		int32 InsertIndex = ComputeInsertIndex(InTime);

		FKeyHandle Handle = InsertKeyImpl(InTime, InsertIndex);
		OnKeyAdded(InsertIndex);

		return Handle;
	}

	/**
	 * Set the time for a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to set the time for
	 * @param NewTime			The time to assign to this key
	 */
	void SetKeyTime(FKeyHandle KeyHandle, TimeTypeRef NewTime)
	{
		const int32 ExistingIndex = KeyHandleLUT->GetIndex(KeyHandle);
		if (ExistingIndex == INDEX_NONE)
		{
			return;
		}

		(*KeyTimes)[ExistingIndex] = NewTime;

		int32 NewIndex = ComputeInsertIndex(NewTime, ExistingIndex);
		if (NewIndex != ExistingIndex)
		{
			// Need to move the key
			RelocateKeyImpl(ExistingIndex, NewIndex);
			OnKeyRelocated(ExistingIndex, NewIndex);
		}
	}


	/**
	 * Remove a key that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to remove
	 */
	void RemoveKey(FKeyHandle KeyHandle)
	{
		int32 RemoveAtIndex = GetIndex(KeyHandle);
		if (RemoveAtIndex != INDEX_NONE)
		{
			KeyTimes->RemoveAtSwap(RemoveAtIndex, EAllowShrinking::No);
			KeyHandleLUT->DeallocateHandle(RemoveAtIndex);

			OnKeyRemoved(RemoveAtIndex);
		}
	}

	/**
	 * Removes all keys.
	 */
	void Reset()
	{
		KeyTimes->Empty();
		KeyHandleLUT->Reset();
		OnReset();
	}

public:

	/**
	 * Get the time that corresponds to the specified key handle
	 *
	 * @param KeyHandle			Handle to the key to get the time for.
	 * @return The time that this key exists at, or an empty container if it was not found
	 */
	TOptional<TimeType> GetKeyTime(FKeyHandle KeyHandle) const
	{
		const int32 Index = KeyHandleLUT->GetIndex(KeyHandle);
		return Index == INDEX_NONE ? TOptional<TimeType>() : (*KeyTimes)[Index];
	}

	/**
	 * Attempt to find a key using a custom predicate
	 *
	 * @param InPredicate		Predicate function to use when searching
	 * @return The key's handle if found, or an empty container if not
	 */
	TOptional<FKeyHandle> FindKey(const TFunctionRef<bool(TimeTypeRef)>& InPredicate) const
	{
		for (int32 Index = 0; Index < KeyTimes->Num(); ++Index)
		{
			if (InPredicate((*KeyTimes)[Index]))
			{
				return KeyHandleLUT->FindOrAddKeyHandle(Index);
			}
		}

		return TOptional<FKeyHandle>();
	}

	/**
	 * Iterate the times stored in the external data structure
	 *
	 * @return An iterator that iterates the keys in order, and can optionally supply handles
	 */
	TKeyTimeIterator<TimeType> IterateKeys() const
	{
		return TKeyTimeIterator<TimeType>(*this);
	}

protected:

	/**
	 * Calculate the index at which to insert the given time such that the container remains sorted
	 */
	int32 ComputeInsertIndex(TimeTypeRef InTime, int32 StartAtIndex = 0) const
	{
		for(; StartAtIndex < KeyTimes->Num() && (*KeyTimes)[StartAtIndex] < InTime; ++StartAtIndex);
		return StartAtIndex;
	}

	/**
	 * Insert the specified time into our container at the specified index
	 */
	FKeyHandle InsertKeyImpl(TimeTypeRef Time, int32 InsertIndex) const
	{
		KeyTimes->Insert(Time, InsertIndex);
		return KeyHandleLUT->AllocateHandle(InsertIndex);
	}

	/**
	 * Get the time of the specified key index. Index is assumed to be valid.
	 */
	TimeType GetKeyTimeChecked(int32 KeyIndex) const
	{
		return (*KeyTimes)[KeyIndex];
	}

	/**
	 * Get the index that corresponds to the specified key handle
	 */
	int32 GetIndex(FKeyHandle KeyHandle) const
	{
		return KeyHandleLUT->GetIndex(KeyHandle);
	}
	
	/**
	 * Get the handle that corresponds to the specified index
	 */
	FKeyHandle GetKeyHandleFromIndex(int32 Index) const
	{
		return KeyTimes->IsValidIndex(Index) ? KeyHandleLUT->FindOrAddKeyHandle(Index) : FKeyHandle::Invalid();
	}

	/**
	 * Move a key from one index to another
	 */
	void RelocateKeyImpl(int32 OldIndex, int32 NewIndex)
	{
		if (OldIndex == NewIndex || OldIndex < 0 || NewIndex < 0)
		{
			return;
		}

		check(KeyTimes->IsValidIndex(OldIndex));

		TimeType Time = (*KeyTimes)[OldIndex];

		KeyTimes->RemoveAtSwap(OldIndex, EAllowShrinking::No);
		KeyTimes->Insert(Time, NewIndex);

		KeyHandleLUT->MoveHandle(OldIndex, NewIndex);
	}

private:

	friend TKeyTimeIterator<TimeType>;

	/** Ptr to array of key times */
	TArray<TimeType>* KeyTimes;

	/** Lookup table for key handles. */
	FKeyHandleLookupTable* KeyHandleLUT;

	/** Lookup table for key handles used when the user did not supply one. */
	mutable FKeyHandleLookupTable TemporaryKeyHandleLUT;
};
