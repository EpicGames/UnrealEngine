// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <initializer_list>
#include <type_traits>


namespace UE::Anim { namespace Private {

// TRangeIndexType extracts RangeType::IndexType if it exists, otherwise void
template<typename RangeType, typename = void>
struct TRangeIndexType 
{
	using Type = void;
};

template<typename RangeType>
struct TRangeIndexType<RangeType, std::void_t<typename std::decay_t<RangeType>::IndexType>>
{
	using Type = typename std::decay_t<RangeType>::IndexType;
};

// IndexType is compatible with RangeType::IndexType if they are the same or if RangeType doesn't have an index type
template<typename IndexType, typename RangeType>
struct TIsCompatibleRangeIndexType
{
	enum
	{
		Value =
			std::is_same_v<IndexType, typename TRangeIndexType<RangeType>::Type> ||
			std::is_void_v<typename TRangeIndexType<RangeType>::Type>
	};
};

} // namespace UE::Anim::Private


template<typename InIndexType, typename InRangeType>
class TTypedIndexRange : public InRangeType
{
public:
	using BaseType = InRangeType;
	using IndexType = InIndexType;
	using ElementType = typename InRangeType::ElementType;
	using SizeType = typename InRangeType::SizeType;

	TTypedIndexRange() = default;

	// Forwarding constructor that disallows mixing index types
	template<typename T>
	TTypedIndexRange(T&& Other)
		: BaseType(Forward<T>(Other))
	{
		static_assert(Private::TIsCompatibleRangeIndexType<IndexType, T>::Value, "TTypedIndexRange can't construct from a different index type");
	}

	TTypedIndexRange(std::initializer_list<ElementType> List)
		: BaseType(List)
	{
	}

	// Forwarding assignment that disallows mixing index types
	template<typename T>
	TTypedIndexRange& operator=(T&& Other)
	{
		static_assert(Private::TIsCompatibleRangeIndexType<IndexType, T>::Value, "TTypedIndexRange can't assign from a different index type");
		BaseType::operator=(Forward<T>(Other));
		return *this;
	}

	TTypedIndexRange& operator=(std::initializer_list<ElementType> List)
	{
		BaseType::operator=(List);
		return *this;
	}

	FORCEINLINE ElementType& operator[](SizeType Index)
	{
		return BaseType::operator[](Index);
	}
	FORCEINLINE const ElementType& operator[](SizeType Index) const
	{
		return BaseType::operator[](Index);
	}
	FORCEINLINE ElementType& operator[](const IndexType& Index)
	{
		return BaseType::operator[]((SizeType)Index);
	}
	FORCEINLINE const ElementType& operator[](const IndexType& Index) const
	{
		return BaseType::operator[]((SizeType)Index);
	}
};

template<typename IndexType, typename ...TArrayArgs>
using TTypedIndexArray = TTypedIndexRange<IndexType, TArray<TArrayArgs...>>;

template<typename IndexType, typename ...TArrayViewArgs>
using TTypedIndexArrayView = TTypedIndexRange<IndexType, TArrayView<TArrayViewArgs...>>;

} // namespace UE::Anim

template<typename ElementType, typename BoneIndexType, typename InAllocatorType = typename TArray<ElementType>::AllocatorType>
using TCustomBoneIndexArray = UE::Anim::TTypedIndexRange<BoneIndexType, TArray<ElementType, InAllocatorType>>;

template<typename ElementType, typename BoneIndexType>
using TCustomBoneIndexArrayView = UE::Anim::TTypedIndexRange<BoneIndexType, TArrayView<ElementType>>;

template <typename ...ArgTypes>
struct TIsContiguousContainer<UE::Anim::TTypedIndexRange<ArgTypes...>>
{
	enum { Value = true };
};