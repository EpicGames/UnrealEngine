// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Components/PrimitiveComponent.h"

namespace physx
{
	class PxRigidActor;
}

/** This interface allows plugins to sync between physx sim results and Unreal Engine data. */
struct FCustomPhysXSyncActors
{
	virtual ~FCustomPhysXSyncActors() = default;

	/** Syncing between PhysX and Unreal Engine can be tricky. The challenge is that PhysX gives us an array of raw pointers that require updating. However, Unreal Engine has many mechanisms for triggering arbitrary callbacks.
	  * Since the callbacks can invalidate the PhysX data it's critical to break the update into two steps.
	  * The first step moves the data from PhysX into Unreal Engine safe structures (see TWeakObjectPtr).
	  * The second step updates the actual Unreal Engine data (components, actors, etc...)
	  * It's up to the client code to ensure this is handled correctly. No PhysX raw data should be stored. Do not update component/actor data in step 1 unless you're absolutely sure it will not trigger a callback
	  */

	/** This is step 1 of the sync. Do not store any raw PhysX data or update any Unreal Engine component/actors which may trigger a callback. Ideally this should build an array of TWeakObjectPtr<UYourComponent>, Transform pairs */
	virtual void BuildSyncData_AssumesLocked(const TArray<physx::PxRigidActor*>& RigidActors) = 0;

	/** This is step 2 of the sync. It's safe to update any Unreal Engine components/actors or trigger callbacks, as long as the code within this function handles it gracefully.
	  * For example if a callback destroys some components you will update, make sure to use TWeakObjectPtr */
	virtual void FinalizeSync() = 0;

private:
	friend class FPhysScene_PhysX;
	friend class FPhysScene_ImmediatePhysX;
	TArray<physx::PxRigidActor*> Actors;
};

struct FBodyInstance;

struct FCustomPhysXPayload
{
	FCustomPhysXPayload(FCustomPhysXSyncActors* InCustomSyncActors) : CustomSyncActors(InCustomSyncActors)
	{
	}

	virtual ~FCustomPhysXPayload()= default;
	
	virtual TWeakObjectPtr<UPrimitiveComponent> GetOwningComponent() const = 0;
	virtual int32 GetItemIndex() const = 0;
	virtual FName GetBoneName() const = 0;
	virtual FBodyInstance* GetBodyInstance() const = 0;

	FCustomPhysXSyncActors* CustomSyncActors;
};

