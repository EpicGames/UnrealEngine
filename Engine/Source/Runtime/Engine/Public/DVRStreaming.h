// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

/**
 * Interface(s) for platform feature modules
 */

// *** TODO: Consider renaming to remove the DVR aspect, since this doesn't do anything with recording

/** Defines the interface to platform's DVR and/or streaming system */
struct FDVRStreamingStatus
{
	bool bIsStreaming;		    // Are we currently broadcasting a video stream?
	bool bIsStreamingEnabled;	// Is streaming currently enabled or disabled?

	// NOTE: These are valid only if IsStreaming is true, otherwise these are empty/0
	int32 ViewerCount;		// Current number of viewers
	FString ProgramName;	// The name of the program being streamed. Likely depends on the streaming provider
	FString HLSUrl;			// HTTP Live Streaming URL
	FString ProviderUrl;	// Url to the streaming provider's site
};

class IDVRStreamingSystem
{
public:
	virtual ~IDVRStreamingSystem() = default;

	/** Fetches the current status of streaming */
	virtual void GetStreamingStatus(FDVRStreamingStatus &StreamingStatus) = 0;

	/** 
		Enables or disables streaming.
		NOTE: This does not prevent the user from streaming, it simply unblocks/blocks video and audio on any current or future stream.
	 */
	virtual void EnableStreaming(bool Enable) = 0;

	// *** TODO: Consider adding support for a delegate here (as with the saved game system) for fetching hardware specific configuration data, since as data rates,
	// *** server side recording permission, etc. Or make those INI file configurations

	// *** TODO: Add more methods here as needed, e.g. social features for streaming or local recording control
};

/** A generic implementation of the DVR/Streaming system, that doesn't support streaming */
class FGenericDVRStreamingSystem : public IDVRStreamingSystem
{
public:
	FGenericDVRStreamingSystem () : bIsStreamingEnabled(false)
	{
	}

	virtual void GetStreamingStatus(FDVRStreamingStatus &StreamingStatus) override
	{
		StreamingStatus.bIsStreaming = false;
		StreamingStatus.bIsStreamingEnabled = bIsStreamingEnabled;

		StreamingStatus.ViewerCount = 0;
		StreamingStatus.ProgramName = TEXT("");
		StreamingStatus.HLSUrl = TEXT("");
		StreamingStatus.ProviderUrl = TEXT("");
	}

	virtual void EnableStreaming(bool Enable) override
	{
		bIsStreamingEnabled = Enable;
	}

private:
	bool bIsStreamingEnabled;
};
