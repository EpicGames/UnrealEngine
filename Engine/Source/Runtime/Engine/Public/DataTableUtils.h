// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"

class UDataTable;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogDataTable, Log, All);

enum class EDataTableExportFlags : uint8
{
	/** No specific options. */
	None = 0,

	/** Export nested structs as JSON objects (JSON exporter only), rather than as exported text. */
	UseJsonObjectsForStructs = 1 << 0,

	/** Export text properties as their display string, rather than their complex lossless form. */
	UseSimpleText = 1 << 1,

	// Deprecated options removed, Native properties/enums are always exported using their internal name, user struct/enums are always exported using the friendly names set in the editor
};
ENUM_CLASS_FLAGS(EDataTableExportFlags);

namespace DataTableUtils
{
	/**
	 * Util to assign a value (given as a string) to a struct property.
	 * This always assigns the string to the given property without adjusting the address.
	 */
	ENGINE_API FString AssignStringToPropertyDirect(const FString& InString, const FProperty* InProp, uint8* InData);

	/**
	 * Util to assign a value (given as a string) to a struct property.
	 * When the property is a static sized array, this will split the string and assign the split parts to each element in the array.
	 */
	ENGINE_API FString AssignStringToProperty(const FString& InString, const FProperty* InProp, uint8* InData);

	/** 
	 * Util to get a property as a string.
	 * This always gets a string for the given property without adjusting the address.
	 */
	ENGINE_API FString GetPropertyValueAsStringDirect(const FProperty* InProp, const uint8* InData, const EDataTableExportFlags InDTExportFlags);

	/** 
	 * Util to get a property as a string.
	 * When the property is a static sized array, this will return a string containing each element in the array.
	 */
	ENGINE_API FString GetPropertyValueAsString(const FProperty* InProp, const uint8* InData, const EDataTableExportFlags InDTExportFlags);

	/** 
	 * Util to get a property as text (this will use the display name of the value where available - use GetPropertyValueAsString if you need an internal identifier).
	 * This always gets a string for the given property without adjusting the address.
	 */
	ENGINE_API FText GetPropertyValueAsTextDirect(const FProperty* InProp, const uint8* InData);

	/** 
	 * Util to get a property as text (this will use the display name of the value where available - use GetPropertyValueAsString if you need an internal identifier).
	 * When the property is a static sized array, this will return a string containing each element in the array. 
	 */
	ENGINE_API FText GetPropertyValueAsText(const FProperty* InProp, const uint8* InData);

	/**
	 * Util to get all property names from a struct.
	 */
	ENGINE_API TArray<FName> GetStructPropertyNames(const UStruct* InStruct);

	/**
	 * Util that removes invalid chars and then make an FName.
	 */
	ENGINE_API FName MakeValidName(const FString& InString);

	/**
	 * Util to see if this property is supported in a row struct.
	 */
	ENGINE_API bool IsSupportedTableProperty(const FProperty* InProp);

	/**
	 * Util to get the friendly display unlocalized name of a given property for export to files.
	 */
	ENGINE_API FString GetPropertyExportName(const FProperty* Prop, const EDataTableExportFlags InDTExportFlags = EDataTableExportFlags::None);

	/**
	 * Util to get the all variants for export names for backwards compatibility.
	 */
	ENGINE_API TArray<FString> GetPropertyImportNames(const FProperty* Prop);
	ENGINE_API void GetPropertyImportNames(const FProperty* Prop, TArray<FString>& OutResult);

	/**
	 * Util to get the localized display name of a given property.
	 */
	ENGINE_API FText GetPropertyDisplayName(const FProperty* Prop, const FString& DefaultName);

	/**
	 * Output each row for a specific column/property in the table (doesn't include the title)
	 */
	ENGINE_API TArray<FString> GetColumnDataAsString(const UDataTable* InTable, const FName& PropertyName, const EDataTableExportFlags InDTExportFlags);

	/**
	 * Add a row from a json format to a datatable
	 */
	ENGINE_API bool AddRowJSON(UDataTable& InDataTable, const FString& InJSONData, TArray<FString>& OutProblems, bool bRemoveDuplicate = false, FName* OutRowName = nullptr);
}
