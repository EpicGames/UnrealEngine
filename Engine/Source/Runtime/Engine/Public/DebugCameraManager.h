// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

class FDebugCameraManager : public FGCObject
{
public:
	FDebugCameraManager();
	virtual ~FDebugCameraManager();
	ENGINE_API static FDebugCameraManager& Get();

	/** Debug camera class used for debug cameras */
	TSubclassOf<class ADebugCameraController>  DebugCameraControllerClass;

	ENGINE_API void EnableDebugCamera(class APlayerController* PlayerController);
	ENGINE_API void DisableDebugCamera(class APlayerController* PlayerController);

	ENGINE_API bool IsDebugCameraActive(const class APlayerController* PlayerController) const;

	ENGINE_API class ADebugCameraController* GetCameraController(const class APlayerController* PlayerController);

	// FGCObject interface
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual FString GetReferencerName() const override { return TEXT("FDebugCameraManager"); }

private:
	struct FDebugCameraControllerStatus
	{
		TObjectPtr<class ADebugCameraController> DebugCameraController;
		bool bIsActive = false;
	};

	FDebugCameraControllerStatus* GetDebugCameraControllerStatus(const class APlayerController* PlayerController);
	const FDebugCameraControllerStatus* GetDebugCameraControllerStatus(const class APlayerController* PlayerController) const;

	/** Debug camera - used to have independent camera without stopping gameplay */
	TMap<TObjectKey<class UPlayer>, FDebugCameraControllerStatus> PlayerToDebugCameraControllerMap;

	void HandleGameInstanceWorldChanged(class UGameInstance* GameInstance, class UWorld* OldWorld, class UWorld* NewWorld);

	FDelegateHandle GameInstanceWorldChangedHandle;
};