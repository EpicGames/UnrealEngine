// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugRenderSceneProxy.h: Useful scene proxy for rendering non performance-critical information.
=============================================================================*/

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "Materials/Material.h"
#include "DynamicMeshBuilder.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "PrimitiveSceneProxy.h"
#include "SceneView.h"

struct FDynamicMeshVertex;
struct FConvexVolume;
class APlayerController;
class FMaterialRenderProxy;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class FRegisterComponentContext;
class UCanvas;
class UMaterial;
class UPrimitiveComponent;

DECLARE_DELEGATE_TwoParams(FDebugDrawDelegate, UCanvas*, APlayerController*);

namespace UE::DebugDrawHelper
{
/** Projects provided 3D world-space location into 2D screen coordinates considering DPI scale and aspect ratio requirement (black bars) */
ENGINE_API FVector3f GetScaleAdjustedScreenLocation(TNotNull<const UCanvas*> Canvas, FVector WorldLocation);

/** Projects provided 3D world-space location into 2D screen coordinates considering DPI scale and aspect ratio requirement (black bars) */
ENGINE_API FVector3f GetScaleAdjustedScreenLocation(TNotNull<const FSceneView*>, const FCanvas& Canvas, FVector WorldLocation);
}

class FDebugRenderSceneProxy : public FPrimitiveSceneProxy
{
public:
	ENGINE_API virtual ~FDebugRenderSceneProxy();
	
	ENGINE_API SIZE_T GetTypeHash() const override;

	enum EDrawType
	{
		SolidMesh = 0,
		WireMesh = 1,
		SolidAndWireMeshes = 2,
		Invalid = 3,
	};
	ENGINE_API FDebugRenderSceneProxy(const UPrimitiveComponent* InComponent);
	ENGINE_API FDebugRenderSceneProxy(FDebugRenderSceneProxy const&);

	// FPrimitiveSceneProxy interface.

	/** 
	 * Draw the scene proxy as a dynamic element
	 */
	ENGINE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	/**
	 * Draws a line with an arrow at the end.
	 *
	 * @param PDI		Draw interface to render to
	 * @param Start		Starting point of the line.
	 * @param End		Ending point of the line.
	 * @param Color		Color of the line.
	 * @param Mag		Size of the arrow.
	 */
	void DrawLineArrow(FPrimitiveDrawInterface* PDI,const FVector &Start,const FVector &End,const FColor &Color,float Mag) const;

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	ENGINE_API uint32 GetAllocatedSize(void) const;

	inline static bool PointInView(const FVector& Location, const FSceneView* View)
	{
		return View ? PointInFrustum(Location, View->ViewFrustum) : false;
	}

	inline static bool SegmentInView(const FVector& StartPoint, const FVector& EndPoint, const FSceneView* View)
	{
		return View ? SegmentInFrustum(StartPoint, EndPoint, View->ViewFrustum) : false;
	}

	inline static bool BoxInView(const FVector& Origin, const FVector& Extent, const FSceneView* View)
	{
		return View ? BoxInFrustum(Origin, Extent, View->ViewFrustum) : false;
	}

	inline static bool SphereInView(const FVector& Center, double Radius, const FSceneView* View)
	{
		return View ? SphereInFrustum(Center, Radius, View->ViewFrustum) : false;
	}

	inline static bool PointInFrustum(const FVector& Location, const FConvexVolume& ViewFrustum)
	{
		return ViewFrustum.IntersectPoint(Location);
	}

	inline static bool SegmentInFrustum(const FVector& StartPoint, const FVector& EndPoint, const FConvexVolume& ViewFrustum)
	{
		return ViewFrustum.IntersectLineSegment(StartPoint, EndPoint);
	}

	inline static bool BoxInFrustum(const FBox& Box, const FConvexVolume& ViewFrustum)
	{
		FVector Center, Extents;
		Box.GetCenterAndExtents(Center, Extents);
		return BoxInFrustum(Center, Extents, ViewFrustum);
	}

	inline static bool BoxInFrustum(const FVector& Origin, const FVector& Extent, const FConvexVolume& ViewFrustum)
	{
		return ViewFrustum.IntersectBox(Origin, Extent);
	}

	inline static bool SphereInFrustum(const FVector& Center, double Radius, const FConvexVolume& ViewFrustum)
	{
		return ViewFrustum.IntersectSphere(Center, static_cast<float>(Radius));
	}

	inline static bool PointInRange(const FVector& Start, const FSceneView* View, double Range)
	{
		return FVector::DistSquared(Start, View->ViewMatrices.GetViewOrigin()) <= FMath::Square(Range);
	}

	// Create a new frustum out of the view, taking into account the far clipping distance (if InFarClippingDistance > 0.0) 
	ENGINE_API static FConvexVolume AdjustViewFrustumForFarClipping(const FSceneView* InView, double InFarClippingDistance);
	
	struct FMaterialCache
	{
		FMaterialCache(FMeshElementCollector& InCollector, bool bUseLight = false, UMaterial* InMaterial = nullptr);
		FMaterialRenderProxy* operator[](FLinearColor Color);

		FMeshElementCollector& Collector;
		TMap<uint32, FMaterialRenderProxy*> MeshColorInstances;
		TWeakObjectPtr<UMaterial> SolidMeshMaterial;
		bool bUseFakeLight = false;
	};

	/** Struct to hold info about lines to render. */
	struct FDebugLine
	{
		FDebugLine(const FVector &InStart, const FVector &InEnd, const FColor &InColor, const float InThickness = 0.f, ESceneDepthPriorityGroup InPriorityGroup = ESceneDepthPriorityGroup::SDPG_World)
			: Start(InStart)
			, End(InEnd)
			, Color(InColor)
			, Thickness(InThickness)
			, PriorityGroup(InPriorityGroup)
		{}

		FVector Start;
		FVector End;
		FColor Color;
		float Thickness;
		ESceneDepthPriorityGroup PriorityGroup;

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI) const;
	};

	/** Struct to hold info about circles to render. */
	struct FCircle
	{
		FCircle(const FVector& InCenter, const FVector& InAxis, const float InRadius, const FColor& InColor, 
			EDrawType InDrawTypeOverride = EDrawType::Invalid, const float InThickness = 0.f)
			: Center(InCenter)
			, Axis(InAxis)
			, Radius(InRadius)
			, Color(InColor)
			, Thickness(InThickness)
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const;

		FVector Center;
		FVector Axis;
		float Radius;
		FColor Color;
		float Thickness;
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	/** Struct to hold info about boxes to render. */
	struct FDebugBox
	{
		FDebugBox(const FBox& InBox, const FColor& InColor, EDrawType InDrawTypeOverride = EDrawType::Invalid, const float InThickness = 0.f)
			: Box(InBox)
			, Color(InColor)
			, Thickness(InThickness)
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		FDebugBox(const FBox& InBox, const FColor& InColor, const FTransform& InTransform, EDrawType InDrawTypeOverride = EDrawType::Invalid, 
			const float InThickness = 0.f)
			: Box(InBox)
			, Color(InColor)
			, Thickness(InThickness)
			, Transform(InTransform)
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const;

		FBox Box;
		FColor Color;
		float Thickness;
		FTransform Transform;
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	/** Struct to hold info about cylinders to render. */
	struct FWireCylinder
	{
		FWireCylinder(const FVector& InBase, const FVector& InDirection, const float InRadius, const float InHalfHeight, const FColor &InColor,
			EDrawType InDrawTypeOverride = EDrawType::Invalid, const float InThickness = 0.f)
			: Base(InBase)
			, Direction(InDirection)
			, Radius(InRadius)
			, HalfHeight(InHalfHeight)
			, Color(InColor) 
			, Thickness(InThickness)
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const;

		FVector Base;
		FVector Direction;
		float Radius;
		float HalfHeight;
		FColor Color;
		float Thickness;
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	/** Struct to hold info about lined stars to render. */
	struct FWireStar
	{
		FWireStar(const FVector &InPosition, const FColor &InColor, const float InSize)
			: Position(InPosition)
			, Color(InColor)
			, Size(InSize) 
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI) const;

		FVector Position;
		FColor Color;
		float Size;
	};

	/** Struct to hold info about arrowed lines to render. */
	struct FArrowLine
	{
		FArrowLine(const FVector &InStart, const FVector &InEnd, const FColor &InColor, const float InMag = 0.0f) 
			: Start(InStart)
			, End(InEnd)
			, Color(InColor) 
			, Mag(InMag > 0.0f ? InMag : 8.0f) // default to a Mag of 8.0 as it used to be the default in the past
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI) const;

		UE_DEPRECATED(5.6, "Use the other Draw function and pass the proper data in the constructor")
		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, const float InMag) const;

		FVector Start;
		FVector End;
		FColor Color;
		float Mag;
	};

	/** Struct to gold info about dashed lines to render. */
	struct FDashedLine
	{
		FDashedLine(const FVector &InStart, const FVector &InEnd, const FColor &InColor, const float InDashSize) 
			: Start(InStart)
			, End(InEnd)
			, Color(InColor)
			, DashSize(InDashSize) 
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI) const;

		FVector Start;
		FVector End;
		FColor Color;
		float DashSize;
	};

	/** Struct to hold info about spheres to render */
	struct FSphere
	{
		FSphere() {}
		FSphere(const float& InRadius, const FVector& InLocation, const FLinearColor& InColor, EDrawType InDrawTypeOverride = EDrawType::Invalid)
			: Radius(InRadius)
			, Location(InLocation)
			, Color(InColor.ToFColor(true)) 
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const;

		float Radius;
		FVector Location;
		FColor Color;
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	/** Struct to hold info about texts to render using 3d coordinates */
	struct FText3d
	{
		FText3d() {}
		FText3d(const FString& InString, const FVector& InLocation, const FLinearColor& InColor)
			: Text(InString)
			, Location(InLocation)
			, Color(InColor.ToFColor(true)) 
		{}

		FString Text;
		FVector Location;
		FColor Color;
	};

	struct FCone
	{
		FCone() {}
		FCone(const FMatrix& InConeToWorld, const float InAngle1, const float InAngle2, const FLinearColor& InColor,
			EDrawType InDrawTypeOverride = EDrawType::Invalid, const float InThickness = 0.f)
			: ConeToWorld(InConeToWorld)
			, Angle1(InAngle1)
			, Angle2(InAngle2)
			, Color(InColor.ToFColor(true))
			, Thickness(InThickness)
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector, TArray<FVector>* VertsCache = nullptr) const;

		FMatrix ConeToWorld;
		float Angle1;
		float Angle2;
		FColor Color;
		float Thickness;
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	struct FMesh
	{
		ENGINE_API FMesh();
		ENGINE_API ~FMesh();
		ENGINE_API FMesh(const FMesh& Other);

		TArray<FDynamicMeshVertex> Vertices;
		TArray <uint32> Indices;
		FBox Box = FBox(ForceInit);
		FColor Color = FColor::White;
	};

	struct FCapsule
	{
		FCapsule() {}
		FCapsule(const FVector& InBase, const float& InRadius, const FVector& x, const FVector& y, const FVector &z, const float& InHalfHeight,
			const FLinearColor& InColor, EDrawType InDrawTypeOverride = EDrawType::Invalid, const float InThickness = 0.f)
			: Radius(InRadius)
			, Base(InBase)
			, Color(InColor.ToFColor(true))
			, Thickness(InThickness)
			, HalfHeight(InHalfHeight)
			, X(x)
			, Y(y)
			, Z(z) 
			, DrawTypeOverride(InDrawTypeOverride)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI, EDrawType InDrawType, uint32 InDrawAlpha, FMaterialCache& MaterialCache, int32 ViewIndex, FMeshElementCollector& Collector) const;

		float Radius;
		FVector Base; //Center point of the base of the cylinder.
		FColor Color;
		float Thickness;
		float HalfHeight;
		FVector X, Y, Z; //X, Y, and Z alignment axes to draw along.
		EDrawType DrawTypeOverride = EDrawType::Invalid;
	};

	// struct to hold coordinate systems to render
	struct FCoordinateSystem
	{
		FCoordinateSystem() = delete;
		FCoordinateSystem(const FVector& InAxisLoc, const FRotator& InAxisRot, const float InScale, const FColor& InColor, const float InThickness)
			: AxisLoc(InAxisLoc)
			, AxisRot(InAxisRot)
			, Scale(InScale)
			, Color(InColor)
			, Thickness(InThickness)
		{}

		ENGINE_API void Draw(FPrimitiveDrawInterface* PDI) const;

		FVector AxisLoc;
		FRotator AxisRot;
		float Scale;
		FColor Color;
		float Thickness;
	};

	TArray<FDebugLine> Lines;
	TArray<FDashedLine>	DashedLines;
	TArray<FArrowLine> ArrowLines;
	TArray<FCircle> Circles;
	TArray<FWireCylinder> Cylinders;
	TArray<FWireStar> Stars;
	TArray<FDebugBox> Boxes;
	TArray<FSphere> Spheres;
	TArray<FText3d> Texts;
	TArray<FCone> Cones;
	TArray<FMesh> Meshes;
	TArray<FCapsule> Capsules;
	TArray<FCoordinateSystem> CoordinateSystems;

	uint32 ViewFlagIndex;
	float TextWithoutShadowDistance;
	FString ViewFlagName;
	FDebugDrawDelegate DebugTextDrawingDelegate;
	FDelegateHandle DebugTextDrawingDelegateHandle;
	EDrawType DrawType;
	uint32 DrawAlpha;
	double FarClippingDistance = 0.0;

	TWeakObjectPtr<UMaterial> SolidMeshMaterial;

protected:
	ENGINE_API virtual void GetDynamicMeshElementsForView(const FSceneView* View, const int32 ViewIndex, const FSceneViewFamily& ViewFamily, const uint32 VisibilityMap, FMeshElementCollector& Collector, FMaterialCache& DefaultMaterialCache, FMaterialCache& SolidMeshMaterialCache) const;
};


struct FDebugDrawDelegateHelper
{
	FDebugDrawDelegateHelper()
		: State(UndefinedState)
		, ViewFlagName(TEXT("Game"))
		, TextWithoutShadowDistance(1500)
		, FarClippingDistance(0.0)
	{}

	virtual ~FDebugDrawDelegateHelper() = default;

protected:
	typedef TArray<FDebugRenderSceneProxy::FText3d> TextArray;

public:
	ENGINE_API void InitDelegateHelper(const FDebugRenderSceneProxy* InSceneProxy);

	/**
	 * Method that should be called at render state creation (i.e. CreateRenderState_Concurrent).
	 * It will either call `RegisterDebugDrawDelegate` when deferring context is not provided
	 * or mark for deferred registration that should be flushed by calling `ProcessDeferredRegister`
	 * on scene proxy creation.
	 * @param Context valid context is provided when primitives are batched for deferred 'add'
	 */
	ENGINE_API void RequestRegisterDebugDrawDelegate(FRegisterComponentContext* Context);

	/**
	 * Method that should be called when creating scene proxy (i.e. CreateSceneProxy) to process any pending registration that might have
	 * been requested from deferred primitive batching (i.e. CreateRenderState_Concurrent(FRegisterComponentContext != nullptr)).
	 */
	ENGINE_API void ProcessDeferredRegister();

	/** called to clean up debug drawing delegate in UDebugDrawService */
	ENGINE_API virtual void UnregisterDebugDrawDelegate();

	ENGINE_API void ReregisterDebugDrawDelegate();

protected:
	/** called to set up debug drawing delegate in UDebugDrawService if you want to draw labels */
	ENGINE_API virtual void RegisterDebugDrawDelegateInternal();

	ENGINE_API void HandleDrawDebugLabels(UCanvas* Canvas, APlayerController* PlayerController);
	ENGINE_API virtual void DrawDebugLabels(UCanvas* Canvas, APlayerController* PlayerController);
	void ResetTexts() { Texts.Reset(); }
	const TextArray& GetTexts() const { return Texts; }
	float GetTextWithoutShadowDistance() const {return TextWithoutShadowDistance; }
	double GetFarClippingDistance() const { return FarClippingDistance; }

protected:
	FDebugDrawDelegate DebugTextDrawingDelegate;
	FDelegateHandle DebugTextDrawingDelegateHandle;
	enum EState
	{
		UndefinedState,
		InitializedState,
		RegisteredState,
	} State;

	bool bDeferredRegister = false;

private:
	/**
	 * Weak pointer to the associated world used when drawing labels to skip canvas from a different world.
	 * This is required since the current implementation relies on a global delegate so multiple canvas might
	 * be active (e.g., LevelEditor + any asset editor with viewport)
	 */
	TWeakObjectPtr<UWorld> AssociatedWorld;
	TextArray Texts;
	FString ViewFlagName;
	float TextWithoutShadowDistance;
	double FarClippingDistance;
};
