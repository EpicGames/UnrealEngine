// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Definition and helpers for debug view modes
 */

#pragma once

#include "CoreMinimal.h"

#define WITH_DEBUG_VIEW_MODES (WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

struct FSlowTask;
class UMaterialInterface;
enum EShaderPlatform : uint16;
namespace ERHIFeatureLevel { enum Type : int; }
namespace EMaterialQualityLevel { enum Type : uint8; }

/** 
 * Enumeration for different Quad Overdraw visualization mode.
 */
enum EDebugViewShaderMode
{
	DVSM_None,						// No debug view.
	DVSM_ShaderComplexity,			// Default shader complexity viewmode
	DVSM_ShaderComplexityContainedQuadOverhead,	// Show shader complexity with quad overdraw scaling the PS instruction count.
	DVSM_ShaderComplexityBleedingQuadOverhead,	// Show shader complexity with quad overdraw bleeding the PS instruction count over the quad.
	DVSM_QuadComplexity,			// Show quad overdraw only.
	DVSM_PrimitiveDistanceAccuracy,	// Visualize the accuracy of the primitive distance computed for texture streaming.
	DVSM_MeshUVDensityAccuracy,		// Visualize the accuracy of the mesh UV densities computed for texture streaming.
	DVSM_MaterialTextureScaleAccuracy, // Visualize the accuracy of the material texture scales used for texture streaming.
	DVSM_OutputMaterialTextureScales,  // Outputs the material texture scales.
	DVSM_RequiredTextureResolution, // Visualize the accuracy of the streamed texture resolution.
	DVSM_LODColoration,				// Visualize primitive LOD .
	DVSM_VisualizeGPUSkinCache,		// Visualize various properties of Skin Cache.
	DVSM_LWCComplexity,				// Visualize usage of LWC functions in materials.
	DVSM_ShadowCasters,				// Visualize shadow casters
	DVSM_StreamingDeficit,			// Visualize asset streaming deficit (texture mips or mesh LODs)
	DVSM_MAX
};

ENGINE_API const TCHAR* DebugViewShaderModeToString(EDebugViewShaderMode InShaderMode);

#if WITH_DEBUG_VIEW_MODES
/** Returns true if debug view shaders are only ODSC compiled (in editor or cooked builds). Returns false if they are can also be compiled for editor only shader maps as well as ODSC. */
ENGINE_API bool IsDebugViewShaderModeODSCOnly();
/** DEPRECATED 5.8 - Returns true if any debug view rendering is supported. Can be used as input for shader cooking. */
UE_DEPRECATED(5.8, "SupportDebugViewModes is deprecated, please use ForceEnableDebugViewSupport/ForceDisableDebugViewSupport instead.")
ENGINE_API bool SupportDebugViewModes();
/** Returns true if a specific debug view supported for a platform. Can be used as input for shader cooking. */
ENGINE_API bool SupportDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform);
/** Returns true if a specific debug view can be displayed at runtime. */
ENGINE_API bool AllowDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform, ERHIFeatureLevel::Type FeatureLevel);
/** Returns true if we are forcing debug view support even for non editor or ODSC use cases. */
ENGINE_API bool ForceEnableDebugViewSupport(EShaderPlatform Platform);
/** Returns true if we are force disabling debug view support even for editor use cases. */
ENGINE_API bool ForceDisableDebugViewSupport(EShaderPlatform Platform);

#else
inline bool IsDebugViewShaderModeODSCOnly() { return false; }
/** DEPRECATED 5.8 - Returns true if any debug view rendering is supported. Can be used as input for shader cooking. */
UE_DEPRECATED(5.8, "SupportDebugViewModes is deprecated, please use ForceEnableDebugViewSupport/ForceDisableDebugViewSupport instead.")
inline bool SupportDebugViewModes() { return false; }
inline bool SupportDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform) { return false; }
inline bool AllowDebugViewShaderMode(EDebugViewShaderMode ShaderMode, EShaderPlatform Platform, ERHIFeatureLevel::Type FeatureLevel) { return false; }
inline bool ForceEnableDebugViewSupport(EShaderPlatform Platform) { return false; }
inline bool ForceDisableDebugViewSupport(EShaderPlatform Platform) { return false; }
#endif

ENGINE_API int32 GetNumActorsInWorld(UWorld* InWorld);
ENGINE_API bool GetUsedMaterialsInWorld(UWorld* InWorld, OUT TSet<UMaterialInterface*>& OutMaterials, FSlowTask* Task);
ENGINE_API bool CompileDebugViewModeShaders(EDebugViewShaderMode Mode, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, TSet<UMaterialInterface*>& Materials, FSlowTask* ProgressTask);
ENGINE_API bool WaitForShaderCompilation(const FText& Message, FSlowTask* ProgressTask);
