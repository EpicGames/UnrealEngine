// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DebugViewModeInterface.h: Contains definitions for rendering debug viewmodes.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "DebugViewModeHelpers.h"
#include "Engine/EngineTypes.h"
#include "RHI.h"
#include "Shader.h"
#include "DrawDebugHelpers.h"

#if ENABLE_DRAW_DEBUG

class FDebugViewModePS;
class FMaterial;
class FMaterialRenderProxy;
class FMeshDrawSingleShaderBindings;
class FPrimitiveSceneProxy;
class FVertexFactoryType;
struct FMeshPassProcessorRenderState;
struct FMaterialShaderTypes;

class FDebugViewModeInterface
{
public:

	struct FRenderState
	{
		FRenderState()
			: BlendState(nullptr)
			, DepthStencilState(nullptr)
		{}
		FRHIBlendState*			BlendState;
		FRHIDepthStencilState*	DepthStencilState;
	};

	FDebugViewModeInterface()
	{}

	virtual ~FDebugViewModeInterface() = default;

	virtual void AddShaderTypes(ERHIFeatureLevel::Type InFeatureLevel,
		const FVertexFactoryType* InVertexFactoryType,
		FMaterialShaderTypes& OutShaderTypes) const = 0;

	ENGINE_API virtual void SetDrawRenderState(EDebugViewShaderMode DebugViewMode, EBlendMode BlendMode, FRenderState& DrawRenderState, bool bHasDepthPrepassForMaskedMaterial, bool bDitheredLODTransition = false) const;

	virtual void GetDebugViewModeShaderBindings(
		const FDebugViewModePS& Shader,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		EDebugViewShaderMode DebugViewMode,
		const FVector& ViewOrigin,
		int32 VisualizeLODIndex,
		const FColor& SkinCacheDebugColor,
		int32 VisualizeElementIndex,
		int32 NumVSInstructions,
		int32 NumPSInstructions,
		int32 LWCComplexityVS,
		int32 LWCComplexityPS,
		int32 ViewModeParam,
		FName ViewModeParamName,
		FMeshDrawSingleShaderBindings& ShaderBindings
	) const {}

	/** Return the interface object for the given viewmode. */
	static const FDebugViewModeInterface* GetInterface(EDebugViewShaderMode InDebugViewMode) 
	{
		return (uint32)InDebugViewMode < DVSM_MAX ? Singleton : nullptr;
	}

	/** Return the interface object for the given viewmode. */
	static ENGINE_API void SetInterface(FDebugViewModeInterface* Interface);
	
	/** Whether this material can be substituted by the default material. */
	static ENGINE_API bool AllowFallbackToDefaultMaterial(const FMaterial* InMaterial);
	static ENGINE_API bool AllowFallbackToDefaultMaterial(bool bHasVertexPositionOffsetConnected, bool bHasPixelDepthOffsetConnected);

private:
	
	static ENGINE_API FDebugViewModeInterface* Singleton;
};

#endif // ENABLE_DRAW_DEBUG
