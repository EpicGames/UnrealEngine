// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/BoxSphereBounds.h"

#define UE_API ENGINE_API

class UDecalComponent;
class UMaterialInterface;

struct FDeferredDecalSceneProxyDesc
{
	FDeferredDecalSceneProxyDesc() = default;
	UE_API FDeferredDecalSceneProxyDesc(const UDecalComponent* Decal);

	UMaterialInterface* DecalMaterial = nullptr;
	const UDecalComponent* Component = nullptr;
	
	FTransform TransformWithDecalScale = FTransform::Identity;
	FBoxSphereBounds Bounds{};

	FLinearColor DecalColor = FLinearColor::White;

	float InitializationWorldTimeSeconds = 0.0f;
	float FadeScreenSize = 0.0f;
	float FadeDuration = 0.0f;
	float FadeStartDelay = 0.0f;
	float FadeInDuration = 0.0f;
	float FadeInStartDelay = 0.0f;
	int32 SortOrder = INDEX_NONE;
		
	uint8 bDrawInGame : 1 = false;
	uint8 bDrawInEditor : 1 = false;
	uint8 bShouldFade : 1 = true;
};

#undef UE_API
