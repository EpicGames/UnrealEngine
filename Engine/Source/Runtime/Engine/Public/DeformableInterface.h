// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Interface.h"

#if WITH_EDITOR
#include "DetailLayoutBuilder.h"
#endif

#include "DeformableInterface.generated.h"


UINTERFACE(MinimalAPI)
class UDeformableInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IDeformableInterface
{
	GENERATED_IINTERFACE_BODY()

public:


#if WITH_EDITOR
	// Take damage
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) = 0;
#endif

};
