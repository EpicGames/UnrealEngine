// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Engine/LatentActionManager.h"
#include "LatentActions.h"

// FDelayAction
// A simple delay action; counts down and triggers it's output link when the time remaining falls to zero
class FDelayAction : public FPendingLatentAction
{
public:
	float TimeRemaining;
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	FDelayAction(float Duration, const FLatentActionInfo& LatentInfo)
		: TimeRemaining(Duration)
		, ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		TimeRemaining -= Response.ElapsedTime();
		Response.FinishAndTriggerIf(TimeRemaining <= 0.0f, ExecutionFunction, OutputLink, CallbackTarget);
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		static const FNumberFormattingOptions DelayTimeFormatOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(3)
			.SetMaximumFractionalDigits(3);
		return FText::Format(NSLOCTEXT("DelayAction", "DelayActionTimeFmt", "Delay ({0} seconds left)"), FText::AsNumber(TimeRemaining, &DelayTimeFormatOptions)).ToString();
	}
#endif
};


/** FDelayUntilNextTickAction
 * A simple delay action; triggers on the object's next tick. This tick *may* occur in the current engine frame, depending on order of operations.
 * See also @CVarLatentActionGuaranteeEngineTickDelay and FDelayUntilNextFrameAction for guaranteed delays until the next frame.
 */
class FDelayUntilNextTickAction : public FPendingLatentAction
{
public:
	uint8 InitialFrameParity; // engine frame parity when this action was created (0=even, 1=odd)
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;

	FDelayUntilNextTickAction(const FLatentActionInfo& LatentInfo)
		: InitialFrameParity((uint8)GFrameCounter & 1)
		, ExecutionFunction(LatentInfo.ExecutionFunction)
		, OutputLink(LatentInfo.Linkage)
		, CallbackTarget(LatentInfo.CallbackTarget)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		const bool bShouldFinish = !LatentActionCVars::GuaranteeEngineTickDelay || (((uint8)GFrameCounter & 1) != InitialFrameParity);
		Response.FinishAndTriggerIf(bShouldFinish, ExecutionFunction, OutputLink, CallbackTarget);
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		return NSLOCTEXT("DelayUntilNextTickAction", "DelayUntilNextTickActionFmt", "Delay for one tick").ToString();
	}
#endif
};

// A variant of the simple delay action that guarantees a delay until the next engine frame
class FDelayUntilNextFrameAction : public FDelayUntilNextTickAction
{
public:
	FDelayUntilNextFrameAction(const FLatentActionInfo& LatentInfo)
		: FDelayUntilNextTickAction(LatentInfo)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		const bool bHasFrameAdvanced = ((uint8)GFrameCounter & 1) != InitialFrameParity;
		Response.FinishAndTriggerIf(bHasFrameAdvanced, ExecutionFunction, OutputLink, CallbackTarget);
	}

#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	virtual FString GetDescription() const override
	{
		return NSLOCTEXT("DelayUntilNextFrameAction", "DelayUntilNextFrameActionFmt", "Delay for one engine frame").ToString();
	}
#endif
};

