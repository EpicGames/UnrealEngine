// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"

#if WITH_EDITOR
#include "WorldPartition/DataLayer/ExternalDataLayerUID.h"
#endif

#include "DeletedObjectPlaceholder.generated.h"

class UActorDescContainer;
class UDeletedObjectPlaceholder;

UCLASS(Transient, MinimalAPI)
class UDeletedObjectPlaceholder : public UObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	virtual bool IsAsset() const override;
	virtual void PostEditUndo() override;
	
	const UObject* GetOriginalObject() const { return OriginalObject.Get(true); }
	const FString& GetDisplayName() const { return DisplayName; }
	const FExternalDataLayerUID GetExternalDataLayerUID() const { return FExternalDataLayerUID(ExternalDataLayerUID); }

	static ENGINE_API UDeletedObjectPlaceholder* Create(UObject* InOuter, UPackage* InPackage, const UObject* InOriginalObject);
	static ENGINE_API UDeletedObjectPlaceholder* FindInPackage(const UPackage* InPackage);
	static ENGINE_API UDeletedObjectPlaceholder* RemoveFromPackage(UPackage* InPackage);

	DECLARE_MULTICAST_DELEGATE_OneParam(FObjectCreated, const UDeletedObjectPlaceholder*);
	static ENGINE_API FObjectCreated OnObjectCreated;
#endif

private:

#if WITH_EDITORONLY_DATA
	/** Original object's display name */
	UPROPERTY()
	FString DisplayName;

	/** Original object's external data layer UID */
	UPROPERTY()
	uint32 ExternalDataLayerUID;

	/** Original object that is replaced by the placeholder */
	UPROPERTY()
	TWeakObjectPtr<const UObject> OriginalObject;
#endif
};