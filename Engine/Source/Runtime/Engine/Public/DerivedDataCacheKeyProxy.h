// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Templates/TypeCompatibleBytes.h"

namespace UE::DerivedData
{

struct FCacheKey;

namespace Private
{
	struct FCacheKeyDummy
	{
		const ANSICHAR* BucketNamePtrDummy = nullptr;
		FIoHash HashDummy;
	};
} // Private

struct FCacheKeyProxy : private TAlignedBytes<sizeof(UE::DerivedData::Private::FCacheKeyDummy), alignof(UE::DerivedData::Private::FCacheKeyDummy)>
{
	FCacheKeyProxy(const FCacheKey& InKey);
	ENGINE_API ~FCacheKeyProxy();
	FCacheKey* AsCacheKey() { return (FCacheKey*)this;  }
	const FCacheKey* AsCacheKey() const { return (const FCacheKey*)this; }
};

} // UE::DerivedData
