// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SourceMeshDataForDerivedDataTask.h
=============================================================================*/

#pragma once

#include "Components.h"
#include "StaticMeshResources.h"

/** Source mesh data. Valid only in specific cases, when StaticMesh doesn't contain original data anymore (e.g. it's replaced by the Nanite coarse mesh representation) */
class FSourceMeshDataForDerivedDataTask
{
public:
	TArray<uint32> TriangleIndices;
	TArray<FVector3f> VertexPositions;
	FStaticMeshSectionArray Sections;

	int32 GetNumIndices() const
	{
		return TriangleIndices.Num();
	}

	int32 GetNumVertices() const
	{
		return VertexPositions.Num();
	}

	bool IsValid() const
	{ 
		return TriangleIndices.Num() > 0;
	}
};