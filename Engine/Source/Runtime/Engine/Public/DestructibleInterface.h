// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Interface.h"
#include "DestructibleInterface.generated.h"

UINTERFACE(MinimalAPI)
class UDestructibleInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IDestructibleInterface
{
	GENERATED_IINTERFACE_BODY()
public:
	// Take damage
	virtual void ApplyDamage(float DamageAmount, const FVector& HitLocation, const FVector& ImpulseDir, float ImpulseStrength) = 0;

	// Take radius damage
	virtual void ApplyRadiusDamage(float BaseDamage, const FVector& HurtOrigin, float DamageRadius, float ImpulseStrength, bool bFullDamage) = 0;
};
