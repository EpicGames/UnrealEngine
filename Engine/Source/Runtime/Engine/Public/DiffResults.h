// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"

class SWidget;

/** Differences found within a graph or object */
namespace EDiffType
{
	/** Differences are presented to the user in the order listed here, so put less important differences lower down */
	enum Type
	{
		// No actual difference
		NO_DIFFERENCE,
		
		// Object does not exist in current version
		OBJECT_REMOVED,

		// Object was added to current version
		OBJECT_ADDED,

		// An individual property differs
		OBJECT_PROPERTY,

		// Requests a full diff to be done on two objects, this will expand into other diffs
		OBJECT_REQUEST_DIFF,

		NODE_REMOVED,
		NODE_ADDED,
		PIN_LINKEDTO_NUM_DEC,
		PIN_LINKEDTO_NUM_INC,
		PIN_DEFAULT_VALUE,
		PIN_TYPE_CATEGORY,
		PIN_TYPE_SUBCATEGORY,
		PIN_TYPE_SUBCATEGORY_OBJECT,
		PIN_TYPE_IS_ARRAY,
		PIN_TYPE_IS_REF,
		PIN_LINKEDTO_NODE,
		PIN_LINKEDTO_PIN, // only used when a pin was relinked to a different pin on the same node
		NODE_MOVED,
		TIMELINE_LENGTH,
		TIMELINE_AUTOPLAY,
		TIMELINE_LOOP,
		TIMELINE_IGNOREDILATION,
		TIMELINE_NUM_TRACKS,
		TIMELINE_TRACK_MODIFIED,
		NODE_PIN_COUNT,
		NODE_COMMENT,
		NODE_PROPERTY,

		// Informational message, does't count as a real diff
		INFO_MESSAGE,

		// User defined custom diff. Allows user to override property diffing and panel widgets
		CUSTOM_OBJECT
	};
	
	enum Category
	{
		ADDITION,
		SUBTRACTION,
		MODIFICATION,
		
		// used for small changes like moving nodes that don't effect the compilation
		MINOR,

		// used for items that are purely informational and don't show up in the diff results
		CONTROL,
	};
}

/** Result of a single difference found on graph or object */
struct FDiffSingleResult
{
	FDiffSingleResult()
	{
		Diff = EDiffType::NO_DIFFERENCE;
		Node1 = nullptr;
		Node2 = nullptr;
		Pin1 = nullptr;
		Pin2 = nullptr;
		Object1 = nullptr;
		Object2 = nullptr;
	}

	/** The type of diff */
	EDiffType::Type Diff;

	EDiffType::Category Category;

	/** The first node involved in diff */
	class UEdGraphNode* Node1;

	/** The second node involved in diff */
	class UEdGraphNode* Node2;

	/** The first pin involved in diff */
	class UEdGraphPin* Pin1;

	/** The second pin involved in diff */
	class UEdGraphPin* Pin2;

	/** First top-level object involved in a diff */
	const UObject* Object1;

	/** Second top-level object involved in a diff */
	const UObject* Object2;

	/** String describing the error to the user */
	FText DisplayString;

	/** Optional tooltip containing more information */
	FText ToolTip; 

	/** Custom Object Diff. Used with CUSTOM_OBJECT diff types */
	TSharedPtr<class IDiffCustomObject> CustomObject;
	TSharedPtr<class IDiffCustomProperty> CustomProperty;

	/** Get the color that is associated with this diff category */
	FLinearColor GetDisplayColor() const
	{
		switch(Category)
		{
		case EDiffType::ADDITION: return FAppStyle::Get().GetSlateColor("SourceControl.Diff.AdditionColor").GetSpecifiedColor();
		case EDiffType::SUBTRACTION: return FAppStyle::Get().GetSlateColor("SourceControl.Diff.SubtractionColor").GetSpecifiedColor();
		case EDiffType::MODIFICATION: return FAppStyle::Get().GetSlateColor("SourceControl.Diff.MajorModificationColor").GetSpecifiedColor();
		case EDiffType::MINOR:  return FAppStyle::Get().GetSlateColor("SourceControl.Diff.MinorModificationColor").GetSpecifiedColor();
		
		default: return FStyleColors::Foreground.GetSpecifiedColor();
		}
	}

	/** Path string of graph, relative to blueprint/asset root */
	FString OwningObjectPath;

	/** Returns true if this is a confirmed difference */
	inline bool IsRealDifference() const 
	{
		switch (Diff)
		{
		case EDiffType::NO_DIFFERENCE:
		case EDiffType::OBJECT_REQUEST_DIFF:
		case EDiffType::INFO_MESSAGE:
			return false;
		default:
			return true;
		}
	}
};

inline bool operator==( const FDiffSingleResult& LHS, const FDiffSingleResult& RHS )
{
	return	LHS.Diff == RHS.Diff &&
			LHS.Node1 == RHS.Node1 &&
			LHS.Node2 == RHS.Node2 &&
			LHS.Pin1 == RHS.Pin1 &&
			LHS.Pin2 == RHS.Pin2 &&
			LHS.Object1 == RHS.Object1 &&
			LHS.Object2 == RHS.Object2 &&
			LHS.DisplayString.ToString() == RHS.DisplayString.ToString() &&
			LHS.ToolTip.ToString() == RHS.ToolTip.ToString() &&
			LHS.Category == RHS.Category;
}

/** Collects the Diffs found for a node/object */
struct FDiffResults
{
	FDiffResults(TArray<FDiffSingleResult>* InResultArray): ResultArray(InResultArray), bHasFoundDiffs(false)
	{}

	/** Add a diff that was found */
	void Add(const FDiffSingleResult& Result)
	{
		if(Result.Diff != EDiffType::NO_DIFFERENCE)
		{
			bHasFoundDiffs = true;
			if(ResultArray)
			{
				ResultArray->Add(Result);
			}
		}
	}

	/** Test if it can store results*/
	bool CanStoreResults() const
	{
		return ResultArray != nullptr;
	}

	/** Get the number of diffs found*/
	int32 Num() const { return ResultArray ? ResultArray->Num() : 0;}

	/** True if diffs were found */
	bool HasFoundDiffs() const { return bHasFoundDiffs; }

private:
	/** Pointer to optional array, passed in by user to store results in */
	TArray<FDiffSingleResult>* ResultArray;
	bool bHasFoundDiffs;
};

/** 
 * Interface to support custom widget for object diffs
 */
class IDiffCustomObject : public TSharedFromThis<IDiffCustomObject>
{
public:
	virtual ~IDiffCustomObject() = default;
	virtual TSharedPtr<SWidget> GetOldWidget() = 0;
	virtual TSharedPtr<SWidget> GetNewWidget() = 0;
};

class IDiffCustomProperty : public TSharedFromThis<IDiffCustomProperty>
{
public:
	virtual ~IDiffCustomProperty() = default;
	virtual void HighlightProperty(const TSharedPtr<IDiffCustomObject> Widget) = 0;
};
