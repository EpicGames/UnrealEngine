// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Tracks what debug information we have switched on
class FDebugDisplayInfo
{
private:

	TArray<FName> DisplayNames;
	TArray<FName> ToggledCategories;

public:

	FDebugDisplayInfo(const TArray<FName>& InDisplayNames, const TArray<FName>& InToggledCategories) : DisplayNames(InDisplayNames), ToggledCategories(InToggledCategories) {}

	bool IsDisplayOn(FName DisplayName) const { return DisplayNames.Contains(DisplayName); }
	bool IsCategoryToggledOn(FName Category, bool bDefaultsToOn) const { return ToggledCategories.Contains(Category) != bDefaultsToOn; }
	int32 NumDisplayNames() const { return DisplayNames.Num(); }
};

// Simple object to track scope indentation
class FIndenter
{
	float& Indent;

public:
	FIndenter(float& InIndent) : Indent(InIndent) { Indent += 4.0f; }
	~FIndenter() { Indent -= 4.0f; }
};
