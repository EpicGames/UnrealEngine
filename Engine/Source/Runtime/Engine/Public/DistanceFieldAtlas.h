// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldAtlas.h
=============================================================================*/

#pragma once

#include "Containers/LockFreeList.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Engine/EngineTypes.h"
#include "UObject/GCObject.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"
#include "TextureLayout3d.h"
#include "IAssetCompilingManager.h"
#include "Templates/UniquePtr.h"
#include "DerivedMeshDataTaskUtils.h"
#include "Async/AsyncWork.h"

#if WITH_EDITOR
#include "MeshUtilities.h"
#endif

struct FAppendToClassSchemaContext;
struct FAssetCompileData;
class FAsyncCompilationNotification;
class FObjectCacheContextScope;
class FDistanceFieldVolumeData;
class UStaticMesh;
class UTexture2D;

template <class T> class TLockFreePointerListLIFO;

// Change DDC key when modifying these (or any DF encoding logic)
namespace DistanceField
{
	// One voxel border around object for handling gradient
	inline constexpr int32 MeshDistanceFieldObjectBorder = 1;
	inline constexpr int32 UniqueDataBrickSize = 7;
	// Half voxel border around brick for trilinear filtering
	inline constexpr int32 BrickSize = 8;
	// Trade off between SDF memory and number of steps required to find intersection
	inline constexpr int32 BandSizeInVoxels = 4;
	inline constexpr int32 NumMips = 3;
	inline constexpr uint32 InvalidBrickIndex = 0xFFFFFFFF;
	inline constexpr EPixelFormat DistanceFieldFormat = PF_G8;

	// Must match LoadDFAssetData
	inline constexpr uint32 MaxIndirectionDimension = 1024;
};

class FLandscapeTextureAtlas : public FRenderResource
{
public:
	enum ESubAllocType
	{
		SAT_Height,
		SAT_Visibility,
		SAT_Num
	};

	ENGINE_API FLandscapeTextureAtlas(ESubAllocType InSubAllocType);

	ENGINE_API void InitializeIfNeeded();

	ENGINE_API void AddAllocation(UTexture2D* Texture, uint32 VisibilityChannel = 0);

	ENGINE_API void RemoveAllocation(UTexture2D* Texture);

	ENGINE_API void UpdateAllocations(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type InFeatureLevel);

	UE_DEPRECATED(5.0, "This method has been refactored to use an FRDGBuilder instead.")
	ENGINE_API void UpdateAllocations(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel);

	ENGINE_API uint32 GetAllocationHandle(UTexture2D* Texture) const;

	ENGINE_API FVector4f GetAllocationScaleBias(uint32 Handle) const;

	bool HasAtlasTexture() const { return AtlasTextureRHI.IsValid(); }

	ENGINE_API FRDGTexture* GetAtlasTexture(FRDGBuilder& GraphBuilder) const;

	uint32 GetSizeX() const
	{
		return AddrSpaceAllocator.DimInTexels;
	}

	uint32 GetSizeY() const
	{
		return AddrSpaceAllocator.DimInTexels;
	}

	uint32 GetGeneration() const
	{
		return Generation;
	}

private:
	ENGINE_API uint32 CalculateDownSampleLevel(uint32 SizeX, uint32 SizeY) const;

	class FSubAllocator
	{
	public:
		void Init(uint32 InTileSize, uint32 InBorderSize, uint32 InDimInTiles);

		uint32 Alloc(uint32 SizeX, uint32 SizeY);

		void Free(uint32 Handle);

		FVector4f GetScaleBias(uint32 Handle) const;

		FIntPoint GetStartOffset(uint32 Handle) const;

	private:
		struct FSubAllocInfo
		{
			uint32 Level;
			uint32 QuadIdx;
			FVector4f UVScaleBias;
		};
		
		uint32 TileSize;
		uint32 BorderSize;
		uint32 TileSizeWithBorder;
		uint32 DimInTiles;
		uint32 DimInTilesShift;
		uint32 DimInTilesMask;
		uint32 DimInTexels;
		uint32 MaxNumTiles;

		float TexelSize;
		float TileScale;

		// 0: Free, 1: Allocated
		TBitArray<> MarkerQuadTree;
		TArray<uint32, TInlineAllocator<8>> LevelOffsets;

		TSparseArray<FSubAllocInfo> SubAllocInfos;

		friend class FLandscapeTextureAtlas;
	};

	struct FAllocation
	{
		UTexture2D* SourceTexture;
		uint32 Handle;
		uint32 VisibilityChannel : 2;
		uint32 RefCount : 30;

		FAllocation();

		FAllocation(UTexture2D* InTexture, uint32 InVisibilityChannel = 0);

		bool operator==(const FAllocation& Other) const
		{
			return SourceTexture == Other.SourceTexture;
		}

		friend uint32 GetTypeHash(const FAllocation& Key)
		{
			return GetTypeHash(Key.SourceTexture);
		}
	};

	struct FPendingUpload
	{
		FRHITexture* SourceTexture;
		FIntVector SizesAndMipBias;
		uint32 VisibilityChannel : 2;
		uint32 Handle : 30;

		FPendingUpload(UTexture2D* Texture, uint32 SizeX, uint32 SizeY, uint32 MipBias, uint32 InHandle, uint32 Channel);

		FIntPoint SetShaderParameters(void* ParamsPtr, const FLandscapeTextureAtlas& Atlas, FRDGTextureUAV* AtlasUAV) const;

	private:
		FIntPoint SetCommonShaderParameters(void* ParamsPtr, const FLandscapeTextureAtlas& Atlas) const;
	};

	FSubAllocator AddrSpaceAllocator;

	TSet<FAllocation> PendingAllocations;
	TSet<FAllocation> FailedAllocations;
	TSet<FAllocation> CurrentAllocations;
	TArray<UTexture2D*> PendingStreamingTextures;

	TRefCountPtr<IPooledRenderTarget> AtlasTextureRHI;

	uint32 MaxDownSampleLevel;
	uint32 Generation;

	const ESubAllocType SubAllocType;
};

extern ENGINE_API TGlobalResource<FLandscapeTextureAtlas> GHeightFieldTextureAtlas;
extern ENGINE_API TGlobalResource<FLandscapeTextureAtlas> GHFVisibilityTextureAtlas;

class FSparseDistanceFieldMip
{
public:

	FSparseDistanceFieldMip() :
		IndirectionDimensions(FInt32Vector::ZeroValue),
		NumDistanceFieldBricks(0),
		VolumeToVirtualUVScale(FVector3f::ZeroVector),
		VolumeToVirtualUVAdd(FVector3f::ZeroVector),
		DistanceFieldToVolumeScaleBias(FVector2f::ZeroVector),
		BulkOffset(0),
		BulkSize(0)
	{}

	FInt32Vector IndirectionDimensions;
	int32 NumDistanceFieldBricks;
	FVector3f VolumeToVirtualUVScale;
	FVector3f VolumeToVirtualUVAdd;
	FVector2f DistanceFieldToVolumeScaleBias;
	uint32 BulkOffset;
	uint32 BulkSize;

	friend FArchive& operator<<(FArchive& Ar,FSparseDistanceFieldMip& Mip)
	{
		Ar << Mip.IndirectionDimensions << Mip.NumDistanceFieldBricks << Mip.VolumeToVirtualUVScale << Mip.VolumeToVirtualUVAdd << Mip.DistanceFieldToVolumeScaleBias << Mip.BulkOffset << Mip.BulkSize;
		return Ar;
	}

	SIZE_T GetResourceSizeBytes() const
	{
		FResourceSizeEx ResSize;
		GetResourceSizeEx(ResSize);
		return ResSize.GetTotalMemoryBytes();
	}

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
	}
};

/** Distance field data payload and output of the mesh build process. */
class FDistanceFieldVolumeData : public FDeferredCleanupInterface
{
public:

	/** Local space bounding box of the distance field volume. */
	FBox3f LocalSpaceMeshBounds;

	/** Whether most of the triangles in the mesh used a two-sided material. */
	bool bMostlyTwoSided;

	bool bAsyncBuilding;

	TStaticArray<FSparseDistanceFieldMip, DistanceField::NumMips> Mips;

	// Lowest resolution mip is always loaded so we always have something
	TArray<uint8> AlwaysLoadedMip;

	// Remaining mips are streamed
	FByteBulkData StreamableMips;

	uint64 Id;

	// For stats
	FName AssetName;

	ENGINE_API FDistanceFieldVolumeData();

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
		
		for (const FSparseDistanceFieldMip& Mip : Mips)
		{
			Mip.GetResourceSizeEx(CumulativeResourceSize);
		}

		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(AlwaysLoadedMip.GetAllocatedSize());
	}

	SIZE_T GetResourceSizeBytes() const
	{
		FResourceSizeEx ResSize;
		GetResourceSizeEx(ResSize);
		return ResSize.GetTotalMemoryBytes();
	}

	bool IsValid() const
	{
		return Mips[0].IndirectionDimensions.GetMax() > 0;
	}

#if WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.6, "Distance fields are always generated from the UStaticMesh's base render data, and platform independent.  Simplified version of function below removes now unused arguments.")
	ENGINE_API void CacheDerivedData(const FString& InStaticMeshDerivedDataKey, const ITargetPlatform* TargetPlatform, UStaticMesh* Mesh, class FStaticMeshRenderData& RenderData, UStaticMesh* GenerateSource, float DistanceFieldResolutionScale, bool bGenerateDistanceFieldAsIfTwoSided);

	ENGINE_API void CacheDerivedData(UStaticMesh* Mesh, UStaticMesh* GenerateSource, float DistanceFieldResolutionScale, bool bGenerateDistanceFieldAsIfTwoSided);

#endif

	ENGINE_API void Serialize(FArchive& Ar, UObject* Owner);

	uint64 GetId() const { return Id; }
};

class FAsyncDistanceFieldTask;
class FAsyncDistanceFieldTaskWorker : public FNonAbandonableTask
{
public:
	FAsyncDistanceFieldTaskWorker(FAsyncDistanceFieldTask& InTask)
		: Task(InTask)
	{
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncDistanceFieldTaskWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();

private:
	FAsyncDistanceFieldTask& Task;
};

/** A task to build a distance field for a single mesh */
class FAsyncDistanceFieldTask
{
public:
	FAsyncDistanceFieldTask();

#if WITH_EDITOR
	TArray<FSignedDistanceFieldBuildSectionData> SectionData;
	TArray<FSignedDistanceFieldBuildPartData> PartData;
#endif
	FSourceMeshDataForDerivedDataTask SourceMeshData;
	UStaticMesh* StaticMesh;
	UStaticMesh* GenerateSource;
	float DistanceFieldResolutionScale;
	bool bGenerateDistanceFieldAsIfTwoSided;
	const ITargetPlatform* TargetPlatform;
	FString DDCKey;
	FDistanceFieldVolumeData* GeneratedVolumeData;
	TUniquePtr<FAsyncTask<FAsyncDistanceFieldTaskWorker>> AsyncTask = nullptr;
	bool bGeneratedCardRepresentation = false;
};

/** Class that manages asynchronous building of mesh distance fields. */
class FDistanceFieldAsyncQueue : IAssetCompilingManager
{
public:

	ENGINE_API FDistanceFieldAsyncQueue();

	ENGINE_API virtual ~FDistanceFieldAsyncQueue();

	/** Adds a new build task. (Thread-Safe) */
	ENGINE_API void AddTask(FAsyncDistanceFieldTask* Task);

	/** Cancel the build on this specific static mesh or block until it is completed if already started. */
	ENGINE_API void CancelBuild(UStaticMesh* StaticMesh);

	/** Cancel the build on these meshes or block until they are completed if already started. */
	ENGINE_API void CancelBuilds(const TSet<UStaticMesh*>& InStaticMeshes);

	/** Blocks the main thread until the async build are either canceled or completed. */
	ENGINE_API void CancelAllOutstandingBuilds();

	/** Blocks the main thread until the async build of the specified mesh is complete. */
	ENGINE_API void BlockUntilBuildComplete(UStaticMesh* StaticMesh, bool bWarnIfBlocked);

	/** Blocks the main thread until all async builds complete. */
	ENGINE_API void BlockUntilAllBuildsComplete();

	/** Called once per frame, fetches completed tasks and applies them to the scene. */
	ENGINE_API void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	/** Fetches completed tasks for a specific static mesh and applies them to the scene */
	ENGINE_API void ProcessAsyncTasksForMesh(const UStaticMesh* InStaticMesh, bool bLimitExecutionTime = false);

	/** Blocks until it is safe to shut down (worker threads are idle). */
	ENGINE_API void Shutdown() override;

	int32 GetNumOutstandingTasks() const
	{
		FScopeLock Lock(&CriticalSection);
		return ReferencedTasks.Num();
	}

	/** Returns the number of outstanding referenced tasks for a specific static mesh. */
	ENGINE_API int32 GetNumOutstandingTasks(const UStaticMesh* InStaticMesh) const;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void FinishAllCompilation() override;

	friend FAsyncDistanceFieldTaskWorker;
	void ProcessPendingTasks();
	void ProcessPendingTasksForMesh(const UStaticMesh* InStaticMesh);

	TUniquePtr<FQueuedThreadPool> ThreadPool;

	/** Builds a single task. Called from the worker thread. */
	void Build(FAsyncDistanceFieldTask* Task);

	/** Change the priority of the background task. */
	void RescheduleBackgroundTask(FAsyncDistanceFieldTask* InTask, EQueuedWorkPriority InPriority);

	/** Task will be sent to a background worker. */
	void StartBackgroundTask(FAsyncDistanceFieldTask* Task);

	/** Cancel or finish any work for any task matching the predicate. */
	void CancelAndDeleteTaskByPredicate(TFunctionRef<bool(FAsyncDistanceFieldTask*)> ShouldCancelPredicate);

	/** Cancel or finish any work for the given task. */
	void CancelAndDeleteTask(const TSet<FAsyncDistanceFieldTask*>& Tasks);

	/** Handle generic finish compilation */
	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	/** Mark compilation of the provided objects as canceled. */
	void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects) override;

	/** Return whether the task has become invalid and should be cancelled (i.e. reference unreachable objects) */
	bool IsTaskInvalid(FAsyncDistanceFieldTask* Task) const;

	/** Used to cancel tasks that are not needed anymore when garbage collection occurs */
	void OnPostReachabilityAnalysis();

	/** Get notified when static mesh finish compiling */
	void OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets);

#if WITH_EDITOR
	/** Finishes a singular async task to completion, but does not delete it */
	void FinalizeAsyncTask(FAsyncDistanceFieldTask* InTask, FObjectCacheContextScope& InObjectCacheScope) const;
#endif

	/** Game-thread managed list of tasks in the async system. */
	TSet<FAsyncDistanceFieldTask*> ReferencedTasks;

	/** Tasks that are waiting on static mesh compilation to proceed */
	TSet<FAsyncDistanceFieldTask*> PendingTasks;

	/** Tasks that have completed processing. */
	TSet<FAsyncDistanceFieldTask*> CompletedTasks;

	FDelegateHandle PostReachabilityAnalysisHandle;

	std::atomic<class IMeshUtilities*> MeshUtilities;

	mutable FCriticalSection CriticalSection;

	TUniquePtr<FAsyncCompilationNotification> Notification;
};

/** Global build queue. */
extern ENGINE_API FDistanceFieldAsyncQueue* GDistanceFieldAsyncQueue;

#if WITH_EDITOR
/**
 * IncrementalCook: Any UClass that serializes cooked data with data loaded from DDC with key created from
 * BuildDistanceFieldDerivedDataKey must add the version numbers for that calculation to their ClassSchema,
 * to invalidate their cooked data when those constants change. This function handles that appending.
 */
extern ENGINE_API void AppendDistanceFieldConstantsToClassSchema(FAppendToClassSchemaContext& Context);

extern ENGINE_API FString BuildDistanceFieldDerivedDataKey(const FString& InMeshKey);
#endif
