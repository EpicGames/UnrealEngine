// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldDownsampling.h
=============================================================================*/

#pragma once

#include "RHI.h"

class FRHICommandListImmediate;
class FRDGBuilder;

struct FDistanceFieldDownsamplingDataTask
{
	FTextureRHIRef VolumeTextureRHI;
	FVector TexelSrcSize;
	FIntVector DstSize;
	FIntVector OffsetInAtlas;
};

class FDistanceFieldDownsampling
{
public:
	static ENGINE_API bool CanDownsample();
	static ENGINE_API void GetDownsampledSize(const FIntVector& Size, float Factor, FIntVector& OutDownsampledSize);
	static ENGINE_API void FillDownsamplingTask(const FIntVector& SrcSize, const FIntVector& DstSize, const FIntVector& OffsetInAtlas, EPixelFormat Format, FDistanceFieldDownsamplingDataTask& OutDataTask, FUpdateTexture3DData& OutTextureUpdateData);
	static ENGINE_API void DispatchDownsampleTasks(FRDGBuilder& GraphBuilder, FRHIUnorderedAccessView* DFAtlasUAV, ERHIFeatureLevel::Type FeatureLevel, TArray<FDistanceFieldDownsamplingDataTask>& DownsamplingTasks, TArray<FUpdateTexture3DData>& UpdateTextureData);

	UE_DEPRECATED(5.0, "This method has been refactored to use an FRDGBuilder instead.")
	static ENGINE_API void DispatchDownsampleTasks(FRHICommandListImmediate& RHICmdList, FRHIUnorderedAccessView* DFAtlasUAV, ERHIFeatureLevel::Type FeatureLevel, TArray<FDistanceFieldDownsamplingDataTask>& DownsamplingTasks, TArray<FUpdateTexture3DData>& UpdateTextureData);
};
