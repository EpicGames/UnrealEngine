// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Distributions.h: Declaration of distributions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Math/RandomStream.h"

/** Usually checkSlow for performance. */
#define checkDistribution check

/**
 * Operation to perform when looking up a value.
 */
enum ERawDistributionOperation
{
	RDO_Uninitialized,
	RDO_None,
	RDO_Random,
	RDO_Extreme,
};

/**
 * Lookup table used to sample distributions at runtime.
 */
struct FDistributionLookupTable
{
	/** Time between values in the lookup table */
	float TimeScale;
	/** Absolute time of the first value */
	float TimeBias;
	/** Values in the table. */
	TArray<float> Values;
	/** Operation for which the table was built. */
	uint8 Op;
	/** Number of entries in the table. */
	uint8 EntryCount;
	/** Number of values between entries [1,8]. */
	uint8 EntryStride;
	/** Number of values between sub-entries [0,4]. */
	uint8 SubEntryStride;
	/** Lock axes flag for vector distributions. */
	uint8 LockFlag;
	
	/** Default constructor. */
	FDistributionLookupTable()
		: TimeScale(0.0f)
		, TimeBias(0.0f)
		, Op(RDO_Uninitialized)
		, EntryCount(0)
		, EntryStride(0)
		, SubEntryStride(0)
		, LockFlag(0)
	{}

	/**
	 * Empties the table of all values.
	 */
	void Empty()
	{
		Op = RDO_Uninitialized;
		EntryCount = 0;
		EntryStride = 0;
		SubEntryStride = 0;
		TimeScale = 0.0f;
		TimeBias = 0.0f;
		LockFlag = 0;
	}

	/**
	 * Returns true if the lookup table contains no values.
	 */
	inline bool IsEmpty() const
	{
		return Values.Num() == 0 || EntryCount == 0;
	}

	/**
	 * Computes the number of Values per entry in the table.
	 */
	inline float GetValuesPerEntry() const
	{
		return (float)(EntryStride - SubEntryStride);
	}

	/**
	 * Compute the number of values contained in the table.
	 */
	inline float GetValueCount() const
	{
		return (float)(Values.Num());
	}

	/**
	 * Get the entry for Time and the one after it for interpolating (along with 
	 * an alpha for interpolation)
	 * 
	 * @param Time The time we are looking to retrieve
	 * @param Entry1 Out variable that is the first (or only) entry
	 * @param Entry2 Out variable that is the second entry (for interpolating)
	 * @param LerpAlpha Out variable that is the alpha for interpolating between Entry1 and Entry2
	 */
	inline void GetEntry( float Time, const float*& Entry1, const float*& Entry2, float& LerpAlpha ) const
	{
		// make time relative to start time
		Time -= TimeBias;
		Time *= TimeScale;
		Time = FMath::FloatSelect( Time, Time, 0.0f );

		// calculate the alpha to lerp between entry1 and entry2
		LerpAlpha = FMath::Fractional(Time);

		// get the entries to lerp between
		const uint32 Index = FMath::TruncToInt(Time);
		const uint32 Index1 = FMath::Min<uint32>( Index + 0, EntryCount - 1 ) * EntryStride;
		const uint32 Index2 = FMath::Min<uint32>( Index + 1, EntryCount - 1 ) * EntryStride;
		Entry1 = &Values[Index1];
		Entry2 = &Values[Index2];
	}

	/**
	 * Get the range of values produced by the table.
	 * @note: in the case of a constant curve, this will not be exact!
	 * @param OutMinValues - The smallest values produced by this table.
	 * @param OutMaxValues - The largest values produced by this table.
	 */
	void GetRange( float* OutMinValues, float* OutMaxValues )
	{
		if ( EntryCount > 0 )
		{
			const int32 ValuesPerEntry = (int32)GetValuesPerEntry();
			const float* Entry = Values.GetData();

			// Initialize to the first entry in the table.
			for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
			{
				OutMinValues[ValueIndex] = Entry[ValueIndex];
				OutMaxValues[ValueIndex] = Entry[ValueIndex + SubEntryStride];
			}

			// Iterate over each entry updating the minimum and maximum values.
			for ( int32 EntryIndex = 1; EntryIndex < EntryCount; ++EntryIndex )
			{
				Entry += EntryStride;
				for ( int32 ValueIndex = 0; ValueIndex < ValuesPerEntry; ++ValueIndex )
				{
					OutMinValues[ValueIndex] = FMath::Min( OutMinValues[ValueIndex], Entry[ValueIndex] );
					OutMaxValues[ValueIndex] = FMath::Max( OutMaxValues[ValueIndex], Entry[ValueIndex + SubEntryStride] );
				}
			}
		}
	}
};

// Helper macro for retrieving random value
#define DIST_GET_RANDOM_VALUE(RandStream)		((RandStream == NULL) ? FMath::SRand() : RandStream->GetFraction())

/**
 * Raw distribution used to quickly sample distributions at runtime.
 */
struct FRawDistribution
{
	/** Default constructor. */
	FRawDistribution()
	{
	}

	/**
	 * Serialization.
	 * @param Ar - The archive with which to serialize.
	 * @returns true if serialization was successful.
	 */
	ENGINE_API bool Serialize( FArchive& Ar );

	/**
	 * Calcuate the float or vector value at the given time 
	 * @param Time The time to evaluate
	 * @param Value An array of (1 or 3) FLOATs to receive the values
	 * @param NumCoords The number of floats in the Value array
	 * @param Extreme For distributions that use one of the extremes, this is which extreme to use
	 */
	ENGINE_API void GetValue(float Time, float* Value, int32 NumCoords, int32 Extreme, struct FRandomStream* InRandomStream) const;

	// prebaked versions of these
	ENGINE_API void GetValue1(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const;
	ENGINE_API void GetValue3(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const;
	inline void GetValue1None(float Time, float* InValue) const
	{
		float* Value = InValue;
		const float* Entry1;
		const float* Entry2;
		float LerpAlpha = 0.0f;
		LookupTable.GetEntry(Time, Entry1, Entry2, LerpAlpha);
		const float* NewEntry1 = Entry1;
		const float* NewEntry2 = Entry2;
		Value[0] = FMath::Lerp(NewEntry1[0], NewEntry2[0], LerpAlpha);
	}
	inline void GetValue3None(float Time, float* InValue) const
	{
		float* Value = InValue;
		const float* Entry1;
		const float* Entry2;
		float LerpAlpha = 0.0f;
		LookupTable.GetEntry(Time, Entry1, Entry2, LerpAlpha);
		const float* NewEntry1 = Entry1;
		const float* NewEntry2 = Entry2;
		float T0 = FMath::Lerp(NewEntry1[0], NewEntry2[0], LerpAlpha);
		float T1 = FMath::Lerp(NewEntry1[1], NewEntry2[1], LerpAlpha);
		float T2 = FMath::Lerp(NewEntry1[2], NewEntry2[2], LerpAlpha);
		Value[0] = T0;
		Value[1] = T1;
		Value[2] = T2;
	}
	ENGINE_API void GetValue1Extreme(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const;
	ENGINE_API void GetValue3Extreme(float Time, float* Value, int32 Extreme, struct FRandomStream* InRandomStream) const;
	ENGINE_API void GetValue1Random(float Time, float* Value, struct FRandomStream* InRandomStream) const;
	ENGINE_API void GetValue3Random(float Time, float* Value, struct FRandomStream* InRandomStream) const;

	inline bool IsSimple() 
	{
		return LookupTable.Op == RDO_None;
	}

	/**
	 * Return the UDistribution* variable if the given StructProperty
	 * points to a FRawDistribution* struct
	 * @param Property Some UStructProperty
	 * @param Data Memory that owns the property
	 * @return The UDisitribution* object if this is a FRawDistribution* struct, 
	 *         or NULL otherwise
	 */
	static ENGINE_API UObject* TryGetDistributionObjectFromRawDistributionProperty(FStructProperty* Property, uint8* Data);

protected:

	/** Lookup table of values */
	FDistributionLookupTable LookupTable;
};

/*-----------------------------------------------------------------------------
	Type safe distributions.

	Like FRawDistribution but typesafe and not tied directly to UObjects.
-----------------------------------------------------------------------------*/

/**
 * Raw distribution from which one float can be looked up per entry.
 */
class FFloatDistribution
{
public:
	/** Default constructor. */
	FFloatDistribution();

	/**
	 * Samples a value from the distribution.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 */
	inline void GetValue( float Time, float* RESTRICT OutValue ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 1 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );
		OutValue[0] = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
	}

	/**
	 * Samples a value randomly distributed between two values.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 * @param RandomStream - Random stream from which to retrieve random fractions.
	 */
	inline void GetRandomValue( float Time, float* RESTRICT OutValue, FRandomStream& RandomStream ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 1 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;
		float RandomAlpha;
		float MinValue, MaxValue;
		const int32 SubEntryStride = LookupTable.SubEntryStride;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );
		MinValue = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
		MaxValue = FMath::Lerp( Entry1[SubEntryStride], Entry2[SubEntryStride], Alpha );
		RandomAlpha = RandomStream.GetFraction();
		OutValue[0] = FMath::Lerp( MinValue, MaxValue, RandomAlpha );
	}

	/**
	 * Computes the range of the distribution.
	 * @param OutMin - The minimum value in the distribution.
	 * @param OutMax - The maximum value in the distribution.
	 */
	void GetRange( float* OutMin, float* OutMax )
	{
		LookupTable.GetRange( OutMin, OutMax );
	}

private:
	/** The lookup table. */
	FDistributionLookupTable LookupTable;

	//@todo.CONSOLE: These are needed until we have a cooker/package in place!
	/** This distribution is built from composable distributions. */
	friend class FComposableDistribution;
};

/**
 * Raw distribution from which three floats can be looked up per entry.
 */
class FVectorDistribution
{
public:
	/** Default constructor. */
	FVectorDistribution();

	/**
	 * Samples a value from the distribution.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 */
	inline void GetValue( float Time, float* RESTRICT OutValue ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 3 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );
		OutValue[0] = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
		OutValue[1] = FMath::Lerp( Entry1[1], Entry2[1], Alpha );
		OutValue[2] = FMath::Lerp( Entry1[2], Entry2[2], Alpha );
	}

	/**
	 * Samples a value randomly distributed between two values.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 * @param RandomStream - Random stream from which to retrieve random fractions.
	 */
	inline void GetRandomValue( float Time, float* RESTRICT OutValue, FRandomStream& RandomStream ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 3 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;
		float RandomAlpha[3];
		float MinValue[3], MaxValue[3];
		const int32 SubEntryStride = LookupTable.SubEntryStride;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );

		MinValue[0] = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
		MinValue[1] = FMath::Lerp( Entry1[1], Entry2[1], Alpha );
		MinValue[2] = FMath::Lerp( Entry1[2], Entry2[2], Alpha );

		MaxValue[0] = FMath::Lerp( Entry1[SubEntryStride + 0], Entry2[SubEntryStride + 0], Alpha );
		MaxValue[1] = FMath::Lerp( Entry1[SubEntryStride + 1], Entry2[SubEntryStride + 1], Alpha );
		MaxValue[2] = FMath::Lerp( Entry1[SubEntryStride + 2], Entry2[SubEntryStride + 2], Alpha );

		RandomAlpha[0] = RandomStream.GetFraction();
		RandomAlpha[1] = RandomStream.GetFraction();
		RandomAlpha[2] = RandomStream.GetFraction();

		OutValue[0] = FMath::Lerp( MinValue[0], MaxValue[0], RandomAlpha[0] );
		OutValue[1] = FMath::Lerp( MinValue[1], MaxValue[1], RandomAlpha[1] );
		OutValue[2] = FMath::Lerp( MinValue[2], MaxValue[2], RandomAlpha[2] );
	}

	/**
	 * Computes the range of the distribution.
	 * @param OutMin - The minimum value in the distribution.
	 * @param OutMax - The maximum value in the distribution.
	 */
	void GetRange(FVector3f* OutMin, FVector3f* OutMax)
	{
		LookupTable.GetRange((float*)OutMin, (float*)OutMax);
	}

	// LWC_TODO: Precision loss
	void GetRange(FVector3d* OutMin, FVector3d* OutMax)
	{
		FVector3f OutMinFloat, OutMaxFloat;
		GetRange(&OutMinFloat, &OutMaxFloat);
		*OutMin = FVector3d(OutMinFloat);
		*OutMax = FVector3d(OutMaxFloat);
	}

private:
	/** The lookup table. */
	FDistributionLookupTable LookupTable;

	/** This distribution is built from composable distributions. */
	friend class FComposableDistribution;
};

/**
 * Raw distribution from which four floats can be looked up per entry.
 */
class FVector4Distribution
{
public:
	/** Default constructor. */
	FVector4Distribution();

	/**
	 * Samples a value from the distribution.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 */
	inline void GetValue( float Time, float* RESTRICT OutValue ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 4 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );
		OutValue[0] = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
		OutValue[1] = FMath::Lerp( Entry1[1], Entry2[1], Alpha );
		OutValue[2] = FMath::Lerp( Entry1[2], Entry2[2], Alpha );
		OutValue[3] = FMath::Lerp( Entry1[3], Entry2[3], Alpha );
	}

	/**
	 * Samples a value randomly distributed between two values.
	 * @param Time - Time at which to sample the distribution.
	 * @param OutValue - Upon return contains the sampled value.
	 * @param RandomStream - Random stream from which to retrieve random fractions.
	 */
	inline void GetRandomValue( float Time, float* RESTRICT OutValue, FRandomStream& RandomStream ) const
	{
		checkDistribution( LookupTable.GetValuesPerEntry() == 4 );

		const float* Entry1;
		const float* Entry2;
		float Alpha;
		float RandomAlpha[4];
		float MinValue[4], MaxValue[4];
		const int32 SubEntryStride = LookupTable.SubEntryStride;

		LookupTable.GetEntry( Time, Entry1, Entry2, Alpha );

		MinValue[0] = FMath::Lerp( Entry1[0], Entry2[0], Alpha );
		MinValue[1] = FMath::Lerp( Entry1[1], Entry2[1], Alpha );
		MinValue[2] = FMath::Lerp( Entry1[2], Entry2[2], Alpha );
		MinValue[3] = FMath::Lerp( Entry1[3], Entry2[3], Alpha );

		MaxValue[0] = FMath::Lerp( Entry1[SubEntryStride + 0], Entry2[SubEntryStride + 0], Alpha );
		MaxValue[1] = FMath::Lerp( Entry1[SubEntryStride + 1], Entry2[SubEntryStride + 1], Alpha );
		MaxValue[2] = FMath::Lerp( Entry1[SubEntryStride + 2], Entry2[SubEntryStride + 2], Alpha );
		MaxValue[3] = FMath::Lerp( Entry1[SubEntryStride + 3], Entry2[SubEntryStride + 3], Alpha );

		RandomAlpha[0] = RandomStream.GetFraction();
		RandomAlpha[1] = RandomStream.GetFraction();
		RandomAlpha[2] = RandomStream.GetFraction();
		RandomAlpha[3] = RandomStream.GetFraction();

		OutValue[0] = FMath::Lerp( MinValue[0], MaxValue[0], RandomAlpha[0] );
		OutValue[1] = FMath::Lerp( MinValue[1], MaxValue[1], RandomAlpha[1] );
		OutValue[2] = FMath::Lerp( MinValue[2], MaxValue[2], RandomAlpha[2] );
		OutValue[3] = FMath::Lerp( MinValue[3], MaxValue[3], RandomAlpha[3] );
	}

	/**
	 * Computes the range of the distribution.
	 * @param OutMin - The minimum value in the distribution.
	 * @param OutMax - The maximum value in the distribution.
	 */
	void GetRange( FVector4f* OutMin, FVector4f* OutMax )
	{
		LookupTable.GetRange( (float*)OutMin, (float*)OutMax );
	}

	// LWC_TODO: Precision loss
	void GetRange(FVector4d* OutMin, FVector4d* OutMax)
	{
		FVector4f OutMinFloat, OutMaxFloat;
		GetRange(&OutMinFloat, &OutMaxFloat);
		*OutMin = (FVector4d)OutMinFloat;
		*OutMax = (FVector4d)OutMaxFloat;
	}

private:
	/** The lookup table. */
	FDistributionLookupTable LookupTable;

	/** This distribution is built from composable distributions. */
	friend class FComposableDistribution;
};

/*-----------------------------------------------------------------------------
	Composable distributions.

	These classes allow code to compose multiple distributions and produce a
	final, optimized raw distribution.
-----------------------------------------------------------------------------*/

//#if WITH_EDITORONLY_DATA

/**
 * Base class for composable distributions. Responsible for building the final
 * distributions that can be sampled.
 */
class FComposableDistribution
{
public:
	/**
	 * Constructs a type-safe floating point distribution.
	 * @param OutDistribution - The distribution to build.
	 * @param X - The source distribution.
	 */
	static void BuildFloat( class FFloatDistribution& OutDistribution, const class FComposableFloatDistribution& X );

	/**
	 * Constructs a type-safe vector distribution.
	 * @param OutDistribution - The distribution to build.
	 * @param XYZ - The source distribution.
	 */
	static void BuildVector( class FVectorDistribution& OutDistribution, const class FComposableVectorDistribution& XYZ );

	/**
	 * Constructs a type-safe 4-vector distribution.
	 * @param OutDistribution - The distribution to build.
	 * @param XYZ - The source distribution for the X, Y, and Z components.
	 * @param W - The source distribution for the W component.
	 */
	static void BuildVector4( class FVector4Distribution& OutDistribution, const class FComposableVectorDistribution& XYZ, const class FComposableFloatDistribution& W );

	/**
	 * Constructs a type-safe 4-vector distribution.
	 * @param OutDistribution - The distribution to build.
	 * @param XY - The source distribution for the X and Y components.
	 * @param Z - The source distribution for the Z component.
	 * @param W - The source distribution for the W component.
	 */
	static void BuildVector4( 
		class FVector4Distribution& OutDistribution,
		const class FComposableVectorDistribution& XY,
		const class FComposableFloatDistribution& Z,
		const class FComposableFloatDistribution& W );

	/**
	 * Constructs a type-safe 4-vector distribution.
	 * @param OutDistribution - The distribution to build.
	 * @param X - The source distribution for the X component.
	 * @param Y - The source distribution for the Y component.
	 * @param Z - The source distribution for the Z component.
	 * @param W - The source distribution for the W component.
	 */
	static void BuildVector4(
		class FVector4Distribution& OutDistribution,
		const class FComposableFloatDistribution& X,
		const class FComposableFloatDistribution& Y,
		const class FComposableFloatDistribution& Z,
		const class FComposableFloatDistribution& W );

	/**
	 * Quantizes a 4-component vector distribution.
	 * @param OutQuantizedSamples - The quantized samples in the distribution.
	 * @param OutScale - Scale to apply to quantized samples.
	 * @param OutBias - Bias to apply to quantized samples.
	 */
	static void QuantizeVector4(
		TArray<FColor>& OutQuantizedSamples,
		FVector4& OutScale,
		FVector4& OutBias,
		const class FVector4Distribution& Distribution );

protected:
	/** The lookup table. */
	FDistributionLookupTable LookupTable;

	/** Constructor. Protected to prevent instantiating the base class. */
	FComposableDistribution() {}

	/** Destructor. */
	~FComposableDistribution() = default;

	/** Copy constructor. Protected to only allow type-safe copying. */
	FComposableDistribution( const FComposableDistribution& Other )
		: LookupTable( Other.LookupTable )
	{
	}

	/** Assignment operator. Protected to only allow type-safe copying. */
	const FComposableDistribution& operator=( const FComposableDistribution& Other )
	{
		LookupTable = Other.LookupTable;
		return *this;
	}
};

/**
 * A composable floating point distribution.
 */
class FComposableFloatDistribution : public FComposableDistribution
{
public:
	/** Default constructor. */
	FComposableFloatDistribution();

	/** Copy constructor. */
	FComposableFloatDistribution( const FComposableFloatDistribution& Other )
		: FComposableDistribution( Other )
	{
	}

	/** Assignment operator. */
	const FComposableFloatDistribution& operator=( const FComposableFloatDistribution& Other )
	{
		FComposableDistribution::operator=( Other );
		return *this;
	}

	/**
	 * Initializes this distribution from that specified.
	 */
	void Initialize( const class UDistributionFloat* FloatDistribution );

	/**
	 * Initializes this distribution with a constant value.
	 */
	void InitializeWithConstant( float Value );

	/**
	 * Scales each sample in the distribution by a constant.
	 */
	void ScaleByConstant( float Scale );

	/**
	 * Scale each sample in the distribution by the corresponding sample in another distribution.
	 */
	void ScaleByDistribution( const class UDistributionFloat* FloatDistribution );

	/**
	 * Adds each sample in the distribution to the corresponding sample in another distribution.
	 */
	void AddDistribution( const class UDistributionFloat* FloatDistribution );

	/**
	 * Normalizes each value to [0,1] and return a scale and bias to reconstruct the original values.
	 */
	void Normalize( float* OutScale, float* OutBias );

	/**
	 * Resamples the distribution to include only information from [MinIn,MaxIn].
	 */
	void Resample( float MinIn, float MaxIn );
};

class FComposableVectorDistribution : public FComposableDistribution
{
public:
	/** Default constructor. */
	FComposableVectorDistribution();

	/** Copy constructor. */
	FComposableVectorDistribution( const FComposableVectorDistribution& Other )
		: FComposableDistribution( Other )
	{
	}

	/** Assignment operator. */
	const FComposableVectorDistribution& operator=( const FComposableVectorDistribution& Other )
	{
		FComposableDistribution::operator=( Other );
		return *this;
	}

	/**
	 * Initializes this distribution from that specified.
	 */
	void Initialize( const class UDistributionVector* VectorDistribution );

	/**
	 * Initializes this distribution with a constant value.
	 */
	void InitializeWithConstant( const FVector& Value );

	/**
	 * Scales each sample in the distribution by a constant.
	 */
	void ScaleByConstant( float Scale );

	/**
	 * Scales each sample in the distribution by a constant vector.
	 */
	void ScaleByConstantVector( const FVector& Scale );

	/**
	 * Adds a constant vector to each sample in the distribution.
	 */
	void AddConstantVector( const FVector& Value );

	/**
	 * Scale each sample in the distribution by the corresponding sample in another distribution.
	 */
	void ScaleByDistribution( const class UDistributionFloat* FloatDistribution );

	/**
	 * Scale each sample in the distribution by the corresponding sample in another distribution.
	 */
	void ScaleByVectorDistribution( const class UDistributionVector* VectorDistribution );

	/**
	 * Adds each sample in the distribution to the corresponding sample in another distribution.
	 */
	void AddDistribution( const class UDistributionVector* VectorDistribution );

	/**
	 * Splats the specified channel to all channels in the distribution.
	 */
	void Splat( int32 ChannelIndex );

	/**
	 * Resamples the distribution to include only information from [MinIn,MaxIn].
	 */
	void Resample( float MinIn, float MaxIn );
};
//#endif // #if WITH_EDITORONLY_DATA
