// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "EngineDefines.h"

class UCanvas;

#ifndef ENABLE_DRAW_DEBUG
#define ENABLE_DRAW_DEBUG  UE_ENABLE_DEBUG_DRAWING
#endif

#if ENABLE_DRAW_DEBUG

/** Flush persistent lines */
ENGINE_API void FlushPersistentDebugLines(const UWorld* InWorld);
/** Draw a debug line */
ENGINE_API void DrawDebugLine(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug point */
ENGINE_API void DrawDebugPoint(const UWorld* InWorld, FVector const& Position, float Size, FColor const& PointColor, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0);
/** Draw directional arrow **/
ENGINE_API void DrawDebugDirectionalArrow(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, float ArrowSize, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug box */
ENGINE_API void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug box with rotation */
ENGINE_API void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, const FQuat& Rotation, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw Debug coordinate system */
ENGINE_API void DrawDebugCoordinateSystem(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw Debug crosshair */
ENGINE_API void DrawDebugCrosshairs(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, const FColor& Color = FColor::White, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0);
/** Draw Debug Circle */
ENGINE_API void DrawDebugCircle(const UWorld* InWorld, const FMatrix& TransformMatrix, double Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f, bool bDrawAxis=true);
/** Draw Debug Circle */
ENGINE_API void DrawDebugCircle(const UWorld* InWorld, FVector Center, double Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f, FVector YAxis=FVector(0.f,1.f,0.f), FVector ZAxis=FVector(0.f,0.f,1.f), bool bDrawAxis=true);
/** Draw Debug Circle Arc */
ENGINE_API void DrawDebugCircleArc(const UWorld* InWorld, const FVector& Center, float Radius, const FVector& Direction, float AngleWidth, int32 Segments, const FColor& Color, bool PersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
ENGINE_API void DrawDebugCircleArc(const UWorld* InWorld, const FVector& Center, float Radius, const FQuat& Rotation, float AngleWidth, int32 Segments, const FColor& Color, bool PersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw Debug 2D donut */
ENGINE_API void DrawDebug2DDonut(const UWorld* InWorld, const FMatrix& TransformMatrix, float InnerRadius, float OuterRadius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug sphere */
ENGINE_API void DrawDebugSphere(const UWorld* InWorld, FVector const& Center, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug cylinder */
ENGINE_API void DrawDebugCylinder(const UWorld* InWorld, FVector const& Start, FVector const& End, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a debug cone. AngleWidth and AngleHeight are given in radians */
ENGINE_API void DrawDebugCone(const UWorld* InWorld, FVector const& Origin, FVector const& Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FColor const& Color, bool bPersistentLines=false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Used by gameplay when defining a cone by a vertical and horizontal dot products. */
ENGINE_API void DrawDebugAltCone(const UWorld* InWorld, FVector const& Origin, FRotator const& Rotation, float Length, float AngleWidth, float AngleHeight, FColor const& DrawColor, bool bPersistentLines=false, float LifeTime=-1.f, uint8 DepthPriority=0, float Thickness = 0.f);
ENGINE_API void DrawDebugString(const UWorld* InWorld, FVector const& TextLocation, const FString& Text, class AActor* TestBaseActor = NULL, FColor const& TextColor = FColor::White, float Duration = -1.000000, bool bDrawShadow = false, float FontScale = 1.f);
ENGINE_API void DrawDebugFrustum(const UWorld* InWorld, const FMatrix& FrustumToWorld, FColor const& Color, bool bPersistentLines = false, float LifeTime=-1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
/** Draw a circle using user define axis and radius */
ENGINE_API void DrawCircle(const UWorld* InWorld, const FVector& Base, const FVector& X, const FVector& Y, const FColor& Color, double Radius, int32 NumSides, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0);
/** Draw a capsule using the LineBatcher */
ENGINE_API void DrawDebugCapsule(const UWorld* InWorld, FVector const& Center, float HalfHeight, float Radius, const FQuat& Rotation, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0);
/** Draw a debug camera shape.  FOV is full angle in degrees. */
ENGINE_API void DrawDebugCamera(const UWorld* InWorld, FVector const& Location, FRotator const& Rotation, float FOVDeg, float Scale=1.f, FColor const& Color=FColor::White, bool bPersistentLines=false, float LifeTime=-1.f, uint8 DepthPriority = 0);
/** Draw a centripetal catmull rom spline. Alpha values between 0 and 1 */
ENGINE_API void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, FColor const& Color, float Alpha = 0.5f, int32 NumSamplesPerSegment = 8, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f);
ENGINE_API void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, TConstArrayView<FColor> Colors, float Alpha = 0.5f, int32 NumSamplesPerSegment = 8, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f);

ENGINE_API void FlushDebugStrings(const UWorld* InWorld);

/** Draw a solid box.  Various API's provided for convenience, including ones that use FBox as well as ones that prefer (Center, Extent). **/
/** Draw a Debug box with optional transform.  */
ENGINE_API void DrawDebugSolidBox(const UWorld* InWorld, FBox const& Box, FColor const& Color, const FTransform& Transform=FTransform::Identity, bool bPersistent=false, float LifeTime=-1.f, uint8 DepthPriority = 0);
/** Draw a Debug box from (Center, Extent) with no rotation (axis-aligned) */
ENGINE_API void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FColor const& Color, bool bPersistent=false, float LifeTime=-1.f, uint8 DepthPriority = 0);
/** Draw a Debug box from (Center, Extent) with specified Rotation */
ENGINE_API void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FQuat const& Rotation, FColor const& Color, bool bPersistent=false, float LifeTime=-1.f, uint8 DepthPriority = 0);

ENGINE_API void DrawDebugMesh(const UWorld* InWorld, TArray<FVector> const& Verts, TArray<int32> const& Indices, FColor const& Color, bool bPersistent=false, float LifeTime=-1.f, uint8 DepthPriority = 0);
ENGINE_API void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, float Size, FColor const& Color, bool bPersistent=false, float LifeTime=-1, uint8 DepthPriority = 0);
ENGINE_API void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, FVector2D const& Extents, FColor const& Color, bool bPersistent=false, float LifeTime=-1, uint8 DepthPriority = 0);

/* Draws a 2D Histogram of size 'DrawSize' based FDebugFloatHistory struct, using DrawTransform for the position in the world. */
ENGINE_API void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FTransform const & DrawTransform, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent = false, float const & LifeTime = -1.f, uint8 const & DepthPriority = 0);

/* Draws a 2D Histogram of size 'DrawSize' based FDebugFloatHistory struct, using DrawLocation for the location in the world, rotation will face camera of first player. */
ENGINE_API void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FVector const & DrawLocation, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent = false, float const & LifeTime = -1.f, uint8 const & DepthPriority = 0);

/**
 * Draws a 2D line on a canvas
 *
 * @param	Canvas		The Canvas to draw on
 * @param	Start		The start of the line in screen space
 * @param	End			The end of the line in screen space
 * @param	LineColor	The color to use for the line
 */

ENGINE_API void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor);
ENGINE_API void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector2D& StartPosition, const FVector2D& EndPosition, const FLinearColor& LineColor, const float& LineThickness = 1.f);
ENGINE_API void DrawDebugCanvas2DCircle(UCanvas* Canvas, const FVector2D& Center, float Radius, int32 NumSides, const FLinearColor& LineColor, const float& LineThickness = 1.f);
ENGINE_API void DrawDebugCanvas2DBox(UCanvas* Canvas, const FBox2D& Box, const FLinearColor& LineColor, const float& LineThickness = 1.f);


/**
 * Draws a line on a canvas.
 *
 * @param	Canvas		The Canvas to draw on.
 * @param	Start		The start of the line in world space
 * @param	End			The end of the line in world space
 * @param	LineColor	The color to use for the line
 */

ENGINE_API void DrawDebugCanvasLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor);

/**
 * Draws a circle using lines.
 *
 * @param	Canvas		The Canvas to draw on.
 * @param	Base			Center of the circle.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Color			Color of the circle.
 * @param	Radius			Radius of the circle.
 * @param	NumSides		Numbers of sides that the circle has.
 */
ENGINE_API void DrawDebugCanvasCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides);

/**
 * Draws a half circle using lines.
 *
 * @param	Canvas			The Canvas to draw on.
 * @param	Base			Center of the circle.
 * @param	X				X alignment axis to draw along.
 * @param	Y				Y alignment axis to draw along.
 * @param	Color			Color of the circle.
 * @param	Radius			Radius of the circle.
 * @param	NumSides		Numbers of sides that the circle has.
 */
ENGINE_API void DrawDebugCanvasHalfCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides);

/**
 * Draws a sphere using circles.
 *
 * @param	Canvas		The Canvas to draw on.
 * @param	Base			Center of the sphere.
 * @param	Color			Color of the sphere.
 * @param	Radius			Radius of the sphere.
 * @param	NumSides		Numbers of sides that the circle has.
 */
ENGINE_API void DrawDebugCanvasWireSphere(UCanvas* Canvas, const FVector& Base, FColor Color, float Radius, int32 NumSides);

/**
 * Draws a wireframe cone
 *
 * @param	Canvas			Canvas to draw on
 * @param	Transform		Generic transform to apply (ex. a local-to-world transform).
 * @param	ConeRadius		Radius of the cone.
 * @param	ConeAngle		Angle of the cone.
 * @param	ConeSides		Numbers of sides that the cone has.
 * @param	Color			Color of the cone.
 */
ENGINE_API void DrawDebugCanvasWireCone(UCanvas* Canvas, const FTransform& Transform, float ConeRadius, float ConeAngle, int32 ConeSides, FColor Color);

/**
 * Draws a box using lines.
 *
 * @param	Canvas			The Canvas to draw on.
 * @param	Transform		Transform to apply to the capsule.
 * @param	Box				The box to be drawn.
 * @param	Color			Color of the capsule.
 */
ENGINE_API void DrawDebugCanvasWireBox(UCanvas* Canvas, const FMatrix& Transform, const FBox& Box, FColor Color);

/**
 * Draws a capsule using lines.
 *
 * @param	Canvas			The Canvas to draw on.
 * @param	Transform		Transform to apply to the capsule.
 * @param	HalfLength		Distance from the center of the capsule to the tip (including the cap)
 * @param	Radius			Radius of the capsule.
 * @param	Color			Color of the capsule.
 */
ENGINE_API void DrawDebugCanvasCapsule(UCanvas* Canvas, const FMatrix& Transform, float HalfLength, float Radius, const FColor& LineColor);

#elif !defined(SHIPPING_DRAW_DEBUG_ERROR) || !SHIPPING_DRAW_DEBUG_ERROR

// Empty versions of above functions
// LWC_TODO: A lot of the floats here should really be FVector::FReals as they will often be calculated using FReals. e.g. Radius, Length etc
inline void FlushPersistentDebugLines(const UWorld* InWorld) {}
inline void DrawDebugLine(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugPoint(const UWorld* InWorld, FVector const& Position, float Size, FColor const& PointColor, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugDirectionalArrow(const UWorld* InWorld, FVector const& LineStart, FVector const& LineEnd, float ArrowSize, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, const FQuat& Rotation, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugCoordinateSystem(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugCrosshairs(const UWorld* InWorld, FVector const& AxisLoc, FRotator const& AxisRot, float Scale, const FColor& Color = FColor::White, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugCircle(const UWorld* InWorld, const FMatrix& TransformMatrix, double Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f, bool bDrawAxis = true) {}
inline void DrawDebugCircle(const UWorld* InWorld, FVector Center, double Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f, FVector YAxis = FVector(0.f, 1.f, 0.f), FVector ZAxis = FVector(0.f, 0.f, 1.f), bool bDrawAxis = true) {}
inline void DrawDebugCircleArc(const UWorld* InWorld, FVector const& Center, float Radius, FVector const& Direction, float AngleWidth, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebug2DDonut(const UWorld* InWorld, const FMatrix& TransformMatrix, float InnerRadius, float OuterRadius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugSphere(const UWorld* InWorld, FVector const& Center, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugCylinder(const UWorld* InWorld, FVector const& Start, FVector const& End, float Radius, int32 Segments, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugCone(const UWorld* InWorld, FVector const& Origin, FVector const& Direction, float Length, float AngleWidth, float AngleHeight, int32 NumSides, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugAltCone(const UWorld* InWorld, FVector const& Origin, FRotator const& Rotation, float Length, float AngleWidth, float AngleHeight, FColor const& DrawColor, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawDebugString(const UWorld* InWorld, FVector const& TextLocation, const FString& Text, class AActor* TestBaseActor = NULL, FColor const& TextColor = FColor::White, float Duration = -1.000000, bool bDrawShadow = false, float FontScale = 1.f) {}
inline void DrawDebugFrustum(const UWorld* InWorld, const FMatrix& FrustumToWorld, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawCircle(const UWorld* InWorld, const FVector& Base, const FVector& X, const FVector& Y, const FColor& Color, double Radius, int32 NumSides, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0) {}
inline void DrawDebugCapsule(const UWorld* InWorld, FVector const& Center, float HalfHeight, float Radius, const FQuat& Rotation, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0) {}
inline void DrawDebugCamera(const UWorld* InWorld, FVector const& Location, FRotator const& Rotation, float FOVDeg, float Scale = 1.f, FColor const& Color = FColor::White, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, FColor const& Color, float Alpha = 0.5f, int32 NumSamplesPerSegment = 8, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void DrawCentripetalCatmullRomSpline(const UWorld* InWorld, TConstArrayView<FVector> Points, TConstArrayView<FColor> Colors, float Alpha = 0.5f, int32 NumSamplesPerSegment = 8, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f) {}
inline void FlushDebugStrings(const UWorld* InWorld) {}
inline void DrawDebugSolidBox(const UWorld* InWorld, FBox const& Box, FColor const& Color, const FTransform& Transform = FTransform::Identity, bool bPersistent = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FColor const& Color, bool bPersistent = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugSolidBox(const UWorld* InWorld, FVector const& Center, FVector const& Extent, FQuat const& Rotation, FColor const& Color, bool bPersistent = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugMesh(const UWorld* InWorld, TArray<FVector> const& Verts, TArray<int32> const& Indices, FColor const& Color, bool bPersistent = false, float LifeTime = -1.f, uint8 DepthPriority = 0) {}
inline void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, float Size, FColor const& Color, bool bPersistent = false, float LifeTime = -1, uint8 DepthPriority = 0) {}
inline void DrawDebugSolidPlane(const UWorld* InWorld, FPlane const& P, FVector const& Loc, FVector2D const& Extents, FColor const& Color, bool bPersistent = false, float LifeTime = -1, uint8 DepthPriority = 0) {}
inline void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FTransform const & DrawTransform, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent = false, float const & LifeTime = -1.f, uint8 const & DepthPriority = 0) {}
inline void DrawDebugFloatHistory(UWorld const & WorldRef, FDebugFloatHistory const & FloatHistory, FVector const & DrawLocation, FVector2D const & DrawSize, FColor const & DrawColor, bool const & bPersistent = false, float const & LifeTime = -1.f, uint8 const & DepthPriority = 0) {}
inline void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor) {}
inline void DrawDebugCanvas2DLine(UCanvas* Canvas, const FVector2D& StartPosition, const FVector2D& EndPosition, const FLinearColor& LineColor, const float& LineThickness = 1.f) {}
inline void DrawDebugCanvas2DCircle(UCanvas* Canvas, const FVector2D& Center, float Radius, int32 NumSides, const FLinearColor& LineColor, const float& LineThickness = 1.f) {}
inline void DrawDebugCanvas2DBox(UCanvas* Canvas, const FBox2D& Box, const FLinearColor& LineColor, const float& LineThickness = 1.f) {}
inline void DrawDebugCanvasLine(UCanvas* Canvas, const FVector& Start, const FVector& End, const FLinearColor& LineColor) {}
inline void DrawDebugCanvasCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides) {}
inline void DrawDebugCanvasHalfCircle(UCanvas* Canvas, const FVector& Base, const FVector& X, const FVector& Y, FColor Color, float Radius, int32 NumSides) {}
inline void DrawDebugCanvasWireSphere(UCanvas* Canvas, const FVector& Base, FColor Color, float Radius, int32 NumSides) {}
inline void DrawDebugCanvasWireCone(UCanvas* Canvas, const FTransform& Transform, float ConeRadius, float ConeAngle, int32 ConeSides, FColor Color) {}
inline void DrawDebugCanvasWireBox(UCanvas* Canvas, const FMatrix& Transform, const FBox& Box, FColor Color) {}
inline void DrawDebugCanvasCapsule(UCanvas* Canvas, const FMatrix& Transform, float HalfLength, float Radius, const FColor& LineColor) {}

#else

// Remove 'no-op' debug draw commands, users need to make sure they are not calling these functions in Shipping/Test builds or it will generate a compile error

#endif // ENABLE_DRAW_DEBUG
