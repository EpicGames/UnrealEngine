// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnrealClient.h"

/**
 * Minimal viewport for assisting with taking screenshots (also used within a plugin)
 * @todo: This should be refactored
 */
class FDummyViewport : public FViewport
{
public:
	UE_DEPRECATED(5.8, "Use FDummyViewport() + SetViewportClient() or static Create(TStrongPtrVariant<FViewportClient>) instead")
	ENGINE_API FDummyViewport(FViewportClient* const Client);

	static ENGINE_API TUniquePtr<FDummyViewport> Create(const TStrongPtrVariant<FViewportClient>& InClient);

	UE_DEPRECATED(5.8, "This is only for backward compatibility with other deprecated functions")
	static ENGINE_API TUniquePtr<FDummyViewport> Create(FViewportClient* InClient);

protected:
	ENGINE_API FDummyViewport();

public:
	ENGINE_API virtual ~FDummyViewport();

	//~ Begin FViewport Interface
	void SetupHDR(EDisplayColorGamut InDisplayColorGamut, EDisplayOutputFormat InDisplayOutputFormat, bool bInSceneHDREnabled)
	{
		DisplayColorGamut = InDisplayColorGamut;
		DisplayOutputFormat = InDisplayOutputFormat;
		bSceneHDREnabled = bInSceneHDREnabled;
	}

	virtual void SetInitialSize(FIntPoint InitialSizeXY) override
	{
	    if (GetSizeXY() == FIntPoint::ZeroValue)
	    {
			SizeX = InitialSizeXY.X;
			SizeY = InitialSizeXY.Y;
	    }
	}

	virtual void*	GetWindow() override { return 0; }
	virtual void	MoveWindow(int32 NewPosX, int32 NewPosY, int32 NewSizeX, int32 NewSizeY) override {}
	virtual void	Destroy() override {}
	virtual bool SetUserFocus(bool bFocus) override { return false; }
	virtual bool	KeyState(FKey Key) const override { return false; }
	virtual int32	GetMouseX() const override { return 0; }
	virtual int32	GetMouseY() const override { return 0; }
	virtual void	GetMousePos( FIntPoint& MousePosition, const bool bLocalPosition = true) override { MousePosition = FIntPoint(0, 0); }
	virtual void	SetMouse(int32 x, int32 y) override { }
	virtual void	ProcessInput( float DeltaTime ) override { }
	virtual FVector2D VirtualDesktopPixelToViewport(FIntPoint VirtualDesktopPointPx) const override { return FVector2D::ZeroVector; }
	virtual FIntPoint ViewportToVirtualDesktopPixel(FVector2D ViewportCoordinate) const override { return FIntPoint::ZeroValue; }
	virtual void InvalidateDisplay() override { }
	virtual void DeferInvalidateHitProxy() override { }
	virtual FCanvas* GetDebugCanvas() override { return DebugCanvas; }
	ENGINE_API virtual EDisplayColorGamut GetDisplayColorGamut() const;
	ENGINE_API virtual EDisplayOutputFormat GetDisplayOutputFormat() const;
	ENGINE_API virtual bool GetSceneHDREnabled() const;
	//~ End FViewport Interface

	//~ Begin FRenderResource Interface
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual FString GetFriendlyName() const override { return FString("FDummyViewport");}
	//~ End FRenderResource Interface
private:
	FCanvas* DebugCanvas;
	EDisplayColorGamut DisplayColorGamut = EDisplayColorGamut::sRGB_D65;
	EDisplayOutputFormat DisplayOutputFormat = EDisplayOutputFormat::SDR_sRGB;
	bool bSceneHDREnabled = false;
};
