// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicMeshBuilder.h: Dynamic mesh builder definitions.
=============================================================================*/

#pragma once

#include "Components.h"
#include "CoreMinimal.h"
#include "PackedNormal.h"
#include "HitProxies.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RenderMath.h"

class FMaterialRenderProxy;
class FMeshBuilderOneFrameResources;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class FPrimitiveUniformShaderParameters;
struct FMeshBatch;

/** The vertex type used for dynamic meshes. */
struct FDynamicMeshVertex
{
	FDynamicMeshVertex() {}
	FDynamicMeshVertex( const FVector3f& InPosition ):
		Position(InPosition),
		TangentX(FVector3f(1,0,0)),
		TangentZ(FVector3f(0,0,1)),
		Color(FColor(255,255,255)) 
	{
		// basis determinant default to +1.0
		TangentZ.Vector.W = 127;

		for (int i = 0; i < MAX_STATIC_TEXCOORDS; i++)
		{
			TextureCoordinate[i] = FVector2f::ZeroVector;
		}
	}

	FDynamicMeshVertex(const FVector3f& InPosition, const FVector2f& InTexCoord, const FColor& InColor) :
		Position(InPosition),
		TangentX(FVector3f(1, 0, 0)),
		TangentZ(FVector3f(0, 0, 1)),
		Color(InColor)
	{
		// basis determinant default to +1.0
		TangentZ.Vector.W = 127;

		for (int i = 0; i < MAX_STATIC_TEXCOORDS; i++)
		{
			TextureCoordinate[i] = InTexCoord;
		}
	}

	FDynamicMeshVertex(const FVector3f& InPosition,const FVector3f& InTangentX,const FVector3f& InTangentZ,const FVector2f& InTexCoord, const FColor& InColor):
		Position(InPosition),
		TangentX(InTangentX),
		TangentZ(InTangentZ),
		Color(InColor)
	{
		// basis determinant default to +1.0
		TangentZ.Vector.W = 127;

		for (int i = 0; i < MAX_STATIC_TEXCOORDS; i++)
		{
			TextureCoordinate[i] = InTexCoord;
		}
	}

	FDynamicMeshVertex(const FVector3f& InPosition, const FVector3f& LayerTexcoords, const FVector2f& WeightmapTexcoords)
		: Position(InPosition)
		, TangentX(FVector3f(1, 0, 0))
		, TangentZ(FVector3f(0, 0, 1))
		, Color(FColor::White)
	{
		// TangentZ.w contains the sign of the tangent basis determinant. Assume +1
		TangentZ.Vector.W = 127;

		TextureCoordinate[0] = FVector2f(LayerTexcoords.X, LayerTexcoords.Y);
		TextureCoordinate[1] = FVector2f(LayerTexcoords.X, LayerTexcoords.Y); // Z not currently set, so use Y
		TextureCoordinate[2] = FVector2f(LayerTexcoords.Y, LayerTexcoords.X); // Z not currently set, so use X
		TextureCoordinate[3] = WeightmapTexcoords;
	};

	void SetTangents( const FVector3f& InTangentX, const FVector3f& InTangentY, const FVector3f& InTangentZ )
	{
		TangentX = InTangentX;
		TangentZ = InTangentZ;
		// store determinant of basis in w component of normal vector
		TangentZ.Vector.W = GetBasisDeterminantSignByte(InTangentX,InTangentY,InTangentZ);
	}

	FVector3f GetTangentY() const
	{
		return FVector3f(GenerateYAxis(TangentX, TangentZ));	//LWC_TODO: Precision loss
	};

	FVector3f Position;
	FVector2f TextureCoordinate[MAX_STATIC_TEXCOORDS];
	FPackedNormal TangentX;
	FPackedNormal TangentZ;
	FColor Color;
};

struct FDynamicMeshDrawOffset
{
	uint32 FirstIndex = 0;
	uint32 MinVertexIndex = 0;
	uint32 MaxVertexIndex = 0;
	uint32 NumPrimitives = 0;
};

struct FDynamicMeshBuilderSettings
{
	bool CastShadow = true;
	bool bDisableBackfaceCulling = false;
	bool bWireframe = false;
	bool bReceivesDecals = true;
	bool bUseSelectionOutline = true;
	bool bCanApplyViewModeOverrides = false;
	bool bUseWireframeSelectionColoring = false;
	bool bIsFirstPerson = false;
};

/**
 * This class provides the vertex/index allocator interface used by FDynamicMeshBuilder which is
 * already implemented internally with caching in mind but can be customized if needed.
 */
class FDynamicMeshBufferAllocator
{
public:
	ENGINE_API int32 GetIndexBufferSize(uint32 NumElements) const;
	ENGINE_API int32 GetVertexBufferSize(uint32 Stride, uint32 NumElements) const;
	ENGINE_API virtual ~FDynamicMeshBufferAllocator();

	ENGINE_API virtual FBufferRHIRef AllocIndexBuffer(FRHICommandListBase& RHICmdList, uint32 NumElements);
	ENGINE_API virtual void ReleaseIndexBuffer(FBufferRHIRef& IndexBufferRHI);
	ENGINE_API virtual FBufferRHIRef AllocVertexBuffer(FRHICommandListBase& RHICmdList, uint32 Stride, uint32 NumElements);
	ENGINE_API virtual void ReleaseVertexBuffer(FBufferRHIRef& VertexBufferRHI);

	UE_DEPRECATED(5.4, "AllocIndexBuffer requires a command list.")
	ENGINE_API virtual FBufferRHIRef AllocIndexBuffer(uint32 NumElements) final;

	UE_DEPRECATED(5.4, "AllocVertexBuffer requires a command list.")
	ENGINE_API virtual FBufferRHIRef AllocVertexBuffer(uint32 Stride, uint32 NumElements) final;
};

/**
 * A utility used to construct dynamically generated meshes, and render them to a FPrimitiveDrawInterface.
 * Note: This is meant to be easy to use, not fast.  It moves the data around more than necessary, and requires dynamically allocating RHI
 * resources.  Exercise caution.
 */
class FDynamicMeshBuilder
{
public:

	/** Initialization constructor. */
	ENGINE_API FDynamicMeshBuilder(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumTexCoords = 1, uint32 InLightmapCoordinateIndex = 0, bool InUse16bitTexCoord = false, FDynamicMeshBufferAllocator* InDynamicMeshBufferAllocator = nullptr);

	/** Destructor. */
	ENGINE_API ~FDynamicMeshBuilder();

	/** Adds a vertex to the mesh. */
	ENGINE_API int32 AddVertex(
		const FVector3f& InPosition,
		const FVector2f& InTextureCoordinate,
		const FVector3f& InTangentX,
		const FVector3f& InTangentY,
		const FVector3f& InTangentZ,
		const FColor& InColor
		);

	/** Adds a vertex to the mesh. */
	ENGINE_API int32 AddVertex(const FDynamicMeshVertex &InVertex);

	/** Adds a triangle to the mesh. */
	ENGINE_API void AddTriangle(int32 V0,int32 V1,int32 V2);

	/** Adds many vertices to the mesh. */
	ENGINE_API int32 AddVertices(const TArray<FDynamicMeshVertex> &InVertices);

	/** Add many indices to the mesh. */
	ENGINE_API void AddTriangles(const TArray<uint32> &InIndices);

	/** Pre-allocate space for the given number of vertices. */
	ENGINE_API void ReserveVertices(int32 InNumVertices);

	/** Pre-allocate space for the given number of triangles. */
	ENGINE_API void ReserveTriangles(int32 InNumTriangles);

	/** Adds a mesh of what's been built so far to the collector. */
	ENGINE_API void GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, int32 ViewIndex, FMeshElementCollector& Collector);
	ENGINE_API void GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, bool bUseSelectionOutline, int32 ViewIndex, 
							FMeshElementCollector& Collector, HHitProxy* HitProxy);
	ENGINE_API void GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, bool bUseSelectionOutline, int32 ViewIndex, 
							FMeshElementCollector& Collector, const FHitProxyId HitProxyId = FHitProxyId());
	ENGINE_API void GetMesh(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, const FDynamicMeshBuilderSettings& Settings, FDynamicMeshDrawOffset const * const DrawOffset, int32 ViewIndex,
		FMeshElementCollector& Collector, const FHitProxyId HitProxyId = FHitProxyId());
	ENGINE_API void GetMesh(const FMatrix& LocalToWorld, const FMatrix& PrevLocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, const FDynamicMeshBuilderSettings& Settings,
		FDynamicMeshDrawOffset const * const DrawOffset, int32 ViewIndex, FMeshElementCollector& Collector, const FHitProxyId HitProxyId = FHitProxyId());

	ENGINE_API void GetMeshElement(const FMatrix& LocalToWorld, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, bool bReceivesDecals, int32 ViewIndex, FMeshBuilderOneFrameResources& OneFrameResource, FMeshBatch& Mesh);
	ENGINE_API void GetMeshElement(const FPrimitiveUniformShaderParameters& PrimitiveParams, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriorityGroup, bool bDisableBackfaceCulling, int32 ViewIndex, FMeshBuilderOneFrameResources& OneFrameResource, FMeshBatch& Mesh);

	/**
	 * Draws the mesh to the given primitive draw interface.
	 * @param PDI - The primitive draw interface to draw the mesh on.
	 * @param LocalToWorld - The local to world transform to apply to the vertices of the mesh.
	 * @param FMaterialRenderProxy - The material instance to render on the mesh.
	 * @param DepthPriorityGroup - The depth priority group to render the mesh in.
	 * @param HitProxyId - Hit proxy to use for this mesh.  Use INDEX_NONE for no hit proxy.
	 */
	ENGINE_API void Draw(FPrimitiveDrawInterface* PDI,const FMatrix& LocalToWorld,const FMaterialRenderProxy* MaterialRenderProxy,uint8 DepthPriorityGroup,bool bDisableBackfaceCulling=false, bool bReceivesDecals=true, const FHitProxyId HitProxyId = FHitProxyId());

private:
	class FMeshBuilderOneFrameResources* OneFrameResources = nullptr;
	class FPooledDynamicMeshIndexBuffer* IndexBuffer = nullptr;
	class FPooledDynamicMeshVertexBuffer* VertexBuffer = nullptr;
	ERHIFeatureLevel::Type FeatureLevel;
	FDynamicMeshBufferAllocator* DynamicMeshBufferAllocator;
};

/** Index Buffer */
class FDynamicMeshIndexBuffer32 : public FIndexBuffer
{
public:
	ENGINE_API FDynamicMeshIndexBuffer32();
	ENGINE_API virtual ~FDynamicMeshIndexBuffer32();

	TArray<uint32> Indices;
	EBufferUsageFlags UsageFlags = EBufferUsageFlags::Static;

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

class FDynamicMeshIndexBuffer16 : public FIndexBuffer
{
public:
	ENGINE_API FDynamicMeshIndexBuffer16();
	ENGINE_API virtual ~FDynamicMeshIndexBuffer16();

	TArray<uint16> Indices;
	EBufferUsageFlags UsageFlags = EBufferUsageFlags::Static;

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "LocalVertexFactory.h"
#endif
