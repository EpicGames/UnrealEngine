// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneView.h"
#include "Engine/EngineTypes.h"
#include "TemporalUpscaler.h"

/** Render thread proxy that holds the heuristic for dynamic resolution. */
class FDynamicResolutionHeuristicProxy
{
public:
	ENGINE_API FDynamicResolutionHeuristicProxy();
	ENGINE_API ~FDynamicResolutionHeuristicProxy();

	/** Resets the proxy. */
	ENGINE_API void Reset_RenderThread();

	/** Create a new previous frame and feeds its timings. */
	void CreateNewPreviousFrameTimings_RenderThread(float GameThreadTimeMs, float RenderThreadTimeMs, float TotalFrameGPUBusyTimeMs);

	/** Refresh resolution fraction's from history. */
	ENGINE_API void RefreshCurrentFrameResolutionFraction_RenderThread();

	/** Returns the view fraction that should be used for current frame. */
	inline DynamicRenderScaling::TMap<float> QueryCurrentFrameResolutionFractions() const
	{
		check(IsInParallelRenderingThread());
		return QueryCurrentFrameResolutionFractions_Internal();
	}

	/** Returns a non thread safe approximation of the current resolution fraction applied on render thread. */
	inline DynamicRenderScaling::TMap<float> GetResolutionFractionsApproximation_GameThread() const
	{
		check(IsInGameThread());
		return QueryCurrentFrameResolutionFractions_Internal();
	}

	/** Returns the view fraction upper bound. */
	ENGINE_API DynamicRenderScaling::TMap<float> GetResolutionFractionUpperBounds() const;

	/** Creates a default dynamic resolution state using this proxy that queries GPU timing from the RHI. */
	static ENGINE_API TSharedPtr<class IDynamicResolutionState> CreateDefaultState();

	/** Applies the minimum/maximum resolution fraction for a third-party temporal upscaler. */
	ENGINE_API void SetTemporalUpscaler(const UE::Renderer::Private::ITemporalUpscaler* InTemporalUpscaler);

private:
	struct FrameHistoryEntry
	{
		float GameThreadTimeMs = -1.0f;
		float RenderThreadTimeMs = -1.0f;

		// Total GPU busy time for the entire frame in milliseconds.
		float TotalFrameGPUBusyTimeMs = -1.0f;

		// Time for each individual timings
		DynamicRenderScaling::TMap<float> BudgetTimingMs;

		// The resolution fraction the frame was rendered with.
		DynamicRenderScaling::TMap<float> ResolutionFractions;

		FrameHistoryEntry()
		{
			ResolutionFractions.SetAll(1.0f);
			BudgetTimingMs.SetAll(-1.0f);
		}

		// Returns whether GPU timings have landed.
		bool HasGPUTimings() const
		{
			return TotalFrameGPUBusyTimeMs >= 0.0f;
		}
	};

	// Circular buffer of the history.
	// We don't use TCircularBuffer because it does not support resizes.
	TArray<FrameHistoryEntry> History;
	int32 PreviousFrameIndex;
	int32 HistorySize;

	// Counts the number of frame since the last screen percentage change.
	int32 NumberOfFramesSinceScreenPercentageChange;

	// Number of frames remaining to ignore.
	int32 IgnoreFrameRemainingCount;

	// Current frame's view fraction.
	DynamicRenderScaling::TMap<float> CurrentFrameResolutionFractions;
	DynamicRenderScaling::TMap<float> CurrentFrameMaxResolutionFractions;
	DynamicRenderScaling::TMap<int32> BudgetHistorySizes;

	// Minimum and maximum resolution fractions supported by the main view family's third-party temporal upscaler.
	float TemporalUpscalerMinResolutionFraction;
	float TemporalUpscalerMaxResolutionFraction;

	const FrameHistoryEntry& GetPreviousFrameEntry(int32 BrowsingFrameId) const
	{
		if (BrowsingFrameId < 0 || BrowsingFrameId >= HistorySize)
		{
			static const FrameHistoryEntry InvalidEntry;
			return InvalidEntry;
		}
		return History[(History.Num() + PreviousFrameIndex - BrowsingFrameId) % History.Num()];
	}

	ENGINE_API DynamicRenderScaling::TMap<float> QueryCurrentFrameResolutionFractions_Internal() const;

	ENGINE_API void RefreshCurrentFrameResolutionFractionUpperBound_RenderThread();

	ENGINE_API void RefreshHeuristicStats_RenderThread();

	ENGINE_API void ResetInternal();

	ENGINE_API void ResizeHistoryIfNeeded();
};
