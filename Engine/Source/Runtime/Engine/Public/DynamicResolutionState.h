// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StereoRendering.h: Abstract stereoscopic rendering interface
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "DynamicRenderScaling.h"
#include "TemporalUpscaler.h"


/** Dynamic resolution of the primary ScreenPercentage. */
extern ENGINE_API DynamicRenderScaling::FBudget GDynamicPrimaryResolutionFraction;


/** Game thread events for dynamic resolution state. */
enum class EDynamicResolutionStateEvent : uint8
{
	// Fired at the very begining of the frame.
	BeginFrame,

	// Fired at the very end of the frame.
	EndFrame
};


/** Interface between the engine and state of dynamic resolution that can be overriden to implement a custom heurstic. */
class IDynamicResolutionState
{
public:
	virtual ~IDynamicResolutionState() = default;

	/** Reset dynamic resolution's history. */
	virtual void ResetHistory() = 0;

	/** Returns whether dynamic resolution is supported on this platform.
	 *
	 * Using dynamic resolution on unsupported platforms is extremely dangerous for gameplay
	 * experience, since it may have a bug dropping resolution or frame rate more than it should.
	 */
	virtual bool IsSupported() const = 0;

	/** Setup a screen percentage driver for a given view family. */
	virtual void SetupMainViewFamily(class FSceneViewFamily& ViewFamily) = 0;

	/** Apply the minimum/maximum resolution fraction for a third-party temporal upscaler. */
	virtual void SetTemporalUpscaler(const UE::Renderer::Private::ITemporalUpscaler* InTemporalUpscaler) = 0;

protected:

	/** Returns a non thread safe aproximation of the current resolution fraction applied on render thread, mostly used for stats and analytic. */
	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsApproximation() const = 0;

	/** Returns the max resolution resolution fraction. */
	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const = 0;

	/** Returns the max resolution resolution fraction as specified in the budget (this can differ from the upper bound if the upper bound is dynamic)*/
	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBoundBudgetValue() const
	{
		return GetResolutionFractionsUpperBound();
	}

	/** Enables/Disables dynamic resolution. This is only called by GEngine automatically. */
	virtual void SetEnabled(bool bEnable) = 0;

	/** Returns whether dynamic resolution is enabled for GEngine to know the EDynamicResolutionStatus. */
	virtual bool IsEnabled() const = 0;

	/** Process dynamic resolution events. UEngine::EmitDynamicResolutionEvent() guareentee to have all events being ordered. */
	virtual void ProcessEvent(EDynamicResolutionStateEvent Event) = 0;

	// Only GEngine can actually enable/disable and emit dynamic resolution event, to force consistency across all implementations.
	friend class UEngine;
};
