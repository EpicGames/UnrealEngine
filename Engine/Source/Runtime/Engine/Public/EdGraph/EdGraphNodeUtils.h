// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EDITOR

/*******************************************************************************
 * FNodeTextCache
 ******************************************************************************/

/** 
 * Constructing FText strings every frame can be costly, so this struct provides 
 * a way to easily cache those strings (node title, tooptip, etc.) for reuse.
 */
struct FNodeTextCache
{
public:
	FNodeTextCache() :
		CacheRefreshID(0)
	{}

	/**
	 * Checks if the title is out of date
	 *
	 * @param InOwningNode		Node that owns this title
	 */
	inline bool IsOutOfDate(const UEdGraphNode* InOwningNode) const
	{
		return CachedText.IsEmpty() || (InOwningNode && InOwningNode->GetGraph() && InOwningNode->GetSchema()->IsCacheVisualizationOutOfDate(CacheRefreshID));
	}

	/**
	 * Checks if the title is out of date
	 *
	 * @param InSchema		Schema to use for checking if externally made out of date
	 */
	inline bool IsOutOfDate(const UEdGraphSchema* InSchema) const
	{
		return CachedText.IsEmpty() || (InSchema && InSchema->IsCacheVisualizationOutOfDate(CacheRefreshID));
	}

	/**
	 * Checks if the title is out of date
	 *
	 * @param InText			Text to cache
	 * @param InOwningNode		Node that owns this title
	 */
	inline void SetCachedText(FText const& InText, const UEdGraphNode* InOwningNode) const
	{
		UpdateCacheIternal(InText);

		if (InOwningNode && InOwningNode->GetGraph())
		{
			// Update the CacheRefreshID, whenever these values do not match the cached value is out of date
			CacheRefreshID = InOwningNode->GetSchema()->GetCurrentVisualizationCacheID();
		}
	}

	/**
	 * Checks if the title is out of date
	 *
	 * @param InText			Text to cache
	 * @param InSchema			Schema to use for checking if externally made out of date
	 */
	inline void SetCachedText(FText const& InText, const UEdGraphSchema* InSchema) const
	{
		UpdateCacheIternal(InText);

		if (InSchema)
		{
			// Update the CacheRefreshID, whenever these values do not match the cached value is out of date
			CacheRefreshID = InSchema->GetCurrentVisualizationCacheID();
		}
	}

	/** */
	inline FText& GetCachedText() const
	{
		return CachedText;
	}

	/** */
	inline void MarkDirty() const
	{
		Clear();
		CacheRefreshID = 0;
	}

	/** */
	inline void Clear() const
	{
		CachedText = FText::GetEmpty();
	}

	/** */
	inline operator FText&() const
	{
		return GetCachedText();
	}

private:
	/** Helper to lookup from the GConfig the correct ini setting for displaying
	 *  node and pin titles localized and updates the cache accordingly */
	inline void UpdateCacheIternal(FText const& InText) const
	{
		bool bShouldUseLocalizedNodeAndPinNames = false;
		GConfig->GetBool( TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), bShouldUseLocalizedNodeAndPinNames, GEditorSettingsIni );

		if (bShouldUseLocalizedNodeAndPinNames)
		{
			CachedText = InText;
		}
		else
		{
			CachedText = FText::FromString(InText.BuildSourceString());
		}
	}

private:
	/** Mutable so that SetCachedText() can remain const (callable by the node's GetNodeTitle() method)*/
	mutable FText CachedText;

	/** ID to check if the title should be considered dirty due to outside conditions that may require the title to refresh */
	mutable int32 CacheRefreshID;
};

/*******************************************************************************
 * FNodeTitleTextTable
 ******************************************************************************/

struct FNodeTitleTextTable
{
public:
	/**
	 * Checks if the title of the passed type is cached
	 *
	 * @param InTitleType		Type of title to be examined
	 * @param InOwningNode		Node that owns this title
	 */
	inline bool IsTitleCached(ENodeTitleType::Type InTitleType, const UEdGraphNode* InOwningNode) const
	{
		return !CachedNodeTitles[InTitleType].IsOutOfDate(InOwningNode);
	}

	/**
	 * Caches the specific title type
	 *
	 * @param InTitleType		Type of title to be examined
	 * @param InText			Text to cache
	 * @param InOwningNode		Node that owns this title
	 */
	inline void SetCachedTitle(ENodeTitleType::Type InTitleType, FText const& InText, const UEdGraphNode* InOwningNode) const
	{
		CachedNodeTitles[InTitleType].SetCachedText(InText, InOwningNode);
	}

	/** */
	inline FText& GetCachedTitle(ENodeTitleType::Type TitleType) const
	{
		return CachedNodeTitles[TitleType].GetCachedText();
	}

	/** */
	inline void MarkDirty() const
	{
		for (uint32 TitleIndex = 0; TitleIndex < ENodeTitleType::MAX_TitleTypes; ++TitleIndex)
		{
			CachedNodeTitles[TitleIndex].MarkDirty();
		}
	}

	/** */
	inline FText& operator[](ENodeTitleType::Type TitleIndex) const
	{
		return GetCachedTitle(TitleIndex);
	}

private:
	/** */
	FNodeTextCache CachedNodeTitles[ENodeTitleType::MAX_TitleTypes];
};

/*******************************************************************************
* FNodeTextTable
******************************************************************************/

struct FNodeTextTable : FNodeTitleTextTable
{
public:
	/**
	 * Checks if there is text cached
	 *
	 * @param InOwningNode		Node that owns this title
	 */
	inline bool IsTooltipCached(const UEdGraphNode* InOwningNode) const
	{
		const UEdGraphSchema* NullSchema = nullptr;
		return !CachedTooltip.IsOutOfDate(NullSchema);
	}

	/**
	 * Caches the tooltip
	 *
	 * @param InText			Text to cache
	 * @param InOwningNode		Node that owns this title
	 */
	inline void SetCachedTooltip(FText const& InText, const UEdGraphNode* InOwningNode) const
	{
		const UEdGraphSchema* NullSchema = nullptr;
		CachedTooltip.SetCachedText(InText, NullSchema);
	}

	/** */
	inline FText GetCachedTooltip() const
	{
		return CachedTooltip.GetCachedText();
	}

	/** */
	inline void MarkDirty() const
	{
		FNodeTitleTextTable::MarkDirty();
		CachedTooltip.MarkDirty();
	}

private:
	/** */
	FNodeTextCache CachedTooltip;
};

#endif