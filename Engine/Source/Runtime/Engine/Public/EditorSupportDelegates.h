// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

namespace UE
{
namespace Shader
{
struct FValue;
}
}

enum class EMaterialParameterType : uint8;

/** 
 * FEditorSupportDelegates
 * Delegates that are needed for proper editor functionality, but are accessed or triggered in engine code.
 **/
struct FEditorSupportDelegates
{
	/** delegate type for when the editor is about to cleanse an object that *must* be purged ( Params: UObject* Object ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPrepareToCleanseEditorObject, UObject*);
	/** delegate type for force property window rebuild events ( Params: UObject* Object ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnForcePropertyWindowRebuild, UObject*); 
	/** delegate type for material texture setting change events ( Params: UMaterialIterface* Material ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialTextureSettingsChanged, class UMaterialInterface*);	
	/** delegate type for windows messageing events ( Params: FViewport* Viewport, uint32 Message )*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWindowsMessage, class FViewport*, uint32);
	/** delegate type for material usage flags change events ( Params: UMaterial* material, int32 FlagThatChanged ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMaterialUsageFlagsChanged, class UMaterial*, int32); 
	/** delegate type for numeric parameter default change event */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnNumericParameterDefaultChanged, class UMaterialExpression*, EMaterialParameterType, FName, const UE::Shader::FValue&);

	/** Called when all viewports need to be redrawn */
	static ENGINE_API FSimpleMulticastDelegate RedrawAllViewports;
	/** Called when the editor is about to cleanse an object that *must* be purged (such as when changing the active map or level) */
	static ENGINE_API FPrepareToCleanseEditorObject PrepareToCleanseEditorObject;
	/** Called when the editor is cleansing of transient references before a map change event */
	static ENGINE_API FSimpleMulticastDelegate CleanseEditor;
	/** Called when the world is modified */
	static ENGINE_API FSimpleMulticastDelegate WorldChange;
	/** Sent to force a property window rebuild */
	static ENGINE_API FOnForcePropertyWindowRebuild ForcePropertyWindowRebuild;
	/** Sent when events happen that affect how the editors UI looks (mode changes, grid size changes, etc) */
	static ENGINE_API FSimpleMulticastDelegate UpdateUI;
	/** Refresh property windows w/o creating/destroying controls */
	static ENGINE_API FSimpleMulticastDelegate RefreshPropertyWindows;
	/** Sent before the given windows message is handled in the given viewport */
	static ENGINE_API FOnWindowsMessage PreWindowsMessage;
	/** Sent after the given windows message is handled in the given viewport */
	static ENGINE_API FOnWindowsMessage PostWindowsMessage;
	/** Sent after the usages flags on a material have changed*/
	static ENGINE_API FOnMaterialUsageFlagsChanged MaterialUsageFlagsChanged;
	/** Sent after numeric param default changed */
	static ENGINE_API FOnNumericParameterDefaultChanged NumericParameterDefaultChanged;
};

#endif // WITH_EDITOR
