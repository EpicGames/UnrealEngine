// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementData.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
struct FTypedElementHandle;

/**
 * Element data that represents an Actor.
 */
struct FActorElementData
{
	UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FActorElementData);

	ENGINE_API FActorElementData();
	ENGINE_API ~FActorElementData();
	ENGINE_API FActorElementData(const FActorElementData& Other);
	ENGINE_API FActorElementData(FActorElementData&& Other);
	ENGINE_API FActorElementData& operator=(const FActorElementData& Other);
	ENGINE_API FActorElementData& operator=(FActorElementData&& Other);

	UE_DEPRECATED(5.7, "Use ActorWeak")
	AActor* Actor = nullptr;

	TWeakObjectPtr<AActor> ActorWeak;
};

template<> ENGINE_API FString GetTypedElementDebugId<FActorElementData>(const FActorElementData& InElementData);

namespace ActorElementDataUtil
{

/**
 * Attempt to get the actor from the given element handle.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor if the element handle contains FActorElementData, otherwise null.
 */
ENGINE_API AActor* GetActorFromHandle(const FTypedElementHandle& InHandle, const bool bSilent = false);

/**
 * Attempt to get the actor from the given element handle, asserting if the element handle doesn't contain FActorElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor.
 */
ENGINE_API AActor* GetActorFromHandleChecked(const FTypedElementHandle& InHandle);

/**
 * Attempt to get the actors from the given element handles.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actors of any element handles that contain FActorElementData, skipping any that don't.
 */
template <typename ElementHandleType>
TArray<AActor*> GetActorsFromHandles(TArrayView<const ElementHandleType> InHandles, const bool bSilent = false)
{
	TArray<AActor*> Actors;
	Actors.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		if (AActor* Actor = GetActorFromHandle(Handle, bSilent))
		{
			Actors.Add(Actor);
		}
	}

	return Actors;
}

/**
 * Attempt to get the actors from the given element handles, asserting if any element handle doesn't contain FActorElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actors.
 */
template <typename ElementHandleType>
TArray<AActor*> GetActorsFromHandlesChecked(TArrayView<const ElementHandleType> InHandles)
{
	TArray<AActor*> Actors;
	Actors.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		Actors.Add(GetActorFromHandleChecked(Handle));
	}

	return Actors;
}

} // namespace ActorElementDataUtil
