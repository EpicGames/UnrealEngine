// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "ActorElementHierarchyInterface.generated.h"

UCLASS(MinimalAPI)
class UActorElementHierarchyInterface : public UObject, public ITypedElementHierarchyInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual void GetChildElements(const FTypedElementHandle& InElementHandle, TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) override;
};
