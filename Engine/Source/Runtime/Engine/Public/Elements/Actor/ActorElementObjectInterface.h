// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "ActorElementObjectInterface.generated.h"

UCLASS(MinimalAPI)
class UActorElementObjectInterface : public UObject, public ITypedElementObjectInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual UObject* GetObject(const FTypedElementHandle& InElementHandle) override;
};
