// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "ActorElementSelectionInterface.generated.h"

UCLASS(MinimalAPI)
class UActorElementSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()
};
