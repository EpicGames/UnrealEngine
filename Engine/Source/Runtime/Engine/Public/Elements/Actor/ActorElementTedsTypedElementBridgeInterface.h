// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Elements/Interfaces/TedsTypedElementBridgeInterface.h"

#include "ActorElementTedsTypedElementBridgeInterface.generated.h"

UCLASS(MinimalAPI)
class UActorElementTedsTypedElementBridgeInterface : public UObject, public ITedsTypedElementBridgeInterface
{
	GENERATED_BODY()

public:
	virtual FTedsRowHandle GetRowHandle(const FTypedElementHandle& InElementHandle) const override;
};