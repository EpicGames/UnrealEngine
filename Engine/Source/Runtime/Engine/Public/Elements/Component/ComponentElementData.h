// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementData.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class UActorComponent;
struct FTypedElementHandle;

/**
 * Element data that represents an Actor Component.
 */
struct FComponentElementData
{
	UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FComponentElementData);

	ENGINE_API FComponentElementData();
	ENGINE_API ~FComponentElementData();
	ENGINE_API FComponentElementData(const FComponentElementData& Other);
	ENGINE_API FComponentElementData(FComponentElementData&& Other);
	ENGINE_API FComponentElementData& operator=(const FComponentElementData& Other);
	ENGINE_API FComponentElementData& operator=(FComponentElementData&& Other);

	// TypedElementHandles are defer destroyed and can outlive their UObjects, leaving a stale
	// handle to the Component.
	// Use ComponentWeak instead to check for validity before use, especially in code that runs during
	// deferred element handle destruction
	UE_DEPRECATED(5.7, "Use ComponentWeak")
	UActorComponent* Component = nullptr;
	
	TWeakObjectPtr<UActorComponent> ComponentWeak = nullptr;
};

template<> ENGINE_API FString GetTypedElementDebugId<FComponentElementData>(const FComponentElementData& InElementData);

namespace ComponentElementDataUtil
{

/**
 * Attempt to get the actor component from the given element handle.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor component if the element handle contains FComponentElementData, otherwise null.
 */
ENGINE_API UActorComponent* GetComponentFromHandle(const FTypedElementHandle& InHandle, const bool bSilent = false);

/**
 * Attempt to get the actor component from the given element handle, asserting if the element handle doesn't contain FComponentElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor.
 */
ENGINE_API UActorComponent* GetComponentFromHandleChecked(const FTypedElementHandle& InHandle);

/**
 * Attempt to get the actor components from the given element handles.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor components of any element handles that contain FComponentElementData, skipping any that don't.
 */
template <typename ElementHandleType>
TArray<UActorComponent*> GetComponentsFromHandles(TArrayView<const ElementHandleType> InHandles, const bool bSilent = false)
{
	TArray<UActorComponent*> Components;
	Components.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		if (UActorComponent* Component = GetComponentFromHandle(Handle, bSilent))
		{
			Components.Add(Component);
		}
	}

	return Components;
}

/**
 * Attempt to get the actor components from the given element handles, asserting if any element handle doesn't contain FComponentElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The actor components.
 */
template <typename ElementHandleType>
TArray<UActorComponent*> GetComponentsFromHandlesChecked(TArrayView<const ElementHandleType> InHandles)
{
	TArray<UActorComponent*> Components;
	Components.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		Components.Add(GetComponentFromHandleChecked(Handle));
	}

	return Components;
}

} // namespace ComponentElementDataUtil
