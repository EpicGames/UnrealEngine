// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "ComponentElementHierarchyInterface.generated.h"

UCLASS(MinimalAPI)
class UComponentElementHierarchyInterface : public UObject, public ITypedElementHierarchyInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual FTypedElementHandle GetParentElement(const FTypedElementHandle& InElementHandle, const bool bAllowCreate = true) override;
	ENGINE_API virtual void GetChildElements(const FTypedElementHandle& InElementHandle, TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) override;
};
