// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "ComponentElementObjectInterface.generated.h"

UCLASS(MinimalAPI)
class UComponentElementObjectInterface : public UObject, public ITypedElementObjectInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual UObject* GetObject(const FTypedElementHandle& InElementHandle) override;
};
