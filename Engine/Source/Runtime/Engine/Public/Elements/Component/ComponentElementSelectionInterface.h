// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "ComponentElementSelectionInterface.generated.h"

UCLASS(MinimalAPI)
class UComponentElementSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()
};
