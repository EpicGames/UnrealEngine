// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "ComponentElementWorldInterface.generated.h"

UCLASS(MinimalAPI)
class UComponentElementWorldInterface : public UObject, public ITypedElementWorldInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual bool CanEditElement(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual bool IsTemplateElement(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual ULevel* GetOwnerLevel(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual UWorld* GetOwnerWorld(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual bool GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds) override;
	ENGINE_API virtual bool CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType) override;
	ENGINE_API virtual bool CanScaleElement(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual bool GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) override;
	ENGINE_API virtual bool SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) override;
	ENGINE_API virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) override;
	ENGINE_API virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) override;
	ENGINE_API virtual bool FindSuitableTransformAlongPath(const FTypedElementHandle& InElementHandle, const FVector& InPathStart, const FVector& InPathEnd, const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, FTransform& OutSuitableTransform) override;
	ENGINE_API virtual bool AddIgnoredElementToCollisionQueryParams(const FTypedElementHandle& InElementHandle, FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements = true) override;
};
