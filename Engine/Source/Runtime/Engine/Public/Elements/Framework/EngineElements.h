// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

extern ENGINE_API FSimpleMulticastDelegate OnRegisterEngineElementsDelegate;

void RegisterEngineElements();
