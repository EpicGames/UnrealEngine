// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EngineElementsLibrary.generated.h"

class UObject;
struct FObjectElementData;

class AActor;
struct FActorElementData;

class UActorComponent;
struct FComponentElementData;

class UInstancedStaticMeshComponent;
struct FSMInstanceElementData;

struct FSMInstanceId;
struct FSMInstanceElementId;

UCLASS(MinimalAPI)
class UEngineElementsLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	ENGINE_API UEngineElementsLibrary();

	static ENGINE_API TTypedElementOwner<FObjectElementData> CreateObjectElement(const UObject* InObject);
	static ENGINE_API void DestroyObjectElement(const UObject* InObject, TTypedElementOwner<FObjectElementData>& InOutObjectElement);
#if WITH_EDITOR
	static ENGINE_API void CreateEditorObjectElement(const UObject* Object);
	static ENGINE_API void DestroyEditorObjectElement(const UObject* Object);
	static ENGINE_API FTypedElementHandle AcquireEditorObjectElementHandle(const UObject* Object, const bool bAllowCreate = true);
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Object", meta=(DisplayName="Acquire Editor Object Element Handle", ScriptMethod="AcquireEditorElementHandle"))
	static ENGINE_API FScriptTypedElementHandle K2_AcquireEditorObjectElementHandle(const UObject* Object, const bool bAllowCreate = true);
	static ENGINE_API void ReplaceEditorObjectElementHandles(const TMap<const UObject*, const UObject*>& ReplacementObjects);
#endif

	static ENGINE_API TTypedElementOwner<FActorElementData> CreateActorElement(const AActor* InActor);
	static ENGINE_API void DestroyActorElement(const AActor* InActor, TTypedElementOwner<FActorElementData>& InOutActorElement);
	static ENGINE_API void RegisterActorElement(const AActor* InActor);
	static ENGINE_API void UnregisterActorElement(const AActor* InActor);
#if WITH_EDITOR
	static ENGINE_API void CreateEditorActorElement(const AActor* Actor);
	static ENGINE_API void DestroyEditorActorElement(const AActor* Actor);
	static ENGINE_API FTypedElementHandle AcquireEditorActorElementHandle(const AActor* Actor, const bool bAllowCreate = true);
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Actor", meta=(DisplayName="Acquire Editor Actor Element Handle", ScriptMethod="AcquireEditorElementHandle"))
	static ENGINE_API FScriptTypedElementHandle K2_AcquireEditorActorElementHandle(const AActor* Actor, const bool bAllowCreate = true);
	static ENGINE_API void ReplaceEditorActorElementHandles(const TMap<const AActor*, const AActor*>& ReplacementActors);
#endif

	static ENGINE_API TTypedElementOwner<FComponentElementData> CreateComponentElement(const UActorComponent* InComponent);
	static ENGINE_API void DestroyComponentElement(const UActorComponent* InComponent, TTypedElementOwner<FComponentElementData>& InOutComponentElement);
#if WITH_EDITOR
	static ENGINE_API void CreateEditorComponentElement(const UActorComponent* Component);
	static ENGINE_API void DestroyEditorComponentElement(const UActorComponent* Component);
	static ENGINE_API FTypedElementHandle AcquireEditorComponentElementHandle(const UActorComponent* Component, const bool bAllowCreate = true);
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Component", meta=(DisplayName="Acquire Editor Component Element Handle", ScriptMethod = "AcquireEditorElementHandle"))
	static ENGINE_API FScriptTypedElementHandle K2_AcquireEditorComponentElementHandle(const UActorComponent* Component, const bool bAllowCreate = true);
	static ENGINE_API void ReplaceEditorComponentElementHandles(const TMap<const UActorComponent*, const UActorComponent*>& ReplacementComponents);
#endif

	static ENGINE_API TTypedElementOwner<FSMInstanceElementData> CreateSMInstanceElement(const FSMInstanceId& InSMInstanceId);
	static ENGINE_API void DestroySMInstanceElement(const FSMInstanceElementId& InSMInstanceElementId, TTypedElementOwner<FSMInstanceElementData>& InOutSMInstanceElement);
#if WITH_EDITOR
	static ENGINE_API void CreateEditorSMInstanceElement(const FSMInstanceId& SMInstanceId);
	static ENGINE_API void DestroyEditorSMInstanceElement(const FSMInstanceElementId& SMInstanceElementId);
	static ENGINE_API FTypedElementHandle AcquireEditorSMInstanceElementHandle(const UInstancedStaticMeshComponent* ISMComponent, const int32 InstanceIndex, const bool bAllowCreate = true);
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|StaticMeshInstance", meta=(DisplayName="Acquire Editor SMInstance Element Handle", ScriptMethod = "AcquireEditorElementHandle"))
	static ENGINE_API FScriptTypedElementHandle K2_AcquireEditorSMInstanceElementHandle(const UInstancedStaticMeshComponent* ISMComponent, const int32 InstanceIndex, const bool bAllowCreate = true);
	static ENGINE_API FTypedElementHandle AcquireEditorSMInstanceElementHandle(const FSMInstanceId& SMInstanceId, const bool bAllowCreate = true);
	static ENGINE_API FTypedElementHandle AcquireEditorSMInstanceElementHandle(const FSMInstanceElementId& SMInstanceElementId);
	static ENGINE_API void ReplaceEditorSMInstanceElementHandles(const TMap<FSMInstanceId, FSMInstanceId>& ReplacementSMInstanceIds);
#endif

private:
	static TTypedElementOwner<FSMInstanceElementData> CreateSMInstanceElementImpl(const FSMInstanceId& InSMInstanceId, const FSMInstanceElementId& InSMInstanceElementId);
#if WITH_EDITOR
	static void DestroyUnreachableEditorObjectElements();
	static void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementObjects);
	static void ReplaceEditorObjectElementHandlesImpl(const TMap<const UObject*, const UObject*>& ReplacementObjects, TArray<FTypedElementHandle>& OutUpdatedElements);
	static void ReplaceEditorActorElementHandlesImpl(const TMap<const AActor*, const AActor*>& ReplacementActors, TArray<FTypedElementHandle>& OutUpdatedElements);
	static void ReplaceEditorComponentElementHandlesImpl(const TMap<const UActorComponent*, const UActorComponent*>& ReplacementComponents, TArray<FTypedElementHandle>& OutUpdatedElements);
	static void ReplaceEditorSMInstanceElementHandlesImpl(const TMap<FSMInstanceElementId, FSMInstanceElementId>& ReplacementSMInstanceIds, TArray<FTypedElementHandle>& OutUpdatedElements);
#endif
};
