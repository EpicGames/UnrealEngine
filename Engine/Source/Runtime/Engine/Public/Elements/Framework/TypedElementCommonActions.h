// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementListProxy.h"
#include "Elements/Framework/TypedElementInterfaceCustomization.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#include "TypedElementCommonActions.generated.h"

class FStringOutputDevice;
class UTypedElementSelectionSet;

/**
 * Customization used to allow asset editors (such as the level editor) to override the base behavior of common actions.
 */
class FTypedElementCommonActionsCustomization
{
public:
	virtual ~FTypedElementCommonActionsCustomization() = default;

	//~ See UTypedElementCommonActions for API docs
	ENGINE_API virtual bool DeleteElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions);
	ENGINE_API virtual void DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements);
	
	ENGINE_API virtual bool IsCopyCapable() const;
	ENGINE_API virtual bool CanCopyElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles);
	ENGINE_API virtual bool CanDuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles) const;
	ENGINE_API virtual void CopyElements(ITypedElementWorldInterface* InWorldInterface,TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out);
	// By default this just calls CopyElements followed by DeleteElements
	ENGINE_API virtual void CutElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles,
		UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions,
		FOutputDevice& Out);

	ENGINE_API virtual TSharedPtr<FWorldElementPasteImporter> GetPasteImporter(ITypedElementWorldInterface* InWorldInterface, const FTypedElementListConstPtr& InSelectedHandles, UWorld* InWorld);
};

/**
 * Utility to hold a typed element handle and its associated world interface and common actions customization.
 */
struct FTypedElementCommonActionsElement
{
public:
	FTypedElementCommonActionsElement() = default;

	FTypedElementCommonActionsElement(TTypedElement<ITypedElementWorldInterface> InElementWorldHandle, FTypedElementCommonActionsCustomization* InCommonActionsCustomization)
		: ElementWorldHandle(MoveTemp(InElementWorldHandle))
		, CommonActionsCustomization(InCommonActionsCustomization)
	{
	}

	FTypedElementCommonActionsElement(const FTypedElementCommonActionsElement&) = default;
	FTypedElementCommonActionsElement& operator=(const FTypedElementCommonActionsElement&) = default;

	FTypedElementCommonActionsElement(FTypedElementCommonActionsElement&&) = default;
	FTypedElementCommonActionsElement& operator=(FTypedElementCommonActionsElement&&) = default;

	inline explicit operator bool() const
	{
		return IsSet();
	}

	inline bool IsSet() const
	{
		return ElementWorldHandle.IsSet()
			&& CommonActionsCustomization;
	}

	//~ See UTypedElementCommonActions for API docs

private:
	TTypedElement<ITypedElementWorldInterface> ElementWorldHandle;
	FTypedElementCommonActionsCustomization* CommonActionsCustomization = nullptr;
};

USTRUCT(BlueprintType)
struct FTypedElementPasteOptions
{
	GENERATED_BODY()

	// Todo Copy And Paste should we consider supporting pasting with surface snapping?

	// Todo Copy And Paste should we add optional options to handle where to paste (Like some sort of per type paste context for example instance can go under existing actor/components or the partition mode)?

	// If provided the SelectionSet selection will only contains the newly added elements
	UPROPERTY(BlueprintReadWrite, Category = "TypedElementInterfaces|World|PasteOptions")
	TObjectPtr<UTypedElementSelectionSet> SelectionSetToModify;

	UPROPERTY(BlueprintReadWrite, Category = "TypedElementInterfaces|World|PasteOptions")
	bool bPasteAtLocation = false;

	UPROPERTY(BlueprintReadWrite, Category = "TypedElementInterfaces|World|PasteOptions")
	FVector PasteLocation = FVector::ZeroVector;
	
	// If not set, the pasted objects will be at the root of the world.
	TOptional<FTypedElementHandle> PasteUnder;

	// C++ Only. Allow for custom callbacks for some custom top level object in the T3D file
	TMap<FStringView, TFunction<TSharedRef<FWorldElementPasteImporter> ()>> ExtraCustomImport;
};

USTRUCT()
struct FTypedElementCommonActions_CopyPasteCapability
{
	GENERATED_BODY()
	
	// There are handlers that are registered with Common Actions that understand the elements and have the capability of copying them
	bool bCapable = false;
	// The CanCopy is allowing copying to occur
	bool bAllowed = false;
};

/**
 * A utility to handle higher-level common actions, but default via UTypedElementWorldInterface,
 * but asset editors can customize this behavior via FTypedElementCommonActionsCustomization.
 */
UCLASS(Transient, MinimalAPI)
class UTypedElementCommonActions : public UObject, public TTypedElementInterfaceCustomizationRegistry<FTypedElementCommonActionsCustomization>
{
	GENERATED_BODY()

public:
	/**
	 * Delete any elements from the given selection set that can be deleted.
	 * @note Internally this just calls DeleteNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Common")
	ENGINE_API bool DeleteSelectedElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementDeletionOptions& DeletionOptions);
	
	/**
	 * Delete any elements from the given list that can be deleted.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	ENGINE_API bool DeleteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& DeletionOptions);

	/**
	 * Duplicate any elements from the given selection set that can be duplicated.
	 * @note Internally this just calls DuplicateNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	ENGINE_API TArray<FTypedElementHandle> DuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet, UWorld* World, const FVector& LocationOffset);
	ENGINE_API bool CanDuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet);
	
	/**
	 * Duplicate any elements from the given list that can be duplicated.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	ENGINE_API virtual TArray<FTypedElementHandle> DuplicateNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FVector& LocationOffset);

	/**
	 * Given elements within a selection set, determine if it is possible to copy them.
	 * Editor and type specific customizations registered to the SelectionSet are given a chance to use auxillary state to make this decision.
	 */
	ENGINE_API FTypedElementCommonActions_CopyPasteCapability CanCopySelectedElements(const UTypedElementSelectionSet* SelectionSet);
	
	/**
	 * Copy any elements from the given selection set that can be copied into the clipboard
	 * @note Internally this just calls CopyNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API bool CopySelectedElements(UTypedElementSelectionSet* SelectionSet);

	/**
	 * Copy any elements from the given selection set that can be copied into the string
	 * @note Internally this just calls CopyNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API bool CopySelectedElementsToString(UTypedElementSelectionSet* SelectionSet, FString& OutputString);

	/*
	 * Copy any elements from the given selection set that can be copied into the clipboard.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	ENGINE_API virtual bool CopyNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, FString* OptionalOutputString = nullptr);

	/**
	 * Cut any elements from the given selection set. This typically amounts to copying any elements that can be copied
	 *  into the clipboard and then deleting them, but it does go through a customization endpoint that may adjust the
	 *  behavior (e.g. for actors, this fires a different delegate than copy/delete would).
	 */
	ENGINE_API bool CutSelectedElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementDeletionOptions& DeletionOptions, FString* OptionalOutputString = nullptr);

	/**
	 * Given an input string (or use clipboard if null string), determine if it is possible to use the string to create elements.
	 * Editor and type specific customizations registered to the SelectionSet are given a chance to use auxillary state to make this decision.
	 */
	ENGINE_API FTypedElementCommonActions_CopyPasteCapability CanPasteElements(const FTypedElementPasteOptions& PasteOption, const FString* OptionalInputString = nullptr);
	
	/**
	 * Paste any elements from the given string or from the clipboard
	 * @note Internally this just calls PasteNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	ENGINE_API TArray<FTypedElementHandle> PasteElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString* OptionalInputString = nullptr);

	/**
	 * Paste any elements from the given string or from the clipboard
	  * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	ENGINE_API virtual TArray<FTypedElementHandle> PasteNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, UWorld* World, const FTypedElementPasteOptions& PasteOptions, const FString* OptionalInputString = nullptr);

	/**
	 * Script Api
	 */

	/**
	 * Delete any elements from the given list that can be deleted.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Common")
	ENGINE_API bool DeleteNormalizedElements(const FScriptTypedElementListProxy ElementList, UWorld* World, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& DeletionOptions);

	/**
	 * Duplicate any elements from the given selection set that can be duplicated.
	 * @note Internally this just calls DuplicateNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, DisplayName="Duplicate Selected Elements", Category = "TypedElementFramework|Common", meta=(ScriptName="DuplicateSelectedElements"))
	ENGINE_API TArray<FScriptTypedElementHandle> K2_DuplicateSelectedElements(const UTypedElementSelectionSet* SelectionSet, UWorld* World, const FVector& LocationOffset);
	
	/**
	 * Duplicate any elements from the given list that can be duplicated.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API TArray<FScriptTypedElementHandle> DuplicateNormalizedElements(const FScriptTypedElementListProxy ElementList, UWorld* World, const FVector& LocationOffset);
	
	/*
	 * Copy any elements from the given selection set that can be copied into the clipboard.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API bool CopyNormalizedElements(const FScriptTypedElementListProxy& ElementList);

	/*
	 * Copy any elements from the given selection set that can be copied into the clipboard.
	 * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API bool CopyNormalizedElementsToString(const FScriptTypedElementListProxy& ElementList, FString& OutputString);

	/**
	 * Paste any elements from the clipboard
	 * @note Internally this just calls PasteNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, DisplayName="Paste Elements", Category = "TypedElementFramework|Common", meta=(ScriptName="PasteElements"))
	ENGINE_API TArray<FScriptTypedElementHandle> K2_PasteElements(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption);

	/**
	 * Paste any elements from the given string
	 * @note Internally this just calls PasteNormalizedElements on the result of UTypedElementSelectionSet::GetNormalizedSelection.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API TArray<FScriptTypedElementHandle> PasteElementsFromString(UTypedElementSelectionSet* SelectionSet, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString& InputString);

	/**
	 * Paste any elements from the clipboard
	  * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, DisplayName="Paste Normalized Elements", Category = "TypedElementFramework|Common", meta=(ScriptName="PasteNormalizedElements"))
	ENGINE_API TArray<FScriptTypedElementHandle> K2_PasteNormalizedElements(const FScriptTypedElementListProxy& ElementList, UWorld* World, const FTypedElementPasteOptions& PasteOption);

	/**
	 * Paste any elements from the given string
	  * @note This list should have been pre-normalized via UTypedElementSelectionSet::GetNormalizedSelection or UTypedElementSelectionSet::GetNormalizedElementList.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Common")
	ENGINE_API TArray<FScriptTypedElementHandle> PasteNormalizedElementsFromString(const FScriptTypedElementListProxy& ElementList, UWorld* World, const FTypedElementPasteOptions& PasteOption, const FString& InputString);

private:
	/**
	 * Helper function to parse paste text and create importers for each element type.
	 * Used by both CanPasteElements and PasteNormalizedElements to avoid code duplication.
	 */
	TArray<TPair<TSharedRef<FWorldElementPasteImporter>, FStringView>> ParsePasteTextAndCreateImporters(
		const FTypedElementPasteOptions& PasteOptions,
		const FString& InputString,
		UTypedElementRegistry* Registry,
		const FTypedElementListConstPtr& ElementListPtr,
		UWorld* World);
	
	/**
	 * Attempt to resolve the selection interface and common actions customization for the given element, if any.
	 */
	FTypedElementCommonActionsElement ResolveCommonActionsElement(const FTypedElementHandle& InElementHandle) const;

	// Helper to share code between copy and cut. The cut/copy operation is passed in as a lambda
	bool CutOrCopyNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, FString* OptionalOutputString,
		TFunctionRef<void(FTypedElementCommonActionsCustomization& CommonActionsCustomization, 
			ITypedElementWorldInterface& WorldInterface, const TArray<FTypedElementHandle>& ElementsIn, FStringOutputDevice& Output)> CutOrCopy);

	// We might expose this as we do for copy/delete, but we can wait to do so until needed.
	bool CutNormalizedElements(const FTypedElementListConstPtr& ElementListPtr, 
		UWorld* World, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& DeletionOptions, FString* OptionalOutputString = nullptr);
};

namespace TypedElementCommonActionsUtils
{
	/**
	 * Is the elements Copy and paste currently enabled?
	 */
	ENGINE_API bool IsElementCopyAndPasteEnabled();
}
