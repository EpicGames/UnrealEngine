// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"

#include "TedsTypedElementBridgeInterface.generated.h"

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UTedsTypedElementBridgeInterface : public UInterface
{
	GENERATED_BODY()
};

class ITedsTypedElementBridgeInterface
{
	GENERATED_BODY()

public:
	virtual FTedsRowHandle GetRowHandle(const FTypedElementHandle& InElementHandle) const { return FTedsRowHandle(); }
};

template <>
struct TTypedElement<ITedsTypedElementBridgeInterface> : public TTypedElementBase<ITedsTypedElementBridgeInterface>
{
	FTedsRowHandle GetRowHandle() const { return InterfacePtr->GetRowHandle(*this); }
};
