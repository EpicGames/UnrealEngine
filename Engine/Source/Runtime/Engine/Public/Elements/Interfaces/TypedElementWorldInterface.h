// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"

#include "Elements/Interfaces/TypedElementSelectionInterface.h"

#include "CoreFwd.h"
#include "UObject/Interface.h"
#include "Templates/SharedPointer.h"

#include "TypedElementWorldInterface.generated.h"

class ULevel;
class UTypedElementSelectionSet; 
class UWorld;
struct FCollisionShape;
struct FCollisionQueryParams;
struct FConvexVolume;
struct FEngineShowFlags;
struct FWorldSelectionElementArgs;

UENUM()
enum class ETypedElementWorldType : uint8
{
	Game,
	Editor,
};

USTRUCT(BlueprintType)
struct FTypedElementDeletionOptions
{
	GENERATED_BODY()

public:
	FTypedElementDeletionOptions& SetVerifyDeletionCanHappen(const bool InVerifyDeletionCanHappen) { bVerifyDeletionCanHappen = InVerifyDeletionCanHappen; return *this; }
	bool VerifyDeletionCanHappen() const { return bVerifyDeletionCanHappen; }

	FTypedElementDeletionOptions& SetWarnAboutReferences(const bool InWarnAboutReferences) { bWarnAboutReferences = InWarnAboutReferences; return *this; }
	bool WarnAboutReferences() const { return bWarnAboutReferences; }

	FTypedElementDeletionOptions& SetWarnAboutSoftReferences(const bool InWarnAboutSoftReferences) { bWarnAboutSoftReferences = InWarnAboutSoftReferences; return *this; }
	bool WarnAboutSoftReferences() const { return bWarnAboutSoftReferences; }
	
private:
	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|World|DeletionOptions", meta=(AllowPrivateAccess=true))
	bool bVerifyDeletionCanHappen = false;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|World|DeletionOptions", meta=(AllowPrivateAccess=true))
	bool bWarnAboutReferences = true;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|World|DeletionOptions", meta=(AllowPrivateAccess=true))
	bool bWarnAboutSoftReferences = true;
};

struct FWorldElementPasteImporter
{
	virtual ~FWorldElementPasteImporter() = default;

	struct FContext
	{
		FTypedElementListConstPtr CurrentSelection;
		UWorld* World;
		FTypedElementHandle PasteUnder;
		FStringView Text;
	};
	
	virtual void Import(FContext& Context) {}
	
	virtual TArray<FTypedElementHandle> GetImportedElements() { return {}; }
};

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementWorldInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementWorldInterface
{
	GENERATED_BODY()

public:
	/**
	 * Is this element considered a template within its world (eg, a CDO or archetype).
	 */
	virtual bool IsTemplateElement(const FTypedElementHandle& InElementHandle) { return false; }

	/**
	 * Can this element actually be edited in the world?
	 */
	virtual bool CanEditElement(const FTypedElementHandle& InElementHandle) { return true; }

	/**
	 * Get the owner level associated with this element, if any.
	 */
	virtual ULevel* GetOwnerLevel(const FTypedElementHandle& InElementHandle) { return nullptr; }

	/**
	 * Get the owner world associated with this element, if any.
	 */
	virtual UWorld* GetOwnerWorld(const FTypedElementHandle& InElementHandle) { return nullptr; }

	/**
	 * Get the bounds of this element, if any.
	 */
	virtual bool GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds) { return false; }

	/**
	 * Can the given element be moved within the world?
	 */
	virtual bool CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType) { return false; }

	/**
	 * Can the given element be scaled?
	 */
	virtual bool CanScaleElement(const FTypedElementHandle& InElementHandle) { return true; }

	/**
	 * Get the transform of this element within its owner world, if any.
	 */
	virtual bool GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) { return false; }
	
	/**
	 * Attempt to set the transform of this element within its owner world.
	 */
	virtual bool SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) { return false; }

	/**
	 * Get the transform of this element relative to its parent, if any.
	 */
	virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) { return GetWorldTransform(InElementHandle, OutTransform); }
	
	/**
	 * Attempt to set the transform of this element relative to its parent.
	 */
	virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) { return SetWorldTransform(InElementHandle, InTransform); }

	/**
	 * Get the local space offset of this element that should be added to its pivot location, if any.
	 */
	virtual bool GetPivotOffset(const FTypedElementHandle& InElementHandle, FVector& OutPivotOffset) { return false; }

	/**
	 * Attempt to set the local space offset of this element that should be added to its pivot location.
	 */
	virtual bool SetPivotOffset(const FTypedElementHandle& InElementHandle, const FVector& InPivotOffset) { return false; }

	/**
	 * Notify that this element is about to be moved.
	 */
	virtual void NotifyMovementStarted(const FTypedElementHandle& InElementHandle) {}

	/**
	 * Notify that this element is currently being moved.
	 */
	virtual void NotifyMovementOngoing(const FTypedElementHandle& InElementHandle) {}

	/**
	 * Notify that this element is done being moved.
	 */
	virtual void NotifyMovementEnded(const FTypedElementHandle& InElementHandle) {}

	/**
	 * Attempt to find a suitable (non-intersecting) transform for the given element at the given point.
	 */
	virtual bool FindSuitableTransformAtPoint(const FTypedElementHandle& InElementHandle, const FTransform& InPotentialTransform, FTransform& OutSuitableTransform)
	{
		OutSuitableTransform = InPotentialTransform;
		return true;
	}

	/**
	 * Attempt to find a suitable (non-intersecting) transform for the given element along the given path.
	 */
	virtual bool FindSuitableTransformAlongPath(const FTypedElementHandle& InElementHandle, const FVector& InPathStart, const FVector& InPathEnd, const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, FTransform& OutSuitableTransform)
	{
		return false;
	}

	/**
	 * Attempt to modify FCollisionQueryParams so that the given element (and optionally any attached 
	 * sub-elements, if the element is part of a hierarchy) is ignored in the query. 
	 *
	 * For instance, this can be used to support dragging elements into the scene and raycasting for a
	 * target location without hitting the preview.
	 * 
	 * @return true if the collision query parameters were modified.
	 */
	virtual bool AddIgnoredElementToCollisionQueryParams(const FTypedElementHandle& InElementHandle,
		FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements = true)
	{
		return false;
	}

	/**
	 * Can the given element be deleted?
	 */
	virtual bool CanDeleteElement(const FTypedElementHandle& InElementHandle) { return false; }

	/**
	 * Delete the given element.
	 * @note Default version calls DeleteElements with a single element.
	 */
	virtual bool DeleteElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
	{
		return DeleteElements(MakeArrayView(&InElementHandle, 1), InWorld, InSelectionSet, InDeletionOptions);
	}

	/**
	 * Delete the given set of elements.
	 * @note If you want to delete an array of elements that are potentially different types, you probably want to use the higher-level UTypedElementCommonActions::DeleteNormalizedElements function instead.
	 */
	virtual bool DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) { return false; }

	/**
	 * Can the given element be duplicated?
	 */
	virtual bool CanDuplicateElement(const FTypedElementHandle& InElementHandle) { return false; }

	/**
	 * Duplicate the given element.
	 * @note Default version calls DuplicateElements with a single element.
	 */
	virtual FTypedElementHandle DuplicateElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, const FVector& InLocationOffset)
	{
		TArray<FTypedElementHandle> NewElements;
		DuplicateElements(MakeArrayView(&InElementHandle, 1), InWorld, InLocationOffset, NewElements);
		return NewElements.Num() > 0 ? MoveTemp(NewElements[0]) : FTypedElementHandle();
	}

	/**
	 * Duplicate the given set of elements.
	 * @note If you want to duplicate an array of elements that are potentially different types, you probably want to use the higher-level UTypedElementCommonActions::DuplicateNormalizedElements function instead.
	 */
	virtual void DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
	{
	}

	virtual bool CanCopyElement(const FTypedElementHandle& InElementHandle) { return false; }

	/**
	 * Copy the given element into a object to export
	 * 
	 * @note Default version calls CopyElements with a single element.
	 */
	virtual void CopyElement(const FTypedElementHandle& InElementHandle, FOutputDevice& Out)
	{
		CopyElements(MakeArrayView(&InElementHandle, 1), Out);
	}

	/**
	 * Copy the given set of elements into a object to export.
	 * @note If you want to copy an array of elements that are potentially different types, you probably want to use the higher-level UTypedElementCommonActions::CopyNormalizedElements function instead.
	 */
	virtual void CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
	{
	}
	
	virtual TSharedPtr<FWorldElementPasteImporter> GetPasteImporter()
	{
		return {};
	}

	/**
	 * Can the element be promoted
	 * Generally available when the element is lighter representation of another element.
	 * Like promoting a static mesh instance from an instanced static static component to a static actor for example.
	 */
	virtual bool CanPromoteElement(const FTypedElementHandle& InElementHandle)
	{
		return false;
	}

	/**
	 * Promote a element when possible
	 * Generally available when the element is lighter representation of another element.
	 * Like promoting a static mesh instance from an instanced static static component to a static actor for example.
	 * 
	 * @param OverrideWorld Override the world in which the promotion might create new elements. Leave it to null to use the world from the handle.
	 */
	virtual FTypedElementHandle PromoteElement(const FTypedElementHandle& InElementHandle, UWorld* OverrideWorld = nullptr)
	{
		return FTypedElementHandle();
	}

	/**
	 * Return true if the element is in the volume
	 * Note: This should only be use for editor tools since it doesn't use the physics system and the performance would probably be to slow for a runtime application.
	 */
	ENGINE_API virtual bool IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement = false);

	/**
	 * Return true if the element is in the Box
	 * Note: This should only be use for editor tools since it doesn't use the physics system and the performance would probably be to slow for a runtime application.
	 */
	ENGINE_API virtual bool IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement = false);

	/**
	 * Get the selectable elements from this element that are inside the the convex volume
	 */
	ENGINE_API TArray<FTypedElementHandle> GetSelectionElementsInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, const FWorldSelectionElementArgs& SelectionArgs);

	/**
	 * Get the selectable elements from this element that are inside the the convex volume
	 */
	ENGINE_API TArray<FTypedElementHandle> GetSelectionElementsInBox(const FTypedElementHandle& Handle, const FBox& InBox, const FWorldSelectionElementArgs& SelectionArgs);

	ENGINE_API virtual TArray<FTypedElementHandle> GetSelectionElementsFromSelectionFunction(
		const FTypedElementHandle& Handle,
		const FWorldSelectionElementArgs& SelectionArgs,
		const TFunction<bool(const FTypedElementHandle& /*ElementHandle*/, const FWorldSelectionElementArgs& /*SelectionArgs*/)>& SelectionFunction
		);


	/**
	 * Script Api
	 */

	/**
	 * Is this element considered a template within its world (eg, a CDO or archetype).
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool IsTemplateElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Can this element actually be edited in the world?
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanEditElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Get the owner level associated with this element, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual ULevel* GetOwnerLevel(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Get the owner world associated with this element, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual UWorld* GetOwnerWorld(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Get the bounds of this element, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool GetBounds(const FScriptTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds);

	/**
	 * Can the given element be moved within the world?
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanMoveElement(const FScriptTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType);

	/**
	 * Can the given element be scaled?
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanScaleElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Get the transform of this element within its owner world, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool GetWorldTransform(const FScriptTypedElementHandle& InElementHandle, FTransform& OutTransform);
	
	/**
	 * Attempt to set the transform of this element within its owner world.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool SetWorldTransform(const FScriptTypedElementHandle& InElementHandle, const FTransform& InTransform);

	/**
	 * Get the transform of this element relative to its parent, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool GetRelativeTransform(const FScriptTypedElementHandle& InElementHandle, FTransform& OutTransform);
	
	/**
	 * Attempt to set the transform of this element relative to its parent.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool SetRelativeTransform(const FScriptTypedElementHandle& InElementHandle, const FTransform& InTransform);

	/**
	 * Get the local space offset of this element that should be added to its pivot location, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool GetPivotOffset(const FScriptTypedElementHandle& InElementHandle, FVector& OutPivotOffset);

	/**
	 * Attempt to set the local space offset of this element that should be added to its pivot location.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool SetPivotOffset(const FScriptTypedElementHandle& InElementHandle, const FVector& InPivotOffset);

	/**
	 * Notify that this element is about to be moved.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual void NotifyMovementStarted(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Notify that this element is currently being moved.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual void NotifyMovementOngoing(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Notify that this element is done being moved.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual void NotifyMovementEnded(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Can the given element be deleted?
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanDeleteElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Delete the given element.
	 * @note Default version calls DeleteElements with a single element.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool DeleteElement(const FScriptTypedElementHandle& InElementHandle, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions);

	/**
	 * Can the given element be duplicated?
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanDuplicateElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Duplicate the given element.
	 * @note Default version calls DuplicateElements with a single element.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual FScriptTypedElementHandle DuplicateElement(const FScriptTypedElementHandle& InElementHandle, UWorld* InWorld, const FVector& InLocationOffset);

	/**
	 * Can the element be promoted
	 * Generally available when the element is a lighter representation of another element.
	 * Like an instance for example.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual bool CanPromoteElement(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Promote an element when possible
	 * Generally available when the element is a lighter representation of another element.
	 * Like an instance for example.
	 * 
	 * @param OverrideWorld Override the world in which the promotion might create new elements. Leave it to null to use the world from the handle.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|World")
	ENGINE_API virtual FScriptTypedElementHandle PromoteElement(const FScriptTypedElementHandle& InElementHandle, UWorld* OverrideWorld = nullptr);

protected:
	/**
	 * Return the registry associated with this interface implementation
	 */
	ENGINE_API virtual class UTypedElementRegistry& GetRegistry() const;
};

template <>
struct TTypedElement<ITypedElementWorldInterface> : public TTypedElementBase<ITypedElementWorldInterface>
{
	bool IsTemplateElement() const { return InterfacePtr->IsTemplateElement(*this); }
	bool CanEditElement() const { return InterfacePtr->CanEditElement(*this); }
	ULevel* GetOwnerLevel() const { return InterfacePtr->GetOwnerLevel(*this); }
	UWorld* GetOwnerWorld() const { return InterfacePtr->GetOwnerWorld(*this); }
	bool GetBounds(FBoxSphereBounds& OutBounds) const { return InterfacePtr->GetBounds(*this, OutBounds); }
	bool CanMoveElement(const ETypedElementWorldType InWorldType) const { return InterfacePtr->CanMoveElement(*this, InWorldType); }
	bool CanScaleElement() const { return InterfacePtr->CanScaleElement(*this); }
	bool GetWorldTransform(FTransform& OutTransform) const { return InterfacePtr->GetWorldTransform(*this, OutTransform); }
	bool SetWorldTransform(const FTransform& InTransform) const { return InterfacePtr->SetWorldTransform(*this, InTransform); }
	bool GetRelativeTransform(FTransform& OutTransform) const { return InterfacePtr->GetRelativeTransform(*this, OutTransform); }
	bool SetRelativeTransform(const FTransform& InTransform) const { return InterfacePtr->SetRelativeTransform(*this, InTransform); }
	bool GetPivotOffset(FVector& OutPivotOffset) const { return InterfacePtr->GetPivotOffset(*this, OutPivotOffset); }
	bool SetPivotOffset(const FVector& InPivotOffset) const { return InterfacePtr->SetPivotOffset(*this, InPivotOffset); }
	void NotifyMovementStarted() const { InterfacePtr->NotifyMovementStarted(*this); }
	void NotifyMovementOngoing() const { InterfacePtr->NotifyMovementOngoing(*this); }
	void NotifyMovementEnded() const { InterfacePtr->NotifyMovementEnded(*this); }
	bool FindSuitableTransformAtPoint(const FTransform& InPotentialTransform, FTransform& OutSuitableTransform) const { return InterfacePtr->FindSuitableTransformAtPoint(*this, InPotentialTransform, OutSuitableTransform); }
	bool FindSuitableTransformAlongPath(const FVector& InPathStart, const FVector& InPathEnd, const FCollisionShape& InTestShape, TArrayView<const FTypedElementHandle> InElementsToIgnore, FTransform& OutSuitableTransform) const { return InterfacePtr->FindSuitableTransformAlongPath(*this, InPathStart, InPathEnd, InTestShape, InElementsToIgnore, OutSuitableTransform); }
	UE_DEPRECATED(5.8, "The element handle is already included in TTypedElement access. Use the version of this function that does not take the FTypedElementHandle parameter.")
	bool AddIgnoredElementToCollisionQueryParams(const FTypedElementHandle& InElementHandle, FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements = true) const { return AddIgnoredElementToCollisionQueryParams(InOutParams, bAlsoIgnoreSubElements); }
	bool AddIgnoredElementToCollisionQueryParams(FCollisionQueryParams& InOutParams, bool bAlsoIgnoreSubElements = true) const { return InterfacePtr->AddIgnoredElementToCollisionQueryParams(*this, InOutParams, bAlsoIgnoreSubElements); }
	bool CanDeleteElement() const { return InterfacePtr->CanDeleteElement(*this); }
	bool DeleteElement(UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) const { return InterfacePtr->DeleteElement(*this, InWorld, InSelectionSet, InDeletionOptions); }
	bool CanDuplicateElement() const { return InterfacePtr->CanDuplicateElement(*this); }
	FTypedElementHandle DuplicateElement(UWorld* InWorld, const FVector& InLocationOffset) const { return InterfacePtr->DuplicateElement(*this, InWorld, InLocationOffset); }
	bool CanCopyElement() const { return InterfacePtr->CanCopyElement(*this); }
	void CopyElement(FOutputDevice& Out) const { InterfacePtr->CopyElement(*this, Out); }
	bool CanPromoteElement() const { return InterfacePtr->CanPromoteElement(*this); }
	FTypedElementHandle PromoteElement(UWorld* OverrideWorld = nullptr) const { return InterfacePtr->PromoteElement(*this, OverrideWorld); }
	bool IsElementInConvexVolume(const FConvexVolume& InVolume, bool bMustEncompassEntireElement = false) const { return InterfacePtr->IsElementInConvexVolume(*this, InVolume, bMustEncompassEntireElement); }
	bool IsElementInBox(const FBox& InBox, bool bMustEncompassEntireElement = false) const { return InterfacePtr->IsElementInBox(*this, InBox, bMustEncompassEntireElement); }
	TArray<FTypedElementHandle> GetSelectionElementsInConvexVolume(const FConvexVolume& InVolume, const FWorldSelectionElementArgs& SelectionArgs) const { return InterfacePtr->GetSelectionElementsInConvexVolume(*this, InVolume, SelectionArgs); }
	TArray<FTypedElementHandle> GetSelectionElementsInBox(const FBox& InBox, const FWorldSelectionElementArgs& SelectionArgs) const { return InterfacePtr->GetSelectionElementsInBox(*this, InBox, SelectionArgs); }
	TArray<FTypedElementHandle> GetSelectionElementsFromSelectionFunction(const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction) const
	{
		return InterfacePtr->GetSelectionElementsFromSelectionFunction(*this, SelectionArgs, SelectionFunction);
	}
};

struct FWorldSelectionElementArgs
{
	// Optional selection set to use to get the selectable elements only
	const UTypedElementSelectionSet* SelectionSet;

	ETypedElementSelectionMethod SelectionMethod = ETypedElementSelectionMethod::Primary;

	FTypedElementSelectionOptions SelectionOptions;

	// Optional show flag to avoid selecting non rendered elements
	FEngineShowFlags* ShowFlags;

	bool bMustEncompassEntireElement = false;

	// To deprecate once the BSP tools are removed from the engine
	bool bBSPSelectionOnly = false;
};
