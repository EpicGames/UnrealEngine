// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementCounter.h"
#include "ObjectElementCounterInterface.generated.h"

UCLASS(MinimalAPI)
class UObjectElementCounterInterface : public UObject, public ITypedElementCounterInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual void IncrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter) override;
	ENGINE_API virtual void DecrementCountersForElement(const FTypedElementHandle& InElementHandle, FTypedElementCounter& InOutCounter) override;

	static ENGINE_API void IncrementCounterForObjectClass(const UObject* InObject, FTypedElementCounter& InOutCounter);
	static ENGINE_API void DecrementCounterForObjectClass(const UObject* InObject, FTypedElementCounter& InOutCounter);
};
