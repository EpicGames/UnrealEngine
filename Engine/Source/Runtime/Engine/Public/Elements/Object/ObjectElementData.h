// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementData.h"
#include "UObject/Object.h"

struct FTypedElementHandle;

/**
 * Element data that represents an Object.
 */
struct FObjectElementData
{
	UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FObjectElementData);

	UObject* Object = nullptr;
};

template <>
inline FString GetTypedElementDebugId<FObjectElementData>(const FObjectElementData& InElementData)
{
	UObject* Object = InElementData.Object;
	return Object
		? Object->GetFullName()
		: TEXT("null");
}

namespace ObjectElementDataUtil
{

/**
 * Attempt to get the object from the given element handle.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The object if the element handle contains FObjectElementData, otherwise null.
 */
ENGINE_API UObject* GetObjectFromHandle(const FTypedElementHandle& InHandle, const bool bSilent = false);

/**
 * Attempt to get the object from the given element handle, asserting if the element handle doesn't contain FObjectElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The object.
 */
ENGINE_API UObject* GetObjectFromHandleChecked(const FTypedElementHandle& InHandle);

/**
 * Attempt to get the objects from the given element handles.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The objects of any element handles that contain FObjectElementData, skipping any that don't.
 */
template <typename ElementHandleType>
TArray<UObject*> GetObjectsFromHandles(TArrayView<const ElementHandleType> InHandles, const bool bSilent = false)
{
	TArray<UObject*> Objects;
	Objects.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		if (UObject* Object = GetObjectFromHandle(Handle, bSilent))
		{
			Objects.Add(Object);
		}
	}

	return Objects;
}

/**
 * Attempt to get the objects from the given element handles, asserting if any element handle doesn't contain FObjectElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The objects.
 */
template <typename ElementHandleType>
TArray<UObject*> GetObjectsFromHandlesChecked(TArrayView<const ElementHandleType> InHandles)
{
	TArray<UObject*> Objects;
	Objects.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		Objects.Add(GetObjectFromHandleChecked(Handle));
	}

	return Objects;
}

} // namespace ObjectElementDataUtil
