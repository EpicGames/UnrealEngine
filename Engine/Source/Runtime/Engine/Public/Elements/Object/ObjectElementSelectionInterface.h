// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "ObjectElementSelectionInterface.generated.h"

UCLASS(MinimalAPI)
class UObjectElementSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()

public:
};
