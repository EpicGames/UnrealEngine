// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementAssetDataInterface.h"

#include "SMInstanceElementAssetDataInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementAssetDataInterface : public UObject, public ITypedElementAssetDataInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual FAssetData GetAssetData(const FTypedElementHandle& InElementHandle) override;
};
