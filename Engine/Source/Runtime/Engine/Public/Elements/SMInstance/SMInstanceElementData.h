// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementData.h"
#include "Elements/SMInstance/SMInstanceManager.h"
#include "UObject/Object.h"

struct FTypedElementHandle;

/**
 * Element data that represents a specific instance within an ISM.
 */
struct FSMInstanceElementData
{
	ENGINE_API UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FSMInstanceElementData);

	FSMInstanceElementId InstanceElementId;
};

template <>
inline FString GetTypedElementDebugId<FSMInstanceElementData>(const FSMInstanceElementData& InElementData)
{
	UObject* Object = (UObject*)InElementData.InstanceElementId.ISMComponent;
	return Object
		? Object->GetFullName()
		: TEXT("null");
}

namespace SMInstanceElementDataUtil
{

/**
 * Test whether static mesh instance elements are currently enabled?
 * @note Controlled by the CVar: "TypedElements.EnableSMInstanceElements".
 */
ENGINE_API bool SMInstanceElementsEnabled();

ENGINE_API FSimpleMulticastDelegate& OnSMInstanceElementsEnabledChanged();

/**
 * Get the static mesh instance manager for the given instance.
 * @return The static mesh instance manager, or null if this instance cannot be managed.
 */
ENGINE_API ISMInstanceManager* GetSMInstanceManager(const FSMInstanceId& InstanceId);

/**
 * Attempt to get the static mesh instance ID from the given element handle.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The static mesh instance ID if the element handle contains FSMInstanceElementData which resolved to a valid FSMInstanceManager, otherwise an invalid ID.
 */
ENGINE_API FSMInstanceManager GetSMInstanceFromHandle(const FTypedElementHandle& InHandle, const bool bSilent = false);

/**
 * Attempt to get the static mesh instance ID from the given element handle, asserting if the element handle doesn't contain FSMInstanceElementData.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The static mesh instance ID, or an invalid ID if the FSMInstanceElementData didn't resolve to a valid FSMInstanceManager.
 */
ENGINE_API FSMInstanceManager GetSMInstanceFromHandleChecked(const FTypedElementHandle& InHandle);

/**
 * Attempt to get the static mesh instance IDs from the given element handles.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The static mesh instance IDs of any element handles that contain FSMInstanceElementData which resolves to a valid FSMInstanceManager, skipping any that don't.
 */
template <typename ElementHandleType>
TArray<FSMInstanceManager> GetSMInstancesFromHandles(TArrayView<const ElementHandleType> InHandles, const bool bSilent = false)
{
	TArray<FSMInstanceManager> SMInstanceIds;
	SMInstanceIds.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		if (FSMInstanceManager SMInstanceId = GetSMInstanceFromHandle(Handle, bSilent))
		{
			SMInstanceIds.Add(MoveTemp(SMInstanceId));
		}
	}

	return SMInstanceIds;
}

/**
 * Attempt to get the static mesh instance IDs from the given element handles, asserting if any element handle doesn't contain FSMInstanceElementData, and skipping any that don't resolve to a valid FSMInstanceManager.
 * @note This is not typically something you'd want to use outside of data access within an interface implementation.
 * @return The static mesh instance IDs.
 */
template <typename ElementHandleType>
TArray<FSMInstanceManager> GetSMInstancesFromHandlesChecked(TArrayView<const ElementHandleType> InHandles)
{
	TArray<FSMInstanceManager> SMInstanceIds;
	SMInstanceIds.Reserve(InHandles.Num());

	for (const FTypedElementHandle& Handle : InHandles)
	{
		if (FSMInstanceManager SMInstanceId = GetSMInstanceFromHandleChecked(Handle))
		{
			SMInstanceIds.Add(MoveTemp(SMInstanceId));
		}
	}

	return SMInstanceIds;
}

} // namespace SMInstanceElementDataUtil
