// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "SMInstanceElementHierarchyInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementHierarchyInterface : public UObject, public ITypedElementHierarchyInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual FTypedElementHandle GetParentElement(const FTypedElementHandle& InElementHandle, const bool bAllowCreate = true) override;
};
