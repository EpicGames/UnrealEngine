// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "InstancedStaticMeshDelegates.h"
#include "SMInstanceElementId.generated.h"

class FSMInstanceElementIdMap;
class UInstancedStaticMeshComponent;
class USMInstanceElementIdMapTransactor;

ENGINE_API extern const FName NAME_SMInstance;

/**
 * ID for a specific instance within an ISM, mapped from its instance index.
 */
struct FSMInstanceId
{
	explicit operator bool() const
	{
		return ISMComponent != nullptr
			&& InstanceIndex != INDEX_NONE;
	}

	bool operator==(const FSMInstanceId& InRHS) const
	{
		return ISMComponent == InRHS.ISMComponent
			&& InstanceIndex == InRHS.InstanceIndex;
	}

	bool operator!=(const FSMInstanceId& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend inline uint32 GetTypeHash(const FSMInstanceId& InId)
	{
		return HashCombine(GetTypeHash(InId.InstanceIndex), GetTypeHash(InId.ISMComponent));
	}

	UInstancedStaticMeshComponent* ISMComponent = nullptr;
	int32 InstanceIndex = INDEX_NONE;
};

/**
 * ID for a specific instance within an ISM, mapped from the instance ID used by typed elements.
 * @note The specific implementation of instance IDs for typed elements is considered private and may change without warning!
 *       Use the FSMInstanceElementIdMap functions to convert this struct to/from a FSMInstanceId rather than access this data directly!
 */
struct FSMInstanceElementId
{
	explicit operator bool() const
	{
		return ISMComponent != nullptr
			&& InstanceId != 0;
	}

	bool operator==(const FSMInstanceElementId& InRHS) const
	{
		return ISMComponent == InRHS.ISMComponent
			&& InstanceId == InRHS.InstanceId;
	}

	bool operator!=(const FSMInstanceElementId& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend inline uint32 GetTypeHash(const FSMInstanceElementId& InId)
	{
		return HashCombine(GetTypeHash(InId.InstanceId), GetTypeHash(InId.ISMComponent));
	}

	UInstancedStaticMeshComponent* ISMComponent = nullptr;
	uint64 InstanceId = 0;
};

/**
 * Entry within a FSMInstanceElementIdMap, tied to a specific ISM component.
 */
struct FSMInstanceElementIdMapEntry
{
	FSMInstanceElementIdMapEntry(FSMInstanceElementIdMap* InOwner, UInstancedStaticMeshComponent* InComponent);
	~FSMInstanceElementIdMapEntry();

	FSMInstanceElementIdMap* Owner = nullptr;

	UInstancedStaticMeshComponent* Component = nullptr;

#if WITH_EDITORONLY_DATA
	TObjectPtr<USMInstanceElementIdMapTransactor> Transactor = nullptr;
#endif	// WITH_EDITORONLY_DATA

	TMap<int32, uint64> InstanceIndexToIdMap;
	TMap<uint64, int32> InstanceIdToIndexMap;
	uint64 NextInstanceId = 1;
};

/**
 * Mapping between the instance ID used by typed elements and the corresponding instance index on the ISM components.
 * This mapping will be kept up-to-date via the add/remove operations of the ISM components, as well as undo/redo.
 */
class FSMInstanceElementIdMap : public FGCObject
{
public:
	static ENGINE_API FSMInstanceElementIdMap& Get();

	ENGINE_API ~FSMInstanceElementIdMap();

	/**
	 * Delegate called when a static mesh instance has been relocated within the instances array on its component (eg, because another instance was removed or added to the array).
	 * @note This may be called with INDEX_NONE as the PreviousInstanceIndex value if the element had previously been unmapped, but was mapped again via serialization (eg, via an undo/redo).
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnInstanceRemapped, const FSMInstanceElementId& /*SMInstanceElementId*/, int32 /*PreviousInstanceIndex*/, int32 /*InstanceIndex*/);
	FOnInstanceRemapped& OnInstanceRemapped()
	{
		return OnInstanceRemappedDelegate;
	}

	/**
	 * Delegate called when a static mesh instance is about to be removed from the instances array on its component, and the associated ID for a SMInstance element will be unmapped.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInstancePreRemoval, const FSMInstanceElementId& /*SMInstanceElementId*/, int32 /*InstanceIndex*/);
	FOnInstancePreRemoval& OnInstancePreRemoval()
	{
		return OnInstancePreRemovalDelegate;
	}

	/**
	 * Delegate called when a static mesh instance has been removed from the instances array on its component, and the associated ID for a SMInstance element has been unmapped.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInstanceRemoved, const FSMInstanceElementId& /*SMInstanceElementId*/, int32 /*InstanceIndex*/);
	FOnInstanceRemoved& OnInstanceRemoved()
	{
		return OnInstanceRemovedDelegate;
	}

	/**
	 * Given a FSMInstanceElementId, attempt to convert it into a FSMInstanceId.
	 */
	ENGINE_API FSMInstanceId GetSMInstanceIdFromSMInstanceElementId(const FSMInstanceElementId& InSMInstanceElementId);

	/**
	 * Given a FSMInstanceId, attempt to convert it into a FSMInstanceElementId.
	 */
	ENGINE_API FSMInstanceElementId GetSMInstanceElementIdFromSMInstanceId(const FSMInstanceId& InSMInstanceId, const bool bAllowCreate = true);

	/**
	 * Given an ISM component, get all FSMInstanceElementId values that are currently mapped for it.
	 */
	ENGINE_API TArray<FSMInstanceElementId> GetSMInstanceElementIdsForComponent(UInstancedStaticMeshComponent* InComponent) const;

	/**
	 * Called when one ISM component is replaced with another, and attempts to copy the ID mapping of the old component to the new.
	 */
	ENGINE_API void OnComponentReplaced(UInstancedStaticMeshComponent* InOldComponent, UInstancedStaticMeshComponent* InNewComponent);

public:
	//~ FGCObject interface
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSMInstanceElementIdMap");
	}

public:
	ENGINE_API void SerializeIdMappings(FSMInstanceElementIdMapEntry* InEntry, FArchive& Ar);

private:
	ENGINE_API void OnInstanceIndexUpdated(UInstancedStaticMeshComponent* InComponent, TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates);

	ENGINE_API void ClearInstanceData_NoLock(UInstancedStaticMeshComponent* InComponent, FSMInstanceElementIdMapEntry& InEntry);

#if WITH_EDITOR
	ENGINE_API void OnObjectModified(UObject* InObject);
#endif	// WITH_EDITOR

	ENGINE_API void RegisterCallbacks();

	ENGINE_API void UnregisterCallbacks();

	mutable FCriticalSection ISMComponentsCS;
	TMap<UInstancedStaticMeshComponent*, TSharedPtr<FSMInstanceElementIdMapEntry>> ISMComponents;

	FOnInstanceRemapped OnInstanceRemappedDelegate;

	FOnInstancePreRemoval OnInstancePreRemovalDelegate;
	FOnInstanceRemoved OnInstanceRemovedDelegate;

	FDelegateHandle OnInstanceIndexUpdatedHandle;
#if WITH_EDITOR
	FDelegateHandle OnObjectModifiedHandle;
#endif
};

/**
 * Transient object instance used as a proxy for storing the current data mapping state
 * in the transaction buffer (for retaining the correct mapping through undo/redo).
 */
UCLASS(Transient)
class USMInstanceElementIdMapTransactor : public UObject
{
	GENERATED_BODY()

public:
	USMInstanceElementIdMapTransactor();

#if WITH_EDITORONLY_DATA
	void SetOwnerEntry(FSMInstanceElementIdMapEntry* InOwnerEntry)
	{
		OwnerEntry = InOwnerEntry;
	}

	//~ UObject interface
	virtual void Serialize(FArchive& Ar) override;

private:
	FSMInstanceElementIdMapEntry* OwnerEntry = nullptr;
#endif	// WITH_EDITORONLY_DATA
};
