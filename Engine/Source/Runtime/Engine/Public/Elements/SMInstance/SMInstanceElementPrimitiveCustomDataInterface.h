// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementPrimitiveCustomDataInterface.h"
#include "SMInstanceElementPrimitiveCustomDataInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementPrimitiveCustomDataInterface : public UObject, public ITypedElementPrimitiveCustomDataInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual void SetCustomData(const FTypedElementHandle& InElementHandle, TArrayView<const float> CustomDataFloats,  bool bMarkRenderStateDirty) override;
	ENGINE_API virtual void SetCustomDataValue(const FTypedElementHandle& InElementHandle, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty) override;
};
