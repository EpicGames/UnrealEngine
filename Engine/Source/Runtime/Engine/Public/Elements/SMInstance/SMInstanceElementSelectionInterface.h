// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "SMInstanceElementSelectionInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual bool SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	ENGINE_API virtual bool DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
};
