// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "SMInstanceElementWorldInterface.generated.h"

UCLASS(MinimalAPI)
class USMInstanceElementWorldInterface : public UObject, public ITypedElementWorldInterface
{
	GENERATED_BODY()

public:
	ENGINE_API virtual bool CanEditElement(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual bool IsTemplateElement(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual ULevel* GetOwnerLevel(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual UWorld* GetOwnerWorld(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual bool GetBounds(const FTypedElementHandle& InElementHandle, FBoxSphereBounds& OutBounds) override;
	ENGINE_API virtual bool CanMoveElement(const FTypedElementHandle& InElementHandle, const ETypedElementWorldType InWorldType) override;
	ENGINE_API virtual bool GetWorldTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) override;
	ENGINE_API virtual bool SetWorldTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) override;
	ENGINE_API virtual bool GetRelativeTransform(const FTypedElementHandle& InElementHandle, FTransform& OutTransform) override;
	ENGINE_API virtual bool SetRelativeTransform(const FTypedElementHandle& InElementHandle, const FTransform& InTransform) override;
	ENGINE_API virtual void NotifyMovementStarted(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual void NotifyMovementOngoing(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual void NotifyMovementEnded(const FTypedElementHandle& InElementHandle) override;
	ENGINE_API virtual TArray<FTypedElementHandle> GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction) override;
};
