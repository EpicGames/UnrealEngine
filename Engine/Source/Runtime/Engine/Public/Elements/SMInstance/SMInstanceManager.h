// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Containers/ArrayView.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "Templates/SubclassOf.h"
#include "SMInstanceManager.generated.h"

enum class ETypedElementWorldType : uint8;
class USMInstanceProxyEditingObject;

/**
 * An interface for actors that manage static mesh instances.
 * This exists so that actors that directly manage instances can inject custom logic around the manipulation of the underlying ISM component.
 * @note The static mesh instances given to this API must be valid and belong to this instance manager. The instance manager implementation is free to assert or crash if that contract is broken.
 */
UINTERFACE(MinimalAPI)
class USMInstanceManager : public UInterface
{
	GENERATED_BODY()
};
class ISMInstanceManager
{
	GENERATED_BODY()

public:
	/**
	 * Get the display name of the given static mesh instance.
	 */
	virtual FText GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const
	{
		return FText();
	}

	/**
	 * Get the tooltip of the given static mesh instance.
	 */
	virtual FText GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const
	{
		return FText();
	}

	/**
	 * Can the given static mesh instance be edited?
	 * @return True if it can be edited, false otherwise.
	 */
	virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const = 0;

	/**
	 * Can the given static mesh instance be moved in the world?
	 * @return True if it can be moved, false otherwise.
	 */
	virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const = 0;

	/**
	 * Attempt to get the transform of the given static mesh instance.
	 * @note The transform should be in the local space of the owner ISM component, unless bWorldSpace is set.
	 * @return True if the transform was retrieved, false otherwise.
	 */
	virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const = 0;

	/**
	 * Attempt to set the transform of the given static mesh instance.
	 * @note The transform should be in the local space of the owner ISM component, unless bWorldSpace is set.
	 * @return True if the transform was updated, false otherwise.
	 */
	virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) = 0;

	/**
	 * Notify that the given static mesh instance is about to be moved.
	 * @note This gives the manager a chance to start a move operation, to avoid performing repeated work until the move is finished.
	 */
	virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given static mesh instance is currently being moved.
	 * @note This gives the manager a chance to update a move operation.
	 */
	virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given static mesh instance is done being moved.
	 * @note This gives the manager a chance to end a move operation.
	 */
	virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given static mesh instance selection state has changed.
	 * @note This gives the manager a chance to sync any internal selection state.
	 */
	virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) = 0;

	/**
	 * Enumerate every static mesh instance element within the selection group that the given static mesh instance belongs to (including the given static mesh instance).
	 * A selection group allows disparate static mesh instances to be considered as a single logical unit for selection, and is mostly used when a manager creates multiple static mesh instances for a single managed instance.
	 * 
	 * If *any* static mesh instance within the group is selected, then *all* static mesh instances within the group are considered selected.
	 * This means that:
	 *	- Querying the selection state for any static mesh element within the group will be consistent, regardless of which static mesh instance is in the selection set.
	 *	- Attempting to select a static mesh instance within the group, when another static mesh instance from the group is already selected, will be a no-op.
	 *	- Attempting to deselect a static mesh instance within the group will deselect all static mesh instances within the group.
	 * 
	 * There is no guarantee of which static mesh instance within the group will actually end up in the selection set, nor which static mesh instance within the group NotifySMInstanceSelectionChanged will be called for.
	 * It is the responsibility of the manager to handle this correctly by treating any static mesh instance within a selection group as one logical unit for all the static mesh instances within the group.
	 */
	virtual void ForEachSMInstanceInSelectionGroup(const FSMInstanceId& InstanceId, TFunctionRef<bool(FSMInstanceId)> Callback)
	{
		Callback(InstanceId);
	}

	/**
	 * Can the given static mesh instance be deleted?
	 * @return True if it can be deleted, false otherwise.
	 */
	virtual bool CanDeleteSMInstance(const FSMInstanceId& InstanceId) const
	{
		return CanEditSMInstance(InstanceId);
	}

	/**
	 * Attempt to delete the given static mesh instances.
	 * @return True if any instances were deleted, false otherwise.
	 */
	virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) = 0;

	/**
	 * Can the given static mesh instance be duplicated?
	 * @return True if it can be duplicated, false otherwise.
	 */
	virtual bool CanDuplicateSMInstance(const FSMInstanceId& InstanceId) const
	{
		return CanEditSMInstance(InstanceId);
	}

	/**
	 * Attempt to duplicate the given static mesh instances, retrieving the IDs of any new instances.
	 * @return True if any instances were duplicated, false otherwise.
	 */
	virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) = 0;

	/**
	* Optionally specify a different class for a proxy object than the default one.
	* This can be useful if a full UObject is needed to represent the SM Instance
	*/
	virtual TSubclassOf<USMInstanceProxyEditingObject> GetSMInstanceEditingProxyClass() const
	{
		return nullptr;
	}
};

UCLASS(MinimalAPI, Abstract)
class USMInstanceProxyEditingObject : public UObject
{
	GENERATED_BODY()

public:
	virtual void Initialize(const FSMInstanceElementId& InSMInstanceElementId) {}
	virtual void Shutdown() {}
private:
	virtual bool SyncProxyStateFromInstance() { return false; }
};

/**
 * An interface for actors that can provide a manager for static mesh instances.
 * This exists so that actors that indirectly manage instances can provide the correct underlying manager for the specific ISM component and instance.
 * @note The static mesh instances given to this API must be valid. The instance manager provider implementation is free to assert or crash if that contract is broken.
 */
UINTERFACE(MinimalAPI)
class USMInstanceManagerProvider : public UInterface
{
	GENERATED_BODY()
};
class ISMInstanceManagerProvider
{
	GENERATED_BODY()

public:
	/**
	 * Attempt to get the instance manager associated with the given static mesh instance, if any.
	 * @return The instance manager, or null if there is no instance manager associated with the given instance.
	 */
	virtual ISMInstanceManager* GetSMInstanceManager(const FSMInstanceId& InstanceId) = 0;
};

/**
 * A static mesh instance manager, tied to a given static mesh instance ID.
 */
struct FSMInstanceManager
{
public:
	FSMInstanceManager() = default;

	FSMInstanceManager(const FSMInstanceId& InInstanceId, ISMInstanceManager* InInstanceManager)
		: InstanceId(InInstanceId)
		, InstanceManager(InInstanceManager)
	{
	}

	explicit operator bool() const
	{
		return InstanceId
			&& InstanceManager;
	}

	bool operator==(const FSMInstanceManager& InRHS) const
	{
		return InstanceId == InRHS.InstanceId
			&& InstanceManager == InRHS.InstanceManager;
	}

	bool operator!=(const FSMInstanceManager& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend inline uint32 GetTypeHash(const FSMInstanceManager& InId)
	{
		return GetTypeHash(InId.InstanceId);
	}

	const FSMInstanceId& GetInstanceId() const { return InstanceId; }
	ISMInstanceManager* GetInstanceManager() const { return InstanceManager; }

	UInstancedStaticMeshComponent* GetISMComponent() const { return InstanceId.ISMComponent; }
	int32 GetISMInstanceIndex() const { return InstanceId.InstanceIndex; }

	//~ ISMInstanceManager interface
	FText GetSMInstanceDisplayName() const { return InstanceManager->GetSMInstanceDisplayName(InstanceId); }
	FText GetSMInstanceTooltip() const { return InstanceManager->GetSMInstanceTooltip(InstanceId); }
	bool CanEditSMInstance() const { return InstanceManager->CanEditSMInstance(InstanceId); }
	bool CanMoveSMInstance(const ETypedElementWorldType WorldType) const { return InstanceManager->CanMoveSMInstance(InstanceId, WorldType); }
	bool GetSMInstanceTransform(FTransform& OutInstanceTransform, bool bWorldSpace = false) const { return InstanceManager->GetSMInstanceTransform(InstanceId, OutInstanceTransform, bWorldSpace); }
	bool SetSMInstanceTransform(const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) const { return InstanceManager->SetSMInstanceTransform(InstanceId, InstanceTransform, bWorldSpace, bMarkRenderStateDirty, bTeleport); }
	void NotifySMInstanceMovementStarted() const { return InstanceManager->NotifySMInstanceMovementStarted(InstanceId); }
	void NotifySMInstanceMovementOngoing() const { return InstanceManager->NotifySMInstanceMovementOngoing(InstanceId); }
	void NotifySMInstanceMovementEnded() const { return InstanceManager->NotifySMInstanceMovementEnded(InstanceId); }
	void NotifySMInstanceSelectionChanged(const bool bIsSelected) const { return InstanceManager->NotifySMInstanceSelectionChanged(InstanceId, bIsSelected); }
	void ForEachSMInstanceInSelectionGroup(TFunctionRef<bool(FSMInstanceId)> Callback) const { return InstanceManager->ForEachSMInstanceInSelectionGroup(InstanceId, Callback); }
	bool CanDeleteSMInstance() const { return InstanceManager->CanDeleteSMInstance(InstanceId); }
	bool DeleteSMInstance() const { return InstanceManager->DeleteSMInstances(MakeArrayView(&InstanceId, 1)); }
	bool CanDuplicateSMInstance() const { return InstanceManager->CanDuplicateSMInstance(InstanceId); }
	bool DuplicateSMInstance(FSMInstanceId& OutNewInstanceId) const
	{
		TArray<FSMInstanceId> NewInstanceIds;
		if (InstanceManager->DuplicateSMInstances(MakeArrayView(&InstanceId, 1), NewInstanceIds))
		{
			OutNewInstanceId = NewInstanceIds[0];
			return true;
		}
		return false;
	}

private:
	FSMInstanceId InstanceId;
	ISMInstanceManager* InstanceManager = nullptr;
};
