// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Engine.h: Unreal engine public header file.
=============================================================================*/

#pragma once

// IWYU pragma: begin_keep

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "Core.h"
#include "CoreUObject.h"
#include "InputCore.h"
#include "EngineDefines.h"
#include "EngineSettings.h"
#include "EngineStats.h"
#include "EngineLogs.h"
#include "EngineGlobals.h"

/*-----------------------------------------------------------------------------
	Engine public includes.
-----------------------------------------------------------------------------*/

#include "ComponentInstanceDataCache.h"
#include "SceneTypes.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"
#include "PrecomputedLightVolume.h"
#include "PixelFormat.h"
#include "Components.h"
#include "GPUSkinPublicDefs.h"
#include "ShowFlags.h"
#include "HitProxies.h"
#include "UnrealClient.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "ConvexVolume.h"
#include "BlendableManager.h"
#include "FinalPostProcessSettings.h"
#include "SceneInterface.h"
#include "DebugViewModeHelpers.h"
#include "SceneView.h"
#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "BoneIndices.h"
#include "ReferenceSkeleton.h"
#include "AnimInterpFilter.h"
#include "Animation/AnimTypes.h"
#include "CustomBoneIndexArray.h"
#include "BoneContainer.h"
#include "PhysxUserData.h"
#include "RawIndexBuffer.h"
#include "Animation/AnimCurveTypes.h"
#include "ClothSimData.h"
#include "SingleAnimationPlayData.h"
#include "Animation/PoseSnapshot.h"
#include "GraphEditAction.h"
#include "BlueprintUtilities.h"
#include "IAudioExtensionPlugin.h"
#include "Audio.h"
#include "AI/RVOAvoidanceInterface.h"
#include "AI/NavDataGenerator.h"
#include "ParticleVertexFactory.h"
#include "TextureResource.h"
#include "StaticParameterSet.h"
#include "MaterialShared.h"
#include "StaticBoundShaderState.h"
#include "BatchedElements.h"
#include "MeshBatch.h"
#include "SceneUtils.h"
#include "MeshParticleVertexFactory.h"
#include "Distributions.h"
#include "ParticleEmitterInstances.h"
#include "Scalability.h"
#include "MaterialExpressionIO.h"
#include "EngineMinimal.h"

#include "Engine/EngineBaseTypes.h"
#include "Engine/DeveloperSettings.h"
#include "Camera/CameraTypes.h"
#include "Engine/EngineTypes.h"
#include "Sound/AmbientSound.h"
#include "LocalVertexFactory.h"
#include "Model.h"
#include "Engine/Brush.h"
#include "GameFramework/Volume.h"
#include "Engine/BlockingVolume.h"
#include "GameFramework/CameraBlockingVolume.h"
#include "Engine/CullDistanceVolume.h"
#include "Engine/LevelStreamingVolume.h"
#include "GameFramework/PhysicsVolume.h"
#include "GameFramework/DefaultPhysicsVolume.h"
#include "GameFramework/KillZVolume.h"
#include "GameFramework/PainCausingVolume.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "Engine/PostProcessVolume.h"
#include "Sound/AudioVolume.h"
#include "Engine/TriggerVolume.h"
#include "Camera/CameraActor.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameFramework/PlayerMuteList.h"
#include "Camera/PlayerCameraManager.h"
#include "GameFramework/Info.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/DebugCameraController.h"
#include "Engine/DecalActor.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "GameFramework/HUD.h"
#include "Engine/ExponentialHeightFog.h"
#include "GameFramework/GameSession.h"
#include "GameFramework/GameState.h"
#include "GameFramework/GameMode.h"
#include "GameFramework/PlayerState.h"
#include "Engine/SkyLight.h"
#include "GameFramework/WorldSettings.h"
#include "Components/LightComponentBase.h"
#include "Tickable.h"
#include "Engine/LevelBounds.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/Light.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/SpotLight.h"
#include "Engine/GeneratedMeshAreaLight.h"
#include "Engine/NavigationObjectBase.h"
#include "GameFramework/PlayerStart.h"
#include "Engine/PlayerStartPIE.h"
#include "Engine/Note.h"
#include "GameFramework/MovementComponent.h"
#include "GameFramework/NavMovementComponent.h"
#include "GameFramework/PawnMovementComponent.h"
#include "Interfaces/NetworkPredictionInterface.h"
#include "GameFramework/FloatingPawnMovement.h"
#include "GameFramework/DefaultPawn.h"
#include "GameFramework/SpectatorPawn.h"
#include "Engine/ReflectionCapture.h"
#include "Engine/BoxReflectionCapture.h"
#include "Engine/PlaneReflectionCapture.h"
#include "Engine/SphereReflectionCapture.h"
#include "PhysicsEngine/RigidBodyBase.h"
#include "PhysicsEngine/PhysicsConstraintActor.h"
#include "PhysicsEngine/PhysicsThruster.h"
#include "PhysicsEngine/RadialForceActor.h"
#include "Engine/SceneCapture.h"
#include "Engine/SceneCapture2D.h"
#include "Engine/SceneCaptureCube.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TargetPoint.h"
#include "VectorField/VectorFieldVolume.h"
#include "Engine/DataAsset.h"
#include "Engine/TextureStreamingTypes.h"
#include "GameFramework/SpectatorPawnMovement.h"
#include "GameFramework/ProjectileMovementComponent.h"
#include "GameFramework/RotatingMovementComponent.h"
#include "Components/PawnNoiseEmitterComponent.h"
#include "PhysicsEngine/PhysicsHandleComponent.h"
#include "Sound/SoundAttenuation.h"
#include "Components/ChildActorComponent.h"
#include "Components/DecalComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/PostProcessComponent.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DrawFrustumComponent.h"
#include "Debug/DebugDrawService.h"
#include "Components/LineBatchComponent.h"
#include "Components/MaterialBillboardComponent.h"
#include "Lightmass/LightmassPrimitiveSettingsObject.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Components/ModelComponent.h"
#include "Components/ShapeComponent.h"
#include "Components/DrawSphereComponent.h"
#include "Components/TextRenderComponent.h"
#include "Components/VectorFieldComponent.h"
#include "PhysicsEngine/RadialForceComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Components/BoxReflectionCaptureComponent.h"
#include "Components/PlaneReflectionCaptureComponent.h"
#include "Components/SphereReflectionCaptureComponent.h"
#include "Components/SceneCaptureComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Components/SceneCaptureComponentCube.h"
#include "Components/TimelineComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/AssetUserData.h"
#include "Engine/BlendableInterface.h"
#include "Engine/BlueprintCore.h"
#include "Engine/Blueprint.h"
#include "Animation/AnimBlueprint.h"
#include "Sound/DialogueTypes.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Animation/AnimSequenceBase.h"
#include "AlphaBlend.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Camera/CameraModifier.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraModifier_CameraShake.h"
#include "GameFramework/CheatManager.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Engine/CurveTable.h"
#include "GameFramework/DamageType.h"
#include "DataTableUtils.h"
#include "Engine/DataTable.h"
#include "Sound/DialogueVoice.h"
#include "Sound/DialogueWave.h"
#include "Distributions/Distribution.h"
#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionFloatConstant.h"
#include "Distributions/DistributionFloatParameterBase.h"
#include "Distributions/DistributionFloatParticleParameter.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionFloatUniform.h"
#include "Distributions/DistributionFloatUniformCurve.h"
#include "Distributions/DistributionVector.h"
#include "Distributions/DistributionVectorConstant.h"
#include "Distributions/DistributionVectorParameterBase.h"
#include "Distributions/DistributionVectorParticleParameter.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "Distributions/DistributionVectorUniform.h"
#include "Distributions/DistributionVectorUniformCurve.h"
#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "Exporters/Exporter.h"
#include "Engine/FontImportOptions.h"
#include "Engine/Font.h"
#include "GameFramework/GameUserSettings.h"
#include "GestureRecognizer.h"
#include "KeyState.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InterpCurveEdSetup.h"
#include "Engine/IntSerialization.h"
#include "Layers/Layer.h"
#include "Engine/Level.h"
#include "LatentActions.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/LevelStreamingPersistent.h"
#include "Lightmass/LightmappedSurfaceCollection.h"
#include "GameFramework/LocalMessage.h"
#include "GameFramework/EngineMessage.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "Engine/NetDriver.h"
#include "Engine/NetworkSettings.h"
#include "Engine/ObjectLibrary.h"
#include "Engine/ObjectReferencer.h"
#include "GameFramework/OnlineSession.h"
#include "Net/DataBunch.h"
#include "Engine/PackageMapClient.h"
#include "Engine/PendingNetGame.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicalMaterials/PhysicalMaterialPropertyBase.h"
#include "PhysicsEngine/PhysicsCollisionHandler.h"
#include "Engine/PlatformInterfaceBase.h"
#include "Engine/CloudStorageBase.h"
#include "Engine/InGameAdManager.h"
#include "Engine/MicroTransactionBase.h"
#include "Engine/TwitterIntegrationBase.h"
#include "Engine/PlatformInterfaceWebResponse.h"
#include "Engine/Player.h"
#include "Engine/LocalPlayer.h"
#include "Engine/NetConnection.h"
#include "Engine/ChildConnection.h"
#include "Engine/Polys.h"
#include "Sound/ReverbEffect.h"
#include "GameFramework/SaveGame.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Animation/Skeleton.h"
#include "Sound/DialogueSoundWaveProxy.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundGroups.h"
#include "Sound/SoundMix.h"
#include "Engine/StaticMeshSocket.h"
#include "Camera/CameraStackTypes.h"
#include "Engine/StreamableManager.h"
#include "Engine/TextureDefines.h"
#include "Tests/TextPropertyTestObject.h"
#include "Engine/TextureLODSettings.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/Texture.h"
#include "Engine/LightMapTexture2D.h"
#include "Engine/ShadowMapTexture2D.h"
#include "Engine/TextureLightProfile.h"
#include "Engine/Texture2DDynamic.h"
#include "Engine/TextureCube.h"
#include "Engine/TextureRenderTarget.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "Engine/TimelineTemplate.h"
#include "GameFramework/TouchInterface.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Animation/MorphTarget.h"
#include "Animation/AnimInstance.h"

#include "SystemSettings.h"
#include "SceneManagement.h"

#include "DrawDebugHelpers.h"
#include "UnrealEngine.h"
#include "CanvasTypes.h"
#include "EngineUtils.h"
#include "TimerManager.h"
#include "SlateCore.h"
#include "SlateBasics.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "ParticleHelper.h"
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Engine/Selection.h"
#endif

// IWYU pragma: end_keep
