// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.3, "#include UObject/ICookInfo.h instead of Engine/ICookInfo.h.")
