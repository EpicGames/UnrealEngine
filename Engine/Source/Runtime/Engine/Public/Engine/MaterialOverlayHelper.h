// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

class FMaterialOverlayHelper
{
public:
	static void AppendAllOverlayMaterial(const TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, TArray<UMaterialInterface*>& OutMaterials, bool& bOutHaveNullEntry)
	{
		bOutHaveNullEntry = false;
		for(const TObjectPtr<UMaterialInterface>& SourceMaterial : SourceMaterials)
		{
			if (SourceMaterial)
			{
				OutMaterials.Add(SourceMaterial);
			}
			else
			{
				bOutHaveNullEntry = true;
			}
		}
	}

	static void AppendAllOverlayMaterial(const TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, TArray<UMaterialInterface*>& OutMaterials)
	{
		bool bHaveNullEntry = false;
		AppendAllOverlayMaterial(SourceMaterials, OutMaterials, bHaveNullEntry);
	}

	static void ForceMaterial(TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, UMaterialInterface* ForceMaterial)
	{
		for (int32 SlotIndex = 0; SlotIndex < SourceMaterials.Num(); ++SlotIndex)
		{
			SourceMaterials[SlotIndex] = ForceMaterial;
		}
	}

	static bool ForceMaterial(TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, const int32 SlotIndex, UMaterialInterface* ForceMaterial)
	{
		if (SourceMaterials.IsValidIndex(SlotIndex))
		{
			SourceMaterials[SlotIndex] = ForceMaterial;
			return true;
		}
		return false;
	}

	static UMaterialInterface* GetOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, const int32 SlotIndex)
	{
		return SourceMaterials.IsValidIndex(SlotIndex) ? SourceMaterials[SlotIndex] : nullptr;
	}
};
