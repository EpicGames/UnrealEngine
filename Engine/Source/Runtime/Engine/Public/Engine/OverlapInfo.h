// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/HitResult.h"
#include "OverlapInfo.generated.h"

class UPrimitiveComponent;

/** Overlap info consisting of the primitive and the body that is overlapping */
USTRUCT()
struct FOverlapInfo
{
	GENERATED_BODY()

	FOverlapInfo()
	{}

	explicit FOverlapInfo(const FHitResult& InSweepResult)
		: bFromSweep(true), OverlapInfo(InSweepResult)
	{
	}

	ENGINE_API explicit FOverlapInfo(UPrimitiveComponent* InComponent, int32 InBodyIndex = INDEX_NONE);

	int32 GetBodyIndex() const { return OverlapInfo.Item; }

	//This function completely ignores SweepResult information. It seems that places that use this function do not care, but it still seems risky
	friend bool operator == (const FOverlapInfo& LHS, const FOverlapInfo& RHS) { return LHS.OverlapInfo.Component == RHS.OverlapInfo.Component && LHS.OverlapInfo.Item == RHS.OverlapInfo.Item; }

	UPROPERTY(Transient)
	bool bFromSweep = false;

	/** Information for both sweep and overlap queries. Different parts are valid depending on bFromSweep.
	  * If bFromSweep is true then FHitResult is completely valid just like a regular sweep result.
	  * If bFromSweep is false only FHitResult::Component, FHitResult::Actor, FHitResult::Item are valid as this is really just an FOverlapResult*/
	UPROPERTY(Transient)
	FHitResult OverlapInfo;
};

/*
 * Predicate for comparing FOverlapInfos when exact weak object pointer index/serial numbers should match, assuming one is not null and not invalid.
 * Compare to operator== for WeakObjectPtr which does both HasSameIndexAndSerialNumber *and* IsValid() checks on both pointers.
 */
struct FFastOverlapInfoCompare
{
	FFastOverlapInfoCompare(const FOverlapInfo& BaseInfo)
		: MyBaseInfo(BaseInfo)
	{
	}

	bool operator() (const FOverlapInfo& Info)
	{
		return MyBaseInfo.OverlapInfo.Component.HasSameIndexAndSerialNumber(Info.OverlapInfo.Component)
			&& MyBaseInfo.GetBodyIndex() == Info.GetBodyIndex();
	}

	bool operator() (const FOverlapInfo* Info)
	{
		return MyBaseInfo.OverlapInfo.Component.HasSameIndexAndSerialNumber(Info->OverlapInfo.Component)
			&& MyBaseInfo.GetBodyIndex() == Info->GetBodyIndex();
	}

private:
	const FOverlapInfo& MyBaseInfo;

};

// All added members of FOverlapInfo are PODs.
template<> struct TIsPODType<FOverlapInfo> { enum { Value = TIsPODType<FHitResult>::Value }; };

typedef TArray<FOverlapInfo, TInlineAllocator<3>> TInlineOverlapInfoArray;
typedef TArrayView<const FOverlapInfo> TOverlapArrayView;
