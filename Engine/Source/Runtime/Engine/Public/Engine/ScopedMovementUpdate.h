// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/OverlapInfo.h"

struct FHitResult;
enum class ETeleportType : uint8;

/**
 * Enum that controls the scoping behavior of FScopedMovementUpdate.
 * Note that EScopedUpdate::ImmediateUpdates is not allowed within outer scopes that defer updates,
 * and any attempt to do so will change the new inner scope to use deferred updates instead.
 */
namespace EScopedUpdate
{
	enum Type
	{
		/** Apply changes immediately, this cannot be nested inside deferred updates */
		ImmediateUpdates,

		/** Apply changes when the last FScopedMovementUpdate on the stack ends. These can be safely nested */
		DeferredUpdates,

		/**
		 * Experimental.
		 * 
		 * Defer the expensive aspects of updating a component's transform to later on the
		 * frame instead of happening immediately upon the end of this movement scope.
		 *
		 * This includes:
		 * - The call to PropagateTransformUpdate, which is where actual physics BodyInstance will be updated with
		 *   the component's transform information.
		 * - The updating overlaps for this component that may have occurred as a result of this component moving.
		 * - Dispatching of any potential FHitResults (collision responses) that may have occurred as a result of
		 *   this component moving.
		 *
		 * @see USceneComponent::ProcessDeferredMovementGroup
		 */
		DeferredGroupUpdates,
	};
}

/**
 * FScopedMovementUpdate creates a new movement scope, within which propagation of moves may be deferred until the end of the outermost scope that does not defer updates.
 * Moves within this scope will avoid updates such as UpdateBounds(), OnUpdateTransform(), UpdatePhysicsVolume(), UpdateChildTransforms() etc
 * until the move is committed (which happens when the last deferred scope goes out of context).
 *
 * Note that non-deferred scopes are not allowed within outer scopes that defer updates, and any attempt to use one will change the inner scope to use deferred updates.
 */
class FScopedMovementUpdate : private FNoncopyable
{
public:
	typedef TArray<FHitResult, TInlineAllocator<2>> TScopedBlockingHitArray;
	typedef TArray<FOverlapInfo, TInlineAllocator<3>> TScopedOverlapInfoArray;

	ENGINE_API FScopedMovementUpdate( USceneComponent* Component, EScopedUpdate::Type ScopeBehavior = EScopedUpdate::DeferredUpdates, bool bRequireOverlapsEventFlagToQueueOverlaps = true );
	ENGINE_API ~FScopedMovementUpdate();

	enum class EHasMovedTransformOption
	{
		eTestTransform,
		eIgnoreTransform
	};

	enum class EOverlapState
	{
		eUseParent,
		eUnknown,
		eIncludesOverlaps,
		eForceUpdate,
	};

	/** Get the scope containing this scope. A scope only has an outer scope if they both defer updates. */
	const FScopedMovementUpdate* GetOuterDeferredScope() const;

	/** Return true if deferring updates, false if updates are applied immediately. */
	bool IsDeferringUpdates() const;

	/** Return true if this movement update should be deferred and applied later in the frame as part of a larger group of components. */
	bool IsGroupUpdate() const;
	
	/** Revert movement to the initial location of the Component at the start of the scoped update. Also clears pending overlaps and sets bHasMoved to false. */
	ENGINE_API void RevertMove();

	/** Returns whether movement has occurred at all during this scope, optionally checking if the transform is different (since changing scale does not go through a move). RevertMove() sets this back to false. */
	bool HasMoved(EHasMovedTransformOption CheckTransform) const;

	/** Returns true if the Component's transform differs from that at the start of the scoped update. */
	ENGINE_API bool IsTransformDirty() const;

	/** Returns true if there are pending overlaps queued in this scope. */
	bool HasPendingOverlaps() const;

	/**
	* Returns true if we require GetGenerateOverlapEvents() on both the moving object and the overlapped object to add them to the pending overlaps list.
	* These flags will still be required when dispatching calls to UpdateOverlaps(), but this allows some custom processing of queued overlaps that would be otherwise missed along the way.
	*/
	bool RequiresOverlapsEventFlag() const;

	/** Returns the pending overlaps within this scope. */
	const TScopedOverlapInfoArray& GetPendingOverlaps() const;

	/** Returns the list of pending blocking hits, which will be used for notifications once the move is committed. */
	const TScopedBlockingHitArray& GetPendingBlockingHits() const;

	//--------------------------------------------------------------------------------------------------------//
	// These methods are intended only to be used by SceneComponent and derived classes.

	/** Add overlaps to the queued overlaps array. This is intended for use only by SceneComponent and its derived classes whenever movement is performed. */
	ENGINE_API void AppendOverlapsAfterMove(const TOverlapArrayView& NewPendingOverlaps, bool bSweep, bool bIncludesOverlapsAtEnd);

	/** Keep current pending overlaps after a move but make note that there was movement (just a symmetric rotation). */
	void KeepCurrentOverlapsAfterRotation(bool bSweep);

	/** Add blocking hit that will get processed once the move is committed. This is intended for use only by SceneComponent and its derived classes. */
	void AppendBlockingHitAfterMove(const FHitResult& Hit);

	/** Clear overlap state at current location, we don't know what it is. */
	void InvalidateCurrentOverlaps();

	/** Force full overlap update once this scope finishes. */
	void ForceOverlapUpdate();

	/** Registers that this move is a teleport */
	void SetHasTeleported(ETeleportType InTeleportType);

	/** The type of teleport which this scoped move should use to determine overlaps. */
	ETeleportType GetTeleportType() const;
	
	/** True if this scoped movement resulted in the component being moved */
	bool GetHasMoved() const;
	
	/** 
	* If not INDEX_NONE, overlaps at this index and beyond in PendingOverlaps are at the final destination
	*/
	int32 GetFinalOverlapCandidatesIndex() const;
	
	/** The current overlap state of this scoped move. */
	EOverlapState GetOverlapState() const;
	
	/** The initial transform when this scoped movement began. */
	const FTransform& GetInitialTransform() const;
	
	/** The owning scene component this scoped movement is modifying. */
	USceneComponent* GetOwner() const;

protected:
	/** Fills in the list of overlaps at the end location (in EndOverlaps). Returns pointer to the list, or null if it can't be computed. */
	ENGINE_API TOptional<TOverlapArrayView> GetOverlapsAtEnd(class UPrimitiveComponent& PrimComponent, TInlineOverlapInfoArray& OutEndOverlaps, bool bTransformChanged) const;

	ENGINE_API bool SetWorldLocationAndRotation(FVector NewLocation, const FQuat& NewQuat, bool bNoPhysics, ETeleportType Teleport);

private:
	/** Notify this scope that the given inner scope completed its update (ie is going out of scope). Only occurs for deferred updates. */
	ENGINE_API void OnInnerScopeComplete(const FScopedMovementUpdate& InnerScope);
	
	// This class can only be created on the stack, otherwise the ordering constraints
	// of the constructor and destructor between encapsulated scopes could be violated.
	void* operator new (size_t) = delete;
	void* operator new[] (size_t) = delete;
	void operator delete (void *) = delete;
	void operator delete[] (void*) = delete;

protected:
	USceneComponent* Owner;
	FScopedMovementUpdate* OuterDeferredScope;

	EOverlapState CurrentOverlapState;
	ETeleportType TeleportType;

	FTransform InitialTransform;
	FVector InitialRelativeLocation;
	FRotator InitialRelativeRotation;
	FVector InitialRelativeScale;

	int32 FinalOverlapCandidatesIndex;		// If not INDEX_NONE, overlaps at this index and beyond in PendingOverlaps are at the final destination
	TScopedOverlapInfoArray PendingOverlaps;	// All overlaps encountered during the scope of moves.
	TScopedBlockingHitArray BlockingHits;		// All blocking hits encountered during the scope of moves.

	uint8 bDeferUpdates:1;
	uint8 bHasMoved:1;
	uint8 bRequireOverlapsEventFlag:1;
	
	/**
	* True when DeferredGroupUpdates is the type applied. See notes above.
	*/
	uint8 bIsGroupDeferedUpdate : 1;

	friend class USceneComponent;
};

//////////////////////////////////////////////////////////////////////////
// FScopedMovementUpdate inlines

inline const FScopedMovementUpdate* FScopedMovementUpdate::GetOuterDeferredScope() const
{
	return OuterDeferredScope;
}

inline bool FScopedMovementUpdate::IsDeferringUpdates() const
{
	return bDeferUpdates;
}

inline bool FScopedMovementUpdate::IsGroupUpdate() const
{
	return bIsGroupDeferedUpdate;
}

inline bool FScopedMovementUpdate::HasMoved(EHasMovedTransformOption CheckTransform) const
{
	return bHasMoved || (CheckTransform == EHasMovedTransformOption::eTestTransform && IsTransformDirty());
}

inline bool FScopedMovementUpdate::HasPendingOverlaps() const
{
	return PendingOverlaps.Num() > 0;
}

inline bool FScopedMovementUpdate::RequiresOverlapsEventFlag() const
{
	return bRequireOverlapsEventFlag;
}

inline const FScopedMovementUpdate::TScopedOverlapInfoArray& FScopedMovementUpdate::GetPendingOverlaps() const
{
	return PendingOverlaps;
}

inline const FScopedMovementUpdate::TScopedBlockingHitArray& FScopedMovementUpdate::GetPendingBlockingHits() const
{
	return BlockingHits;
}

inline void FScopedMovementUpdate::AppendBlockingHitAfterMove(const FHitResult& Hit)
{
	BlockingHits.Add(Hit);
}

inline void FScopedMovementUpdate::KeepCurrentOverlapsAfterRotation(bool bSweep)
{
	bHasMoved = true;
	// CurrentOverlapState is unchanged
}

inline void FScopedMovementUpdate::InvalidateCurrentOverlaps()
{
	bHasMoved = true;
	CurrentOverlapState = EOverlapState::eUnknown;
	FinalOverlapCandidatesIndex = INDEX_NONE;
}

inline void FScopedMovementUpdate::ForceOverlapUpdate()
{
	bHasMoved = true;
	CurrentOverlapState = EOverlapState::eForceUpdate;
	FinalOverlapCandidatesIndex = INDEX_NONE;
}

inline void FScopedMovementUpdate::SetHasTeleported(ETeleportType InTeleportType)
{
	// Request an initialization. Teleport type can only go higher - i.e. if we have requested a reset, then a teleport will still reset fully
	TeleportType = ((InTeleportType > TeleportType) ? InTeleportType : TeleportType); 
}

inline ETeleportType FScopedMovementUpdate::GetTeleportType() const
{
	return TeleportType;
}

inline bool FScopedMovementUpdate::GetHasMoved() const
{
	return bHasMoved;
}

inline int32 FScopedMovementUpdate::GetFinalOverlapCandidatesIndex() const
{
	return FinalOverlapCandidatesIndex;
}

inline FScopedMovementUpdate::EOverlapState FScopedMovementUpdate::GetOverlapState() const
{
	return CurrentOverlapState;
}

inline const FTransform& FScopedMovementUpdate::GetInitialTransform() const
{
	return InitialTransform;
}

inline USceneComponent* FScopedMovementUpdate::GetOwner() const
{
	return Owner;
}
