// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "IAnalyticsProviderET.h" // NOTE: Consider changing the code to replace IAnalyticProvider.h by IAnalyticProviderET.h

class FEngineSessionManager;
class FAnalyticsSessionSummaryManager;
class IAnalyticsProvider;
class IAnalyticsProviderET;
struct FAnalyticsEventAttribute;

/**
 * The public interface for the editor's analytics provider singleton.
 * 
 * WARNING: This is an analytics provider instance that is created whenever UE editor is launched. 
 * It is intended ONLY for use by Epic Games. This is NOT intended for games to send 
 * game-specific telemetry. Create your own provider instance for your game and configure
 * it independently.
 *
 * It is called FEngineAnalytics for legacy reasons, and is only used for editor telemetry.
 */
class FEngineAnalytics : FNoncopyable
{
public:
	/**
	 * Return the provider instance. Not valid outside of Initialize/Shutdown calls.
	 * Note: must check IsAvailable() first else this code will assert if the provider is not valid.
	 */
	static ENGINE_API IAnalyticsProviderET& GetProvider();

#if WITH_EDITOR
	/** This is intended ONLY for use by Epic Games so that plugins can append to the session summary. */
	static ENGINE_API FAnalyticsSessionSummaryManager& GetSummaryManager();
#endif

	/** Helper function to determine if the provider is valid. */
	static bool IsAvailable() { return Analytics.IsValid(); }

	/** Called to initialize the singleton. */
	static ENGINE_API void Initialize();
#if WITH_EDITOR
	/** A hook for analytics initialization after the SummaryManager is initialized */
	static ENGINE_API FSimpleMulticastDelegate OnInitializeEngineAnalytics;
#endif

	/** Called to shut down the singleton */
	static ENGINE_API void Shutdown(bool bIsEngineShutdown = false);
#if WITH_EDITOR
	/** A hook for analytics shutdown before the SummaryManager is shutdown */
	static ENGINE_API FSimpleMulticastDelegate OnShutdownEngineAnalytics;
#endif

	static ENGINE_API void Tick(float DeltaTime);

	static ENGINE_API void LowDriveSpaceDetected();

	static ENGINE_API FString CreateAnalyticsUserId(const FString& EpicAccountId);

private:
	static void AppendMachineStats(TArray<FAnalyticsEventAttribute>& EventAttributes);
	static void SendMachineInfoForAccount(const FString& EpicAccountId);
	static void OnEpicAccountIdChanged(const FString& EpicAccountId);

	static bool bIsInitialized;
	static ENGINE_API TSharedPtr<IAnalyticsProviderET> Analytics;
	static ENGINE_API TSet<FString> SessionEpicAccountIds;
};

namespace UE::Analytics::Private {

DECLARE_DELEGATE_OneParam(FOnEpicAccountIdChanged, const FString&);

/**
  * Interface for allowing changes to engine analytics configuration intended for use by internal tools.
  * 
  * Internal use only, not intended for licensee use.
  */
class IEngineAnalyticsConfigOverride
{
public:
	virtual ~IEngineAnalyticsConfigOverride() = default;
	virtual void ApplyConfiguration(FAnalyticsET::Config& Config) = 0;
	virtual void OnInitialized(IAnalyticsProviderET& provider, const FOnEpicAccountIdChanged& OnEpicAccountIdChanged) = 0;
};

extern ENGINE_API IEngineAnalyticsConfigOverride* EngineAnalyticsConfigOverride;

/* UE::Analytics::Private */ }
