// Copyright Epic Games, Inc. All Rights Reserved.
// 
// Public defines form the Engine

#pragma once

#include "HAL/Platform.h"

/*-----------------------------------------------------------------------------
	Configuration defines
-----------------------------------------------------------------------------*/

#ifndef UE_ENABLE_DEBUG_DRAWING
	// we can debug render data in Debug/Development builds, or the Editor in Shipping or Test builds (not a common situation, but it is possible)
	#define UE_ENABLE_DEBUG_DRAWING (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
#endif

#ifndef ENABLE_VISUAL_LOG
	#define ENABLE_VISUAL_LOG (PLATFORM_DESKTOP && !NO_LOGGING && UE_ENABLE_DEBUG_DRAWING)
#endif

// Whether lightmass generates FSHVector2 or FSHVector3. Linked with VER_UE4_INDIRECT_LIGHTING_SH3
#define NUM_INDIRECT_LIGHTING_SH_COEFFICIENTS 9

// The number of lights to consider for sky/atmospheric light scattering
#define NUM_ATMOSPHERE_LIGHTS 2

// Enables StateStream for actor/component system
// Set in .target.cs to enable
#ifndef WITH_STATE_STREAM_ACTOR
	#define WITH_STATE_STREAM_ACTOR 0
#endif

/*-----------------------------------------------------------------------------
	Size of the world.
-----------------------------------------------------------------------------*/
#define UE_OLD_WORLD_MAX			2097152.0						/* UE4 maximum world size */
#define UE_OLD_HALF_WORLD_MAX		(UE_OLD_WORLD_MAX * 0.5)		/* UE4 half maximum world size */
#define UE_OLD_HALF_WORLD_MAX1		(UE_OLD_HALF_WORLD_MAX - 1)		/* UE4 half maximum world size minus one */

#define UE_LARGE_WORLD_MAX			8796093022208.0					/* LWC maximum world size, Approx 87,960,930.2 km across, or 43,980,465.1 km from the origin */
#define UE_LARGE_HALF_WORLD_MAX		(UE_LARGE_WORLD_MAX * 0.5)		/* LWC half maximum world size */
#define UE_LARGE_HALF_WORLD_MAX1	(UE_LARGE_HALF_WORLD_MAX - 1)	/* LWC half maximum world size minus one */

#ifndef UE_USE_UE4_WORLD_MAX
#define UE_USE_UE4_WORLD_MAX		0						// Force UE4 WORLD_MAX for converted UE4 titles that explicitly rely on it.
#endif

// Note: Modifying WORLD_MAX affects UE_LWC_RENDER_TILE_SIZE in Engine\Source\Runtime\Core\Private\Misc\LargeWorldRenderPosition.cpp and may introduce precision issues in shaders using world coordinates.
#if UE_USE_UE4_WORLD_MAX
#define WORLD_MAX					(UE_OLD_WORLD_MAX)
#else
#define WORLD_MAX					(UE_LARGE_WORLD_MAX)
#endif

#define HALF_WORLD_MAX				(WORLD_MAX * 0.5)		/* Half the maximum size of the world */
#define HALF_WORLD_MAX1				(HALF_WORLD_MAX - 1.0)	/* Half the maximum size of the world minus one */

#define UE_FLOAT_HUGE_DISTANCE		1048576.0				/* Maximum distance representable by a float whilst maintaining precision of at least 0.0625 units (1/16th of a cm) - Precision issues may occur for positions/distances represented by float types that exceed this value */
#define UE_DOUBLE_HUGE_DISTANCE		562949953421312.0		/* Maximum distance representable by a double whilst maintaining precision of at least 0.0625 units (1/16th of a cm) - Precision issues may occur for positions/distances represented by double types that exceed this value */

#define MIN_ORTHOZOOM				1.0							/* 2D ortho viewport zoom >= MIN_ORTHOZOOM */
#define MAX_ORTHOZOOM				1e25						/* 2D ortho viewport zoom <= MAX_ORTHOZOOM */
#define DEFAULT_ORTHOZOOM			10000.0						/* Default 2D ortho viewport zoom */
#define DEFAULT_ORTHOWIDTH			1536.0f						/* Default 2D ortho viewport width */
#define DEFAULT_ORTHONEARPLANE		-DEFAULT_ORTHOWIDTH/2.0f	/* Default 2D ortho viewport nearplane */
#define DEFAULT_ORTHOFARPLANE		UE_OLD_WORLD_MAX			/* Default 2D ortho viewport farplane */

/** bits needed to store DPG value */
#define SDPG_NumBits 3
