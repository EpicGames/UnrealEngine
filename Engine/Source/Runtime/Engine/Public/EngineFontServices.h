// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/SlateRenderer.h"

class FSlateFontCache;
class FSlateFontMeasure;

/** 
 * A shim around FSlateFontServices that provides access from the render thread (where FSlateApplication::Get() would assert)
 */
class FEngineFontServices
{
public:
	/** Create the singular instance of this class - must be called from the game thread */
	static ENGINE_API void Create();

	/** Destroy the singular instance of this class - must be called from the game thread */
	static ENGINE_API void Destroy();

	/** Check to see if the singular instance of this class is currently initialized and ready */
	static ENGINE_API bool IsInitialized();

	/** Get the singular instance of this class */
	static ENGINE_API FEngineFontServices& Get();

	/** Get the font cache to use for the current thread */
	ENGINE_API TSharedPtr<FSlateFontCache> GetFontCache();

	/** Get the font measure to use for the current thread */
	ENGINE_API TSharedPtr<FSlateFontMeasure> GetFontMeasure();

	/** Update the cache for the current thread */
	ENGINE_API void UpdateCache();

	/** Delegate called after releasing the rendering resources used by this font service */
	ENGINE_API FOnReleaseFontResources& OnReleaseResources();

private:
	/** Constructor - must be called from the game thread */
	FEngineFontServices();

	/** Destructor - must be called from the game thread */
	~FEngineFontServices();

	/** Slate font services instance being wrapped */
	TSharedPtr<class FSlateFontServices> SlateFontServices;

	/** Singular instance of this class */
	static FEngineFontServices* Instance;
};
