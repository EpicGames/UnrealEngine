// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"

/*----------------------------------------------------------------------------
	Forward declarations
----------------------------------------------------------------------------*/

class AActor;
class ABrush;
class ACameraActor;
class AController;
class AGameModeBase;
class AGameStateBase;
class ANavigationData;
class APawn;
class APlayerCameraManager;
class APlayerController;
class APlayerState;
class AWorldSettings;

class UActorChannel;
class UActorComponent;
class UAnimBlueprint;
class UAnimBlueprintGeneratedClass;
class UAnimCompress;
class UAnimInstance;
class UAnimMontage;
class UAnimSequence;
class UAnimSequenceBase;
class UAnimSingleNodeInstance;
class UAnimationAsset;
class UAudioComponent;
class UBillboardComponent;
class UBlendProfile;
class UBlendSpace;
class UBlueprint;
class UBlueprintGeneratedClass;
class UBodySetup;
class UCameraComponent;
class UCanvas;
class UChannel;
class UCharacterMovementComponent;
class UCurveFloat;
class UCurveTable;
class UCurveVector;
class UDataTable;
class UDeviceProfile;
class UDynamicBlueprintBinding;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema;
class UFont;
class UGameInstance;
class UGameViewportClient;
class UInputComponent;
class ULayer;
class ULevel;
class ULevelStreaming;
class ULightComponent;
class ULocalPlayer;
class UMapBuildDataRegistry;
class UMaterial;
class UMaterialExpression;
class UMaterialInstance;
class UMaterialInstanceConstant;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UMeshComponent;
class UModel;
class UMorphTarget;
class UNetConnection;
class UNetDriver;
class UParticleEmitter;
class UParticleLODLevel;
class UParticleModule;
class UParticleModuleTypeDataBase;
class UParticleSystem;
class UParticleSystemComponent;
class UPhysicalMaterial;
class UPhysicsAsset;
class UPlayer;
class UPrimitiveComponent;
class USCS_Node;
class USceneComponent;
class USelection;
class USimpleConstructionScript;
class USkeletalMesh;
class USkeletalMeshComponent;
class USkeletalMeshSocket;
class USkeleton;
class USkinnedMeshComponent;
class USlateBrushAsset;
class USoundBase;
class USoundClass;
class USoundCue;
class USoundNode;
class USoundSubmix;
class USoundWave;
class UStaticMesh;
class UStaticMeshComponent;
class UStaticMeshSocket;
class UTexture2D;
class UTexture;
class UTextureCube;
class UTextureLODSettings;
class UTextureRenderTarget2D;
class UThumbnailInfo;
class UUserDefinedEnum;
class UUserDefinedStruct;
class UWorld;

class FAudioDevice;
class FAudioDeviceManager;
class FCanvas;
class FColorVertexBuffer;
class FDebugDisplayInfo;
class FExportObjectInnerContext;
class FFinalPostProcessSettings;
class FInBunch;
class FLightSceneProxy;
class FLightingBuildOptions;
class FMaterial;
class FMaterialCompiler;
class FMaterialRenderProxy;
class FMaterialResource;
class FMaterialShaderMap;
class FMeshElementCollector;
class FMeshMapBuildData;
class FNetworkNotify;
class FOutBunch;
class FPoly;
class FPositionVertexBuffer;
class FPreviewScene;
class FPrimitiveDrawInterface;
class FPrimitiveSceneProxy;
class FRenderTarget;
class FSceneInterface;
class FSceneView;
class FSceneViewFamily;
class FSceneViewStateInterface;
class FSceneViewport;
class FShadowMapData2D;
class FSoundSource;
class FStaticMeshRenderData;
class FStaticPrimitiveDrawInterface;
class FStreamingTextureLevelContext;
class FTextureResource;
class FTimerManager;
class FViewport;
class FViewportClient;

class HHitProxy;

class INavLinkCustomInterface;

enum class ECacheApplyPhase;
enum class ETeleportType : uint8;

struct FActiveSound;
struct FActorComponentInstanceData;
struct FActorComponentTickFunction;
struct FActorTickFunction;
struct FAnimAssetTickContext;
struct FAnimInstanceProxy;
struct FAnimMontageInstance;
struct FAnimNode_Base;
struct FAnimNotifyEvent;
struct FAnimTickRecord;
struct FAnimationCacheBonesContext;
struct FAnimationInitializeContext;
struct FAnimationUpdateContext;
struct FBaseParticle;
struct FBlendSample;
struct FBlendedCurve;
struct FBodyInstance;
struct FBoneContainer;
struct FBspNode;
struct FBspSurf;
struct FCollisionQueryParams;
struct FCollisionResponseContainer;
struct FCollisionShape;
struct FConstraintInstance;
struct FConvexVolume;
struct FCurveEdEntry;
struct FDisplayDebugManager;
struct FEdGraphPinType;
struct FEngineShowFlags;
struct FExpressionInput;
struct FGraphActionListBuilderBase;
struct FGraphContextMenuBuilder;
struct FGraphDisplayInfo;
struct FHitResult;
struct FIntegralCurve;
struct FKAggregateGeom;
struct FKConvexElem;
struct FKSphereElem;
struct FLatentActionManager;
struct FLatentResponse;
struct FLevelCollection;
struct FMaterialRenderContext;
struct FMeshBatch;
struct FMeshBatchAndRelevance;
struct FMeshBatchElement;
struct FMinimalViewInfo;
struct FNavAgentProperties;
struct FNavigableGeometryExport;
struct FNavigationPath;
struct FNavigationRelevantData;
struct FNodeDebugData;
struct FParticleEmitterInstance;
struct FParticleRandomSeedInfo;
struct FPoseContext;
struct FPostProcessSettings;
struct FPrimitiveViewRelevance;
struct FReferenceSkeleton;
struct FRichCurve;
struct FRichCurveKey;
struct FSmartNameMapping;
struct FSoundParseParameters;
struct FStaticMeshLODResources;
struct FStaticMeshSection;
struct FURL;
struct FUniqueNetIdRepl;
struct FVisualLogEntry;
struct FWaveInstance;
struct FWorldContext;
