// Copyright Epic Games, Inc. All Rights Reserved.
// 
// Engine global defines

#pragma once

#include "CoreMinimal.h"

/** when set, disallows all network travel (pending level rejects all client travel attempts) */
extern ENGINE_API bool						GDisallowNetworkTravel;

