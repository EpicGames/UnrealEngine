// Copyright Epic Games, Inc. All Rights Reserved.
// 
// Log categories used by the Engine

#pragma once

#include "Logging/LogMacros.h"

class Error;

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogPath, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogController, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogPhysics, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogBlueprint, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogBlueprintUserMessages, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogAnimation, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogRootMotion, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogLevel, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMesh, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogStaticMesh, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNet, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetLifecycle, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetSubObject, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogRep, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetPlayerMovement, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetTraffic, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogRepTraffic, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogNetDormancy, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalControl, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSubtitle, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogTexture, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogTextureUpload, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogPlayerManagement, Error, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogSecurity, Warning, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogEngineSessionManager, Log, All);
ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogViewport, Log, All);
