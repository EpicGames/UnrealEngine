// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EngineMinimal.h: Commonly used include for developing projects with UE
=============================================================================*/

#pragma once

// IWYU pragma: begin_keep

// Boilerplate
#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

// UObject core
#include "CoreUObject.h"

// Actor based classes
#include "ComponentInstanceDataCache.h"
#include "EngineLogs.h"
#include "EngineDefines.h"
#include "TimerManager.h"
#include "SceneTypes.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "PixelFormat.h"
#include "Components.h"
#include "GPUSkinPublicDefs.h"
#include "ShowFlags.h"
#include "HitProxies.h"
#include "UnrealClient.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "ConvexVolume.h"
#include "BlendableManager.h"
#include "FinalPostProcessSettings.h"
#include "SceneInterface.h"
#include "DebugViewModeHelpers.h"
#include "SceneView.h"
#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "BoneIndices.h"
#include "ReferenceSkeleton.h"
#include "AnimInterpFilter.h"
#include "Animation/AnimTypes.h"
#include "CustomBoneIndexArray.h"
#include "BoneContainer.h"
#include "GameFramework/Character.h"
#include "GameFramework/Controller.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/GameModeBase.h"

// ActorComponent based classes
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "PhysxUserData.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"
#include "RawIndexBuffer.h"
#include "Components/StaticMeshComponent.h"
#include "Animation/AnimCurveTypes.h"
#include "ClothSimData.h"
#include "SingleAnimationPlayData.h"
#include "Animation/PoseSnapshot.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/CapsuleComponent.h"
#include "Components/SphereComponent.h"
#include "Components/BoxComponent.h"
#include "Components/InputComponent.h"
#include "GraphEditAction.h"
#include "BlueprintUtilities.h"
#include "IAudioExtensionPlugin.h"
#include "Audio.h"
#include "Components/AudioComponent.h"
#include "Camera/CameraComponent.h"
#include "GameFramework/SpringArmComponent.h"
#include "AI/RVOAvoidanceInterface.h"
#include "AI/NavDataGenerator.h"
#include "AI/NavigationSystemBase.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "ParticleVertexFactory.h"
#include "TextureResource.h"
#include "StaticParameterSet.h"
#include "MaterialShared.h"
#include "StaticBoundShaderState.h"
#include "BatchedElements.h"
#include "MeshBatch.h"
#include "SceneUtils.h"
#include "SceneManagement.h"
#include "MeshParticleVertexFactory.h"
#include "Distributions.h"
#include "ParticleEmitterInstances.h"
#include "Scalability.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "Components/LightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"

// Other
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/CollisionProfile.h"
#include "Kismet/GameplayStatics.h"
#include "MaterialExpressionIO.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundWave.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "ParticleHelper.h"
#endif

// IWYU pragma: end_keep
