// Copyright Epic Games, Inc. All Rights Reserved.
// 
// Engine module class

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class IRendererModule;

/** Implements the engine module. */
class FEngineModule : public FDefaultModuleImpl
{
public:

	// IModuleInterface
	virtual void StartupModule();
	virtual void ShutdownModule();
};

/** Accessor that gets the renderer module and caches the result. */
extern ENGINE_API IRendererModule& GetRendererModule();
