// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FMessageEndpoint;
class IMessageContext;
struct FEngineServiceAuthDeny;
struct FEngineServiceAuthGrant;
struct FEngineServiceExecuteCommand;
struct FEngineServicePing;
struct FEngineServiceTerminate;
struct FMessageAddress;

/**
 * Implements an application session service.
 */
class FEngineService
{
public:

	/** Default constructor. */
	ENGINE_API FEngineService();

protected:

	/**
	 * Sends a notification to the specified recipient.
	 *
	 * @param NotificationText The notification text.
	 * @param Recipient The recipient's message address.
	 */
	void SendNotification( const TCHAR* NotificationText, const FMessageAddress& Recipient );

	/**
	 * Publishes a ping response.
	 *
	 * @param Context The message context of the received Ping message.
	 */
	void SendPong( const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

private:

	/** Handles FEngineServiceAuthGrant messages. */
	void HandleAuthGrantMessage( const FEngineServiceAuthGrant& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FEngineServiceAuthGrant messages. */
	void HandleAuthDenyMessage( const FEngineServiceAuthDeny& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FEngineServiceExecuteCommand messages. */
	void HandleExecuteCommandMessage( const FEngineServiceExecuteCommand& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FEngineServicePing messages. */
	void HandlePingMessage( const FEngineServicePing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Handles FEngineServiceTerminate messages. */
	void HandleTerminateMessage( const FEngineServiceTerminate& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

private:

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
};
