// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateSharedPCH.h"

// From ApplicationCore:
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/ICursor.h"
#include "GenericPlatform/IInputInterface.h"

// From AssetRegistry:
#include "AssetRegistry/AssetData.h"

// From AudioExtensions:
#include "IAudioExtensionPlugin.h"

// From CoreOnline:
#include "Online/CoreOnline.h"

// From DeveloperSettings:
#include "Engine/DeveloperSettings.h"

// From Json:
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "JsonGlobals.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerMacros.h"

// From RHI:
#include "RHIDefinitions.h"
#include "RHI.h"
#include "RHIStaticStates.h"

// From RenderCore:
#include "GlobalShader.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "UniformBuffer.h"
#include "PackedNormal.h"
#include "RenderUtils.h"
#include "ShaderParameters.h"
#include "ShaderCore.h"
#include "Shader.h"
#include "VertexFactory.h"
#include "ShaderParameterUtils.h"
#include "StaticBoundShaderState.h"

// From PacketHandler:
#include "PacketHandler.h"

// From PhysicsCore:
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsSettingsEnums.h"

// From Engine:
#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavQueryFilter.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AlphaBlend.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimCompositeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimLinkableElement.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimStateMachineTypes.h"
#include "Animation/AnimStats.h"
#include "Animation/AnimTypes.h"
#include "Animation/PreviewAssetAttachComponent.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "AnimInterpFilter.h"
#include "Audio.h"
#include "BatchedElements.h"
#include "BlendableManager.h"
#include "BlueprintUtilities.h"
#include "BoneContainer.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraTypes.h"
#include "Camera/PlayerCameraManager.h"
#include "ClothSimData.h"
#include "CollisionQueryParams.h"
#include "ComponentInstanceDataCache.h"
#include "Components.h"
#include "Components/ActorComponent.h"
#include "Components/InputComponent.h"
#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "ConvexVolume.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/IndexedCurve.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "CustomBoneIndexArray.h"
#include "DataTableUtils.h"
#include "DebugViewModeHelpers.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintCore.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Brush.h"
#include "Engine/Channel.h"
#include "Engine/ChildConnection.h"
#include "Engine/CurveTable.h"
#include "Engine/DataAsset.h"
#include "Engine/DataTable.h"
#include "Engine/DebugDisplayProperty.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Engine/GameViewportDelegates.h"
#include "Engine/LatentActionManager.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LocalPlayer.h"
#include "Engine/MaterialMerging.h"
#include "Engine/MemberReference.h"
#include "Engine/NetConnection.h"
#include "Engine/NetDriver.h"
#include "Engine/NetSerialization.h"
#include "Engine/PendingNetGame.h"
#include "Engine/Player.h"
#include "Engine/PoseWatch.h"
#include "Engine/Scene.h"
#include "Engine/ScriptViewportClient.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureLightProfile.h"
#include "Engine/TextureStreamingTypes.h"
#include "Engine/TitleSafeZone.h"
#include "Engine/ViewportSplitScreen.h"
#include "Engine/World.h"
#include "EngineDefines.h"
#include "EngineGlobals.h"
#include "EngineLogs.h"
#include "EngineStats.h"
#include "FinalPostProcessSettings.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Controller.h"
#include "GameFramework/DamageType.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "GameFramework/Info.h"
#include "GameFramework/OnlineReplStructs.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerMuteList.h"
#include "GameFramework/Volume.h"
#include "GameFramework/WorldSettings.h"
#include "GPUSkinPublicDefs.h"
#include "HitProxies.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "LatentActions.h"
#include "LocalVertexFactory.h"
#include "MaterialExpressionIO.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/GenericOctree.h"
#include "Math/GenericOctreePublic.h"
#include "MeshBatch.h"
#include "MeshMaterialShaderType.h"
#include "Model.h"
#include "Net/DataBunch.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysxUserData.h"
#include "PixelFormat.h"
#include "PreviewScene.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveViewRelevance.h"
#include "RawIndexBuffer.h"
#include "ReferenceSkeleton.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneTypes.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "SingleAnimationPlayData.h"
#include "Sound/AudioVolume.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundConcurrency.h"
#include "Sound/SoundGroups.h"
#include "Sound/SoundWave.h"
#include "StaticParameterSet.h"
#include "TextureResource.h"
#include "Tickable.h"
#include "TimerManager.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"
#include "Vehicles/TireType.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "WorldCollision.h"

// Chaos
#include "Chaos/ParticleHandle.h"
#include "Chaos/Framework/PhysicsProxy.h"
