// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

/**
 * Engine stats
 */
/** Input stat */
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Input Latency"),STAT_InputLatencyTime,STATGROUP_Engine, );
/** HUD stat */
DECLARE_CYCLE_STAT_EXTERN(TEXT("HUD Time"),STAT_HudTime,STATGROUP_Engine, );
/** Static mesh tris rendered */
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Static Mesh Tris"),STAT_StaticMeshTriangles,STATGROUP_Engine, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Skel Skin Time"),STAT_SkinningTime,STATGROUP_Engine, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Cloth Verts Time"),STAT_UpdateClothVertsTime,STATGROUP_Engine, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Skel Mesh Tris"),STAT_SkelMeshTriangles,STATGROUP_Engine, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Skel Mesh Draw Calls"),STAT_SkelMeshDrawCalls,STATGROUP_Engine, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Skel Verts CPU Skin"),STAT_CPUSkinVertices,STATGROUP_Engine, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Skel Verts GPU Skin"),STAT_GPUSkinVertices,STATGROUP_Engine, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameEngine Tick"),STAT_GameEngineTick,STATGROUP_Engine, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GameViewport Tick"),STAT_GameViewportTick,STATGROUP_Engine, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RedrawViewports"),STAT_RedrawViewports,STATGROUP_Engine, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Level Streaming"),STAT_UpdateLevelStreaming,STATGROUP_Engine, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RHI Game Tick"),STAT_RHITickTime,STATGROUP_Engine, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Debug Hitch"),STAT_IntentionalHitch,STATGROUP_Engine, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Deferred Tick Time"),STAT_DeferredTickTime,STATGROUP_Engine, ENGINE_API);

/** Unit time stats */
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("StatUnit FrameTime"), STAT_UnitFrame, STATGROUP_Engine, );
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("StatUnit RenderThreadTime"), STAT_UnitRender, STATGROUP_Engine, );
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("StatUnit RHIThreadTime"), STAT_UnitRHIT, STATGROUP_Engine, );
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("StatUnit GameThreadTime"), STAT_UnitGame, STATGROUP_Engine, );
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("StatUnit GPUTime"), STAT_UnitGPU, STATGROUP_Engine, );


/**
 * Game stats
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async Physics Time"),STAT_PhysicsTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SpawnActor"), STAT_SpawnActorTime, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Actor BeginPlay"), STAT_ActorBeginPlay, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("MoveComponent(Primitive) Time"),STAT_MoveComponentTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("MoveComponent(SceneComp) Time"), STAT_MoveComponentSceneComponentTime, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateOverlaps Time"),STAT_UpdateOverlaps,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PerformOverlapQuery Time"), STAT_PerformOverlapQuery, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePhysicsVolume Time"),STAT_UpdatePhysicsVolume,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("EndScopedMovementUpdate Time"),STAT_EndScopedMovementUpdate,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("TeleportTo Time"),STAT_TeleportToTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GC Mark Time"),STAT_GCMarkTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GC Sweep Time"),STAT_GCSweepTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick Time"),STAT_TickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("World Tick Time"),STAT_WorldTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Camera Time"),STAT_UpdateCameraTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Char Movement Total"), STAT_CharacterMovement, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PlayerController Tick"), STAT_PlayerControllerTick, STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Post Tick Component Update"),STAT_PostTickComponentUpdate,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Post Tick Component Wait"),STAT_PostTickComponentUpdateWait,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recreate"),STAT_PostTickComponentRecreate,STATGROUP_Game, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Transform or RenderData"),STAT_PostTickComponentLW,STATGROUP_Game,EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Particle Data"),STAT_ParticleManagerUpdateData,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async Work Wait"),STAT_AsyncWorkWaitTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Visual Logging time"), STAT_VisualLog, STATGROUP_Game, ENGINE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Net Tick Time"),STAT_NetWorldTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Nav Tick Time"),STAT_NavWorldTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Reset Async Trace Time"),STAT_ResetAsyncTraceTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GT Tickable Time"),STAT_TickableTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Runtime Movie Tick Time"),STAT_RuntimeMovieSceneTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Finish Async Trace Time"),STAT_FinishAsyncTraceTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Net Broadcast Tick Time"),STAT_NetBroadcastTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("ServerReplicateActors Time"),STAT_NetServerRepActorsTime,STATGROUP_Game, );
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("ServerReplicateActors Gather Prioritze Time"),STAT_NetServerGatherPrioritizeRepActorsTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Consider Actors Time"),STAT_NetConsiderActorsTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Unmapped Objects Time"),STAT_NetUpdateUnmappedObjectsTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Inital Dormant Time"),STAT_NetInitialDormantCheckTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Prioritize Actors Time"),STAT_NetPrioritizeActorsTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Replicate Actor Time"),STAT_NetReplicateActorTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Property Rep Time"),STAT_NetReplicateDynamicPropTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Property Compare Time"),STAT_NetReplicateDynamicPropCompareTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Property Send Time"),STAT_NetReplicateDynamicPropSendTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Property Send BackCompat Time"),STAT_NetReplicateDynamicPropSendBackCompatTime,STATGROUP_Game, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Skipped Dynamic Props"),STAT_NetSkippedDynamicProps,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("NetSerializeItemDelta Time"),STAT_NetSerializeItemDeltaTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("NetUpdateGuidToReplicatorMap Time"), STAT_NetUpdateGuidToReplicatorMap,STATGROUP_Game, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Static Property Rep Time"),STAT_NetReplicateStaticPropTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Rebuild Conditionals"),STAT_NetRebuildConditionalTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Net Post BC Tick Time"),STAT_NetBroadcastPostTickTime,STATGROUP_Game, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Net PackageMap SerializeObject"),STAT_PackageMap_SerializeObjectTime,STATGROUP_Game, );


/**
 * Path finding stats
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async navmesh tile building"),STAT_Navigation_TileBuildAsync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync tiles building preparation"),STAT_Navigation_TileBuildPreparationSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Processing world for building navmesh"),STAT_Navigation_ProcessingActorsForNavMeshBuilding,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync actors' geometry export"),STAT_Navigation_ActorsGeometryExportSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync BSP geometry export"),STAT_Navigation_BSPExportSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync gathering navigation modifiers"),STAT_Navigation_GatheringNavigationModifiersSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async debug OBJ file export"),STAT_Navigation_TileGeometryExportToObjAsync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async voxel filtering"),STAT_Navigation_TileVoxelFilteringAsync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync pathfinding"),STAT_Navigation_PathfindingSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync requests for async pathfinding"),STAT_Navigation_RequestingAsyncPathfinding,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async pathfinding"),STAT_Navigation_PathfindingAsync,STATGROUP_Navigation, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num observed nav paths"), STAT_Navigation_ObservedPathsCount, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Offset from corners"), STAT_Navigation_OffsetFromCorners, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Visibility test for path optimisation"), STAT_Navigation_PathVisibilityOptimisation, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync queries"),STAT_Navigation_QueriesTimeSync,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync meta nav area preparation"),STAT_Navigation_MetaAreaTranslation,STATGROUP_Navigation, ); 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async nav areas sorting"),STAT_Navigation_TileNavAreaSorting,STATGROUP_Navigation, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num update nav octree"),STAT_Navigation_UpdateNavOctree,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Registering Nav Octree Element"), STAT_Navigation_RegisterNavOctreeElement, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Unregistering Nav Octree Element"), STAT_Navigation_UnregisterNavOctreeElement, STATGROUP_Navigation, )
DECLARE_CYCLE_STAT_EXTERN(TEXT("Adding actors to navoctree"),STAT_Navigation_AddingActorsToNavOctree,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Adjusting nav links"),STAT_Navigation_AdjustingNavLinks,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: sync add generated tiles"), STAT_Navigation_RecastAddGeneratedTiles,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: sync add generated tile Layer"), STAT_Navigation_RecastAddGeneratedTileLayer, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: tick"),STAT_Navigation_RecastTick,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: pathfinding"),STAT_Navigation_RecastPathfinding, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: test path"),STAT_Navigation_RecastTestPath, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: storing compressed layers"), STAT_Navigation_StoringCompressedLayers, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: create tile generator"), STAT_Navigation_CreateTileGenerator, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: do work"), STAT_Navigation_DoWork, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: remove layers"), STAT_Navigation_RemoveLayers, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build compressed layers"),STAT_Navigation_RecastBuildCompressedLayers,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: create height field"),STAT_Navigation_RecastCreateHeightField,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: compute rasterizaion masks"), STAT_Navigation_RecastComputeRasterizationMasks, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: rasterize triangles"),STAT_Navigation_RecastRasterizeTriangles,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: create voxel filter"),STAT_Navigation_RecastVoxelFilter,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: create filters"),STAT_Navigation_RecastFilter,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: filter ledge spans"), STAT_Navigation_FilterLedgeSpans, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build compact height field"),STAT_Navigation_RecastBuildCompactHeightField,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: erode walkable areas"),STAT_Navigation_RecastErodeWalkable,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build layers"),STAT_Navigation_RecastBuildLayers,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build tile cache"),STAT_Navigation_RecastBuildTileCache,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build navmesh"),STAT_Navigation_RecastBuildNavigation,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build navigation layer"), STAT_Navigation_GenerateNavigationDataLayer, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build links"), STAT_Navigation_RecastBuildLinks, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build links, find edges"), STAT_Navigation_RecastBuildLinks_FindEdges, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build links, sample"), STAT_Navigation_RecastBuildLinks_Sample, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build regions"), STAT_Navigation_RecastBuildRegions, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: mark areas"), STAT_Navigation_RecastMarkAreas, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build contours"), STAT_Navigation_RecastBuildContours, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build poly mesh"), STAT_Navigation_RecastBuildPolyMesh, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: build poly detail"), STAT_Navigation_RecastBuildPolyDetail, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: gather off mesh data"), STAT_Navigation_RecastGatherOffMeshData, STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Recast: create mesh data"), STAT_Navigation_RecastCreateNavMeshData, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Nav tree memory"),STAT_Navigation_CollisionTreeMemory,STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Nav data memory"),STAT_Navigation_NavDataMemory,STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Tile cache memory"),STAT_Navigation_TileCacheMemory,STATGROUP_Navigation, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Out of nodes path"),STAT_Navigation_OutOfNodesPath,STATGROUP_Navigation, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Partial path"),STAT_Navigation_PartialPath,STATGROUP_Navigation, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Navmesh cumulative build Time"),STAT_Navigation_CumulativeBuildTime,STATGROUP_Navigation, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Navmesh build time"),STAT_Navigation_BuildTime,STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Recast memory"), STAT_Navigation_RecastMemory, STATGROUP_Navigation, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour TEMP"), STAT_Navigation_DetourTEMP, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM"), STAT_Navigation_DetourPERM, STATGROUP_Navigation, ); // This stat should not really be used and will be deprecated, favor stats that define the preferred type of allocation.
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_AVOIDANCE"), STAT_Navigation_DetourPERM_AVOIDANCE, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_CROWD"), STAT_Navigation_DetourPERM_CROWD, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_LOOKUP"), STAT_Navigation_DetourPERM_LOOKUP, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_NAVQUERY"), STAT_Navigation_DetourPERM_NAVQUERY, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_NAVMESH"), STAT_Navigation_DetourPERM_NAVMESH, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_NODE_POOL"), STAT_Navigation_DetourPERM_NODE_POOL, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_PATH_CORRIDOR"), STAT_Navigation_DetourPERM_PATH_CORRIDOR, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_PATH_QUEUE"), STAT_Navigation_DetourPERM_PATH_QUEUE, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_PROXY_GRID"), STAT_Navigation_DetourPERM_PROXY_GRID, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_TILE_DATA"), STAT_Navigation_DetourPERM_TILE_DATA, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_TILE_DYNLINK"), STAT_Navigation_DetourPERM_TILE_DYNLINK, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_TILE_DYNLINK_CLUSTER"), STAT_Navigation_DetourPERM_TILE_DYNLINK_CLUSTER, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_TILES"), STAT_Navigation_DetourPERM_TILES, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour PERM_TILE_LINK_BUILDER"), STAT_Navigation_DetourPERM_TILE_LINK_BUILDER, STATGROUP_Navigation, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Memory"), STAT_DetourTileMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Mesh Header Memory"), STAT_DetourTileMeshHeaderMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Nav Verts Memory"), STAT_DetourTileNavVertsMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Nav Polys Memory"), STAT_DetourTileNavPolysMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Links Memory"), STAT_DetourTileLinksMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Detail Meshes Memory"), STAT_DetourTileDetailMeshesMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Detail Verts Memory"), STAT_DetourTileDetailVertsMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Detail Tris Memory"), STAT_DetourTileDetailTrisMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile BV Tree Memory"), STAT_DetourTileBVTreeMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Off Mesh Cons Memory"), STAT_DetourTileOffMeshConsMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Off Mesh Segs Memory"), STAT_DetourTileOffMeshSegsMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Poly CLusters Memory"), STAT_DetourTileClustersMemory, STATGROUP_Navigation, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Detour Tile Poly CLusters Memory"), STAT_DetourTilePolyClustersMemory, STATGROUP_Navigation, );

/**
* Canvas Stats
*/
DECLARE_CYCLE_STAT_EXTERN(TEXT("Flush Time"),STAT_Canvas_FlushTime,STATGROUP_Canvas, );	
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw Texture Tile Time"),STAT_Canvas_DrawTextureTileTime,STATGROUP_Canvas, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw Material Tile Time"),STAT_Canvas_DrawMaterialTileTime,STATGROUP_Canvas, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Draw String Time"),STAT_Canvas_DrawStringTime,STATGROUP_Canvas, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Word Wrapping Time"),STAT_Canvas_WordWrappingTime,STATGROUP_Canvas, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Get Batched Element Time"),STAT_Canvas_GetBatchElementsTime,STATGROUP_Canvas,EStatFlags::Verbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Add Material Tile Time"),STAT_Canvas_AddTileRenderTime,STATGROUP_Canvas, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Add Material Triangle Time"),STAT_Canvas_AddTriangleRenderTime,STATGROUP_Canvas, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Batches Created"),STAT_Canvas_NumBatchesCreated,STATGROUP_Canvas, );	

/**
 * Network stats counters
 */

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Ping"),STAT_Ping,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Channels"),STAT_Channels,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Packet Overhead - max among connections"), STAT_MaxPacketOverhead, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Rate (bytes)"),STAT_InRate,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Rate (bytes)"),STAT_OutRate,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Rate (bytes) - max among clients"),STAT_InRateClientMax,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Rate (bytes) - min among clients"),STAT_InRateClientMin,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Rate (bytes) - avg among clients"),STAT_InRateClientAvg,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Packets - max among clients"), STAT_InPacketsClientMax, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Packets - min among clients"), STAT_InPacketsClientMin, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Packets - avg among clients"), STAT_InPacketsClientAvg, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Rate (bytes) - max among clients"),STAT_OutRateClientMax,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Rate (bytes) - min among clients"),STAT_OutRateClientMin,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Rate (bytes) - avg among clients"),STAT_OutRateClientAvg,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Packets - max among clients"), STAT_OutPacketsClientMax, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Packets - min among clients"), STAT_OutPacketsClientMin, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Packets - avg among clients"), STAT_OutPacketsClientAvg, STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Clients"),STAT_NetNumClients,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Packets"),STAT_InPackets,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Packets"),STAT_OutPackets,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Bunches"),STAT_InBunches,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Bunches"),STAT_OutBunches,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out Loss"),STAT_OutLoss,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In Loss"),STAT_InLoss,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Voice bytes sent"),STAT_VoiceBytesSent,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Voice bytes recv"),STAT_VoiceBytesRecv,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Voice packets sent"),STAT_VoicePacketsSent,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Voice packets recv"),STAT_VoicePacketsRecv,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("In % Voice"),STAT_PercentInVoice,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Out % Voice"),STAT_PercentOutVoice,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Actor Channels"),STAT_NumActorChannels,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Considered Actors"),STAT_NumConsideredActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Prioritized Actors"),STAT_PrioritizedActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Relevant Actors"),STAT_NumRelevantActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Relevant Deleted Actors"),STAT_NumRelevantDeletedActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Actors"),STAT_NumActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Network Actors"),STAT_NumNetActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Dormant Actors"),STAT_NumDormantActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Initially Dormant Actors"),STAT_NumInitiallyDormantActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num ACKd NetGUIDs"),STAT_NumNetGUIDsAckd,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Pending NetGUIDs"),STAT_NumNetGUIDsPending,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num UnACKd NetGUIDs"),STAT_NumNetGUIDsUnAckd,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Object path (bytes)"),STAT_ObjPathBytes,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NetGUID Out Rate (bytes)"),STAT_NetGUIDOutRate,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NetGUID In Rate (bytes)"),STAT_NetGUIDInRate,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Saturated"),STAT_NetSaturated,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Imported NetGuids"),STAT_ImportedNetGuids,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Pending Outer NetGuids"),STAT_PendingOuterNetGuids,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Unmapped Replicators"),STAT_UnmappedReplicators,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Replicated Actors Sent"),STAT_NumReplicatedActors,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Replicated Actor Bytes Sent"),STAT_NumReplicatedActorBytes,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Largest Reliable Outgoing Queue Size"),STAT_OutgoingReliableMessageQueueMaxSize,STATGROUP_Net, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Largest Reliable Incoming Queue Size"),STAT_IncomingReliableMessageQueueMaxSize,STATGROUP_Net, );

#if !UE_BUILD_SHIPPING
/**
 * Packet stats counters
 */
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Max Packet (bits)"), STAT_MaxPacket, STATGROUP_Packet, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Max Packet Minus Reserved (bits)"), STAT_MaxPacketMinusReserved, STATGROUP_Packet, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Reserved Total (bits)"), STAT_PacketReservedTotal, STATGROUP_Packet, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Reserved NetConnection (bits)"), STAT_PacketReservedNetConnection, STATGROUP_Packet, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Reserved PacketHandler Total (bits)"), STAT_PacketReservedPacketHandler, STATGROUP_Packet, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Reserved Handshake (bits)"), STAT_PacketReservedHandshake, STATGROUP_Packet, );
#endif
