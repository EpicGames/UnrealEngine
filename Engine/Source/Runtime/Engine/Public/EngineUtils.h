// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Engine.h: Unreal engine public header file.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "HitProxies.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/UObjectHash.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/StringOutputDevice.h"

#if WITH_EDITOR
#include "Algo/Accumulate.h"
#include "Algo/Copy.h"
#endif

class FCanvas;
class FViewport;
class UCanvas;
class UConsole;
class UPrimitiveComponent;

/*-----------------------------------------------------------------------------
	Hit proxies.
-----------------------------------------------------------------------------*/

// Hit an actor.
struct HActor : public HHitProxy
{
	DECLARE_HIT_PROXY( ENGINE_API )
	TObjectPtr<AActor> Actor;
	TObjectPtr<const UPrimitiveComponent> PrimComponent;
	int32 SectionIndex;
	int32 MaterialIndex;

	HActor( AActor* InActor, const UPrimitiveComponent* InPrimComponent )
		: Actor( InActor ) 
		, PrimComponent( InPrimComponent )
		, SectionIndex( -1 )
		, MaterialIndex( -1 )
		{}

	HActor(AActor* InActor, const UPrimitiveComponent* InPrimComponent, int32 InSectionIndex, int32 InMaterialIndex)
		: Actor( InActor )
		, PrimComponent( InPrimComponent )
		, SectionIndex(InSectionIndex)
		, MaterialIndex( InMaterialIndex )
		{}

	HActor( AActor* InActor, const UPrimitiveComponent* InPrimComponent, EHitProxyPriority InPriority) 
		: HHitProxy( InPriority )
		, Actor( InActor )
		, PrimComponent( InPrimComponent )
		, SectionIndex( -1 )
		, MaterialIndex( -1 )
		{}

	HActor(AActor* InActor, const UPrimitiveComponent* InPrimComponent, EHitProxyPriority InPriority, int32 InSectionIndex, int32 InMaterialIndex)
		: HHitProxy(InPriority)
		, Actor(InActor)
		, PrimComponent(InPrimComponent)
		, SectionIndex(InSectionIndex)
		, MaterialIndex(InMaterialIndex)
		{}

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual EMouseCursor::Type GetMouseCursor() override;
	ENGINE_API virtual FTypedElementHandle GetElementHandle() const override;
	ENGINE_API bool AlwaysAllowsTranslucentPrimitives() const override;
};

//
//	HBSPBrushVert
//

struct HBSPBrushVert : public HHitProxy
{
	DECLARE_HIT_PROXY( ENGINE_API );
	TWeakObjectPtr<ABrush>	Brush;
	FVector3f* Vertex;
	HBSPBrushVert(ABrush* InBrush,FVector3f* InVertex):
		HHitProxy(HPP_UI),
		Brush(InBrush),
		Vertex(InVertex)
	{}
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		//@todo: Brush Hit proxies are currently referencing between UWorld's (undesired), 
		// once this issue is resolved remove the TWeakObjectPtr and replace with Standard ABrush*.
		// Also uncomment this line:
		//Collector.AddReferencedObject( Brush );
	}
};

//
//	HStaticMeshVert
//

struct HStaticMeshVert : public HHitProxy
{
	DECLARE_HIT_PROXY( ENGINE_API );
	TObjectPtr<AActor>	Actor;
	FVector Vertex;
	HStaticMeshVert(AActor* InActor,FVector InVertex):
		HHitProxy(HPP_UI),
		Actor(InActor),
		Vertex(InVertex)
	{}
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject( Actor );
	}
};

// Hit an actor even with translucency
struct HTranslucentActor : public HActor
{
	DECLARE_HIT_PROXY( ENGINE_API )
	HTranslucentActor( AActor* InActor, const UPrimitiveComponent* InPrimComponent ) 
		: HActor( InActor, InPrimComponent ) 
		{}

	HTranslucentActor( AActor* InActor, const UPrimitiveComponent* InPrimComponent, EHitProxyPriority InPriority) 
		: HActor(InActor, InPrimComponent, InPriority)
		{}

	ENGINE_API virtual EMouseCursor::Type GetMouseCursor() override;
	ENGINE_API virtual bool AlwaysAllowsTranslucentPrimitives() const override;
};


/*-----------------------------------------------------------------------------
	Iterator for the editor that loops through all selected actors.
-----------------------------------------------------------------------------*/

/**
 * Abstract base class for actor iteration. Implements all operators and relies on IsActorSuitable
 * to be overridden by derived class.
 * Note that when Playing In Editor, this will find actors only in CurrentWorld.
 */
class FActorIteratorState
{
public:
	/** Current world we are iterating upon						*/
	const UWorld* CurrentWorld;
	/** Results from the GetObjectsOfClass query				*/
	TArray<AActor*> ActorArray;
	/** index of the current element in the object array		*/
	int32 Index;
	/** Whether we already reached the end						*/
	bool ReachedEnd;
	/** Number of actors that have been considered thus far		*/
	int32 ConsideredCount;
	/** Current actor pointed to by actor iterator				*/
	AActor*	CurrentActor;
	/** Contains any actors spawned during iteration			*/
	TArray<AActor*> SpawnedActorArray;
	/** The class type we are iterating, kept for filtering		*/
	UClass* DesiredClass;
	/** Handle to the registered OnActorSpawned delegate		*/
	FDelegateHandle ActorSpawnedDelegateHandle;

	/**
	 * Default ctor, inits everything
	 */
	FActorIteratorState(const UWorld* InWorld, const TSubclassOf<AActor> InClass) :
		CurrentWorld( InWorld ),
		Index( -1 ),
		ReachedEnd( false ),
		ConsideredCount( 0 ),
		CurrentActor(nullptr),
		DesiredClass(InClass)
	{
		check(IsInGameThread());
		check(CurrentWorld);

#if WITH_EDITOR
		TArray<AActor*> LocalActorsOfClass;
		if (InClass != AActor::StaticClass())
		{
			GetObjectsOfClass(InClass, reinterpret_cast<TArray<UObject*>&>(LocalActorsOfClass), true, RF_ClassDefaultObject, EInternalObjectFlags::Garbage);
			if (LocalActorsOfClass.IsEmpty())
			{
				return;
			}
		}

		// In the editor, you are more likely to have many worlds in memory at once.
		// As an optimization to avoid iterating over many actors that are not in the world we are asking for,
		// just use the actors that are in the world you asked for (persistent level + streaming levels basically).
		// This could be useful in runtime code as well if there are many worlds in memory, but for now we will
		// leave it in editor code.

		// First determine the number of actors in the world to reduce reallocations when we append them to the array below.
		int32 MaxActors = InWorld->PersistentLevel ? InWorld->PersistentLevel->Actors.Num() : 0;
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level && Level != InWorld->PersistentLevel)
			{
				MaxActors += Level->Actors.Num();
			}
		}
		TArray<AActor*> LocalWorldActors;
		LocalWorldActors.Reserve(MaxActors);

		// Add persistent level actors, we need to do this explicitly to handle worlds that are not initialized yet, as
		// InWorld->GetLevels() would return an empty level list even if the world's persistent level contains actors.
		if (InWorld->PersistentLevel)
		{
			LocalWorldActors.Append(InWorld->PersistentLevel->Actors);
		}

		// Add streaming levels actors
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level && Level != InWorld->PersistentLevel)
			{
				LocalWorldActors.Append(Level->Actors);
			}
		}

		if (InClass != AActor::StaticClass())
		{
			// Intersect the 2 arrays now to only retain the actors of the proper type. This is faster this way as we only rely on the pointer values to filter the actors (since LocalActorsOfClass 
			//  is already a superset of all possible actors of class) and it avoids iterating on useless actors in the iteration loop (along with expensive RTTI calls to IsA()) 
			
			// LocalActorsOfClass is the superset of all actors of the proper type so there will never be more than this amount : 
			ActorArray.Reserve(LocalActorsOfClass.Num());
			// Pick the array that has the least element and turn it into a TSet for fast lookup
			const bool bLookUpInActorsOfClass = LocalActorsOfClass.Num() < LocalWorldActors.Num();
			TSet<AActor*> LookUpSet(bLookUpInActorsOfClass ? LocalActorsOfClass : LocalWorldActors);
			TArray<AActor*>& ArrayToIterate = bLookUpInActorsOfClass ? LocalWorldActors : LocalActorsOfClass;
			for (AActor* Actor : ArrayToIterate)
			{
				if (LookUpSet.Contains(Actor))
				{
					ActorArray.Add(Actor);
				}
			}
		}
		else
		{
			// If we're interested in all AActors, just use the World actors we have retrieved
			Swap(LocalWorldActors, ActorArray);
		}
#else // WITH_EDITOR
		constexpr EObjectFlags ExcludeFlags = RF_ClassDefaultObject;
		GetObjectsOfClass(InClass, reinterpret_cast<TArray<UObject*>&>(ActorArray), true, ExcludeFlags, EInternalObjectFlags::Garbage);
#endif

		const auto ActorSpawnedDelegate = FOnActorSpawned::FDelegate::CreateRaw(this, &FActorIteratorState::OnActorSpawned);
		ActorSpawnedDelegateHandle = CurrentWorld->AddOnActorSpawnedHandler(ActorSpawnedDelegate);
	}

	~FActorIteratorState()
	{
		CurrentWorld->RemoveOnActorSpawnedHandler(ActorSpawnedDelegateHandle);
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	inline AActor* GetActorChecked() const
	{
		check(CurrentActor);
		checkf(!CurrentActor->IsUnreachable(), TEXT("%s"), *CurrentActor->GetFullName());
		return CurrentActor;
	}

private:
	void OnActorSpawned(AActor* InActor)
	{
		if (InActor->IsA(DesiredClass))
		{
#if WITH_EDITOR
			// In the editor, the actor list is pre-filtered by world so perform the test now instead of in the loop like at runtime :
			if (ULevel* ActorLevel = InActor->GetLevel(); ActorLevel && (ActorLevel->GetWorld() == CurrentWorld))
#endif // WITH_EDITOR
			{
				SpawnedActorArray.AddUnique(InActor);
			}
		}
	}
};

/** Type enum, used to represent the special End iterator */
enum class EActorIteratorType
{
	End
};

/** Iteration flags, specifies which types of levels and actors should be iterated */
enum class EActorIteratorFlags
{
	AllActors			= 0x00000000, // No flags, iterate all actors
	SkipPendingKill		= 0x00000001, // Skip pending kill actors
	OnlySelectedActors	= 0x00000002, // Only iterate actors that are selected
	OnlyActiveLevels	= 0x00000004, // Only iterate active levels
};
ENUM_CLASS_FLAGS(EActorIteratorFlags);

/**
 * Template class used to filter actors by certain characteristics
 */
template <typename Derived>
class TActorIteratorBase
{
public:
	/**
	 * Iterates to next suitable actor.
	 */
	void operator++()
	{
		// Use local version to avoid LHSs as compiler is not required to write out member variables to memory.
		AActor* LocalCurrentActor = nullptr;
		int32 LocalIndex = State->Index;
		TArray<AActor*>& LocalActorArray = State->ActorArray;
		TArray<AActor*>& LocalSpawnedActorArray = State->SpawnedActorArray;
		const UWorld* LocalCurrentWorld = State->CurrentWorld;
		// LocalActorArray is immutable within the loop so we can cache its number of elements :
		const int32 LocalActorArrayNum = LocalActorArray.Num();
		while(++LocalIndex < (LocalActorArrayNum + LocalSpawnedActorArray.Num()))
		{
			if (LocalIndex < LocalActorArrayNum)
			{
				LocalCurrentActor = LocalActorArray[LocalIndex];
			}
			else
			{
				LocalCurrentActor = LocalSpawnedActorArray[LocalIndex - LocalActorArrayNum];
			}
			State->ConsideredCount++;
			
			ULevel* ActorLevel = LocalCurrentActor ? LocalCurrentActor->GetLevel() : nullptr;
			if ( ActorLevel
				&& static_cast<const Derived*>(this)->IsActorSuitable(LocalCurrentActor)
				&& static_cast<const Derived*>(this)->CanIterateLevel(ActorLevel)
#if !WITH_EDITOR
				// No need to perform the world test if ActorArray has already been pre-filtered, which is the case in editor :
				&& (ActorLevel->GetWorld() == LocalCurrentWorld)
#endif // !WITH_EDITOR
				)
			{
				// ignore non-persistent world settings
				if (ActorLevel == LocalCurrentWorld->PersistentLevel || !LocalCurrentActor->IsA(AWorldSettings::StaticClass()))
				{
					State->CurrentActor = LocalCurrentActor;
					State->Index = LocalIndex;
					return;
				}
			}
		}
		State->CurrentActor = nullptr;
		State->ReachedEnd = true;
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	FORCEINLINE AActor* operator*() const
	{
		return State->GetActorChecked();
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	FORCEINLINE AActor* operator->() const
	{
		return State->GetActorChecked();
	}
	/**
	 * Returns whether the iterator has reached the end and no longer points
	 * to a suitable actor.
	 *
	 * @return true if iterator points to a suitable actor, false if it has reached the end
	 */
	FORCEINLINE explicit operator bool() const
	{
		return !State->ReachedEnd;
	}

	/**
	 * Clears the current Actor in the array (setting it to NULL).
	 */
	void ClearCurrent()
	{
		check(!State->ReachedEnd);
		State->CurrentWorld->RemoveActor(State->CurrentActor, true);
	}

	/**
	 * Returns the number of actors considered thus far. Can be used in combination
	 * with GetProgressDenominator to gauge progress iterating over all actors.
	 *
	 * @return number of actors considered thus far.
	 */
	int32 GetProgressNumerator() const
	{
		return State->ConsideredCount;
	}

protected:
	/**
	 * Hide the constructors as construction on this class should only be done by subclasses
	 */
	explicit TActorIteratorBase(EActorIteratorType)
		: Flags(EActorIteratorFlags::AllActors)
	{
	}

	TActorIteratorBase(const UWorld* InWorld, TSubclassOf<AActor> InClass, const EActorIteratorFlags InFlags)
		: Flags(InFlags)
	{
		State.Emplace(InWorld, InClass);
	}

	/**
	 * Determines whether this is a valid actor or not.
	 * This function should be redefined (thus hiding this one) in a derived class if it wants special actor filtering.
	 *
	 * @param	Actor	Actor to check
	 * @return	true
	 */
	inline bool IsActorSuitable(const AActor* Actor) const
	{
		if (EnumHasAnyFlags(Flags, EActorIteratorFlags::SkipPendingKill) && !IsValid(Actor))
		{
			return false;
		}

		if (EnumHasAnyFlags(Flags, EActorIteratorFlags::OnlySelectedActors) && !Actor->IsSelected())
		{
			return false;
		}

		return true;
	}

	/**
	 * Used to examine whether this level is valid for iteration or not
	 * This function should be redefined (thus hiding this one) in a derived class if it wants special level filtering.
	 *
	 * @param Level the level to check for iteration
	 * @return true if the level can be iterated, false otherwise
	 */
	inline bool CanIterateLevel(const ULevel* Level) const
	{
		if (EnumHasAnyFlags(Flags, EActorIteratorFlags::OnlyActiveLevels))
		{
			const bool bIsLevelVisibleOrAssociating = (Level->bIsVisible && !Level->bIsBeingRemoved) || Level->bIsAssociatingLevel || Level->bIsDisassociatingLevel;

			// Only allow iteration of Level if it's in the currently active level collection of the world, or is a static level.
			const FLevelCollection* const ActorLevelCollection = Level->GetCachedLevelCollection();
			const FLevelCollection* const ActiveLevelCollection = Level->OwningWorld ? Level->OwningWorld->GetActiveLevelCollection() : nullptr;

			// If the world's active level collection is null, we can't apply any meaningful filter,
			// so just allow iteration in this case.
			const bool bIsCurrentLevelCollectionTicking = !ActiveLevelCollection || (ActorLevelCollection == ActiveLevelCollection);

			const bool bIsLevelCollectionNullOrStatic = !ActorLevelCollection || ActorLevelCollection->GetType() == ELevelCollectionType::StaticLevels;
			const bool bShouldIterateLevelCollection = bIsCurrentLevelCollectionTicking || bIsLevelCollectionNullOrStatic;

			return bIsLevelVisibleOrAssociating && bShouldIterateLevelCollection;
		}

		return true;
	}

private:
	EActorIteratorFlags Flags;
	TOptional<FActorIteratorState> State;

	friend bool operator==(const TActorIteratorBase& Lhs, const TActorIteratorBase& Rhs) { check(!Rhs.State); return  !Lhs; }
	friend bool operator!=(const TActorIteratorBase& Lhs, const TActorIteratorBase& Rhs) { check(!Rhs.State); return !!Lhs; }
};

/**
 * Actor iterator
 * Note that when Playing In Editor, this will find actors only in CurrentWorld
 */
class FActorIterator : public TActorIteratorBase<FActorIterator>
{
	friend class TActorIteratorBase<FActorIterator>;
	typedef TActorIteratorBase<FActorIterator> Super;

public:
	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 * @param InFlags	Iteration flags indicating which types of levels and actors should be iterated
	 */
	explicit FActorIterator(const UWorld* InWorld, const EActorIteratorFlags InFlags = EActorIteratorFlags::OnlyActiveLevels | EActorIteratorFlags::SkipPendingKill)
		: Super(InWorld, AActor::StaticClass(), InFlags)
	{
		++(*this);
	}

	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 * @param InClass	The type of actors to be iterated over.
	 * @param InFlags	Iteration flags indicating which types of levels and actors should be iterated
	 */
	explicit FActorIterator(const UWorld* InWorld, const TSubclassOf<AActor> InClass, const EActorIteratorFlags InFlags = EActorIteratorFlags::OnlyActiveLevels | EActorIteratorFlags::SkipPendingKill)
		: Super(InWorld, InClass, InFlags)
	{
		++(*this);
	}

	/**
	 * Constructor for creating an end iterator
	 */
	explicit FActorIterator(EActorIteratorType)
		: Super(EActorIteratorType::End)
	{
	}
};

/**
 * Actor range for ranged-for support.
 * Note that when Playing In Editor, this will find actors only in CurrentWorld
 */
class FActorRange
{
public:
	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 * @param InFlags	Iteration flags indicating which types of levels and actors should be iterated
	 */
	explicit FActorRange(const UWorld* InWorld, const EActorIteratorFlags InFlags = EActorIteratorFlags::OnlyActiveLevels | EActorIteratorFlags::SkipPendingKill)
		: Flags(InFlags)
		, World(InWorld)
	{
	}

private:
	EActorIteratorFlags	Flags;
	const UWorld* World;

	friend FActorIterator begin(const FActorRange& Range) { return FActorIterator(Range.World, Range.Flags); }
	friend FActorIterator end  (const FActorRange& Range) { return FActorIterator(EActorIteratorType::End); }
};

/**
 * Template actor iterator.
 */
template <typename ActorType>
class TActorIterator : public TActorIteratorBase<TActorIterator<ActorType>>
{
	friend class TActorIteratorBase<TActorIterator>;
	typedef TActorIteratorBase<TActorIterator> Super;

public:
	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 * @param InClass	The subclass of actors to be iterated over.
	 * @param InFlags	Iteration flags indicating which types of levels and actors should be iterated
	 */
	explicit TActorIterator(const UWorld* InWorld, TSubclassOf<ActorType> InClass = ActorType::StaticClass(), EActorIteratorFlags InFlags = EActorIteratorFlags::OnlyActiveLevels | EActorIteratorFlags::SkipPendingKill)
		: Super(InWorld, InClass, InFlags)
	{
		++(*this);
	}

	/**
	 * Constructor for creating an end iterator
	 */
	explicit TActorIterator(EActorIteratorType)
		: Super(EActorIteratorType::End)
	{
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	FORCEINLINE ActorType* operator*() const
	{
		return CastChecked<ActorType>(**static_cast<const Super*>(this));
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	FORCEINLINE ActorType* operator->() const
	{
		return **this;
	}
};

/**
 * Template actor range for ranged-for support.
 */
template <typename ActorType>
class TActorRange
{
public:
	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 * @param InClass	The subclass of actors to be iterated over.
	 * @param InFlags	Iteration flags indicating which types of levels and actors should be iterated
	 */
	explicit TActorRange(const UWorld* InWorld, TSubclassOf<ActorType> InClass = ActorType::StaticClass(), const EActorIteratorFlags InFlags = EActorIteratorFlags::OnlyActiveLevels | EActorIteratorFlags::SkipPendingKill)
		: Flags(InFlags)
		, World(InWorld)
		, Class(InClass)
	{
	}

private:
	EActorIteratorFlags		Flags;
	const UWorld*			World;
	TSubclassOf<ActorType>	Class;

	friend TActorIterator<ActorType> begin(const TActorRange& Range) { return TActorIterator<ActorType>(Range.World, Range.Class, Range.Flags); }
	friend TActorIterator<ActorType> end(const TActorRange& Range) { return TActorIterator<ActorType>(EActorIteratorType::End); }
};

/**
 * Selected actor iterator, this is for ease of use but the same can be done by adding EActorIteratorFlags::OnlySelectedActors to 
 */
class FSelectedActorIterator : public TActorIteratorBase<FSelectedActorIterator>
{
	friend class TActorIteratorBase<FSelectedActorIterator>;
	typedef TActorIteratorBase<FSelectedActorIterator> Super;

public:
	/**
	 * Constructor
	 *
	 * @param InWorld	The world whose actors are to be iterated over.
	 */
	explicit FSelectedActorIterator(const UWorld* InWorld)
		: Super(InWorld, AActor::StaticClass(), EActorIteratorFlags::SkipPendingKill | EActorIteratorFlags::OnlySelectedActors)
	{
		++(*this);
	}

	/**
	 * Constructor
	 *
	 * @param  InWorld  The world whose actors are to be iterated over.
	 * @param  InClass  The type of actors to be iterated over.
	 */
	explicit FSelectedActorIterator(const UWorld* InWorld, const TSubclassOf<AActor> InClass)
		: Super(InWorld, InClass, EActorIteratorFlags::SkipPendingKill | EActorIteratorFlags::OnlySelectedActors)
	{
		++(*this);
	}

	/**
	 * Constructor for creating an end iterator
	 */
	explicit FSelectedActorIterator(EActorIteratorType)
		: Super(EActorIteratorType::End)
	{
	}
};

/**
 * Selected actor range for ranged-for support.
 */
class FSelectedActorRange
{
public:
	/**
	 * Constructor
	 *
	 * @param  InWorld  The world whose actors are to be iterated over.
	 */
	explicit FSelectedActorRange(const UWorld* InWorld)
		: World(InWorld)
	{
	}

private:
	const UWorld* World;

	friend FSelectedActorIterator begin(const FSelectedActorRange& Range) { return FSelectedActorIterator(Range.World); }
	friend FSelectedActorIterator end  (const FSelectedActorRange& Range) { return FSelectedActorIterator(EActorIteratorType::End); }
};

/**
 * An output device that forwards output to both the log and the console.
 */
class FConsoleOutputDevice : public FStringOutputDevice
{
	typedef FStringOutputDevice Super;

public:

	/**
	 * Minimal initialization constructor.
	 */
	FConsoleOutputDevice(class UConsole* InConsole):
		Super(TEXT("")),
		Console(InConsole)
	{}

	ENGINE_API virtual void Serialize(const TCHAR* Text, ELogVerbosity::Type Verbosity, const class FName& Category) override;

private:

	/** The console which output is written to. */
	UConsole* Console;
};


/**
 *	Renders stats
 *
 *	@param InWorld			The World to render stats
 *	@param Viewport			The viewport to render to
 *	@param Canvas			Canvas object to use for rendering
 *	@param CanvasObject		Optional canvas object for visualizing properties
 *	@param DebugProperties	List of properties to visualize (in/out)
 *	@param ViewLocation		Location of camera
 *	@param ViewRotation		Rotation of camera
 */
ENGINE_API void DrawStatsHUD( UWorld* InWorld, FViewport* Viewport, FCanvas* Canvas, UCanvas* CanvasObject, TArray<struct FDebugDisplayProperty>& DebugProperties, const FVector& ViewLocation, const FRotator& ViewRotation );

/** SubLevel Actor breakdown information **/
struct FSubLevelActorDetails
{
	FSubLevelActorDetails() :
		Count(0)
		, NativeClassName(NAME_None)
	{
	}

	int32 Count;
	FName NativeClassName;
};

/** SubLevel status information */
struct FSubLevelStatus
{
	FName				PackageName;
	FString				LevelLabel;
	EStreamingStatus	StreamingStatus;
	int32				LODIndex;
	bool				bInConsiderList;
	bool				bPlayerInside;
	int32				ActorCount;
	TMap<FName, FSubLevelActorDetails>	ActorMapToCount;
};

/**
 *	Gathers SubLevels status from a provided world
 *	@param InWorld		World to gather sublevels stats from
 *	@return				sublevels status (streaming state, LOD index, where player is)
 */
ENGINE_API TArray<FSubLevelStatus> GetSubLevelsStatus( UWorld* InWorld, bool SortByActorCount = false );

#if !UE_BUILD_SHIPPING

// Helper class containing information about UObject assets referenced.
struct FContentComparisonAssetInfo
{
	/** Name of the asset */
	FString AssetName;
	/** The resource size of the asset */
	int32 ResourceSize;

	/** Constructor */
	FContentComparisonAssetInfo()
	{
		FMemory::Memzero(this, sizeof(FContentComparisonAssetInfo));
	}

	/** operator == */
	bool operator==(const FContentComparisonAssetInfo& Other) const
	{
		return (
			(AssetName == Other.AssetName) &&
			(ResourceSize == Other.ResourceSize)
			);
	}
};

/** Helper class for performing the content comparison console command */
class FContentComparisonHelper
{
public:
	FContentComparisonHelper();
	virtual ~FContentComparisonHelper();

	/**
	 *	Compare the classes derived from the given base class.
	 *
	 *	@param	InBaseClassName			The base class to perform the comparison on.
	 *	@param	InRecursionDepth		How deep to recurse when walking the object reference chain. (Max = 4)
	 *
	 *	@return	bool					true if successful, false if not
	 */
	virtual bool CompareClasses(const FString& InBaseClassName, int32 InRecursionDepth);

	/**
	 *	Compare the classes derived from the given base class, ignoring specified base classes.
	 *
	 *	@param	InBaseClassName			The base class to perform the comparison on.
	 *	@param	InBaseClassesToIgnore	The base classes to ignore when processing objects.
	 *	@param	InRecursionDepth		How deep to recurse when walking the object reference chain. (Max = 4)
	 *
	 *	@return	bool					true if successful, false if not
	 */
	virtual bool CompareClasses(const FString& InBaseClassName, const TArray<FString>& InBaseClassesToIgnore, int32 InRecursionDepth);

	/**
	 *	Recursive function for collecting objects referenced by the given object.
	 *
	 *	@param	InStartObject			The object to collect the references for
	 *	@param	InCurrDepth				The current depth being processed
	 *	@param	InMaxDepth				The maximum depth to traverse the reference chain
	 *	@param	OutCollectedReferences	The resulting referenced object list
	 */
	void RecursiveObjectCollection(UObject* InStartObject, int32 InCurrDepth, int32 InMaxDepth, TMap<UObject*,bool>& OutCollectedReferences);

protected:
	TMap<FString,bool> ReferenceClassesOfInterest;
};
#endif

namespace EngineUtils
{
	enum EAssetToLoad
	{
		ATL_Regular,
		ATL_Class,
	};

	/** Loads all the assets found in the specified path and subpaths */
	ENGINE_API bool FindOrLoadAssetsByPath(const FString& Path, TArray<UObject*>& OutAssets, EAssetToLoad Type);
}

/** Helper class for serializing flags describing which data have been stripped (if any). */
class FStripDataFlags
{
	/** Serialized engine strip flags (up to 8 flags). */
	uint8 GlobalStripFlags = 0;

	/** Serialized per-class strip flags (user defined, up to 8 flags). */
	uint8 ClassStripFlags = 0;

public:

	/** Engine strip flags */
	enum class EStrippedData : uint8
	{
		None = 0,

		/* This flag means that Editor-only data is stripped */
		EditorOnly = 1,

		/* This flag means that AudioVisual data is stripped (e.g. target is dedicated server). */
		AudioVisual = 2,

		/* This flag means that all data needed to cook packages will be stripped. What it means is specific to 
		  an asset type, and it might be a subset of both EditorOnly and AudioVisual. This flag is normally set, except
		  when cooking for a "cooked cooker" target. */
		NeededForCooking = 4,

		// Add global flags here (up to 8 including the already defined ones).

		/** All flags */
		All = 0xff
	};

	/** 
	 * Constructor.
	 * Serializes strip data flags. Global (engine) flags are automatically generated from target platform
	 * when saving. Class flags need to be defined by the user.
	 *
	 * @param Ar - Archive to serialize with.
	 * @param InClassFlags - User defined per class flags .
	 * @param InVersion - Minimal strip version required to serialize strip flags
	 */
	ENGINE_API explicit FStripDataFlags(FArchive& Ar, uint8 InClassFlags = 0, const FPackageFileVersion& InVersion = GOldestLoadablePackageFileUEVersion);

	/** 
	 * Constructor.
	 * Serializes strip data flags. Global (engine) flags are user defined and will not be automatically generated
	 * when saving. Class flags also need to be defined by the user.
	 *
	 * @param Ar - Archive to serialize with.
	 * @param InGlobalFlags Engine flags
	 * @param InClassFlags - User defined per class flags.
	 * @param InVersion - Minimal version required to serialize strip flags
	 */
	ENGINE_API FStripDataFlags( FArchive& Ar, uint8 InGlobalFlags, uint8 InClassFlags, const FPackageFileVersion& InVersion = GOldestLoadablePackageFileUEVersion);

	/**
	* Constructor.
	* Serializes strip data flags. Global (engine) flags are automatically generated from target platform
	* when saving. Class flags need to be defined by the user.
	*
	* @param Slot - Slot holding the archive to serialize with.
	* @param InClassFlags - User defined per class flags .
	* @param InVersion - Minimal strip version required to serialize strip flags
	*/
	ENGINE_API explicit FStripDataFlags(FStructuredArchive::FSlot Slot, uint8 InClassFlags = 0, const FPackageFileVersion& InVersion = GOldestLoadablePackageFileUEVersion);

	/**
	* Constructor.
	* Serializes strip data flags. Global (engine) flags are user defined and will not be automatically generated
	* when saving. Class flags also need to be defined by the user.
	*
	* @param Slot - Slot holding the archive to serialize with.
	* @param InGlobalFlags - Engine flags.
	* @param InClassFlags - User defined per class flags.
	* @param InVersion - Minimal version required to serialize strip flags
	*/
	ENGINE_API FStripDataFlags(FStructuredArchive::FSlot Slot, uint8 InGlobalFlags, uint8 InClassFlags, const FPackageFileVersion& InVersion = GOldestLoadablePackageFileUEVersion);

	/**
	 * Checks if FStripDataFlags::Editor flag is set or not
	 *
	 * @return true if FStripDataFlags::Editor is set, false otherwise.
	 */
	inline bool IsEditorDataStripped() const
	{
		return (GlobalStripFlags & static_cast<uint8>(FStripDataFlags::EStrippedData::EditorOnly)) != 0;
	}

	/**
	 * Checks if FStripDataFlags::AudioVisual flag is set or not
	 *
	 * @return true if FStripDataFlags::AudioVisual is set, false otherwise.
	 */
	bool IsAudioVisualDataStripped() const
	{
		return (GlobalStripFlags & static_cast<uint8>(FStripDataFlags::EStrippedData::AudioVisual)) != 0;
	}

	/**
	 * Checks if FStripDataFlags::NeededForCooking flag is set or not. It should be set for all non-content-worker targets
	 *
	 * @return true if FStripDataFlags::NeededForCooking is set, false otherwise.
	 */
	bool IsDataNeededForCookingStripped() const
	{
		return (GlobalStripFlags & static_cast<uint8>(FStripDataFlags::EStrippedData::NeededForCooking)) != 0;
	}

	/**
	 * Checks if user defined flags are set or not.
	 *
	 * @InFlags - User defined flags to check.
	 * @return true if the specified user defined flags are set, false otherwise.
	 */
	bool IsClassDataStripped(uint8 InFlags) const
	{
		return (ClassStripFlags & InFlags) != 0;
	}
};

class UTexture;
namespace VirtualTextureUtils
{
	/** Function that will test if the passed in texture is VT. If so, print to Messagelog that the property does not support VT textures */
	ENGINE_API void CheckAndReportInvalidUsage(const UObject* Owner, const FName& PropertyName, const UTexture* Texture);
}