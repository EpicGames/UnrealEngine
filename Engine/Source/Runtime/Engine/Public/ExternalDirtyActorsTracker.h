// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "GameFramework/Actor.h"
#include "TickableEditorObject.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "Editor.h"

class ULevel;

/**
 * TExternalDirtyActorsTracker is a tracker for dirty external actors, with custom storage through the StoreType interface.
 */
template <typename StoreType>
class TExternalDirtyActorsTracker : public FTickableEditorObject
{
public:
	using Super = TExternalDirtyActorsTracker<StoreType>;
	using ValueType = typename StoreType::Type;
	using MapType = TMap<TWeakObjectPtr<AActor>, ValueType>;

	TExternalDirtyActorsTracker(const ULevel* InLevel, typename StoreType::OwnerType* InOwner)
		: Level(InLevel)
		, Owner(InOwner)
	{
		UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &TExternalDirtyActorsTracker::OnPackageDirtyStateChanged);
		FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &TExternalDirtyActorsTracker::OnObjectsReplaced);
		FEditorDelegates::OnEditorActorReplaced.AddRaw(this, &TExternalDirtyActorsTracker::OnEditorActorReplaced);
	}

	~TExternalDirtyActorsTracker()
	{
		UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
		FEditorDelegates::OnEditorActorReplaced.RemoveAll(this);
	}

	const MapType& GetDirtyActors() const { return DirtyActors; }

	/** Returns false if tracking this actor in unnecessary */
	virtual bool OnAddDirtyActor(const TWeakObjectPtr<AActor> InActor) { return true; }
	virtual void OnRemoveInvalidDirtyActor(const TWeakObjectPtr<AActor> InActor, ValueType& InValue) {}
	virtual void OnRemoveNonDirtyActor(const TWeakObjectPtr<AActor> InActor, ValueType& InValue) {}

protected:
	void OnPackageDirtyStateChanged(UPackage* InPackage)
	{
		if (AActor* Actor = AActor::FindActorInPackage(InPackage))
		{
			if (ULevel* OuterLevel = Actor->GetTypedOuter<ULevel>())
			{
				if (OuterLevel == Level)
				{
					if (InPackage->IsDirty())
					{
						if (OnAddDirtyActor(Actor))
						{
							DirtyActors.Add(Actor, StoreType::Store(Owner, Actor));
						}
					}
					else
					{
						ValueType Value;
						if (DirtyActors.RemoveAndCopyValue(Actor, Value))
						{
							if (IsValid(Actor))
							{
								OnRemoveNonDirtyActor(Actor, Value);
							}
							else
							{
								OnRemoveInvalidDirtyActor(Actor, Value);
							}
						}
					}
				}
			}
		}
	}

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewObjectMap)
	{
		for (const TPair<UObject*, UObject*>& Itr : OldToNewObjectMap)
		{
			if (AActor* OldActor = Cast<AActor>(Itr.Key))
			{
				if (AActor* NewActor = Cast<AActor>(Itr.Value))
				{
					OnEditorActorReplaced(OldActor, NewActor);
				}
			}
		}
	}

	void OnEditorActorReplaced(AActor* InOldActor, AActor* InNewActor)
	{
		ValueType ValueToCopy;
		if (DirtyActors.RemoveAndCopyValue(InOldActor, ValueToCopy))
		{
			DirtyActors.Add(InNewActor, ValueToCopy);
		}
	}

	//~ Begin FTickableEditorObject interface
	virtual TStatId GetStatId() const override
	{
		return TStatId();
	}

	virtual void Tick(float DeltaTime) override
	{
		for (typename MapType::TIterator DirtyActorIt(DirtyActors); DirtyActorIt; ++DirtyActorIt)
		{
			if (DirtyActorIt.Key().IsValid())
			{
				if (!DirtyActorIt.Key().Get()->GetPackage()->IsDirty())
				{
					OnRemoveNonDirtyActor(DirtyActorIt.Key(), DirtyActorIt.Value());
					DirtyActorIt.RemoveCurrent();
				}
			}
			else if (!DirtyActorIt.Key().IsValid(true))
			{
				OnRemoveInvalidDirtyActor(DirtyActorIt.Key(), DirtyActorIt.Value());
				DirtyActorIt.RemoveCurrent();
			}
		}
	}
	//~ End FTickableEditorObject interface

	const ULevel* Level;
	typename StoreType::OwnerType* Owner;
	MapType DirtyActors;
};
#endif