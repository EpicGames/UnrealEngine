// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/UObjectThreadContext.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Delegates/DelegateCombinations.h"
#include "WorldPartition/DataLayer/DataLayerInstanceProviderInterface.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "Engine/Level.h"
#include "UObject/CoreRedirects.h"

class FExternalPackageHelper
{
public:
	class FRenameExternalObjectsHelperContext
	{
	public:
		FRenameExternalObjectsHelperContext() = delete;
		FRenameExternalObjectsHelperContext(FRenameExternalObjectsHelperContext&) = delete;
		FRenameExternalObjectsHelperContext& operator=(FRenameExternalObjectsHelperContext&) = delete;
		
		ENGINE_API explicit FRenameExternalObjectsHelperContext(const UObject* SourceObject, ERenameFlags Flags);
		ENGINE_API ~FRenameExternalObjectsHelperContext();
	private:		
		const UObject* OldObject = nullptr;
		const UPackage* SourcePackage = nullptr;
	};
	DECLARE_EVENT_TwoParams(FExternalPackageHelper, FOnObjectPackagingModeChanged, UObject*, bool /* bExternal */);
	static ENGINE_API FOnObjectPackagingModeChanged OnObjectPackagingModeChanged;

	/**
	 * Create an external package
	 * @param InObjectOuter the object's outer
	 * @param InObjectPath the fully qualified object path, in the format: 'Outermost.Outer.Name'
	 * @param InFlags the package flags to apply
	 * @return the created package
	 */
	static ENGINE_API UPackage* CreateExternalPackage(const UObject* InObjectOuter, const FString& InObjectPath, EPackageFlags InFlags = FExternalPackageHelper::GetDefaultExternalPackageFlags(), const UExternalDataLayerAsset* InExternalDataLayerAsset = nullptr);

	/** Returns default external package flags used to create external packages. */
	static ENGINE_API EPackageFlags GetDefaultExternalPackageFlags();

	/**
	 * Set the object packaging mode.
	 * @param InObject the object on which to change the packaging mode
	 * @param InObjectOuter the object's outer
	 * @param bInIsPackageExternal will set the object packaging mode to external if true, to internal otherwise
	 * @param bInShouldDirty should dirty or not the object's outer package
	 * @param InExternalPackageFlags the flags to apply to the external package if bInIsPackageExternal is true
	 */
	static ENGINE_API void SetPackagingMode(UObject* InObject, const UObject* InObjectOuter, bool bInIsPackageExternal, bool bInShouldDirty = true, EPackageFlags InExternalPackageFlags = FExternalPackageHelper::GetDefaultExternalPackageFlags());

	/**
	 * Get the path containing the external objects for this path
	 * @param InOuterPackageName The package name to get the external objects path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the path
	 */
	static ENGINE_API FString GetExternalObjectsPath(const FString& InOuterPackageName, const FString& InPackageShortName = FString());

	/**
	 * Get the path containing the external objects for this Outer
	 * @param InPackage The package to get the external objects path of
	 * @param InPackageShortName Optional short name to use instead of the package short name
	 * @return the path
	 */
	static ENGINE_API FString GetExternalObjectsPath(UPackage* InPackage, const FString& InPackageShortName = FString(), bool bTryUsingPackageLoadedPath = false);


	/**
	 * Get the external package name for this object
	 * @param InOuterPackageName The name of the package of that contains the outer of the object.
	 * @param InObjectPath the fully qualified object path, in the format: 'Outermost.Outer.Name'
	 * @return the package name
	 */
	static ENGINE_API FString GetExternalPackageName(const FString& InOuterPackageName, const FString& InObjectPath);

	/**
	 * Loads objects from an external package
	 */
	template<typename T>
	static void LoadObjectsFromExternalPackages(UObject* InOuter, TFunctionRef<void(T*)> Operation);

	enum class EGetExternalSaveableObjectsFlags : uint32
	{
		// No flags
		None                   = 0,

		// Whether to check the object's package dirty flag. Controls whether only dirty or all external objects are returned.
		CheckDirty             = (1 << 0) 
	};

	FRIEND_ENUM_CLASS_FLAGS(EGetExternalSaveableObjectsFlags);

	/**
	 * Get the saveable external objects that should be saved alongside this outer's package
	 * @param InOuter		The external object's outer
	 * @param OutObjects	The objects that should be saved
	 * @param InFlags		Flags controlling behavior @see EGetExternalSaveableObjectsFlags
	 */
	static ENGINE_API void GetExternalSaveableObjects(UObject* InOuter, TArray<UObject*>& OutObjects, EGetExternalSaveableObjectsFlags InFlags = EGetExternalSaveableObjectsFlags::CheckDirty);

	/**
	 * Returns an array of external package file paths for the provided objects
	 * @param InObjects		The objects to process
	 */
	static ENGINE_API TArray<FString> GetObjectsExternalPackageFilePath(const TArray<const UObject*>& InObjects);

	/*
	 * Copies the file path of objects external package to the clipboard
	 */
	static ENGINE_API void CopyObjectsExternalPackageFilePathToClipboard(const TArray<const UObject*>& InObjects);
	
	/**
	 * Call AssetRegistry.GetAssets and sort the results for deterministic use in cooked data.
	 */
	static ENGINE_API void GetSortedAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssets);

    /**
    * Duplicates all ExternalPackage for any UObject outered to InObject (UObject themselves are handled by regular DuplicateObject behavior)
    */
	static ENGINE_API void DuplicateExternalPackages(const UObject* InObject, FObjectDuplicationParameters& InDuplicationParameters, EActorPackagingScheme ActorPackagingScheme = EActorPackagingScheme::Reduced);
private:
	/** Get the external object package instance name. */
	static ENGINE_API FString GetExternalObjectPackageInstanceName(const FString& OuterPackageName, const FString& ObjectPackageName);
};

template<typename T>
void FExternalPackageHelper::LoadObjectsFromExternalPackages(UObject* InOuter, TFunctionRef<void(T*)> Operation)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FExternalPackageHelper::LoadObjectsFromExternalPackages);

	check(InOuter);
	UPackage* OutermostPackage = InOuter->IsA<UPackage>() ? CastChecked<UPackage>(InOuter) : InOuter->GetOutermostObject()->GetPackage();
	const FString OutermostPackageName = !OutermostPackage->GetLoadedPath().IsEmpty() ? OutermostPackage->GetLoadedPath().GetPackageName() : OutermostPackage->GetName();
	const IDataLayerInstanceProvider* DataLayerInstanceProvider = Cast<IDataLayerInstanceProvider>(InOuter);
	const UExternalDataLayerAsset* ExternalDataLayerAsset = DataLayerInstanceProvider ? DataLayerInstanceProvider->GetRootExternalDataLayerAsset() : nullptr;
	const FString RootPath = ExternalDataLayerAsset ? FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(ExternalDataLayerAsset, OutermostPackageName) : OutermostPackageName;
	const FString ExternalObjectsPath = FExternalPackageHelper::GetExternalObjectsPath(RootPath);
	TArray<FString> ObjectPackageNames;

	// Do a synchronous scan of the world external objects path.			
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.ScanSynchronous({ ExternalObjectsPath }, TArray<FString>());

	// Find any redirects for this class and any of its derived classes so we can load older external objects (asset registry class does not redirect)
	TArray<FTopLevelAssetPath> ClassPaths;
	ClassPaths.Add(T::StaticClass()->GetClassPathName());

	if(T::StaticClass() != UObject::StaticClass())
	{
		TArray<UClass*> DerivedClasses;
		DerivedClasses.Add(T::StaticClass());
		GetDerivedClasses(T::StaticClass(), DerivedClasses);

		for(UClass* Class : DerivedClasses)
		{
			TArray<FCoreRedirectObjectName> PreviousRedirectedNames;
			FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(Class->GetClassPathName()), PreviousRedirectedNames);

			for(const FCoreRedirectObjectName& PreviousRedirectedName : PreviousRedirectedNames)
			{
				ClassPaths.Add(FTopLevelAssetPath(PreviousRedirectedName.PackageName, PreviousRedirectedName.ObjectName));
			}
		}
	}

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	Filter.ClassPaths = MoveTemp(ClassPaths);
	Filter.bRecursiveClasses = true;
	Filter.PackagePaths.Add(*ExternalObjectsPath);
	TArray<FAssetData> Assets;
	GetSortedAssets(Filter, Assets);

	ObjectPackageNames.Reserve(Assets.Num());
	for (const FAssetData& Asset : Assets)
	{
		ObjectPackageNames.Add(Asset.PackageName.ToString());
	}

	FLinkerInstancingContext InstancingContext;
	TArray<UPackage*> InstancePackages;

	UPackage* OuterPackage = InOuter->GetPackage();
	FName PackageResourceName = OuterPackage->GetLoadedPath().GetPackageFName();
	const bool bInstanced = !PackageResourceName.IsNone() && (PackageResourceName != OuterPackage->GetFName());
	if (bInstanced)
	{
		const FLinkerInstancingContext* OuterInstancingContext = nullptr;
		if (FLinkerLoad* OuterLinker = InOuter->GetLinker())
		{
			OuterInstancingContext = &OuterLinker->GetInstancingContext();
		}

		// Add packages of all outers to instancing context
		TSet<const UPackage*> OuterPackages;
		UObject* ItObj = InOuter;
		while (ItObj)
		{
			const UPackage* Package = ItObj->GetPackage();
			bool bIsAlreadyInSet = false;
			OuterPackages.Add(Package, &bIsAlreadyInSet);
			if (!bIsAlreadyInSet)
			{
				InstancingContext.AddPackageMapping(Package->GetLoadedPath().GetPackageFName(), Package->GetFName());
			}
			ItObj = ItObj->GetOuter();
		}

		for (const FString& ObjectPackageName : ObjectPackageNames)
		{
			FName InstancedName;
			
			FName ObjectPackageFName = *ObjectPackageName;
			if (OuterInstancingContext)
			{
				InstancedName = OuterInstancingContext->RemapPackage(ObjectPackageFName);
			}

			// Remap to the a instanced package if it wasn't remapped already by the outer instancing context
			if (InstancedName == ObjectPackageFName || InstancedName.IsNone())
			{
				InstancedName = *GetExternalObjectPackageInstanceName(OuterPackage->GetName(), ObjectPackageName);
			}

			InstancingContext.AddPackageMapping(ObjectPackageFName, InstancedName);

			// Create instance package
			UPackage* InstancePackage = CreatePackage(*InstancedName.ToString());
			// Propagate RF_Transient
			if (OuterPackage->HasAnyFlags(RF_Transient))
			{
				InstancePackage->SetFlags(RF_Transient);
			}
			InstancePackages.Add(InstancePackage);
		}
	}

	const ELoadFlags LoadFlags = InOuter->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor) ? LOAD_PackageForPIE : LOAD_None;
	for (int32 i = 0; i < ObjectPackageNames.Num(); i++)
	{
		if (UPackage* Package = LoadPackage(bInstanced ? InstancePackages[i] : nullptr, *ObjectPackageNames[i], LoadFlags, nullptr, &InstancingContext))
		{
			T* LoadedObject = nullptr;
			ForEachObjectWithPackage(Package, [&LoadedObject](UObject* Object)
			{
				if (T* TypedObj = Cast<T>(Object))
				{
					LoadedObject = TypedObj;
					return false;
				}
				return true;
			}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Unreachable);

			if (ensure(LoadedObject))
			{
				Operation(LoadedObject);
			}
		}
	}
}

#endif
