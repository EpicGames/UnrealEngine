// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "MaterialShared.h"
#include "HAL/CriticalSection.h"

/**
 * Stores a registry of external textures mapped to their GUIDs.
 */
class FExternalTextureRegistry
{
	/**
	 * Registry singleton instance.
	 *
	 * @see Get
	 */
	static FExternalTextureRegistry* Singleton;

	/** Default constructor. */
	FExternalTextureRegistry() { }

	/** An entry in the registry. */
	struct FExternalTextureEntry
	{
		FExternalTextureEntry(FTextureRHIRef& InTextureRHI, FSamplerStateRHIRef& InSamplerStateRHI, const FLinearColor& InCoordinateScaleRotation, const FLinearColor& InCoordinateOffset)
			: TextureRHI(InTextureRHI)
			, SamplerStateRHI(InSamplerStateRHI)
			, CoordinateScaleRotation(InCoordinateScaleRotation)
			, CoordinateOffset(InCoordinateOffset)
		{}

		const FTextureRHIRef TextureRHI;
		const FSamplerStateRHIRef SamplerStateRHI;
		FLinearColor CoordinateScaleRotation;
		FLinearColor CoordinateOffset;
	};

	/** Maps unique identifiers to texture entries. */
	TMap<FGuid, FExternalTextureEntry> TextureEntries;

	/** Set of registered material render proxies. */
	TSet<const FMaterialRenderProxy*> ReferencingMaterialRenderProxies;

	/** Critical section to protect the set and map above from modification during access with threaded rendering. */
	FCriticalSection CriticalSection;

public:

	/**
	 * Register an external texture, its sampler state and coordinate scale/bias against a GUID.
	 *
	 * @param InGuid The texture's unique identifier.
	 * @param InTextureRHI The texture.
	 * @param InSamplerStateRHI The texture's sampler state.
	 * @param InCoordinateScaleRotation Texture coordinate scale and rotation parameters (optional; packed into color value).
	 * @param InCoordinateOffset Texture coordinate offset (optional; packed into color value).
	 * @see GetExternalTexture, UnregisterExternalTexture
	 */
	ENGINE_API void RegisterExternalTexture(const FGuid& InGuid, FTextureRHIRef& InTextureRHI, FSamplerStateRHIRef& InSamplerStateRHI, const FLinearColor& InCoordinateScaleRotation = FLinearColor(1, 0, 0, 1), const FLinearColor& InCoordinateOffset = FLinearColor(0, 0, 0, 0));

	/**
	 * Removes an external texture given a GUID.
	 *
	 * @param InGuid The unique identifier of the texture to unregister.
	 * @see RegisterExternalTexture
	 */
	ENGINE_API void UnregisterExternalTexture(const FGuid& InGuid);

	/**
	 * Removes the specified MaterialRenderProxy from the list of those using an external texture.
	 *
	 * @param MaterialRenderProxy The material render proxy to remove.
	 * @see GetExternalTexture, RegisterExternalTexture, UnregisterExternalTexture
	 */
	ENGINE_API void RemoveMaterialRenderProxyReference(const FMaterialRenderProxy* MaterialRenderProxy);

	/**
	 * Get the external texture with the specified identifier.
	 *
	 * @param MaterialRenderProxy The material render proxy that is using the texture (will be registered).
	 * @param InGuid The texture's unique identifier.
	 * @param OutTextureRHI Will contain the external texture.
	 * @param OutSamplerStateRHI Will contain the texture's sampler state.
	 * @return true on success, false if the identifier was not found.
	 * @see RegisterExternalTexture, UnregisterExternalTexture
	 */
	ENGINE_API bool GetExternalTexture(const FMaterialRenderProxy* MaterialRenderProxy, const FGuid& InGuid, FTextureRHIRef& OutTextureRHI, FSamplerStateRHIRef& OutSamplerStateRHI);

	/**
	 * Looks up an external texture's coordinate scale rotation.
	 *
	 * @param InGuid The texture's unique identifier.
	 * @param OutCoordinateScaleRotation Will contain the texture's coordinate scale and rotation (packed into color value).
	 * @return true on success, false if the identifier was not found.
	 * @see GetExternalTextureCoordinateOffset
	 */
	ENGINE_API bool GetExternalTextureCoordinateScaleRotation(const FGuid& InGuid, FLinearColor& OutCoordinateScaleRotation);

	/**
	 * Looks up an texture coordinate offset for given a given GUID.
	 *
	 * @param InGuid The texture's unique identifier.
	 * @param OutCoordinateOffset Will contain the texture's coordinate offset (packed into color value).
	 * @return true on success, false if the identifier was not found.
	 * @see GetExternalTextureCoordinateScaleRotation
	 */
	ENGINE_API bool GetExternalTextureCoordinateOffset(const FGuid& InGuid, FLinearColor& OutCoordinateOffset);

public:

	/**
	 * Get the registry singleton instance.
	 *
	 * @return External texture registry.
	 */
	static ENGINE_API FExternalTextureRegistry& Get();
};
