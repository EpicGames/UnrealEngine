// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FXSystem.h: Interface to the effects system.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RenderUtils.h"
#include "RenderGraphDefinitions.h"

class FCanvas;
class FGPUSortManager;
struct FGPUSpriteEmitterInfo;
struct FGPUSpriteResourceData;
class FGPUSpriteResources;
struct FParticleEmitterInstance;
class FRHIUniformBuffer;
class FScene;
class FSceneInterface;
class UVectorFieldComponent;
class FGlobalDistanceFieldParameterData;
class FSceneUniformBuffer;
class FSceneViewFamily;
class FSceneTextureUniformParameters;
class FViewInfo;
template<typename TBufferStruct> class TUniformBufferRef;

/*-----------------------------------------------------------------------------
	Forward declarations.
-----------------------------------------------------------------------------*/

/** Particle emitter instance. */
struct FParticleEmitterInstance;
/** Information required for runtime simulation of GPU sprites. */
struct FGPUSpriteEmitterInfo;
/** Data for initializing GPU sprite render resources. */
struct FGPUSpriteResourceData;
/** Render resources needed for GPU sprite simulation. */
class FGPUSpriteResources;
/** A vector field asset. */
class UVectorField;
/** A vector field component. */
class UVectorFieldComponent;
/** Resource belonging to a vector fied. */
class FVectorFieldResource;
/** An instance of a vector field in the world. */
class FVectorFieldInstance;
/** Interface with which primitives can be drawn. */
class FPrimitiveDrawInterface;
/** View from which a scene is rendered. */
class FSceneView;
/** Canvas that can be drawn on. */
class FCanvas;

/*------------------------------------------------------------------------------
	FX Console variables.
------------------------------------------------------------------------------*/

/**
 * WARNING: These variables must only be changed via the console manager!
 */
namespace FXConsoleVariables
{
	/** Visualize GPU particle simulation. */
	extern int32 VisualizeGPUSimulation;
	/** true if GPU emitters are permitted to sort. */
	ENGINE_API extern int32 bAllowGPUSorting;
	/** true if emitters can be culled. */
	extern int32 bAllowCulling;
	/** true if GPU particle simulation is frozen. */
	extern int32 bFreezeGPUSimulation;
	/** true if particle simulation is frozen. */
	extern int32 bFreezeParticleSimulation;
	/** true if we allow async ticks */
	extern int32 bAllowAsyncTick;
	/** Amount of slack to allocate for GPU particles to prevent tile churn as percentage of total particles. */
	extern float ParticleSlackGPU;
	/** Maximum tile preallocation for GPU particles. */
	extern int32 MaxParticleTilePreAllocation;
	/** Maximum number of CPU particles to allow per-emitter. */
	extern int32 MaxCPUParticlesPerEmitter;
	/** Maximum number of GPU particles to spawn per-frame. */
	extern int32 MaxGPUParticlesSpawnedPerFrame;
	/** Warning threshold for spawning of GPU particles. */
	extern int32 GPUSpawnWarningThreshold;
	/** Depth bounds for GPU collision checks. */
	extern float GPUCollisionDepthBounds;
	/** Specify a sorting test to run. */
	extern TAutoConsoleVariable<int32> TestGPUSort;
	/** true if GPU particles are allowed. */
	extern int32 bAllowGPUParticles;
}

/**
 * Returns true if the shader platform supports GPU particles.
 */
inline bool SupportsGPUParticles(EShaderPlatform Platform)
{
	return true; // For editor mobile preview 
}

/*
 * Returns true if the current RHI supports GPU particles 
 */
//@todo rename this function. 
// Unlike other RHI* functions which are static, it actually returns true if the
// RHI on the current hardware is able to support GPU particles.
inline bool RHISupportsGPUParticles()
{
	return FXConsoleVariables::bAllowGPUParticles
		&& GSupportsWideMRT
		&& GPixelFormats[PF_G32R32F].Supported 
		&& GSupportsTexture3D;
}

class FFXSystemInterface;
class FGPUSortManager;
DECLARE_DELEGATE_RetVal_ThreeParams(FFXSystemInterface*, FCreateCustomFXSystemDelegate, ERHIFeatureLevel::Type, EShaderPlatform, FGPUSortManager*);

/*-----------------------------------------------------------------------------
	The interface to the FX system runtime.
-----------------------------------------------------------------------------*/

/**
 * The interface to an effects system.
 */
class FFXSystemInterface : public TSharedFromThis<FFXSystemInterface>
{
	struct FFXInterfaceDeletor
	{
		void operator()(FFXSystemInterface* ObjectToDelete) const;
	};

public:
	/**
	 * Create an effects system instance.
	 */
	ENGINE_API static TSharedRef<FFXSystemInterface> Create(ERHIFeatureLevel::Type InFeatureLevel, FSceneInterface* Scene);

	/**
	 * Notify the effect system instance that it should release resources in preperation for destruction.
	 * Once the interface is marked pending kill using the interface is undefined, as the scene (for example) may be invalid.
	 * The destruction of the instance will happen later on the render thread, and may be much later due to shared references to the interface.
	 */
	ENGINE_API static void MarkPendingKill(FFXSystemInterface* FXSystem);

	/**
	 * Queue Destroy the gpu simulation on the render thread
	 */
	ENGINE_API static void QueueDestroyGPUSimulation(FFXSystemInterface* FXSystem);

	/**
	 * Register a custom FX system implementation.
	 */
	ENGINE_API static void RegisterCustomFXSystem(const FName& InterfaceName, const FCreateCustomFXSystemDelegate& InCreateDelegate);

	/**
	 * Unregister a custom FX system implementation.
	 */
	ENGINE_API static void UnregisterCustomFXSystem(const FName& InterfaceName);

	/**
	 * Return the interface bound to the given name.
	 */
	virtual FFXSystemInterface* GetInterface(const FName& InName) { return nullptr; };

	/**
	 * Gamethread callback when MarkPendingKill is called, allows to clean up references.
	 * Once marked pending kill the interface can release resources and is in an undefined state.
	 */
	virtual void OnMarkPendingKill() { bIsPendingKill = true; }

	/**
	 * Gamethread callback when destroy gets called, allows to clean up references.
	 */
	virtual void DestroyGPUSimulation() { }

	/**
	 * Tick the effects system.
	 * @param DeltaSeconds The number of seconds by which to step simulations forward.
	 */
	virtual void Tick(UWorld* World, float DeltaSeconds) = 0;

	UE_DEPRECATED(5.7, "Destroy is no longer supported, please update your code call MarkPendingKill")
	ENGINE_API static void Destroy(FFXSystemInterface* FXSystem);

protected:
	UE_DEPRECATED(5.7, "OnDestroy is no longer supported, please update your code call/implement OnMarkPendingKill")
	virtual void OnDestroy() final {}
public:

#if WITH_EDITOR
	/**
	 * Suspend the FX system. This will cause internal state to be released.
	 * Has no effect if the system was already suspended.
	 */
	virtual void Suspend() = 0;

	/**
	 * Resume the FX system.
	 * Has no effect if the system was not suspended.
	 */
	virtual void Resume() = 0;
#endif // #if WITH_EDITOR

	/**
	 * Draw desired debug information related to the effects system.
	 * @param Canvas The canvas on which to draw.
	 */
	virtual void DrawDebug(FCanvas* Canvas) = 0;

	/**
	 * Does the FX system require DrawDebug_RenderThread to be called at all?
	 * There is cost to enabling this so only return true when we have something to present.
	 */
	virtual bool ShouldDebugDraw_RenderThread() const { return false; }

	/**
	 * Draw desired debug information related to the effects system on the render thread.
	 * This is called at the end of the rendering when
	 * @param GraphBuilder Render Graph Builder to use
	 * @param View The view we are rendering for
	 * @param Output The output buffer information
	 */
	virtual void DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FScreenPassRenderTarget& Output) {}
	
	UE_DEPRECATED(5.3, "Passing FViewInfo is deprecated in favor of FSceneView. This function will do nothing.")
	inline void DrawDebug_RenderThread(FRDGBuilder& GraphBuilder, const FViewInfo& View, const struct FScreenPassRenderTarget& Output) {}

	/**
	 * Call to handle debug drawing to the scene (i.e. where depth is available)
	 */
	virtual void DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth) {}

	UE_DEPRECATED(5.3, "Passing FViewInfo is deprecated in favor of FSceneView. This function will do nothing.")
	inline void DrawSceneDebug_RenderThread(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth) {}

	/**
	 * Add a vector field to the FX system.
	 * @param VectorFieldComponent The vector field component to add.
	 */
	virtual void AddVectorField(UVectorFieldComponent* VectorFieldComponent) = 0;

	/**
	 * Remove a vector field from the FX system.
	 * @param VectorFieldComponent The vector field component to remove.
	 */
	virtual void RemoveVectorField(UVectorFieldComponent* VectorFieldComponent) = 0;

	/**
	 * Update a vector field registered with the FX system.
	 * @param VectorFieldComponent The vector field component to update.
	 */
	virtual void UpdateVectorField(UVectorFieldComponent* VectorFieldComponent) = 0;

	/**
	 * Notification from the renderer that it is about to perform visibility
	 * checks on FX belonging to this system.
	 */
	virtual void PreInitViews(FRDGBuilder& GraphBuilder, bool bAllowGPUParticleUpdate, const TArrayView<const FSceneViewFamily*>& ViewFamilies, const FSceneViewFamily* CurrentFamily) = 0;

	virtual void PostInitViews(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, bool bAllowGPUParticleUpdate) = 0;

	UE_DEPRECATED(5.3, "Passing an array of FViewInfo is deprecated in favor of FSceneView. This function will do nothing.")
	inline void PostInitViews(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo> Views, bool bAllowGPUParticleUpdate) {}

	virtual bool UsesGlobalDistanceField() const = 0;

	virtual bool UsesDepthBuffer() const = 0;

	virtual bool RequiresEarlyViewUniformBuffer() const = 0;

	virtual bool RequiresRayTracingScene() const = 0;

	/**
	 * Notification from the renderer that it is about to draw FX belonging to
	 * this system.
	 */
	virtual void PreRender(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) = 0;

	UE_DEPRECATED(5.3, "Passing an array of FViewInfo is deprecated in favor of FSceneView. This function will do nothing.")
	inline void PreRender(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo> Views, bool bAllowGPUParticleUpdate) {}

	/**
	 * Notification from the renderer that opaque primitives have rendered.
	 */
	virtual void PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstStridedView<FSceneView> Views, FSceneUniformBuffer &SceneUniformBuffer, bool bAllowGPUParticleUpdate) = 0;

	UE_DEPRECATED(5.3, "Passing an array of FViewInfo is deprecated in favor of FSceneView. This function will do nothing.")
	inline void PostRenderOpaque(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo> Views, bool bAllowGPUParticleUpdate) {}

	bool IsPendingKill() const { return bIsPendingKill; }

	/** Get the shared SortManager, used in the rendering loop to call FGPUSortManager::OnPreRender() and FGPUSortManager::OnPostRenderOpaque() */
	virtual FGPUSortManager* GetGPUSortManager() const = 0;

	virtual void SetSceneTexturesUniformBuffer(const TUniformBufferRef<FSceneTextureUniformParameters>& InSceneTexturesUniformParams) {}

	inline FSceneInterface* GetSceneInterface() const { return SceneInterface; }
	inline void SetSceneInterface(FSceneInterface* InSceneInterface) { SceneInterface = InSceneInterface; }

	inline FScene* GetScene()const { return Scene; }
	inline void SetScene(FScene* InScene) { Scene = InScene; }

protected:
	
	friend class FFXSystemSet;

	/** By making the destructor protected, an instance must be destroyed via FFXSystemInterface::Destroy. */
	virtual ~FFXSystemInterface() = default;

private:

	bool bIsPendingKill = false;

	FSceneInterface* SceneInterface = nullptr;
	FScene* Scene = nullptr;

	static TMap<FName, FCreateCustomFXSystemDelegate> CreateCustomFXDelegates;
};

/*-----------------------------------------------------------------------------
	FX resource management.
-----------------------------------------------------------------------------*/

/**
 * Allocates memory to hold GPU sprite resources and begins the resource
 * initialization process.
 * @param InResourceData The data with which to create resources.
 * @returns a pointer to sprite resources.
 */
FGPUSpriteResources* BeginCreateGPUSpriteResources(const FGPUSpriteResourceData& InResourceData);

/**
 * Updates GPU sprite resources.
 * @param Resources			Sprite resources to update.
 * @param InResourceData	Data with which to update resources.
 */
void BeginUpdateGPUSpriteResources(FGPUSpriteResources* Resources, const FGPUSpriteResourceData& InResourceData);

/**
 * Begins the process of releasing GPU sprite resources. Memory will be freed
 * during this time and the pointer should be considered invalid after this call.
 * @param Resources The resources to be released.
 */
void BeginReleaseGPUSpriteResources(FGPUSpriteResources* Resources);

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneView.h"
#endif
