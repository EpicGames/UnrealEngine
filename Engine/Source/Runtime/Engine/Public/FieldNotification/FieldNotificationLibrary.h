// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "FieldNotificationId.h"

#include "FieldNotificationLibrary.generated.h"


/**
 * The Field Notification system allows a user to know when a property value is modified at runtime (note that it can be a function return value).
 * The class needs to implement the INotifyFieldValueChanged interface.
 * The property' setter  usually follows this pattern "if (new value != old value) assign the new value; broadcast that the value changed;".
 * The Blueprint implementation handles that setter logic automatically with SetPropertyValueAndBroadcast.
 * When a property value is modified by replication, the RepNotify will call BroadcastFieldValueChanged.
 * A function can also be a Field Notify. The function needs to be const and return a single value.
 */
UCLASS(MinimalAPI)
class UFieldNotificationLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Broadcast that the Field value changed. */
	UFUNCTION(BlueprintCallable, Category = "FieldNotification", meta = (FieldNotifyInterfaceParam="Object", DisplayName = "Broadcast Field Value Changed"))
	static void BroadcastFieldValueChanged(UObject* Object, FFieldNotificationId FieldId);

	/** Broadcast that a list of Field values changed. */
	UFUNCTION(BlueprintCallable, Category = "FieldNotification", meta = (FieldNotifyInterfaceParam = "Object", DisplayName = "Broadcast Fields Value Changed"))
	static void BroadcastFieldsValueChanged(UObject* Object, TArray<FFieldNotificationId> FieldIds);

	UFUNCTION(BlueprintCallable, CustomThunk, DisplayName = "Set w/ Broadcast", Category = "FieldNotification", meta = (CustomStructureParam = "NewValue,OldValue", FieldNotifyInterfaceParam = "Object", BlueprintInternalUseOnly = "true"))
	static bool SetPropertyValueAndBroadcast(bool NewValueByRef, UPARAM(ref) const int32& OldValue, UPARAM(ref) const int32& NewValue, UObject* Object, UObject* NetOwner, bool bHasLocalRepNotify, bool bShouldFlushDormancyOnSet, bool bIsNetProperty);

	UFUNCTION(BlueprintCallable, CustomThunk, DisplayName = "Set w/ Broadcast", Category = "FieldNotification", meta = (CustomStructureParam = "NewValue,OldValue", FieldNotifyInterfaceParam = "Object", BlueprintInternalUseOnly = "true"))
	static bool SetPropertyValueAndBroadcastFields(bool NewValueByRef, UPARAM(ref) const int32& OldValue, UPARAM(ref) const int32& NewValue, UObject* Object, UObject* NetOwner, bool bHasLocalRepNotify, bool bShouldFlushDormancyOnSet, bool bIsNetProperty, TArray<FFieldNotificationId> ExtraFieldIds);

private:
	DECLARE_FUNCTION(execSetPropertyValueAndBroadcast);
	DECLARE_FUNCTION(execSetPropertyValueAndBroadcastFields);
};
