// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Scene.h"
#include "BlendableManager.h"

struct FImagePixelPipe;

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;

/** All blended postprocessing in one place, non lerpable data is stored in non merged form */
class FFinalPostProcessSettings : public FPostProcessSettings
{
public:
	FFinalPostProcessSettings()
	: VisualizeCalibrationColorMaterial(NULL)
	, VisualizeCalibrationGrayscaleMaterial(NULL)
	, VisualizeCalibrationCustomMaterial(NULL)
	, HighResScreenshotMaterial(NULL)
	, HighResScreenshotMaskMaterial(NULL)
	, HighResScreenshotCaptureRegionMaterial(NULL)
	, bBufferVisualizationDumpRequired(false)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, bBufferVisualizationOverviewTargetIsSelected(false)
#endif
	{
		// to avoid reallocations we reserve some 
		ContributingCubemaps.Reserve(8);
		ContributingLUTs.Reserve(8);

		SetLUT(0);
	}
	
	struct FCubemapEntry
	{
		// 0..
		FLinearColor AmbientCubemapTintMulScaleValue;
		// can be 0
		class UTexture* AmbientCubemap;

		FCubemapEntry() :
			AmbientCubemapTintMulScaleValue(FLinearColor(0, 0, 0, 0)),
			AmbientCubemap(NULL)
		{}
	};

	struct FLUTBlenderEntry
	{
		// 0..1
		float Weight;
		// can be 0
		class UTexture* LUTTexture;
	};

	// Updated existing one or creates a new one
	// this allows to combined volumes for fading between them but also to darken/disable volumes in some areas
	void UpdateEntry(const FCubemapEntry &Entry, float Weight)
	{
		bool Existing = false;
		// Always verify the index is valid since elements can be removed!
		for(int32 i = 0; ContributingCubemaps.IsValidIndex(i); ++i)
		{
			FCubemapEntry& Local = ContributingCubemaps[i];
			
			if(Local.AmbientCubemap == Entry.AmbientCubemap)
			{
				Local.AmbientCubemapTintMulScaleValue = FMath::Lerp(Local.AmbientCubemapTintMulScaleValue, Entry.AmbientCubemapTintMulScaleValue, Weight);
				Existing = true;
			}
			else
			{
				Local.AmbientCubemapTintMulScaleValue *= 1.0f - Weight;
			}

			if(Local.AmbientCubemapTintMulScaleValue.IsAlmostBlack())
			{
				ContributingCubemaps.RemoveAt(i, EAllowShrinking::No);
				i--; // Maintain same index in the loop after loop increment since we removed an element.
				continue; // Not strictly necessary but protect against future code using i incorrectly.
			}
		}

		if( !Existing )
		{
			// We didn't find the entry previously, so we need to Lerp up from zero.
			FCubemapEntry WeightedEntry = Entry;
			WeightedEntry.AmbientCubemapTintMulScaleValue *= Weight;
			if(!WeightedEntry.AmbientCubemapTintMulScaleValue.IsAlmostBlack()) // Only bother to add it if the lerped value isn't near-black.
			{
				ContributingCubemaps.Push(WeightedEntry);
			}
		}
	}

	// @param InTexture - must not be 0
	// @param Weight - 0..1
	void LerpTo(UTexture* InTexture, float Weight)
	{
		check(InTexture);
		check(Weight >= 0.0f && Weight <= 1.0f);

		if(Weight > 254.0f/255.0f || !ContributingLUTs.Num())
		{
			SetLUT(InTexture);
			return;
		}

		for(uint32 i = 0; i < (uint32)ContributingLUTs.Num(); ++i)
		{
			ContributingLUTs[i].Weight *= 1.0f - Weight;
		}

		uint32 ExistingIndex = FindIndex(InTexture);
		if(ExistingIndex == 0xffffffff)
		{
			PushLUT(InTexture, Weight);
		}
		else
		{
			ContributingLUTs[ExistingIndex].Weight += Weight;
		}
	}
	
	/**
	 * add a LUT(look up table) to the ones that are blended together
	 * @param Texture - can be 0
	 * @param Weight - 0..1
	 */
	void PushLUT(UTexture* Texture, float Weight)
	{
		check(Weight >= 0.0f && Weight <= 1.0f);

		FLUTBlenderEntry Entry;

		Entry.LUTTexture = Texture;
		Entry.Weight = Weight;

		ContributingLUTs.Add(Entry);
	}

	/** @return 0xffffffff if not found */
	uint32 FindIndex(UTexture* Tex) const
	{
		for(uint32 i = 0; i < (uint32)ContributingLUTs.Num(); ++i)
		{
			if(ContributingLUTs[i].LUTTexture == Tex)
			{
				return i;
			}
		}

		return 0xffffffff;
	}

	void SetLUT(UTexture* Texture)
	{
		// intentionally no deallocations
		ContributingLUTs.Reset();

		PushLUT(Texture, 1.0f);
	}

	// was not merged during blending unlike e.g. BloomThreshold 
	TArray<FCubemapEntry> ContributingCubemaps;
	TArray<FLUTBlenderEntry> ContributingLUTs;

	// List of materials to use in the buffer visualization overview
	TArray<UMaterialInterface*> BufferVisualizationOverviewMaterials;

	// Material to use for calibration color visualization
	UMaterialInterface* VisualizeCalibrationColorMaterial;

	// Material to use for debug grayscale visualization
	UMaterialInterface* VisualizeCalibrationGrayscaleMaterial;

	// Material to use for debug custom post-processing visualization
	UMaterialInterface* VisualizeCalibrationCustomMaterial;

	// Material to use for rendering high res screenshot with mask. Post process expects this material to be set all the time.
	UMaterialInterface* HighResScreenshotMaterial;

	// Material to use for rendering just the high res screenshot mask. Post process expects this material to be set all the time.
	UMaterialInterface* HighResScreenshotMaskMaterial;

	// Material to use for rendering the high resolution screenshot capture region. Post processing only draws the region if this material is set.
	UMaterialInstanceDynamic* HighResScreenshotCaptureRegionMaterial;

	// Current buffer visualization dumping settings
	bool bBufferVisualizationDumpRequired;
	FString BufferVisualizationDumpBaseFilename;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Optional selected buffer in the buffer visualization overview
	bool bBufferVisualizationOverviewTargetIsSelected;
	FString BufferVisualizationOverviewSelectedTargetMaterialName;
#endif

	/**
	 * A map of buffer visualization material names to visualization pipes onto which each intermediate render target will be pushed, if set.
	 * Will always be used if set, regardless of r.BufferVisualizationDumpFrames
	 */
	TMap<FName, TSharedPtr<FImagePixelPipe, ESPMode::ThreadSafe>> BufferVisualizationPipes;

	// Maintains a container with IBlendableInterface objects and their data
	FBlendableManager BlendableManager;
};
