// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/Paths.h"
#include "Misc/Optional.h"

class ULevel;
class UActorFolder;
class UWorld;

struct FFolder
{
#if WITH_EDITOR
	typedef FObjectKey FRootObject;

	// Only used by containers
	FFolder()
		: bPathInitialized(true)
		, Path(GetEmptyPath())
		, RootObject(GetInvalidRootObject())
	{}

	FFolder(const FRootObject& InRootObject, const FName& InPath = GetEmptyPath())
		: bPathInitialized(true)
		, Path(InPath)
		, RootObject(InRootObject)
	{
		if (Path != GetEmptyPath())
		{
			TStringBuilder<512> Result;
			TArray<FString> Parts;
			Path.ToString().ParseIntoArray(Parts, TEXT("/"), true);
			for (FString& Part : Parts)
			{
				if (Result.Len())
				{
					Result += TEXT("/");
				}
				Result += Part.TrimStartAndEnd();
			}
			Path = *Result;
		}
	}

	FFolder(const FRootObject& InRootObject, const FGuid& InActorFolderGuid)
		: bPathInitialized(false)
		, Path(NAME_None)
		, RootObject(InRootObject)
		, ActorFolderGuid(InActorFolderGuid)
	{
		check(ActorFolderGuid.IsValid());
	}

	inline static UObject* GetRootObjectPtr(const FRootObject& InRootObject)
	{
		return InRootObject.ResolveObjectPtr();
	}

	inline static bool IsRootObjectValid(const FRootObject& Key)
	{
		return Key != GetInvalidRootObject();
	}

	inline static FName GetEmptyPath()
	{
		return NAME_None;
	}

	inline bool IsRootObjectValid() const
	{
		return FFolder::IsRootObjectValid(RootObject);
	}

	inline ULevel* GetRootObjectAssociatedLevel() const
	{
		return FFolder::GetRootObjectAssociatedLevel(RootObject);
	}

	inline bool IsRootObjectPersistentLevel() const
	{
		return FFolder::IsRootObjectPersistentLevel(RootObject);
	}

	inline bool IsValid() const
	{
		return IsRootObjectValid();
	}

	inline bool IsChildOf(const FFolder& InParent) const
	{
		if (RootObject != InParent.RootObject)
		{
			return false;
		}
		return PathIsChildOf(GetPath(), InParent.GetPath());
	}

	inline bool IsNone() const
	{
		return GetPath().IsNone();
	}

	inline const FRootObject& GetRootObject() const
	{
		return RootObject;
	}

	inline UObject* GetRootObjectPtr() const
	{
		return FFolder::GetRootObjectPtr(RootObject);
	}

	inline const FGuid& GetActorFolderGuid() const
	{
		return ActorFolderGuid;
	}

	inline FName GetLeafName() const
	{
		FName PathLocal = GetPath();
		FString PathString = PathLocal.ToString();
		int32 LeafIndex = 0;
		if (PathString.FindLastChar('/', LeafIndex))
		{
			return FName(*PathString.RightChop(LeafIndex + 1));
		}
		else
		{
			return PathLocal;
		}
	}

	inline bool operator == (const FFolder& InOther) const
	{
		return (RootObject == InOther.RootObject) && (GetPath() == InOther.GetPath()); // don't check for ActorFolderGuid as it's only used as an accelerator
	}

	inline bool operator != (const FFolder& InOther) const
	{
		return !operator==(InOther);
	}
	
	inline FString ToString() const
	{ 
		return GetPath().ToString();
	}

	inline friend FArchive& operator<<(FArchive& Ar, FFolder& Folder)
	{
		check(!Ar.IsPersistent());
		return Ar << Folder.bPathInitialized << Folder.Path << Folder.RootObject << Folder.ActorFolderGuid;
	}

	ENGINE_API const FName GetPath() const;
	ENGINE_API FFolder GetParent() const;
	ENGINE_API UActorFolder* GetActorFolder() const;

	// Helpers methods
	static ENGINE_API const FFolder& GetInvalidFolder();
	static ENGINE_API const FRootObject& GetInvalidRootObject();
	static ENGINE_API bool IsRootObjectPersistentLevel(const FRootObject& Key);
	static ENGINE_API TOptional<FRootObject> GetOptionalFolderRootObject(const ULevel* InLevel);
	static ENGINE_API ULevel* GetRootObjectAssociatedLevel(const FRootObject& Key);
	static ENGINE_API FFolder GetWorldRootFolder(UWorld* InWorld);
	static ENGINE_API bool GetFolderPathsAndCommonRootObject(const TArray<FFolder>& InFolders, TArray<FName>& OutFolders, FRootObject& OutCommonRootObject);

private:
	bool PathIsChildOf(const FString& InPotentialChild, const FString& InParent) const
	{
		const int32 ParentLen = InParent.Len();

		// If parent is empty and child isn't, consider that path is child of parent
		if ((InPotentialChild.Len() > 0) && (ParentLen == 0))
		{
			return true;
		}

		return
			InPotentialChild.Len() > ParentLen &&
			InPotentialChild[ParentLen] == '/' &&
			InPotentialChild.Left(ParentLen) == InParent;
	}

	bool PathIsChildOf(const FName& InPotentialChild, const FName& InParent) const
	{
		return PathIsChildOf(InPotentialChild.ToString(), InParent.ToString());
	}

	mutable bool bPathInitialized;
	mutable FName Path;
	FRootObject RootObject;
	mutable FGuid ActorFolderGuid; // Optional : Used to find Level's ActorFolder faster than by using the path

	friend inline uint32 GetTypeHash(const FFolder& InFolder)
	{
		return HashCombine(GetTypeHash(InFolder.GetPath()), GetTypeHash(InFolder.GetRootObject()));
	}
#endif
};
