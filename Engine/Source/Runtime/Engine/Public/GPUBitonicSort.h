// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUBitonicSort.h: Interface for bitonic sorting buffers on the GPU.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

struct FGPUSortBuffers;

/**
 * Sort a buffer on the GPU using a bitonic merge sort.
 *
 * @param RHICmdList    The command list used to issue dispatches.
 * @param SortBuffers   The buffer to sort including required views and a ping-pong location of appropriate size.
 * @param Count         How many items in the buffer need to be sorted.
 * @param FeatureLevel  The current feature level.
 * @returns The index of the buffer containing sorted results.
 */
ENGINE_API int32 BitonicSortGPUBuffers(
	FRHICommandList& RHICmdList,
	FGPUSortBuffers SortBuffers,
	int32 BufferIndex,
	int32 Count,
	ERHIFeatureLevel::Type FeatureLevel);
