// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright (C) Microsoft. All rights reserved.

/*=============================================================================
	GPUSkinCache.h: Performs skinning on a compute shader into a buffer to avoid vertex buffer skinning.
=============================================================================*/

// Requirements
// * Compute shader support (with Atomics)
// * Project settings needs to be enabled (r.SkinCache.CompileShaders)
// * feature need to be enabled (r.SkinCache.Mode)

// Features
// * Skeletal mesh, 4 / 8 weights per vertex, 16/32 index buffer
// * Supports Morph target animation (morph target blending is not done by this code)
// * Saves vertex shader computations when we render an object multiple times (EarlyZ, velocity, shadow, BasePass, CustomDepth, Shadow masking)
// * Fixes velocity rendering (needed for MotionBlur and TemporalAA) for WorldPosOffset animation and morph target animation
// * RecomputeTangents results in improved tangent space for WorldPosOffset animation and morph target animation
// * Velocity Rendering for MotionBlur and TemporalAA (test Velocity in BasePass)
// * r.SkinCache.Mode and r.SkinCache.RecomputeTangents can be toggled at runtime

// TODO:
// * Quality/Optimization: increase TANGENT_RANGE for better quality or accumulate two components in one 32bit value
// * Bug: UpdateMorphVertexBuffer needs to handle SkinCacheObjects that have been rejected by the SkinCache (e.g. because it was running out of memory)
// * Refactor: Unify the 3 compute shaders to use the same C++ setup code for the variables
// * Optimization: Dispatch calls can be merged for better performance, stalls between Dispatch calls can be avoided (DX11 back door, DX12, console API)
// * Feature: Cloth is not supported yet (Morph targets is a similar code)
// * Feature: Support Static Meshes ?

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderGraphDefinitions.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"
#include "GPUSkinPublicDefs.h"
#include "VertexFactory.h"
#include "CanvasTypes.h"
#include "CachedGeometry.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/SkinnedAssetCommon.h"

enum class EGPUSkinCacheDispatchFlags : uint8;
enum class EGPUSkinCacheBufferBits : uint8;

class FGPUSkinPassthroughVertexFactory;
class FGPUBaseSkinVertexFactory;
class FMorphVertexBuffer;
class FSkeletalMeshObject;
class FSkeletalMeshLODRenderData;
class FSkeletalMeshObjectGPUSkin;
class FSkeletalMeshVertexClothBuffer;
class FVertexOffsetBuffers;
class FDynamicMeshBoundsBuffer;
struct FClothSimulData;
struct FSkelMeshRenderSection;
struct FVertexBufferAndSRV;
struct FRayTracingGeometrySegment;
struct FGlobalShaderPermutationParameters;

UE_DEPRECATED(5.6, "Use GPUSkinCacheStoreDuplicatedVertices instead to check if duplicate vertices need to be stored.")
extern ENGINE_API bool GPUSkinCacheNeedsDuplicatedVertices();

extern ENGINE_API bool GPUSkinCacheStoreDuplicatedVertices();

extern ENGINE_API ESkinCacheDefaultBehavior GetSkinCacheDefaultBehavior();

UE_DEPRECATED(5.8, "Use FGPUSkinCache::IsGPUSkinCacheEnabled()" )
extern ENGINE_API int32 GEnableGPUSkinCache;

class FGPUSkinCacheEntry;

struct FClothSimulEntry
{
	FVector3f Position;
	FVector3f Normal;

	/**
	 * Serializer
	 *
	 * @param Ar - archive to serialize with
	 * @param V - vertex to serialize
	 * @return archive that was used
	 */
	friend FArchive& operator<<(FArchive& Ar, FClothSimulEntry& V)
	{
		Ar << V.Position
		   << V.Normal;
		return Ar;
	}
};

enum class EGPUSkinCacheEntryMode
{
	Raster,
	RayTracing
};

BEGIN_SHADER_PARAMETER_STRUCT(FDynamicMeshBoundsShaderParameters, ENGINE_API)
	SHADER_PARAMETER(uint32, DynamicMeshBoundsMax)
	SHADER_PARAMETER_SRV(StructuredBuffer<int4>, DynamicMeshBoundsBuffer)
END_SHADER_PARAMETER_STRUCT()

// Get the dynamic mesh bounds assigned by GPU skin cache on this graph builder, or dummy values otherwise.
ENGINE_API FDynamicMeshBoundsShaderParameters GetDynamicMeshBoundsShaderParameters(FRDGBuilder& GraphBuilder);

// Set required shader defines for any shader using the dynamic mesh bounds (through FDynamicMeshBoundsShaderParameters), 
// without this the operations are compiled out as well as if r.SkinCache.DynamicMeshBounds is off.
ENGINE_API void DynamicMeshBoundsModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

UE_DEPRECATED(5.7, "This is just a stub. Use the above version that takes a FGlobalShaderPermutationParameters argument.")
inline void DynamicMeshBoundsModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment) { };


class FGPUSkinCache
{
public:
	enum ESkinCacheInitSettings
	{
		// max 256 bones as we use a byte to index
		MaxUniformBufferBones = 256,
		// Controls the output format on GpuSkinCacheComputeShader.usf
		RWTangentXOffsetInFloats = 0,	// Packed U8x4N
		RWTangentZOffsetInFloats = 1,	// Packed U8x4N

		// 3 ints for normal, 3 ints for tangent, 1 for orientation = 7, rounded up to 8 as it should result in faster math and caching
		IntermediateAccumBufferNumInts = 8,
	};

	UE_DEPRECATED(5.7, "SkinCache no longer requires a memory limit")
	FGPUSkinCache(ERHIFeatureLevel::Type InFeatureLevel, bool bInRequiresMemoryLimit, UWorld* InWorld)
		: FGPUSkinCache(InFeatureLevel, InWorld)
	{}

	ENGINE_API FGPUSkinCache(ERHIFeatureLevel::Type InFeatureLevel, UWorld* InWorld);
	ENGINE_API ~FGPUSkinCache();

	static void UpdateSkinWeightBuffer(FGPUSkinCacheEntry* Entry);
	static void SetEntryGPUSkin(FGPUSkinCacheEntry* Entry, FSkeletalMeshObject* Skin);

	struct FProcessEntrySection
	{
		FGPUBaseSkinVertexFactory* SourceVertexFactory = nullptr;
		const FSkelMeshRenderSection* Section = nullptr;
		int32 SectionIndex = 0;
		const FClothSimulData* ClothSimulationData = nullptr;
		FMatrix44f ClothToLocal = FMatrix44f::Identity;
	};

	struct FProcessEntryInputs
	{
		EGPUSkinCacheEntryMode Mode;
		TConstArrayView<FProcessEntrySection> Sections;
		FSkeletalMeshObject* Skin = nullptr;
		FGPUSkinPassthroughVertexFactory* TargetVertexFactory = nullptr;
		FMorphVertexBuffer* MorphVertexBuffer = nullptr;
		const FSkeletalMeshVertexClothBuffer* ClothVertexBuffer = nullptr;
		FVector3f ClothWorldScale = FVector3f(1.0f, 1.0f, 1.0f);
		float ClothBlendWeight = 0.0f;
		uint32 CurrentRevisionNumber = 0;
		int32 LODIndex = 0;
		bool bRecreating = false;
	};

	void ProcessEntry(FRHICommandList& RHICmdList, const FProcessEntryInputs& Inputs, FGPUSkinCacheEntry*& InOutEntry);

	/** Dequeue a skin cache entry from the pending skin cache dispatches */
	static void Dequeue(FGPUSkinCacheEntry* SkinCacheEntry);
	static void Release(FGPUSkinCacheEntry*& SkinCacheEntry);

	static bool IsEntryValid(FGPUSkinCacheEntry* SkinCacheEntry, int32 Section);
	static FColor GetVisualizationDebugColor(const FName& GPUSkinCacheVisualizationMode, FGPUSkinCacheEntry* Entry, FGPUSkinCacheEntry* RayTracingEntry, uint32 SectionIndex);
	ENGINE_API void DrawVisualizationInfoText(const FName& GPUSkinCacheVisualizationMode, FScreenMessageWriter& ScreenMessageWriter) const;

	ENGINE_API static bool IsGPUSkinCacheEnabled(ERHIFeatureLevel::Type FeatureLevel);
	static int32 GetGPUSkinCacheRecomputeTangentsMode(ERHIFeatureLevel::Type FeatureLevel);
	static bool IsGPUSkinCacheRayTracingSupported(ERHIFeatureLevel::Type FeatureLevel);

	static FRWBuffer* GetPositionBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex);
	static FRWBuffer* GetPreviousPositionBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex);
	static FRWBuffer* GetTangentBuffer(FRDGBuilder& GraphBuilder, FGPUSkinCacheEntry const* Entry, uint32 SectionIndex);

#if RHI_RAYTRACING
	void ProcessRayTracingGeometryToUpdate(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* SkinCacheEntry);
#endif // RHI_RAYTRACING
	
	ENGINE_API static ERHIPipeline GetDispatchPipeline(FRDGBuilder& GraphBuilder);

	ENGINE_API UE::Tasks::FTask Dispatch(FRDGBuilder& GraphBuilder, const UE::Tasks::FTask& PrerequisitesTask, ERHIPipeline Pipeline);

	ENGINE_API static void AddAsyncComputeSignal(FRDGBuilder& GraphBuilder);
	ENGINE_API static void AddAsyncComputeWait(FRDGBuilder& GraphBuilder);

	inline ERHIFeatureLevel::Type GetFeatureLevel() const
	{
		return FeatureLevel;
	}

	inline bool HasWork() const
	{
		return !BatchDispatches.IsEmpty();
	}

	UE_DEPRECATED(5.6, "Use Dispatch instead.")
	void DoDispatch(FRHICommandList& RHICmdList) {}

	UE_DEPRECATED(5.6, "GetUpdatedFrame is no longer used")
	static uint32 GetUpdatedFrame(FGPUSkinCacheEntry const*, uint32) { return 0; }

	UE_DEPRECATED(5.7, "Skin Cache no longer has an explicit budget.")
	uint64 GetExtraRequiredMemoryAndReset() { return 0; }

	UE_DEPRECATED(5.7, "Use FProcessEntryInputs")
	bool ProcessEntry(
		EGPUSkinCacheEntryMode Mode,
		FRHICommandList& RHICmdList, 
		FGPUBaseSkinVertexFactory* VertexFactory,
		FGPUSkinPassthroughVertexFactory* TargetVertexFactory, 
		const FSkelMeshRenderSection& BatchElement, 
		FSkeletalMeshObject* Skin,
		const FMorphVertexBuffer* MorphVertexBuffer, 
		const FSkeletalMeshVertexClothBuffer* ClothVertexBuffer, 
		const FClothSimulData* SimData,
		const FMatrix44f& ClothToLocal,
		float ClothBlendWeight, 
		FVector3f ClothScale,
		uint32 RevisionNumber, 
		int32 Section,
		int32 LOD,
		bool& bRecreating,
		FGPUSkinCacheEntry*& InOutEntry)
	{
		return false;
	}

	// Allocates a new slot for dynamic bounds for a mesh with NumSlots sections.
	ENGINE_API int32 AllocateDynamicMeshBoundsSlot(int32 NumSlots);

	// Releases the dynamic mesh bounds slot associated with a mesh.
	ENGINE_API void ReleaseDynamicMeshBoundsSlot(int32 Offset, int32 NumSlots);

	static void OnCVarUpdate();

private:
	friend FGPUSkinCacheEntry;

	struct FDispatchEntry
	{
		FGPUSkinCacheEntry* SkinCacheEntry = nullptr;
		uint32 Section = 0;
	};

	struct FSortedDispatchEntry
	{
		int32 ShaderIndex;
		int32 BatchIndex;
	};

	enum
	{
		NUM_BUFFERS = 2,
	};

	struct FSkinCacheRWBuffer;
	struct FRWBuffersAllocationInitializer;
	struct FRWBuffersAllocation;
	struct FRWBufferTracker;

	struct FTaskData;
	void DispatchPassSetup(FTaskData& TaskData);
	void DispatchPassExecute(FTaskData& TaskData);

	void TransitionBuffers(FRHICommandList& RHICmdList, TArray<FSkinCacheRWBuffer*>& Buffers, ERHIAccess ToState);
	void TransitionBufferUAVs(FRHICommandList& RHICmdList, TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator>& Transitions, const TArray<FSkinCacheRWBuffer*>& InBuffers, TArray<FRHIUnorderedAccessView*>& OutUAVs);
	void TransitionBufferUAVs(FRHICommandList& RHICmdList, const TArray<FSkinCacheRWBuffer*>& InBuffers, TArray<FRHIUnorderedAccessView*>& OutUAVs);

	TArray<FRWBuffersAllocation*> Allocations;
	TArray<FGPUSkinCacheEntry*> Entries;
	TSet<FGPUSkinCacheEntry*> PendingProcessRTGeometryEntries;
	TArray<FDispatchEntry> BatchDispatches;

	void DispatchUpdateSkinTangentsVertexPass(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* Entry, int32 SectionIndex, FSkinCacheRWBuffer*& StagingBuffer);
	void DispatchUpdateSkinTangentsTrianglePass(FRHICommandList& RHICmdList, FGPUSkinCacheEntry* Entry, int32 SectionIndex, FSkinCacheRWBuffer*& StagingBuffer);

	void Cleanup();
	static void ReleaseSkinCacheEntry(FGPUSkinCacheEntry* SkinCacheEntry);
	void InvalidateAllEntries();

	uint64 UsedMemoryInBytes = 0;
	int32 FlushCounter = 0;

	TPimplPtr<FDynamicMeshBoundsBuffer> DynamicMeshBoundsBuffer;

	// For recompute tangents, holds the data required between compute shaders
	TArray<FSkinCacheRWBuffer> StagingBuffers;
	int32 CurrentStagingBufferIndex = 0;

	ERHIFeatureLevel::Type FeatureLevel;
	UWorld* World;

	uint32 GetNextTransitionFence() { return ++TransitionFence; }
	uint32 TransitionFence = 0;

	void PrintMemorySummary() const;
	FString GetSkeletalMeshObjectName(const FSkeletalMeshObject* GPUSkin) const;
	FDebugName GetSkeletalMeshObjectDebugName(const FSkeletalMeshObject* GPUSkin) const;
};

DECLARE_STATS_GROUP(TEXT("GPU Skin Cache"), STATGROUP_GPUSkinCache, STATCAT_Advanced);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Sections Skinned"), STAT_GPUSkinCache_TotalNumChunks, STATGROUP_GPUSkinCache,);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Vertices Skinned"), STAT_GPUSkinCache_TotalNumVertices, STATGROUP_GPUSkinCache,);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Total Memory Bytes Used"), STAT_GPUSkinCache_TotalMemUsed, STATGROUP_GPUSkinCache, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Intermediate buffer for Recompute Tangents"), STAT_GPUSkinCache_TangentsIntermediateMemUsed, STATGROUP_GPUSkinCache, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Triangles for Recompute Tangents"), STAT_GPUSkinCache_NumTrianglesForRecomputeTangents, STATGROUP_GPUSkinCache, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Sections Processed"), STAT_GPUSkinCache_NumSectionsProcessed, STATGROUP_GPUSkinCache, );
