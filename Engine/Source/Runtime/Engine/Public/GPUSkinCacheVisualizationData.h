// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"

class FGPUSkinCacheVisualizationData
{
public:
	enum class FModeType : uint8
	{
		Overview,
		Memory,
		RayTracingLODOffset,
		Num
	};

	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString   ModeString;
		FName     ModeName;
		FText     ModeText;
		FText     ModeDesc;
		FModeType ModeType;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TMap<FName, FModeRecord> TModeMap;

public:

	/** Initialize the system. */
	void Initialize();

	/** Check if system was initialized. */
	inline bool IsInitialized() const { return bIsInitialized.load(std::memory_order_relaxed); }

	/** Check if visualization is active. */
	ENGINE_API bool IsActive() const;

	/** Update visualization state. */
	ENGINE_API bool Update(const FName& InViewMode);

	/** Get the display name of a named mode from the available mode map. **/
	ENGINE_API FText GetModeDisplayName(const FName& InModeName) const;

	FModeType GetModeType(const FName& InModeName) const;

	inline const TModeMap& GetModeMap() const
	{
		return ModeMap;
	}

	inline FModeType GetActiveModeType() const
	{
		return ActiveVisualizationModeType;
	}

	inline const FName& GetActiveModeName() const
	{
		return ActiveVisualizationModeName;
	}

	/** Return the console command name for enabling single mode visualization. */
	static const TCHAR* GetVisualizeConsoleCommandName()
	{
		return TEXT("r.SkinCache.Visualize");
	}

private:
	/** Internal helper function for creating the VirtualShadowMap visualization system console commands. */
	void ConfigureConsoleCommand();

	void AddVisualizationMode(
		const TCHAR* ModeString,
		const FText& ModeText,
		const FText& ModeDesc,
		const FModeType ModeType
	);

	void SetActiveMode(FModeType ModeType, const FName& ModeName);

private:
	UE::FMutex Mutex;

	/** The name->mode mapping table */
	TModeMap ModeMap;

	FModeType ActiveVisualizationModeType = FModeType::Num;
	FName ActiveVisualizationModeName = NAME_None;

	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;

	/** Flag indicating if system is initialized. **/
	std::atomic_bool bIsInitialized = { false };
};

ENGINE_API FGPUSkinCacheVisualizationData& GetGPUSkinCacheVisualizationData();
