// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSkinPublicDefs.h: Public definitions for GPU skinning.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/** Max number of bone influences that a single skinned vert can have. */
#define MAX_TOTAL_INFLUENCES		12

/** Number of bone influences when using extra bone influences. */
#define EXTRA_BONE_INFLUENCES		8

/** Max number of bone influences that a single skinned vert can have per vertex stream. */
#define MAX_INFLUENCES_PER_STREAM	4

/** max number of clothing vertices for uniform buffer which is up to 64kb */
#define MAX_APEXCLOTH_VERTICES_FOR_UB 2048 
/** but maximum number will be 65536 when using vertex buffer */
#define MAX_APEXCLOTH_VERTICES_FOR_VB 65536

namespace SkinningTools
{
	/**
	 * Returns the bone weight index to use for rigid skinning
	 *
	 * @return Bone weight index to use
	 */
	inline int32 GetRigidInfluenceIndex()
	{
		// The index of the rigid influence in the uint8[4] of influences for a skinned vertex, accounting for byte-swapping. 
#if PLATFORM_LITTLE_ENDIAN
		return 0;
#else
		return 3;
#endif
	}
}

// Number of frames buffered
#define GPUSKINCACHE_FRAMES 3
