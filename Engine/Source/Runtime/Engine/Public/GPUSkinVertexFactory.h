// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSkinVertexFactory.h: GPU skinning vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "RenderResource.h"
#include "BoneIndices.h"
#include "GPUSkinPublicDefs.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "LocalVertexFactory.h"
#include "ResourcePool.h"
#include "Matrix3x4.h"
#include "SkeletalMeshTypes.h"
#include "SkinningDefinitions.h"

template <class T> class TConsoleVariableData;

#define SET_BONE_DATA(B, X) B.SetMatrixTranspose(X)

/** Shared data & implementation for the different types of pool */
class FSharedPoolPolicyData
{
public:
	/** Buffers are created with a simple byte size */
	typedef uint32 CreationArguments;
	enum
	{
		NumSafeFrames = 4, /** Number of frames to leaves buffers before reclaiming/reusing */
		NumPoolBucketSizes = 18, /** Number of pool buckets */
		NumToDrainPerFrame = 10, /** Max. number of resources to cull in a single frame */
		CullAfterFramesNum = 30 /** Resources are culled if unused for more frames than this */
	};
	
	/** Get the pool bucket index from the size
	 * @param Size the number of bytes for the resource 
	 * @returns The bucket index.
	 */
	uint32 GetPoolBucketIndex(uint32 Size);
	
	/** Get the pool bucket size from the index
	 * @param Bucket the bucket index
	 * @returns The bucket size.
	 */
	uint32 GetPoolBucketSize(uint32 Bucket);
	
private:
	/** The bucket sizes */
	static uint32 BucketSizes[NumPoolBucketSizes];
};

/** Struct to pool the vertex buffer & SRV together */
struct FVertexBufferAndSRV
{
	FVertexBufferAndSRV() = default;

	FVertexBufferAndSRV(FVertexBufferAndSRV&& RHS)
	{
		*this = MoveTemp(RHS);
	}

	FVertexBufferAndSRV& operator=(FVertexBufferAndSRV&& RHS)
	{
		VertexBufferRHI = MoveTemp(RHS.VertexBufferRHI);
		VertexBufferSRV = MoveTemp(RHS.VertexBufferSRV);
		Size = RHS.Size;
		return *this;
	}

	FVertexBufferAndSRV(const FVertexBufferAndSRV& RHS)
	{
		*this = RHS;
	}

	FVertexBufferAndSRV& operator=(const FVertexBufferAndSRV& RHS)
	{
		VertexBufferRHI = RHS.VertexBufferRHI;
		VertexBufferSRV = RHS.VertexBufferSRV;
		Size = RHS.Size;
		return *this;
	}

	void SafeRelease()
	{
		VertexBufferRHI.SafeRelease();
		VertexBufferSRV.SafeRelease();
		Size = 0;
	}

	FBufferRHIRef VertexBufferRHI;
	FShaderResourceViewRHIRef VertexBufferSRV;
	uint32 Size = 0;
};

/**
 * Helper function to test whether the buffer is valid.
 * @param Buffer Buffer to test
 * @returns True if the buffer is valid otherwise false
 */
inline bool IsValidRef(const FVertexBufferAndSRV& Buffer)
{
	return IsValidRef(Buffer.VertexBufferRHI) && IsValidRef(Buffer.VertexBufferSRV);
}

/** The policy for pooling bone vertex buffers */
class FBoneBufferPoolPolicy : public FSharedPoolPolicyData
{
public:
	enum
	{
		NumSafeFrames = FSharedPoolPolicyData::NumSafeFrames,
		NumPoolBuckets = FSharedPoolPolicyData::NumPoolBucketSizes,
		NumToDrainPerFrame = FSharedPoolPolicyData::NumToDrainPerFrame,
		CullAfterFramesNum = FSharedPoolPolicyData::CullAfterFramesNum
	};
	/** Creates the resource 
	 * @param Args The buffer size in bytes.
	 */
	FVertexBufferAndSRV CreateResource(FRHICommandListBase& RHICmdList, FSharedPoolPolicyData::CreationArguments Args);
	
	/** Gets the arguments used to create resource
	 * @param Resource The buffer to get data for.
	 * @returns The arguments used to create the buffer.
	 */
	FSharedPoolPolicyData::CreationArguments GetCreationArguments(const FVertexBufferAndSRV& Resource);
	
	/** Frees the resource
	 * @param Resource The buffer to prepare for release from the pool permanently.
	 */
	void FreeResource(const FVertexBufferAndSRV& Resource);
};

/** A pool for vertex buffers with consistent usage, bucketed for efficiency. */
class FBoneBufferPool : public TRenderResourcePool<FVertexBufferAndSRV, FBoneBufferPoolPolicy, FSharedPoolPolicyData::CreationArguments>
{
public:
	/** Destructor */
	virtual ~FBoneBufferPool();

	uint32 GetBufferSize(uint32 NumBones);

public: // From FTickableObjectRenderThread
	virtual TStatId GetStatId() const override;
};

/** A unified bone buffer shared between vertex factories. */
class FUnifiedBoneBuffer
{
public:
	FUnifiedBoneBuffer(TConstArrayView<FBoneIndexType> BoneMap);
	~FUnifiedBoneBuffer();

	void Allocate(FRHICommandList& RHICmdList, bool bPrevious);

	void Update(FRHICommandList& RHICmdList, const FName& AssetPathName, TConstArrayView<FMatrix44f> ReferenceToLocalMatrices, bool bPrevious);

	void FlipBuffers()
	{
		CurrentIndex = 1 - CurrentIndex;
	}

	FVertexBufferAndSRV& GetBuffer(bool bPrevious)
	{
		return Buffers[CurrentIndex ^ (uint32)bPrevious];
	}

	const FVertexBufferAndSRV& GetBuffer(bool bPrevious) const
	{
		return Buffers[CurrentIndex ^ (uint32)bPrevious];
	}

	TConstArrayView<FBoneIndexType> GetBoneMap() const
	{
		return BoneMap;
	}

	uint32 GetNumBones() const
	{
		return BoneMap.Num();
	}

	uint32 GetBufferSize() const
	{
		return BufferSize;
	}

private:
	TConstArrayView<FBoneIndexType> BoneMap;
	TStaticArray<FVertexBufferAndSRV, 2> Buffers;
	uint32 CurrentIndex = 0;
	uint32 BufferSize = 0;
};

/** The policy for pooling bone vertex buffers */
class FClothBufferPoolPolicy : public FBoneBufferPoolPolicy
{
public:
	/** Creates the resource 
	 * @param Args The buffer size in bytes.
	 */
	FVertexBufferAndSRV CreateResource(FRHICommandListBase& RHICmdList, FSharedPoolPolicyData::CreationArguments Args);
};

/** A pool for vertex buffers with consistent usage, bucketed for efficiency. */
class FClothBufferPool : public TRenderResourcePool<FVertexBufferAndSRV, FClothBufferPoolPolicy, FSharedPoolPolicyData::CreationArguments>
{
public:
	/** Destructor */
	virtual ~FClothBufferPool();
	
public: // From FTickableObjectRenderThread
	virtual TStatId GetStatId() const override;
};

enum GPUSkinBoneInfluenceType
{
	DefaultBoneInfluence,	// up to 8 bones per vertex
	UnlimitedBoneInfluence	// unlimited bones per vertex
};

/** Stream component data bound to GPU skinned vertex factory */
struct FGPUSkinDataType : public FStaticMeshDataType
{
	/** The stream to read the bone indices from */
	FVertexStreamComponent BoneIndices;

	/** The stream to read the extra bone indices from */
	FVertexStreamComponent ExtraBoneIndices;

	/** The stream to read the bone weights from */
	FVertexStreamComponent BoneWeights;

	/** The stream to read the extra bone weights from */
	FVertexStreamComponent ExtraBoneWeights;

	/** The stream to read the blend stream offset and num of influences from */
	FVertexStreamComponent BlendOffsetCount;

	/** Number of bone influences */
	uint32 NumBoneInfluences = 0;

	/** If the bone indices are 16 or 8-bit format */
	bool bUse16BitBoneIndex = 0;

	/** If this is a morph target */
	bool bMorphTarget = false;

	/** Morph target stream which has the position deltas to add to the vertex position */
	FVertexStreamComponent DeltaPositionComponent;

	/** Morph target stream which has the TangentZ deltas to add to the vertex normals */
	FVertexStreamComponent DeltaTangentZComponent;

	/** Morph vertex buffer pool double buffering delta data  */
	class FMorphVertexBufferPool* MorphVertexBufferPool = nullptr;
};

/** Vertex factory with vertex stream components for GPU skinned vertices */
class FGPUBaseSkinVertexFactory : public FVertexFactory
{
public:
	class FUpdateScope
	{
	public:
		FUpdateScope();

	private:
		FClothBufferPool::FLockScope Cloth;
		FBoneBufferPool::FLockScope Bone;
	};

	struct FShaderDataType
	{
		FShaderDataType()
		{
			// BoneDataOffset and BoneTextureSize are not set as they are only valid if IsValidRef(BoneTexture)
			MaxGPUSkinBones = GetMaxGPUSkinBones();
			check(MaxGPUSkinBones <= GHardwareMaxGPUSkinBones);
			FMemory::Memset(&SkinningHeader, 0, sizeof(FSkinningHeader));
		}

		void AllocateBoneBuffer(FRHICommandList& RHICmdList, uint32 BufferSize, FVertexBufferAndSRV& OutBoneBuffer);

		void UpdateBoneData(
			FRHICommandList& RHICmdList,
			const FName& AssetPathName,
			TConstArrayView<FMatrix44f> ReferenceToLocalMatrices,
			TConstArrayView<FBoneIndexType> BoneMap,
			FRHIBuffer* VertexBufferRHI);

		void ReleaseBoneData();

		bool HasBoneBufferForReading(bool bPrevious) const
		{
			bPrevious = GetPreviousForRead(bPrevious);
			const FVertexBufferAndSRV* Output = &GetBoneBufferInternal(bPrevious);
			if (bPrevious && !Output->VertexBufferRHI.IsValid())
			{
				Output = &GetBoneBufferInternal(false);
			}
			return Output->VertexBufferRHI.IsValid();
		}

		const FVertexBufferAndSRV& GetBoneBufferForReading(bool bPrevious) const
		{
			bPrevious = GetPreviousForRead(bPrevious);
			const FVertexBufferAndSRV* Output = &GetBoneBufferInternal(bPrevious);

			if(!Output->VertexBufferRHI.IsValid())
			{
				// Data is only allowed to be null when requesting the previous buffers.
				checkf(bPrevious, TEXT("Trying to access current bone buffer for reading, but it is null. BoneBuffer[0] = %p, BoneBuffer[1] = %p, CurrentRevisionNumber = %u, PreviousRevisionNumber = %u"),
					BoneBuffer[0].VertexBufferRHI.GetReference(), BoneBuffer[1].VertexBufferRHI.GetReference(), CurrentRevisionNumber, PreviousRevisionNumber);

				Output = &GetBoneBufferInternal(false);
				check(Output->VertexBufferRHI.IsValid());
			}

			return *Output;
		}

		FVertexBufferAndSRV& GetBoneBufferForWriting(bool bPrevious)
		{
			return const_cast<FVertexBufferAndSRV&>(GetBoneBufferInternal(bPrevious));
		}

		void SetRevisionNumbers(uint32 InCurrentRevisionNumber, uint32 InPreviousRevisionNumber)
		{
			PreviousRevisionNumber = InPreviousRevisionNumber != INDEX_NONE ? InPreviousRevisionNumber : CurrentRevisionNumber;
			CurrentRevisionNumber = InCurrentRevisionNumber;
			CurrentBuffer = 1 - CurrentBuffer;
		}

		uint32 GetRevisionNumber(bool bPrevious) const
		{
			return bPrevious ? PreviousRevisionNumber : CurrentRevisionNumber;
		}

		UE_DEPRECATED(5.6, "Use SetRevisionNumbers instead")
		void SetCurrentRevisionNumber(uint32) {}

		int32 InputWeightIndexSize = 0;
		FShaderResourceViewRHIRef InputWeightStream;

		// References a bone map buffer to use as an indirection on the GPU.
		FShaderResourceViewRHIRef SectionBoneMapBuffer;

		// The current skinning header assigned by the scene extension.
		FSkinningHeader SkinningHeader;

		// Frame number of the bone data that is last updated
		uint32 UpdatedFrameNumber = 0;

		// Whether bone buffers are shared between all sections.
		bool bUnifiedBoneBuffer = false;

	private:
		TStaticArray<FVertexBufferAndSRV, 2> BoneBuffer;
		uint32 CurrentBuffer = 0;
		uint32 PreviousRevisionNumber = 0;
		uint32 CurrentRevisionNumber = 0;

		static uint32 MaxGPUSkinBones;

		bool GetPreviousForRead(bool bPrevious) const
		{
			// If the revision number has incremented too much, ignore the request and use the current buffer.
			// With ClearMotionVector calls, we intentionally increment revision number to retrieve current buffer for bPrevious true.
			return (CurrentRevisionNumber - PreviousRevisionNumber) <= 1 ? bPrevious : false;
		}

		const FVertexBufferAndSRV& GetBoneBufferInternal(bool bPrevious) const
		{
			return BoneBuffer[CurrentBuffer ^ (uint32)bPrevious];
		}
	};

	struct FInitializer
	{
		ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
		uint32 NumBones = 0;
		uint32 BoneOffset = 0;
		uint32 NumVertices = 0;
		uint32 BaseVertexIndex = 0;
		bool bUsedForPassthroughVertexFactory = false;
		bool bUnifiedBoneBuffer = false;
	};

	FGPUBaseSkinVertexFactory(const FInitializer& Initializer);

	UE_DEPRECATED(5.7, "Use FInitializer instead")
	FGPUBaseSkinVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumBones, uint32 InNumVertices, uint32 InBaseVertexIndex, bool bInUsedForPassthroughVertexFactory)
		: FGPUBaseSkinVertexFactory(FInitializer
		{
			  .FeatureLevel    = InFeatureLevel
			, .NumBones        = InNumBones
			, .NumVertices     = InNumVertices
			, .BaseVertexIndex = InBaseVertexIndex
			, .bUsedForPassthroughVertexFactory = bInUsedForPassthroughVertexFactory
		})
	{}

	virtual ~FGPUBaseSkinVertexFactory() = default;

	/** accessor */
	inline FShaderDataType& GetShaderData()
	{
		return ShaderData;
	}

	inline const FShaderDataType& GetShaderData() const
	{
		return ShaderData;
	}

	virtual void SetData(FRHICommandListBase& RHICmdList, const FGPUSkinDataType* InData);

	uint32 GetBoneBufferSize() const
	{
		return BoneBufferSize;
	}

	uint32 GetNumBones() const
	{
		return NumBones;
	}

	uint32 GetBoneOffset() const
	{
		return BoneOffset;
	}

	uint32 GetNumVertices() const
	{
		return NumVertices;
	}

	uint32 GetBaseVertexIndex() const
	{
		return BaseVertexIndex;
	}

	/*
	 * Return the smallest platform MaxGPUSkinBones value.
	 */
	ENGINE_API static int32 GetMinimumPerPlatformMaxGPUSkinBonesValue();
	ENGINE_API static int32 GetMaxGPUSkinBones(const class ITargetPlatform* TargetPlatform = nullptr);

	static const uint32 GHardwareMaxGPUSkinBones = 65536;
	
	ENGINE_API static bool UseUnlimitedBoneInfluences(uint32 MaxBoneInfluences, const ITargetPlatform* TargetPlatform = nullptr);
	ENGINE_API static bool GetUnlimitedBoneInfluences(const ITargetPlatform* TargetPlatform = nullptr);

	/*
	 * Returns the maximum number of bone influences that should be used for a skeletal mesh, given
	 * the user-requested limit.
	 * 
	 * If the requested limit is 0, the limit will be determined from the project settings.
	 * 
	 * The return value is guaranteed to be greater than zero, but note that it may be higher than
	 * the maximum supported bone influences.
	 */
	ENGINE_API static int32 GetBoneInfluenceLimitForAsset(int32 AssetProvidedLimit, const ITargetPlatform* TargetPlatform = nullptr);

	/**
	 * Returns true if mesh LODs with Unlimited Bone Influences must always be rendered using a
	 * Mesh Deformer for the given shader platform.
	 */
	ENGINE_API static bool GetAlwaysUseDeformerForUnlimitedBoneInfluences(EShaderPlatform Platform);

	/** Morph vertex factory functions */
	void UpdateMorphState(FRHICommandListBase& RHICmdList, bool bUseMorphTarget);
	const class FMorphVertexBuffer* GetMorphVertexBuffer(bool bPrevious) const;
	uint32 GetMorphVertexBufferUpdatedFrameNumber() const;

	/** Cloth vertex factory access. */
	virtual class FGPUBaseSkinAPEXClothVertexFactory* GetClothVertexFactory() { return nullptr; }
	virtual class FGPUBaseSkinAPEXClothVertexFactory const* GetClothVertexFactory() const { return nullptr; }

	virtual GPUSkinBoneInfluenceType GetBoneInfluenceType() const				{ return DefaultBoneInfluence; }
	virtual uint32 GetNumBoneInfluences() const									{ return Data.IsValid() ? Data->NumBoneInfluences : 0; }
	virtual bool Use16BitBoneIndex() const										{ return Data.IsValid() ? Data->bUse16BitBoneIndex : false; }
	virtual const FShaderResourceViewRHIRef GetPositionsSRV() const				{ return Data.IsValid() ? Data->PositionComponentSRV : nullptr; }
	virtual const FShaderResourceViewRHIRef GetTangentsSRV() const				{ return Data.IsValid() ? Data->TangentsSRV : nullptr; }
	virtual const FShaderResourceViewRHIRef GetTextureCoordinatesSRV() const	{ return Data.IsValid() ? Data->TextureCoordinatesSRV : nullptr; }
	virtual const FShaderResourceViewRHIRef GetColorComponentsSRV() const		{ return Data.IsValid() ? Data->ColorComponentsSRV : nullptr; }
	virtual uint32 GetNumTexCoords() const										{ return Data.IsValid() ? Data->NumTexCoords : 0; }
	virtual const uint32 GetColorIndexMask() const								{ return Data.IsValid() ? Data->ColorIndexMask : 0; }
	virtual bool IsMorphTarget() const											{ return Data.IsValid() ? Data->bMorphTarget : false; }

	virtual FShaderResourceViewRHIRef GetTriangleSortingPositionSRV() const override
	{
		return GetPositionsSRV();
	}

	void UpdateUniformBuffer(FRHICommandListBase& RHICmdList, const FSkinningHeader* SkinningHeader, FRHIShaderResourceView* TransformDataBufferSRV);

	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer;
	}

	const FVertexStreamComponent& GetPositionStreamComponent() const
	{
		check(Data.IsValid() && Data->PositionComponent.VertexBuffer != nullptr);
		return Data->PositionComponent;
	}
	
	const FVertexStreamComponent& GetTangentStreamComponent(int Index) const
	{
		check(Data.IsValid() && Data->TangentBasisComponents[Index].VertexBuffer != nullptr);
		return Data->TangentBasisComponents[Index];
	}

	void CopyDataTypeForLocalVertexFactory(FLocalVertexFactory::FDataType& OutDestData) const;

	void GetOverrideVertexStreams(FVertexInputStreamArray& VertexStreams) const;

	bool IsReadyForStaticMeshCaching() const { return UniformBuffer != nullptr; }

	void MarkUniformBufferDirty()
	{
		bUniformBufferDirty = true;
	}

	bool IsUniformBufferValid() const
	{
		return UniformBuffer.IsValid();
	}

protected:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	/**
	* Add the decl elements for the streams
	* @param InData - type with stream components
	* @param OutElements - vertex decl list to modify
	*/
	virtual void AddVertexElements(FVertexDeclarationElementList& OutElements) = 0;

	/** dynamic data need for setting the shader */ 
	FShaderDataType ShaderData;

	/** stream component data bound to this vertex factory */
	TUniquePtr<FGPUSkinDataType> Data;

	/** Shader bindings are stored here in the uniform buffer. */
	FUniformBufferRHIRef UniformBuffer;

	TRefCountPtr<FRHIStreamSourceSlot> MorphDeltaBufferSlot;
	int32 MorphDeltaStreamIndex = -1;

private:
	uint32 NumBones;
	uint32 BoneOffset;
	uint32 BoneBufferSize;
	uint32 NumVertices;
	uint32 BaseVertexIndex;
	bool bUsedForPassthroughVertexFactory;
	bool bUniformBufferDirty = true;
};

/** Vertex factory with vertex stream components for GPU skinned vertices */
template<GPUSkinBoneInfluenceType BoneInfluenceType>
class TGPUSkinVertexFactory : public FGPUBaseSkinVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(TGPUSkinVertexFactory<BoneInfluenceType>);

public:
	TGPUSkinVertexFactory(const FInitializer& Initializer)
		: FGPUBaseSkinVertexFactory(Initializer)
	{}

	UE_DEPRECATED(5.7, "Use FInitializer instead")
	TGPUSkinVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumBones, uint32 InNumVertices, uint32 InBaseVertexIndex, bool bInUsedForPassthroughVertexFactory)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: FGPUBaseSkinVertexFactory(InFeatureLevel, InNumBones, InNumVertices, InBaseVertexIndex, bInUsedForPassthroughVertexFactory)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{}

	virtual GPUSkinBoneInfluenceType GetBoneInfluenceType() const override
	{
		return BoneInfluenceType;
	}

	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);	
	static void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FGPUSkinDataType& GPUSkinData, FVertexDeclarationElementList& Elements);

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

protected:
	/**
	* Add the decl elements for the streams
	* @param InData - type with stream components
	* @param OutElements - vertex decl list to modify
	*/
	virtual void AddVertexElements(FVertexDeclarationElementList& OutElements) override;

	static void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FGPUSkinDataType& GPUSkinData, FVertexDeclarationElementList& Elements, FVertexStreamList& InOutStreams, int32& OutMorphDeltaStreamIndex);
};


/** Vertex factory with vertex stream components for GPU-skinned and morph target streams */
class FGPUBaseSkinAPEXClothVertexFactory
{
public:

	struct ClothShaderType
	{
		ClothShaderType()
		{
			Reset();
		}

		void UpdateClothSimulationData(
			FRHICommandList& RHICmdList,
			TConstArrayView<FVector3f> InSimulPositions,
			TConstArrayView<FVector3f> InSimulNormals,
			uint32 RevisionNumber,
			const FName& AssetPathName);

		void ReleaseClothSimulData();

		void EnableDoubleBuffer()	{ bDoubleBuffer = true; }

		void SetCurrentRevisionNumber(uint32 RevisionNumber);

		FVertexBufferAndSRV& GetClothBufferForWriting();
		bool HasClothBufferForReading(bool bPrevious) const;
		const FVertexBufferAndSRV& GetClothBufferForReading(bool bPrevious) const;

		FMatrix44f& GetClothToLocalForWriting();
		const FMatrix44f& GetClothToLocalForReading(bool bPrevious) const;

		/**
		 * weight to blend between simulated positions and key-framed poses
		 * if ClothBlendWeight is 1.0, it shows only simulated positions and if it is 0.0, it shows only key-framed animation
		 */
		float ClothBlendWeight = 1.0f;
		/** Scale of the owner actor */
		FVector3f WorldScale = FVector3f::OneVector;
		uint32 NumInfluencesPerVertex = 1;

		/** Whether cloth simulation is currently enabled. */
		bool bEnabled = false;

	private:
		// Helper for GetClothBufferIndexForWriting and GetClothBufferIndexForReading
		uint32 GetClothBufferIndexInternal(bool bPrevious) const;
		// Helper for GetClothBufferForWriting and GetClothToLocalForWriting
		uint32 GetClothBufferIndexForWriting() const;
		// Helper for GetClothBufferForReading and GetClothToLocalForReading
		uint32 GetClothBufferIndexForReading(bool bPrevious) const;

		FVertexBufferAndSRV ClothSimulPositionNormalBuffer[2];

		/**
		 * Matrix to apply to positions/normals
		 */
		FMatrix44f ClothToLocal[2];

		/** Whether to double buffer. */
		bool bDoubleBuffer = false;

		// 0 / 1 to index into BoneBuffer
		uint32 CurrentBuffer = 0;
		// RevisionNumber Tracker
		uint32 PreviousRevisionNumber = 0;
		uint32 CurrentRevisionNumber = 0;

		void Reset()
		{
			CurrentBuffer = 0;
			PreviousRevisionNumber = 0;
			CurrentRevisionNumber = 0;

			ClothToLocal[0] = FMatrix44f::Identity;
			ClothToLocal[1] = FMatrix44f::Identity;

			bDoubleBuffer = false;
		}
	};

	FGPUBaseSkinAPEXClothVertexFactory(uint32 InNumInfluencesPerVertex)
	{
		ClothShaderData.NumInfluencesPerVertex = InNumInfluencesPerVertex;
	}

	virtual ~FGPUBaseSkinAPEXClothVertexFactory() {}

	/** accessor */
	inline ClothShaderType& GetClothShaderData()
	{
		return ClothShaderData;
	}

	inline const ClothShaderType& GetClothShaderData() const
	{
		return ClothShaderData;
	}

	static bool IsClothEnabled(EShaderPlatform Platform);

	virtual FGPUBaseSkinVertexFactory* GetVertexFactory() = 0;
	virtual const FGPUBaseSkinVertexFactory* GetVertexFactory() const = 0;

	/** Get buffer containing cloth influences. */
	virtual FShaderResourceViewRHIRef GetClothBuffer() { return nullptr; }
	virtual const FShaderResourceViewRHIRef GetClothBuffer() const { return nullptr; }
	/** Get offset from vertex index to cloth influence index at a given vertex index. The offset will be constant for all vertices in the same section. */
	virtual uint32 GetClothIndexOffset(uint32 VertexIndex, uint32 LODBias = 0) const { return 0; }

protected:
	ClothShaderType ClothShaderData;
};

/** Stream component data bound to Apex cloth vertex factory */
struct FGPUSkinAPEXClothDataType : public FGPUSkinDataType
{
	FShaderResourceViewRHIRef ClothBuffer;
	// Packed Map: u32 Key, u32 Value
	TArray<FClothBufferIndexMapping> ClothIndexMapping;
};

template<GPUSkinBoneInfluenceType BoneInfluenceType>
class TGPUSkinAPEXClothVertexFactory : public FGPUBaseSkinAPEXClothVertexFactory, public TGPUSkinVertexFactory<BoneInfluenceType>
{
	DECLARE_VERTEX_FACTORY_TYPE(TGPUSkinAPEXClothVertexFactory<BoneInfluenceType>);

	typedef TGPUSkinVertexFactory<BoneInfluenceType> Super;

public:
	inline FShaderResourceViewRHIRef GetClothBuffer() override
	{
		return ClothDataPtr ? ClothDataPtr->ClothBuffer : nullptr;
	}

	const FShaderResourceViewRHIRef GetClothBuffer() const override
	{
		return ClothDataPtr ? ClothDataPtr->ClothBuffer : nullptr;
	}

	uint32 GetClothIndexOffset(uint32 VertexIndex, uint32 LODBias = 0) const override
	{
		if (ClothDataPtr)
		{
			for (const FClothBufferIndexMapping& Mapping : ClothDataPtr->ClothIndexMapping)
			{
				if (Mapping.BaseVertexIndex == VertexIndex)
				{
					return Mapping.MappingOffset + Mapping.LODBiasStride * LODBias;
				}
			}
		}

		checkf(0, TEXT("Cloth Index Mapping not found for Vertex Index %u"), VertexIndex);
		return 0;
	}

	TGPUSkinAPEXClothVertexFactory(const FGPUBaseSkinVertexFactory::FInitializer& Initializer, uint32 NumInfluencesPerVertex)
		: FGPUBaseSkinAPEXClothVertexFactory(NumInfluencesPerVertex)
		, TGPUSkinVertexFactory<BoneInfluenceType>(Initializer)
	{}

	UE_DEPRECATED(5.7, "Use FInitializer instead")
	TGPUSkinAPEXClothVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, uint32 InNumBones, uint32 InNumVertices, uint32 InBaseVertexIndex, uint32 InNumInfluencesPerVertex, bool bInUsedForPassthroughVertexFactory)
		: FGPUBaseSkinAPEXClothVertexFactory(InNumInfluencesPerVertex)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, TGPUSkinVertexFactory<BoneInfluenceType>(InFeatureLevel, InNumBones, InNumVertices, InBaseVertexIndex, bInUsedForPassthroughVertexFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{}

	/**
	 * Destructor takes care of the Data pointer. Since FGPUBaseSkinVertexFactory does not know the real type of the Data,
	 * delete the data here instead.
	 */
	virtual ~TGPUSkinAPEXClothVertexFactory() override
	{
		checkf(!ClothDataPtr->ClothBuffer.IsValid(), TEXT("ClothBuffer RHI resource should have been released in ReleaseRHI"));
		delete ClothDataPtr;
		(void)this->Data.Release();
	}

	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	* An implementation of the interface used by TSynchronizedResource to 
	* update the resource with new data from the game thread.
	* @param	InData - new stream component data
	*/
	virtual void SetData(FRHICommandListBase& RHICmdList, const FGPUSkinDataType* InData) override;

	virtual FGPUBaseSkinVertexFactory* GetVertexFactory() override
	{
		return this;
	}

	virtual const FGPUBaseSkinVertexFactory* GetVertexFactory() const override
	{
		return this;
	}

	virtual FGPUBaseSkinAPEXClothVertexFactory* GetClothVertexFactory() override 
	{
		return this; 
	}
	
	virtual FGPUBaseSkinAPEXClothVertexFactory const* GetClothVertexFactory() const override 
	{
		return this; 
	}

	// FRenderResource interface.

	/**
	* Creates declarations for each of the vertex stream components and
	* initializes the device resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

protected:
	/** Alias pointer to TUniquePtr<FGPUSkinDataType> Data of FGPUBaseSkinVertexFactory. Note memory isn't managed through this pointer. */
	FGPUSkinAPEXClothDataType* ClothDataPtr = nullptr;
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSkinPassThroughFactoryLooseParameters, ENGINE_API)
	SHADER_PARAMETER(uint32, FrameNumber)
	SHADER_PARAMETER_SRV(Buffer<float>, PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, PreviousPositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, PreSkinnedTangentBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Vertex factory with vertex stream components for GPU-skinned streams.
 * This enables Passthrough mode where vertices have been pre-skinned.
 * Individual vertex attributes can be flagged so that they can be overridden by externally owned buffers.
 */
class FGPUSkinPassthroughVertexFactory : public FLocalVertexFactory
{
public:
	/** SRVs that we can provide. */
	enum EShaderResource
	{
		Position,
		PreviousPosition,
		Tangent,
		Color,
		TexCoord,
		NumShaderResources
	};

	/** Vertex attributes that we can override. */
	enum EVertexAttribute
	{
		VertexPosition,
		VertexTangent,
		VertexColor,
		NumAttributes
	};

	enum class EVertexAttributeFlags : uint8
	{
		None = 0,
		Position = 1 << EVertexAttribute::VertexPosition,
		Tangent  = 1 << EVertexAttribute::VertexTangent,
		Color    = 1 << EVertexAttribute::VertexColor,
	};

	UE_DEPRECATED(5.7, "FGPUSkinPassthroughVertexFactory now requires NumVertices.")
	FGPUSkinPassthroughVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, EVertexAttributeFlags InVertexAttributeMask)
		: FGPUSkinPassthroughVertexFactory(InFeatureLevel, InVertexAttributeMask, 0)
	{}

	FGPUSkinPassthroughVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, EVertexAttributeFlags InVertexAttributeMask, uint32 InNumVertices);
	
	/** Structure used for calls to SetVertexAttributes(). */
	struct FAddVertexAttributeDesc
	{
		FAddVertexAttributeDesc() : StreamBuffers(InPlace, nullptr), SRVs(InPlace, nullptr) {}

		/** Frame number at animation update. Used to determine if animation motion is valid and needs to output velocity. */
		uint32 FrameNumber = ~0U;

		/** Set of stream buffers to override. */
		TStaticArray<FRHIBuffer*, EVertexAttribute::NumAttributes> StreamBuffers;

		/** SRVs for binding. These are only be used by platforms that support manual vertex fetch. */
		TStaticArray<FRHIShaderResourceView*, EShaderResource::NumShaderResources> SRVs;
	};
	
	/**
	 * Reset all added vertex attributes and SRVs.
	 * This doesn't reset the vertex factory itself. Call SetData() to do that.
	 */
	void ResetVertexAttributes(FRHICommandListBase& RHICmdList);

	void SetVertexAttributes(FRHICommandListBase& RHICmdList, FGPUBaseSkinVertexFactory const* InSourceVertexFactory, FAddVertexAttributeDesc const& InDesc);

	// Begin FVertexFactory Interface.
	bool SupportsPositionOnlyStream() const override { return false; }
	bool SupportsPositionAndNormalOnlyStream() const override { return false; }
	// End FVertexFactory Interface.

	void GetOverrideVertexStreams(FVertexInputStreamArray& VertexStreams) const;

	EPixelFormat GetTangentFormat() const { return TangentFormat; }

	uint32 GetNumVertices() const { return NumVertices; }

	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FDataType& Data, FVertexDeclarationElementList& Elements);

public:
	TUniformBufferRef<FGPUSkinPassThroughFactoryLooseParameters> LooseParametersUniformBuffer;

private:
	void InitRHI(FRHICommandListBase& RHICmdList) override;
	void ReleaseRHI() override
	{
		LooseParametersUniformBuffer.SafeRelease();
		FLocalVertexFactory::ReleaseRHI();
	}
	void UpdateUniformBuffer(FRHICommandListBase& RHICmdList, FGPUBaseSkinVertexFactory const* InSourceVertexFactory);
	void UpdateLooseUniformBuffer(FRHICommandListBase& RHICmdList, FGPUBaseSkinVertexFactory const* InSourceVertexFactory, uint32 InFrameNumber);
	
	TStaticArray<FRHIBuffer*, EVertexAttribute::NumAttributes> SourceStreamBuffers{ InPlace, nullptr };
	TStaticArray<TRefCountPtr<FRHIStreamSourceSlot>, EVertexAttribute::NumAttributes> StreamSourceSlots;
	TStaticArray<FRHIShaderResourceView*, EShaderResource::NumShaderResources> SRVs{ InPlace, nullptr };
	FRHIShaderResourceView* PreSkinnedTangentSRV = nullptr;
	uint32 UpdatedFrameNumber = ~0u;
	uint32 NumVertices;
	EVertexAttributeFlags VertexAttributesRequested;
	EVertexAttributeFlags VertexAttributesToBind = EVertexAttributeFlags::None;
	EPixelFormat TangentFormat = PF_Unknown;
};

ENUM_CLASS_FLAGS(FGPUSkinPassthroughVertexFactory::EVertexAttributeFlags)
