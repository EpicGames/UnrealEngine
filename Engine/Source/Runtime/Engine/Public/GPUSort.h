// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSort.cpp: Interface for sorting buffers on the GPU.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

/**
 * The input buffers required for sorting on the GPU.
 */
struct FGPUSortBuffers
{
	/** Shader resource views for vertex buffers containing the keys. */
	FRHIShaderResourceView* RemoteKeySRVs[2];
	/** Unordered access views for vertex buffers containing the keys. */
	FRHIUnorderedAccessView* RemoteKeyUAVs[2];

	/** Shader resource views for vertex buffers containing the values. */
	FRHIShaderResourceView* RemoteValueSRVs[2];
	/** Unordered access views for vertex buffers containing the values. */
	FRHIUnorderedAccessView* RemoteValueUAVs[2];

	/** Shader resource view holding the initial state of the values. */
	FRHIShaderResourceView* FirstValuesSRV = nullptr;
	/** Unordered access view holding the final state of the value. */
	FRHIUnorderedAccessView* FinalValuesUAV = nullptr;

	/** Default constructor. */
	FGPUSortBuffers()
	{
	}
};

/**
 * Get the number of passes we will need to make in order to sort
 */
ENGINE_API int32 GetGPUSortPassCount(uint32 KeyMask);

/**
 * Sort a buffer on the GPU.
 * @param SortBuffers - The buffer to sort including required views and a ping-
 *			pong location of appropriate size.
 * @param BufferIndex - Index of the buffer containing keys.
 * @param KeyMask - Bitmask indicating which key bits contain useful information.
 * @param Count - How many items in the buffer need to be sorted.
 * @returns The index of the buffer containing sorted results.
 */
ENGINE_API int32 SortGPUBuffers(FRHICommandList& RHICmdList, FGPUSortBuffers SortBuffers, int32 BufferIndex, uint32 KeyMask, int32 Count, ERHIFeatureLevel::Type FeatureLevel);

/**
 * GPU sorting tests.
 */
enum EGPUSortTest
{
	/** Tests the sort on a small set of elements. */
	GPU_SORT_TEST_SMALL = 1,
	/** Tests the sort on a large set of elements. */
	GPU_SORT_TEST_LARGE,
	/** Tests the sort on many different sizes of elements. */
	GPU_SORT_TEST_EXHAUSTIVE,
	GPU_SORT_TEST_RANDOM
};

/**
 * Test that GPU sorting works.
 * @param TestToRun - The test to run.
 */
void TestGPUSort(EGPUSortTest TestToRun, ERHIFeatureLevel::Type FeatureLevel);
