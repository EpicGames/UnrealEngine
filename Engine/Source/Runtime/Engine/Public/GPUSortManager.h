// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSortManager.h: Interface manage sorting between different FXSystemInterface
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "RHI.h"
#include "RenderResource.h"

/**
 * Copy one uint buffer into several overlapping targets.
 * Each target is expected to be bigger than the previous one, so that copy resumes where it left.
 * Ex : (ABCDEFGHIJK) -> (ABC--, ---DEFGHI-, ---------JK---)
 * This is essentially used to manage growable buffers in cases the
 * binding needs to be set before the final size required is known. 
 * In this scenario, the smaller buffers are temporary and only the final (biggest) buffer
 * becomes persistent, this is why it has apparently unused space at the beginning but
 * it will be used the next frame to hold all of the data.
 * Note that required calls to RHICmdList.Transition need to be handled before and after this function.
 *
 * @param RHICmdList - The command list used to issue the dispatch.
 * @param FeatureLevel - The current feature level, used to access the global shadermap.
 * @param SourceSRV - The source uint buffer to be copied into the others.
 * @param TargetUAVs - The destination overlapping uint buffers.
 * @param TargetSizes - The copy size of each of the buffers. Once a buffer size is reached, the copy targets the next buffer.
 * @param StartingOffset - The starting position at which the copy starts. Applies for both the source and targets.
 * @param NumTargets - The number of elements in TargetUAVs and TargetSizes.
 */
ENGINE_API void CopyUIntBufferToTargets(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FRHIShaderResourceView* SourceSRV, FRHIUnorderedAccessView*const* TargetUAVs, int32* TargetSizes, int32 StartingOffset, int32 NumTargets);

struct FGPUSortBuffers;

/** Buffers in GPU memory used to sort particles. */
class FParticleSortBuffers : public FRenderResource
{
public:

	void SetBufferSize(int32 InBufferSize)
	{
		BufferSize = InBufferSize;
	}

	/** Initialize RHI resources. */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release RHI resources. */
	virtual void ReleaseRHI() override;

	/** Retrieve the UAV for writing particle sort keys. */
	FRHIUnorderedAccessView* GetKeyBufferUAV(int32 BufferIndex)
	{
		check((BufferIndex & 0xFFFFFFFE) == 0);
		return KeyBufferUAVs[BufferIndex];
	}

	/** Retrieve buffers needed to sort on the GPU. */
	FGPUSortBuffers GetSortBuffers();

	/** Retrieve the sorted vertex buffer that results will always be located at. */
	FRHIBuffer* GetSortedVertexBufferRHI(int32 BufferIndex)
	{
		check((BufferIndex & 0xFFFFFFFE) == 0);
		return VertexBuffers[BufferIndex];
	}

	/** Get the size allocated for sorted vertex buffers. */
	int32 GetSize() { return BufferSize; }

private:

	/** Vertex buffer storage for particle sort keys. */
	FBufferRHIRef KeyBuffers[2];
	/** Shader resource view for particle sort keys. */
	FShaderResourceViewRHIRef KeyBufferSRVs[2];
	/** Unordered access view for particle sort keys. */
	FUnorderedAccessViewRHIRef KeyBufferUAVs[2];

	/** Vertex buffer containing sorted particle vertices. */
	FBufferRHIRef VertexBuffers[2];
	/** Shader resource view for sorting particle vertices. */
	FShaderResourceViewRHIRef VertexBufferSortSRVs[2];
	/** Unordered access view for sorting particle vertices. */
	FUnorderedAccessViewRHIRef VertexBufferSortUAVs[2];

	/** Size allocated for buffers. */
	int32 BufferSize = 0;
};

/** Different sort flags used to define constraint for FGPUSortManager tasks. */
enum class EGPUSortFlags : uint32
{
	None			= 0x00,
	/**
	 * Sorting happens on either 16 bit keys or 32 bit uint keys. Those can come from float point formats being converted to uint.
	 * Because sort tasks are grouped into batches, the keys also have to start with the batch element index, which ends up either
	 * increasing the 16 bit keys or taking space in the 32 bit keys (by the number of bits required to store the element indices).
	 * The more task are created, the more bits are required to encode the element index (up to a limit of 16^2 elements).
	 * Low precision keys have the advantage of being quicker on the GPU when performing the radix sort, which is affected by the number of bits.
	 */
	LowPrecisionKeys	= 0x01,		// Low precision keys ((EleIdx << 16) | (16 bits key))
	HighPrecisionKeys	= 0x02,		// High precision keys ((EleIdx << (32 - IdxBitCount)) | (32 bits key >> IdxBitCount))

	AnyKeyPrecision		= 0x03,

	/**
	 * The sort task are created in FPrimitiveSceneProxy::GetDynamicMeshElements() but have different requirements
	 * in terms of when the keys can be generated, and when the sorted values are required within the frame.
	 * Cascade renders the particles after GPU simulation (either in PreRender(), or in PostRenderOpaque() see ::CollisionDepthBuffer).
	 * Niagara, on the other hand, renders the particle before they get updated by simulation (either in PreInitViews() or PostRenderOpaque())
	 * For Niagara emitters updated in PreInitViews(), this happens before GetDynamicMeshElements() so doesn't impose any restriction in the end.
	 * This means each system has some flexibility on when the sort keys can be generated depending on the simulation point for each emitter.
	 */
	KeyGenAfterPreRender			= 0x04,		// Typically Cascade emitters simulated in PreRender() or any Niagara emitters.
	KeyGenAfterPostRenderOpaque		= 0x08,		// Typically any Cascade emitters, or Niagara emitters simulated in PreInitViews().

	AnyKeyGenLocation				= 0x0C,

	/**
	 * Once keys are generated, they must be sorted and the indices only need to be ready before the primitive are rendered.
	 * The (frame) location only depends on whether the emitter uses opaque materials or translucent materials.
	 * It is possible to have an incorrect setup if the keys can only be generated after PostRenderOpaque() but the sorting must be ready after PreRender().
	 * This happens with opaque Cascade emitters when simulation needs the depth buffer to do collision tests.
	 * A sort requests can have both SortIndicesAfterPreRender and SortIndicesAfterPostRenderOpaque when it is allowed to sort keys at either point.
	 */
	SortAfterPreRender		 	= 0x10,		// Any Emitter since good for both opaque or translucent
	SortAfterPostRenderOpaque	= 0x20,		// Emitter using translucent material.

	AnySortLocation				= 0x30,

	/**
	 *  The sorted values can either be read as 1D or 2D, but doesn't otherwise affect the sorting implementation as long as the storage size is the same.
	 *  Currently it is 32 bits in both cases. Note that the sorting shader reads the indices as UInt32 (see FParticleSortBuffers::InitRHI()).
	 */
	ValuesAsG16R16F	= 0x40,
	ValuesAsInt32	= 0x80,

	AnyValueFormat	= 0xC0,
};

ENUM_CLASS_FLAGS(EGPUSortFlags)

/**
 * Callback used by the FGPUSortManager to make each client system initialize the sort keys for a specific sort batch.
 * Sort batches are created when GPU sort tasks are registered in FGPUSortManager::AddTask().
 * Note that the system that is getting this callback must only initialize the data for the elements is has registered in this batch.
 * 
 * @param RHICmdList - The command list used to initiate the keys and values on GPU.
 * @param BatchId - The GPUSortManager batch id (regrouping several similar sort tasks).
 * @param NumElementsInBatch - The number of elements grouped in the batch (each element maps to a sort task)
 * @param Flags - Details about the key precision (see EGPUSortFlags::AnyKeyPrecision) and the keygen location (see EGPUSortFlags::AnyKeyGenLocation).
 * @param KeysUAV - The UAV that holds all the initial keys used to sort the values.
 * @param ValuesUAV - The UAV that holds the initial values to be sorted accordingly to the keys.
 */
DECLARE_DELEGATE_SixParams(FGPUSortKeyGenDelegate, FRHICommandListImmediate&, int32, int32, EGPUSortFlags, FRHIUnorderedAccessView*, FRHIUnorderedAccessView*);

/**
 * A manager that handles different GPU sort tasks. 
 * Each task has different constraints about when the data can be generated and when the sorted results need to be available.
 * The usecase involes registering a sort task through AddTask() and then initializing the sort data through the KeyGen callback.
 * The sort manager can be enabled or disabled through "fx.AllowGPUSorting"
 */
class FGPUSortManager : public FRefCountedObject
{
public:

	/** Information about the bindings for a given sort task. */
	struct FAllocationInfo
	{
		/** The buffer that will hold the final sorted values */
		FShaderResourceViewRHIRef BufferSRV;
		/** The offset within BufferSRV where the sort task data starts. */
		uint32 BufferOffset = INDEX_NONE;
		/** The batch Id in which the sort task is grouped. */
		int32 SortBatchId = INDEX_NONE;
		/** The element index of the sort task within the batch. */
		int32 ElementIndex = INDEX_NONE;
	};

	/** A little helper to generate the batch element keys based on the number of elements in the batch and the sort precision. */
	struct FKeyGenInfo
	{
		ENGINE_API FKeyGenInfo(uint32 NumElements, bool bHighPrecisionKeys);

		/**
		 * ElementKey = (ElementIndex & EmitterKeyMask) << ElementKeyShift.
		 * SortKey = (Key32 >> SortKeyShift) & SortKeyMask.
		 */
		uint32 ElementKeyMask;
		uint32 ElementKeyShift;
		uint32 SortKeyMask;
		/** (SortKeyMask, SortKeyShift, SortKeySignBit, -) */
		FUintVector4 SortKeyParams; 
	};

private:

	/** Holds data relative to client systems of the GPU sort manager. */
	struct FCallbackInfo
	{
		/** A delegate used to initialize the keys and values. */
		FGPUSortKeyGenDelegate Delegate;
		/** The flag this client system might use when creating batches. */
		EGPUSortFlags Flags = EGPUSortFlags::None;
		/** The name of this client system. Used for GPU markers. */
		FName Name;
	};

	typedef TArray<FCallbackInfo> FCallbackArray;

	/** Different settings used to manage the buffer allocations. */
	struct FSettings
	{
		/** The union of all clients flags. */
		EGPUSortFlags AllowedFlags = EGPUSortFlags::None;
		/** The buffer slack when increasing or shrinking. See "fx.GPUSort.BufferSlack" */
		float BufferSlack = 2.f;
		/** The number of frames before shrinking a buffer that is too big. See "fx.GPUSort.FrameCountBeforeShrinking" */
		int32 FrameCountBeforeShrinking = 100;
		/** The minimal size of the sort buffers. See "fx.GPUSort.MinBufferSize" */
		int32 MinBufferSize = 8192;
	};

	/** A vertex buffer owning the final sorted values. */
	class FValueBuffer : public FVertexBuffer
	{
	public:

		/**
		 * Create a value buffer of a given size. The render resource are created immediately.
		 *
		 * @param InAllocatedCount - The buffer size, allocated now.
		 * @param InUsedCount - An initial used count, typically the used size of the previous buffer in FDynamicValueBuffer::ValueBuffers.
		 * @param InSettings - The buffer allocation settings, in particular what data types are requires for SRV and UAV.
		 */
		FValueBuffer(FRHICommandListBase& RHICmdList, int32 InAllocatedCount, int32 InUsedCount, const FSettings& InSettings);

		~FValueBuffer() { ReleaseRHI(); }

		/** Release the render resources. */
		void ReleaseRHI() override final;

		/** Used for sorting */
		FShaderResourceViewRHIRef UInt32SRV;
		FUnorderedAccessViewRHIRef UInt32UAV;
		/** EGPUSortFlags::IndicesAsInt32 */
		FShaderResourceViewRHIRef Int32SRV;
		FUnorderedAccessViewRHIRef Int32UAV;
		/** EGPUSortFlags::IndicesAsG16R16F */
		FShaderResourceViewRHIRef G16R16SRV;
		FUnorderedAccessViewRHIRef G16R16UAV;

		/** The actual buffer size. */
		int32 AllocatedCount = 0;
		/** The currently used size. */
		int32 UsedCount = 0;

		/**
		 * Allocate some space into the buffer. The buffer needs to be big enough to hold the allocation.
		 * 
		 * @param OutInfo - The details about the allocation bindings.
		 * @param ValueCount - The number of elements to add to the buffer.
		 * @param Flags - Flag about what data format needs to be bound, see EGPUSortFlags::AnyValueFormat.
		 */
		inline void Allocate(FAllocationInfo& OutInfo, int32 ValueCount, EGPUSortFlags Flags);
	};

	typedef TIndirectArray<FValueBuffer, TInlineAllocator<4>> FValueBufferArray;

	/**
	 * Encapsulates the idea of a growable FValueBuffer, that dynamically change in size depending on requirements.
	 * What is specific here is that even after being grown, previous references to the smaller version are still valid for a frame.
	 * This requirement comes client system registering GPU sort tasks in InitViews(), before the total size required for the frame is known.
	 */
	struct FDynamicValueBuffer
	{
		~FDynamicValueBuffer() { ReleaseRHI(); }

		/** The current list of increasingly bigger value buffers. There will only be more than one element the frame the buffer grows. */
		FValueBufferArray ValueBuffers;

		/** A counter used to delay shrinking by several frames. */
		int32 NumFramesRequiringShrinking = 0;

		/** The last batch flags that used this buffer. Used to help rebind the same SortBatch between frames. */
		EGPUSortFlags LastSortBatchFlags = EGPUSortFlags::None;

		/** The sort batch using this buffer for this frame. */
		int32 CurrentSortBatchId = INDEX_NONE;

		/**
		 * Allocate some space into the buffer for this frame.
		 * 
		 * @param OutInfo - The details about the allocation bindings.
		 * @param InSettings - Buffer allocation settings.
		 * @param ValueCount - The number of elements to add to the buffer.
		 * @param Flags - Flag about what data format needs to be bound, see EGPUSortFlags::AnyValueFormat.
		 */
		void Allocate(FRHICommandListBase& RHICmdList, FAllocationInfo& OutInfo, const FSettings& InSettings, int32 ValueCount, EGPUSortFlags Flags);

		/**
		 * Shrinks the buffer after it has been used this frame if applicable.
		 * Resets other data so that it can be reused on the next frame.
		 * This includes releasing the smaller buffers that where allocated 
		 * this frame if the buffer needed to be grown.
		 * 
		 * @param InSettings - Buffer allocation settings, in particular the number of frames required before actually shrinking.
		 */
		void SkrinkAndReset(FRHICommandListBase& RHICmdList, const FSettings& InSettings);
		
		/** Release resources */
		void ReleaseRHI();
		/** Get currently used size for this frame. */
		inline int32 GetUsedCount() const { return ValueBuffers.Num() ? ValueBuffers.Last().UsedCount : 0; }
		/** Get the buffer allocated size . */
		inline int32 GetAllocatedCount() const { return ValueBuffers.Num() ? ValueBuffers.Last().AllocatedCount : 0; }
	};

	typedef TIndirectArray<FDynamicValueBuffer, TInlineAllocator<4>> FDynamicValueBufferArray;

	/**
	 * Defines the order in which sort batches are processed every frame.  
	 * KeyGenAfterPreRenderAndSortAfterPostRenderOpaque is the most complex case since
	 * the KeyGen is called after PreRendeR() while the sort only happens after PostRenderOpaque().
	 * This creates a situation where the FParticleSortBuffers need to be keep bound to the batch
	 * and can not be reused by other batches with this processing order.
	 * Note also that KeyGenAfterPreRenderAndSortAfterPostRenderOpaque could include KeyGen after PostRenderOpaque().
	 */
	enum class ESortBatchProcessingOrder : uint32
	{
		Undefined = 0,
		KeyGenAndSortAfterPreRender = 1,
		KeyGenAfterPreRenderAndSortAfterPostRenderOpaque = 2,
		KeyGenAndSortAfterPostRenderOpaque = 3,
	};

	/**
	 * Defines a sort batch that regroups several sort tasks created by FGPUSortManager::AddTask(), becoming sort the batch elements.
	 * The condition to regroup task in the same batch rely mostly around the key precisions and the sort location (PreRender or PostRenderOpaque).
	 * Each SortBatch maps to a single GPU sort task.
	 */
	struct FSortBatch
	{
		/** An Id defining this batch. Passed along FGPUSortKeyGenDelegate. */
		int32 Id = INDEX_NONE;
		/** The number of elements (tasks) grouped in this batch. */
		int32 NumElements = 0;

		/**
		 * The common requirement flag for this batch. Compatible with each element.
		 * As tasks are added, the flag can become more restrictive, and new registered tasks 
		 * could require creating new batches if their requirements are too different.
		 */
		EGPUSortFlags Flags = EGPUSortFlags::None;

		/** The buffer to hold the final sorted values. */
		FDynamicValueBuffer* DynamicValueBuffer = nullptr;

		/** The scratch buffers used to hold the initial keys and values and perform the GPU sort. */
		FParticleSortBuffers* SortBuffers = nullptr;

		/** This batch processing order in the frame. */
		ESortBatchProcessingOrder ProcessingOrder = ESortBatchProcessingOrder::Undefined;

		/** Compute the processing order from the current value of Flags. */
		void UpdateProcessingOrder();

		/**
		 * Call each delegate registered in the FGPUSortManager so they setup their task data for each element they own in the batch. 
		 * 
		 * @param RHICmdList - The command list used to perform the task.
		 * @param InCallbacks - The client system callbacks, from FGPUSortManager::Register().
		 * @param KeyGenLocation - The location in the frame where we are generating keys (see EGPUSortFlags::AnyKeyGenLocation).
		 */
		void GenerateKeys(FRHICommandListImmediate& RHICmdList, const FCallbackArray& InCallbacks, EGPUSortFlags KeyGenLocation);

		/** 
		 * Perform the GPU sort and resolve to all buffers referenced in FGPUSortManager::AddTask(), if the buffer grew in size.
		 *
		 * @param RHICmdList - The command list used to perform the dispatches.
		 * @param InFeatureLevel - The current feature level, used to access the global shadermap.
		 */
		void SortAndResolve(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel);

		/** Get batch currently used size for this frame. */
		inline int32 GetUsedValueCount() const { return DynamicValueBuffer ? DynamicValueBuffer->GetUsedCount() : 0; }
		/** Get the batch buffer allocated size. */
		inline int32 GetAllocatedValueCount() const { return DynamicValueBuffer ? DynamicValueBuffer->GetAllocatedCount() : 0; }
	};

	/** Expected that few will be enough, since we only need severals batches when ESortBatchProcessingOrder::KeyGenAfterPreRenderAndSortAfterPostRenderOpaque. */
	typedef TArray<FSortBatch, TInlineAllocator<4>> FSortBatchArray;

public:

	/** Creates the sort manager, this is when the settings are configured. */
	ENGINE_API FGPUSortManager(ERHIFeatureLevel::Type InFeatureLevel);

	ENGINE_API ~FGPUSortManager();

	/**
	 * Register a client system into the sort manager. Client systems are systems that will call AddTask().
	 * Those systems need to register a callback, the possible used flags for their tasks and a name.
	 * The callback is required because the initial content to perform GPU sorts is only configured long after
	 * AddTask() gets called in the rendering loop.
	 *
	 * @param CallbackDelegate - The callback that will be used to set the initial keys and values for each batch elements.
	 * @param UsedFlags - The possibly used flags for each of the client system tasks, used to perform some optimizations.
	 * @param InName - A name defining this client system, used for GPU markers.
	 */
	ENGINE_API void Register(const FGPUSortKeyGenDelegate& CallbackDelegate, EGPUSortFlags UsedFlags, const FName& InName);	

	/**
	 * Add a GPU sort task to process this frame. Tasks are expected to be created before OnPreRender() gets called.
	 *
	 * @param OutInfo - The details about the allocation bindings, if success.
	 * @param IndexCount - The number of values that need to be sorted by this task.
	 * @param SortFlags - The sort task requirements.
	 * 
	 * Return true if the task was added to a batch, false otherwise, could be because GPU sorting is disabled.
	 */
	ENGINE_API bool AddTask(FRHICommandListBase& RHICmdList, FAllocationInfo& OutInfo, int32 ValueCount, EGPUSortFlags TaskFlags);

	/**
	 * Callback that needs to be called in the rendering loop, after calls to FFXSystemInterface::PreRender() are issued.
	 * At this point, no other GPU sort tasks are allowed to be created through AddTask() as the sort batches
	 * get finalized. At this point, any task with the EGPUSortFlags::SortAfterPreRender will possibly be done,
	 * unless it also had the EGPUSortFlags::SortAfterPostRenderOpaque flag.
	 * Tasks with EGPUSortFlags::KeyGenAfterPreRender get their client callbacks issued to set the initial keys and values. 
	 *
	 * @param RHICmdList - The command list to be used.
	 */
	ENGINE_API void OnPreRender(class FRDGBuilder& GraphBuilder);

	/**
	 * Callback that needs to be called in the rendering loop, after calls to FFXSystemInterface::PostRenderOpaque() are issued.
	 * Tasks with EGPUSortFlags::KeyGenAfterPostRenderOpaque get their client callbacks issued to set the initial keys and values. 
	 * After this any sort task that hasn't been processed yet, in OnPreRender(), gets processed now.
	 *
	 * @param RHICmdList - The command list to be used.
	 */
	ENGINE_API void OnPostRenderOpaque(class FRDGBuilder& GraphBuilder);

	/**
	 * Event to register and receive post-prerender notification.
	 */	
	DECLARE_EVENT_OneParam(FGPUSortManager, FPostPreRenderEvent, FRHICommandListImmediate&);
	FPostPreRenderEvent PostPreRenderEvent;

	/**
	 * Event to register and receive post-postrender notification.
	 */
	DECLARE_EVENT_OneParam(FGPUSortManager, FPostPostRenderEvent, FRHICommandListImmediate&);
	FPostPreRenderEvent PostPostRenderEvent;

private:

	/** A helper to test that a flag has one and exactly one of A or B. */
	static inline bool HasEither(EGPUSortFlags Flags, EGPUSortFlags A, EGPUSortFlags B);
	/** Create the initial batch flags from a task flags. */
	static inline EGPUSortFlags GetBatchFlags(EGPUSortFlags TaskFlags);
	/** Update a batch flags after adding a task into it. */
	static inline EGPUSortFlags CombineBatchFlags(EGPUSortFlags BatchFlags, EGPUSortFlags TaskFlags);
	/** Test whether a batch flags are compatible with a given task. Used to know if the task can be merged in the batch.  */
	static inline bool TestBatchFlags(EGPUSortFlags BatchFlags, EGPUSortFlags TaskFlags);
	/** Convert the precision flags into a string, used for GPU markers. */
	static inline auto GetPrecisionString(EGPUSortFlags BatchFlags) -> TCHAR const(&)[1];

	/**
	 * Find an unused buffer to assign to a newly created FSortBatch. 
	 * Although the sort batches are recreated every frame depending on what is being visible,
	 * the buffers used to sort the indices and store the sorted results are persistent to some extent 
	 * (see FBufferAllocationSettings which define the allocation and lifetime strategy).
	 *
	 * Each persistent buffer being assigned to one frame batch, we try to rebind the same buffers to the same batches
	 * by using the constraints being imposed on the batch through the sortflags. The reason to do so is that 
	 * the allocation strategy for the buffer is meant to be optimal if the required buffer size is consistent between frames.
	 * (the batch flags are linked to the emitters being grouped into it, and so is the batch sorted indices count)
	 *
	 * @param TaskFlags - Some constraints being applied to the batch, using to help match the same buffers to the same batches between frames.
	 * @param SortBatchId - The newly created batch index which will use the buffer this frame.
	 * 
	 * Return a buffer to store the sorted particle indices.
	 */
	FDynamicValueBuffer* GetDynamicValueBufferFromPool(EGPUSortFlags TaskFlags, int32 SortBatchId);

	/** Setup the final sort flags and the processing order of all batches. Called after no other tasks will be added this frame. */
	void FinalizeSortBatches();
	/** Make sure there is enough GPU sort buffers to satisfy all batches created this frame. Free one used ones. */
	void UpdateSortBuffersPool(FRHICommandListBase& RHICmdList);
	/** Resize (shrink) the DynamicValueBuffers and free the unused ones. */
	void ResetDynamicValuesBuffers(FRHICommandListBase& RHICmdList);
	/** Delete all GPU sort buffers. */
	void ReleaseSortBuffers();

	/** Used to ensure no concurrent access during AddTask */
	UE::FMutex AddTaskMutex;

	/**
	 * The different batches that will be used this frame, each one ending up being sorting work on the GPU.
	 * In many cases, there will only be one batch but some cases makes it possible to have 2 
	 * or more (for example when different key precision are used).
	 */
	FSortBatchArray SortBatches;

	/** Settings to handle the buffer reallocation strategy. */
	FSettings Settings;

	/** A pool of buffer to hold the final sorted values for each batch. Reused between frames. */
	FDynamicValueBufferArray DynamicValueBufferPool;

	/** A pool of GPU sort buffers. The same sort GPU buffer can normally be used in several sort batches. */
	TArray<FParticleSortBuffers*, TInlineAllocator<2>> SortBuffersPool;

	/** The client system callbacks, see Register(). */
	FCallbackArray Callbacks;

	/** The current feature level, needed to access the global shader map. */
	ERHIFeatureLevel::Type FeatureLevel;
};
