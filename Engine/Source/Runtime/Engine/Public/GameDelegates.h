// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

struct FUniqueNetIdRepl;

/**
 * Collection of delegates for various components to call into game code
 */

// enum class with size so it can be forward declared.
enum class EGameDelegates_SaveGame : short
{	
	MaxSize,
	Icon,
	Title,
	SubTitle,
	Detail,	
};

/** Delegate to modify cooking behavior - return extra packages to cook, load up the asset registry, etc */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FModifyCookDelegate,
	TConstArrayView<const ITargetPlatform*> /* InTargetPlatforms */, TArray<FName>& /* InOutPackagesToCook */,
	TArray<FName>& /* InOutPackagesToNeverCook */);

/** Delegate to assign a disc layer to a chunk */
typedef const TMap<FName, FString> FAssignLayerChunkMap;
DECLARE_DELEGATE_FourParams(FAssignLayerChunkDelegate, const FAssignLayerChunkMap* /*ChunkManifest*/, const FString& /*Platform*/, const int32 /*ChunkIndex*/, int32& /*OutChunkLayer*/);

/** A delegate for platforms that need extra information to flesh out save data information (name of an icon, for instance) */
DECLARE_DELEGATE_ThreeParams(FExtendedSaveGameInfoDelegate, const TCHAR* /*SaveName*/, const EGameDelegates_SaveGame /*Key*/, FString& /*Value*/);

/** A delegate for a web server running in engine to tell the game about events received from a client, and for game to respond to the client */
// using a TMap in the DECLARE_DELEGATE macro caused compiler problems (in clang anyway), a typedef solves it
typedef TMap<FString, FString> StringStringMap;
DECLARE_DELEGATE_FiveParams(FWebServerActionDelegate, int32 /*UserIndex*/, const FString& /*Action*/, const FString& /*URL*/, const StringStringMap& /*Params*/, StringStringMap& /*Response*/);

/** Delegate called before a map change at runtime */
DECLARE_MULTICAST_DELEGATE_TwoParams(FPreCommitMapChangeDelegate, const FString& /*PreviousMapName*/, const FString& /*NextMapName*/);

/** Delegate called when a connection is lost, but there is no owning actor to handle it */
DECLARE_MULTICAST_DELEGATE_OneParam(FPendingConnectionLostDelegate, const FUniqueNetIdRepl& /*ConnectionUniqueId*/);
/** Delegate to handle when a connection is disconnecting */
DECLARE_MULTICAST_DELEGATE_TwoParams(FHandleDisconnectDelegate, class UWorld* /*InWorld*/, class UNetDriver* /*NetDriver*/);

/** Delegate to allow game to use a custom ICookedEditorPackageManager class */
DECLARE_DELEGATE_RetVal(TUniquePtr<class ICookedEditorPackageManager>, FCookedEditorPackageManagerFactoryDelegate);

/** Delegate to inform those interested that a view target has changed. */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FViewTargetChangedDelegate, class APlayerController* /*PC*/, class AActor* /*OldViewTarget*/, class AActor* /*NewViewTarget*/);

// Helper defines to make defining the delegate members easy
#define DEFINE_GAME_DELEGATE(DelegateType) \
	public: F##DelegateType& Get##DelegateType() { return DelegateType; } \
	private: F##DelegateType DelegateType;

#define DEFINE_GAME_DELEGATE_TYPED(DelegateVariable, DelegateType) \
	public: DelegateType& Get##DelegateVariable() { return DelegateVariable; } \
	private: DelegateType DelegateVariable;

/** Class to set and get game callbacks */
class FGameDelegates
{
public:

	/** Return a single FGameDelegates object */
	static ENGINE_API FGameDelegates& Get();

	// Called when an exit command is received
	DEFINE_GAME_DELEGATE_TYPED(ExitCommandDelegate, FSimpleMulticastDelegate);

	// Called when ending playing a map
	DEFINE_GAME_DELEGATE_TYPED(EndPlayMapDelegate, FSimpleMulticastDelegate);

	// Called when a pending connection has been lost 
	DEFINE_GAME_DELEGATE_TYPED(PendingConnectionLostDelegate, FPendingConnectionLostDelegate);

	// Called when pre/post committing a map change at runtime
	DEFINE_GAME_DELEGATE(PreCommitMapChangeDelegate);
	DEFINE_GAME_DELEGATE_TYPED(PostCommitMapChangeDelegate, FSimpleMulticastDelegate);

	// Called when a player is disconnecting due to network failure
	DEFINE_GAME_DELEGATE(HandleDisconnectDelegate);

	//Called when a PlayerController view target changes.
	DEFINE_GAME_DELEGATE_TYPED(ViewTargetChangedDelegate, FViewTargetChangedDelegate);

	// Implement all delegates declared above
	DEFINE_GAME_DELEGATE(AssignLayerChunkDelegate);
	DEFINE_GAME_DELEGATE(ExtendedSaveGameInfoDelegate);
	DEFINE_GAME_DELEGATE(WebServerActionDelegate);	

	// Called at start of CookByTheBook
	DEFINE_GAME_DELEGATE(ModifyCookDelegate);

	DEFINE_GAME_DELEGATE(CookedEditorPackageManagerFactoryDelegate);
};
