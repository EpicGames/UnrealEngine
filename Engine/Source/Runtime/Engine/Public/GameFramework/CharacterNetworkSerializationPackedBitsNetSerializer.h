// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "CharacterNetworkSerializationPackedBitsNetSerializer.generated.h"

USTRUCT()
struct FCharacterNetworkSerializationPackedBitsNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

	// Value used to sanity check incoming data so that we do not over-allocate dynamic memory
	UPROPERTY()
	uint32 MaxAllowedDataBits = 8192; 
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FCharacterNetworkSerializationPackedBitsNetSerializer, ENGINE_API);

}
