// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/Prioritization/NetObjectCountLimiter.h"
#include "PlayerStateCountLimiterConfig.generated.h"

/**
 * If one wishes to use the NetObjectCountLimiter prioritizer for PlayerStates
 * this class makes it possible to have specific settings for PlayerStates in
 * case other classes also wish to use said prioritizer.
 * Only available if running with Iris replication.
 * The PlayerState will look for a prioritizer aptly named PlayerState and
 * set that prioritizer for it. It could be any kind of prioritizer but
 * if the NetObjectCountLimiter is used this config will come in handy.
 */
UCLASS(transient, config=Engine, MinimalAPI)
class UPlayerStateCountLimiterConfig : public UNetObjectCountLimiterConfig
{
	GENERATED_BODY()

public:
	ENGINE_API UPlayerStateCountLimiterConfig();
};
