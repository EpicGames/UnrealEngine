// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/PolymorphicNetSerializer.h"
#include "RootMotionSourceGroupNetSerializer.generated.h"

USTRUCT()
struct FRootMotionSourceGroupNetSerializerConfig : public FPolymorphicArrayStructNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FRootMotionSourceGroupNetSerializer, ENGINE_API);

}
