// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "UniqueNetIdReplNetSerializer.generated.h"

USTRUCT()
struct FUniqueNetIdReplNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FUniqueNetIdReplNetSerializer, ENGINE_API);

}
