// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"

/** Contains all the timings of a gaming frame, to handle pause and time dilation (for instance bullet time) of the world. */
struct FGameTime
{
	inline FGameTime()
		: RealTimeSeconds(0.0)
		, WorldTimeSeconds(0.0)
		, DeltaRealTimeSeconds(0.0f)
		, DeltaWorldTimeSeconds(0.0f)
	{ }

	FGameTime(const FGameTime&) = default;
	FGameTime& operator = (const FGameTime&) = default;

	// Returns the game time since GStartTime.
	static ENGINE_API FGameTime GetTimeSinceAppStart();

	static inline FGameTime CreateUndilated(double InRealTimeSeconds, float InDeltaRealTimeSeconds)
	{
		return FGameTime::CreateDilated(InRealTimeSeconds, InDeltaRealTimeSeconds, InRealTimeSeconds, InDeltaRealTimeSeconds);
	}

	static inline FGameTime CreateDilated(double InRealTimeSeconds, float InDeltaRealTimeSeconds, double InWorldTimeSeconds, float InDeltaWorldTimeSeconds)
	{
		return FGameTime(InRealTimeSeconds, InDeltaRealTimeSeconds, InWorldTimeSeconds, InDeltaWorldTimeSeconds);
	}

	/** Returns time in seconds since level began play, but IS NOT paused when the game is paused, and IS NOT dilated/clamped. */
	inline double GetRealTimeSeconds() const
	{
		return RealTimeSeconds;
	}

	/** Returns frame delta time in seconds with no adjustment for time dilation and pause. */
	inline float GetDeltaRealTimeSeconds() const
	{
		return DeltaRealTimeSeconds;
	}

	/** Returns time in seconds since level began play, but IS paused when the game is paused, and IS dilated/clamped. */
	inline double GetWorldTimeSeconds() const
	{
		return WorldTimeSeconds;
	}

	/** Returns frame delta time in seconds adjusted by e.g. time dilation. */
	inline float GetDeltaWorldTimeSeconds() const
	{
		return DeltaWorldTimeSeconds;
	}

	/** Returns how much world time is slowed compared to real time. */
	inline float GetTimeDilation() const
	{
		ensure(DeltaRealTimeSeconds > 0.0f);
		return DeltaWorldTimeSeconds / DeltaRealTimeSeconds;
	}

	/** Returns whether the world time is paused. */
	inline bool IsPaused() const
	{
		return DeltaWorldTimeSeconds == 0.0f;
	}

private:
	double RealTimeSeconds;
	double WorldTimeSeconds;

	float DeltaRealTimeSeconds;
	float DeltaWorldTimeSeconds;

	inline FGameTime(double InRealTimeSeconds, float InDeltaRealTimeSeconds, double InWorldTimeSeconds, float InDeltaWorldTimeSeconds)
		: RealTimeSeconds(InRealTimeSeconds)
		, WorldTimeSeconds(InWorldTimeSeconds)
		, DeltaRealTimeSeconds(InDeltaRealTimeSeconds)
		, DeltaWorldTimeSeconds(InDeltaWorldTimeSeconds)
	{ }

};
