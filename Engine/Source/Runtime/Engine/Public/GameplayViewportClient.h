// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportClient.h"

/** Defines behavior of viewport clients that draw the contents of a UGameInstance world. */
class FGameplayViewportClient : public FCommonViewportClient
{
public:
	// FViewportClient interface
	ENGINE_API virtual bool InputChar(FViewport* const Viewport, const int32 ControllerId, const TCHAR Character) override;
	ENGINE_API virtual bool IsFocused(FViewport* Viewport) override;
	ENGINE_API virtual bool RequiresHitProxyStorage() override;
	ENGINE_API virtual bool ShouldDPIScaleSceneCanvas() const override;
	// ~FViewportClient interface
};
