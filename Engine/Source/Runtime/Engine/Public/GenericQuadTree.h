// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogQuadTree, Log, Log);

template <typename ElementType, int32 NodeCapacity = 4>
class TQuadTree
{
	typedef TQuadTree<ElementType, NodeCapacity> TreeType;
public:

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	TQuadTree();

	TQuadTree(const FBox2D& InBox, float InMinimumQuadSize = 100.f);

	/** Gets the TreeBox so systems can test insertions before trying to do so with invalid regions */
	const FBox2D& GetTreeBox() const { return TreeBox; }

	/** Inserts an object of type ElementType with an associated 2D box of size Box (log n). Pass in a DebugContext so when an issue occurs the log can report what requested this insert. */
	void Insert(const ElementType& Element, const FBox2D& Box, const TCHAR* DebugContext = nullptr);

	/** Given a 2D box, returns an array of elements within the box. There will not be any duplicates in the list. */
	template<typename ElementAllocatorType>
	void GetElements(const FBox2D& Box, TArray<ElementType, ElementAllocatorType>& ElementsOut) const;

	/** Given a 2D box, iterates over intersecting elements and execute the callback for each.
	*   If Callback returns true, it will continue processing, if it returns false, iteration will end.*/
	template<typename CallbackType>
	inline void GetElements(const FBox2D& Box, CallbackType&& Callback) const;

	/** Removes an object of type ElementType with an associated 2D box of size Box (log n). Does not cleanup tree*/
	bool Remove(const ElementType& Instance, const FBox2D& Box);

	/** Does a deep copy of the tree by going through and re-creating the internal data. Cheaper than re-insertion as it should be linear instead of nlogn */
	void Duplicate(TreeType& OutDuplicate) const;

	/** Removes all elements of the tree */
	void Empty();

	void Serialize(FArchive& Ar);

	TQuadTree(const TQuadTree&);
	TQuadTree& operator=(const TQuadTree& Other);

	~TQuadTree();

private:
	enum QuadNames
	{
		TopLeft = 0,
		TopRight = 1,
		BottomLeft = 2,
		BottomRight = 3
	};

	/** Node used to hold the element and its corresponding 2D box*/
	struct FNode
	{
		FBox2D Box;
		ElementType Element;

		FNode() {};

		FNode(const ElementType& InElement, const FBox2D& InBox)
			: Box(InBox)
			, Element(InElement)
		{}

		friend FArchive& operator<<(FArchive& Ar, typename TQuadTree<ElementType, NodeCapacity>::FNode& Node)
		{
			return Ar << Node.Box << Node.Element;
		}
	};

	/** Given a 2D box, return the subtrees that are touched. Returns 0 for leaves. */
	int32 GetQuads(const FBox2D& Box, TreeType* Quads[4]) const;

	/** Split the tree into 4 sub-trees */
	void Split();

	/** Given a list of nodes, return which ones actually intersect the box */
	template<typename ElementAllocatorType>
	void GetIntersectingElements(const FBox2D& Box, TArray<ElementType, ElementAllocatorType>& ElementsOut) const;

	/** Given a list of nodes, execute Callback for any elements that intersect the box */
	template<typename CallbackType>
	bool GetIntersectingElements(const FBox2D& Box, CallbackType& Callback) const;

	/** Given a list of nodes, remove the node that contains the given element */
	bool RemoveNodeForElement(const ElementType& Element);

	/** Internal recursive implementation of @see Insert */
	void InsertElementRecursive(const ElementType& Element, const FBox2D& Box, const TCHAR* DebugContext);

private:

	/**
	 * Contains the actual elements this tree is responsible for. Nodes are used to keep track of each element's AABB as well.
	 * For a non-internal leaf, this is the list of nodes that are fully contained within this tree.
	 * For an internal tree, this contains the nodes that overlap multiple subtrees.
	 */
	TArray<FNode> Nodes;

	/** The sub-trees of this tree */
	TreeType* SubTrees[4] = {nullptr};

	/** AABB of the tree */
	FBox2D TreeBox;

	/** Center position of the tree */
	FVector2D Position;

	/** The smallest size of a quad allowed in the tree */
	float MinimumQuadSize;

	/** Whether this is a leaf or an internal sub-tree */
	bool bInternal;
};

template <typename ElementType, int32 NodeCapacity /*= 4*/>
TQuadTree<ElementType, NodeCapacity>::TQuadTree(const TQuadTree& Other)
{
	Other.Duplicate(*this);
}

template <typename ElementType, int32 NodeCapacity /*= 4*/>
TQuadTree<ElementType, NodeCapacity>& TQuadTree<ElementType, NodeCapacity>::operator=(const TQuadTree& Other)
{
	Other.Duplicate(*this);
	return *this;
}

template <typename ElementType, int32 NodeCapacity /*= 4*/>
void TQuadTree<ElementType, NodeCapacity>::Serialize(FArchive& Ar)
{
	Ar << Nodes;

	bool SubTreeFlags[4] = { SubTrees[0] != nullptr, SubTrees[1] != nullptr, SubTrees[2] != nullptr, SubTrees[3] != nullptr };
	Ar << SubTreeFlags[0] << SubTreeFlags[1] << SubTreeFlags[2] << SubTreeFlags[3];

	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		if (SubTreeFlags[Idx])
		{
			if (Ar.IsLoading())
			{
				SubTrees[Idx] = new TreeType(FBox2D(), MinimumQuadSize);
			}

			SubTrees[Idx]->Serialize(Ar);
		}
	}

	Ar << TreeBox;
	Ar << Position;
	Ar << bInternal;
}

template <typename ElementType, int32 NodeCapacity>
TQuadTree<ElementType, NodeCapacity>::TQuadTree(const FBox2D& Box, float InMinimumQuadSize)
	: TreeBox(Box)
	, Position(Box.GetCenter())
	, MinimumQuadSize(InMinimumQuadSize)
	, bInternal(false)
{
	SubTrees[0] = SubTrees[1] = SubTrees[2] = SubTrees[3] = nullptr;
}

template <typename ElementType, int32 NodeCapacity>
TQuadTree<ElementType, NodeCapacity>::TQuadTree()
{
	EnsureRetrievingVTablePtrDuringCtor(TEXT("TQuadTree()"));
}

template <typename ElementType, int32 NodeCapacity>
TQuadTree<ElementType, NodeCapacity>::~TQuadTree()
{
	for (TreeType* SubTree : SubTrees)
	{
		delete SubTree;
		SubTree = nullptr;
	}
}

template <typename ElementType, int32 NodeCapacity>
void TQuadTree<ElementType, NodeCapacity>::Split()
{
	check(bInternal == false);

	const FVector2D Extent = TreeBox.GetExtent();
	const FVector2D XExtent = FVector2D(Extent.X, 0.f);
	const FVector2D YExtent = FVector2D(0.f, Extent.Y);

	/************************************************************************
	 *  ___________max
	 * |     |     |
	 * |     |     |
	 * |-----c------
	 * |     |     |
	 * min___|_____|
	 *
	 * We create new quads by adding xExtent and yExtent
	 ************************************************************************/

	const FVector2D C = Position;
	const FVector2D TM = C + YExtent;
	const FVector2D ML = C - XExtent;
	const FVector2D MR = C + XExtent;
	const FVector2D BM = C - YExtent;
	const FVector2D BL = TreeBox.Min;
	const FVector2D TR = TreeBox.Max;

	SubTrees[TopLeft] = new TreeType(FBox2D(ML, TM), MinimumQuadSize);
	SubTrees[TopRight] = new TreeType(FBox2D(C, TR), MinimumQuadSize);
	SubTrees[BottomLeft] = new TreeType(FBox2D(BL, C), MinimumQuadSize);
	SubTrees[BottomRight] = new TreeType(FBox2D(BM, MR), MinimumQuadSize);

	//mark as no longer a leaf
	bInternal = true;

	// Place existing nodes and place them into the new subtrees that contain them
	// If a node overlaps multiple subtrees, we retain the reference to it here in this quad
	TArray<FNode> OverlappingNodes;
	for (const FNode& Node : Nodes)
	{
		TreeType* Quads[4];
		const int32 NumQuads = GetQuads(Node.Box, Quads);
		check(NumQuads > 0);

		if (NumQuads == 1)
		{
			Quads[0]->Nodes.Add(Node);
		}
		else
		{
			OverlappingNodes.Add(Node);
		}
	}

	// Hang onto the nodes that don't fit cleanly into a single subtree
	Nodes = OverlappingNodes;
}

template <typename ElementType, int32 NodeCapacity>
void TQuadTree<ElementType, NodeCapacity>::Insert(const ElementType& Element, const FBox2D& Box, const TCHAR* DebugContext)
{
#if !UE_BUILD_SHIPPING
	if (!Box.Intersect(TreeBox))
	{
		// Elements shouldn't be added outside the bounds of the top-level quad
		UE_LOGF(LogQuadTree, Warning, "[%ls] Adding element (%ls) that is outside the bounds of the quadtree root (%ls). Consider resizing.", DebugContext ? DebugContext : TEXT("Unknown Source"), *Box.ToString(), *TreeBox.ToString());
	}
#endif // !UE_BUILD_SHIPPING

	InsertElementRecursive(Element, Box, DebugContext);
}

template <typename ElementType, int32 NodeCapacity>
void TQuadTree<ElementType, NodeCapacity>::InsertElementRecursive(const ElementType& Element, const FBox2D& Box, const TCHAR* DebugContext)
{
	TreeType* Quads[4];
	const int32 NumQuads = GetQuads(Box, Quads);
	if (NumQuads == 0)
	{
		// This should only happen for leaves
		check(!bInternal);

		// It's possible that all elements in the leaf are bigger than the leaf or that more elements than NodeCapacity exist outside the top level quad
		// In either case, we can get into an endless spiral of splitting
		const bool bCanSplitTree = TreeBox.GetSize().SizeSquared() > FMath::Square(MinimumQuadSize);
		if (!bCanSplitTree || Nodes.Num() < NodeCapacity)
		{
			Nodes.Add(FNode(Element, Box));

			if (!bCanSplitTree)
			{
				UE_LOGF(LogQuadTree, Verbose, "[%ls] Minimum size %f reached for quadtree at %ls. Filling beyond capacity %d to %d", DebugContext ? DebugContext : TEXT("Unknown Source"), MinimumQuadSize, *Position.ToString(), NodeCapacity, Nodes.Num());
			}
		}
		else
		{
			// This quad is at capacity, so split and try again
			Split();
			InsertElementRecursive(Element, Box, DebugContext);
		}
	}
	else if (NumQuads == 1)
	{
		check(bInternal);

		// Fully contained in a single subtree, so insert it there
		Quads[0]->InsertElementRecursive(Element, Box, DebugContext);
	}
	else
	{
		// Overlaps multiple subtrees, store here
		check(bInternal);
		Nodes.Add(FNode(Element, Box));
	}
}

template <typename ElementType, int32 NodeCapacity>
bool TQuadTree<ElementType, NodeCapacity>::RemoveNodeForElement(const ElementType& Element)
{
	int32 ElementIdx = INDEX_NONE;
	for (int32 NodeIdx = 0, NumNodes = Nodes.Num(); NodeIdx < NumNodes; ++NodeIdx)
	{
		if (Nodes[NodeIdx].Element == Element)
		{
			ElementIdx = NodeIdx;
			break;
		}
	}

	if (ElementIdx != INDEX_NONE)
	{
		Nodes.RemoveAtSwap(ElementIdx, EAllowShrinking::No);
		return true;
	}

	return false;
}

template <typename ElementType, int32 NodeCapacity>
bool TQuadTree<ElementType, NodeCapacity>::Remove(const ElementType& Element, const FBox2D& Box)
{
	bool bElementRemoved = false;

	TreeType* Quads[4];
	const int32 NumQuads = GetQuads(Box, Quads);

	// Remove from nodes referenced by this quad
	bElementRemoved = RemoveNodeForElement(Element);

	// Try to remove from subtrees if necessary
	for (int32 QuadIndex = 0; QuadIndex < NumQuads && !bElementRemoved; QuadIndex++)
	{
		bElementRemoved = Quads[QuadIndex]->Remove(Element, Box);
	}

	return bElementRemoved;
}

template <typename ElementType, int32 NodeCapacity>
template <typename ElementAllocatorType>
void TQuadTree<ElementType, NodeCapacity>::GetElements(const FBox2D& Box, TArray<ElementType, ElementAllocatorType>& ElementsOut) const
{
	TreeType* Quads[4];
	const int32 NumQuads = GetQuads(Box, Quads);

	// Always include any nodes contained in this quad
	GetIntersectingElements(Box, ElementsOut);

	// As well as all relevant subtrees
	for (int32 QuadIndex = 0; QuadIndex < NumQuads; QuadIndex++)
	{
		Quads[QuadIndex]->GetElements(Box, ElementsOut);
	}
}

template <typename ElementType, int32 NodeCapacity>
template <typename ElementAllocatorType>
void TQuadTree<ElementType, NodeCapacity>::GetIntersectingElements(const FBox2D& Box, TArray<ElementType, ElementAllocatorType>& ElementsOut) const
{
	ElementsOut.Reserve(ElementsOut.Num() + Nodes.Num());
	for (const FNode& Node : Nodes)
	{
		if (Box.Intersect(Node.Box))
		{
			//Debug performance will be at least 1 order slower
			checkSlow(!ElementsOut.Contains(Node.Element));
			ElementsOut.Add(Node.Element);
		}
	}
}

template <typename ElementType, int32 NodeCapacity>
template<typename CallbackType>
void TQuadTree<ElementType, NodeCapacity>::GetElements(const FBox2D& Box, CallbackType&& Callback) const
{
	TreeType* Quads[4];
	const int32 NumQuads = GetQuads(Box, Quads);

	// Always include any nodes contained in this quad
	if (!GetIntersectingElements(Box, Callback))
	{
		return;
	}

	// As well as all relevant subtrees
	for (int32 QuadIndex = 0; QuadIndex < NumQuads; QuadIndex++)
	{
		Quads[QuadIndex]->GetElements(Box, Callback);
	}
}

template <typename ElementType, int32 NodeCapacity>
template<typename CallbackType>
bool TQuadTree<ElementType, NodeCapacity>::GetIntersectingElements(const FBox2D& Box, CallbackType& Callback) const
{
	for (const FNode& Node : Nodes)
	{
		// Only execute the callback if the boxes intersect
		// Return if the Callback returns false, ending iteration
		if (Box.Intersect(Node.Box) && !Callback(Node.Element))
		{
			return false;
		}
	}

	return true;
}

template <typename ElementType, int32 NodeCapacity>
int32 TQuadTree<ElementType, NodeCapacity>::GetQuads(const FBox2D& Box, TreeType* Quads[4]) const
{
	int32 QuadCount = 0;
	if (bInternal)
	{
		bool bNegX = Box.Min.X <= Position.X;
		bool bNegY = Box.Min.Y <= Position.Y;

		bool bPosX = Box.Max.X >= Position.X;
		bool bPosY = Box.Max.Y >= Position.Y;

		if (bNegX && bNegY)
		{
			Quads[QuadCount++] = SubTrees[BottomLeft];
		}

		if (bPosX && bNegY)
		{
			Quads[QuadCount++] = SubTrees[BottomRight];
		}

		if (bNegX && bPosY)
		{
			Quads[QuadCount++] = SubTrees[TopLeft];
		}

		if (bPosX && bPosY)
		{
			Quads[QuadCount++] = SubTrees[TopRight];
		}
	}

	return QuadCount;
}

template <typename ElementType, int32 NodeCapacity>
void TQuadTree<ElementType, NodeCapacity>::Duplicate(TreeType& OutDuplicate) const
{
	for (int32 TreeIdx = 0; TreeIdx < 4; ++TreeIdx)
	{
		if (TreeType* SubTree = SubTrees[TreeIdx])
		{
			OutDuplicate.SubTrees[TreeIdx] = new TreeType(FBox2D(0, 0), MinimumQuadSize);
			SubTree->Duplicate(*OutDuplicate.SubTrees[TreeIdx]);	//duplicate sub trees
		}

	}

	OutDuplicate.Nodes = Nodes;
	OutDuplicate.TreeBox = TreeBox;
	OutDuplicate.Position = Position;
	OutDuplicate.MinimumQuadSize = MinimumQuadSize;
	OutDuplicate.bInternal = bInternal;
}

template <typename ElementType, int32 NodeCapacity>
void TQuadTree<ElementType, NodeCapacity>::Empty()
{
	for (int32 TreeIdx = 0; TreeIdx < 4; ++TreeIdx)
	{
		if (TreeType* SubTree = SubTrees[TreeIdx])
		{
			delete SubTree;
			SubTrees[TreeIdx] = nullptr;
		}

	}

	Nodes.Empty();
	bInternal = false;
}
