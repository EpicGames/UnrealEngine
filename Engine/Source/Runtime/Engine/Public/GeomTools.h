// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UStaticMesh;

/** An edge in 3D space, used by these utility functions. */
struct FUtilEdge3D
{
	/** Start of edge in 3D space. */
	FVector3f V0;
	/** End of edge in 3D space. */
	FVector3f V1;
};

/** An edge in 2D space, used by these utility functions. */
struct FUtilEdge2D
{
	/** Start of edge in 2D space. */
	FVector2D V0;
	/** End of edge in 2D space. */
	FVector2D V1;
};

/** A triangle vertex in 2D space, with UV information. */
struct FUtilVertex2D
{
	FVector2D Pos;
	FColor Color;
	FVector2D UV;
	bool bInteriorEdge;

	FUtilVertex2D() {}

	FUtilVertex2D(const FVector2D& InPos) 
		: Pos(InPos), Color(255,255,255,255), UV(0.f, 0.f), bInteriorEdge(false)
	{}

	FUtilVertex2D(const FVector2D& InPos, const FColor& InColor) 
		: Pos(InPos), Color(InColor), UV(0.f, 0.f), bInteriorEdge(false)
	{}
};

/** A polygon in 2D space, used by utility function. */
struct FUtilPoly2D
{
	/** Set of verts, in order, for polygon. */
	TArray<FUtilVertex2D> Verts;
};

/** A set of 2D polygons, along with a transform for going into world space. */
struct FUtilPoly2DSet
{
	TArray<FUtilPoly2D>	Polys;
	FMatrix PolyToWorld;
};

/** Triangle in 2D space. */
struct FUtilTri2D
{
	FUtilVertex2D Verts[3];
};

/** Temp vertex struct for one vert of a static mesh triangle. */
struct FClipSMVertex
{
	FVector3f Pos;
	FVector3f TangentX;
	FVector3f TangentY;
	FVector3f TangentZ;
	FVector2D UVs[8];
	FColor Color;
};

/** Properties of a clipped static mesh face. */
struct FClipSMFace
{
	int32 MaterialIndex;
	uint32 SmoothingMask;
	int32 NumUVs;
	bool bOverrideTangentBasis;

	FVector FaceNormal;

	FMatrix TangentXGradient;
	FMatrix TangentYGradient;
	FMatrix TangentZGradient;
	FMatrix UVGradient[8];
	FMatrix ColorGradient;

	void CopyFace(const FClipSMFace& OtherFace)
	{
		FMemory::Memcpy(this,&OtherFace,sizeof(FClipSMFace));
	}
};

/** Properties of a clipped static mesh triangle. */
struct FClipSMTriangle : FClipSMFace
{
	FClipSMVertex Vertices[3];

	/** Compute the triangle's normal, and the gradients of the triangle's vertex attributes over XYZ. */
	ENGINE_API void ComputeGradientsAndNormal();

	FClipSMTriangle(int32 Init)
	{
		FMemory::Memzero(this, sizeof(FClipSMTriangle));
	}
};

/** Properties of a clipped static mesh polygon. */
struct FClipSMPolygon : FClipSMFace
{
	TArray<FClipSMVertex> Vertices;

	FClipSMPolygon(int32 Init)
	{
		FMemory::Memzero(this, sizeof(FClipSMPolygon));
	}
};

namespace FGeomTools
{
	/** Extracts the triangles from a static-mesh as clippable triangles. */
	ENGINE_API void GetClippableStaticMeshTriangles(TArray<FClipSMTriangle>& OutClippableTriangles, const UStaticMesh* StaticMesh);

	/** Take the input mesh and cut it with supplied plane, creating new verts etc. Also outputs new edges created on the plane. */
	ENGINE_API void ClipMeshWithPlane(TArray<FClipSMTriangle>& OutTris, TArray<FUtilEdge3D>& OutClipEdges, const TArray<FClipSMTriangle>& InTriangles, const FPlane& Plane);

	/** Take a set of 3D Edges and project them onto the supplied plane. Also returns matrix use to convert them back into 3D edges. */
	ENGINE_API void ProjectEdges(TArray<FUtilEdge2D>& Out2DEdges, FMatrix& ToWorld, const TArray<FUtilEdge3D>& In3DEdges, const FPlane& InPlane);

	/** Given a set of edges, find the set of closed polygons created by them. */
	ENGINE_API void Buid2DPolysFromEdges(TArray<FUtilPoly2D>& OutPolys, const TArray<FUtilEdge2D>& InEdges, const FColor& VertColor);

	/** Given a polygon, decompose into triangles and append to OutTris.
	  * @return	true if the triangulation was successful.
	  */
	ENGINE_API bool TriangulatePoly(TArray<FClipSMTriangle>& OutTris, const FClipSMPolygon& InPoly, bool bKeepColinearVertices = false);

	/** Transform triangle from 2D to 3D static-mesh triangle. */
	ENGINE_API FClipSMPolygon Transform2DPolygonToSMPolygon(const FUtilPoly2D& InTri, const FMatrix& InMatrix);

	/** Does a simple planar map using the bounds of this 2D polygon. */
	ENGINE_API void GeneratePlanarFitPolyUVs(FUtilPoly2D& Polygon);

	/** Applies tiling UVs to the verts of this polygon */
	ENGINE_API void GeneratePlanarTilingPolyUVs(FUtilPoly2D& Polygon, float TileSize);

	/** Given a set of triangles, remove those which share an edge and could be collapsed into one triangle. */
	ENGINE_API void RemoveRedundantTriangles(TArray<FClipSMTriangle>& Tris);

	/** Split 2D polygons with a 3D plane. */
	ENGINE_API void Split2DPolysWithPlane(FUtilPoly2DSet& PolySet, const FPlane& Plane, const FColor& ExteriorVertColor, const FColor& InteriorVertColor);

	/** Given three direction vectors, indicates if A and B are on the same 'side' of Vec. */
	ENGINE_API bool VectorsOnSameSide(const FVector3f& Vec, const FVector3f& A, const FVector3f& B, const float SameSideDotProductEpsilon = 0.0f );

	/** Util to see if P lies within triangle created by A, B and C. */
	ENGINE_API bool PointInTriangle(const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& P, const float InsideTriangleDotProductEpsilon = 0.0f);

};


class FGeomTools2D
{
public:
	// Corrects the polygon winding to match bNegativeWinding
	// Ie. If the vertex order doesn't match the winding, it is reversed
	// Returns empty array of points if num points < 3
	static ENGINE_API void CorrectPolygonWinding(TArray<FVector2D>& OutVertices, const TArray<FVector2D>& Vertices, const bool bNegativeWinding);

	// Returns true if the points forming a polygon have CCW winding
	// Returns true if the polygon isn't valid
	static ENGINE_API bool IsPolygonWindingCCW(const TArray<FVector2D>& Points);
	static ENGINE_API bool IsPolygonWindingCCW(const TArray<FIntPoint>& Points);

	// Checks that these polygons can be successfully triangulated	
	static ENGINE_API bool ArePolygonsValid(const TArray<TArray<FVector2D>>& Polygons);

	// Merge additive and subtractive polygons, split them up into additive polygons
	// Assumes all polygons and overlapping polygons are valid, and the windings match the setting on the polygon
	static ENGINE_API TArray<TArray<FVector2D>> ReducePolygons(const TArray<TArray<FVector2D>>& Polygons, const TArray<bool>& PolygonNegativeWinding);

	// Triangulate a polygon. Check notes in implementation
	static ENGINE_API bool TriangulatePoly(TArray<FVector2D>& OutTris, const TArray<FVector2D>& PolygonVertices, bool bKeepColinearVertices = false);

	// 2D version of RemoveRedundantTriangles from GeomTools
	static ENGINE_API void RemoveRedundantTriangles(TArray<FVector2D>& OutTriangles, const TArray<FVector2D>& InTriangles);

	// Generate convex shapes
	static ENGINE_API void GenerateConvexPolygonsFromTriangles(TArray<TArray<FVector2D>>& OutPolygons, const TArray<FVector2D>& InTriangles);

	// Generate convex hull from points
	static ENGINE_API void GenerateConvexHullFromPoints(TArray<FVector2D>& OutConvexHull, TArray<FVector2D>& Points);

	// Returns true if TestPoint is inside the polygon defined by PolygonPoints
	static ENGINE_API bool IsPointInPolygon(const FVector2D& TestPoint, const TArray<FVector2D>& PolygonPoints);
};

