// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUSkinVertexFactory.h: GPU skinning vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "LocalVertexFactory.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCacheVertexFactoryUniformBufferParameters, ENGINE_API)
	SHADER_PARAMETER(FVector3f, MeshOrigin)
	SHADER_PARAMETER(FVector3f, MeshExtension)
	SHADER_PARAMETER(FVector3f, MotionBlurDataOrigin)
	SHADER_PARAMETER(FVector3f, MotionBlurDataExtension)
	SHADER_PARAMETER(float, MotionBlurPositionScale)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FGeometryCacheVertexFactoryUniformBufferParameters> FGeometryCacheVertexFactoryUniformBufferParametersRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCacheManualVertexFetchUniformBufferParameters, ENGINE_API)
	SHADER_PARAMETER_SRV(Buffer<float>, Position)
	SHADER_PARAMETER_SRV(Buffer<float>, MotionBlurData)
	SHADER_PARAMETER_SRV(Buffer<half4>, TangentX)
	SHADER_PARAMETER_SRV(Buffer<half4>, TangentZ)
	SHADER_PARAMETER_SRV(Buffer<float4>, Color)
	SHADER_PARAMETER_SRV(Buffer<float>, TexCoords)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FGeometryCacheManualVertexFetchUniformBufferParameters> FGeometryCacheManualVertexFetchUniformBufferParametersRef;

/**
 * The mesh batch element user data should point to an instance of this struct
 */
struct FGeometryCacheVertexFactoryUserData
{
	const FVertexBuffer* PositionBuffer;
	const FVertexBuffer* MotionBlurDataBuffer;

	// Gpu vertex decompression parameters
	FVector3f MeshOrigin;
	FVector3f MeshExtension;

	// Motion blur parameters
	FVector3f MotionBlurDataOrigin;
	FVector3f MotionBlurDataExtension;
	float MotionBlurPositionScale;

	FGeometryCacheVertexFactoryUniformBufferParametersRef UniformBuffer;

	FShaderResourceViewRHIRef PositionSRV;
	FShaderResourceViewRHIRef TangentXSRV;
	FShaderResourceViewRHIRef TangentZSRV;
	FShaderResourceViewRHIRef ColorSRV;
	FShaderResourceViewRHIRef MotionBlurDataSRV;
	FShaderResourceViewRHIRef TexCoordsSRV;

	FUniformBufferRHIRef ManualVertexFetchUniformBuffer;
};

class FGeometryCacheVertexFactoryShaderParameters;

/** 
 * Vertex factory for geometry caches. Allows specifying explicit motion blur data as
 * previous frames or motion vectors.
 */
class FGeometryCacheVertexVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FGeometryCacheVertexVertexFactory, ENGINE_API);

	typedef FVertexFactory Super;

public:
	FGeometryCacheVertexVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FVertexFactory(InFeatureLevel), PositionStreamIndex(-1), MotionBlurDataStreamIndex(-1)
	{}

	struct FDataType
	{
		/** The stream to read the vertex position from. */
		FVertexStreamComponent PositionComponent;

		/** The streams to read the tangent basis from. */
		FVertexStreamComponent TangentBasisComponents[2];

		/** The streams to read the texture coordinates from. */
		TArray<FVertexStreamComponent, TFixedAllocator<MAX_STATIC_TEXCOORDS / 2> > TextureCoordinates;

		/** The stream to read the vertex color from. */
		FVertexStreamComponent ColorComponent;

		/** The stream to read the motion blur data from. */
		FVertexStreamComponent MotionBlurDataComponent;
		
		bool bIsDummyData = false;
	};

	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	* An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	*/
	UE_DEPRECATED(5.4, "SetData requires an RHI command list.")
	ENGINE_API void SetData(const FDataType& InData);

	ENGINE_API void SetData(FRHICommandListBase& RHICmdList, const FDataType& InData);

	UE_DEPRECATED(5.3, "CreateManualVertexFetchUniformBuffer requires a command list.")
	ENGINE_API void CreateManualVertexFetchUniformBuffer(
		const FVertexBuffer* PoistionBuffer,
		const FVertexBuffer* MotionBlurBuffer,
		FGeometryCacheVertexFactoryUserData& OutUserData) const;

	ENGINE_API void CreateManualVertexFetchUniformBuffer(
		FRHICommandListBase& RHICmdList,
		const FVertexBuffer* PoistionBuffer,
		const FVertexBuffer* MotionBlurBuffer,
		FGeometryCacheVertexFactoryUserData& OutUserData) const;

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& OutElements);
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FDataType& StreamComponentData, FVertexDeclarationElementList& OutElements);

	friend FGeometryCacheVertexFactoryShaderParameters;
	
protected:

	static void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, FDataType& StreamComponentData, FVertexDeclarationElementList& OutElements, FVertexStreamList& InOutStreams, int32& OutPositionStreamIndex, int32& OutMotionBlurDataStreamIndex);

	// Vertex buffer required for creating the Vertex Declaration
	FVertexBuffer VBAlias;

	int32 PositionStreamIndex;
	int32 MotionBlurDataStreamIndex;

	FDataType Data;
};

