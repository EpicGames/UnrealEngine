// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBuilder.h"
#include "EngineGlobals.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "ShaderParameterUtils.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGeometryCollectionVertexFactoryUniformShaderParameters, ENGINE_API)
	SHADER_PARAMETER(FIntVector4,VertexFetch_Parameters)
	SHADER_PARAMETER(uint32,LODLightmapDataIndex)
	SHADER_PARAMETER_SRV(Buffer<float2>, VertexFetch_TexCoordBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, VertexFetch_PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_PackedTangentsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_ColorComponentsBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FGeometryCollectionVertexFactoryUniformShaderParameters> FGeometryCollectionVertexFactoryUniformShaderParametersRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGCBoneLooseParameters, ENGINE_API)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_BoneTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_BonePrevTransformBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, VertexFetch_BoneMapBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FGCBoneLooseParameters> FGCBoneLooseParametersRef;

class FGeometryCollectionVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FGeometryCollectionVertexFactoryShaderParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
	}	

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;

};

/**
 * A vertex factory for Geometry Collections
 */
struct FGeometryCollectionVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FGeometryCollectionVertexFactory, ENGINE_API);

public:
	FGeometryCollectionVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
	: FVertexFactory(InFeatureLevel)
	, LooseParameterUniformBuffer(nullptr)
	{
	}

	// Data includes what we need for transform and everything in local vertex factory too
	struct FDataType : public FStaticMeshDataType
	{
		FRHIShaderResourceView* BoneTransformSRV = nullptr;
		FRHIShaderResourceView* BonePrevTransformSRV = nullptr;
		FRHIShaderResourceView* BoneMapSRV = nullptr;
	};

	//
	// Permutations are controlled by the material flag
	//
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	//
	// Modify compile environment to enable instancing
	// @param OutEnvironment - shader compile environment to modify
	//
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static ENGINE_API void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);

	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	//
	// Set the data on the vertex factory
	//
	void SetData(FRHICommandListBase& RHICmdList, const FDataType& InData)
	{
		Data = InData;
		UpdateRHI(RHICmdList);
	}

	//
	// Copy the data from another vertex factory
	// @param Other - factory to copy from
	//
	void Copy(const FGeometryCollectionVertexFactory& Other)
	{
		FGeometryCollectionVertexFactory* VertexFactory = this;
		const FDataType* DataCopy = &Other.Data;
		ENQUEUE_RENDER_COMMAND(FGeometryCollectionVertexFactoryCopyData)(
			[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
		{
			VertexFactory->Data = *DataCopy;
		});
		BeginUpdateResourceRHI(this);
	}

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;

	inline FRHIShaderResourceView* GetPositionsSRV() const
	{
		return Data.PositionComponentSRV;
	}

	inline FRHIShaderResourceView* GetTangentsSRV() const
	{
		return Data.TangentsSRV;
	}

	inline FRHIShaderResourceView* GetTextureCoordinatesSRV() const
	{
		return Data.TextureCoordinatesSRV;
	}

	inline FRHIShaderResourceView* GetColorComponentsSRV() const
	{
		return Data.ColorComponentsSRV;
	}

	inline const uint32 GetColorIndexMask() const
	{
		return Data.ColorIndexMask;
	}

	inline const int GetLightMapCoordinateIndex() const
	{
		return Data.LightMapCoordinateIndex;
	}

	inline const int GetNumTexcoords() const
	{
		return Data.NumTexCoords;
	}

	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer.GetReference();
	}

	FUniformBufferRHIRef GetLooseParameterBuffer() const
	{
		return LooseParameterUniformBuffer;
	}

	inline void SetBoneTransformSRV(FRHIShaderResourceView* BoneTransformSRV)
	{
		Data.BoneTransformSRV = BoneTransformSRV;
	}
	
	inline FRHIShaderResourceView* GetBoneTransformSRV() const
	{
		return Data.BoneTransformSRV;
	}

	inline void SetBonePrevTransformSRV(FRHIShaderResourceView* BonePrevTransformSRV)
	{
		Data.BonePrevTransformSRV = BonePrevTransformSRV;
	}

	inline FRHIShaderResourceView* GetBonePrevTransformSRV() const
	{
		return Data.BonePrevTransformSRV;
	}

	inline void SetBoneMapSRV(FRHIShaderResourceView* BoneMapSRV)
	{
		Data.BoneMapSRV = BoneMapSRV;
	}

	inline FRHIShaderResourceView* GetBoneMapSRV() const
	{
		return Data.BoneMapSRV;
	}

	FUniformBufferRHIRef LooseParameterUniformBuffer;

	static ENGINE_API bool UseShaderBoneTransform(EShaderPlatform ShaderPlatform);

private:
	FDataType Data;
	TUniformBufferRef<FGeometryCollectionVertexFactoryUniformShaderParameters> UniformBuffer;
	int32 ColorStreamIndex = INDEX_NONE;
};
