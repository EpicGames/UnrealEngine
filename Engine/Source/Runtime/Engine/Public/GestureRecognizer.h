// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	 GestureRecognizer - handles detecting when gestures happen
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"

class UPlayerInput;
namespace UE
{
	struct FTouchState;
}

class FGestureRecognizer
{
public:
	/** Constructor **/
	FGestureRecognizer() : bIsReadyForPinch(false), AnchorDistanceSq(0.0f), StartAngle(0.0f),
		bIsReadyForFlick(false), FlickTime(0.0f), PreviousTouchCount(0)
	{};

	/** Attempt to detect touch gestures */
	void DetectGestures(TArrayView<const UE::FTouchState> TouchStates, UPlayerInput* PlayerInput, float DeltaTime);

	/** Save the distance between the anchor points */
	void SetAnchorDistanceSquared(const FVector2D FirstPoint, const FVector2D SecontPoint);

protected:
	/** Internal processing of gestures */
	void HandleGesture(UPlayerInput* PlayerInput, FKey Gesture, bool bStarted, bool bEnded);
	
	/** A mapping of a gesture to it's current value (how far swiped, pinch amount, etc) */
	TMap<FKey, float> CurrentGestureValues;

	/** Special gesture tracking values */
	FVector2D AnchorPoints[EKeys::NUM_TOUCH_KEYS];

	/** Special pinch tracking values */
	FVector2D LastPinchPoint_Start;
	FVector2D LastPinchPoint_End;

	bool bIsReadyForPinch;
	float AnchorDistanceSq;
	float StartAngle;
	bool bIsReadyForFlick;
	FVector2D FlickCurrent;
	float FlickTime;
	int32 PreviousTouchCount;
};



