// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** This is the type of action that occurred on a given graph */
enum EEdGraphActionType
{
	/** A default edit with no information occurred */
	GRAPHACTION_Default = 0x0,

	/** A node was added to the graph */
	GRAPHACTION_AddNode = 0x1 << 0,

	/** A node was selected */
	GRAPHACTION_SelectNode = 0x1 << 1,

	/** A node was removed from the graph at the user's request */
	GRAPHACTION_RemoveNode = 0x1 << 2,

	/** A node was edited in the graph by a user */
	GRAPHACTION_EditNode = 0x1 << 3,
};


/** Struct containing information about what actions occurred on the graph */
struct FEdGraphEditAction
{
	/** The action(s) that occurred */
	EEdGraphActionType Action;
	/** The graph the action occurred on */
	class UEdGraph* Graph;
	/** [Optional] The object the action occurred on */
	TSet<const class UEdGraphNode*> Nodes;
	/** Whether the user invoked this change or not */
	bool bUserInvoked;

	FEdGraphEditAction()
		: Action(GRAPHACTION_Default)
		, Graph (NULL)
		, bUserInvoked(false)
	{}

	explicit FEdGraphEditAction(EEdGraphActionType InAction, class UEdGraph* InGraph, class UEdGraphNode* InNode, bool bInUserInvoked)
		: Action(InAction) 
		, Graph(InGraph)
		, bUserInvoked(bInUserInvoked)
	{
		Nodes.Add(InNode);
	}
};
