// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"

class FSceneView;

enum class EGroomViewMode : uint8
{
	None,
	SimHairStrands,    // Guide
	RenderHairStrands, // Guide influence + add cluster
	UV,
	RootUV,
	RootUDIM,
	Seed,
	Dimension,
	RadiusVariation,
	Tangent,
	Color,
	Roughness,
	AO,
	ClumpID,
	Cluster,
	ClusterAABB,
	Group,
	LODColoration,
	ControlPoints,
	MacroGroups,
	LightBounds,
	DeepOpacityMaps,
	MacroGroupScreenRect,
	SamplePerPixel,
	CoverageType,
	TAAResolveType,
	VoxelsDensity,
	MeshProjection, // Rename RootBinding
	Coverage,
	MaterialDepth,
	MaterialBaseColor,
	MaterialRoughness,
	MaterialSpecular,
	MaterialTangent,
	CardGuides,
	Tile,
	Memory,
	Bounds,
	Count
};

class FGroomVisualizationData
{
public:
	
	/** Describes a single available visualization mode. */
	struct FModeRecord
	{
		FString        ModeString;
		FName          ModeName;
		FText          ModeText;
		FText          ModeDesc;
		EGroomViewMode Mode;

		// Whether or not this mode (by default) composites with regular scene depth.
		bool           DefaultComposited;
	};

	/** Mapping of FName to a visualization mode record. */
	typedef TMultiMap<FName, FModeRecord> TModeMap;

public:
	FGroomVisualizationData()
	: bIsInitialized(false)
	{
	}

	/** Initialize the system. */
	void Initialize();

	/** Check if system was initialized. */
	inline bool IsInitialized() const { return bIsInitialized; }

	/** Get the display name of a named mode from the available mode map. **/
	ENGINE_API FText GetModeDisplayName(const FName& InModeName) const;

	ENGINE_API EGroomViewMode GetViewMode(const FName& InModeName) const;

	ENGINE_API bool GetModeDefaultComposited(const FName& InModeName) const;

	inline const TModeMap& GetModeMap() const
	{
		return ModeMap;
	}

	/** Return the console command name for enabling single mode visualization. */
	static const TCHAR* GetVisualizeConsoleCommandName()
	{
		return TEXT("r.HairStrands.ViewMode");
	}

private:

	/** The name->mode mapping table */
	TModeMap ModeMap;

	/** Storage for console variable documentation strings. **/
	FString ConsoleDocumentationVisualizationMode;

	/** Flag indicating if system is initialized. **/
	std::atomic_bool bIsInitialized;

	/** Mutex for initialization. */
	UE::FMutex Mutex;
};

ENGINE_API FGroomVisualizationData& GetGroomVisualizationData();
ENGINE_API EGroomViewMode GetGroomViewMode(const FSceneView& View);
ENGINE_API const TCHAR* GetGroomViewModeName(EGroomViewMode In);
ENGINE_API bool IsGroomEnabled();
ENGINE_API void SetGroomEnabled(bool In);