// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FGrowOnlySpanAllocator
{
public:

	FGrowOnlySpanAllocator() :
		MaxSize(0)
	{}

	// Allocate a range.  Returns allocated StartOffset.
	ENGINE_API int32 Allocate(int32 Num);

	// Free an already allocated range.  
	ENGINE_API void Free(int32 BaseOffset, int32 Num);

	int32 GetSparselyAllocatedSize() const
	{
		int32 AllocatedSize = MaxSize;

		for (int32 i = 0; i < FreeSpans.Num(); i++)
		{
			AllocatedSize -= FreeSpans[i].Num;
		}

		return AllocatedSize;
	}

	int32 GetMaxSize() const
	{
		return MaxSize;
	}

	int32 GetNumFreeSpans() const
	{
		return FreeSpans.Num();
	}

#if DO_CHECK
	/**
	 * Loop over all free spans and check if Index is in any of them. 
	 * Note: Very costly, only intended for debugging use, and probably best if under a toggle even then.
	 */
	inline bool IsFree(int32 Index) const
	{
		// If outside the max size, it is considered free as the allocator can grow at will
		if (Index >= MaxSize)
		{
			return true;
		}

		for (const auto& FreeSpan : FreeSpans)
		{
			if (Index >= FreeSpan.StartOffset && Index < FreeSpan.StartOffset + FreeSpan.Num)
			{
				return true;
			}
		}

		return false;
	}
#endif // DO_CHECK

	/** 
	 * Between these calls to Free just appends the allocation to the free list, rather than trying to merge with existing allocations.
	 * At EndDeferMerges the free list is consolidated by sorting and merging all spans. This amortises the cost of the merge over many calls.
	 */
	ENGINE_API void BeginDeferMerges();
	ENGINE_API void EndDeferMerges();

private:
	class FLinearAllocation
	{
	public:

		FLinearAllocation(int32 InStartOffset, int32 InNum) :
			StartOffset(InStartOffset),
			Num(InNum)
		{}

		int32 StartOffset;
		int32 Num;

		bool Contains(FLinearAllocation Other)
		{
			return StartOffset <= Other.StartOffset && (StartOffset + Num) >= (Other.StartOffset + Other.Num);
		}

		bool operator<(const FLinearAllocation& Other) const
		{
			return StartOffset < Other.StartOffset;
		}
	};

	// Size of the linear range used by the allocator
	int32 MaxSize;

	// Unordered free list
	TArray<FLinearAllocation, TInlineAllocator<10>> FreeSpans;

	bool bDeferMerges = false;

	ENGINE_API int32 SearchFreeList(int32 Num);
};