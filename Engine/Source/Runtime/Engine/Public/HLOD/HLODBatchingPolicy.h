// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through other

#include "CoreTypes.h"

#include "HLODBatchingPolicy.generated.h"

/** Determines how the geometry of a component will be incorporated in proxy (simplified) HLODs. */
UENUM()
enum class EHLODBatchingPolicy : uint8
{
	/** No batching to be performed, geometry is to be simplified. */ 
	None,

	/** Batch this component geometry (using the lowest LOD) as a separate mesh section, grouping by material. */
	MeshSection,

	/** Batch this component geometry (using the lowest LOD) as a separate instanced static mesh component in the generated actor. */
	Instancing
};
