// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Engine/World.h"
#include "HLODEngineSubsystem.generated.h"

UCLASS(MinimalAPI)
class UHLODEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

#if WITH_EDITOR

public:
	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	/**
	 * Cleanup invalid LOD actors in the given world
	 * @return true if any cleanup was performed
	 */
	ENGINE_API bool CleanupHLODs(UWorld* InWorld);

	/**
	 * Cleanup invalid LOD actors in the given level
	 * @return true if any cleanup was performed
	 */
	ENGINE_API bool CleanupHLODs(ULevel* InLevel);
	
	/**
	 * By default, invalid LODActors are cleared when loading maps
	 * Use this method to change that behavior
	 */
	ENGINE_API void DisableHLODCleanupOnLoad(bool bInDisableHLODCleanup);

	/**
	 * By default, when HLODs are saved to HLOD packages, they are spawned as transient on load
	 * Use this method to disable spawning.
	 */
	ENGINE_API void DisableHLODSpawningOnLoad(bool bInDisableHLODSpawning);

	// Should be called when the "Save LOD Actors to HLOD Packages" option is toggled.
	ENGINE_API void OnSaveLODActorsToHLODPackagesChanged();

private:
	// Recreate LOD actors for all levels in the provided world.
	void RecreateLODActorsForWorld(UWorld* InWorld, const UWorld::InitializationValues InInitializationValues);
	
	// Recreate LOD actors for the given level.
	void RecreateLODActorsForLevel(ULevel* InLevel, UWorld* InWorld);

	void OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext);

	void UnregisterRecreateLODActorsDelegates();
	void RegisterRecreateLODActorsDelegates();

	bool CleanupHLOD(class ALODActor* InLODActor);

private:
	FDelegateHandle OnPostWorldInitializationDelegateHandle;
	FDelegateHandle OnLevelAddedToWorldDelegateHandle;
	FDelegateHandle OnPreSaveWorlDelegateHandle;

	bool bDisableHLODCleanupOnLoad;
	bool bDisableHLODSpawningOnLoad;

#endif // WITH_EDITOR
};

