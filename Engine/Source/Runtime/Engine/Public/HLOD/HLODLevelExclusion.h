// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "HLODLevelExclusion.generated.h"

/** Bitflag enum to allow editing of UPrimitiveComponent::ExcludeFromHLODLevels as a bitmask in the properties */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EHLODLevelExclusion : uint8
{
	HLOD0 = 1 << 0,
	HLOD1 = 1 << 1,
	HLOD2 = 1 << 2,
	HLOD3 = 1 << 3,
	HLOD4 = 1 << 4,
	HLOD5 = 1 << 5,
	HLOD6 = 1 << 6,
	HLOD7 = 1 << 7
};
ENUM_CLASS_FLAGS(EHLODLevelExclusion);