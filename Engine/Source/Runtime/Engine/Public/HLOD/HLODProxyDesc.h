// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/MaterialMerging.h"
#include "SceneTypes.h"
#include "HLODProxyDesc.generated.h"

class ALODActor;
class UStaticMesh;
class UInstancedStaticMeshComponent;
class UMaterialInterface;

/** Describe a LODActor ISM component */
USTRUCT()
struct FHLODISMComponentDesc
{
	GENERATED_BODY()

public:
	FHLODISMComponentDesc() = default;
	FHLODISMComponentDesc(const UInstancedStaticMeshComponent* InISMComponent);

	bool operator == (const FHLODISMComponentDesc& Other) const;

public:
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;
	
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY()
	TArray<FTransform> Instances;

	UPROPERTY()
	TArray<FCustomPrimitiveData> InstancesCustomPrimitiveData;
};

/** Describe a LODActor */
UCLASS()
class UHLODProxyDesc : public UObject
{
	GENERATED_BODY()

	friend class UHLODProxy;

public:
#if WITH_EDITOR
	/** Test whether this description should be updated. */
	bool ShouldUpdateDesc(const ALODActor* InLODActor) const;
	
	/** 
	 * Update the HLODDesc using a LODActor
	 * @return true if the description changed.
	 */
	bool UpdateFromLODActor(const ALODActor* InLODActor);
	
	/** Spawn a LODActor from this description. */
	ALODActor* SpawnLODActor(ULevel* InLevel) const;
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FName> SubActors;

	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	UPROPERTY()
	TArray<FHLODISMComponentDesc> ISMComponentsDesc;

	UPROPERTY()
	float LODDrawDistance;

	UPROPERTY()
	bool bOverrideMaterialMergeSettings;

	UPROPERTY()
	FMaterialProxySettings MaterialSettings;

	UPROPERTY()
	bool bOverrideTransitionScreenSize;

	UPROPERTY()
	float TransitionScreenSize;

	UPROPERTY()
	bool bOverrideScreenSize;

	UPROPERTY()
	int32 ScreenSize;

	UPROPERTY()
	FName Key;

	UPROPERTY()
	int32 LODLevel;

	UPROPERTY()
	FString LODActorTag;

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	FTransform HLODBakingTransform;

	UPROPERTY()
	TArray<TSoftObjectPtr<UHLODProxyDesc>> SubHLODDescs;
#endif
};
