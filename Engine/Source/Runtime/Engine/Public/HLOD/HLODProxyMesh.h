// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/LODActor.h"
#include "HLODProxyMesh.generated.h"

class UStaticMesh;

/** A mesh proxy entry */
USTRUCT()
struct FHLODProxyMesh
{
	GENERATED_BODY()

	FHLODProxyMesh();

#if WITH_EDITOR
	FHLODProxyMesh(UStaticMesh* InStaticMesh, const FName& InKey);

	FHLODProxyMesh(ALODActor* InLODActor, UStaticMesh* InStaticMesh, const FName& InKey);
	
	bool operator==(const FHLODProxyMesh& InHLODProxyMesh) const;
#endif

	/** Get the mesh for this proxy mesh */
	const UStaticMesh* GetStaticMesh() const;

	/** Get the actor for this proxy mesh */
	const TLazyObjectPtr<ALODActor>& GetLODActor() const;

	/** Get the key for this proxy mesh */
	const FName& GetKey() const;

private:
	/** The ALODActor that we were generated from */
	UPROPERTY()
	TLazyObjectPtr<ALODActor> LODActor;

	/** The mesh used to display this proxy */
	UPROPERTY(VisibleAnywhere, Category = "Proxy Mesh")
	TObjectPtr<UStaticMesh> StaticMesh;

	/** The key generated from an ALODActor. If this differs from that generated from the ALODActor, then the mesh needs regenerating. */
	UPROPERTY(VisibleAnywhere, Category = "Proxy Mesh")
	FName Key;
};
