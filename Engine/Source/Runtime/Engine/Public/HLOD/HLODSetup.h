// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"
#include "MeshMerge/MeshApproximationSettings.h"
#include "HLODSetup.generated.h"


UENUM()
enum class EHierarchicalSimplificationMethod : uint8
{
	None = 0			UMETA(hidden),
	Merge = 1,
	Simplify = 2,
	Approximate = 3
};


USTRUCT()
struct FHierarchicalSimplification
{
	GENERATED_USTRUCT_BODY()

	/** The screen radius an mesh object should reach before swapping to the LOD actor, once one of parent displays, it won't draw any of children. */
	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, meta = (UIMin = "0.00001", ClampMin = "0.000001", UIMax = "1.0", ClampMax = "1.0"))
	float TransitionScreenSize;

	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay, meta = (UIMin = "1.0", ClampMin = "1.0", UIMax = "50000.0", editcondition="bUseOverrideDrawDistance"))
	float OverrideDrawDistance;

	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay, meta = (InlineEditConditionToggle))
	uint8 bUseOverrideDrawDistance:1;

	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay)
	uint8 bAllowSpecificExclusion : 1;

	/** Only generate clusters for HLOD volumes */
	UPROPERTY(EditAnywhere, Category = FHierarchicalSimplification, AdvancedDisplay, meta = (editcondition = "!bReusePreviousLevelClusters", DisplayAfter="MinNumberOfActorsToBuild"))
	uint8 bOnlyGenerateClustersForVolumes:1;

	/** Will reuse the clusters generated for the previous (lower) HLOD level */
	UPROPERTY(EditAnywhere, Category = FHierarchicalSimplification, AdvancedDisplay, meta=(DisplayAfter="bOnlyGenerateClustersForVolumes"))
	uint8 bReusePreviousLevelClusters:1;

	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere)
	EHierarchicalSimplificationMethod SimplificationMethod;

	/** Simplification settings, used if SimplificationMethod is Simplify */
	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay)
	FMeshProxySettings ProxySetting;

	/** Merge settings, used if SimplificationMethod is Merge */
	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay)
	FMeshMergingSettings MergeSetting;

	/** Approximate settings, used if SimplificationMethod is Approximate */
	UPROPERTY(Category = FHierarchicalSimplification, EditAnywhere, AdvancedDisplay)
	FMeshApproximationSettings ApproximateSettings;

	/** Desired Bounding Radius for clustering - this is not guaranteed but used to calculate filling factor for auto clustering */
	UPROPERTY(EditAnywhere, Category=FHierarchicalSimplification, AdvancedDisplay, meta=(UIMin=10.f, ClampMin=10.f, editcondition = "!bReusePreviousLevelClusters"))
	float DesiredBoundRadius;

	/** Desired Filling Percentage for clustering - this is not guaranteed but used to calculate filling factor  for auto clustering */
	UPROPERTY(EditAnywhere, Category=FHierarchicalSimplification, AdvancedDisplay, meta=(ClampMin = "0", ClampMax = "100", UIMin = "0", UIMax = "100", editcondition = "!bReusePreviousLevelClusters"))
	float DesiredFillingPercentage;

	/** Min number of actors to build LODActor */
	UPROPERTY(EditAnywhere, Category=FHierarchicalSimplification, AdvancedDisplay, meta=(ClampMin = "1", UIMin = "1", editcondition = "!bReusePreviousLevelClusters"))
	int32 MinNumberOfActorsToBuild;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty))
	uint8 bSimplifyMesh_DEPRECATED:1;
#endif

	ENGINE_API FHierarchicalSimplification();

#if WITH_EDITORONLY_DATA
	ENGINE_API bool Serialize(FArchive& Ar);

	/** Handles deprecated properties */
	ENGINE_API void PostSerialize(const FArchive& Ar);
#endif

	/** Retrieve the correct material proxy settings based on the simplification method. */
	ENGINE_API FMaterialProxySettings* GetSimplificationMethodMaterialSettings();
};

template<>
struct TStructOpsTypeTraits<FHierarchicalSimplification> : public TStructOpsTypeTraitsBase2<FHierarchicalSimplification>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
#endif
};

UCLASS(Blueprintable, MinimalAPI)
class UHierarchicalLODSetup : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API UHierarchicalLODSetup();

	/** Hierarchical LOD Setup */
	UPROPERTY(EditAnywhere, Category = HLODSystem)
	TArray<struct FHierarchicalSimplification> HierarchicalLODSetup;

	UPROPERTY(EditAnywhere, Category = HLODSystem)
	TSoftObjectPtr<UMaterialInterface> OverrideBaseMaterial;

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};
