// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HModel.h: HModel definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectPtr.h"
#include "HitProxies.h"

class FSceneView;
class UModel;
class UModelComponent;

/**
 * A hit proxy representing a UModel.
 */
class HModel : public HHitProxy
{
	DECLARE_HIT_PROXY( ENGINE_API );
public:

	/** Initialization constructor. */
	HModel(UModelComponent* InComponent, UModel* InModel):
		Component(InComponent),
		Model(InModel)
	{}

	/**
	 * Finds the surface at the given screen coordinates of a view family.
	 * @return True if a surface was found.
	 */
	ENGINE_API bool ResolveSurface(const FSceneView* View,int32 X,int32 Y,uint32& OutSurfaceIndex) const;

	// HHitProxy interface.
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual EMouseCursor::Type GetMouseCursor() override;

	// Accessors.
	UModelComponent* GetModelComponent() const { return Component; }
	UModel* GetModel() const { return Model; }

private:
	
	TObjectPtr<UModelComponent> Component;
	TObjectPtr<UModel> Model;
};

