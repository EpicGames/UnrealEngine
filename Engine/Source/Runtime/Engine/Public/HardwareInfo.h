// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HardwareInfo.h: Declares the hardware info class
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

/** Hardware entry lookups */
extern ENGINE_API const FName NAME_RHI;
extern ENGINE_API const FName NAME_TextureFormat;
extern ENGINE_API const FName NAME_DeviceType;


struct FHardwareInfo
{
	/**
	 * Register with the hardware info a detail which we may want to keep track of
	 *
	 * @param SpecIdentifier - The piece of hardware information we are registering, must match the lookups above
	 * @param HarwareInfo - The information we want to be associated with the input key.
	 */
	static ENGINE_API void RegisterHardwareInfo( const FName SpecIdentifier, const FString& HardwareInfo );

	/**
	 * Get the hardware info detail you wanted to keep track of.
	 *
	 * @param SpecIdentifier - The piece of hardware information we are registering, must match the lookups above
	 */
	static ENGINE_API FString GetHardwareInfo(const FName SpecIdentifier);

	/**
	 * Get the full details of hardware information which has been registered in string format
	 *
	 * @return The details of the hardware which were registered in a string format.
	 */
	static ENGINE_API const FString GetHardwareDetailsString();
};
