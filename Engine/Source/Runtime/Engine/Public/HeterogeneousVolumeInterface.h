// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Stats/Stats.h"
#include "CoreMinimal.h"
#include "SceneManagement.h"
#include "PrimitiveSceneProxy.h"

class IHeterogeneousVolumeInterface
{
public:
	virtual ~IHeterogeneousVolumeInterface() = default;
	virtual const FPrimitiveSceneProxy* GetPrimitiveSceneProxy() const = 0;
	
	// Local-Space
	virtual const FBoxSphereBounds& GetBounds() const = 0;
	virtual const FBoxSphereBounds& GetLocalBounds() const = 0;
	virtual const FMatrix& GetLocalToWorld() const = 0;

	// Instance-Space
	virtual const FMatrix& GetInstanceToLocal() const = 0;
	virtual const FMatrix GetInstanceToWorld() const = 0;

	// Volume
	virtual FIntVector GetVoxelResolution() const = 0;
	virtual float GetMinimumVoxelSize() const = 0;
	virtual bool IsPivotAtCentroid() const = 0;

	// Lighting
	virtual float GetStepFactor() const = 0;
	virtual float GetShadowStepFactor() const = 0;
	virtual float GetShadowBiasFactor() const = 0;
	virtual float GetLightingDownsampleFactor() const = 0;
	virtual float GetMipBias() const = 0;

	// Rendering
	virtual bool IsHoldout() const = 0;

	// Debug
	virtual FString GetReadableName() const = 0;
};

class FPrimitiveSceneProxy;

class FHeterogeneousVolumeData : public IHeterogeneousVolumeInterface, FOneFrameResource
{
public:
	explicit FHeterogeneousVolumeData(const FPrimitiveSceneProxy* SceneProxy)
		: PrimitiveSceneProxy(SceneProxy)
		, InstanceToLocal(FMatrix::Identity)
		, VoxelResolution(FIntVector::ZeroValue)
		, MinimumVoxelSize(0.1f)
		, StepFactor(1.0f)
		, ShadowStepFactor(8.0f)
		, ShadowBiasFactor(0.0f)
		, LightingDownsampleFactor(1.0f)
		, MipBias(0.0f)
		, bPivotAtCentroid(false)
		, bHoldout(false)
	{}

	FHeterogeneousVolumeData(const FPrimitiveSceneProxy* SceneProxy, FString Name)
		: PrimitiveSceneProxy(SceneProxy)
		, InstanceToLocal(FMatrix::Identity)
		, VoxelResolution(FIntVector::ZeroValue)
		, MinimumVoxelSize(0.1f)
		, StepFactor(1.0f)
		, ShadowStepFactor(8.0f)
		, ShadowBiasFactor(0.0f)
		, LightingDownsampleFactor(1.0f)
		, MipBias(0.0f)
		, bPivotAtCentroid(false)
		, bHoldout(false)
#if ACTOR_HAS_LABELS
		, ReadableName(Name)
#endif // ACTOR_HAS_LABELS
	{}
	virtual ~FHeterogeneousVolumeData() = default;

	virtual const FPrimitiveSceneProxy* GetPrimitiveSceneProxy() const { return PrimitiveSceneProxy; }

	// Local-space
	virtual const FBoxSphereBounds& GetBounds() const { return PrimitiveSceneProxy->GetBounds(); }
	virtual const FBoxSphereBounds& GetLocalBounds() const { return PrimitiveSceneProxy->GetLocalBounds(); }
	virtual const FMatrix& GetLocalToWorld() const { return PrimitiveSceneProxy->GetLocalToWorld(); }
	virtual const FMatrix& GetInstanceToLocal() const { return InstanceToLocal; }
	virtual const FMatrix GetInstanceToWorld() const { return InstanceToLocal * PrimitiveSceneProxy->GetLocalToWorld(); }

	// Volume
	virtual FIntVector GetVoxelResolution() const { return VoxelResolution; }
	virtual float GetMinimumVoxelSize() const { return MinimumVoxelSize; }
	virtual bool IsPivotAtCentroid() const { return bPivotAtCentroid; }

	// Lighting
	virtual float GetStepFactor() const { return StepFactor; }
	virtual float GetShadowStepFactor() const { return ShadowStepFactor; }
	virtual float GetShadowBiasFactor() const { return ShadowBiasFactor; }
	virtual float GetLightingDownsampleFactor() const { return LightingDownsampleFactor; }
	virtual float GetMipBias() const { return MipBias; }

	// Rendering
	virtual bool IsHoldout() const { return bHoldout; }

	const FPrimitiveSceneProxy* PrimitiveSceneProxy;
	FMatrix InstanceToLocal;
	FIntVector VoxelResolution;
	float MinimumVoxelSize;
	float StepFactor;
	float ShadowStepFactor;
	float ShadowBiasFactor;
	float LightingDownsampleFactor;
	float MipBias;
	bool bPivotAtCentroid;
	bool bHoldout;

#if ACTOR_HAS_LABELS
	FString ReadableName;
	virtual FString GetReadableName() const { return ReadableName; }
#else
	virtual FString GetReadableName() const { return PrimitiveSceneProxy->GetResourceName().ToString(); }
#endif // ACTOR_HAS_LABELS
};

DECLARE_STATS_GROUP(TEXT("Heterogeneous Volumes RT"), STATGROUP_HeterogeneousVolumesRT, STATCAT_HeterogeneousVolumes);
DECLARE_STATS_GROUP(TEXT("Heterogeneous Volumes GT"), STATGROUP_HeterogeneousVolumesGT, STATCAT_HeterogeneousVolumes);
