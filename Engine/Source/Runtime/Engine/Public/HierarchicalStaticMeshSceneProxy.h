// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "Engine/InstancedStaticMesh.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "StaticMeshResources.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Async/Mutex.h"

struct FClusterNode;
struct FFoliageElementParams;
struct FFoliageRenderInstanceParams;
struct FFoliageCullInstanceParams;

struct FFoliageOcclusionResults
{
	TArray<bool> Results; // we keep a copy from the View as the view will get destroyed too often
	int32 ResultsStart;
	int32 NumResults;
	uint32 FrameNumberRenderThread;

	FFoliageOcclusionResults(TArray<bool>* InResults, int32 InResultsStart, int32 InNumResults)
	: Results(*InResults)
	, ResultsStart(InResultsStart)
	, NumResults(InNumResults)
	, FrameNumberRenderThread(GFrameNumberRenderThread)
	{
	}
};

class FHierarchicalStaticMeshSceneProxy final : public FInstancedStaticMeshSceneProxy
{
	TSharedRef<TArray<FClusterNode>, ESPMode::ThreadSafe> ClusterTreePtr;
	const TArray<FClusterNode>& ClusterTree;

	TArray<FBox> UnbuiltBounds;
	int32 FirstUnbuiltIndex;
	int32 InstanceCountToRender;

	int32 FirstOcclusionNode;
	int32 LastOcclusionNode;
	TArray<FBoxSphereBounds> OcclusionBounds;
	TMap<uint32, FFoliageOcclusionResults> OcclusionResults;
	UE::FMutex OcclusionResultsMutex;
	EHISMViewRelevanceType ViewRelevance;
	bool bDitheredLODTransitions;
	uint32 SceneProxyCreatedFrameNumberRenderThread;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	mutable TArray<uint32> SingleDebugRuns[MAX_STATIC_MESH_LODS];
	mutable int32 SingleDebugTotalInstances[MAX_STATIC_MESH_LODS];
	mutable TArray<uint32> MultipleDebugRuns[MAX_STATIC_MESH_LODS];
	mutable int32 MultipleDebugTotalInstances[MAX_STATIC_MESH_LODS];
	mutable int32 CaptureTag;
#endif

public:
	ENGINE_API SIZE_T GetTypeHash() const override;

	FHierarchicalStaticMeshSceneProxy(UHierarchicalInstancedStaticMeshComponent* InComponent, ERHIFeatureLevel::Type InFeatureLevel);

	ENGINE_API void SetupOcclusion(UHierarchicalInstancedStaticMeshComponent* InComponent);

	// FPrimitiveSceneProxy interface.
	
	ENGINE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	
	ENGINE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	ENGINE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	ENGINE_API virtual const TArray<FBoxSphereBounds>* GetOcclusionQueries(const FSceneView* View) const override;
	
	ENGINE_API virtual void AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults) override;

	virtual bool AllowInstanceCullingOcclusionQueries() const override { return false; }

	virtual bool HasSubprimitiveOcclusionQueries() const override
	{
		return FirstOcclusionNode > 0;
	}

	virtual bool GetInstanceDrawDistanceMinMax(FVector2f& OutDistanceMinMax) const override;

	ENGINE_API virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;

	ENGINE_API virtual void ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset) override;

	ENGINE_API void FillDynamicMeshElements(const FSceneView* View, FMeshElementCollector& Collector, const FFoliageElementParams& ElementParams, const FFoliageRenderInstanceParams& Instances) const;

	template<bool TUseVector, bool THasWPODisplacement>
	void Traverse(const FFoliageCullInstanceParams& Params, int32 Index, int32 MinLOD, int32 MaxLOD, bool bFullyContained = false) const;
};
