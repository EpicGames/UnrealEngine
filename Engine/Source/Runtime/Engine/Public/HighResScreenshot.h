// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FSceneViewport;
class IImageWrapper;
class UMaterial;
class FImageWriteTask;
class IImageWriteQueue;

DECLARE_LOG_CATEGORY_EXTERN(LogHighResScreenshot, Log, All);

struct FHighResScreenshotConfig
{
	static ENGINE_API const float MinResolutionMultipler;
	static ENGINE_API const float MaxResolutionMultipler;

	FIntRect UnscaledCaptureRegion;
	FIntRect CaptureRegion;
	float ResolutionMultiplier;
	float ResolutionMultiplierScale;
	bool bMaskEnabled;
	bool bDateTimeBasedNaming;
	bool bDumpBufferVisualizationTargets;
	TWeakPtr<FSceneViewport> TargetViewport;
	bool bDisplayCaptureRegion;
	bool bCaptureHDR;
	bool bForce128BitRendering;
	FString FilenameOverride;

	// Materials used in the editor to help with the capture of highres screenshots
	UMaterial* HighResScreenshotMaterial;
	UMaterial* HighResScreenshotMaskMaterial;
	UMaterial* HighResScreenshotCaptureRegionMaterial;

	/** Pointer to the image write queue to use for async image writes */
	IImageWriteQueue* ImageWriteQueue;

	ENGINE_API FHighResScreenshotConfig();

	/** Initialize the Image write queue necessary for asynchronously saving screenshots **/
	ENGINE_API void Init();

	/** Populate the specified task with parameters from the current high-res screenshot request */
	ENGINE_API void PopulateImageTaskParams(FImageWriteTask& InOutTask);

	/** Point the screenshot UI at a different viewport **/
	ENGINE_API void ChangeViewport(TWeakPtr<FSceneViewport> InViewport);

	/** Parse screenshot parameters from the supplied console command line **/
	ENGINE_API bool ParseConsoleCommand(const FString& InCmd, FOutputDevice& Ar);

	/** Utility function for merging the mask buffer into the alpha channel of the supplied bitmap, if masking is enabled.
	  * Returns true if the mask was written, and false otherwise.
	**/
	ENGINE_API bool MergeMaskIntoAlpha(TArray<FColor>& InBitmap, const FIntRect& ViewRect);
	ENGINE_API bool MergeMaskIntoAlpha(TArray<FLinearColor>& InBitmap, const FIntRect& ViewRect);

	/** Enable/disable HDR capable captures **/
	ENGINE_API void SetHDRCapture(bool bCaptureHDRIN);

	/** Enable/disable forcing 128-bit rendering pipeline for capture **/
	ENGINE_API void SetForce128BitRendering(bool bForce);

	/** Configure taking a high res screenshot */
	ENGINE_API bool SetResolution(uint32 ResolutionX, uint32 ResolutionY, float ResolutionScale = 1.0f);

	/** Configure screenshot filename */
	ENGINE_API void SetFilename(FString Filename);

	/** Configure screenshot mask is enabled */
	ENGINE_API void SetMaskEnabled(bool bShouldMaskBeEnabled);
};

ENGINE_API FHighResScreenshotConfig& GetHighResScreenshotConfig();
