// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HitProxies.h: Hit proxy definitions.
=============================================================================*/

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/RefCounting.h"

#include "HitProxies.generated.h"

class FReferenceCollector;
struct FTypedElementHandle;

namespace EMouseCursor { enum Type : int; }

/**
 * The priority a hit proxy has when choosing between several hit proxies near the point the user clicked.
 * HPP_World - this is the default priority
 * HPP_Wireframe - the priority of items that are drawn in wireframe, such as volumes
 * HPP_UI - the priority of the UI components such as the translation widget
 */
UENUM()
enum EHitProxyPriority : int
{
	HPP_World = 0,
	HPP_Wireframe = 1,
	HPP_Foreground = 2,
	HPP_UI = 3
};

/**
 * Represents a hit proxy class for runtime type checks.
 */
class HHitProxyType
{
public:
	HHitProxyType(HHitProxyType* InParent,const TCHAR* InName):
		Parent(InParent),
		Name(InName)
	{}
	HHitProxyType* GetParent() const { return Parent; }
	const TCHAR* GetName() const { return Name; }
private:
	HHitProxyType* Parent;
	const TCHAR* Name;
};

/**
 * A macro which creates a HHitProxyType for a HHitProxy-derived class.
 * (Doxygen cannot match this, and a different version is given explicitly when building documentation).
 */
#if !UE_BUILD_DOCS
#define DECLARE_HIT_PROXY_STATIC( ... /* APIDecl */) \
	public: \
	/* NOTE: This static method must NOT be inlined as it will not work across DLL boundaries if so */ \
	static __VA_ARGS__ HHitProxyType* StaticGetType();

#define DECLARE_HIT_PROXY( ... /* APIDecl */) \
	DECLARE_HIT_PROXY_STATIC( __VA_ARGS__ ) \
	virtual HHitProxyType* GetType() const override \
	{ \
		return StaticGetType(); \
	}
#endif

/**
 * A macro which creates a HHitProxyType for a HHitProxy-derived class.
 */
#define IMPLEMENT_HIT_PROXY_BASE(TypeName,ParentType) \
	HHitProxyType* TypeName::StaticGetType() \
	{ \
		static HHitProxyType StaticType(ParentType,TEXT(#TypeName)); \
		return &StaticType; \
	}

#define IMPLEMENT_HIT_PROXY(TypeName,ParentTypeName) \
	IMPLEMENT_HIT_PROXY_BASE(TypeName,ParentTypeName::StaticGetType())

/**
 * Encapsulates a hit proxy ID.
 */
class FHitProxyId
{
	friend class HHitProxy;
public:

	/** A special hit proxy ID that can be used to omit rendering a primitive to the hit proxy buffer entirely.  This is useful when
	    rendering translucent primitives that you never want to obscure selection of objects behind them*/
	ENGINE_API static const FHitProxyId InvisibleHitProxyId;

	/** Default constructor. */
	FHitProxyId(): Index(INDEX_NONE) {}

	/** Color conversion constructor. */
	ENGINE_API FHitProxyId(FColor Color);

	/**
	 * Maps the ID to a color which can be used to represent the ID.
	 */
	ENGINE_API FColor GetColor() const;

	/**
	 * Maps a hit proxy ID to its hit proxy.  If the ID doesn't map to a valid hit proxy, NULL is returned.
	 * @param ID - The hit proxy ID to match.
	 * @return The hit proxy with matching ID, or NULL if no match.
	 */
	friend ENGINE_API class HHitProxy* GetHitProxyById(FHitProxyId Id);

	friend bool operator==(FHitProxyId X, FHitProxyId Y)
	{
		return X.Index == Y.Index;
	}

	friend bool operator!=(FHitProxyId X, FHitProxyId Y)
	{
		return X.Index != Y.Index;
	}

private:
	
	/** Initialization constructor. */
	FHitProxyId(int32 InIndex): Index(InIndex) {}

	/** A uniquely identifying index for the hit proxy. */
	int32 Index;
};

/**
 * Base class for detecting user-interface hits.
 */
class HHitProxy : public FRefCountedObject
{
	//DECLARE_HIT_PROXY( ENGINE_API )
	//We separate the GetType function implementation here because this is the base class/
	//The base class function should not have the override keyword here.
	DECLARE_HIT_PROXY_STATIC( ENGINE_API )
	virtual HHitProxyType* GetType() const
	{
		return StaticGetType();
	}
public:

	/** The priority a hit proxy has when choosing between several hit proxies near the point the user clicked. */
	const EHitProxyPriority Priority;

	/** Used in the ortho views, defaults to the same value as Priority */
	const EHitProxyPriority OrthoPriority;

	/** The hit proxy's ID. */
	FHitProxyId Id;

	ENGINE_API HHitProxy(EHitProxyPriority InPriority = HPP_World);
	HHitProxy(EHitProxyPriority InPriority, EHitProxyPriority InOrthoPriority);
	ENGINE_API virtual ~HHitProxy();
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) {}

	/**
	 * Determines whether the hit proxy is of the given type.
	 */
	ENGINE_API bool IsA(HHitProxyType* TestType) const;

	/**
		Override to change the mouse based on what it is hovering over.
	*/
	ENGINE_API virtual EMouseCursor::Type GetMouseCursor();

	/**
	 * Method that specifies whether the hit proxy *always* allows translucent primitives to be associated with it or not,
	 * regardless of any other engine/editor setting. For example, if translucent selection was disabled, any hit proxies
	 * returning true would still allow translucent selection.
	 *
	 * @return	true if translucent primitives are always allowed with this hit proxy; false otherwise
	 */
	virtual bool AlwaysAllowsTranslucentPrimitives() const
	{
		return false;
	}

	/**
	 * Get the element handle associated with this hit proxy, if any.
	 */
	ENGINE_API virtual FTypedElementHandle GetElementHandle() const;

private:
	void InitHitProxy();
};

// Dynamically cast a HHitProxy object type-safely.
template<typename DesiredType>
DesiredType* HitProxyCast(HHitProxy* Src)
{
	return ((Src != NULL) && (Src->IsA(DesiredType::StaticGetType()))) ? ((DesiredType*)Src) : NULL;
}

/**
 * Hit proxy class for UObject references.
 */
struct HObject : HHitProxy
{
	DECLARE_HIT_PROXY( ENGINE_API );

	UObject*	Object;

	HObject(UObject* InObject): HHitProxy(HPP_UI), Object(InObject) {}
	virtual void Serialize(FArchive& Ar)
	{
		Ar << Object;
	}
};

/**
 * An interface to a hit proxy consumer.
 */
class FHitProxyConsumer
{
public:

	/**
	 * Called when a new hit proxy is rendered.  The hit proxy consumer should keep a TRefCountPtr to the HitProxy to prevent it from being
	 * deleted before the rendered hit proxy map.
	 */
	virtual void AddHitProxy(HHitProxy* HitProxy) = 0;
};
