// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/ArrayView.h"

class FName;
class FTextFormat;
class UObject;

namespace AssetCompilation
{
	struct FProcessAsyncTaskParams
	{
		/* Limits the execution time instead of processing everything */
		bool bLimitExecutionTime = false;

		/* Limits processing for assets required for PIE only. */
		bool bPlayInEditorAssetsOnly = false;
	};
}

struct IAssetCompilingManager
{
	/**
	 * A unique name among all asset compiling manager to identify the type of asset this manager handles.
	 */
	virtual FName GetAssetTypeName() const = 0;

	/**
	 * Returns an FTextFormat representing a localized singular/plural formatter for this resource name.
	 *
	 * Note: Should be in a similar form as "{0}|plural(one=Singular Name,other=Plural Name)"
	 */
	virtual FTextFormat GetAssetNameFormat() const = 0;

	/**
	 * Return other asset types that should preferably be handled before this one.
	 */
	virtual TArrayView<FName> GetDependentTypeNames() const = 0;

	/**
	 * Returns the number of remaining compilations.
	 */
	virtual int32 GetNumRemainingAssets() const = 0;

	/**
	 * Blocks until completion of the requested objects.
	 */
	virtual void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) {} /* Optional for backward compatibility */

	/**
	 * Mark compilation of the provided objects as canceled.
	 * This is a hint that the compilations are no longer needed and may leave the objects in a bad state.
	 * There is no guarantee that anything will be canceled, but if supported, it might remove heavy CPU work earlier.
	 * If you need to make sure that any activity is done on these objects, a call to FinishCompilationForObjects is still required.
	 */
	virtual void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects) {} /* Optional for backward compatibility */

	/**
	 * Blocks until completion of all assets.
	 */
	virtual void FinishAllCompilation() = 0;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	virtual void Shutdown() = 0;

protected:
	friend class FAssetCompilingManager;

	/**
	 * Called once per frame, fetches completed tasks and applies them to the scene.
	 */
	virtual void ProcessAsyncTasks(bool bLimitExecutionTime = false) = 0;

	/**
	 * Called once per frame, fetches completed tasks and applies them to the scene.
	 */
	virtual void ProcessAsyncTasks(const AssetCompilation::FProcessAsyncTaskParams& Params)
	{
		/* Forward for backward compatibility */
		ProcessAsyncTasks(Params.bLimitExecutionTime);
	}

	virtual ~IAssetCompilingManager() = default;
};
