// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IAssetRegistryTagProviderInterface.generated.h"

/**
 * Interface to allow for CDO to add additional tags to generated class
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAssetRegistryTagProviderInterface : public UInterface
{
	GENERATED_BODY()
};

class IAssetRegistryTagProviderInterface
{
	GENERATED_BODY()

public:

	/** By default calls 'GetAssetRegistryTags' on the CDO, to add the BP's tags to the BPGC. */
	virtual bool ShouldAddCDOTagsToBlueprintClass() const { return false; }
};
