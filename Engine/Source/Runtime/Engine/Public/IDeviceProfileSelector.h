// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IDeviceProfileSelector.h: Declares the IDeviceProfileSelector interface.
=============================================================================*/

#pragma once


/**
 * DeviceProfileSelector
 */
class IDeviceProfileSelector
{
public:

	/**
	 * Virtual destructor.
	 */
	~IDeviceProfileSelector() = default;
};
