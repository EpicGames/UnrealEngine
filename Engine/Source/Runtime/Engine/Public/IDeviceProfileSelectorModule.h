// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IDeviceProfileSelectorModule.h: Declares the IDeviceProfileSelectorModule interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "DeviceProfiles/DeviceProfileMatching.h"
#if WITH_EDITOR
#include "RHIShaderPlatform.h"
#endif

class UDeviceProfile;

/**
 * Device Profile Selector module
 */
class IDeviceProfileSelectorModule
	: public IModuleInterface
{
public:
	IDeviceProfileSelectorModule() :
		ConstrainedAspectRatio(0.0f), 
		SafeZones(FVector4f::Zero())
	{}
	
	/**
	 * Run the logic to choose an appropriate device profile for this session
	 *
	 * @return The name of the device profile to use for this session
	 */
	virtual const FString GetRuntimeDeviceProfileName() = 0;

#if WITH_EDITOR
	/**
	 * Save Device Jsons to the location provided
	 * @param FolderLocation Folder where to store the Jsons
	 */
	virtual void ExportDeviceParametersToJson(FString& FolderLocation, TArray<FString>& OutExportedFiles){}
	
	/**
	 * Can we export device parameters to Jsons
	 *  @return Wheter we can export device parameters to Jsonsn
	 */
	virtual bool CanExportDeviceParametersToJson() { return false; }

	/**
	 *  @return If the Json is compatible with this version of the DeviceProfileSelector
	 */
	virtual bool IsJsonCompatible(const FString& JsonLocation) { return true; }

	/**
	 * Get the Device Parameters from the Json
	* @Param JsonLocation Json location
	* @Param OutDeviceParameters the device parameters read from Json
	 */
	virtual void GetDeviceParametersFromJson(FString& JsonLocation, TMap<FString, FString>& OutDeviceParameters) {}
	
	/**
	 * Get the ConstrainedAspectRatio of the Device
	 *  @return ConstrainedAspectRatio of the Device
	 */
	virtual float GetConstrainedAspectRatio() { return ConstrainedAspectRatio; }
	
	/**
	 * Get the Safe Zones of the Device
	 *  @return Safe Zones of the Device
	 */
	virtual FVector4f GetSafeZones() { return SafeZones; }
	
	/**
	 * Can we get device parameters from Json
	 *  @return Wheter we can get device parameters from Json
	 */
	virtual bool CanGetDeviceParametersFromJson() { return false; }

	/**
	 * @return Return the PreviewShaderPlatform from DeviceProfile
	 */
	virtual const EShaderPlatform GetPreviewShaderPlatform() { return EShaderPlatform::SP_NumPlatforms; }
#endif

	/**
	* Run the logic to choose an appropriate device profile for this session.
	* @param DeviceParameters	A map of parameters to be used by device profile logic.
	* @return The name of the device profile to use for this session
	*/
	virtual const FString GetDeviceProfileName() { return FString(); }

	/**
	* Set or override the selector specific properties.
	* @param SelectorProperties	A map of parameters to be used by device profile matching logic.
	*/
	virtual void SetSelectorProperties(const TMap<FString, FString>& SelectorProperties) { }

	/*
	* Find a custom profile selector property value.
	* @Param PropertyType The information requested
	* @Param PropertyValueOUT the value of PropertyType
	* @return Whether the PropertyType was recognized by the profile selector.
	*/
	virtual bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) { PropertyValueOUT.Reset(); return false; }

	/**
	 * Virtual destructor.
	 */
	virtual ~IDeviceProfileSelectorModule() = default;

protected:
	float ConstrainedAspectRatio;
	FVector4f SafeZones;
};
