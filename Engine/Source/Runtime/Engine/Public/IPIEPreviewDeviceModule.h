// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IDeviceProfileSelectorModule.h"
#include "Widgets/SWindow.h"

class IPIEPreviewDeviceModule : public IDeviceProfileSelectorModule
{
	public:
		//~ Begin IDeviceProfileSelectorModule Interface
		virtual const FString GetRuntimeDeviceProfileName() override;
		//~ End IDeviceProfileSelectorModule Interface


		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

		/**
		* Virtual destructor.
		*/
		virtual ~IPIEPreviewDeviceModule() = default;

		//~ Begin IPIEPreviewDeviceModule Interface

		/**
		 * Used to set the scalability preview platform
		 */
		virtual FName GetPreviewPlatformName() = 0;

		/**
		* Create PieWindow Ref
		*/
		virtual  TSharedRef<SWindow>   CreatePIEPreviewDeviceWindow(FVector2D ClientSize, FText WindowTitle, EAutoCenter AutoCenterType, FVector2D ScreenPosition, TOptional<float> MaxWindowWidth, TOptional<float> MaxWindowHeight) = 0;

		/**
		 * should be called after the window is created and registered and before scene rendering begins
		*/
		virtual void OnWindowReady(TSharedRef<SWindow> Window) {};

		/**
		* Apply PieWindow device parameters
		*/
		virtual void ApplyPreviewDeviceState() = 0;

		/** we need the game layer manager to control the DPI scaling behavior and this function can be called should be called when the manager is available */
		virtual void SetGameLayerManagerWidget(TSharedPtr<class SGameLayerManager> GameLayerManager)
		{
		}

		virtual void SetPreviewDevice(const FString& DeviceName) = 0;
		//~ End IDeviceProfileSelectorModule Interface
};
