// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTypeTraits.h"

class UInstancedStaticMeshComponent;
class UInstancedSkinnedMeshComponent;

/** Struct that allows batching of transforms and custom data of multiple (possibly instanced) static mesh components */
struct FISMComponentBatcher
{
public:
	FISMComponentBatcher()
	{
	}

	FISMComponentBatcher(bool bInBuildMappingInfo)
		: bBuildMappingInfo(bInBuildMappingInfo)
	{
	}

	/**
	 * Add a single component to be batched
	 * @param	InActorComponent	Component to be batched
	 */
	ENGINE_API void Add(const UActorComponent* InActorComponent);

	/**
	 * Add a single component to be batched
	 * @param	InActorComponent	Component to be batched
	 * @param	InTransformFunc		Function that takes the world space transform of an instance and modifies it. Must return a world space transform.
	 */
	ENGINE_API void Add(const UActorComponent* InActorComponent, TFunctionRef<FTransform(const FTransform&)> InTransformFunc);

	/**
	 * Add a single component to be batched
	 * @param	InActorComponent	Component to be batched
	 * @param	InFilterFunc		Function that can be used to filter out instances based on their world bounds.
	 */
	ENGINE_API void Add(const UActorComponent* InComponent, TFunctionRef<bool(const FBox&)> InFilterFunc);

	/**
	 * Add a single component to be batched
	 * @param	InActorComponent	Component to be batched
	 * @param	InTransformFunc		Function that takes the world space transform of an instance and modifies it. Must return a world space transform.
	 * @param	InFilterFunc		Function that can be used to filter out instances based on their world bounds.
	 */
	ENGINE_API void Add(const UActorComponent* InComponent, TFunctionRef<FTransform(const FTransform&)> InTransformFunc, TFunctionRef<bool(const FBox&)> InFilterFunc);
	
	/**
	 * Add an array of component to be batched
	 * @param	InComponents		Components to be batched
	 */
	template<typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	void Append(const TArray<TComponentClass*>& InComponents)
	{
		for (const UActorComponent* InComponent : InComponents)
		{
			Add(InComponent);
		}
	}

	/**
	 * Add an array of components to be batched
	 * @param	InComponents		Components to be batched
	 * @param	InTransformFunc		Function that takes the world space transform of an instance and modifies it. Must return a world space transform.
	 */
	template<typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	void Append(const TArray<TComponentClass*>& InComponents, TFunctionRef<FTransform(const FTransform&)> InTransformFunc)
	{
		for (const UActorComponent* InComponent : InComponents)
		{
			Add(InComponent, InTransformFunc);
		}
	}

	/**
	 * Add an array of components to be batched
	 * @param	InComponents		Components to be batched
	 * @param	InFilterFunc		Function that can be used to filter out instances based on their world bounds.
	 */
	template<typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	void Append(const TArray<TComponentClass*>& InComponents, TFunctionRef<bool(const FBox&)> InFilterFunc)
	{
		for (const UActorComponent* InComponent : InComponents)
		{
			Add(InComponent, InFilterFunc);
		}
	}

	/**
	 * Add an array of components to be batched
	 * @param	InComponents		Components to be batched
	 * @param	InFilterFunc		Function that can be used to filter out instances based on their world bounds.
	 * @param	InTransformFunc		Function that takes the world space transform of an instance and modifies it. Must return a world space transform.
	 */
	template<typename TComponentClass = UActorComponent, typename = typename TEnableIf<TIsDerivedFrom<TComponentClass, UActorComponent>::IsDerived>::Type>
	void Append(const TArray<TComponentClass*>& InComponents, TFunctionRef<FTransform(const FTransform&)> InTransformFunc, TFunctionRef<bool(const FBox&)> InFilterFunc)
	{
		for (const UActorComponent* InComponent : InComponents)
		{
			Add(InComponent, InTransformFunc, InFilterFunc);
		}
	}

	/**
	 * Return the number of instances batched so far.
	 */
	int32 GetNumInstances() const { return NumInstances; }

	/**
	 * Initialize the instances of the provided instanced static mesh component using the batched data stored in this class.
	 * @param	ISMComponent	Instanced static mesh component which will be modified.
	 */
	ENGINE_API void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const;

	/**
	 * Initialize the instances of the provided instanced skinned mesh component using the batched data stored in this class.
	 * @param	ISMComponent	Instanced skinned mesh component which will be modified.
	 */
	ENGINE_API void InitComponent(UInstancedSkinnedMeshComponent* ISMComponent) const;
	
	inline uint32 GetHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

	/**
     * For a given batched component, store the range of the instances associated to it
     */
	struct FComponentToInstancesMapping
	{
		const UActorComponent* Component;
		uint32 InstancesStart;
		uint32 InstancesCount;
	};

	/**
	 * Retrieve the component to instances mapping.
	 * Will only be populated if bBuildMappingInfo=true is provided to the batcher's constructor.
	 */
	ENGINE_API TArray<FComponentToInstancesMapping> GetComponentsToInstancesMap();
	
protected:
	void ENGINE_API AddInternal(const UActorComponent* InComponent, TOptional<TFunctionRef<FTransform(const FTransform&)>> InTransformFunc, TOptional<TFunctionRef<bool(const FBox&)>> InFilterFunc);
	void ENGINE_API ComputeHash() const;

	mutable uint32 Hash = 0;
	int32 NumInstances = 0;
	int32 NumCustomDataFloats = 0;

	TArray<FTransform> InstancesTransformsWS;
	TArray<float> InstancesCustomData;

	// For ISMC only
	int32 InstancingRandomSeed = 0;
	TArray<FInstancedStaticMeshRandomSeed> AdditionalRandomSeeds; 

	// For ISKMC only
	TArray<int32> AnimationIndices;

	// Mapping info to be able to use GetComponentsToInstancesMap() to retrieve the range of instances associated with a component added to the ISM batcher.
	bool bBuildMappingInfo = false;
	TArray<FComponentToInstancesMapping> ComponentsToInstancesMap;
};
