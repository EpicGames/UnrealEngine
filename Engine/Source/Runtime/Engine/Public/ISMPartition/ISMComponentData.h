// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "ISMComponentData.generated.h"

class UInstancedStaticMeshComponent;

/** Represents one component instance with a mapping back to the Client instance. The should be a 1 to 1 relationship between those and the actual Component instances. (equal count) */
USTRUCT()
struct FISMComponentInstance
{
	GENERATED_USTRUCT_BODY()

	FISMComponentInstance(const int32& InClientIndex, const int32& InInstanceIndex, const int32& InInstanceSubIndex)
		: ClientIndex(InClientIndex)
		, InstanceIndex(InInstanceIndex)
		, InstanceSubIndex(InInstanceSubIndex)
	{

	}

	FISMComponentInstance()
		: ClientIndex(-1)
		, InstanceIndex(-1)
		, InstanceSubIndex(-1)
	{

	}

	/** Client Index in the AISMPartitionActor */
	UPROPERTY()
	int32 ClientIndex;

	/** Instance Index in the FISMClientData struct */
	UPROPERTY()
	int32 InstanceIndex;

	/** Instance Index in the FISMClientInstance struct */
	UPROPERTY()
	int32 InstanceSubIndex;
};

/** Represents the component instances that 1 Client instance represents */
USTRUCT()
struct FISMClientInstance
{
	GENERATED_USTRUCT_BODY()

	/** Instance Index in the ISM Component */
	UPROPERTY()
	TArray<int32> ComponentIndices;
};

/** Represents the list of instances for 1 Client */
USTRUCT()
struct FISMClientData
{
	GENERATED_USTRUCT_BODY()

	/** Instance list for 1 client */
	UPROPERTY()
	TArray<FISMClientInstance> Instances;
};

/** Bookkeeping struct that contains the data that allows linking Client Instances to actual Component Instances */
USTRUCT()
struct FISMComponentData
{
	GENERATED_USTRUCT_BODY()

	FISMComponentData();

#if WITH_EDITOR
	void RegisterDelegates();
	void UnregisterDelegates();
	void HandleComponentMeshBoundsChanged(const FBoxSphereBounds& NewBounds);
#endif

#if WITH_EDITORONLY_DATA
	/** Instances here must match Component instances */
	UPROPERTY()
	TArray<FISMComponentInstance> Instances;

	/** Per Client Instances. Indexed using a Client index that must match the indexing in AISMPartitionActor */
	UPROPERTY()
	TArray<FISMClientData> ClientInstances;

	UPROPERTY()
	TObjectPtr<UInstancedStaticMeshComponent> Component;

	/** If Lighting cache should be invalidated at the end of a BeginUpdate/EndUpdate */
	bool bInvalidateLightingCache;

	/** Component's property value before setting it to false in the BeginUpdate so we can restore it in EndUpdate */
	bool bAutoRebuildTreeOnInstanceChanges;

	/** If Modified as already been called between a BeginUpdate/EndUpdate (avoid multiple Modify calls on the component) */
	bool bWasModifyCalled;
#endif
};