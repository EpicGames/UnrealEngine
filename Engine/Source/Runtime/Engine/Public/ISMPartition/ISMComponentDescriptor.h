// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Components/PrimitiveComponent.h"
#include "ISMComponentDescriptor.generated.h"

class UInstancedStaticMeshComponent;
class UStaticMesh;
enum class ERuntimeVirtualTextureMainPassType : uint8;
enum class ERendererStencilMask : uint8;

/** Struct that holds the relevant properties that can help decide if instances of different 
	StaticMeshComponents/InstancedStaticMeshComponents/HISM can be merged into a single component. */
USTRUCT()
struct FISMComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FISMComponentDescriptorBase();
	ENGINE_API explicit FISMComponentDescriptorBase(ENoInit);
	ENGINE_API FISMComponentDescriptorBase(const FISMComponentDescriptorBase&);
	ENGINE_API virtual ~FISMComponentDescriptorBase();

	ENGINE_API UInstancedStaticMeshComponent* CreateComponent(UObject* Outer, FName Name = NAME_None, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) const;

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true);
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const;

	ENGINE_API void PostLoadFixup(UObject* Loader);

	ENGINE_API bool operator!=(const FISMComponentDescriptorBase& Other) const;
	ENGINE_API bool operator==(const FISMComponentDescriptorBase& Other) const;

	static UMaterialInterface* DuplicateMaterialInterface(const UMaterialInterface& InMaterialInterface, UObject& NewOuter);

	friend inline uint32 GetTypeHash(const FISMComponentDescriptorBase& Key)
	{
		return Key.GetTypeHash();
	}

	uint32 GetTypeHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

public:
	UPROPERTY()
	mutable uint32 Hash = 0;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TSubclassOf<UInstancedStaticMeshComponent> ComponentClass;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<EComponentMobility::Type> Mobility;
		
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ELightmapType LightmapType;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	FLightingChannels LightingChannels;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 RayTracingGroupId;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERayTracingGroupCullingPriority RayTracingGroupCullingPriority;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<EHasCustomNavigableGeometry::Type> bHasCustomNavigableGeometry;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	ERendererStencilMask CustomDepthStencilWriteMask;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	FBodyInstance BodyInstance;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceMinDrawDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceStartCullDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 InstanceEndCullDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	float InstanceLODDistanceScale;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 VirtualTextureCullMips;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 TranslucencySortPriority;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 OverriddenLightMapRes;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 CustomDepthStencilValue;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	EHLODBatchingPolicy HLODBatchingPolicy;
#endif
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEmissiveLightSource : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDynamicIndirectLighting : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDynamicIndirectLightingWhileHidden : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bAffectDistanceFieldLighting : 1;
		
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastDynamicShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastStaticShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastVolumetricTranslucentShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastContactShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bSelfShadowOnly : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastFarShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastInsetShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastCinematicShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastHiddenShadow : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCastShadowAsTwoSided : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bReceivesDecals : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bOverrideLightMapRes : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bUseAsOccluder : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEnableDensityScaling : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEnableDiscardOnLoad : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRenderCustomDepth : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisibleInRayTracing : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisibleInReflections : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRenderInMainPass : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRenderInDepthPass : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bHiddenInGame : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bIsEditorOnly : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bVisible : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bSelectable : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bHasPerInstanceHitProxies : 1;
	
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bEvaluateWorldPositionOffset : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bReverseCulling : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bUseGpuLodSelection : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bIncludeInHLOD : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bConsiderForActorPlacementWhenHidden : 1;
#endif

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "bGenerateOverlapEvents"))
	uint8 bUseDefaultCollision : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "CustomDepthStencilWriteMask"))
	uint8 bGenerateOverlapEvents : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bOverrideNavigationExport : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bForceNavigationObstacle : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bCanEverAffectNavigation : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bFillCollisionUnderneathForNavmesh : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	uint8 bRasterizeAsFilledConvexVolume : 1;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 WorldPositionOffsetDisableDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	int32 NanitePixelProgrammableDistance;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	UPROPERTY(EditAnywhere, Category = "Component Settings")
	TEnumAsByte<enum EDetailMode> DetailMode;
};

USTRUCT()
struct FISMComponentDescriptor : public FISMComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FISMComponentDescriptor();
	ENGINE_API FISMComponentDescriptor(const FISMComponentDescriptor&);
	ENGINE_API ~FISMComponentDescriptor();
	ENGINE_API explicit FISMComponentDescriptor(const FSoftISMComponentDescriptor& Other);
	static ENGINE_API FISMComponentDescriptor CreateFrom(const TSubclassOf<UStaticMeshComponent>& ComponentClass);

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const override;

	TWeakObjectPtr<const UStaticMeshComponent> GetStaticMeshComponent() const
	{
		return StaticMeshComponent;
	}
		
	ENGINE_API bool operator!=(const FISMComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FISMComponentDescriptor& Other) const;

	friend inline bool operator<(const FISMComponentDescriptor& Lhs, const FISMComponentDescriptor& Rhs)
	{
		return Lhs.Hash < Rhs.Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TObjectPtr<UMaterialInterface> OverlayMaterial;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	FTransform LocalTransform = FTransform::Identity;

private:
	UPROPERTY(Transient)
	TWeakObjectPtr<const UStaticMeshComponent> StaticMeshComponent = nullptr;
};

USTRUCT()
struct FSoftISMComponentDescriptor : public FISMComponentDescriptorBase
{
	GENERATED_BODY()

	ENGINE_API FSoftISMComponentDescriptor();
	ENGINE_API ~FSoftISMComponentDescriptor();
	ENGINE_API explicit FSoftISMComponentDescriptor(const FISMComponentDescriptor& Other);
	ENGINE_API explicit FSoftISMComponentDescriptor(const FSoftISMComponentDescriptor& Other);
	static ENGINE_API FSoftISMComponentDescriptor CreateFrom(const TSubclassOf<UStaticMeshComponent>& ComponentClass);

	ENGINE_API virtual void InitFrom(const UStaticMeshComponent* Component, bool bInitBodyInstance = true) override;
	ENGINE_API virtual uint32 ComputeHash() const;
	ENGINE_API virtual void InitComponent(UInstancedStaticMeshComponent* ISMComponent) const override;

	ENGINE_API bool operator!=(const FSoftISMComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FSoftISMComponentDescriptor& Other) const;

	friend inline bool operator<(const FSoftISMComponentDescriptor& Lhs, const FSoftISMComponentDescriptor& Rhs)
	{
		return Lhs.Hash < Rhs.Hash;
	}

public:
	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TSoftObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TSoftObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TSoftObjectPtr<UMaterialInterface> OverlayMaterial;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (DisplayAfter = "ComponentClass"))
	TArray<TSoftObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;
};
