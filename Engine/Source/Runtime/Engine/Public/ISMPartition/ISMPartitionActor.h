// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ActorPartition/PartitionActor.h"
#include "ISMPartition/ISMPartitionClient.h"
#include "ISMPartition/ISMPartitionInstanceManager.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "ISMPartition/ISMComponentData.h"
#include "Templates/Tuple.h"
#include "Containers/SortedMap.h"
#include "Elements/SMInstance/SMInstanceManager.h"
#include "ISMPartitionActor.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogISMPartition, Log, All);

/** Actor base class for instance containers placed on a grid.
	See UActorPartitionSubsystem. */
UCLASS(Abstract, MinimalAPI)
class AISMPartitionActor : public APartitionActor, public ISMInstanceManager, public ISMInstanceManagerProvider
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR	
	//~ Begin AActor Interface
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	//~ End AActor Interface	

	ENGINE_API FISMClientHandle RegisterClient(const FGuid& ClientGuid);
	ENGINE_API void UnregisterClient(FISMClientHandle& ClientHandle);

	ENGINE_API void RegisterClientInstanceManager(const FISMClientHandle& Handle, IISMPartitionInstanceManager* ClientInstanceManager);
	ENGINE_API void RegisterClientInstanceManagerProvider(const FISMClientHandle& Handle, IISMPartitionInstanceManagerProvider* ClientInstanceManagerProvider);

	ENGINE_API int32 RegisterISMComponentDescriptor(const FISMComponentDescriptor& Descriptor);
	const FISMComponentDescriptor& GetISMComponentDescriptor(int32 DescriptorIndex) const { return Descriptors[DescriptorIndex]; }

	ENGINE_API TArray<FSMInstanceId> AddISMInstance(const FISMClientHandle& Handle, const FTransform& InstanceTransform, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition);
	ENGINE_API void RemoveISMInstance(const FISMClientHandle& Handle, int32 InstanceIndex, bool* bOutIsEmpty = nullptr);
	ENGINE_API void RemoveISMInstances(const FISMClientHandle& Handle);
	ENGINE_API void SelectISMInstances(const FISMClientHandle& Handle, bool bSelect, const TSet<int32>& Indices);
	ENGINE_API void SetISMInstanceTransform(const FISMClientHandle& Handle, int32 InstanceIndex, const FTransform& NewTransform, bool bTeleport, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition);
	ENGINE_API int32 GetISMInstanceIndex(const FISMClientHandle& Handle, const UInstancedStaticMeshComponent* ISMComponent, int32 ComponentIndex) const;
	ENGINE_API FBox GetISMInstanceBounds(const FISMClientHandle& Handle, const TSet<int32>& Indices) const;
	ENGINE_API void ReserveISMInstances(const FISMClientHandle& Handle, int32 AddedInstanceCount, const TSortedMap<int32, TArray<FTransform>>& InstanceDefinition);

	ENGINE_API bool IsISMComponent(const UPrimitiveComponent* Component) const;
	ENGINE_API void BeginUpdate();
	ENGINE_API void EndUpdate();
	ENGINE_API void UpdateHISMTrees(bool bAsync, bool bForce);
	ENGINE_API void ForEachClientComponent(const FISMClientHandle& Handle, TFunctionRef<bool(UInstancedStaticMeshComponent*)> Callback) const;
	ENGINE_API void ForEachClientSMInstance(const FISMClientHandle& Handle, TFunctionRef<bool(FSMInstanceId)> Callback) const;
	ENGINE_API void ForEachClientSMInstance(const FISMClientHandle& Handle, int32 InstanceIndex, TFunctionRef<bool(FSMInstanceId)> Callback) const;
	ENGINE_API void OutputStats() const;

protected:

private:
	ENGINE_API void RemoveISMInstancesInternal(FISMComponentData& ComponentData, FISMClientData& OwnerData, int32 InstanceIndex);

	ENGINE_API void InvalidateComponentLightingCache(FISMComponentData& ComponentData);
	ENGINE_API int32 AddInstanceToComponent(FISMComponentData& ComponentData, const FTransform& WorldTransform);
	ENGINE_API void UpdateInstanceTransform(FISMComponentData& ComponentData, int32 ComponentInstanceIndex, const FTransform& WorldTransform, bool bTeleport);
	ENGINE_API void RemoveInstanceFromComponent(FISMComponentData& ComponentData, int32 ComponentInstanceIndex);
	ENGINE_API bool DestroyComponentIfEmpty(FISMComponentDescriptor& Descriptor, FISMComponentData& ComponentData);
	ENGINE_API void ModifyComponent(FISMComponentData& ComponentData);
	ENGINE_API void CreateComponent(const FISMComponentDescriptor& Descriptor, FISMComponentData& ComponentData);
	ENGINE_API void ModifyActor();
#endif

private:
	//~ ISMInstanceManager interface
	ENGINE_API virtual FText GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const override final;
	ENGINE_API virtual FText GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const override final;
	ENGINE_API virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const override final;
	ENGINE_API virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const override final;
	ENGINE_API virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override final;
	ENGINE_API virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) override final;
	ENGINE_API virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) override final;
	ENGINE_API virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) override final;
	ENGINE_API virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) override final;
	ENGINE_API virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) override final;
	ENGINE_API virtual void ForEachSMInstanceInSelectionGroup(const FSMInstanceId& InstanceId, TFunctionRef<bool(FSMInstanceId)> Callback) override final;
	ENGINE_API virtual bool CanDeleteSMInstance(const FSMInstanceId& InstanceId) const override final;
	ENGINE_API virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) override final;
	ENGINE_API virtual bool CanDuplicateSMInstance(const FSMInstanceId& InstanceId) const override final;
	ENGINE_API virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) override final;

#if WITH_EDITOR
	ENGINE_API FISMPartitionInstanceManager GetISMPartitionInstanceManager(const FSMInstanceId& InstanceId) const;
	ENGINE_API FISMPartitionInstanceManager GetISMPartitionInstanceManagerChecked(const FSMInstanceId& InstanceId) const;
#endif

protected:
	//~ ISMInstanceManagerProvider interface
	ENGINE_API virtual ISMInstanceManager* GetSMInstanceManager(const FSMInstanceId& InstanceId) override /*final*/; // Note: This should also be final and private, but AInstancedFoliageActor needs to override it to support ISMC foliage

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FGuid> Clients;
		
	UPROPERTY()
	TArray<FISMComponentDescriptor> Descriptors;

	UPROPERTY()
	TArray<FISMComponentData> DescriptorComponents;

	UPROPERTY()
	TMap<FGuid, FISMClientInstanceManagerData> ClientInstanceManagers;

	/** If greater than 0 means we are between a BeginUpdate/EndUpdate call and there are some things we can delay/optimize */
	int32 UpdateDepth;

	/** If Modified as already been called between a BeginUpdate/EndUpdate (avoid multiple Modify calls on the component) */
	bool bWasModifyCalled;
#endif
};
