// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ISMPartitionClient.generated.h"

USTRUCT()
struct FISMClientHandle
{
	GENERATED_BODY();

	FISMClientHandle() = default;

	bool IsValid() const { return Index >= 0 && Guid.IsValid(); }

	explicit operator bool() const
	{
		return IsValid();
	}

	bool operator==(const FISMClientHandle& InRHS) const
	{
		return Index == InRHS.Index
			&& Guid == InRHS.Guid;
	}

	bool operator!=(const FISMClientHandle& InRHS) const
	{
		return !(*this == InRHS);
	}

	void Serialize(FArchive& Ar)
	{
		Ar << Index;
		Ar << Guid;
	}

	friend inline uint32 GetTypeHash(const FISMClientHandle& InHandle)
	{
		return GetTypeHash(InHandle.Guid);
	}

private:
	friend class AISMPartitionActor;

	FISMClientHandle(int32 ClientIndex, FGuid ClientGuid)
		: Index(ClientIndex)
		, Guid(ClientGuid)
	{

	}

	UPROPERTY()
	int32 Index = INDEX_NONE;

	UPROPERTY()
	FGuid Guid;
};

struct FISMClientInstanceId
{
	explicit operator bool() const
	{
		return Handle.IsValid()
			&& Index != INDEX_NONE;
	}

	bool operator==(const FISMClientInstanceId& InRHS) const
	{
		return Handle == InRHS.Handle
			&& Index == InRHS.Index;
	}

	bool operator!=(const FISMClientInstanceId& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend inline uint32 GetTypeHash(const FISMClientInstanceId& InId)
	{
		return HashCombine(GetTypeHash(InId.Index), GetTypeHash(InId.Handle));
	}

	FISMClientHandle Handle;
	int32 Index = INDEX_NONE;
};
