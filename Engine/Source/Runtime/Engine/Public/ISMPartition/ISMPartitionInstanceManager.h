// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Containers/ArrayView.h"
#include "ISMPartition/ISMPartitionClient.h"
#include "ISMPartitionInstanceManager.generated.h"

enum class ETypedElementWorldType : uint8;

/**
 * An interface for clients that manage ISM instances within a partition actor.
 * This exists so that clients can inject custom logic around the manipulation of the underlying ISM instance.
 * @note The client ISM instances given to this API must be valid and belong to this instance manager. The instance manager implementation is free to assert or crash if that contract is broken.
 */
UINTERFACE(MinimalAPI)
class UISMPartitionInstanceManager : public UInterface
{
	GENERATED_BODY()
};
class IISMPartitionInstanceManager
{
	GENERATED_BODY()

public:
	/**
	 * Get the display name of the given client ISM instance.
	 */
	virtual FText GetISMPartitionInstanceDisplayName(const FISMClientInstanceId& InstanceId) const
	{
		return FText();
	}

	/**
	 * Get the tooltip of the given client ISM instance.
	 */
	virtual FText GetISMPartitionInstanceTooltip(const FISMClientInstanceId& InstanceId) const
	{
		return FText();
	}

	/**
	 * Can the given client ISM instance be edited?
	 * @return True if it can be edited, false otherwise.
	 */
	virtual bool CanEditISMPartitionInstance(const FISMClientInstanceId& InstanceId) const = 0;

	/**
	 * Can the given client ISM instance be moved in the world?
	 * @return True if it can be moved, false otherwise.
	 */
	virtual bool CanMoveISMPartitionInstance(const FISMClientInstanceId& InstanceId, const ETypedElementWorldType WorldType) const = 0;

	/**
	 * Attempt to get the transform of the given client ISM instance.
	 * @note The transform should be in the local space of the owner partition actor, unless bWorldSpace is set.
	 * @return True if the transform was retrieved, false otherwise.
	 */
	virtual bool GetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const = 0;

	/**
	 * Attempt to set the world transform of the given client ISM instance.
	 * @note The transform should be in the local space of the owner partition actor, unless bWorldSpace is set.
	 * @return True if the transform was updated, false otherwise.
	 */
	virtual bool SetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bTeleport = false) = 0;

	/**
	 * Notify that the given client ISM instance is about to be moved.
	 * @note This gives the manager a chance to start a move operation, to avoid performing repeated work until the move is finished.
	 */
	virtual void NotifyISMPartitionInstanceMovementStarted(const FISMClientInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given client ISM instance is currently being moved.
	 * @note This gives the manager a chance to update a move operation.
	 */
	virtual void NotifyISMPartitionInstanceMovementOngoing(const FISMClientInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given client ISM instance is done being moved.
	 * @note This gives the manager a chance to end a move operation.
	 */
	virtual void NotifyISMPartitionInstanceMovementEnded(const FISMClientInstanceId& InstanceId) = 0;

	/**
	 * Notify that the given client ISM instance selection state has changed.
	 * @note This gives the manager a chance to sync any internal selection state.
	 */
	virtual void NotifyISMPartitionInstanceSelectionChanged(const FISMClientInstanceId& InstanceId, const bool bIsSelected) = 0;

	/**
	 * Can the given client ISM instance be deleted?
	 * @return True if it can be deleted, false otherwise.
	 */
	virtual bool CanDeleteISMPartitionInstance(const FISMClientInstanceId& InstanceId) const
	{
		return CanEditISMPartitionInstance(InstanceId);
	}

	/**
	 * Attempt to delete the given client ISM instances.
	 * @return True if any instances were deleted, false otherwise.
	 */
	virtual bool DeleteISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds) = 0;

	/**
	 * Can the given client ISM instance be duplicated?
	 * @return True if it can be duplicated, false otherwise.
	 */
	virtual bool CanDuplicateISMPartitionInstance(const FISMClientInstanceId& InstanceId) const
	{
		return CanEditISMPartitionInstance(InstanceId);
	}

	/**
	 * Attempt to duplicate the given client ISM instances, retrieving the IDs of any new instances.
	 * @return True if any instances were duplicated, false otherwise.
	 */
	virtual bool DuplicateISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds, TArray<FISMClientInstanceId>& OutNewInstanceIds) = 0;
};

/**
 * An interface for actors that can provide a manager for ISM instances within a partition actor.
 * This exists so that objects that indirectly manage clients can provide the correct underlying manager for the specific client.
 */
UINTERFACE(MinimalAPI)
class UISMPartitionInstanceManagerProvider : public UInterface
{
	GENERATED_BODY()
};
class IISMPartitionInstanceManagerProvider
{
	GENERATED_BODY()

public:
	/**
	 * Attempt to get the instance manager associated with the given client, if any.
	 * @return The instance manager, or null if there is no instance manager associated with the given client.
	 */
	virtual IISMPartitionInstanceManager* GetISMPartitionInstanceManager(const FISMClientHandle& ClientHandle) = 0;
};

/**
 * An ISM partition instance manager, tied to a given client instance ID.
 */
struct FISMPartitionInstanceManager
{
public:
	FISMPartitionInstanceManager() = default;

	FISMPartitionInstanceManager(const FISMClientInstanceId& InInstanceId, IISMPartitionInstanceManager* InInstanceManager)
		: InstanceId(InInstanceId)
		, InstanceManager(InInstanceManager)
	{
	}

	explicit operator bool() const
	{
		return InstanceId
			&& InstanceManager;
	}

	bool operator==(const FISMPartitionInstanceManager& InRHS) const
	{
		return InstanceId == InRHS.InstanceId
			&& InstanceManager == InRHS.InstanceManager;
	}

	bool operator!=(const FISMPartitionInstanceManager& InRHS) const
	{
		return !(*this == InRHS);
	}

	friend inline uint32 GetTypeHash(const FISMPartitionInstanceManager& InId)
	{
		return GetTypeHash(InId.InstanceId);
	}

	const FISMClientInstanceId& GetInstanceId() const { return InstanceId; }
	IISMPartitionInstanceManager* GetInstanceManager() const { return InstanceManager; }

	//~ IISMPartitionInstanceManager interface
	FText GetISMPartitionInstanceDisplayName() const { return InstanceManager->GetISMPartitionInstanceDisplayName(InstanceId); }
	FText GetISMPartitionInstanceTooltip() const { return InstanceManager->GetISMPartitionInstanceTooltip(InstanceId); }
	bool CanEditISMPartitionInstance() const { return InstanceManager->CanEditISMPartitionInstance(InstanceId); }
	bool CanMoveISMPartitionInstance(const ETypedElementWorldType WorldType) const { return InstanceManager->CanMoveISMPartitionInstance(InstanceId, WorldType); }
	bool GetISMPartitionInstanceTransform(FTransform& OutInstanceTransform, bool bWorldSpace = false) const { return InstanceManager->GetISMPartitionInstanceTransform(InstanceId, OutInstanceTransform, bWorldSpace); }
	bool SetISMPartitionInstanceTransform(const FTransform& InstanceTransform, bool bWorldSpace = false, bool bTeleport = false) const { return InstanceManager->SetISMPartitionInstanceTransform(InstanceId, InstanceTransform, bWorldSpace, bTeleport); }
	void NotifyISMPartitionInstanceMovementStarted() const { return InstanceManager->NotifyISMPartitionInstanceMovementStarted(InstanceId); }
	void NotifyISMPartitionInstanceMovementOngoing() const { return InstanceManager->NotifyISMPartitionInstanceMovementOngoing(InstanceId); }
	void NotifyISMPartitionInstanceMovementEnded() const { return InstanceManager->NotifyISMPartitionInstanceMovementEnded(InstanceId); }
	void NotifyISMPartitionInstanceSelectionChanged(const bool bIsSelected) const { return InstanceManager->NotifyISMPartitionInstanceSelectionChanged(InstanceId, bIsSelected); }
	bool CanDeleteISMPartitionInstance() const { return InstanceManager->CanDeleteISMPartitionInstance(InstanceId); }
	bool DeleteISMPartitionInstance() const { return InstanceManager->DeleteISMPartitionInstances(MakeArrayView(&InstanceId, 1)); }
	bool CanDuplicateISMPartitionInstance() const { return InstanceManager->CanDuplicateISMPartitionInstance(InstanceId); }
	bool DuplicateISMPartitionInstance(FISMClientInstanceId& OutNewInstanceId) const
	{
		TArray<FISMClientInstanceId> NewInstanceIds;
		if (InstanceManager->DuplicateISMPartitionInstances(MakeArrayView(&InstanceId, 1), NewInstanceIds))
		{
			OutNewInstanceId = NewInstanceIds[0];
			return true;
		}
		return false;
	}

private:
	FISMClientInstanceId InstanceId;
	IISMPartitionInstanceManager* InstanceManager = nullptr;
};

USTRUCT()
struct FISMClientInstanceManagerData
{
	GENERATED_BODY()
	
public:
	bool Serialize(FArchive& Ar)
	{
		SerializePtr(Ar, Manager);
		SerializePtr(Ar, ManagerProvider);
		return true;
	}

	void SetInstanceManager(IISMPartitionInstanceManager* InManager)
	{
		Manager = InManager;
		ManagerProvider = nullptr;
	}

	void SetInstanceManagerProvider(IISMPartitionInstanceManagerProvider* InManagerProvider)
	{
		Manager = nullptr;
		ManagerProvider = InManagerProvider;
	}
	
	IISMPartitionInstanceManager* ResolveInstanceManager(const FISMClientHandle& ClientHandle) const
	{
		return ManagerProvider
			? ManagerProvider->GetISMPartitionInstanceManager(ClientHandle)
			: Manager;
	}
	
private:
	template <typename T>
	static void SerializePtr(FArchive& Ar, T*& Ptr)
	{
		enum class ERefType : uint8
		{
			Null,
			Object,
			Raw,
		};

		if (Ar.IsSaving())
		{
			if (UObject* Obj = Cast<UObject>(Ptr))
			{
				ERefType RefType = ERefType::Object;
				Ar << RefType;

				Ar << Obj;
			}
			else if (Ptr && !Ar.IsPersistent())
			{
				ERefType RefType = ERefType::Raw;
				Ar << RefType;

				uint64 PtrInt = (uint64)Ptr;
				Ar << PtrInt;
			}
			else
			{
				ERefType RefType = ERefType::Null;
				Ar << RefType;
			}
		}
		else if (Ar.IsLoading())
		{
			ERefType RefType;
			Ar << RefType;

			switch (RefType)
			{
			case ERefType::Object:
				{
					UObject* Obj = nullptr;
					Ar << Obj;
					Ptr = Cast<T>(Obj);
				}
				break;
			case ERefType::Raw:
				{
					uint64 PtrInt = 0;
					Ar << PtrInt;
					Ptr = (T*)PtrInt;
				}
				break;
			default:
				Ptr = nullptr;
				break;
			}
		}
	};

	IISMPartitionInstanceManager* Manager = nullptr;
	IISMPartitionInstanceManagerProvider* ManagerProvider = nullptr;
};

template<>
struct TStructOpsTypeTraits<FISMClientInstanceManagerData> : public TStructOpsTypeTraitsBase2<FISMClientInstanceManagerData>
{
	enum
	{
		WithSerializer = true,
	};
};
