// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

#include "ProceduralISMComponentDescriptor.generated.h"

class UStaticMesh;
class UMaterialInterface;
class URuntimeVirtualTexture;
enum class ERayTracingGroupCullingPriority : uint8;
enum class ERendererStencilMask : uint8;
enum class ERuntimeVirtualTextureMainPassType : uint8;
enum class EShadowCacheInvalidationBehavior : uint8;
enum EDetailMode : int;
namespace EComponentMobility { enum Type : int; }
struct FSoftISMComponentDescriptor;

/** Struct that holds properties that can be used to initialize Procedural ISM Components. */
USTRUCT()
struct FProceduralISMComponentDescriptor
{
	GENERATED_BODY()

public:
	ENGINE_API FProceduralISMComponentDescriptor();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FProceduralISMComponentDescriptor() = default;
	FProceduralISMComponentDescriptor(const FProceduralISMComponentDescriptor&) = default;
	FProceduralISMComponentDescriptor(FProceduralISMComponentDescriptor&&) = default;
	FProceduralISMComponentDescriptor& operator=(const FProceduralISMComponentDescriptor&) = default;
	FProceduralISMComponentDescriptor& operator=(FProceduralISMComponentDescriptor&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ENGINE_API FProceduralISMComponentDescriptor& operator=(const FSoftISMComponentDescriptor& Other);

	ENGINE_API virtual uint32 ComputeHash() const;

	ENGINE_API bool operator!=(const FProceduralISMComponentDescriptor& Other) const;
	ENGINE_API bool operator==(const FProceduralISMComponentDescriptor& Other) const;

	friend inline uint32 GetTypeHash(const FProceduralISMComponentDescriptor& Key)
	{
		return Key.GetTypeHash();
	}

	uint32 GetTypeHash() const
	{
		if (Hash == 0)
		{
			ComputeHash();
		}
		return Hash;
	}

public:
	UPROPERTY()
	mutable uint32 Hash = 0;

	UPROPERTY()
	TSoftObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> OverlayMaterial;

	UPROPERTY()
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	UPROPERTY()
	int32 NumInstances;

	UPROPERTY()
	int32 NumCustomFloats;

	UPROPERTY()
	FBox WorldBounds;

	UPROPERTY()
	int32 InstanceMinDrawDistance;

	UPROPERTY()
	int32 InstanceStartCullDistance;

	UPROPERTY()
	int32 InstanceEndCullDistance;

	UPROPERTY()
	TEnumAsByte<EComponentMobility::Type> Mobility;
		
	UPROPERTY()
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType;
	
	UPROPERTY()
	FLightingChannels LightingChannels;

	UPROPERTY()
	ERendererStencilMask CustomDepthStencilWriteMask;

	UPROPERTY()
	int32 VirtualTextureCullMips;

	UPROPERTY()
	int32 TranslucencySortPriority;

	UPROPERTY()
	int32 CustomDepthStencilValue;

	UPROPERTY()
	int32 RayTracingGroupId;

	UPROPERTY()
	ERayTracingGroupCullingPriority RayTracingGroupCullingPriority;

	UPROPERTY()
	uint8 bVisibleInRayTracing : 1;

	UPROPERTY()
	uint8 bVisibleInReflections : 1;

	UPROPERTY()
	uint8 bCastShadow : 1;

	UPROPERTY()
	uint8 bEmissiveLightSource : 1;
		
	UPROPERTY()
	uint8 bCastDynamicShadow : 1;

	UPROPERTY()
	uint8 bCastStaticShadow : 1;

	UPROPERTY()
	uint8 bCastContactShadow : 1;

	UPROPERTY()
	uint8 bCastShadowAsTwoSided : 1;

	UPROPERTY()
	uint8 bCastHiddenShadow : 1;

	UPROPERTY()
	uint8 bReceivesDecals : 1;

	UPROPERTY()
	uint8 bUseAsOccluder : 1;

	UPROPERTY()
	uint8 bRenderCustomDepth : 1;

	UPROPERTY()
	uint8 bEvaluateWorldPositionOffset : 1;

	UPROPERTY()
	uint8 bReverseCulling : 1;

	UPROPERTY()
	int32 WorldPositionOffsetDisableDistance;

	UPROPERTY()
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	UPROPERTY()
	TEnumAsByte<enum EDetailMode> DetailMode;
};
