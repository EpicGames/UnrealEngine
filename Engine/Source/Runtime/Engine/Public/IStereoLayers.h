// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IStereoLayers.h: Abstract interface for adding in stereoscopically projected
	layers on top of the world
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "StereoLayerShapes.h"
#include "RHI.h"
#include "HAL/Platform.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Engine/Texture.h"

class IStereoLayers
{
	// The pointer to the shape of a stereo layer is wrapped in this simple class that clones and destroys the shape.
	// This is to avoid having to implement non-default copy and assignment operations for the entire FLayerDesc struct.
	struct FShapeWrapper
	{
		TUniquePtr<IStereoLayerShape> Wrapped;
		FShapeWrapper(IStereoLayerShape* InWrapped)
			: Wrapped(InWrapped)
		{}

		FShapeWrapper(const FShapeWrapper& In) :
			Wrapped(In.IsValid()?In.Wrapped->Clone():nullptr)
		{}

		FShapeWrapper(FShapeWrapper&& In)
		{
			Wrapped = MoveTemp(In.Wrapped);
		}

		FShapeWrapper& operator= (IStereoLayerShape* InWrapped)
		{
			Wrapped = TUniquePtr<IStereoLayerShape>(InWrapped);
			return *this;
		}

		FShapeWrapper& operator= (const FShapeWrapper& In)
		{
			return operator=(In.IsValid() ? In.Wrapped->Clone() : nullptr);
		}

		FShapeWrapper& operator= (FShapeWrapper&& In)
		{
			Wrapped = MoveTemp(In.Wrapped);
			return *this;
		}

		bool IsValid() const { return Wrapped.IsValid(); }
	};

public:

	enum ELayerType
	{
		WorldLocked,
		TrackerLocked,
		FaceLocked
	};

	enum ELayerFlags
	{
		// Internally copies the texture on every frame for video, etc.
		LAYER_FLAG_TEX_CONTINUOUS_UPDATE	= 0x00000001,
		// Ignore the textures alpha channel, this makes the stereo layer opaque. Flag is ignored on Steam VR.
		LAYER_FLAG_TEX_NO_ALPHA_CHANNEL		= 0x00000002,
		// Quad Y component will be calculated based on the texture dimensions
		LAYER_FLAG_QUAD_PRESERVE_TEX_RATIO	= 0x00000004,
		// The layer will intersect with the scene's depth. Currently only supported on Oculus platforms.
		LAYER_FLAG_SUPPORT_DEPTH = 0x00000008,
		// Required on some platforms to enable rendering of external textures.
		LAYER_FLAG_TEX_EXTERNAL = 0x00000010,
		// When set, this layer will not be rendered.
		LAYER_FLAG_HIDDEN = 0x00000020,
		// When this is set and the HMD implementation is compatible, the layer will be copied to the spectator screen.
		LAYER_FLAG_DEBUG = 0x00000040,
		// Max flag value, update this when new flags are added!
		LAYER_FLAG_MAX_VALUE = LAYER_FLAG_DEBUG << 1,
	};


	/**
	 * Structure describing the visual appearance of a single stereo layer
	 */
	struct FLayerDesc
	{
		FLayerDesc()
			: Shape(new FQuadLayer())
		{
		}

		FLayerDesc(const IStereoLayerShape& InShape)
			: Shape(InShape.Clone())
		{
		}

		FLayerDesc(const FLayerDesc&) = default;
		FLayerDesc(FLayerDesc&&) = default;
		FLayerDesc& operator=(const FLayerDesc&) = default;
		FLayerDesc& operator=(FLayerDesc&&) = default;

		bool HasValidTexture() const { return (TextureObj.IsValid() && TextureObj->GetResource() != nullptr); }
		bool IsVisible() const { return !(Flags & LAYER_FLAG_HIDDEN) && HasValidTexture(); }

		// Layer IDs must be larger than 0
		const static uint32	INVALID_LAYER_ID = 0; 
		// The layer's ID
		uint32				Id			= INVALID_LAYER_ID;
		// View space transform
		FTransform			Transform	 = FTransform::Identity;
		// Size of rendered quad
		FVector2D			QuadSize	 = FVector2D(1.0f, 1.0f);
		// UVs of rendered quad in UE units
		FBox2D				UVRect		 = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));

		// Size of texture that the compositor should allocate. Unnecessary if Texture is provided. The compositor will allocate a cubemap whose faces are of LayerSize if ShapeType is CubemapLayer.
		FIntPoint			LayerSize = FIntPoint(0, 0);
		// Render order priority, higher priority render on top of lower priority. Face-Locked layers are rendered on top of other layer types regardless of priority. 
		int32				Priority	 = 0;
		// Which space the layer is locked within
		ELayerType			PositionType = ELayerType::FaceLocked;
		// which shape of layer it is. FQuadLayer is the only shape supported by all VR platforms.
		// queries the shape of the layer at run time
		template <typename T> 
		[[nodiscard]] bool HasShape() const { return ensure(Shape.IsValid()) && Shape.Wrapped->GetShapeName() == T::ShapeName; }
		
		// returns Shape cast to the supplied type. It's up to the caller to have ensured the cast is valid before calling this method.
		template <typename T> T& GetShape() { check(HasShape<T>()); return *static_cast<T*>(Shape.Wrapped.Get()); }
		template <typename T> const T& GetShape() const { check(HasShape<T>()); return *static_cast<const T*>(Shape.Wrapped.Get()); }
		template <typename T, typename... InArgTypes> void SetShape(InArgTypes&&... Args) { Shape = FShapeWrapper(new T(Forward<InArgTypes>(Args)...)); }


		// Texture mapped for right eye (if one texture provided, mono assumed)
		// Layers known to the IStereoLayers will pin this texture in memory, preventing GC
		TWeakObjectPtr<class UTexture>		TextureObj	 = nullptr;
		// Texture mapped for left eye (if one texture provided, mono assumed)
		// Layers known to the IStereoLayers will pin this texture in memory, preventing GC
		TWeakObjectPtr<class UTexture>		LeftTextureObj = nullptr;
		// Uses LAYER_FLAG_... -- See: ELayerFlags
		uint32				Flags		 = 0;

#if WITH_EDITOR
		FString Name;
#endif
		
	private:
		FShapeWrapper		Shape;
	};

	virtual ~IStereoLayers() = default;

	/**
	 * Creates a new layer from a given texture resource, which is projected on top of the world as a quad.
	 *
	 * @param	InLayerDesc		A reference to the texture resource to be used on the quad
	 * @return	A unique identifier for the layer created
	 */
	virtual uint32 CreateLayer(const FLayerDesc& InLayerDesc) = 0;
	
	/**
	 * Destroys the specified layer, stopping it from rendering over the world.
	 *
	 * @param	LayerId		The ID of layer to be destroyed
	 */
	virtual void DestroyLayer(uint32 LayerId) = 0;

	/**
	 * Saves the current stereo layer state on a stack to later restore them.
	 *
	 * Useful for creating temporary overlays that should be torn down later.
	 *
	 * When bPreserve is false, existing layers will be temporarily disabled and restored again when calling PopLayerState()
	 * The disabled layer's properties are still accessible by calling Get and SetLayerDesc, but nothing will change until after
	 * the state has been restored. Calling DestroyLayer on an inactive layer, will prevent it from being restored when PopLayerState() is called.
	 *
	 * When bPreserve is true, existing layers will remain active, but when calling PopLayerState(), any changed properties
	 * will be restored back to their previous values. Calling DestroyLayer on an active layer id will make the layer inactive. The layer
	 * will be reactivated when the state is restored. (You can call DestroyLayer multiple times on the same layer id to remove successively older
	 * versions of a layer.)
	 *
	 * In either case, layers created after PushLayerState() will be destroyed upon calling PopLayerState(). 
	 * 
	 * @param	bPreserve	Whether the existing layers should be preserved after saving the state. If false all existing layers will be disabled.
	 */
	virtual void PushLayerState(bool bPreserve = false) {};

	/**
	 * Restores the stereo layer state from the last save state. 
	 * 
	 * Currently active layers will be destroyed and replaced with the previous state.
	 */
	virtual void PopLayerState() {}

	/**
	 * Returns true if the StereoLayers implementation supports saving and restoring state using Push/PopLayerState()
	 */
	virtual bool SupportsLayerState() { return false; }

	/** 
	 * Optional method to hide the 3D scene and only render the stereo overlays. 
	 * No-op if not supported by the platform.
	 *
	 * If pushing and popping layer state is supported, the visibility of the background layer should be part of
	 * the saved state.
	 */
	virtual void HideBackgroundLayer() {}

	/**
	 * Optional method to undo the effect of hiding the 3D scene.
	 * No-op if not supported by the platform.
	 */
	virtual void ShowBackgroundLayer() {}

	/** 
	 * Tell if the background layer is visible. Platforms that do not implement Hide/ShowBackgroundLayer() 
	 * always return true.
	 */
	virtual bool IsBackgroundLayerVisible() const { return true; }

	/**
	 * Set the a new layer description
	 *
	 * @param	LayerId		The ID of layer to be set the description
	 * @param	InLayerDesc	The new description to be set
	 */
	virtual void SetLayerDesc(uint32 LayerId, const FLayerDesc& InLayerDesc) = 0;

	/**
	 * Get the currently set layer description
	 *
	 * @param	LayerId			The ID of layer to be set the description
	 * @param	OutLayerDesc	The returned layer description
	 * @return	Whether the returned layer description is valid
	 */
	virtual bool GetLayerDesc(uint32 LayerId, FLayerDesc& OutLayerDesc) = 0;

	/**
	 * Get a reference to the internal FLayerDesc structure for a given layer ID
	 * Only finds layers from the top of the Push/PopLayerState stack
	 * * Deprecated 5.6 Behavior: Implementations may choose not to implement this.
	 * * Callers supporting pre-5.6 plugins should fallback to GetLayerDesc if this returns nullptr.
	 * @param	LayerId		The ID of the layer to find
	 * @return	Pointer to the layer desc, or null if no layer was found
	 */
	virtual const FLayerDesc* FindLayerDesc(uint32 LayerId) const
	{
		return nullptr;
	};

	/**
	 * Marks this layers texture for update
	 *
	 * @param	LayerId			The ID of layer to be set the description
	 */
	virtual void MarkTextureForUpdate(uint32 LayerId) = 0;

	/**
	 * Update splash screens from current state
	 */
	UE_DEPRECATED(5.6, "This unused function will be removed. Use IXRLoadingScreen::AddSplash instead")
	virtual void UpdateSplashScreen() final {}

	/**
	 * Returns a list of debug textures to be rendered onto the default spectator screen.
	 * This is intended to be the textures for layers which have LAYER_FLAG_DEBUG set.
	 * All textures in this array should be nonnull and 2D in dimension.
	 */
	virtual TArray<FTextureRHIRef, TInlineAllocator<2>> GetDebugLayerTextures_RenderThread() { return {}; }

public:

	virtual ENGINE_API FLayerDesc GetDebugCanvasLayerDesc(class UTextureRenderTarget2D* Texture);
};
