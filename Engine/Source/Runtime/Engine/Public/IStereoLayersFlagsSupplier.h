// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "UObject/NameTypes.h"

class IStereoLayersFlagsSupplier : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("StereoLayersFlagsSupplier");
		return FeatureName;
	}

	/**
	 * Adds stereo layer flags provided by this supplier to the 
	 * list of all flags. Flags are unique identifiers and, if two suppliers
	 * provide a flag with the same name, only one will be kept.
	 * IStereoLayersFlagsSuppliers must not remove flags from the
	 * OutFlags parameter but should only add them.
	 * 
	 * The flags are intended to be added to StereoLayerComponents to 
	 * add functionalities.
	 * 
	 * @param OutFlags		The set containing all flags
	 * 
	 */
	virtual void EnumerateFlags(TSet<FName>& OutFlags) = 0;
};
