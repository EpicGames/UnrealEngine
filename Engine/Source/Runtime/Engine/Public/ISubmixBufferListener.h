// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#include "Templates/SharedPointer.h"

// Forward Declarations
class USoundSubmix;

/** Abstract interface for receiving audio data from a given submix. */
class ISubmixBufferListener : public TSharedFromThis<ISubmixBufferListener, ESPMode::ThreadSafe>
{
public:
	virtual ~ISubmixBufferListener() = default;
	
	/**
	Called when a new buffer has been rendered for a given submix
	@param OwningSubmix	The submix object which has rendered a new buffer
	@param AudioData		Ptr to the audio buffer
	@param NumSamples		The number of audio samples in the audio buffer
	@param NumChannels		The number of channels of audio in the buffer (e.g. 2 for stereo, 6 for 5.1, etc)
	@param SampleRate		The sample rate of the audio buffer
	@param AudioClock		Double audio clock value, from start of audio rendering.
	*/
	virtual void OnNewSubmixBuffer(const USoundSubmix* OwningSubmix, float* AudioData, int32 NumSamples, int32 NumChannels, const int32 SampleRate, double AudioClock) = 0;

	/**
	 * Called if the submix is evaluating disabling itself for the next buffer of audio in FMixerSubmix::IsRenderingAudio()
	 * if this returns true, FMixerSubmix::IsRenderingAudio() will return true.
	 * Otherwise the submix will evaluate playing sounds, children submixes, etc to see if it should auto-disable
	 *
	 * This is called every evaluation.
	 * ISubmixListeners that intermittently render audio should only return true when they have work to do.
	 */
	virtual bool IsRenderingAudio() const
	{
		return false;
	}

	virtual const FString& GetListenerName() const
	{
		static FString UnsetName = "Unset";
		return UnsetName;
	}

protected:
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
};
