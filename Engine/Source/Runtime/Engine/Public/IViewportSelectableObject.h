// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IViewportSelectableObject.generated.h"

/**
 * Interface for objects selectable in the viewport.
 * Only used by Sequencer in editor at the moment to allow AControlRigShapeActor (runtime)
 * to be selectable when Sequencer is selection limiting.
 * Could be extended or removed in the future where there is a more generalized, low level selection system.
 **/
UINTERFACE(MinimalApi, meta=(CannotImplementInterfaceInBlueprint))
class UViewportSelectableObject : public UInterface
{
	GENERATED_BODY()
};

class IViewportSelectableObject
{
	GENERATED_BODY()

public:
	virtual bool IsSelectable() const { return true; }
};
