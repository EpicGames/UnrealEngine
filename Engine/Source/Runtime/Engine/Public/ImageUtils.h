// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ImageUtils.h: Image/Textures utility functions 
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ImageCore.h"
#include "Engine/Texture.h"

/*

NOTE: Please prefer to work with images through ImageCore FImage/FImageView
(not TextureSource/TextureSourceFormat or raw arrays of bytes)

Use FImageUtils::CompressImage/DecompressImage

The easiest way to load/save an image is FImageUtils::LoadImage/SaveImageByExtension

Typically let the user's filename choose the image format, don't hard-code them in code.

*/

class UTexture2D;
class UTexture2DArray;
class UTextureCube;
class UTextureCubeArray;
class UVolumeTexture;
class UTextureRenderTarget;
class UTextureRenderTarget2D;
class UTextureRenderTargetCube;
class IImageWrapperModule;

/**
 *	Parameters used for creating a Texture2D from a simple color buffer.
 * 
 */
struct FCreateTexture2DParameters
{
	/** True if alpha channel is used */
	bool						bUseAlpha;

	/** Compression settings to use for texture */
	TextureCompressionSettings	CompressionSettings;

	/** If texture should be compressed right away, or defer until package is saved */
	bool						bDeferCompression;

	/** If texture should be set as SRGB */
	bool						bSRGB;

	/** If texture should be a virtual texture */
	bool						bVirtualTexture;

	/** Mip-map generation settings */
	TextureMipGenSettings		MipGenSettings;

	/** Group this texture belongs to */
	TextureGroup				TextureGroup;

	/* The Guid hash to use part of the texture source's DDC key */
	FGuid						SourceGuidHash;

	FCreateTexture2DParameters()
		:	bUseAlpha(false),
			CompressionSettings(TC_Default),
			bDeferCompression(false),
			bSRGB(true),
			bVirtualTexture(false),
			MipGenSettings(TMGS_FromTextureGroup),
			TextureGroup(TEXTUREGROUP_MAX)
	{
	}
};

/**
 * Class of static image utility functions.
 * 
 * Provides load/save of FImage
 * and conversion from Texture2D/RenderTarget to/from FImage
 * 
 *  in Engine, cannot be used by standalone texture build workers
 */
class FImageUtils
{
public:

	/**
	 * Load an image of any type supported by the ImageWrapper module
	 *
	 * @param Filename				File name to load
	 * @param OutImage				Filled with the loaded image, allocated as needed
	 */
	ENGINE_API static bool LoadImage(const TCHAR * Filename, FImage & OutImage);
		
	/**
	 * Save an image.  Extension of Filename will be used for output image file format.
	 * FImageView can be made from any surface pointer.
	 *
	 * @param Filename				File name to save to, with extension to identify format
	 * @param InImage				Image to save
	 * @param Quality				Mainly for JPEG, but special values have some meaning for other formats
	 *
	 * If Filename has no extension, an appropriate one is automatically generated based on image format.
	 */
	ENGINE_API static bool SaveImageByExtension(const TCHAR * Filename, const FImageView & InImage, int32 Quality=0);

	/**
	 * Save an image.  Output format will be chosen automatically based on the Image pixel format
	 * eg. EXR for float, PNG for BGRA8
	 * extension will be added to file name
	 *
	 * @param Filename				File name to save to, auto extension will be added
	 * @param InImage				Image to save
	 * @param Quality				Mainly for JPEG, but special values have some meaning for other formats
	 */
	ENGINE_API static bool SaveImageAutoFormat(const TCHAR * Filename, const FImageView & InImage, int32 Quality=0);
	
	/**
	 * "Compress" an image to a file format.  Here "Compress" really means put an image into file format binary.
	 * Extension is used to identify file format, eg. png/jpeg/etc.
	 * FImageView can be made from any surface pointer.
	 *
	 * @param OutData				Filled with file format Image data
	 * @param ToFormatExtension		can be a full name like "xxx.png" or just the extension like "png"
	 *								ToFormatExtension == null is okay & means use default output format
	 * @param InImage				Image to save
	 * @param Quality				Mainly for JPEG, but special values have some meaning for other formats
	 */
	ENGINE_API static bool CompressImage(TArray64<uint8> & OutData, const TCHAR * ToFormatExtension, const FImageView & InImage, int32 Quality = 0);
	
	/**
	 * "Decompress" an image to a file format.  Here "Compress" really means put an image into file format binary.
	 *
	 * @param InCompressedData		File formatted image bits to unpack
	 * @param InCompressedSize		Size of InCompressedData
	 * @param OutImage				Filled with the decompressed image and allocated as needed
	 */
	ENGINE_API static bool DecompressImage(const void* InCompressedData, int64 InCompressedSize, FImage & OutImage);
	
	/**
	*  Export Texture (2D,Array,Cube,CubeArray,Volume) to DDS
	*  reads from texture source, not platform data
	* 
	* @param OutData    Filled with DDS file format
	* @param Texture    Texture (2D,Array,Cube,CubeArray,Volume) to export
	* @param BlockIndex Block to export (optional)
	* @param LayerIndex Layer to export (optional)
	* 
	*/
	ENGINE_API static bool ExportTextureSourceToDDS(TArray64<uint8> & OutData, UTexture * Texture, int BlockIndex=0, int LayerIndex=0);
	
	/**
	*  Export Texture RenderTarget (2D,Array,Cube,CubeArray,Volume) to DDS
	*  reads from texture rendertarget RHI texture
	* 
	* @param OutData    Filled with DDS file format
	* @param TexRT    Texture RenderTarget (2D,Array,Cube,CubeArray,Volume) to export
	* 
	*/
	ENGINE_API static bool ExportRenderTargetToDDS(TArray64<uint8> & OutData, UTextureRenderTarget * TexRT);

	/**
	 * if Texture source is available, get it as an FImage
	 *
	 * @param Texture		Texture to get gets from
	 * @param OutImage		Filled with the image copy and allocated as needed
	 */
	ENGINE_API static bool GetTexture2DSourceImage(	UTexture2D* Texture, FImage & OutImage);

	/**
	 * Resizes the given image using FImageCore::ResizeImage.
	 * Destination array will be emptied and resized.
	 *
	 * prefer FImageCore::ResizeImage instead.
	 * prefer using FImage or FImageView not FColor arrays.
	 *
	 * @param SrcWidth				Source image width.
	 * @param SrcHeight				Source image height.
	 * @param SrcData				Source image data.
	 * @param DstWidth				Destination image width.
	 * @param DstHeight				Destination image height.
	 * @param DstData				Destination image data. (will be resized)
	 * @param bResizeSRGBinLinearSpace			Convert sRGB to linear for gamma-correct resize.  Should be true if the Src is SRGB diffuse color; should be false for masks and normal maps.
	 * @param bForceOpaqueOutput	Always output 255 for the alpha channel.  BEWARE defaults to true so alpha is killed by default.
	 *
	 * BEWARE: the FColor variant of this function fills with opaque alpha by default.  The FLinearColor variant does not
	 */
	ENGINE_API static void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData,  int32 DstWidth, int32 DstHeight, TArray<FColor> &DstData, bool bResizeSRGBinLinearSpace, bool bForceOpaqueOutput=true);

	/**
	 * Resizes the given image using FImageCore::ResizeImage.
	 * Accepts TArrayViews but requires that DstData be pre-sized appropriately
	 *
	 * prefer FImageCore::ResizeImage instead.
	 * prefer using FImage or FImageView not FColor arrays.
	 *
	 * @param SrcWidth				Source image width.
	 * @param SrcHeight				Source image height.
	 * @param SrcData				Source image data.
	 * @param DstWidth				Destination image width.
	 * @param DstHeight				Destination image height.
	 * @param DstData				Destination image data. (must already be sized to DstWidth*DstHeight)
 	 * @param bResizeSRGBinLinearSpace			Convert sRGB to linear for gamma-correct resize.  Should be true if the Src is SRGB diffuse color; should be false for masks and normal maps.
	 * @param bForceOpaqueOutput	Always output 255 for the alpha channel.  BEWARE defaults to true so alpha is killed by default.
	 *
	 * BEWARE: the FColor variant of this function fills with opaque alpha by default.  The FLinearColor variant does not
	 */
	ENGINE_API static void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArrayView<const FColor> &SrcData, int32 DstWidth, int32 DstHeight, const TArrayView<FColor> &DstData, bool bResizeSRGBinLinearSpace, bool bForceOpaqueOutput=true);

	/**
	 * Resizes the given image using FImageCore::ResizeImage.
	 * Destination array will be emptied and resized.
	 *
	 * prefer FImageCore::ResizeImage instead.
	 * prefer using FImage or FImageView not FColor arrays.
	 *
	 * @param SrcWidth	Source image width.
	 * @param SrcHeight	Source image height.
	 * @param SrcData	Source image data.
	 * @param DstWidth	Destination image width.
	 * @param DstHeight Destination image height.
	 * @param DstData	Destination image data. (will be resized)
	 *
	 * BEWARE: the FColor variant of this function fills with opaque alpha by default.  The FLinearColor variant does not
	 */
	ENGINE_API static void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArray64<FLinearColor>& SrcData, int32 DstWidth, int32 DstHeight, TArray64<FLinearColor>& DstData);

	/**
	 * Resizes the given image using FImageCore::ResizeImage.
	 * Accepts TArrayViews but requires that DstData be pre-sized appropriately
	 *
	 * prefer FImageCore::ResizeImage instead.
	 * prefer using FImage or FImageView not FColor arrays.
	 *
	 * @param SrcWidth	Source image width.
	 * @param SrcHeight	Source image height.
	 * @param SrcData	Source image data.
	 * @param DstWidth	Destination image width.
	 * @param DstHeight Destination image height.
	 * @param DstData	Destination image data. (must already be sized to DstWidth*DstHeight)
	 *
	 * BEWARE: the FColor variant of this function fills with opaque alpha by default.  The FLinearColor variant does not
	 */
	ENGINE_API static void ImageResize(int32 SrcWidth, int32 SrcHeight, const TArrayView64<const FLinearColor>& SrcData, int32 DstWidth, int32 DstHeight, const TArrayView64<FLinearColor>& DstData);

	/**
	 * Creates a 2D texture from a array of raw color data.
	 *
	 * @param SrcWidth		Source image width.
	 * @param SrcHeight		Source image height.
	 * @param SrcData		Source image data.
	 * @param Outer			Outer for the texture object.
	 * @param Name			Name for the texture object.
	 * @param Flags			Object flags for the texture object.
	 * @param InParams		Params about how to set up the texture.
	 * @return				Returns a pointer to the constructed 2D texture object.
	 *
	 * this fills a TextureSource , and will then Build a Platform texture from that
	 * can be used WITH_EDITOR only
	 * contrast to CreateTexture2DFromImage
	 *
	 * Prefer the more modern CreateTexture from FImage.
	 */
	ENGINE_API static UTexture2D* CreateTexture2D(int32 SrcWidth, int32 SrcHeight, const TArray<FColor> &SrcData, UObject* Outer, const FString& Name, const EObjectFlags &Flags, const FCreateTexture2DParameters& InParams);
	
	/**
	 * Creates a texture of any type from an Image
	 * This is the modern preferred way to create a texture.
	 *
	 * If you need to change the default settings, then use DoPostEditChange = false, and call PostEditChange() yourself after setting all properties.
	 * Typically you may want to set LODGroup and CompressionSettings.
	 *
	 * this fills the TextureSource , and will then Build a Platform texture from that
	 * can be used WITH_EDITOR only
	 *
	 */
	ENGINE_API static UTexture * CreateTexture(ETextureClass TextureClass, const FImageView & Image, UObject* Outer, const FString& Name, EObjectFlags Flags = RF_NoFlags, bool DoPostEditChange = true );
	
	/**
	 * Creates a 2D texture from an FImage
	 * 
	 * @param Image			Image that will be copied into a Texture
	 * @return				Returns a pointer to the constructed 2D texture object.
	 *
	 * NOTE: this makes a Transient texture with the Image in the PlatformData
	 * this is different than making a Texture via TextureSource.Init(Image)
	 */	
	ENGINE_API static UTexture2D* CreateTexture2DFromImage(const FImageView & Image);

	/**
	 * Crops, and scales an image from a raw image array.
	 *
	 * @param SrcWidth			Source image width.
	 * @param SrcHeight			Source image height.
	 * @param DesiredWidth		Desired Width.
	 * @param DesiredHeight		Desired Height.
	 * @param SrcData			Raw image array.
	 * @param DstData			compressed image array.
	 *
	 */
	ENGINE_API static void CropAndScaleImage( int32 SrcWidth, int32 SrcHeight, int32 DesiredWidth, int32 DesiredHeight, const TArray<FColor> &SrcData, TArray<FColor> &DstData  );

	/**
	 * Compress image to thumbnail enabled format (png or jpg) uint8 array.
	 *
	 * @param ImageHeight		Source image width.
	 * @param ImageWidth		Source image height.
	 * @param SrcData			Raw image array.
	 * @param DstData			compressed image array.
	 *
	 */
	ENGINE_API static void ThumbnailCompressImageArray( int32 ImageWidth, int32 ImageHeight, const TArray<FColor> &SrcData, TArray<uint8> &DstData );

	UE_DEPRECATED(5.1, "Please use PNGCompressImageArray or ThumbnailCompressImageArray")
	static void CompressImageArray( int32 ImageWidth, int32 ImageHeight, const TArray<FColor> &SrcData, TArray<uint8> &DstData )
	{
		ThumbnailCompressImageArray(ImageWidth,ImageHeight,SrcData,DstData);
	}

	/**
	 * Compress image to PNG format uint8 array.
	 * deprecated, use CompressImage instead
	 *
	 * @param ImageHeight		Source image width.
	 * @param ImageWidth		Source image height.
	 * @param SrcData			Raw image array.
	 * @param DstData			compressed image array.
	 *
	 */
	ENGINE_API static void PNGCompressImageArray(int32 ImageWidth, int32 ImageHeight, const TArrayView64<const FColor>& SrcData, TArray64<uint8>& DstData);

	/**
	 * Creates a new UTexture2D with a checkerboard pattern.
	 *
	 * @param ColorOne		The color of half of the squares.
	 * @param ColorTwo		The color of the other half of the squares.
	 * @param CheckerSize	The size in pixels of each side of the texture.
	 *
	 */
	ENGINE_API static UTexture2D* CreateCheckerboardTexture(FColor ColorOne = FColor(64, 64, 64), FColor ColorTwo = FColor(128, 128, 128), int32 CheckerSize = 32);

	/**
	 * Creates a new UTexture2DArray with a checkerboard pattern.
	 *
	 * @param ColorOne		The color of half of the squares.
	 * @param ColorTwo		The color of the other half of the squares.
	 * @param CheckerSize	The size in pixels of each side of the texture.
	 * @param ArraySize		The number of elements in the array.
	 *
	 */
	ENGINE_API static UTexture2DArray* CreateCheckerboardTexture2DArray(FColor ColorOne = FColor(64, 64, 64), FColor ColorTwo = FColor(128, 128, 128), int32 CheckerSize = 32, int32 ArraySize = 1);

	/**
	 * Creates a new UTextureCube with a checkerboard pattern.
	 *
	 * @param ColorOne		The color of half of the squares.
	 * @param ColorTwo		The color of the other half of the squares.
	 * @param CheckerSize	The size in pixels of each side of the texture.
	 *
	 */
	ENGINE_API static UTextureCube* CreateCheckerboardCubeTexture(FColor ColorOne = FColor(64, 64, 64), FColor ColorTwo = FColor(128, 128, 128), int32 CheckerSize = 32);

	/**
	 * Creates a new UTextureCubeArray with a checkerboard pattern.
	 *
	 * @param ColorOne		The color of half of the squares.
	 * @param ColorTwo		The color of the other half of the squares.
	 * @param CheckerSize	The size in pixels of each side of the texture.
	 * @param ArraySize		The number of elements in the array.
	 *
	 */
	ENGINE_API static UTextureCubeArray* CreateCheckerboardTextureCubeArray(FColor ColorOne = FColor(64, 64, 64), FColor ColorTwo = FColor(128, 128, 128), int32 CheckerSize = 32, int32 ArraySize = 1);

	/**
	 * Creates a new UVolumeTexture with a checkerboard pattern.
	 *
	 * @param ColorOne		The color of half of the squares.
	 * @param ColorTwo		The color of the other half of the squares.
	 * @param CheckerSize	The size in pixels of each side of the texture.
	 *
	 */
	ENGINE_API static UVolumeTexture* CreateCheckerboardVolumeTexture(FColor ColorOne = FColor(64, 64, 64), FColor ColorTwo = FColor(128, 128, 128), int32 CheckerSize = 16);

	/**
	 * Exports a UTextureRenderTarget2D as an HDR image on the disk.
	 * HDR is very low quality, prefer EXR or PNG instead
	 * Deprecated.  Prefer GetRenderTargetImage.
	 *
	 * @param TexRT		The render target to export
	 * @param Ar			Archive to fill with data.
	 * @return			Export operation success or failure.
	 *
	 */
	ENGINE_API static bool ExportRenderTarget2DAsHDR(UTextureRenderTarget2D* TexRT, FArchive& Ar);
	
	/**
	 * Exports a UTextureRenderTarget2D as a EXR image on the disk.
	 * Deprecated.  Prefer GetRenderTargetImage.
	 *
	 * @param TexRT		The render target to export
	 * @param Ar		Archive to fill with data.
	 * @return			Export operation success or failure.
	 *
	 */
	ENGINE_API static bool ExportRenderTarget2DAsEXR(UTextureRenderTarget2D* TexRT, FArchive& Ar);

	/**
	 * Exports a UTextureRenderTarget2D as a PNG image on the disk.
	 * Deprecated.  Prefer GetRenderTargetImage.
	 *
	 * @param TexRT		The render target to export
	 * @param Ar		Archive to fill with data.
	 * @return			Export operation success or failure.
	 *
	 */
	ENGINE_API static bool ExportRenderTarget2DAsPNG(UTextureRenderTarget2D* TexRT, FArchive& Ar);

	/**
	* Exports a UTexture2D as an HDR image on the disk.
	* Deprecated.  Prefer GetTexture2DSourceImage + SaveImage. 
	* HDR is very low quality, prefer EXR or PNG instead
	*
	* @param TexRT		The texture to export
	* @param Ar			Archive to fill with data.
	* @return			Export operation success or failure.
	*
	*/
	ENGINE_API static bool ExportTexture2DAsHDR(UTexture2D* TexRT, FArchive& Ar);

	/**
	 * Imports a texture file from disk and creates Texture2D from it
	 *
	 * note this make a Transient / PlatformData only Texture (no TextureSource)
	 *
	 */
	ENGINE_API static UTexture2D* ImportFileAsTexture2D(const FString& Filename);

	/**
	 * Imports a texture a buffer and creates Texture2D from it
	 *
	 * note this make a Transient / PlatformData only Texture (no TextureSource)
	 *
	 */
	ENGINE_API static UTexture2D* ImportBufferAsTexture2D(TArrayView64<const uint8> Buffer);
	ENGINE_API static UTexture2D* ImportBufferAsTexture2D(const TArray<uint8>& Buffer);
	
	/**
	* Exports a UTextureRenderTargetCube as an HDR image on the disk.
	* this does GenerateLongLatUnwrap
	* 
	* @param TexRT		The render target cube to export
	* @param Ar			Archive to fill with data.
	* @return			Export operation success or failure.
	*
	*/
	ENGINE_API static bool ExportRenderTargetCubeAsHDR(UTextureRenderTargetCube* TexRT, FArchive& Ar);

	/**
	* Exports a UTextureCube as an HDR image on the disk.
	* this does GenerateLongLatUnwrap
	* 
	* @param TexRT		The texture cube to export
	* @param Ar			Archive to fill with data.
	* @return			Export operation success or failure.
	*
	*/
	ENGINE_API static bool ExportTextureCubeAsHDR(UTextureCube* TexRT, FArchive& Ar);

	UE_DEPRECATED(5.5, "Use GetRenderTargetImage")
	ENGINE_API static bool GetRawData(UTextureRenderTarget2D* TexRT, TArray64<uint8>& RawData);
	
	/**
	* Get the contents of a RenderTarget into an Image
	* 
	* @param TexRT		The texture rendertarget to copy from
	* @param OutImage	Filled with the image, allocated as needed
	*
	* Works for Cubes, Volumes, etc.  (fills Image slices)
	*/
	ENGINE_API static bool GetRenderTargetImage(UTextureRenderTarget* TexRT, FImage & OutImage);
	ENGINE_API static bool GetRenderTargetImage(UTextureRenderTarget* TexRT, FImage & OutImage, const FIntRect & Rect);

};
