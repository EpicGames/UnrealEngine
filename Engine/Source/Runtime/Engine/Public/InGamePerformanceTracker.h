// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"

struct FInGameCycleHistory
{
	FInGameCycleHistory()
	{
		Reset();
	}

	FInGameCycleHistory(int32 InNumSamples)
	{
		FrameCycles.SetNumZeroed(InNumSamples);
		Reset();
	}

	void Reset()
	{
		CurrFrameCycles = 0;
		int32 NumSamples = FrameCycles.Num();
		FrameCycles.Reset(NumSamples);
		FrameCycles.SetNumZeroed(NumSamples);
		TotalCycles = 0;
		CachedAverageCycles = 0;
		FrameIdx = 0;
	}

	/** Current number of cycles for this frame. */
	TAtomic<uint64> CurrFrameCycles;
	//Cycles for each frame in history.
	TArray<uint64> FrameCycles;

	//Cached running total for an overall average.
	uint64 TotalCycles = 0;
	/** Current average cycles for access by others, possibly on other threads. */
	TAtomic<uint64> CachedAverageCycles;

	//Index of next frame in history to replace.
	int32 FrameIdx = 0;

	/** Adds a number of cycles for the current frame. Thread safe. */
	inline void AddCycles(uint64 NewCycles)
	{
		CurrFrameCycles += NewCycles;
	}
	
	inline uint64 GetAverageCycles()const
	{
		return CachedAverageCycles.Load();
	}

	/** Places the current data on the history and advances to the next frame index. Not thread safe.*/
	ENGINE_API void NextFrame();
};

/**
Helper class to track code timings.
Similar to stats but much more minimal as they're intended to be used at InGame. Though it's still not free so use wisely.
*/
class FInGamePerformanceTracker
{
private:

	/** An N frame history of cycle counts. */
	FInGameCycleHistory History;

	/** How many times we've entered a directly tracked section of code. Used to time the first and last entry/exit points. */
	uint32 DirectSectionTime_EntryCount;
	/** Initial cycle reading when timing a section of code directly. */
	uint32 DirectSectionTime_BeginCycles;
	
public:

	ENGINE_API FInGamePerformanceTracker();
	ENGINE_API FInGamePerformanceTracker(uint32 FrameHistorySize);

	ENGINE_API void Tick();

	/** Adds cycles collected from an external helper object. */
	inline void AddCycles(uint32 Cycles)
	{
		return History.AddCycles(Cycles);
	}

	inline double GetAverageTimeSeconds() const
	{
		return FPlatformTime::ToSeconds64(History.GetAverageCycles());
	}
	
	/** Enters a timed section of code. Can be re-entrant and will take the first entry point time. Only callable from game thread. */
	ENGINE_API void EnterTimedSection();

	/** Enters a timed section of code. Can be re-entrant and will take the final exit point time. Only callable from game thread. */
	ENGINE_API void ExitTimedSection();
	
	static ENGINE_API class IConsoleVariable* Enabled;
	static ENGINE_API int32 CachedEnabled;
	static ENGINE_API class IConsoleVariable* HistorySize;
};

class FInGameCycleCounter
{
private:
	FInGamePerformanceTracker* Tracker;
	uint32 BeginCycles;

public:
	FInGameCycleCounter(FInGamePerformanceTracker* InTracker)
	: Tracker(InTracker)
	, BeginCycles(0)
	{
	}

	inline void Begin()
	{
		if (Tracker && FInGamePerformanceTracker::CachedEnabled)
		{
			if (IsInGameThread())
			{
				Tracker->EnterTimedSection();//On the GT we use the tracker directly so we can allow re-entrance.
			}
			else
			{
				//On other threads we don't allow (or check for) re-entrance and store our own cycle data.
				BeginCycles = FPlatformTime::Cycles();
			}
		}
	}
	inline void End()
	{
		if (Tracker && FInGamePerformanceTracker::CachedEnabled)
		{
			if (IsInGameThread())
			{
				Tracker->ExitTimedSection();//On the GT we use the tracker directly so we can allow re-entrance.
			}
			else
			{
				//On other threads we don't allow (or check for) re-entrance and store our own cycle data.
				uint32 Cycles = FPlatformTime::Cycles() - BeginCycles;
				Tracker->AddCycles(Cycles);
				BeginCycles = 0;
			}
		}
	}
};

// In game performance trackers. 
// Not the most extensible system so should likely be improved in the future.
enum class EInGamePerfTrackers : uint8
{
	VFXSignificance,
	//Others?
	Num,
};

enum class EInGamePerfTrackerThreads : uint8
{
	GameThread,
	RenderThread,
	OtherThread,
	Num,
};

class FInGameScopedCycleCounter : public FInGameCycleCounter
{
public:
	FInGameScopedCycleCounter(class UWorld* InWorld, EInGamePerfTrackers Tracker, EInGamePerfTrackerThreads TrackerThread, bool bEnabled);
	~FInGameScopedCycleCounter();
};


/** Collection of in game performance trackers for a world. */
class FWorldInGamePerformanceTrackers
{
	FInGamePerformanceTracker InGamePerformanceTrackers[(int32)EInGamePerfTrackers::Num][(int32)EInGamePerfTrackerThreads::Num];

public:

	FWorldInGamePerformanceTrackers();
	~FWorldInGamePerformanceTrackers();

	FInGamePerformanceTracker& GetInGamePerformanceTracker(EInGamePerfTrackers InTracker, EInGamePerfTrackerThreads InThread)
	{
		//UE-38057 - Additional checks to catch bug.
		int32 Tracker = (int32)InTracker;
		int32 Thread = (int32)InThread;
		check(Tracker >= 0 && Tracker < (int32)EInGamePerfTrackers::Num);
		check(Thread >= 0 && Thread < (int32)EInGamePerfTrackerThreads::Num);
		return InGamePerformanceTrackers[Tracker][Thread];
	}
};
