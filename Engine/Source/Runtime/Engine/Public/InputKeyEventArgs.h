// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "InputCoreTypes.h"
#include "Math/Vector2D.h"

class FViewport;

/**
 * Input key event arguments. This is data that is required for the viewport client to
 * process input via its InputKey/InputAxis functions.
 *
 * It represents a single "input event" which has happened which we want game or editor code
 * to be able to process. This may accumulate several "raw" input events from the message
 * handler into one with multiple sample readings, or it could be just one single key press event.
 *
 * This data can represent gamepad, keyboard, mouse, or touch data. 
 */
struct FInputKeyEventArgs
{
public:

	FInputKeyEventArgs() = default;
	
	/**
	 * Construct a FInputKeyEventArgs based on data acquired from the Slate input events.
	 * These event args are used to translate from Slate input event arguments into a
	 * standardized form for the ViewportClient and then the rest of the gameplay framework.
	 * 
	 * @param InViewport		The viewport which the axis movement is from.
	 * @param InInputDevice		The input device that triggered this axis movement
	 * @param InKey				The name of the axis which moved.
	 * @param InEvent			The key event that these args represent.
	 * @param InAmountDepressed	For axis events, the amount that the key is depressed
	 * @param bInIsTouchEvent	True if this event represents a touch input
	 * @param InEventTimestamp	The timestamp of when this input event was originally polled in the form of CPU cycles.
	 */
	ENGINE_API FInputKeyEventArgs(
		FViewport* InViewport,
		const FInputDeviceId InInputDevice,
		const FKey& InKey,
		const EInputEvent InEvent,
		const float InAmountDepressed,
		const bool bInIsTouchEvent,
		const uint64 InEventTimestamp);

	/**
	 * Construct a FInputKeyEventArgs based on data acquired from the Slate input events.
	 * These event args are used to translate from Slate input event arguments into a
	 * standardized form for the ViewportClient and then the rest of the gameplay framework.
	 * 
	 * @param InViewport		The viewport which the axis movement is from.
	 * @param InInputDevice		The input device that triggered this axis movement
	 * @param InKey				The name of the axis which moved.
	 * @param InEvent			The key event that these args represent.
	 * @param InEventTimestamp	The timestamp of when this input event was originally polled in the form of CPU cycles.
	 */
	FInputKeyEventArgs(
		FViewport* InViewport,
		const FInputDeviceId InInputDevice,
		const FKey& InKey,
		const EInputEvent InEvent,
		const uint64 InEventTimestamp)
		: FInputKeyEventArgs(
		 InViewport,
		 InInputDevice, 
		 InKey, 
		 InEvent,
		 /*AmountDepressed*/1.0f,
		 /*bIsTouch*/ false,
		 InEventTimestamp)
	{
	}

	/**
	 * Construct a FInputKeyEventArgs based on the old params of the ViewportClient InputAxis function. 
	 * 
	 * @param InViewport		The viewport which the axis movement is from.
	 * @param InInputDevice		The input device that triggered this axis movement
	 * @param InKey				The name of the axis which moved.
	 * @param InDelta			The axis movement delta.
	 * @param InDeltaTime		The time since the last axis update.
	 * @param InNumSamples		The number of device samples that contributed to this Delta, useful for things like smoothing
	 * @param InEventTimestamp	The timestamp of when this input event was originally polled in the form of CPU cycles.
	 */
	FInputKeyEventArgs(
		FViewport* InViewport,
		const FInputDeviceId InInputDevice,
		const FKey& InKey,
		const float InDelta,
		const float InDeltaTime,
		const int32 InNumSamples,
		const uint64 InEventTimestamp)
		: FInputKeyEventArgs(
			InViewport,
			InInputDevice,
			InKey,
			IE_Axis,
			/*AmountDepressed*/InDelta,
			/*bIsTouchEvent*/ false,
			InEventTimestamp)
	{
		DeltaTime = InDeltaTime;
		NumSamples = InNumSamples;
	}

	/**
	 * Construct a FInputKeyEventArgs for a 2D gesture input event.
	 *
	 * @param InInputDevice		The input device that triggered this event
	 * @param InKey				The gesture key (e.g. EKeys::Gesture_Pan)
	 * @param InEvent			The input event type (IE_Pressed, IE_Repeat, IE_Released)
	 * @param InAmountDepressed2D	The 2D gesture delta value
	 * @param InEventTimestamp	The timestamp of when this input event was originally polled
	 */
	FInputKeyEventArgs(
		const FInputDeviceId InInputDevice,
		const FKey& InKey,
		const EInputEvent InEvent,
		const FVector2D& InAmountDepressed2D,
		const uint64 InEventTimestamp)
		: FInputKeyEventArgs(
			/*Viewport*/ nullptr,
			InInputDevice,
			InKey,
			InEvent,
			/*AmountDepressed*/ 0.0f,
			/*bIsTouchEvent*/ false,
			InEventTimestamp)
	{
		AmountDepressed2D = InAmountDepressed2D;
	}

	static ENGINE_API FInputKeyEventArgs CreateSimulated(
		const FKey& InKey,
		const EInputEvent InEvent,
		const float AmountDepressed,
		const int32 InNumSamplesOverride = -1,
		const FInputDeviceId InputDevice = INPUTDEVICEID_NONE,
		const bool bIsTouchEvent = false,
		FViewport* Viewport = nullptr
	);
	
	/**
	 * @return True if this input event is for a gamepad key, or should be treated as such.
	 * 
	 * This may be true for non-gamepad FKeys if the input event is simulated input.
	 */
	bool IsGamepad() const { return Key.IsGamepadKey(); }

	bool IsSimulatedInput() const { return bIsSimulatedInput; }

	/**
	 * @return The platform user which this input event originated from based on its input device.
	 */
	ENGINE_API FPlatformUserId GetPlatformUser() const;

public:
	
	/**
	 * The viewport from which this key event originated from.
	 *
	 * This viewport may be null if this input event is from a simulated source,
	 * such as a unit test or a widget simulating player input.
	 */
	FViewport* Viewport = nullptr;

	// The controller which the key event is from.
	int32 ControllerId;
	
	/**
	 * The input device which this event originated from.
	 */
	FInputDeviceId InputDevice = INPUTDEVICEID_NONE;

	/**
	 * The Key that this input event is for
	 */
	FKey Key = {};
	
	/**
	 * The type of event which occurred.
	 */
	EInputEvent Event = IE_MAX;
	
	/**
	 * The value that this input event represents.
	 *
	 * For analog keys, the depression percent.
	 */
	float AmountDepressed;

	/**
	 * 2D value for this input event (e.g. gesture pan delta).
	 * When set, this takes precedence over AmountDepressed for 2D input.
	 */
	FVector2D AmountDepressed2D = FVector2D::ZeroVector;

	/** The time between the previous frame and the current one */
	float DeltaTime = 1.0f / 60.f;

	/**
	 * For analog key events. The number of analog input samples
	 * which are contained in this event's InputValue.
	 */
	int32 NumSamples = 1;
	
	/**
	 * True if this input event originated from a touch surface.
	 * 
	 * Note: This may be set to true for simulated touch inputs from things like a mouse button.
	 */
	bool bIsTouchEvent = false;

private:

	/**
	 * True if this input event is NOT sourced from a physical Human Interface Device (controller, keyboard, mouse, etc.)
	 * and is instead sourced from code, such as faking input events or simulating input for touch screens.
	 */
	bool bIsSimulatedInput = false;
	
public:
	
	/**
	 * The timestamp of when this input event was originally polled.
	 * 
	 * This data should be set to be as representative as possible of the time of when the
	 * input event originated, most of the time from the raw Slate input event (FInputEvent)
	 * on the message handler.
	 *
	 * This timestamp is in terms of CPU Cycles, gathered from the platform's high resolution clock
	 * via the FPlatformTime::Cycles64() function.
	 *
	 * A timestamp value of 0u means that this event was not initialized with a valid timestamp
	 */
	uint64 EventTimestamp = 0u;
};
