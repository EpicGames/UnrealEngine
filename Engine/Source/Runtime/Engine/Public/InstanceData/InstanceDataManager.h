// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneTypes.h"
#include "InstanceDataSceneProxy.h"
#include "InstancedStaticMesh/InstanceAttributeTracker.h"

class FPrimitiveSceneProxy;
class HHitProxy;
class FPrecomputedInstanceSpatialHashData;
class FInstanceUpdateChangeSet;
using FPrecomputedInstanceSpatialHashDataPtr = TSharedPtr<const FPrecomputedInstanceSpatialHashData, ESPMode::ThreadSafe>;

/**
 * Data descriptor representing the component state abstracting the UPrimitiveComponent, needs to be passed into the change flushing.
 * The intention is to decouple the manager from the UComponent or any other supplier of instance data & scene proxies.
 */
struct FInstanceDataManagerSourceDataDesc
{
	FMatrix PrimitiveLocalToWorld;
	EComponentMobility::Type ComponentMobility = EComponentMobility::Movable;
	FRenderBounds MeshBounds;
	FInstanceDataFlags Flags;

	FPrimitiveMaterialPropertyDescriptor PrimitiveMaterialDesc;

	// Number of instances in the source arrays (e.g., in the component)
	int32 NumInstances = -1;

	// 
	int32 NumCustomDataFloats = 0;

	// Callback to fill in the required change set with source data.
	TFunction<void(FInstanceUpdateChangeSet& ChangeSet)> BuildChangeSet;
};

class InstanceDataManagerSourceInterface
{
public:

	enum class FDirtyType
	{
		Incremental,
		Full,
	};

	/**
	 * Called to let the source (UComponent or whatever) know there has been a change that will needs to be flushed.
	 */
	virtual void InstanceDataManagerMarkDirty(FDirtyType DirtyFlags) = 0;

	/**
	 * Called to retrieve the data for 
	 */
	virtual FInstanceDataManagerSourceDataDesc GetInstanceDataSourceDesc() = 0;
};

/**
 * Manager class that tracks changes to instance data within the component, and is responsible for dispatching updates of the proxy.
 * Tracks instance index changes to be able to maintain a persistent ID mapping for use on the render thread.
 * The ID mapping is not serialized and will be reset when the proxy is recreated.
 * Not responsible for storing the component representation of the instance data.
 * NOTE/TODO: This is tied to the ISM use-case, mostly because of legacy (HISM) interactions. Will be refactored and sub-classed or something.
 *            Also: Still somewhat tied to the UComponent, which also can be refactored a bit to make it more general.
 */
class FInstanceDataManager : public FInstanceIdIndexMap
{
public:
	//ENGINE_API FInstanceDataManager(IInstanceDataProvider* InInstanceDataProvider);
	ENGINE_API FInstanceDataManager(UPrimitiveComponent* InPrimitiveComponent);

	/**
	 * Current tracking state, 
	 */
	enum class ETrackingState : uint8
	{
		// In the initial state, there is no proxy and therefore changes do not need to be tracked, e.g., during initial setup of an ISM component.
		Initial, 
		// Tracking changes to send on next flush
		Tracked,
		// Prevent any changes from being tracked (e.g., if we have no renderer)
		Disabled,
		// In the optimized state there's no need to track any delta changes, but if anything changes at all we must rebuild.
		Optimized, 
	};

	/**
	 * Tracking functions that mirror what is done to each instance in the source instance data array.
	 */

	FPrimitiveInstanceId Add(int32 InInstanceAddAtIndex);
	void RemoveAtSwap(int32 InstanceIndex);
	void RemoveAt(int32 InstanceIndex);

	void TransformChanged(int32 InstanceIndex);
	void TransformChanged(FPrimitiveInstanceId InstanceId);
	void TransformsChangedAll();

	void CustomDataChanged(int32 InstanceIndex);

	void SkinningDataChanged(int32 InstanceIndex);

	void BakedLightingDataChanged(int32 InstanceIndex);
	void BakedLightingDataChangedAll();

	void NumCustomDataChanged();

#if WITH_EDITOR
	void EditorDataChangedAll();
#endif

	void PrimitiveTransformChanged();

	void ClearInstances();

	ENGINE_API bool HasAnyInstanceChanges() const;

	/** 
	 * Returns true if there are explicitly tracked instance changes, or the state is not tracked (because no proxy has been created yet),
	 * and the tracking state is not Disabled.
	 */
	inline bool HasAnyChanges() const { return GetState() != ETrackingState::Disabled && (GetState() != ETrackingState::Tracked || HasAnyInstanceChanges());}

	/**
	 * Queries the tracker for changes and builds an update build command to enqueue to the render thread. 
	 * The ComponentData supplies source data through a callback as needed.
	 */
	ENGINE_API bool FlushChanges(FInstanceDataManagerSourceDataDesc&& ComponentData);

	/**
	 * Clear all tracked changes (will result in a full update when next one is flushed)
	 */
	void ClearChangeTracking();

	int32 GetMaxAllocatedInstanceId() const;

	ETrackingState GetState() const { return TrackingState; }

	ENGINE_API TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> GetOrCreateProxy();
	ENGINE_API TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> GetProxy();

#if DO_GUARD_SLOW
	void ValidateMapping() const;
#else
	inline void ValidateMapping() const {};
#endif

#if WITH_EDITOR
	void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform, FInstanceDataManagerSourceDataDesc&& ComponentData);
	void WriteCookedRenderData(FArchive& Ar, FInstanceDataManagerSourceDataDesc&& ComponentData);
#endif
	void ReadCookedRenderData(FArchive& Ar);

	/**
	 */
	void Serialize(FArchive& Ar, bool bCooked);

	/**
	 * Compute the instance order used to build the instance hierarchy (spatial hashes) and return the reordering needed for this.
	 * This should be used by the data source (i.e., UComponent etc) to reorder the source instance buffers.
	 * Returns a reordering table where each index stores the old index for the given new index.
	 */
	TArray<int32> Optimize(FInstanceDataManagerSourceDataDesc&& ComponentData, bool bShouldRetainIdMap);

	/**
	 */
	ENGINE_API SIZE_T GetAllocatedSize() const;

protected:
	static bool ShouldUsePrecomputed();

	void CreateExplicitIdentityMapping();

	using EChangeFlag = FInstanceAttributeTracker::EFlag;

	template<EChangeFlag Flag>
	inline void MarkChangeHelper(int32 InstanceIndex);
	template<EChangeFlag Flag>
	inline void MarkChangeHelper(FPrimitiveInstanceId InstanceId);

	void MarkComponentRenderInstancesDirty();

	bool HasIdentityMapping() const;

	void FreeInstanceId(FPrimitiveInstanceId InstanceId);

	/** */
	void GatherDefaultData(const FInstanceDataManagerSourceDataDesc& ComponentData, FInstanceUpdateChangeSet& ChangeSet) const;

	/**
	 * Initialize a chage set from the component data & manager state but not using any delta information or updating tracked state.
	 */
	void InitChangeSet(const FInstanceDataManagerSourceDataDesc &ComponentData, FInstanceUpdateChangeSet& ChangeSet) const;

	void InitChangeSet(const struct FChangeDesc2 &ChangeDesc, const FInstanceDataManagerSourceDataDesc& ComponentData, FInstanceUpdateChangeSet& ChangeSet);

#if WITH_EDITOR
	// TODO: implement
	bool ShouldWriteCookedData(const ITargetPlatform* TargetPlatform, int32 NumInstancesToBuildFor);

	/**
	 * Build precomputed data from the ComponentData.
	 */
	FPrecomputedInstanceSpatialHashData PrecomputeOptimizationData(FInstanceDataManagerSourceDataDesc&& ComponentData);

	/**
	 * Build precomputed data from a ChangeSet.
	 */
	static FPrecomputedInstanceSpatialHashData PrecomputeOptimizationData(FInstanceUpdateChangeSet& ChangeSet);

#endif
	TSharedPtr<class FUpdatableInstanceDataSceneProxy, ESPMode::ThreadSafe> GetOrCreateProxyInternal();

	// Change set.
	FInstanceAttributeTracker InstanceUpdateTracker;

	// Id allocation tracking
	TBitArray<> ValidInstanceIdMask;
	int32 IdSearchStartIndex = 0;

	ETrackingState TrackingState = ETrackingState::Initial;
	FInstanceDataFlags AllChangedFlags;
	uint8 bNumCustomDataChanged : 1;
	uint8 bTransformChangedAllInstances : 1;
	uint8 bPrimitiveTransformChanged : 1;
	uint8 bAnyInstanceChange : 1;

	TSharedPtr<class FUpdatableInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataProxy;
	TWeakObjectPtr<UPrimitiveComponent> PrimitiveComponent = nullptr;

	FPrecomputedInstanceSpatialHashDataPtr PrecomputedOptimizationData;

	// Must track this to detect changes 
	FInstanceDataFlags Flags;
	int32 NumCustomDataFloats = 0;
	float AbsMaxDisplacement = 0.0f;
	FRenderBounds MeshBounds;
};