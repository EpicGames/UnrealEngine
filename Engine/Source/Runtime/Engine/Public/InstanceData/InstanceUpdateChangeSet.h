// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceDataSceneProxy.h"
#include "Containers/StridedView.h"
#include "InstancedStaticMesh/InstanceAttributeTracker.h"

class HHitProxy;
class FOpaqueHitProxyContainer;

/**
 * Helper to make it possible to use the same paths for gather / scatter even if the per-instance delta is not tracked.
 */
class FIdentityDeltaRange
{
public:
	FIdentityDeltaRange(int32 InNum) : NumItems(InNum) {}

	/**
		*/
	inline bool IsEmpty() const { return NumItems == 0; }

	/**
		*/
	inline bool IsDelta() const { return false; }

	/**
		* Returns the number of items in this range - i.e., the number of items that need to be copied to collect an update.
		*/
	inline int32 GetNumItems() const { return NumItems; }

	/**
		*/
	struct FConstIterator
	{
		int32 ItemIndex = 0;
		int32 MaxNum = 0;

		FConstIterator(int32 InIndex, int32 InMaxNum)
		:	ItemIndex(InIndex),
			MaxNum(InMaxNum)
		{
		}

		void operator++() {  ++ItemIndex; }

		/**
			* Get the index of the data in the source / destination arrays. 
			*/
		int32 GetIndex() const { return ItemIndex; }

		/**
			* Get the continuous index of the data item in the collected item array.
			*/
		int32 GetItemIndex() const { return ItemIndex; }

		explicit operator bool() const {  return ItemIndex < MaxNum; }
	};

	FConstIterator GetIterator() const
	{
		return FConstIterator(0, NumItems);
	}

private:
	int32 NumItems = 0;
};

template <typename ElementType, typename DeltaType>
struct TDeltaSetup
{
	using FElement = ElementType;
	using FDelta = DeltaType;

	TArray<FElement>& DeltaDataArray;
	FDelta Delta;
	int32 ElementStride = 1;
	bool bIsEnabled = true;
	int32 NumInstances = 0;


	/**
	 * Binds a specific delta type to an array for writing.
	 */
	struct FWriter
	{
		TDeltaSetup Setup;

		void Gather(TFunctionRef<FElement(int32)> DataSourceFunc)
		{
			if (!Setup.bIsEnabled)
			{
				return;
			}

			if (Setup.Delta.IsEmpty())
			{
				Setup.DeltaDataArray.Reset();
			}
			else
			{
				Setup.DeltaDataArray.Reset(Setup.Delta.GetNumItems());
				for (auto It = Setup.Delta.GetIterator(); It; ++It)
				{
					check(Setup.DeltaDataArray.Num() < Setup.Delta.GetNumItems());
					check(Setup.DeltaDataArray.Num() == It.GetItemIndex());
					Setup.DeltaDataArray.Emplace(DataSourceFunc(It.GetIndex()));
				}
			}
		}

		template <typename InElementType>
		void Gather(const TArrayView<InElementType> SourceData, int32 InElementStride = 1)
		{
			if (!Setup.bIsEnabled)
			{
				return;
			}

			if (Setup.Delta.IsEmpty())
			{
				Setup.DeltaDataArray.Reset();
			}
			// strides & element count matches - just copy the data
			else 
			{
				check(InElementStride == Setup.ElementStride);
				// It is a full update if either it is not a delta, or we're sending everything anyway
				bool bIsFull = !Setup.Delta.IsDelta() || SourceData.Num() == Setup.Delta.GetNumItems() * Setup.ElementStride;

				if (bIsFull)
				{
					// full update, bulk-copy
					Setup.DeltaDataArray = SourceData;
				}
				else
				{
					Setup.DeltaDataArray.Reset(Setup.Delta.GetNumItems() * Setup.ElementStride);
					for (auto It = Setup.Delta.GetIterator(); It; ++It)
					{
						check(Setup.DeltaDataArray.Num() < Setup.Delta.GetNumItems() * Setup.ElementStride);
						Setup.DeltaDataArray.Append(&SourceData[It.GetIndex() * Setup.ElementStride], Setup.ElementStride);
					}
				}
			}
		}

		void Gather(const FElement& SingleSourceElement)
		{
			if (!Setup.bIsEnabled)
			{
				return;
			}

			if (Setup.Delta.IsEmpty())
			{
				Setup.DeltaDataArray.Reset();
			}
			else 
			{
				const int32 TotalNumItems = Setup.Delta.GetNumItems() * Setup.ElementStride;
				bool bIsFull = !Setup.Delta.IsDelta();

				if (bIsFull)
				{
					// full update, bulk initialization
					Setup.DeltaDataArray.Init(SingleSourceElement, TotalNumItems);
				}
				else
				{
					Setup.DeltaDataArray.Reset();
					for (auto It = Setup.Delta.GetIterator(); It; ++It)
					{
						check(Setup.DeltaDataArray.Num() + Setup.ElementStride <= TotalNumItems);
						for (int32 i = 0; i < Setup.ElementStride; ++i)
						{
							Setup.DeltaDataArray.Add(SingleSourceElement);
						}
					}
				}
			}
		}
	};

	/**
	 * Binds a specific delta type to an array for writing.
	 */
	struct FReader
	{
		TDeltaSetup Setup;

		template <typename IndexRemapType>
		void Scatter(TArray<ElementType>& OutDataArray, const IndexRemapType& IndexRemap)
		{
			static_assert(std::is_same_v<decltype(OutDataArray), decltype(Setup.DeltaDataArray)>, "The types should match or it won't be able to use MoveTemp properly");
			if (!Setup.bIsEnabled)
			{
				OutDataArray.Reset();
			}
			else 
			{
				check(Setup.DeltaDataArray.Num() == Setup.Delta.GetNumItems() * Setup.ElementStride);

				bool bIsFull = !Setup.Delta.IsDelta() || Setup.NumInstances * Setup.ElementStride == Setup.DeltaDataArray.Num();;
				if (bIsFull && IndexRemap.IsIdentity())
				{
					check(Setup.DeltaDataArray.Num() == Setup.NumInstances * Setup.ElementStride);
					OutDataArray = MoveTemp(Setup.DeltaDataArray);
				}
				else
				{
					OutDataArray.SetNumUninitialized(Setup.NumInstances * Setup.ElementStride);
					for (auto It = Setup.Delta.GetIterator(); It; ++It)
					{
						int32 ItemIndex = It.GetItemIndex();
						int32 DestIndex = It.GetIndex();
						IndexRemap.Remap(ItemIndex, DestIndex);
						FMemory::Memcpy(&OutDataArray[DestIndex * Setup.ElementStride], &Setup.DeltaDataArray[ItemIndex * Setup.ElementStride], Setup.ElementStride * sizeof(FElement));
					}
				}
			}
		}

		template <typename ElementTransformFuncType, typename IndexRemapType>
		void Scatter(TArray<ElementType>& OutDataArray, ElementTransformFuncType ElementTransformFunc, const IndexRemapType& IndexRemap)
		{
			static_assert(std::is_same_v<decltype(OutDataArray), decltype(Setup.DeltaDataArray)>, "The types should match or it won't be able to use MoveTemp properly");
			if (!Setup.bIsEnabled)
			{
				OutDataArray.Reset();
			}
			else 
			{
				check(Setup.ElementStride == 1);
				bool bIsFull = !Setup.Delta.IsDelta() || Setup.NumInstances * Setup.ElementStride == Setup.DeltaDataArray.Num();;
				if (bIsFull && IndexRemap.IsIdentity())
				{
					check(Setup.DeltaDataArray.Num() == Setup.NumInstances * Setup.ElementStride);
					// Just change ownership of the array
					OutDataArray = MoveTemp(Setup.DeltaDataArray);
					// Then apply transform in-place
					for (int32 Index = 0; Index < Setup.NumInstances; ++Index)
					{
						ElementTransformFunc(OutDataArray[Index]);
					}
				}
				else
				{
					check(Setup.DeltaDataArray.Num() == Setup.Delta.GetNumItems() * Setup.ElementStride);

					OutDataArray.SetNumUninitialized(Setup.NumInstances);
					for (auto It = Setup.Delta.GetIterator(); It; ++It)
					{
						int32 ItemIndex = It.GetItemIndex();
						int32 DestIndex = It.GetIndex();
						IndexRemap.Remap(ItemIndex, DestIndex);
						ElementType Element = Setup.DeltaDataArray[ItemIndex];
						ElementTransformFunc(Element);
						OutDataArray[DestIndex] = Element;
					}
				}
			}
		}
	};

	FReader GetReader()
	{
		check(!bIsEnabled || DeltaDataArray.Num() == Delta.GetNumItems() * ElementStride);
		return FReader{ *this };
	}

	FWriter GetWriter()
	{
		check(!bIsEnabled || DeltaDataArray.IsEmpty());
		return FWriter{ *this };
	}
};

/**
 * Precomputed optimization data that descrives the spatial hashes and reordering needed.
 */
class FPrecomputedInstanceSpatialHashData
{
public:
	TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> Hashes;
	TArray<int32> ProxyIndexToComponentIndexRemap;
	uint32 PrimitiveLocalToWorldHash = 0u;
};

using FPrecomputedInstanceSpatialHashDataPtr = TSharedPtr<const FPrecomputedInstanceSpatialHashData, ESPMode::ThreadSafe>;



/**
 * Collects changed instance data (and what else is needed to update the instance data proxy) from the source and 
 */
class FInstanceUpdateChangeSet
{
public:
	/**
	 * Construct a full change set, with no delta (which will collect all enabled data).
	 */
	FInstanceUpdateChangeSet(int32 InNumSourceInstances, FInstanceDataFlags InFlags) 
		: bNeedFullUpdate(true) 
		, Flags(InFlags)
		, ForceFullFlags(InFlags)
		, NumSourceInstances(InNumSourceInstances)
	{
	}

	/**
	 * Construct a delta change set, but which can be forced to full using bInNeedFullUpdate.
	 */
	FInstanceUpdateChangeSet(bool bInNeedFullUpdate, FInstanceAttributeTracker&& InInstanceAttributeTracker, int32 InNumSourceInstances) 
		: InstanceAttributeTracker(MoveTemp(InInstanceAttributeTracker))
		, bNeedFullUpdate(bInNeedFullUpdate) 
		, NumSourceInstances(InNumSourceInstances)
	{
	}

	template <FInstanceAttributeTracker::EFlag Flag>
	FInstanceAttributeTracker::FDeltaRange<Flag> GetDelta(bool bForceEmpty, bool bForceFull = false) const 
	{ 
		if (bForceEmpty)
		{
			return FInstanceAttributeTracker::FDeltaRange<Flag>();
		}
		return InstanceAttributeTracker.GetDeltaRange<Flag>(IsFullUpdate() || bForceFull, NumSourceInstances); 
	}

	FInstanceAttributeTracker::FDeltaRange<FInstanceAttributeTracker::EFlag::TransformChanged> GetTransformDelta() const 
	{ 
		return GetDelta<FInstanceAttributeTracker::EFlag::TransformChanged>(false, bUpdateAllInstanceTransforms);
	}

	FInstanceAttributeTracker::FDeltaRange<FInstanceAttributeTracker::EFlag::IndexChanged> GetIndexChangedDelta() const 
	{ 
		return GetDelta<FInstanceAttributeTracker::EFlag::IndexChanged>(false);
	}

	template <FInstanceAttributeTracker::EFlag DeltaFlag, typename ElementType>
	TDeltaSetup<ElementType, FInstanceAttributeTracker::FDeltaRange<DeltaFlag>> GetSetup(bool bEnabledFlag, bool bForceFullFlag, TArray<ElementType> &DataArray, int32 ElementStride = 1)
	{
		FInstanceAttributeTracker::FDeltaRange<DeltaFlag> Delta = GetDelta<DeltaFlag>(!bEnabledFlag, bForceFullFlag);
		return TDeltaSetup<ElementType, FInstanceAttributeTracker::FDeltaRange<DeltaFlag>>{ DataArray, Delta, ElementStride, bEnabledFlag, NumSourceInstances };
	}

	template <typename ElementType>
	TDeltaSetup<ElementType, FIdentityDeltaRange> GetSetup(bool bEnabledFlag, TArray<ElementType> &DataArray, int32 ElementStride = 1)
	{
		FIdentityDeltaRange Delta = FIdentityDeltaRange(bEnabledFlag ? NumSourceInstances : 0);
		return TDeltaSetup<ElementType, FIdentityDeltaRange>{ DataArray, Delta, ElementStride, bEnabledFlag, NumSourceInstances };
	}

	// These setups define the mapping from a delta attribute bit in the tracker to the data array. 
	// This is not a simple 1:1 mapping as we only track a few bits, and there is also special overrides to take into account.
	auto GetTransformSetup()		{ return GetSetup<FInstanceAttributeTracker::EFlag::TransformChanged>(true, bUpdateAllInstanceTransforms, Transforms); }
	auto GetPrevTransformSetup()	{ return GetSetup<FInstanceAttributeTracker::EFlag::TransformChanged>(Flags.bHasPerInstanceDynamicData, bUpdateAllInstanceTransforms || ForceFullFlags.bHasPerInstanceDynamicData, PrevTransforms); }
	auto GetCustomDataSetup()		{ return GetSetup<FInstanceAttributeTracker::EFlag::CustomDataChanged>(Flags.bHasPerInstanceCustomData, ForceFullFlags.bHasPerInstanceCustomData, PerInstanceCustomData, NumCustomDataFloats); }
	auto GetLocalBoundsSetup()		{ return GetSetup<FInstanceAttributeTracker::EFlag::SkinningDataChanged>(Flags.bHasPerInstanceLocalBounds, ForceFullFlags.bHasPerInstanceLocalBounds, InstanceLocalBounds); }
	auto GetSkinningDataSetup()		{ return GetSetup<FInstanceAttributeTracker::EFlag::SkinningDataChanged>(Flags.bHasPerInstanceSkinningData, ForceFullFlags.bHasPerInstanceSkinningData, InstanceSkinningData); }

	// Convenience functions to get a reader / writer for a given array
	auto GetTransformWriter()		{ return GetTransformSetup().GetWriter(); }
	auto GetPrevTransformWriter()	{ return GetPrevTransformSetup().GetWriter(); }
	auto GetCustomDataWriter()		{ return GetCustomDataSetup().GetWriter(); }
	auto GetLocalBoundsWriter()		{ return GetLocalBoundsSetup().GetWriter(); }
	auto GetSkinningDataWriter()	{ return GetSkinningDataSetup().GetWriter(); }

	auto GetTransformReader()		{ return GetTransformSetup().GetReader(); }
	auto GetPrevTransformReader()	{ return GetPrevTransformSetup().GetReader(); }
	auto GetCustomDataReader()		{ return GetCustomDataSetup().GetReader(); }
	auto GetLocalBoundsReader()		{ return GetLocalBoundsSetup().GetReader(); }
	auto GetSkinningDataReader()	{ return GetSkinningDataSetup().GetReader(); }

	auto GetLightShadowUVBiasReader() { return GetSetup(Flags.bHasPerInstanceLMSMUVBias, InstanceLightShadowUVBias).GetReader(); }

#if WITH_EDITOR
	auto GetEditorDataReader() { return GetSetup(Flags.bHasPerInstanceEditorData, InstanceEditorData).GetReader(); }
	// Set editor data 
	void SetEditorData(const TArray<TRefCountPtr<HHitProxy>>& HitProxies, const TBitArray<> &SelectedInstances);
#endif

	/**
	 * Used to set a single, shared, instance local bounds, only allowed when Flags.bHasPerInstanceLocalBounds is false.
	 */
	ENGINE_API void SetSharedLocalBounds(const FRenderBounds &Bounds);

	bool IsFullUpdate() const
	{
		return bNeedFullUpdate;
	}

	FInstanceAttributeTracker InstanceAttributeTracker;
	bool bNeedFullUpdate = false;

	/**
	 * Flags that dictate what attributes will be gathered from the source (and deposited at the destination). These must be present during gather (or they should have been disabled earlier).
	 * Constructed as the intersection of the flags indicated by the component & what is supported by the proxy(?).
	 */
	FInstanceDataFlags Flags;
	
	/**
	 * Flags that can be set to force individual attributes to use a full update has effect IFF Flags is set for the given attribute.
	 * If an attribute was enabled after not being used (e.g., on a material change, perhaps) a full update must be sent.
	 */
	FInstanceDataFlags ForceFullFlags;

	// Needs its own bool because it is always present and thus doesn't have a flag in FInstanceDataFlags
	bool bUpdateAllInstanceTransforms = false;
	bool bIdentityIdMap = false;
	TArray<FPrimitiveInstanceId> IndexToIdMapDeltaData;

	int32 NumCustomDataFloats = 0;
	TArray<FRenderTransform> Transforms;
	TArray<FRenderTransform> PrevTransforms;
	TArray<float> PerInstanceCustomData;
	TArray<uint32> InstanceSkinningData;
	TArray<FVector4f> InstanceLightShadowUVBias;
	TArray<FRenderBounds> InstanceLocalBounds;

#if WITH_EDITOR
	TArray<uint32> InstanceEditorData;
	TBitArray<> SelectedInstances;
	TPimplPtr<FOpaqueHitProxyContainer> HitProxyContainer;
#endif

	// Function that can generate all the random IDs on demand, if this is supplied is must generate all, otherwise it is zero-filled if they are requested.
	TFunction<void(TArray<float> &InstanceRandomIDs)> GeneratePerInstanceRandomIds;

	FRenderTransform PrimitiveToRelativeWorld;
	FVector PrimitiveWorldSpaceOffset;
	TOptional<FRenderTransform> PreviousPrimitiveToRelativeWorld;
	float AbsMaxDisplacement = 0.0f;
	int32 NumSourceInstances = 0;
	int32 MaxInstanceId = 0;

	/**
	 * Describes precomputed spatial hashes and the instance reordering that is needed to use these.
	 * If set, the update must reorder the data appropriately.
	 * Should only ever be used together with a full rebuild.
	 */
	FPrecomputedInstanceSpatialHashDataPtr PrecomputedOptimizationData;
};


#if WITH_EDITOR

ENGINE_API TPimplPtr<FOpaqueHitProxyContainer> MakeOpaqueHitProxyContainer(const TArray<TRefCountPtr<HHitProxy>>& InHitProxies);


#endif
