// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceDataTypes.h"
#include "RenderTransform.h"
#include "Tasks/Task.h"
#include "Engine/EngineTypes.h"
#include "Rendering/RenderingSpatialHash.h"

class FPrimitiveDrawInterface;
class FStaticMeshInstanceBuffer;

class FInstanceIdIndexMap
{
public:
	FInstanceIdIndexMap() = default;

	/**
	 * Returns true if the mapping is an identity mapping. I.e., each instance ID corresponds to the same index.
	 */
	inline bool IsIdentity() const { return IndexToIdMap.IsEmpty(); }

	/**
	 * Returms the upper bound on the instance ID represented as an integer, may be larger than the number of instances but is never larger than the max that has ever been allocated.
	 */
	inline  int32 GetMaxInstanceId() const { return IsIdentity() ? NumInstances : IdToIndexMap.Num();}

	/**
	 * Returns the maximum instance index (or rather the maximum valid number plus one), AKA the number of instances represented.
	 */
	inline  int32 GetMaxInstanceIndex() const { return IsIdentity() ? NumInstances : IndexToIdMap.Num();}

	/**
	 * Returns true if InstanceId is in the mapped range AND .
	 */
	inline  bool IsValidId(FPrimitiveInstanceId InstanceId) const { return InstanceId.Id >= 0 && InstanceId.Id < GetMaxInstanceId() && (IsIdentity() || IdToIndexMap[InstanceId.Id] != INDEX_NONE); }
	/**
	 * Translate from FPrimitiveInstanceId to Index
	 */
	inline  int32 IdToIndex(FPrimitiveInstanceId InstanceId) const
	{ 
		return IsIdentity() ? InstanceId.Id : IdToIndexMap[InstanceId.Id]; 
	}

	/**
	 * Translate from Index to FPrimitiveInstanceId
	 */
	inline  FPrimitiveInstanceId IndexToId(int32 InstanceIndex) const
	{ 
		check(InstanceIndex < GetMaxInstanceIndex());
		return IsIdentity() ? FPrimitiveInstanceId{ InstanceIndex } : IndexToIdMap[InstanceIndex]; 
	}

	/**
	 * Set the index mapping for the given InstanceId to INDEX_NONE.
	 */
	inline void SetInvalid(FPrimitiveInstanceId InstanceId)
	{
		IdToIndexMap[InstanceId.Id] = INDEX_NONE;
	}

	/**
	 * Update the mapping from ID to index (and vice versa).
	 * Does _not_ reset any previous mapping for the given ID.
	 */
	inline void Update(FPrimitiveInstanceId InstanceId, int32 InstanceIndex)
	{
		IndexToIdMap[InstanceIndex] = InstanceId;
		IdToIndexMap[InstanceId.Id] = InstanceIndex;
	}

	/**
	 * Reset the mapping to an identity map of size InNumInstances.
	 */
	ENGINE_API void Reset(int32 InNumInstances);

	/**
	 * Make sure the mapping is explicit and add enough space to accommodate the InNumInstances & MaxInstanceId.
	 */
	ENGINE_API void ResizeExplicit(int32 InNumInstances, int32 MaxInstanceId);

	/**
	 * Convert an implicit identity mapping to an explicit one, by filling in the mapping arrays. 
	 * Not allowed to be called if the mapping is already explicit.
	 */
	ENGINE_API void CreateExplicitIdentityMapping();

	/**
	 */
	ENGINE_API void Serialize(FArchive& Ar);

	/**
	 */	
	ENGINE_API void RebuildFromIndexToIdMap(TArray<FPrimitiveInstanceId>&& InIndexToIdMap, int32 MaxInstanceId);

protected:

	/**
	 */	
	ENGINE_API void RebuildFromIndexToIdMap(int32 MaxInstanceId);

	// Bidirectional mapping to / from ID.
	TArray<FPrimitiveInstanceId> IndexToIdMap;
	TArray<int32> IdToIndexMap;
	// used when the mapping is implicit (i.e., identity)
	int32 NumInstances = 0;
};

#if WITH_EDITOR

struct FInstanceEditorData
{
	static inline uint32 Pack(const FColor& HitProxyColor, bool bSelected)
	{
		return uint32(HitProxyColor.R) | uint32(HitProxyColor.G) << 8u | uint32(HitProxyColor.B) << 16u | (bSelected ? 1u << 24u : 0u);
	}

	static inline void Unpack(const uint32 Packed, FColor& OutHitProxyColor, bool& bOutSelected)
	{
		OutHitProxyColor.R = uint8((Packed >>  0u) & 0xFFu);
		OutHitProxyColor.G = uint8((Packed >>  8u) & 0xFFu);
		OutHitProxyColor.B = uint8((Packed >> 16u) & 0xFFu);
		bOutSelected = (Packed >> 24u) != 0;
	}
};

#endif

class FInstanceSceneDataImmutable;
using FInstanceSceneDataImmutablePtr = TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe>;
/**
 */
class FInstanceSceneDataBuffers
{
public:
	ENGINE_API FInstanceSceneDataBuffers();
	ENGINE_API FInstanceSceneDataBuffers(bool InbInstanceDataIsGPUOnly);
	FInstanceSceneDataBuffers(FInstanceSceneDataBuffers&& Other) = default;

	struct FAccessTag
	{
		enum class EKind
		{
			Reader,
			Writer
		};
		FAccessTag() : WriterTag(0), Kind(EKind::Reader) { }
		
		// WriterTag must be non-zero because 0 is the "no writer" sentinel in CurrentWriterTag
		FAccessTag(uint32 InWriterTag) : WriterTag(InWriterTag ? InWriterTag : ~0u), Kind(EKind::Writer) {}
		
		uint32 WriterTag;
		EKind Kind;
	};

	inline const FInstanceDataFlags& GetFlags() const { return Flags; }

	inline int32 GetNumInstances(FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag); return bInstanceDataIsGPUOnly ? NumInstancesGPUOnly : InstanceToPrimitiveRelative.Num(); }
	inline int32 GetNumCustomDataFloats(FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag); return NumCustomDataFloats; }

	ENGINE_API static uint32 CalcPayloadDataStride(FInstanceDataFlags Flags, int32 InNumCustomDataFloats, int32 InNumPayloadExtensionFloat4s);


	ENGINE_API uint32 GetPayloadDataStride(FAccessTag AccessTag = FAccessTag()) const;

	/**
	 * Clamps the index to the InstanceLocalBounds size (it is always 1:1 with the instance count or exactly 1).
	 */
	ENGINE_API FRenderBounds GetInstanceLocalBounds(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const;

	/**
	 * Get the primitive-relative bounds for the instance. These are the local bounds transformed into Primitive relative space using InstanceToPrimitiveRelative transform.
	 * Note that this may contain instance rotation which may possibly lead to expansion of the bounds that is less tight than a transformed sphere bounds.
	 * Clamps the index to the InstanceLocalBounds size (it is always 1:1 with the instance count or exactly 1).
	 */
	ENGINE_API FRenderBounds GetInstancePrimitiveRelativeBounds(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const;

	/**
	 * Clamps the index to the InstanceLocalBounds size (it is always 1:1 with the instance count or exactly 1).
	 */
	ENGINE_API FBoxSphereBounds GetInstanceWorldBounds(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const;
	
	/**
	 */
	ENGINE_API FMatrix GetInstanceToWorld(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const;

	/** Whether instance data resides only in the GPU Scene, not backed by a copy in CPU memory. */
	inline bool IsInstanceDataGPUOnly(FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag); return bInstanceDataIsGPUOnly; }

	/**
	 */
	inline FRenderTransform GetInstanceToPrimitiveRelative(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag); return InstanceToPrimitiveRelative[InstanceIndex]; }

	/**
	 */
	inline FRenderTransform GetPrevInstanceToPrimitiveRelative(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag); return PrevInstanceToPrimitiveRelative.IsEmpty() ? InstanceToPrimitiveRelative[InstanceIndex] : PrevInstanceToPrimitiveRelative[InstanceIndex]; }
	
	/**
	 * Get the offset for the primitive-relative space used for transforms and bounds.
	 */
	inline const FVector &GetPrimitiveWorldSpaceOffset(FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag);return PrimitiveWorldSpaceOffset; }

	inline bool GetInstanceVisible(int32 InstanceIndex, FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag);return !Flags.bHasPerInstanceVisible || VisibleInstances[InstanceIndex]; }

	inline const FRenderTransform &GetPrimitiveToRelativeWorld(FAccessTag AccessTag = FAccessTag()) const { ValidateAccess(AccessTag);return PrimitiveToRelativeWorld; }

	ENGINE_API FRenderTransform ComputeInstanceToPrimitiveRelative(const FMatrix44f &InstanceToPrimitive, FAccessTag AccessTag = FAccessTag());

	ENGINE_API void SetPrimitiveLocalToWorld(const FMatrix &PrimitiveLocalToWorld, FAccessTag AccessTag = FAccessTag());

	ENGINE_API FInstanceDataBufferHeader GetHeader(FAccessTag AccessTag = FAccessTag()) const;

	ENGINE_API void ValidateData() const;

	struct FCompressedSpatialHashItem
	{
		bool operator==(const FCompressedSpatialHashItem& Other) const = default;

		// TODO: compress the location - store delta to primitive loc for example
		RenderingSpatialHash::FLocation64 Location;
		int32 NumInstances;

		/** Explicit cell-space bounds for this cell's instances. */
		FRenderBounds ExplicitBounds;

		friend inline FArchive& operator<<(FArchive& Ar, FCompressedSpatialHashItem& Item)
		{
			Ar << Item.Location;
			Ar << Item.NumInstances;
			Ar << Item.ExplicitBounds;
			return Ar;
		}
	};

	struct FWriteView
	{
		FInstanceSceneDataBuffers::FAccessTag AccessTag;
		FRenderTransform& PrimitiveToRelativeWorld;
		FVector& PrimitiveWorldSpaceOffset;
		TArray<FRenderBounds>& InstanceLocalBounds;
		TArray<float>& InstanceCustomData;
		TArray<float>& InstanceRandomIDs;
		TArray<FVector4f>& InstanceLightShadowUVBias;
		TArray<uint32>& InstanceSkinningData;
		TArray<uint32>& InstanceHierarchyOffset;
		TArray<FVector4f>& InstancePayloadExtension;
		TArray<FRenderTransform>& InstanceToPrimitiveRelative;
		TArray<FRenderTransform>& PrevInstanceToPrimitiveRelative;
	#if WITH_EDITOR
		TArray<uint32>& InstanceEditorData;
		TBitArray<>& SelectedInstances;
	#endif
		TBitArray<>& VisibleInstances;

		int32& NumInstancesGPUOnly;
		int32& NumCustomDataFloats;
		FInstanceDataFlags& Flags;
	};

	FWriteView BeginWriteAccess(FAccessTag AccessTag)
	{
		check(AccessTag.Kind == FAccessTag::EKind::Writer && AccessTag.WriterTag != 0u);
		uint32 PrevTagValue = 0u;
#if DO_CHECK
		check(CurrentWriterTag.Value.compare_exchange_strong(PrevTagValue, AccessTag.WriterTag));
#endif
		return FWriteView
		{
			AccessTag,
			PrimitiveToRelativeWorld,
			PrimitiveWorldSpaceOffset,
			InstanceLocalBounds,
			InstanceCustomData,
			InstanceRandomIDs,
			InstanceLightShadowUVBias,
			InstanceSkinningData,
			InstanceHierarchyOffset,
			InstancePayloadExtension,
			InstanceToPrimitiveRelative,
			PrevInstanceToPrimitiveRelative,
		#if WITH_EDITOR
			InstanceEditorData,
			SelectedInstances,
		#endif
			VisibleInstances,
			NumInstancesGPUOnly,
			NumCustomDataFloats,
			Flags
		};
	}
	void EndWriteAccess(FAccessTag AccessTag)
	{
		check(bInstanceDataIsGPUOnly ? InstanceToPrimitiveRelative.IsEmpty() : (NumInstancesGPUOnly == 0));
		check(AccessTag.Kind == FAccessTag::EKind::Writer && AccessTag.WriterTag != 0u);
		uint32 PrevTagValue = AccessTag.WriterTag;
#if DO_CHECK
		check(CurrentWriterTag.Value.compare_exchange_strong(PrevTagValue, 0U));
#endif
	}


	struct FReadView
	{
		const FRenderTransform& PrimitiveToRelativeWorld;
		const FVector& PrimitiveWorldSpaceOffset;
		const TArray<FRenderBounds>& InstanceLocalBounds;
		const TArray<float>& InstanceCustomData;
		const TArray<float>& InstanceRandomIDs;
		const TArray<FVector4f>& InstanceLightShadowUVBias;
		const TArray<uint32>& InstanceSkinningData;
		const TArray<uint32>& InstanceHierarchyOffset;
		const TArray<FVector4f>& InstancePayloadExtension;
		const TArray<FRenderTransform>& InstanceToPrimitiveRelative;
		const TArray<FRenderTransform>& PrevInstanceToPrimitiveRelative;
	#if WITH_EDITOR
		const TArray<uint32>& InstanceEditorData;
		const TBitArray<>& SelectedInstances;
	#endif
		const TBitArray<>& VisibleInstances;

		int32 NumInstancesGPUOnly;
		int32 NumCustomDataFloats;
		FInstanceDataFlags Flags;
		bool bInstanceDataIsGPUOnly;
	};

	FReadView GetReadView(FAccessTag AccessTag = FAccessTag()) const
	{
		check(AccessTag.Kind == FAccessTag::EKind::Reader && AccessTag.WriterTag == 0u);
		ValidateAccess(AccessTag);
		return FReadView
		{
			PrimitiveToRelativeWorld,
			PrimitiveWorldSpaceOffset,
			InstanceLocalBounds,
			InstanceCustomData,
			InstanceRandomIDs,
			InstanceLightShadowUVBias,
			InstanceSkinningData,
			InstanceHierarchyOffset,
			InstancePayloadExtension,
			InstanceToPrimitiveRelative,
			PrevInstanceToPrimitiveRelative,
#if WITH_EDITOR
			InstanceEditorData,
			SelectedInstances,
#endif
			VisibleInstances,
			NumInstancesGPUOnly,
			NumCustomDataFloats,
			Flags,
			bInstanceDataIsGPUOnly
		};
	}

	ENGINE_API void SetImmutable(FInstanceSceneDataImmutable &&ImmutableData, FAccessTag AccessTag);
	inline FInstanceSceneDataImmutablePtr GetImmutable() const { return Immutable; }

protected:
	FRenderTransform PrimitiveToRelativeWorld;
	FVector PrimitiveWorldSpaceOffset;
	TArray<FRenderBounds> InstanceLocalBounds;
	TArray<float> InstanceCustomData;
	TArray<float> InstanceRandomIDs;
	TArray<FVector4f> InstanceLightShadowUVBias;
	TArray<uint32> InstanceSkinningData;
	TArray<uint32> InstanceHierarchyOffset;
	TArray<FVector4f> InstancePayloadExtension;
	TArray<FRenderTransform> InstanceToPrimitiveRelative;
	TArray<FRenderTransform> PrevInstanceToPrimitiveRelative;
#if WITH_EDITOR
	TArray<uint32> InstanceEditorData;
	TBitArray<> SelectedInstances;
#endif
	TBitArray<> VisibleInstances;

	int32 NumInstancesGPUOnly = 0;
	int32 NumCustomDataFloats = 0;
	FInstanceDataFlags Flags;

	/** Instance data stored only in GPUScene, not stored in the above arrays in CPU memory. */
	bool bInstanceDataIsGPUOnly : 1 = false;

#if DO_CHECK
	struct FCurrentWriterTag
	{
		FCurrentWriterTag() {}
		FCurrentWriterTag(FCurrentWriterTag&& Other)
		{
			check(Other.Value == 0u);
		}

		std::atomic<uint32> Value = 0;
	};
	FCurrentWriterTag CurrentWriterTag;

	inline void ValidateAccess(const FAccessTag& AccessTag) const
	{
		check(AccessTag.Kind == FAccessTag::EKind::Reader && CurrentWriterTag.Value == 0u
		|| AccessTag.Kind == FAccessTag::EKind::Writer && CurrentWriterTag.Value == AccessTag.WriterTag);
	}
#else
	inline void ValidateAccess(const FAccessTag& AccessTag) const {}
#endif

	FInstanceSceneDataImmutablePtr Immutable;
};

/**
 * Stores instance data that is immutable - i.e., it is never changed after construction, it can be held onto by the renderer through a shared pointer.
 */
class FInstanceSceneDataImmutable
{
public:
	inline FInstanceSceneDataImmutable(const TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> &InCompressedInstanceSpatialHashes) : CompressedInstanceSpatialHashes(InCompressedInstanceSpatialHashes) {}

	inline const TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> &GetCompressedInstanceSpatialHashes() const { return CompressedInstanceSpatialHashes; }
private:
	// Encodes consecutive ranges of instances that share the same spatial hash location
	TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> CompressedInstanceSpatialHashes;
};

/**
 * Trivial helper to manage single-instance primitives that can be embedded in the primtive proxy & alias the view to the single data elements.
 */
class FSingleInstanceDataBuffers : public FInstanceSceneDataBuffers
{
public:
	ENGINE_API FSingleInstanceDataBuffers();

	// 
	ENGINE_API void UpdateDefaultInstance(const FMatrix &PrimitiveLocalToWorld, const FRenderBounds LocalBounds);
};

/**
 *
 */
class FInstanceDataUpdateTaskInfo
{
public:
	/**
	 * The header is always available, and so does not cause a sync.
	 * TODO: in the future, it might be possible to have other update tasks (that generate the data on the fly perhaps) that don't have this info handy at dispatch time, those would need to sync & fetch.
	 */
	inline const FInstanceDataBufferHeader &GetHeader() const { return InstanceDataBufferHeader; }
	
	/**
	 * Must call this before accessing the majority of the data in the instance data buffers.
	 */
	ENGINE_API void WaitForUpdateCompletion();

private:
	template <typename TaskLambdaType>
	friend void BeginInstanceDataUpdateTask(FInstanceDataUpdateTaskInfo& InstanceDataUpdateTaskInfo, TaskLambdaType&& TaskLambda, const FInstanceDataBufferHeader& InInstanceDataBufferHeader);


	UE::Tasks::FTask UpdateTaskHandle;
	FInstanceDataBufferHeader InstanceDataBufferHeader;
};

class FInstanceDataSceneProxy
{
public:
	ENGINE_API FInstanceDataSceneProxy();
	ENGINE_API FInstanceDataSceneProxy(FInstanceSceneDataBuffers&& InInstanceSceneDataBuffers);
	ENGINE_API virtual ~FInstanceDataSceneProxy();

	virtual const FInstanceSceneDataBuffers* GeInstanceSceneDataBuffers() const { return &InstanceSceneDataBuffers; }

	/**
	 * Implement to provide syncable task info, if this returns a nullptr it is required that GeInstanceSceneDataBuffers() performs any needed synchronization.
	 */
	virtual FInstanceDataUpdateTaskInfo *GetUpdateTaskInfo() { return nullptr; }

	virtual void DebugDrawInstanceChanges(FPrimitiveDrawInterface* DebugPDI, ESceneDepthPriorityGroup SceneDepthPriorityGroup) {};

	/**
	 * Workaround to support the ISMC non GPU scene rendering path.
	 */
	virtual FStaticMeshInstanceBuffer* GetLegacyInstanceBuffer() { return nullptr; }

protected:
	ENGINE_API void IncStatCounters();
	ENGINE_API void DecStatCounters();

	FInstanceSceneDataBuffers InstanceSceneDataBuffers;
};
