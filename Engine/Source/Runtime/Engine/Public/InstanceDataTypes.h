// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InstanceDataTypes.generated.h"

class FInstanceSceneDataBuffers;
class FInstanceDataUpdateTaskInfo;
class FInstanceDataSceneProxy;

/**
 * ID of an instance within a primitive, persistent for the lifetime of the instance. 
 * Can be used as a linear index as the maximum possible value is never higher than the high-water-mark for the number of instances in the component.
 */
USTRUCT(BlueprintType)
struct FPrimitiveInstanceId
{
	GENERATED_BODY()
	inline bool IsValid() const { return Id != INDEX_NONE; }
	inline bool operator==(const FPrimitiveInstanceId &Other) const { return Other.Id == Id;}

	inline int32 GetAsIndex() const { return Id; }

	friend FArchive& operator<<(FArchive& Ar, FPrimitiveInstanceId& InstanceData)
	{
		Ar << InstanceData.Id;
		return Ar;
	}

	int32 Id = INDEX_NONE;
};

union FInstanceDataFlags
{
	uint16 Packed;
	struct
	{
		uint16 bHasPerInstanceRandom : 1;
		uint16 bHasPerInstanceCustomData : 1;
		uint16 bHasPerInstanceDynamicData : 1;
		uint16 bHasPerInstanceSkinningData : 1;
		uint16 bHasPerInstanceLMSMUVBias : 1;
		uint16 bHasPerInstanceLocalBounds : 1;
		uint16 bHasPerInstanceHierarchyOffset : 1;
		uint16 bHasPerInstancePayloadExtension : 1;
		uint16 bHasPerInstanceVisible : 1;
		uint16 bHasPerInstanceEditorData : 1;
		uint16 bHasCompressedSpatialHash : 1;
	};

	inline bool operator == (FInstanceDataFlags Other) const { return Packed == Other.Packed; }

	inline FInstanceDataFlags()
	{
		Packed = 0;
	}
};

/**
 * Header data for the instance data buffer that might be produced early (before the heavy lifting of a data update) to move the main sync point back & off the render thread.
 */
struct FInstanceDataBufferHeader
{
	int32 NumInstances = 0;
	uint32 PayloadDataStride = 0u;
	FInstanceDataFlags Flags;
	bool bInstanceDataIsGPUOnly : 1 = false;

	inline bool operator==(const FInstanceDataBufferHeader& Other) const
	{
		return NumInstances == Other.NumInstances && PayloadDataStride == Other.PayloadDataStride && Flags == Other.Flags && bInstanceDataIsGPUOnly == Other.bInstanceDataIsGPUOnly;
	}

	ENGINE_API static const FInstanceDataBufferHeader SinglePrimitiveHeader;
};

class FInstanceSceneDataBuffers;
class FInstanceDataUpdateTaskInfo;
class FInstanceDataSceneProxy;