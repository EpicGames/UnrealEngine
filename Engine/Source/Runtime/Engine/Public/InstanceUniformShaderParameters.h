// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "SceneTypes.h"
#include "RenderResource.h"
#include "RenderTransform.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"
#include "Containers/StaticArray.h"
#include "SceneDefinitions.h"

#define INVALID_LAST_UPDATE_FRAME 0xFFFFFFFFu

struct FInstanceSceneData
{
	FRenderTransform LocalToPrimitive;

	// Should always use this accessor so shearing is properly
	// removed from the concatenated transform.
	inline FRenderTransform ComputeLocalToWorld(const FRenderTransform& PrimitiveToWorld) const
	{
		FRenderTransform LocalToWorld = LocalToPrimitive * PrimitiveToWorld;

	// TODO: Enable when scale is decomposed within FRenderTransform, so we don't have 3x
	// length calls to retrieve the scale when checking for non-uniform scaling.
	#if 0
		// Shearing occurs when applying a rotation and then non-uniform scaling. It is much more likely that 
		// an instance would be non-uniformly scaled than the primitive, so we'll check if the primitive has 
		// non-uniform scaling, and orthogonalize in that case.
		if (PrimitiveToWorld.IsScaleNonUniform())
	#endif
		{
			LocalToWorld.Orthogonalize();
		}

		return LocalToWorld;
	}
};

struct FInstanceDynamicData
{
	FRenderTransform PrevLocalToPrimitive;

	// Should always use this accessor so shearing is properly
	// removed from the concatenated transform.
	inline FRenderTransform ComputePrevLocalToWorld(const FRenderTransform& PrevPrimitiveToWorld) const
	{
		FRenderTransform PrevLocalToWorld = PrevLocalToPrimitive * PrevPrimitiveToWorld;

	// TODO: Enable when scale is decomposed within FRenderTransform, so we don't have 3x
	// length calls to retrieve the scale when checking for non-uniform scaling.
	#if 0
		// Shearing occurs when applying a rotation and then non-uniform scaling. It is much more likely that 
		// an instance would be non-uniformly scaled than the primitive, so we'll check if the primitive has 
		// non-uniform scaling, and orthogonalize in that case.
		if (PrevPrimitiveToWorld.IsScaleNonUniform())
	#endif
		{
			PrevLocalToWorld.Orthogonalize();
		}

		return PrevLocalToWorld;
	}
};

struct FInstanceSceneShaderData
{
private:
	// Must match GetInstanceSceneData() in SceneData.ush
	// Allocate the max Float4s usage when compressed transform is used.
	static constexpr uint32 CompressedTransformDataStrideInFloat4s = 3;
	static constexpr uint32 UnCompressedTransformDataStrideInFloat4s = 4;

public:
	static ENGINE_API bool SupportsCompressedTransforms();

	static ENGINE_API uint32 GetDataStrideInFloat4s();

	static uint32 GetEffectiveNumBytes()
	{
		return (GetDataStrideInFloat4s() * sizeof(FVector4f));
	}

	FInstanceSceneShaderData() : Data(InPlace, NoInit)
	{
	}

	ENGINE_API void Build
	(
		uint32 PrimitiveId,
		uint32 RelativeId,
		uint32 InstanceFlags,
		uint32 LastUpdateFrame,
		uint32 CustomDataCount,
		float RandomID,
		bool bIsVisible = true
	);

	ENGINE_API void Build
	(
		uint32 PrimitiveId,
		uint32 RelativeId,
		uint32 InstanceFlags,
		uint32 LastUpdateFrame,
		uint32 CustomDataCount,
		float RandomID,
		const FRenderTransform& LocalToPrimitive,
		const FRenderTransform& PrimitiveToWorld,
		bool bIsVisible = true
	);

	inline void BuildInternal
	(
		uint32 PrimitiveId,
		uint32 RelativeId,
		uint32 InstanceFlags,
		uint32 LastUpdateFrame,
		uint32 CustomDataCount,
		float RandomID,
		const FRenderTransform& LocalToWorld,
		bool bIsVisible,
		bool bSupportsCompressedTransforms
	)
	{
		// Note: layout must match GetInstanceData in SceneData.ush and InitializeInstanceSceneData in GPUSceneWriter.ush

		const float RotDeterminant = LocalToWorld.RotDeterminant();
		if (RotDeterminant < 0.0f)
		{
			InstanceFlags |= INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN;
		}
		else
		{
			InstanceFlags &= ~INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN;
		}

		// Mark zero scaled instances as hidden.
		if (!bIsVisible || RotDeterminant == 0.0f)
		{
			InstanceFlags |= INSTANCE_SCENE_DATA_FLAG_HIDDEN;
		}

		checkSlow((PrimitiveId		& 0x000FFFFF) == PrimitiveId);
		checkSlow((InstanceFlags	& 0x00000FFF) == InstanceFlags);
		checkSlow((RelativeId		& 0x00FFFFFF) == RelativeId);
		checkSlow((CustomDataCount	& 0x000000FF) == CustomDataCount);

		const uint32 Packed0 = (InstanceFlags   << 20u) | PrimitiveId;
		const uint32 Packed1 = (CustomDataCount << 24u) | RelativeId;

		Data[0].X  = *(const float*)&Packed0;
		Data[0].Y  = *(const float*)&Packed1;
		Data[0].Z  = *(const float*)&LastUpdateFrame;
		Data[0].W  = *(const float*)&RandomID;

		if (bSupportsCompressedTransforms)
		{
			FCompressedTransform CompressedLocalToWorld(LocalToWorld);
			Data[1] = *(const FVector4f*)&CompressedLocalToWorld.Rotation[0];
			Data[2] = *(const FVector3f*)&CompressedLocalToWorld.Translation;
		}
		else
		{
			// Note: writes 3x float4s
			LocalToWorld.To3x4MatrixTranspose((float*)&Data[1]);
		}
	}

	TStaticArray<FVector4f, UnCompressedTransformDataStrideInFloat4s> Data;
};
