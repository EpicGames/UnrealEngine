// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/InstancedSkinnedMeshComponent.h"
#include "InstanceData/InstanceDataManager.h"
#include "InstanceData/InstanceUpdateChangeSet.h"
#include "InstancedSkinnedMeshSceneProxyDesc.h"
#include "SkinnedMeshComponentHelper.h"

/** Helper class used to share implementation for different InstancedSkinnedMeshComponent types */
class FInstancedSkinnedMeshComponentHelper
{
public:
	template <class T, bool bSupportHitProxies = true>
	static FInstanceDataManagerSourceDataDesc GetComponentDesc(T& InComponent, EShaderPlatform ShaderPlatform);
	
	template <class T>
	static FBoxSphereBounds CalcBounds(const T& InComponent, const FTransform& LocalToWorld);

	template <class T>
	static FSkeletalMeshObject* CreateMeshObject(const T& InComponent, const FInstancedSkinnedMeshSceneProxyDesc& InSceneProxyDesc);

	template <class T>
	static bool IsEnabled(const T& InComponent);

	template <class T>
	static FPrimitiveSceneProxy* CreateSceneProxy(const T& InComponent, const FInstancedSkinnedMeshSceneProxyDesc& Desc);
};

template <class T, bool bSupportHitProxies>
FInstanceDataManagerSourceDataDesc FInstancedSkinnedMeshComponentHelper::GetComponentDesc(T& InComponent, EShaderPlatform ShaderPlatform)
{
	FInstanceDataManagerSourceDataDesc ComponentDesc;

	ComponentDesc.PrimitiveMaterialDesc = FPrimitiveComponentHelper::GetUsedMaterialPropertyDesc(InComponent, ShaderPlatform);

	FInstanceDataFlags Flags;
	Flags.bHasPerInstanceRandom = ComponentDesc.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom;
	Flags.bHasPerInstanceCustomData = ComponentDesc.PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData && InComponent.NumCustomDataFloats != 0;
#if WITH_EDITOR
	if constexpr (bSupportHitProxies)
	{
		Flags.bHasPerInstanceEditorData = GIsEditor != 0 && InComponent.bHasPerInstanceHitProxies;
	}
#endif

	const USkinnedAsset* SkinnedAsset = InComponent.GetSkinnedAsset();
	const UTransformProviderData* TransformProvider = InComponent.GetTransformProvider();
	const bool bForceRefPose = UInstancedSkinnedMeshComponent::ShouldForceRefPose();
	const bool bValidTransformProvider = !bForceRefPose && TransformProvider != nullptr && TransformProvider->IsEnabled();
	TConstArrayView<FTransform3f> InstancePrevTransforms = InComponent.GetInstancePrevTransforms();

	Flags.bHasPerInstanceHierarchyOffset = false;
	Flags.bHasPerInstanceLocalBounds = TransformProvider ? TransformProvider->HasAnimationBounds() : false;
	const bool bHasCPUPrevTransforms = InstancePrevTransforms.Num() > 0 && InstancePrevTransforms.Num() == InComponent.GetInstanceCount();
	bool bHasAttachmentParent = false;
	if constexpr (std::is_same_v<T, UInstancedSkinnedMeshComponent>)
	{
		bHasAttachmentParent = InComponent.HasBoneAttachmentParents();
	}
	Flags.bHasPerInstanceDynamicData = bHasCPUPrevTransforms || bHasAttachmentParent;
	Flags.bHasPerInstanceSkinningData = true;

	Flags.bHasPerInstanceLMSMUVBias = false;//IsStaticLightingAllowed();

	ComponentDesc.Flags = Flags;

	const FTransform& ComponentTransform = InComponent.GetComponentTransform();
	const FVector3f Position(ComponentTransform.GetTranslation());

	ComponentDesc.MeshBounds = SkinnedAsset ? FRenderBounds(SkinnedAsset->GetBounds()) : FRenderBounds(Position, Position);
	ComponentDesc.NumCustomDataFloats = InComponent.NumCustomDataFloats;
	ComponentDesc.NumInstances = InComponent.InstanceData.Num();

	ComponentDesc.PrimitiveLocalToWorld = InComponent.GetRenderMatrix();
	ComponentDesc.ComponentMobility = InComponent.GetMobility();

	ComponentDesc.BuildChangeSet =
	[
		&InComponent,
		ComponentTransform,
		InstancePrevTransforms,
		TransformProvider = bValidTransformProvider ? TransformProvider : nullptr,
		MeshBounds = ComponentDesc.MeshBounds,
		bHasPerInstanceLocalBounds = Flags.bHasPerInstanceLocalBounds
	] (FInstanceUpdateChangeSet& ChangeSet)
	{
		// publish data
		ChangeSet.GetTransformWriter().Gather([&InComponent](int32 InstanceIndex) -> FRenderTransform { return FRenderTransform(InComponent.InstanceData[InstanceIndex].Transform.ToMatrixWithScale()); });

		if (InstancePrevTransforms.Num() == InComponent.GetInstanceCount())
		{
			ChangeSet.GetPrevTransformWriter().Gather([InstancePrevTransforms](int32 InstanceIndex) -> FRenderTransform { return FRenderTransform(InstancePrevTransforms[InstanceIndex].ToMatrixWithScale()); });
		}
		else if (ChangeSet.Flags.bHasPerInstanceDynamicData)
		{
			// No CPU previous transforms, but dynamic data is required (e.g., GPU-driven bone attachments).
			// Seed with current transforms so the first frame has no motion blur artifacts.
			ChangeSet.GetPrevTransformWriter().Gather([&InComponent](int32 InstanceIndex) -> FRenderTransform { return FRenderTransform(InComponent.InstanceData[InstanceIndex].Transform.ToMatrixWithScale()); });
		}
		ChangeSet.GetCustomDataWriter().Gather(MakeArrayView(InComponent.InstanceCustomData), InComponent.NumCustomDataFloats);

		if (TransformProvider)
		{
			ChangeSet.GetSkinningDataWriter().Gather(
				[&InComponent, TransformProvider, ComponentTransform](int32 InstanceIndex)->uint32
				{
					check(InComponent.InstanceData.IsValidIndex(InstanceIndex));
					const FSkinnedMeshInstanceData& Instance = InComponent.InstanceData[InstanceIndex];
					return TransformProvider->GetSkinningDataOffset(InstanceIndex, ComponentTransform, Instance);
				});
		}
		else
		{
			ChangeSet.GetSkinningDataWriter().Gather(0u);
		}

		if (TransformProvider && bHasPerInstanceLocalBounds)
		{
			ChangeSet.GetLocalBoundsWriter().Gather(
				[&InComponent, TransformProvider, MeshBounds](int32 InstanceIndex) -> FRenderBounds
				{
					const uint32 AnimationIndex = InComponent.InstanceData[InstanceIndex].AnimationIndex;
					FRenderBounds AnimationBounds;
					if (TransformProvider->GetAnimationBounds(AnimationIndex, AnimationBounds))
					{
						return AnimationBounds;
					}
					return MeshBounds;
				});
		}
		else
		{
			ChangeSet.GetLocalBoundsWriter().Gather(MeshBounds);
		}

#if WITH_EDITOR
		if constexpr (bSupportHitProxies)
		{
			if (ChangeSet.Flags.bHasPerInstanceEditorData)
			{
				// TODO: the way hit proxies are managed seems daft, why don't we just add them when needed and store them in an array alongside the instances?
				//       this will always force us to update all the hit proxy data for every instances.
				TArray<TRefCountPtr<HHitProxy>> HitProxies;
				InComponent.CreateHitProxyData(HitProxies);
				ChangeSet.SetEditorData(HitProxies, InComponent.SelectedInstances);
			}
		}
#endif


	};

	return ComponentDesc;
}

template <class T>
FBoxSphereBounds FInstancedSkinnedMeshComponentHelper::CalcBounds(const T& InComponent, const FTransform& LocalToWorld)
{
	// Helper to union parent attachment bounds for child-driven bone attachments.
	// Expands parent bounds by child mesh extent to account for geometry extending
	// beyond the attachment point (e.g., a rifle attached to a hand bone).
	auto UnionParentBounds = [&InComponent](FBoxSphereBounds InBounds) -> FBoxSphereBounds
	{
		if constexpr (std::is_same_v<T, UInstancedSkinnedMeshComponent>)
		{
			if (InComponent.BoneAttachmentSockets.Num() > 0)
			{
				const USkinnedAsset* ChildAsset = InComponent.GetSkinnedAsset();
				const float ChildMeshExtent = ChildAsset ? ChildAsset->GetBounds().SphereRadius : 0.0f;

				for (const auto& Socket : InComponent.BoneAttachmentSockets)
				{
					if (Socket.Parent.IsValid())
					{
						FBoxSphereBounds ExpandedParentBounds = Socket.Parent->Bounds;
						ExpandedParentBounds.SphereRadius += ChildMeshExtent;
						ExpandedParentBounds.BoxExtent += FVector(ChildMeshExtent);
						InBounds = InBounds + ExpandedParentBounds;
					}
				}
			}
		}
		return InBounds;
	};

	const USkinnedAsset* SkinnedAssetPtr = InComponent.GetSkinnedAsset();
	if (SkinnedAssetPtr && InComponent.InstanceData.Num() > 0)
	{
		const FMatrix BoundTransformMatrix = LocalToWorld.ToMatrixWithScale();

		FBoxSphereBounds::Builder BoundsBuilder;

		const bool bUseAnimationBounds = UInstancedSkinnedMeshComponent::ShouldUseAnimationBounds();
		const UTransformProviderData* TransformProvider = InComponent.GetTransformProvider();
		if (TransformProvider != nullptr && TransformProvider->IsEnabled() && TransformProvider->HasAnimationBounds() && !TransformProvider->IsCompiling())
		{
			const uint32 AnimationCount = TransformProvider->GetUniqueAnimationCount();

			// Trade per sequence bounds (tighter fitting) for faster builds with high instance counts.
			const bool bFastBuild = false;
			if (bFastBuild)
			{
				FBox MergedBounds;

				for (uint32 AnimationIndex = 0; AnimationIndex < AnimationCount; ++AnimationIndex)
				{
					FRenderBounds AnimationBounds;
					if (TransformProvider->GetAnimationBounds(AnimationIndex, AnimationBounds))
					{
						MergedBounds += AnimationBounds.ToBox();
					}
				}

				if (MergedBounds.IsValid)
				{
					for (int32 InstanceIndex = 0; InstanceIndex < InComponent.InstanceData.Num(); InstanceIndex++)
					{
						const FSkinnedMeshInstanceData& Instance = InComponent.InstanceData[InstanceIndex];
						BoundsBuilder += MergedBounds.TransformBy(FTransform(Instance.Transform) * LocalToWorld);
					}
				}
			}
			else
			{
				for (int32 InstanceIndex = 0; InstanceIndex < InComponent.InstanceData.Num(); InstanceIndex++)
				{
					const FSkinnedMeshInstanceData& Instance = InComponent.InstanceData[InstanceIndex];
					FRenderBounds AnimationBounds;
					if (TransformProvider->GetAnimationBounds(Instance.AnimationIndex, AnimationBounds))
					{
						BoundsBuilder += AnimationBounds.ToBox().TransformBy(FTransform(Instance.Transform) * LocalToWorld);
					}
				}
			}

			// Only use bounds if valid, otherwise use the skinned asset bounds in ref pose.
			if (BoundsBuilder.IsValid())
			{
				return UnionParentBounds(BoundsBuilder);
			}
		}

		const FBox InstanceBounds = SkinnedAssetPtr->GetBounds().GetBox();
		if (InstanceBounds.IsValid)
		{
			for (int32 InstanceIndex = 0; InstanceIndex < InComponent.InstanceData.Num(); InstanceIndex++)
			{
				BoundsBuilder += InstanceBounds.TransformBy(FTransform(InComponent.InstanceData[InstanceIndex].Transform) * LocalToWorld);
			}

			return UnionParentBounds(BoundsBuilder);
		}
	}

	return UnionParentBounds(InComponent.CalcMeshBound(FVector3f::ZeroVector, false, LocalToWorld));
}

template <class T>
FSkeletalMeshObject* FInstancedSkinnedMeshComponentHelper::CreateMeshObject(const T& InComponent, const FInstancedSkinnedMeshSceneProxyDesc& InSceneProxyDesc)
{
	return FInstancedSkinnedMeshSceneProxyDesc::CreateMeshObject(InSceneProxyDesc, FSkinnedMeshComponentHelper::GetSkeletalMeshRenderData(InComponent), InComponent.GetScene()->GetFeatureLevel());
}

template <class T>
bool FInstancedSkinnedMeshComponentHelper::IsEnabled(const T& InComponent)
{
	USkeletalMesh* SkeletalMeshPtr = Cast<USkeletalMesh>(InComponent.GetSkinnedAsset());
	if (SkeletalMeshPtr && SkeletalMeshPtr->GetResourceForRendering())
	{
		return InComponent.GetInstanceCount() > 0;
	}

	return false;
}

template <class T>
FPrimitiveSceneProxy* FInstancedSkinnedMeshComponentHelper::CreateSceneProxy(const T& InComponent, const FInstancedSkinnedMeshSceneProxyDesc& InSceneProxyDesc)
{
	const bool bShouldNaniteSkin = FSkinnedMeshComponentHelper::ShouldNaniteSkin(InComponent);
	const bool bEnabled = FInstancedSkinnedMeshComponentHelper::IsEnabled(InComponent);
	return FInstancedSkinnedMeshSceneProxyDesc::CreateSceneProxy(InSceneProxyDesc, InComponent.bHideSkin, bShouldNaniteSkin, bEnabled);
}
