// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkinnedMeshSceneProxyDesc.h"
#include "Animation/AnimBank.h"

class UInstancedSkinnedMeshComponent;

struct FInstancedSkinnedMeshSceneProxyDesc : public FSkinnedMeshSceneProxyDesc
{
	ENGINE_API static FSkeletalMeshObject* CreateMeshObject(const FInstancedSkinnedMeshSceneProxyDesc& Desc, FSkeletalMeshRenderData* InRenderData, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API static FPrimitiveSceneProxy* CreateSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin, bool bShouldNaniteSkin, bool bIsEnabled);

	UE_DEPRECATED(5.8, "MeshDesc now has MinLODLevel and InClampedLODIndex is unused")
	static FPrimitiveSceneProxy* CreateSceneProxy(const FInstancedSkinnedMeshSceneProxyDesc& Desc, bool bHideSkin, bool bShouldNaniteSkin, bool bIsEnabled, int32 InClampedLODIndex)
	{
		FInstancedSkinnedMeshSceneProxyDesc CopyDesc = Desc;
		CopyDesc.MinLODLevel = InClampedLODIndex;
		return CreateSceneProxy(CopyDesc, bHideSkin, bShouldNaniteSkin, bIsEnabled);
	}

	ENGINE_API FInstancedSkinnedMeshSceneProxyDesc(const UInstancedSkinnedMeshComponent* InComponent);
	FInstancedSkinnedMeshSceneProxyDesc() = default;

	ENGINE_API void InitializeFromInstancedSkinnedMeshComponent(const UInstancedSkinnedMeshComponent* InComponent);

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataSceneProxy;

	TObjectPtr<class UTransformProviderData> TransformProvider;

	// Properties
	float AnimationMinScreenSize = 0.0f;
	int32 InstanceMinDrawDistance = 0;
	int32 InstanceStartCullDistance = 0;
	int32 InstanceEndCullDistance = 0;

#if WITH_EDITOR
	/** One bit per instance if the instance is selected. */
	TBitArray<> SelectedInstances;
#endif

	bool bUseGpuLodSelection = true;
	bool bForceAnimateSockets = false;
};
