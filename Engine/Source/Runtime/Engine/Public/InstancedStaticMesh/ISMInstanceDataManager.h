// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneTypes.h"
#include "ISMInstanceDataSceneProxy.h"
#include "InstanceAttributeTracker.h"

class FPrimitiveSceneProxy;
class HHitProxy;

/**
 * Data descriptor representing the component state abstracting the UPrimitiveComponent, needs to be passed into the change flushing.
 * The intention is to decouple the manager from the UComponent or any other supplier of instance data & scene proxies.
 */
struct FInstanceUpdateComponentDesc
{
	FMatrix PrimitiveLocalToWorld;
	EComponentMobility::Type ComponentMobility = EComponentMobility::Movable;
	FRenderBounds StaticMeshBounds;
	FInstanceDataFlags Flags;
	FPrimitiveMaterialPropertyDescriptor PrimitiveMaterialDesc;

	// The number of instances that will be allocated in the proxy
	int32 NumProxyInstances = -1;
	// Number of instances in the source (e.g., the component)
	int32 NumSourceInstances = -1;
	// 
	int32 NumCustomDataFloats = 0;

	// Callback to fill in the required change set use with delta-update capable onwing components.
	// TODO: move elsewhere?
	TFunction<void(FISMInstanceUpdateChangeSet &ChangeSet)> BuildChangeSet;
};

/**
 * Manager class that tracks changes to instance data within the component, and is responsible for dispatching updates of the proxy.
 * Tracks instance index changes to be able to maintain a persistent ID mapping for use on the render thread.
 * The ID mapping is not serialized and will be reset when the proxy is recreated.
 * Not responsible for storing the component representation of the instance data.
 * NOTE/TODO: This is tied to the ISM use-case, mostly because of legacy (HISM) interactions. Will be refactored and sub-classed or something.
 *            Also: Still somewhat tied to the UComponent, which also can be refactored a bit to make it more general.
 */
class FPrimitiveInstanceDataManager : public FInstanceIdIndexMap
{
public:
	ENGINE_API FPrimitiveInstanceDataManager(UPrimitiveComponent* InPrimitiveComponent);

	/**
	 * Current tracking state, 
	 */
	enum class ETrackingState : uint8
	{
		Initial, // In the initial state, there is no proxy and therefore changes do not need to be tracked, e.g., during initial setup of an ISM component.
		Tracked,
		Disabled,
		Optimized, // In the optimized state there's no need to track any delta changes, but if anything changes at all we must rebuild.
	};

	enum class EMode : uint8
	{
		Default,
		Legacy, // In this mode, we create a legacy supporting proxy.
		ExternalLegacyData // In this mode, it is illegal to call the incremental state tracking methods & updates can only be sent to the proxy if there has been and extranal one queued.
	};

	/**
	 */
	ENGINE_API void SetMode(EMode InMode);

	/**
	 */
	EMode GetMode() const { return Mode; }

	void Add(int32 InInstanceAddAtIndex, bool bInsert);

	void RemoveAtSwap(int32 InstanceIndex);
	
	void RemoveAt(int32 InstanceIndex);

	void TransformChanged(int32 InstanceIndex);
	void TransformChanged(FPrimitiveInstanceId InstanceId);
	
	void TransformsChangedAll();

	void CustomDataChanged(int32 InstanceIndex);

	void BakedLightingDataChanged(int32 InstanceIndex);

	void BakedLightingDataChangedAll();

	void NumCustomDataChanged();

#if WITH_EDITOR
	void EditorDataChangedAll();
#endif

	void PrimitiveTransformChanged();

	ENGINE_API bool HasAnyInstanceChanges() const;


#if USE_NULL_RHI
	/**
	 * Stopgap fix for server, where the manager reporting that it has changes causes (inadvertent) clearing of the instances in the HISM.
	 * TODO: Remove when proper fix (not clearing instances) is implemented.
	 */
	inline bool HasAnyChanges() const { return false;}
#else
	/** 
	 * Returns true if there are explicitly tracked instance changes, or the state is not tracked (because no proxy has been created yet),
	 * and the tracking state is not Disabled.
	 */
	inline bool HasAnyChanges() const { return GetState() != ETrackingState::Disabled && (GetState() != ETrackingState::Tracked || HasAnyInstanceChanges());}
#endif
	bool FlushChanges(FInstanceUpdateComponentDesc &&ComponentData, bool bNewPrimitiveProxy);

	// 
	void PostLoad(int32 InNumInstances);

	/**
	 * Clear the ID/Index association and reset the mapping to identity & number of instances to the given number.
	 * Also clears the change tracking state.
	 */
	void ClearIdTracking(int32 InNumInstances);

	/**
	 * Clear all tracked changes (will result in a full update when next one is flushed)
	 */
	void ClearChangeTracking();

	int32 GetMaxInstanceId() const;

	int32 GetMaxInstanceIndex() const;

	void CreateExplicitIdentityMapping();

	ETrackingState GetState() const { return TrackingState; }

	ENGINE_API TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe> GetOrCreateProxy(ERHIFeatureLevel::Type InFeatureLevel);
	const TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe>& GetProxy() const { return Proxy; }
	void Invalidate(int32 InNumInstances);
	
#if DO_GUARD_SLOW
	void ValidateMapping() const;
#else
	inline void ValidateMapping() const {};
#endif

	/**
	 * Call to mark the manager as needing a full rebuild & having an external driver for this.
	 */
	ENGINE_API void MarkForRebuildFromLegacy(TUniquePtr<FStaticMeshInstanceData> &&LegacyInstanceData, const TArray<int32> &InstanceReorderTable, const TArray<TRefCountPtr<HHitProxy>> &HitProxies);

	/**
	 */
	ENGINE_API SIZE_T GetAllocatedSize() const;


	inline void ResetComponentDirtyTracking() { bComponentMarkedDirty = false; }

	/**
	 * Called by the corresponding function in the owner UPrimitiveComponent
	 * Because of the multifarious ways the engine shoves data into the properties it is possible for the count to get out of sync. Also at this point we may assume that we have no idea of the state of individual members.
	 * Thus, this function will reset tracking state to force a full update as well as, conditionally - if the counts are mismatched - reset the ID mapping.
	 */
	void OnRegister(int32 InNumInstances);

#if WITH_EDITOR
	// Early out test to save performance if no features are enabled that require BeginCacheForCookedPlatformData to be called
	bool ShouldWriteCookedData(const ITargetPlatform* TargetPlatform, int32 NumInstancesToBuildFor);

	void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform, FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms);

	void WriteCookedRenderData(FArchive& Ar, FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms);
#endif
	void ReadCookedRenderData(FArchive& Ar);

private:
	static bool ShouldUsePrecomputed();

	using EChangeFlag = FInstanceAttributeTracker::EFlag;

	template<EChangeFlag Flag>
	inline void MarkChangeHelper(int32 InstanceIndex);
	template<EChangeFlag Flag>
	inline void MarkChangeHelper(FPrimitiveInstanceId InstanceId);

	void MarkComponentRenderInstancesDirty();

	bool HasIdentityMapping() const;

	void FreeInstanceId(FPrimitiveInstanceId InstanceId);

	void InitChangeSet(const union FChangeDesc &ChangeDesc, const FInstanceUpdateComponentDesc &ComponentData, FISMInstanceUpdateChangeSet &ChangeSet);

#if WITH_EDITOR
	/**
	 * Build precomputed data from the input.
	 */
	static FISMPrecomputedSpatialHashData PrecomputeOptimizationData(FInstanceUpdateComponentDesc &&ComponentData, TStridedView<FMatrix> InstanceTransforms);
#endif

	// Change set.
	FInstanceAttributeTracker InstanceUpdateTracker;

	// Id allocation tracking
	TBitArray<> ValidInstanceIdMask;
	int32 IdSearchStartIndex = 0;

	EMode Mode = EMode::Default;
	ETrackingState TrackingState = ETrackingState::Initial;

	uint8 bNumCustomDataChanged : 1;
	uint8 bBakedLightingDataChanged : 1;
	uint8 bTransformChangedAllInstances : 1;
#if WITH_EDITOR
	uint8 bAnyEditorDataChanged : 1;
#endif	
	uint8 bPrimitiveTransformChanged : 1;
	uint8 bAnyInstanceChange : 1;

	uint8 bComponentMarkedDirty : 1;
	uint8 bEnableTracking : 1;
	uint8 bFirstFlush : 1;

	TSharedPtr<FISMCInstanceDataSceneProxy, ESPMode::ThreadSafe> Proxy;
	TWeakObjectPtr<UPrimitiveComponent> PrimitiveComponent = nullptr;

	TPimplPtr<struct FLegacyBuildData> LegacyBuildData;

	using PrecomputedOptimizationDataPtr = FISMCInstanceDataSceneProxy::FISMPrecomputedSpatialHashDataPtr;

	PrecomputedOptimizationDataPtr PrecomputedOptimizationData;

	// Must track this to detect changes 
	// TODO: make event driven and save the storage?
	FMatrix PrimitiveLocalToWorld;
	FInstanceDataFlags Flags;
	int32 NumCustomDataFloats = 0;
	float AbsMaxDisplacement = 0.0f;
	FRenderBounds StaticMeshBounds;
};
