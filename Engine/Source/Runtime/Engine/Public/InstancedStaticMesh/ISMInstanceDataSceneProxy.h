// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceDataSceneProxy.h"
#include "Containers/StridedView.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"

class FOpaqueHitProxyContainer;
class FStaticMeshInstanceBuffer;
class FStaticMeshInstanceData;
class FISMInstanceUpdateChangeSet;
class HHitProxy;

DECLARE_LOG_CATEGORY_EXTERN(LogInstanceProxy, Log, All)

/**
 * Precomputed optimization data that descrives the spatial hashes and reordering needed.
 */
class FPrecomputedInstanceSpatialHashData;
using FISMPrecomputedSpatialHashData = FPrecomputedInstanceSpatialHashData;

/**
 * Proxy class that represents scene instance data to the renderer.
 * Responsible for preparing data for render use per-platform & serializing such data for cooked builds.
 * Supplies a persistent ID mapping for use in FScene. The ID mapping is reset when the proxy is recreated. 
 * Thus the FScene will generate full remove/add handling for proxy recreate events.
 * Not responsible for storing the component representation of the instance data.
 * 
 * TODO: Add concept of async update and needing to sync the proxy before use. 
 *       Probably split between immutable data (e.g, flags and stuff that don't depend on a data conversion process) and the bulk of the data.
 *       The immutable part is always valid & up to date on the RT (set in command, or before it is added).
 */
class FISMCInstanceDataSceneProxy : public FInstanceDataSceneProxy
{
public:
	FISMCInstanceDataSceneProxy(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel);

	inline const FInstanceSceneDataBuffers& GetData() const { return InstanceSceneDataBuffers; }

	//void BuildLegacyData();

	/**
	 * Overridable functions to update / build proxy data from a change set. 
	 */
	virtual void Update(FISMInstanceUpdateChangeSet&& ChangeSet);
	virtual void Build(FISMInstanceUpdateChangeSet&& ChangeSet);

	virtual void BuildFromLegacyData(TUniquePtr<FStaticMeshInstanceData> &&ExternalLegacyData, const FRenderBounds &InstanceLocalBounds, TArray<int32> &&InLegacyInstanceReorderTable) {};

	/**
	 * Handle only updating the primitive transform, could make use of special cases such as translation only if implemented properly.
	 */
	virtual void UpdatePrimitiveTransform(FISMInstanceUpdateChangeSet&& ChangeSet) { check(false); }

	/**
	 * Make sure the shader platform & feature levels match.
	 */
	inline bool CheckPlatformFeatureLevel(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel) { return ShaderPlatform == InShaderPlatform && InFeatureLevel == FeatureLevel; }

	virtual FInstanceDataUpdateTaskInfo *GetUpdateTaskInfo() { return &InstanceDataUpdateTaskInfo; }

	ENGINE_API void DebugDrawInstanceChanges(FPrimitiveDrawInterface* DebugPDI, ESceneDepthPriorityGroup SceneDepthPriorityGroup);

	ENGINE_API static FVector3f GetLocalBoundsPadExtent(const FRenderTransform& LocalToWorld, float PadAmount);

	using FISMPrecomputedSpatialHashDataPtr = TSharedPtr<const FISMPrecomputedSpatialHashData, ESPMode::ThreadSafe>;
protected:
	/**
	 * Build an optimized instance buffer, where the order is sorted such that instances with the same spatial hash are consecutive.
	 */
	void BuildFromOptimizedDataBuffers(FISMInstanceUpdateChangeSet& ChangeSet, FInstanceIdIndexMap &OutInstanceIdIndexMap, FInstanceSceneDataBuffers::FWriteView &OutData);

	// Update the InstanceIdIndexMap given the change set.
	template <typename IndexRemapType>
	void UpdateIdMapping(FISMInstanceUpdateChangeSet& ChangeSet, const IndexRemapType &IndexRemap);

	template <typename IndexRemapType>
	void ApplyDataChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, int32 PostUpdateNumInstances, FInstanceSceneDataBuffers::FWriteView &ProxyData);
	
	template <typename IndexRemapType>
	void ApplyAttributeChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, FInstanceSceneDataBuffers::FWriteView &ProxyData);

	void TestAndApplyInstanceBufferSizeFixup(int32 PostUpdateNumInstances);

	FStaticShaderPlatform ShaderPlatform;
	ERHIFeatureLevel::Type FeatureLevel;
	bool bUseLegacyRenderingPath = false;


	// Id allocation tracking
	TBitArray<> ValidInstanceIdMask; // redundant, unsure if we need a copy of that, except insofar as to be able to scan the valid ones quicky? IdToIndexMap has the same info.
	FInstanceIdIndexMap InstanceIdIndexMap;

	FInstanceDataUpdateTaskInfo InstanceDataUpdateTaskInfo;
	// True when it has never been updated before.
	bool bIsNew = true;
	// This should be set when constructing for a static primitive that wants pre-built instances.
	bool bBuildOptimized = false;
#if WITH_EDITOR
	/**Container for hitproxies that are used by the instances, uses the FDeferredCleanupInterface machinery to delete itself back on the game thread when replaced. */
	TPimplPtr<FOpaqueHitProxyContainer> HitProxyContainer;
#endif

	friend class FPrimitiveInstanceDataManager;

	FISMPrecomputedSpatialHashDataPtr PrecomputedOptimizationData;
};

/**
 * Proxy that supports legacy reordered (HISM) data management.
 */
class FISMCInstanceDataSceneProxyLegacyReordered : public FISMCInstanceDataSceneProxy
{
public:
	FISMCInstanceDataSceneProxyLegacyReordered(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, bool bInLegacyReordered);

	virtual void Update(FISMInstanceUpdateChangeSet&& ChangeSet) override;
	virtual void Build(FISMInstanceUpdateChangeSet&& ChangeSet) override;
	virtual void BuildFromLegacyData(TUniquePtr<FStaticMeshInstanceData> &&InExternalLegacyData, const FRenderBounds &InstanceLocalBounds, TArray<int32> &&InLegacyInstanceReorderTable) override;
	virtual void UpdatePrimitiveTransform(FISMInstanceUpdateChangeSet&& ChangeSet) override;

protected:
	void UpdateInstancesTransforms(FInstanceSceneDataBuffers::FWriteView &ProxyData, const FStaticMeshInstanceData &LegacyInstanceData);

	TArray<int32> LegacyInstanceReorderTable;
	bool bLegacyReordered = false;
	// Must hang on to this to be able to suppor primitive transform changes, otherwise we have no access to the original instance transforms.
	// We could potentially pull out and store those explicitly instead as we don't need to retain all the other garbage.
	TUniquePtr<FStaticMeshInstanceData> ExternalLegacyData;
};


/**
 * Proxy that supports legacy NoGPUScene data management (and HISM).
 */
class FISMCInstanceDataSceneProxyNoGPUScene : public FISMCInstanceDataSceneProxyLegacyReordered
{
public:
	FISMCInstanceDataSceneProxyNoGPUScene(FStaticShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, bool bInLegacyReordered);
	~FISMCInstanceDataSceneProxyNoGPUScene();

	/**
	 * Helper to pass the unique reference to LegacyInstanceBuffer to the render thread and call ReleaseResource() to clean up vertex buffers.
	 */
	void ReleaseStaticMeshInstanceBuffer();

	virtual void Update(FISMInstanceUpdateChangeSet&& ChangeSet) override;
	virtual void Build(FISMInstanceUpdateChangeSet&& ChangeSet) override;
	virtual void BuildFromLegacyData(TUniquePtr<FStaticMeshInstanceData> &&ExternalLegacyData, const FRenderBounds &InstanceLocalBounds, TArray<int32> &&InLegacyInstanceReorderTable) override;
	virtual void UpdatePrimitiveTransform(FISMInstanceUpdateChangeSet&& ChangeSet) override;

	template <typename IndexRemapType>
	void ApplyDataChanges(FISMInstanceUpdateChangeSet &ChangeSet, const IndexRemapType &IndexRemap, int32 PostUpdateNumInstances, FInstanceSceneDataBuffers::FWriteView &ProxyData, FStaticMeshInstanceData &LegacyInstanceData);

	/**
	 */
	virtual ENGINE_API FStaticMeshInstanceBuffer* GetLegacyInstanceBuffer() override;


	TUniquePtr<FStaticMeshInstanceBuffer> LegacyInstanceBuffer;
};
