// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceDataSceneProxy.h"
#include "Containers/StridedView.h"
#include "InstanceData/InstanceUpdateChangeSet.h"

/**
 * Extends the change set with ISM specifics.
 */
class FISMInstanceUpdateChangeSet : public FInstanceUpdateChangeSet
{
public:
	// Upcycle constructors
	using FInstanceUpdateChangeSet::FInstanceUpdateChangeSet;

	FInstanceAttributeTracker::FDeltaRange<FInstanceAttributeTracker::EFlag::CustomDataChanged> GetCustomDataDelta() const 
	{ 
		// Force empty range if no custom data
		return GetDelta<FInstanceAttributeTracker::EFlag::CustomDataChanged>(NumCustomDataFloats == 0 || !Flags.bHasPerInstanceCustomData, ForceFullFlags.bHasPerInstanceCustomData);
	}

	FIdentityDeltaRange GetInstanceLightShadowUVBiasDelta() const
	{
		return FIdentityDeltaRange(InstanceLightShadowUVBias.Num());
	}

#if WITH_EDITOR
	FIdentityDeltaRange GetInstanceEditorDataDelta() const
	{
		return FIdentityDeltaRange(InstanceEditorData.Num());
	}
#endif

	/**
	 * Add a value, must be done in the order represented in the InstanceLightShadowUVBiasDelta.
	 */
	inline void AddInstanceLightShadowUVBias(const FVector4f &Value)
	{
		InstanceLightShadowUVBias.Emplace(Value); 
	}

	ENGINE_API void SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms, const FVector Offset);
	ENGINE_API void SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms);

	/**
	 * This version produces the bounds of the gathered transforms as a side-effect.
	 */
	ENGINE_API void SetInstanceTransforms(TStridedView<FMatrix> InInstanceTransforms, FBox const& InInstanceBounds, FBox& OutGatheredBounds);

	ENGINE_API void SetInstancePrevTransforms(TArrayView<FMatrix> InPrevInstanceTransforms, const FVector &Offset);
	ENGINE_API void SetInstancePrevTransforms(TArrayView<FMatrix> InPrevInstanceTransforms);
	ENGINE_API void SetCustomData(const TArrayView<const float> &InPerInstanceCustomData, int32 InNumCustomDataFloats);
	
	UE_DEPRECATED(5.6, "Use SetSharedLocalBounds instead")
	void SetInstanceLocalBounds(const FRenderBounds &Bounds) { SetSharedLocalBounds(Bounds); }
	TArray<int32> LegacyInstanceReorderTable;
	int32 PostUpdateNumInstances = 0;
};
