// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/BitArray.h"
#include "Containers/Array.h"
#include "InstanceDataTypes.h"

// Implementation details
namespace UE::InstanceAttributeTracker::Implementation
{
template <typename ElementType>
struct FBitManip
{
	// Utility to, at compile time, replicate the given bit mask 'Mask' as many times as MaskSize inside sizeof(ElementType)*8.
	// e.g. turning Mask=0x1 into 0x11111111 if MaskSize=4 & ElementType is uint32
	template <ElementType Mask, uint32 MaskSize>
	struct FRepMask
	{
		template <uint32 Step>
		struct FIterator;
		template <>
		struct FIterator<0u>
		{
			static constexpr ElementType Result = Mask;
		};
		template <uint32 Step>
		struct FIterator
		{
			static constexpr ElementType Result = (Mask << (Step * MaskSize)) | FIterator<Step - 1u>::Result;
			
		};

		static constexpr uint32 NumSteps = (sizeof(ElementType) * 8u / MaskSize) - 1u;
		static constexpr ElementType Result = FIterator<NumSteps>::Result;
	};
	/**
	 * Make it possible to select the function based on the overloaded type for 32/64-bit use.
	 */
	static inline uint32 CountTrailingZeros(ElementType Value);
};


template <>
inline uint32 FBitManip<uint32>::CountTrailingZeros(uint32 Value)  
{ 
	return FMath::CountTrailingZeros(Value); 
}
template <>
inline uint32 FBitManip<uint64>::CountTrailingZeros(uint64 Value)  
{ 
	return uint32(FMath::CountTrailingZeros64(Value)); 
}


} // namespace UE::InstanceAttributeTracker::Implementation

/**
 * Tracks changes for instances, but that assumes storing stuff per index except the removed status which must be kept ID-based
 * Removed items are tracked by ID and kept separate as we need to remember if an item was ever removed in order to handle these correctly.
 * Designed to use move semantics to clear the state and transfer to the updating worker task.
 */
class FInstanceAttributeTracker
{
public:
	// Underlying type to store per-instance masks packed in. For example, with EFlag::Num == 4, and uint32 we can store 8 masks in each element.
	using ElementType = uint32;

	using FBitManip = UE::InstanceAttributeTracker::Implementation::FBitManip<ElementType>;

	/**
	 * Flag describing the changed attribute. 
	 * Note that not all are created equal, see comments.
	 */
	enum class EFlag : ElementType
	{
		Added, /** Set when */
		TransformChanged,
		CustomDataChanged,
		SkinningDataChanged,
		IndexChanged, /** Implicitly set when calling Remove or RemoveAt as this causes movement to fill holes. */
		Num,
	};

	static constexpr uint32 BitsPerElement = sizeof(ElementType) * 8u;
	static constexpr int32 MasksPerElement = BitsPerElement / uint32(EFlag::Num);
	static constexpr ElementType AnyFlagMask = (ElementType(1U) << uint32(EFlag::Num)) - ElementType(1U);
	static constexpr uint32 PaddingBits = BitsPerElement - MasksPerElement * uint32(EFlag::Num);

	// Helper struct to convert the flag to its bitmask at comile time.
	template <EFlag Flag>
	struct FToBit
	{
		static constexpr ElementType Bit = ElementType(1U) << uint32(Flag);
	};

	/**
	 */
	FInstanceAttributeTracker();

	/**
	 */
	FInstanceAttributeTracker(FInstanceAttributeTracker &&Other);

	/**
	 */
	void operator=(FInstanceAttributeTracker &&Other);

	/**
	 */
	void Reset();
	
	/**
	 */
	static void Move(FInstanceAttributeTracker &Dest, FInstanceAttributeTracker &Source);

	/**
	 * Get an iterator that iterates over the IDs (note, not index as they don't exist any more) of removed instances. The remove flag is not cleared when a new instances is added.
	 */
	TConstSetBitIterator<> GetRemovedIterator() const;

	/**
	 * Lazy allocate bits in the bit vector tracking removed instances and set the flag for the instance ID.
	 * Also updates the FirstRemovedIdIndex, such that we can start iteration from that bit.
	 */
	inline void MarkRemoved(FPrimitiveInstanceId Id)
	{
		FirstRemovedIdIndex = FMath::Min(FirstRemovedIdIndex, Id.GetAsIndex());
		// Lazy allocate the memory so there is nothing to move around in case nothing is removed
		if (Id.GetAsIndex() >= Removed.Num())
		{
			// TBitArray does a lot of work for each added bit, first clearing the slack redundantly and then also clearing the bit even though it was already cleared because the slack was cleared...
			// so we pad the allocation request with at least a dword at a time.
			int32 RemovedPadded = FMath::DivideAndRoundUp(Id.GetAsIndex() + 1, 32) * 32;
			Removed.SetNum(RemovedPadded, false);
		}
		Removed[Id.GetAsIndex()] = true;
	}


	/**
	 * Mark removed & update the tracked bits (remove & swap operation)
	 */
	inline void RemoveAtSwap(FPrimitiveInstanceId Id, int32 Index, int32 MaxInstanceIndex)
	{
		Validate();
		
		SetNum(FMath::Max(Index, MaxInstanceIndex));

		MarkRemoved(Id);

		// book-keeping
		DecChangedCounts(Index);

		// Remove the tracked state bits
		int32 LastIndex = MaxIndex - 1;
		if (Index != LastIndex)
		{
			// Move tracked & tag as moved.
			ElementType PrevFlags = GetFlags(LastIndex);
			if ((((FToBit<EFlag::IndexChanged>::Bit | FToBit<EFlag::Added>::Bit)) & PrevFlags) == 0u)
			{
				check(NumChanged[uint32(EFlag::IndexChanged)] >= 0);
				NumChanged[uint32(EFlag::IndexChanged)] += 1;
			}
			SetMaskTo(Index, FToBit<EFlag::IndexChanged>::Bit | PrevFlags);
			FirstChangedIndex = FMath::Min(FirstChangedIndex, Index);
			LastChangedIndex = FMath::Max(LastChangedIndex, Index);
		}
		// clear the last one so that it is all zero after the last mask (important for when we do implicit add, as we do at the moment).
		ClearLastElement();
		MaxIndex -= 1;

		Validate();
	}

	/**
	 * Mark removed & update the tracked bits (remove & move operation)
	 * Note: this is not something that should be used really as it forces the tracker to touch all the masks after the removed item...
	 */
	inline void RemoveAt(FPrimitiveInstanceId Id, int32 Index, int32 MaxInstanceIndex)
	{
		Validate();

		SetNum(FMath::Max(Index, MaxInstanceIndex));

		MarkRemoved(Id);

		DecChangedCounts(Index);

		// Remove the tracked state bits
		int32 LastIndex = MaxIndex - 1;
		if (Index != LastIndex)
		{			
			int32 FirstElementIndex = Index / MasksPerElement;
			int32 LastElementIndex = LastIndex / MasksPerElement;

			ElementType CurrentElement = Data[FirstElementIndex];
			ElementType NextElement = LoadCountAndMask(FirstElementIndex + 1, FirstElementIndex != LastElementIndex);
			// 1. Move up to "Num" bits to cover the removed item, shifting in zero.
			{
				uint32 ElementSubIndex = (Index % MasksPerElement);
				// Count mask changes only in the moved part (not including the removed or any previous).
				uint32 NumMasksInElement = FMath::Min(MasksPerElement, MaxIndex - FirstElementIndex * MasksPerElement);
				CountMaskChanges(CurrentElement, ElementSubIndex + 1, NumMasksInElement);
				ElementType CurrentElementResult = (CurrentElement >> uint32(EFlag::Num)) | IndexChangedElementMask;
				if (ElementSubIndex != 0u)
				{
					uint32 ElementSubShift = ElementSubIndex * uint32(EFlag::Num);
					ElementType Mask = ~ElementType(0) << ElementSubShift;
					// we shifted some as we shouldn't, put them back
					CurrentElementResult = (CurrentElementResult & Mask) | (CurrentElement & (~Mask));
				}
				// Move the first item from the next step
				CurrentElementResult |= NextElement << (MasksPerElement - 1u) * uint32(EFlag::Num);
				Data[FirstElementIndex] = CurrentElementResult;
			}
			// 2. Do the middle, whole, elements...
			for (int32 ElementIndex = FirstElementIndex + 1; ElementIndex < LastElementIndex; ++ElementIndex)
			{
				CurrentElement = NextElement >> uint32(EFlag::Num);
				NextElement = LoadCountAndMask(ElementIndex + 1, true);
				ElementType CurrentElementResult = CurrentElement | NextElement << (MasksPerElement - 1u) * uint32(EFlag::Num);
				Data[ElementIndex] = CurrentElementResult;
			}
			// 2. Do the last element...
			if (LastElementIndex != FirstElementIndex)
			{
				Data[LastElementIndex] = NextElement >> uint32(EFlag::Num);
			}

			FirstChangedIndex = FMath::Min(FirstChangedIndex, Index);
			LastChangedIndex = FMath::Max(LastChangedIndex, LastIndex);
		}	
		ClearLastElement();
		MaxIndex -= 1;

		Validate();
	}

	/**
	 */
	template <EFlag Flag>
	inline void MarkIndex(int32 Index, int32 MaxInstanceIndex)
	{
		Validate();
		
		// Lazy allocate
		SetNum(FMath::Max(Index, MaxInstanceIndex));

		check(Index < MaxIndex);

		FirstChangedIndex = FMath::Min(FirstChangedIndex, Index);
		LastChangedIndex = FMath::Max(LastChangedIndex, Index);

		uint32 ElementSubShift = (Index % MasksPerElement) * uint32(EFlag::Num);

		ElementType SetMask = FToBit<Flag>::Bit << ElementSubShift;

		int32 ElementIndex = Index / MasksPerElement;

		ElementType PrevElement = Data[ElementIndex];

		if ((((FToBit<Flag>::Bit | FToBit<EFlag::Added>::Bit) << ElementSubShift) & PrevElement) == 0u)
		{
			check(NumChanged[uint32(Flag)] >= 0);
			NumChanged[uint32(Flag)] += 1;
		}

		Data[ElementIndex] = PrevElement | SetMask;

		Validate();
	}

	/**
	 * Get the masked flags for a given index.
	 */
	inline ElementType GetFlags(int32 Index) const
	{
		check(Index < MaxIndex);
		ElementType Element = Data[Index / MasksPerElement];
		uint32 ElementSubShift = (Index % MasksPerElement) * uint32(EFlag::Num);
		return (Element >> ElementSubShift) & AnyFlagMask;
	}

	/**
	 * Test a specific flag at a given index.
	 */
	template <EFlag Flag>
	inline bool TestFlag(int32 Index) const
	{
		return (GetFlags(Index) & FToBit<Flag>::Bit) != 0u;
	}

	/**
	 */
	inline void SetNum(int32 InstanceIndexMax)
	{
		Validate();

		int32 ElementIndex = FMath::DivideAndRoundUp(InstanceIndexMax, int32(MasksPerElement));

		if (ElementIndex > Data.Num())
		{
			Data.SetNumZeroed(ElementIndex);
		}
		MaxIndex = InstanceIndexMax;
		Validate();
	}

	/**
	 * Iterator for iterating tracked state with the given flags. Uses bit logic to efficiently skip empty (under the given mask) sections.
	 */
	template <uint32 Mask = AnyFlagMask>
	class FAnyValidIterator
	{
	public:
		inline FAnyValidIterator() = default;
		inline FAnyValidIterator(const FInstanceAttributeTracker *InTracker, int32 StartIndex) : Tracker(InTracker)
		{
			ElementIndex = StartIndex / MasksPerElement;
			ElementOffset = StartIndex % MasksPerElement;
			CurrentElement = Tracker->Data.IsValidIndex(ElementIndex) ? (Tracker->Data[ElementIndex] & ElementMask) : 0;
			CurrentElement >>= uint32(EFlag::Num) * ElementOffset;
			AdvanceUntilAny();
		}

		inline void AdvanceUntilAny()
		{
			while(ElementIndex < Tracker->Data.Num())
			{
				// Skip the whole word if nothing is set.
				if (CurrentElement)
				{
					uint32 ZeroCount = FBitManip::CountTrailingZeros(CurrentElement);
					uint32 Steps = ZeroCount / uint32(EFlag::Num);
					CurrentElement >>= uint32(EFlag::Num) * Steps;
					ElementOffset += Steps;
					return;
				}
				++ElementIndex;
				ElementOffset = 0;
				if (ElementIndex >= Tracker->Data.Num())
				{
					CurrentElement = 0u;
					return;
				}
				CurrentElement = Tracker->Data[ElementIndex] & ElementMask;
			}
		}

		inline void AdvanceToNext()
		{
			// Clear the current value (which will cause AdvanceUntilAny to move forwards)
			CurrentElement &= ~ElementType(Mask);
			AdvanceUntilAny();
		}

		inline FAnyValidIterator &operator++()
		{
			AdvanceToNext();
			return *this;
		}

		inline explicit operator bool() const
		{
			return GetIndex() < FMath::Min(Tracker->MaxIndex, Tracker->LastChangedIndex + 1);
		}

		inline int32 GetIndex() const
		{
			return ElementIndex * MasksPerElement + ElementOffset;
		}

		/* 
		 * Returns the mask for the current item
		 * NOTE: does not clear any bits outside the search mask.
		 */
		inline uint32 GetMask() const
		{
			return CurrentElement;
		}

		template <EFlag Flag>
		inline bool TestFlag() const
		{
			return (CurrentElement & FToBit<Flag>::Bit) != 0u;
		}
	private:
		const FInstanceAttributeTracker *Tracker = nullptr;
		int32 ElementIndex = 0;
		ElementType CurrentElement;
		int32 ElementOffset = 0;
		static constexpr ElementType ElementMask = FBitManip::FRepMask<Mask, uint32(EFlag::Num)>::Result;
	};

	template <uint32 Mask = AnyFlagMask>
	inline FAnyValidIterator<Mask> GetChangedIterator() const
	{
		return FAnyValidIterator<Mask>(this, FMath::Min(MaxIndex, FirstChangedIndex));
	}

#if DO_GUARD_SLOW
	ENGINE_API void Validate() const;
#else
	inline void Validate() const { }
#endif

	/**
	 * Represents the delta range for a given attribute, can either refer to the tracked data or just a span, or be empty.
	 * "Added" is always implied & added on top. It is a helper to make it easier to iterate a change set.
	 */
	template <EFlag Flag>
	class FDeltaRange
	{
	public:
		static constexpr uint32 IteratorMask = FToBit<Flag>::Bit | FToBit<EFlag::Added>::Bit;

		inline FDeltaRange() = default;

		inline FDeltaRange(bool bInFullUpdate, int32 InNumItems, const FInstanceAttributeTracker *InInstanceUpdateTracker)
			: bFullUpdate(bInFullUpdate)
			, NumItems(InNumItems)
			, Tracker(InInstanceUpdateTracker)
		{
			int32 NumAdded = Tracker->NumChanged[uint32(EFlag::Added)];
			bFullUpdate = bFullUpdate || (NumAdded + Tracker->NumChanged[uint32(Flag)] == NumItems);
			if (!bFullUpdate)
			{
				NumItems = NumAdded + Tracker->NumChanged[uint32(Flag)];
			}
		}

		/**
		 */
		inline bool IsEmpty() const { return NumItems == 0; }

		/**
		 */
		inline bool IsDelta() const { return !bFullUpdate; }

		/**
		 * Returns the number of items in this range - i.e., the number of items that need to be copied to collect an update.
		 */
		inline int32 GetNumItems() const { return NumItems; }

		/**
		 * Iterator for traversing the range of Items that need update. Provides two indexes:
		 * 1. The sparse Index of the source & destination data
		 * 2. The continuous Item Index that represents the linear index into the copied data.
		 */
		struct FConstIterator
		{
			bool bUseIterator = true;
			int32 ItemIndex = 0;
			int32 MaxNum = 0;
			FAnyValidIterator<IteratorMask> It;

			inline FConstIterator(int32 InIndex, int32 InMaxNum)
			:	bUseIterator(false),
				ItemIndex(InIndex),
				MaxNum(InMaxNum)
			{
			}

			inline FConstIterator(FAnyValidIterator<IteratorMask> &&InIt)
			:	bUseIterator(true),
				ItemIndex(0),
				MaxNum(),
				It(MoveTemp(InIt))
			{
			}

			inline void operator++() 
			{ 
				++ItemIndex;
				if (bUseIterator)
				{
					++It;
				}
			}

			/**
			 * Get the index of the data in the source / destination arrays. 
			 */
			inline int32 GetIndex() const 
			{ 
				if (bUseIterator)
				{
					return It.GetIndex();
				}
				return ItemIndex;
			}

			/**
			 * Get the continuous index of the data item in the collected item array.
			 */
			inline int32 GetItemIndex() const
			{
				return ItemIndex;
			}

			inline explicit operator bool() const 
			{ 
				if (bUseIterator)
				{
					return bool(It);
				}
				return ItemIndex < MaxNum;
			}
		};

		inline FConstIterator GetIterator() const
		{
			if (bFullUpdate)
			{
				return FConstIterator(0, NumItems);
			}
			return FConstIterator(Tracker->GetChangedIterator<IteratorMask>());
		}

	private:
		bool bFullUpdate = true;
		int32 NumItems = 0;
		const FInstanceAttributeTracker *Tracker = nullptr;
	};
	
	template <EFlag Flag>
	inline FDeltaRange<Flag> GetDeltaRange(bool bForceFullUpdate, int32 InNumItems) const
	{
		return FDeltaRange<Flag>(bForceFullUpdate, InNumItems, this);

	}

	inline bool HasAnyChanges() const
	{
		return MaxIndex > 0 || !Removed.IsEmpty();
	}

	inline SIZE_T GetAllocatedSize() const
	{
		return Data.GetAllocatedSize() + Removed.GetAllocatedSize();
	}

private:

	/**
	 */
	inline void SetMaskTo(int32 Index, ElementType Mask)
	{
		check(Index < MaxIndex);
		check(Mask <= AnyFlagMask);

		const int32 ElementIndex = Index / MasksPerElement;
		const uint32 ElementSubShift = (Index % MasksPerElement) * uint32(EFlag::Num);
		// Load and clear the mask for this index
		ElementType CurrentElement = Data[ElementIndex] & ~(AnyFlagMask << ElementSubShift);
		// add the new mask and store
		Data[ElementIndex] =  CurrentElement | (Mask << ElementSubShift);
	}

	/**
	 * Clear the last item & all remaining in element to ensure it is always all zero
	 */
	inline void ClearLastElement()
	{
		int32 LastIndex = MaxIndex - 1;
		const int32 ElementIndex = LastIndex / MasksPerElement;
		const int32 SubIndex = LastIndex % MasksPerElement;
		if (SubIndex > 0)
		{
			const uint32 ElementSubShift = (MasksPerElement - (LastIndex % MasksPerElement)) * uint32(EFlag::Num) + PaddingBits;
			// Load and clear the mask for this index
			ElementType CurrentElement = Data[ElementIndex] & ((~ElementType(0)) >> ElementSubShift);
			Data[ElementIndex] =  CurrentElement;
		}
		else
		{
			Data[ElementIndex] =  ElementType(0);
		}
	}

	/**
	 */
	template <EFlag Flag>
	inline void DecChangedCount(uint32 Flags)
	{
		if (Flags & FToBit<Flag>::Bit)
		{
			NumChanged[uint32(Flag)] -= 1;
			check(NumChanged[uint32(Flag)] >= 0);
		}
	}

	/**
	 */
	inline void DecChangedCounts(int32 Index)
	{
		const uint32 PrevFlags = GetFlags(Index);
		if ((PrevFlags & FToBit<EFlag::Added>::Bit) == 0)
		{
			DecChangedCount<EFlag::SkinningDataChanged>(PrevFlags);
			DecChangedCount<EFlag::CustomDataChanged>(PrevFlags);
			DecChangedCount<EFlag::TransformChanged>(PrevFlags);
			DecChangedCount<EFlag::IndexChanged>(PrevFlags);
		}
		else
		{
			NumChanged[uint32(EFlag::Added)] -= 1;
			check(NumChanged[uint32(EFlag::Added)] >= 0);
		}
	}

	/**
	 */
	inline void CountMaskChanges(ElementType Element, uint32 Offset, uint32 NumMasksInElement)
	{
		// If the offset is equal to the number of masks then there can't be any changes
		if (Offset < MasksPerElement)
		{
			// 1. Element & IndexChangedElementMask - 1 if it already had index changed
			ElementType HasIndexChanged = (Element & IndexChangedElementMask);
			// 2. Element & AddedElementMask - 1 if already had added << to align
			ElementType HasAdded = (Element & AddedElementMask) << (uint32(EFlag::IndexChanged) - uint32(EFlag::Added));
			ElementType CombinedMask = HasIndexChanged | HasAdded;

			int32 Delta = NumMasksInElement - Offset - FMath::CountBits(CombinedMask >> (Offset * uint32(EFlag::Num)));
			check(Delta >= 0);
			NumChanged[uint32(EFlag::IndexChanged)] += Delta;
		}
	};
	/**
	 * Helper to load and count mask changes, returns the element with the IndexChanged applied to all items. Used while moving elements in the compacting RemoveAt 
	 */
	inline ElementType LoadCountAndMask(int32 ElementIndex, bool bHasNextElement)
	{
		if (bHasNextElement)
		{
			uint32 NumMasksInElement = FMath::Min(MasksPerElement, MaxIndex - ElementIndex * MasksPerElement);
			ElementType Element = Data[ElementIndex];
			CountMaskChanges(Element, 0, NumMasksInElement);
			return Element | IndexChangedElementMask;
		}
		return ElementType(0);
	};


private:
	/** Data for packed per-instance change attribute mask. */
	TArray<ElementType> Data;
	
	/** Removed items are tracked by ID and kept separate as we need to remember if an item was ever removed in order to handle these correctly. */
	TBitArray<> Removed;

	/** Maximum valid index. */
	int32 MaxIndex = 0;

	/** Tracks the index of the first non-zero bit in the Removed bitvector. */
	int32 FirstRemovedIdIndex = MAX_int32;

	/** Tracks the first & last mask with any change flag set */
	int32 FirstChangedIndex = MAX_int32;
	int32 LastChangedIndex = 0;

	/** 
	 * Tracks the number of changes for each attribute in such a way that 
	 * NumChanged[Added] + NumChanged[OtherAttribute] is the total number we need to copy over. This works by counting other attributes ONLY if the
	 * Added flag is not also set.
	 */
	int32 NumChanged[uint32(EFlag::Num)];

	/** Masks replicated to all masks that fit inside ElementType */
	static constexpr ElementType IndexChangedElementMask = FBitManip::FRepMask<FToBit<EFlag::IndexChanged>::Bit, uint32(EFlag::Num)>::Result;
	static constexpr ElementType AddedElementMask = FBitManip::FRepMask<FToBit<EFlag::Added>::Bit, uint32(EFlag::Num)>::Result;
};
