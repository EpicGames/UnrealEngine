// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticMeshComponentHelper.h"
#include "PrimitiveComponentHelper.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/NavigationSystemHelpers.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "Engine/InstancedStaticMesh.h"
#include "NaniteVertexFactory.h"
#include "PSOPrecacheFwd.h"
#include "PSOPrecache.h"

/** Helper class used to share implementation for different InstancedStaticMeshComponent types */
class FInstancedStaticMeshComponentHelper
{
public:
	template<class T>
	static void CollectPSOPrecacheData(const T& Component, const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams);

	template<class T>
	static void GetNavigationData(const T& Component, FNavigationRelevantData& Data, const FNavDataPerInstanceTransformDelegate& Delegate);

	template<class T>
	static bool DoCustomNavigableGeometryExport(const T& Component, FNavigableGeometryExport& GeomExport, const FNavDataPerInstanceTransformDelegate& Delegate);

	template<class T>
	static void GetNavigationPerInstanceTransforms(const T& Component, const FBox& AreaBox, TArray<FTransform>& InstanceData);

	template<class T>
	static FBox GetInstanceNavigationBounds(const T& Component);

private:
	// Dummy instance buffer used for PSO pre-caching
	class FDummyStaticMeshInstanceBuffer : public FStaticMeshInstanceBuffer
	{
	public:
		FDummyStaticMeshInstanceBuffer()
			: FStaticMeshInstanceBuffer(GMaxRHIFeatureLevel, false /*InRequireCPUAccess*/)
		{
			InstanceData = MakeShared<FStaticMeshInstanceData, ESPMode::ThreadSafe>(GVertexElementTypeSupport.IsSupported(VET_Half2));
		}
	};

	static TGlobalResource<FDummyStaticMeshInstanceBuffer> DummyStaticMeshInstanceBuffer;
};

template<class T>
void FInstancedStaticMeshComponentHelper::CollectPSOPrecacheData(const T& Component, const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (DoesPlatformUseInstancedStaticMeshVertexFactory(GMaxRHIShaderPlatform))
	{
		if (Component.GetStaticMesh() == nullptr || Component.GetStaticMesh()->GetRenderData() == nullptr)
		{
			return;
		}

		const bool bCanUseGPUScene = UseGPUScene(GMaxRHIShaderPlatform);
		FStaticMeshInstanceBuffer* InstanceBuffer = bCanUseGPUScene ? nullptr : &FInstancedStaticMeshComponentHelper::DummyStaticMeshInstanceBuffer;
		int32 LightMapCoordinateIndex = Component.GetStaticMesh()->GetLightMapCoordinateIndex();

		auto ISMC_GetElements = [LightMapCoordinateIndex, InstanceBuffer, &Component](const FStaticMeshLODResources& LODRenderData, int32 LODIndex, bool bSupportsManualVertexFetch, FVertexDeclarationElementList& Elements)
		{
			FInstancedStaticMeshDataType InstanceData;
			FInstancedStaticMeshVertexFactory::FDataType Data;
			const FColorVertexBuffer* ColorVertexBuffer = LODRenderData.bHasColorVertexData ? &(LODRenderData.VertexBuffers.ColorVertexBuffer) : nullptr;
			if (Component.LODData.IsValidIndex(LODIndex) && Component.LODData[LODIndex].OverrideVertexColors)
			{
				ColorVertexBuffer = Component.LODData[LODIndex].OverrideVertexColors;
			}
			FInstancedStaticMeshVertexFactory::InitInstancedStaticMeshVertexFactoryComponents(LODRenderData.VertexBuffers, ColorVertexBuffer, InstanceBuffer, nullptr /*VertexFactory*/, LightMapCoordinateIndex, bSupportsManualVertexFetch, Data, InstanceData);
			FInstancedStaticMeshVertexFactory::GetVertexElements(GMaxRHIFeatureLevel, EVertexInputStreamType::Default, bSupportsManualVertexFetch, Data, InstanceData, Elements);
		};

		Nanite::FMaterialAudit NaniteMaterials{};
		if (Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(Component, &NaniteMaterials))
		{
			FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl(Component, &FNaniteVertexFactory::StaticType, BasePrecachePSOParams, ISMC_GetElements, OutParams);
		}
		else
		{
			FStaticMeshComponentHelper::CollectPSOPrecacheDataImpl(Component, &FInstancedStaticMeshVertexFactory::StaticType, BasePrecachePSOParams, ISMC_GetElements, OutParams);
		}
	}
	else
	{
		FStaticMeshComponentHelper::CollectPSOPrecacheData(Component, BasePrecachePSOParams, OutParams);
	}
}

template<class T>
void FInstancedStaticMeshComponentHelper::GetNavigationData(const T& Component, FNavigationRelevantData& Data, const FNavDataPerInstanceTransformDelegate& Delegate)
{
	FPrimitiveComponentHelper::AddNavigationModifier(Component, Data);

	// Navigation data will get refreshed once async compilation finishes
	UStaticMesh* StaticMesh = Component.GetStaticMesh();
	if (StaticMesh && !StaticMesh->IsCompiling() && StaticMesh->GetNavCollision())
	{
		UNavCollisionBase* NavCollision = StaticMesh->GetNavCollision();
		if (Component.ShouldExportAsObstacle(*NavCollision))
		{
			Data.Modifiers.MarkAsPerInstanceModifier();
			NavCollision->GetNavigationModifier(Data.Modifiers, FTransform::Identity);

			// Hook per instance transform delegate
			Data.NavDataPerInstanceTransformDelegate = Delegate;
		}
	}
}

template<class T>
bool FInstancedStaticMeshComponentHelper::DoCustomNavigableGeometryExport(const T& Component, FNavigableGeometryExport& GeomExport, const FNavDataPerInstanceTransformDelegate& Delegate)
{
	UStaticMesh* StaticMesh = Component.GetStaticMesh();
	if (StaticMesh && StaticMesh->GetNavCollision())
	{
		UNavCollisionBase* NavCollision = StaticMesh->GetNavCollision();
		if (Component.ShouldExportAsObstacle(*NavCollision))
		{
			return false;
		}

		if (NavCollision->HasConvexGeometry())
		{
			NavCollision->ExportGeometry(FTransform::Identity, GeomExport);
		}
		else
		{
			UBodySetup* BodySetup = StaticMesh->GetBodySetup();
			if (BodySetup)
			{
				GeomExport.ExportRigidBodySetup(*BodySetup, FTransform::Identity);
			}
		}

		// Hook per instance transform delegate
		GeomExport.SetNavDataPerInstanceTransformDelegate(Delegate);
	}

	// we don't want "regular" collision export for this component
	return false;
}

template<class T>
void FInstancedStaticMeshComponentHelper::GetNavigationPerInstanceTransforms(const T& Component, const FBox& AreaBox, TArray<FTransform>& InstanceData)
{
	const FBox InstanceBounds = FInstancedStaticMeshComponentHelper::GetInstanceNavigationBounds(Component);
	if (InstanceBounds.IsValid)
	{
		const FBox LocalAreaBox = AreaBox.InverseTransformBy(Component.GetComponentTransform());
		Component.ForEachInstanceMatrix([&Component, &InstanceBounds, &LocalAreaBox, &InstanceData](const FMatrix& InstanceTransform)
		{
			const FTransform InstanceToComponent(InstanceTransform);
			if (!InstanceToComponent.GetScale3D().IsZero())
			{
				const FBoxSphereBounds TransformedInstanceBounds = InstanceBounds.TransformBy(InstanceTransform);
				if (LocalAreaBox.Intersect(TransformedInstanceBounds.GetBox()))
				{
					InstanceData.Add(InstanceToComponent * Component.GetComponentTransform());
				}
			}
		});
	}
}

template<class T>
FBox FInstancedStaticMeshComponentHelper::GetInstanceNavigationBounds(const T& Component)
{
	if (const UStaticMesh* Mesh = Component.GetStaticMesh())
	{
		const UNavCollisionBase* NavCollision = Mesh->GetNavCollision();
		const FBox NavBounds = NavCollision ? NavCollision->GetBounds() : FBox();
		return NavBounds.IsValid ? NavBounds : Mesh->GetBounds().GetBox();
	}

	return FBox();
}