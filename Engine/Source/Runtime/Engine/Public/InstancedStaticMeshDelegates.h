// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"

class UInstancedStaticMeshComponent;
class UHierarchicalInstancedStaticMeshComponent;

struct FInstancedStaticMeshDelegates
{
	enum class EInstanceIndexUpdateType : uint8
	{
		/** An instance has been added */
		Added,
		/** An instance has been removed */
		Removed,
		/** An instance has been relocated within the array of instances */
		Relocated,
		/** All instances have been removed */
		Cleared,
		/** All instances have been removed, and the component is being destroyed */
		Destroyed,
	};

	struct FInstanceIndexUpdateData
	{
		/**
		 * The type of this update operation.
		 */
		EInstanceIndexUpdateType Type = EInstanceIndexUpdateType::Cleared;

		/**
		 * The index of the affected instance, when Type==Added, Type==Removed or Type==Relocated.
		 * The index of the last instance, when Type==Cleared or Type==Destroyed.
		 */
		int32 Index = INDEX_NONE;

		/**
		 * The previous index of the affected instance, when Type==Relocated.
		 */
		int32 OldIndex = INDEX_NONE;
	};

	/**
	 * Delegate called when the index of the instances within an ISM are updated.
	 * @note: This is only fired for instances added or removed via the various Add/Remove/ClearInstances functions, and not for instances updated via serialization or undo/redo.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInstanceIndexUpdated, UInstancedStaticMeshComponent* /*Component*/, TArrayView<const FInstanceIndexUpdateData> /*IndexUpdates*/);
	static ENGINE_API FOnInstanceIndexUpdated OnInstanceIndexUpdated;
};

struct FHierarchicalInstancedStaticMeshDelegates
{
	/**
	 * Delegate called when the tree of a HISM has been (re)built.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTreeBuilt, UHierarchicalInstancedStaticMeshComponent* /*Component*/, bool /*bWasAsyncBuild*/);
	static ENGINE_API FOnTreeBuilt OnTreeBuilt;
};
