// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticMeshSceneProxyDesc.h"
#include "Engine/InstancedStaticMesh.h"

class UInstancedStaticMeshComponent;

struct FInstancedStaticMeshSceneProxyDesc : public FStaticMeshSceneProxyDesc
{		
	FInstancedStaticMeshSceneProxyDesc() = default;
	ENGINE_API FInstancedStaticMeshSceneProxyDesc(const UInstancedStaticMeshComponent*);

	ENGINE_API void InitializeFromInstancedStaticMeshComponent(const UInstancedStaticMeshComponent*);

	UE_DEPRECATED(5.5, "Use InitializeFromInstancedStaticMeshComponent instead.")
	void InitializeFrom(const UInstancedStaticMeshComponent* InComponent) { InitializeFromInstancedStaticMeshComponent(InComponent); }

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> InstanceDataSceneProxy;
#if WITH_EDITOR
	bool bHasSelectedInstances = false;
#endif

	int32 InstanceMinDrawDistance = 0;
	int32 InstanceStartCullDistance = 0;
	int32 InstanceEndCullDistance = 0;
	float InstanceLODDistanceScale = 1.0f;

	bool bUseGpuLodSelection = true;

};
