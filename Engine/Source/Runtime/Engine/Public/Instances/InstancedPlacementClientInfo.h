// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISMPartition/ISMComponentDescriptor.h"

#if WITH_EDITORONLY_DATA
#include "Containers/SortedMap.h"
#include "ISMPartition/ISMPartitionClient.h"
#include "ISMPartition/ISMPartitionInstanceManager.h"
#include "Instances/InstancedPlacementHash.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#endif //WITH_EDITORONLY_DATA

#include "InstancedPlacementClientInfo.generated.h"

#if WITH_EDITORONLY_DATA
struct FPropertyChangedEvent;
class AInstancedPlacementPartitionActor;

enum class EPlacementInstanceFlags : uint32
{
	None = 0,
	AlignToNormal = 1 << 0,
	NoRandomYaw = 1 << 1,
	Readjusted = 1 << 2,
};

/**
 * Editor information about where and how an instance was placed
 */
struct FPlacementInstance
{
public:
	friend FArchive& operator<<(FArchive& Ar, FPlacementInstance& Instance);
	ENGINE_API FTransform GetInstanceWorldTransform() const;
	ENGINE_API void SetInstanceWorldTransform(const FTransform& Transform);
	ENGINE_API void AlignToNormal(const FVector& InNormal, int32 AlignMaxAngle = 0);

	double ZOffset = 0.0;
	uint32 Flags = (uint32)EPlacementInstanceFlags::None;

protected:
	FTransform WorldTransform = FTransform(FQuat::Identity, FVector::ZeroVector, FVector3d::OneVector);
	FQuat PreAlignRotation = FQuat::Identity;
};
#endif //WITH_EDITORONLY_DATA

// Settings which can be shared across partition actors
UCLASS(Abstract, hideCategories=Object, editinlinenew, collapsecategories, MinimalAPI)
class UInstancedPlacemenClientSettings : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid UpdateGuid;

	UPROPERTY(EditAnywhere, Category = "Placement")
	FSoftObjectPath ObjectPath;

	UPROPERTY(EditAnywhere, Category = "Component Settings", meta = (ShowOnlyInnerProperties))
	FISMComponentDescriptor InstancedComponentSettings;
#endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void RegisterISMDescriptors(AInstancedPlacementPartitionActor* ParentPartitionActor, TSortedMap<int32, TArray<FTransform>>& ISMDefinition) const;
#endif	// WITH_EDITOR
};

#if WITH_EDITORONLY_DATA
// Settings which are unique per partition actor
struct FClientPlacementInfo : public IISMPartitionInstanceManager
{
	ENGINE_API FClientPlacementInfo();

	friend FArchive& operator<<(FArchive& Ar, FClientPlacementInfo& PlacementInfo);

	// Allows detection of client updates while a partition wasn't loaded
	FGuid UpdateGuid;

	// The client guid we can regenerate the client handle from
	FGuid ClientGuid;

	// Editor-only placed instances
	TArray<FPlacementInstance> Instances;

	// The client we map to in the partition actor
	FISMClientHandle ClientHandle;

	// Definition of the set of ISM descriptors and their local space transforms which make up this client
	TSortedMap<int32, TArray<FTransform>> ISMDefinition;

	// Transient, editor-only locality hash of instances
	FInstancedPlacementHash InstanceHash;

	// The display name of this client
	FString ClientDisplayName;

	TWeakObjectPtr<AInstancedPlacementPartitionActor> ParentPartitionActor;

	TSet<FISMClientInstanceId> MovingInstances;

	using FClientDescriptorFunc = TFunctionRef<void(AInstancedPlacementPartitionActor*, TSortedMap<int32, TArray<FTransform>>&)>;
	ENGINE_API bool Initialize(const FGuid& InClientGuid, const FString& InClientDisplayName, AInstancedPlacementPartitionActor* InParentPartitionActor, FClientDescriptorFunc RegisterDescriptorFunc);
	ENGINE_API void Uninitialize();
	ENGINE_API void PostLoad(AInstancedPlacementPartitionActor* InParentPartitionActor);
	ENGINE_API void PostSerialize(FArchive& Ar, AInstancedPlacementPartitionActor* InParentPartitionActor);
	ENGINE_API bool IsInitialized() const;
	ENGINE_API void PreEditUndo();
	ENGINE_API void PostEditUndo();
	ENGINE_API TArray<FSMInstanceId> AddInstances(TArrayView<const FPlacementInstance> InWorldTransforms);

private:
	void RemoveInstancesFromPartitionActor(TArrayView<const FISMClientInstanceId> InstanceIds, bool bUpdateHISMTrees, TFunctionRef<void(int32)> RemoveFn);

	//~ IISMPartitionInstanceManager interface
	ENGINE_API virtual FText GetISMPartitionInstanceDisplayName(const FISMClientInstanceId& InstanceId) const override;
	ENGINE_API virtual FText GetISMPartitionInstanceTooltip(const FISMClientInstanceId& InstanceId) const override;
	ENGINE_API virtual bool CanEditISMPartitionInstance(const FISMClientInstanceId& InstanceId) const override;
	ENGINE_API virtual bool CanMoveISMPartitionInstance(const FISMClientInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const override;
	ENGINE_API virtual bool GetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	ENGINE_API virtual bool SetISMPartitionInstanceTransform(const FISMClientInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bTeleport = false) override;
	ENGINE_API virtual void NotifyISMPartitionInstanceMovementStarted(const FISMClientInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifyISMPartitionInstanceMovementOngoing(const FISMClientInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifyISMPartitionInstanceMovementEnded(const FISMClientInstanceId& InstanceId) override;
	ENGINE_API virtual void NotifyISMPartitionInstanceSelectionChanged(const FISMClientInstanceId& InstanceId, const bool bIsSelected) override;
	ENGINE_API virtual bool DeleteISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds) override;
	ENGINE_API virtual bool DuplicateISMPartitionInstances(TArrayView<const FISMClientInstanceId> InstanceIds, TArray<FISMClientInstanceId>& OutNewInstanceIds) override;
};

#endif	// WITH_EDITORONLY_DATA
