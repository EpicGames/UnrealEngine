// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA
#include "Misc/HashBuilder.h"

/**
 * FPartitionInstanceHash
 * Used for editor locality of placed instances in a partition actor to control density and avoid overlapping instances
 */
struct FInstancedPlacementHash
{
private:
	static constexpr int32 MaxHashCellBits = 9;	// 512x512x512 grid
	struct FKey
	{
		int64 X;
		int64 Y;
		int64 Z;

		FKey() = default;

		FKey(int64 InX, int64 InY, int64 InZ)
			: X(InX), Y(InY), Z(InZ) {}

		// returns the minimum distance from the given position to the cell described by this key, squared
		double GetMinDistSquared(const FVector& Position, const int32 LocalHashCellBits)
		{
			// compute distance to the cell cube
			int32 CellSize = (1 << LocalHashCellBits);
			int32 HalfCellSize = CellSize >> 1;
			FVector CellCenter(static_cast<FVector::FReal>(X * CellSize + HalfCellSize), static_cast<FVector::FReal>(Y * CellSize + HalfCellSize), static_cast<FVector::FReal>(Z * CellSize + HalfCellSize));
			FVector AbsRelativePosition = (Position - CellCenter).GetAbs();			
			FVector CubeDeltaVector = (AbsRelativePosition - HalfCellSize).ComponentMax(FVector::ZeroVector);
			return CubeDeltaVector.SizeSquared();
		}

		bool operator==(const FKey& Other) const
		{
			return (X == Other.X) && (Y == Other.Y) && (Z == Other.Z);
		}

		friend uint32 GetTypeHash(const FKey& Key)
		{
			FHashBuilder HashBuilder;
			HashBuilder << Key.X << Key.Y << Key.Z;
			return HashBuilder.GetHash();
		}

		friend FArchive& operator<<(FArchive& Ar, FKey& Key)
		{
			Ar << Key.X;
			Ar << Key.Y;
			Ar << Key.Z;
			return Ar;
		}
	};

	const int32 HashCellBits;
	TMap<FKey, TSet<int32>> CellMap;

	FKey MakeKey(const FVector& Location) const
	{
		return FKey(FMath::FloorToInt64(Location.X) >> HashCellBits, FMath::FloorToInt64(Location.Y) >> HashCellBits, FMath::FloorToInt64(Location.Z) >> HashCellBits);
	}

	FVector MakeLocation(FKey CellKey) const
	{
		return FVector(static_cast<FVector::FReal>(CellKey.X << HashCellBits),
		               static_cast<FVector::FReal>(CellKey.Y << HashCellBits),
		               static_cast<FVector::FReal>(CellKey.Z << HashCellBits));
	}

public:
	FInstancedPlacementHash(int32 InHashCellBits = MaxHashCellBits)
		: HashCellBits(InHashCellBits)
	{}

	~FInstancedPlacementHash() = default;

	void InsertInstance(const FVector& InstanceLocation, int32 InstanceIndex)
	{
		FKey Key = MakeKey(InstanceLocation);

		CellMap.FindOrAdd(Key).Add(InstanceIndex);
	}

	void RemoveInstance(const FVector& InstanceLocation, int32 InstanceIndex, bool bChecked = true)
	{
		FKey Key = MakeKey(InstanceLocation);

		if (bChecked)
		{
			int32 RemoveCount = CellMap.FindChecked(Key).Remove(InstanceIndex);
			check(RemoveCount == 1);
		}
		else if (TSet<int32>* Value = CellMap.Find(Key))
		{
			Value->Remove(InstanceIndex);
		}
	}

	void SwapInstance(const FVector& InstanceLocation, int32 OldIndex, int32 NewIndex, bool bCheckedRemoval = true)
	{
		RemoveInstance(InstanceLocation, OldIndex, bCheckedRemoval);
		InsertInstance(InstanceLocation, NewIndex);
	}

	template<typename FunctionType>
	bool IsAnyInstanceInSphere(FunctionType InstanceLocationGetter, const FVector& SphereCenter, double SphereRadius)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IsAnyInstanceInSphere);

		double SphereRadiusSquared = SphereRadius * SphereRadius;

		// if there are more potential cells within range than the number of actually populated cells...
		int32 CellSize = (1 << HashCellBits);
		double SphereRadiusCells = SphereRadius / CellSize;
		constexpr double SphereVolumeConstant = 4.0 / 3.0 * UE_PI;
		double ApproxCellsToCheck = SphereVolumeConstant * (SphereRadiusCells * SphereRadiusCells * SphereRadiusCells);
		if (ApproxCellsToCheck > CellMap.Num())
		{
			// then it's probably faster to just check all populated cells and be done
			for (auto& Pair : CellMap)
			{
				if (Pair.Key.GetMinDistSquared(SphereCenter, HashCellBits) <= SphereRadiusSquared)
				{
					for (int32 InstanceIndex : Pair.Value)
					{
						double DistSquared = (InstanceLocationGetter(InstanceIndex) - SphereCenter).SizeSquared();
						if (DistSquared < SphereRadiusSquared)
						{
							return true;
						}
					}
				}
			}
			return false;
		}

		// otherwise we check all potential cells within range
		FBox SphereBox = FBox::BuildAABB(SphereCenter, FVector(SphereRadius));
		FKey MinKey = MakeKey(SphereBox.Min);
		FKey MaxKey = MakeKey(SphereBox.Max);
		for (int64 z = MinKey.Z; z <= MaxKey.Z; ++z)
		{
			for (int64 y = MinKey.Y; y <= MaxKey.Y; y++)
			{
				for (int64 x = MinKey.X; x <= MaxKey.X; x++)
				{
					FKey Key(x, y, z);
					auto* SetPtr = CellMap.Find(Key);
					if (SetPtr)
					{
						for (int32 InstanceIndex : *SetPtr)
						{
							float DistSquared = (InstanceLocationGetter(InstanceIndex) - SphereCenter).SizeSquared();
							if (DistSquared < SphereRadiusSquared)
							{
								return true;
							}
						}
					}
				}
			}
		}

		return false;
	}

	void GetInstancesOverlappingBox(const FBox& InBox, TArray<int32>& OutInstanceIndices) const
	{
		FKey MinKey = MakeKey(InBox.Min);
		FKey MaxKey = MakeKey(InBox.Max);

		// compute in doubles to avoid overflow issues (queries using WORLD_MAX bounds can produce very large numbers...)
		double CellsX = static_cast<double>((MaxKey.X - MinKey.X + 1));
		double CellsY = static_cast<double>((MaxKey.Y - MinKey.Y + 1));
		double CellsZ = static_cast<double>((MaxKey.Z - MinKey.Z + 1));
		double CellCount = CellsX * CellsY * CellsZ;

		// The idea here is to decide when it is faster to just check every populated cell.
		// The actual ideal threshold will depend on the exact state of the cells, but we just pick
		// some arbitrary ratio where we switch to checking populated cells over potential cells within range.
		// In practice the exact threshold is not that important; we just want to know if there are different orders of magnitude involved.
		// (i.e. it is faster to check 32 populated cells than 3,000,000,000 potential cells...)
		constexpr double RelativeCostOfCheckingPopulatedCells = 2.0;
		double Threshold = RelativeCostOfCheckingPopulatedCells * CellMap.Num();
		if (CellCount > Threshold)
		{
			// check every populated cell
			for (auto& Pair : CellMap)
			{
				if ((Pair.Key.X >= MinKey.X) &&
					(Pair.Key.X <= MaxKey.X) &&
					(Pair.Key.Y >= MinKey.Y) &&
					(Pair.Key.Y <= MaxKey.Y) &&
					(Pair.Key.Z >= MinKey.Z) &&
					(Pair.Key.Z <= MaxKey.Z))
				{
					OutInstanceIndices.Reserve(OutInstanceIndices.Num() + Pair.Value.Num());
					for (int32 InstanceIndex : Pair.Value)
					{
						OutInstanceIndices.Add(InstanceIndex);
					}
				}
			}
		}
		else
		{
			// check all potential cells within range
			for (int64 z = MinKey.Z; z <= MaxKey.Z; ++z)
			{
				for (int64 y = MinKey.Y; y <= MaxKey.Y; y++)
				{
					for (int64 x = MinKey.X; x <= MaxKey.X; x++)
					{
						auto* SetPtr = CellMap.Find(FKey(x, y, z));
						if (SetPtr)
						{
							OutInstanceIndices.Reserve(OutInstanceIndices.Num() + SetPtr->Num());
							for (int32 InstanceIndex : *SetPtr)
							{
								OutInstanceIndices.Add(InstanceIndex);
							}
						}
					}
				}
			}
		}
	}

	TArray<int32> GetInstancesOverlappingBox(const FBox& InBox) const
	{
		TArray<int32> Result;
		GetInstancesOverlappingBox(InBox, Result);
		return Result;
	}

	void CheckInstanceCount(int32 InCount) const
	{
		int32 HashCount = 0;
		for (const auto& Pair : CellMap)
		{
			HashCount += Pair.Value.Num();
		}

		check(HashCount == InCount);
	}

	FBox GetBounds() const
	{
		FBox HashBounds(ForceInit);
		for (const auto& Pair : CellMap)
		{
			HashBounds += MakeLocation(Pair.Key);
		}

		return HashBounds;
	}

	void Empty()
	{
		CellMap.Empty();
	}

	friend FArchive& operator<<(FArchive& Ar, FInstancedPlacementHash& Hash)
	{
		Ar << Hash.CellMap;
		return Ar;
	}
};
#endif
