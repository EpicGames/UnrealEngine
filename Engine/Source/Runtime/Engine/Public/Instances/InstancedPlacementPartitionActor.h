// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISMPartition/ISMPartitionActor.h"
#include "Instances/InstancedPlacementClientInfo.h"

#include "InstancedPlacementPartitionActor.generated.h"

class UInstancedPlacementClientSettings;

/**
 * The base class used by any editor placement of instanced objects, which holds any relevant runtime data for the placed instances.
 */
UCLASS(MinimalAPI)
class AInstancedPlacementPartitionActor : public AISMPartitionActor
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;

#if WITH_EDITOR	
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	ENGINE_API virtual FGuid GetGridGuid() const override;

	ENGINE_API void SetGridGuid(const FGuid& InGuid);

	using FClientDescriptorFunc = FClientPlacementInfo::FClientDescriptorFunc;
	ENGINE_API FClientPlacementInfo* PreAddClientInstances(const FGuid& ClientGuid, const FString& InClientDisplayString, FClientDescriptorFunc RegisterDefinitionFunc);
	ENGINE_API void PostAddClientInstances();
	ENGINE_API void NotifySettingsObjectChanged(UInstancedPlacementClientSettings* InSettingsObject);
#endif

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid PlacementGridGuid;

	// Placed Client info by corresponding client Guid
	TMap<FGuid, TUniqueObj<FClientPlacementInfo>> PlacedClientInfo;
#endif
};
