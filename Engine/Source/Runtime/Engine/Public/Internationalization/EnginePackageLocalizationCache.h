// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/PackageLocalizationCache.h"

struct FAssetData;

/** Implementation of a package localization cache that takes advantage of the asset registry. */
class FEnginePackageLocalizationCache : public FPackageLocalizationCache
{
public:
	ENGINE_API FEnginePackageLocalizationCache();
	ENGINE_API virtual ~FEnginePackageLocalizationCache();

protected:
	//~ FPackageLocalizationCache interface
	ENGINE_API virtual void FindLocalizedPackages(const TMap<FString, TArray<FString>>& NewSourceToLocalizedPaths, TMap<FName, TArray<FName>>& InOutSourcePackagesToLocalizedPackages) override;
	ENGINE_API virtual void FindAssetGroupPackages(const FName InAssetGroupName, const FTopLevelAssetPath& InAssetClassName) override;

private:
	/**
	 * Callback handler for when a new asset is added to the asset registry.
	 *
	 * @param InAssetData		Data about the asset that was added.
	 */
	void HandleAssetAdded(const FAssetData& InAssetData);

	/**
	 * Callback handler for when an existing asset is removed from the asset registry.
	 *
	 * @param InAssetData		Data about the asset that was removed.
	 */
	void HandleAssetRemoved(const FAssetData& InAssetData);

	/**
	 * Callback handler for when an existing asset is renamed in the asset registry.
	 *
	 * @param InAssetData		Data about the asset under its new name.
	 * @param InOldObjectPath	The old name of the asset.
	 */
	void HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath);

	/** True if we are currently within an asset registry ScanPathsSynchronous call. */
	bool bIsScanningPath;
};
