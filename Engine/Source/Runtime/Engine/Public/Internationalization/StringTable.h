// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Internationalization/StringTableCoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "StringTable.generated.h"

class FPropertyLocalizationDataGatherer;
enum class EPropertyLocalizationGathererTextFlags : uint8;

namespace UE::StringTable
{

#if WITH_EDITORONLY_DATA
/**
 * Utility to gather the string table data stored within an asset.
 * @note Primarily used internally by UStringTable, but may be used by any other assets that host a FStringTable and need to handle gathering it for localization via a FPropertyLocalizationDataGatherer.
 */
ENGINE_API void GatherForLocalization(const FString& SourceLocation, const FStringTableConstRef& StringTable, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
#endif	// WITH_EDITORONLY_DATA

}

/** String table wrapper asset */
UCLASS(MinimalAPI)
class UStringTable : public UObject
{
	GENERATED_BODY()

public:
	ENGINE_API UStringTable();

	/** Called during Engine init to initialize the engine bridge instance */
	static ENGINE_API void InitializeEngineBridge();

	//~ UObject interface
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;

	/** Notifies that a plugin unmounted */
	ENGINE_API static void OnPluginUnloaded(FStringView PluginName);

	/** Get the string table ID that should be used by this asset */
	ENGINE_API FName GetStringTableId() const;

	/** Get the underlying string table owned by this asset */
	ENGINE_API FStringTableConstRef GetStringTable() const;

	/** Get the underlying string table owned by this asset */
	ENGINE_API FStringTableRef GetMutableStringTable() const;

private:
	/** Internal string table. Will never be null, but is a TSharedPtr rather than a TSharedRef so we can use a forward declaration */
	FStringTablePtr StringTable;

	/** Cached ID of this string table asset in the string table registry */
	FName StringTableId;
};
