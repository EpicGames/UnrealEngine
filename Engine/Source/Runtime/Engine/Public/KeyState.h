// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"

/*=============================================================================
	 KeyState - contains details about a key's state and recent events
=============================================================================*/

struct FKeyState
{
	/** This is the most recent raw value reported by the device.  For digital buttons, 0 or 1.  For analog buttons, 0->1.  For axes, -1->1. The X field is for non-vector keys */
	FVector RawValue;

	/** The final "value" for this control, after any optional processing. */
	FVector Value;

	/** Global time of last up->down or down->up transition. */
	float LastUpDownTransitionTime;

	/** True if this key is "down", false otherwise. */
	uint8 bDown:1;

	/** Queued state information.  This data is updated or flushed once player input is processed. */
	uint8 bDownPrevious:1;

	/** True if this key has been "consumed" by an InputComponent and should be ignored for further components during this update. */
	uint8 bConsumed:1;

	/** 
	* True if this key was flushed this frame. This is used to flag to input processing that we may be receiving 
	* a IE_Repeat event, but the IE_Pressed event accumulator may have been reset. 
	*/
	uint8 bWasJustFlushed:1;

	/** Flag paired axes that have been sampled this tick. X = LSB, Z = MSB */
	uint8 PairSampledAxes : 3;

	/** How many samples contributed to RawValueAccumulator. Used for smoothing operations, e.g. mouse */
	uint8 SampleCountAccumulator;

	/** How many of each event type had been received when input was last processed. */
	TArray<uint32> EventCounts[IE_MAX];

	/** Used to accumulate events during the frame and flushed when processed. */
	TArray<uint32> EventAccumulator[IE_MAX];

	/** Used to accumulate input values during the frame and flushed after processing. */
	FVector RawValueAccumulator;

	FKeyState()
		: RawValue(0.f, 0.f, 0.f)
		, Value(0.f, 0.f, 0.f)
		, LastUpDownTransitionTime(0.f)
		, bDown(false)
		, bDownPrevious(false)
		, bConsumed(false)
		, bWasJustFlushed(false)
		, PairSampledAxes(0)
		, SampleCountAccumulator(0)
		, RawValueAccumulator(0.f, 0.f, 0.f)
	{
	}
};
