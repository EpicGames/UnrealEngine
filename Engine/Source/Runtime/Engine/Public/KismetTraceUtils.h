// Copyright Epic Games, Inc. All Rights Reserved.



#pragma once 

#include "CoreMinimal.h"
#include "DrawDebugHelpers.h"
#include "CollisionQueryParams.h"
#include "Kismet/KismetSystemLibrary.h"

FCollisionQueryParams ConfigureCollisionParams(FName TraceTag, bool bTraceComplex, const TArray<AActor*>& ActorsToIgnore, bool bIgnoreSelf, const UObject* WorldContextObject);

FCollisionObjectQueryParams ConfigureCollisionObjectParams(const TArray<TEnumAsByte<EObjectTypeQuery> > & ObjectTypes);

#if ENABLE_DRAW_DEBUG

ENGINE_API void DrawDebugSweptSphere(const UWorld* InWorld, FVector const& Start, FVector const& End, float Radius, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0);
ENGINE_API void DrawDebugSweptBox(const UWorld* InWorld, FVector const& Start, FVector const& End, FRotator const & Orientation, FVector const & HalfSize, FColor const& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0);

ENGINE_API void DrawDebugLineTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);
ENGINE_API void DrawDebugLineTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);

ENGINE_API void DrawDebugBoxTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, const FVector HalfSize, const FRotator Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);
ENGINE_API void DrawDebugBoxTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, const FVector HalfSize, const FRotator Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);

ENGINE_API void DrawDebugSphereTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);
ENGINE_API void DrawDebugSphereTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);

/** Debug Drawing for single trace capsules using rotation FRotator::ZeroRotator. */
ENGINE_API void DrawDebugCapsuleTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);
/** Debug Drawing for multi trace capsules using rotation FRotator::ZeroRotator. */
ENGINE_API void DrawDebugCapsuleTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);

/** Debug Drawing for single trace capsules with orientation. */
ENGINE_API void DrawDebugCapsuleTraceSingle(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, const FRotator& Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const FHitResult& OutHit, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);
/** Debug Drawing for multi trace capsules with orientation. */
ENGINE_API void DrawDebugCapsuleTraceMulti(const UWorld* World, const FVector& Start, const FVector& End, float Radius, float HalfHeight, const FRotator& Orientation, EDrawDebugTrace::Type DrawDebugType, bool bHit, const TArray<FHitResult>& OutHits, FLinearColor TraceColor, FLinearColor TraceHitColor, float DrawTime);

#endif // ENABLE_DRAW_DEBUG


