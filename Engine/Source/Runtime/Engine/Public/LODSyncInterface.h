// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"
#include "LODSyncInterface.generated.h"

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class ULODSyncInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};


/* This is interface class for getting/setting LOD info by LODSyncComponent
 *
 * Implemented by SkeletalMeshComponent, GroomComponent
 */

class ILODSyncInterface
{
	GENERATED_IINTERFACE_BODY()

	/**
	 * Returns the renderable's desired LOD as a 0-based index.
	 * 
	 * If LOD streaming is enabled, the desired LOD returned from this function is not guaranteed
	 * to be available (i.e. loaded). Call GetBestAvailableLOD to find out which LODs are available.
	 * 
	 * Returns INDEX_NONE if the desired LOD is not known, e.g. if there's no mesh set on the
	 * component. In this case, the LOD Sync Component won't take this component's desired LOD into
	 * account, but will still set a forced LOD on it in case it does get rendered.
	 */
	virtual int32 GetDesiredSyncLOD() const = 0;

	/**
	 * Returns the best (i.e. lowest numbered) LOD that is streamed in and can be used for
	 * rendering immediately.
	 * 
	 * Callers can assume that any LOD worse (higher numbered) than this is also available.
	 *
	 * Returns INDEX_NONE if no LODs are available, e.g. if there's no mesh set on the component.
	 * In this case, the LOD Sync Component won't take this component's desired LOD into account,
	 * but will still set a forced LOD on it in case it does get rendered.
	 */
	virtual int32 GetBestAvailableLOD() const = 0;

	/**
	 * Returns the total number of LODs, including LODs that are streamed out and unavailable.
	 *
	 * Returns 0 if there are no LODs, e.g. if there's no mesh set on the component.
	 */
	virtual int32 GetNumSyncLODs() const = 0;

	/**
	 * Sets the LOD to force to stream in, as a 0-based index.
	 * 
	 * If LODIndex is non-negative and LOD streaming is enabled, this call requests that the
	 * streaming system stream in LODs up to and including the specified LOD index, ignoring
	 * any other heuristics such as size on screen.
	 * 
	 * Note that there is no guarantee of how long it will take the requested LOD to stream in,
	 * and if memory is low it may never stream in.
	 *
	 * If LODIndex is negative, the streaming behavior will be returned to normal.
	 *
	 * This call has no effect while LOD streaming is disabled.
	 */
	virtual void SetForceStreamedLOD(int32 LODIndex) = 0;

	/**
	 * Sets the LOD to use for rendering, as a 0-based index.
	 *
	 * If LODIndex is negative, the rendering LOD selection behavior will be returned to normal.
	 *
	 * This call has no effect on streaming behavior. If a LOD is requested that is not streamed
	 * in, the best available LOD will be used instead. If the requested LOD is streamed in later,
	 * it will be used without the caller needing to call this function again.
	 */
	virtual void SetForceRenderedLOD(int32 LODIndex) = 0;

	UE_DEPRECATED(5.4, "SetSyncLOD is deprecated and will be removed. For the same effect, call both SetForceStreamedLOD and SetForceRenderedLOD.")
	virtual void SetSyncLOD(int32 LODIndex)
	{
		SetForceStreamedLOD(LODIndex);
		SetForceRenderedLOD(LODIndex);
	}

	/**
	 * Returns the LOD that will be forced to stream in, or INDEX_NONE if there is no forced LOD
	 * for streaming.
	 * 
	 * Note that this is not guaranteed to return the value that was set by the last call to
	 * SetForceStreamedLOD, as the implementation is permitted to change the force streamed LOD
	 * outside of calls to that function.
	 */
	virtual int32 GetForceStreamedLOD() const = 0;

	/**
	 * Returns the LOD that will be forced to use for rendering, or INDEX_NONE if there is no
	 * forced LOD for rendering.
	 * 
	 * Note that this is not guaranteed to return the value that was set by the last call to
	 * SetForceRenderedLOD, as the implementation is permitted to change the force rendered LOD
	 * outside of calls to that function.
	 */
	virtual int32 GetForceRenderedLOD() const = 0;

	UE_DEPRECATED(5.4, "GetCurrentSyncLOD has been renamed to GetForceRenderedLOD. GetCurrentSyncLOD will be removed.")
	virtual int32 GetCurrentSyncLOD() const
	{
		return GetForceRenderedLOD();
	}
};
