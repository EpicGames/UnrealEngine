// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

// The response to updating a latent action
struct FLatentResponse
{
private:
	struct FExecutionInfo
	{
	public:
		FName	ExecutionFunction;
		int32		LinkID;
		FWeakObjectPtr CallbackTarget;

		FExecutionInfo(FName InExecutionFunction, int32 InLinkID, FWeakObjectPtr InCallbackTarget)
			: ExecutionFunction(InExecutionFunction)
			, LinkID(InLinkID)
			, CallbackTarget(InCallbackTarget)
		{
		}

	private:
		FExecutionInfo()
		{
		}

	};

protected:
	TArray< FExecutionInfo, TInlineAllocator<4> > LinksToExecute;
	bool bRemoveAction;
	float DeltaTime;

	friend struct FLatentActionManager;
public:
	FLatentResponse(float InDeltaTime)
		: bRemoveAction(false)
		, DeltaTime(InDeltaTime)
	{
	}

	FLatentResponse& DoneIf(bool Condition)
	{
		bRemoveAction = Condition;
		return *this;
	}

	FLatentResponse& TriggerLink(FName ExecutionFunction, int32 LinkID, FWeakObjectPtr InCallbackTarget)
	{
		LinksToExecute.Add(FExecutionInfo(ExecutionFunction, LinkID, InCallbackTarget));
		return *this;
	}

	FLatentResponse& FinishAndTriggerIf(bool Condition, FName ExecutionFunction, int32 LinkID, FWeakObjectPtr InCallbackTarget)
	{
		bRemoveAction = Condition;
		if (bRemoveAction)
		{
			LinksToExecute.Add(FExecutionInfo(ExecutionFunction, LinkID, InCallbackTarget));
		}
		return *this;
	}

	float ElapsedTime() const { return DeltaTime; }
};

// A pending latent action
class FPendingLatentAction
{
public:
	FPendingLatentAction()
	{
	}

	virtual ~FPendingLatentAction() = default;

	// Return true when the action is completed
	virtual void UpdateOperation(FLatentResponse& Response)
	{
		Response.DoneIf(true);
	}

	// Lets the latent action know that the object which originated it has been garbage collected
	// and the action is going to be destroyed (no more UpdateOperation calls will occur and
	// CallbackTarget is already NULL)
	// This is only called when the object goes away before the action is finished; perform normal
	// cleanup when responding that the action is completed in UpdateOperation
	virtual void NotifyObjectDestroyed() {}

	virtual void NotifyActionAborted() {}
#if WITH_EDITOR
	// Returns a human readable description of the latent operation's current state
	ENGINE_API virtual FString GetDescription() const;
#endif
};
