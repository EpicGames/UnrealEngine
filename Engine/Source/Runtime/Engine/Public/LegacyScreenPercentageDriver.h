// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneView.h"

#include "LegacyScreenPercentageDriver.generated.h"

/**
 * Default screen percentage interface that just apply View->FinalPostProcessSettings.ScreenPercentage.
 */
class FLegacyScreenPercentageDriver : public ISceneViewFamilyScreenPercentage
{
public:
	ENGINE_API FLegacyScreenPercentageDriver(
		const FSceneViewFamily& InViewFamily,
		float InGlobalResolutionFraction);

	ENGINE_API FLegacyScreenPercentageDriver(
		const FSceneViewFamily& InViewFamily,
		float InGlobalResolutionFraction,
		float InGlobalResolutionFractionUpperBound);

	/** Gets the view rect fraction from the r.ScreenPercentage cvar. */
	static ENGINE_API float GetCVarResolutionFraction();

private:
	// View family to take care of.
	const FSceneViewFamily& ViewFamily;

	// ViewRect fraction to apply to all view of the view family.
	const float GlobalResolutionFraction;

	// ViewRect fraction to apply to all view of the view family.
	const float GlobalResolutionFractionUpperBound;


	// Implements ISceneViewFamilyScreenPercentage
	ENGINE_API virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const override;
	ENGINE_API virtual DynamicRenderScaling::TMap<float> GetResolutionFractions_RenderThread() const override;
	ENGINE_API virtual ISceneViewFamilyScreenPercentage* Fork_GameThread(const class FSceneViewFamily& ForkedViewFamily) const override;
};

// Status of view being rendered to select the corresponding screen percentage setting
UENUM()
enum class EViewStatusForScreenPercentage
{
	// For editor viewports not refreshing every frames.
	NonRealtime UMETA(DisplayName = "Non-Realtime"),

	// For desktop renderer
	Desktop UMETA(DisplayName = "Desktop Rendered"),

	// For mobile renderer
	Mobile UMETA(DisplayName = "Mobile Rendered"),

	// For VR rendering
	VR UMETA(DisplayName = "VR Rendered"),

	// For path tracer
	PathTracer UMETA(DisplayName = "Path Traced"),
};

// Mode for the computation of the screen percentage.
UENUM()
enum class EScreenPercentageMode
{
	// Directly controls the screen percentage manually
	Manual UMETA(DisplayName="Manual"),

	// Automatic control the screen percentage based on the display resolution
	BasedOnDisplayResolution UMETA(DisplayName="Based on display resolution"),

	// Based on DPI scale
	BasedOnDPIScale UMETA(DisplayName="Based on operating system's DPI scale"),
};

/**
 * Heuristic to automatically compute a default resolution fraction based user settings and display information.
 */
struct FStaticResolutionFractionHeuristic
{
	FStaticResolutionFractionHeuristic() = default;

	UE_DEPRECATED(5.3, "Uses PullRunTimeRenderingSettings(EViewStatusForScreenPercentage) instead")
	ENGINE_API FStaticResolutionFractionHeuristic(const FEngineShowFlags& EngineShowFlags);

	// User configurable settings
	struct FUserSettings
	{
		EScreenPercentageMode Mode = EScreenPercentageMode::Manual;

		// r.ScreenPercentage when Mode = EMode::Manual.
		float GlobalResolutionFraction = 1.0f;

		// r.ScreenPercentage.{Min,Max}Resolution
		float MinRenderingResolution = 0.0f;
		float MaxRenderingResolution = 0.0f;

		// r.ScreenPercentage.Auto.* Mode = EMode::BasedOnDisplayResolution.
		float AutoPixelCountMultiplier = 1.0f;

		// stereo HMDs cannot use percentage modes based on 2D monitor
		bool bAllowDisplayBasedScreenPercentageMode = true;

		/** Return whether should use the editor settings for PIE. */
#if WITH_EDITOR
		static ENGINE_API bool EditorOverridePIESettings();
#else
		static inline bool EditorOverridePIESettings()
		{
			return false;
		}
#endif

		/** Pulls the user settings from the gameplay runtime cvars. */
		ENGINE_API void PullRunTimeRenderingSettings(EViewStatusForScreenPercentage ViewStatus);

		/** Pulls the user settings from the editor cvars. */
		ENGINE_API void PullEditorRenderingSettings(EViewStatusForScreenPercentage ViewStatus);

		UE_DEPRECATED(5.3, "Uses PullRunTimeRenderingSettings(EViewStatusForScreenPercentage) instead")
		ENGINE_API void PullRunTimeRenderingSettings();

		UE_DEPRECATED(5.3, "Uses PullEditorRenderingSettings(EViewStatusForScreenPercentage) instead")
		ENGINE_API void PullEditorRenderingSettings(bool bIsRealTime, bool bIsPathTraced);
	};

	FUserSettings Settings;

	// [Required] total number of pixel being display into viewport.
	int32 TotalDisplayedPixelCount = 0;

	// [Required] secondary resolution fraction.
	float SecondaryViewFraction = 1.0f;

	// [Required] DPI scale
	float DPIScale = 1.0f;

	// Fetches rendering information from the ViewFamily.
	ENGINE_API void PullViewFamilyRenderingSettings(const FSceneViewFamily& ViewFamily);

	ENGINE_API float ResolveResolutionFraction() const;
};
