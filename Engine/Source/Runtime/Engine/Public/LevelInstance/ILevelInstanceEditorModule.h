// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "UObject/UnrealType.h"

class ILevelInstanceInterface;

/**
 * The module holding all of the UI related pieces for LevelInstance management
 */
class ILevelInstanceEditorModule : public IModuleInterface
{
public:
	virtual ~ILevelInstanceEditorModule() = default;

	UE_DEPRECATED(5.5, "This method is deprecated.")
	virtual void ActivateEditorMode() {}
	UE_DEPRECATED(5.5, "This method is deprecated.")
	virtual void DeactivateEditorMode() {}

	virtual void BroadcastTryExitEditorMode() = 0;

	/** Broadcasts before exiting mode */
	DECLARE_EVENT(ILevelInstanceEditorModule, FExitEditorModeEvent);
	virtual FExitEditorModeEvent& OnExitEditorMode() = 0;

	DECLARE_EVENT(ILevelInstanceEditorModule, FTryExitEditorModeEvent);
	virtual FTryExitEditorModeEvent& OnTryExitEditorMode() = 0;

	/** Broadcasts when the state of a LI being edited changes */
	enum class ELevelInstanceEditMode : uint8
	{
		None,     /** The level instance is not being edited */
		Original, /** The level instance's original is being edited */
		Override  /** A level instance override is being edited */
	};
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLevelInstanceEditModeChanged, ILevelInstanceInterface* /*LevelInstance*/, ELevelInstanceEditMode /*NewMode*/);
	virtual FOnLevelInstanceEditModeChanged& OnLevelInstanceEditModeChanged() = 0;
	virtual void BroadcastOnLevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstance, ELevelInstanceEditMode NewMode) = 0;

	virtual bool IsEditInPlaceStreamingEnabled() const = 0;
	virtual bool IsSubSelectionEnabled() const = 0;

	virtual void UpdateAllPackedLevelActorsForWorldAsset(const TSoftObjectPtr<UWorld>& InWorldAsset, bool bInLoadedOnly = false) = 0;

protected:
	friend class ULevelInstanceSubsystem;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;

	// Called by ULevelInstanceSubsystem to update if the Editor Mode should be active or not
	virtual void UpdateEditorMode(bool bActivated) = 0;

	// Policy Proxy so that ULevelStreamingLevelInstanceEditorPropertyOverride can register policies through this module without knowing about PropertyEditor module
	class IPropertyOverridePolicy
	{
	public:
		virtual UObject* GetArchetypeForObject(const UObject* Object) const = 0;

		virtual bool CanEditProperty(const FEditPropertyChain& PropertyChain, const UObject* Object) const = 0;
		virtual bool CanEditProperty(const FProperty* Property, const UObject* Object) const = 0;
	};

	virtual UObject* GetArchetype(const UObject* Object) = 0;

	virtual bool IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object) = 0;
	virtual bool IsPropertyEditConst(const FProperty* Property, UObject* Object) = 0;

	virtual void SetPropertyOverridePolicy(IPropertyOverridePolicy* Policy) = 0;
};
