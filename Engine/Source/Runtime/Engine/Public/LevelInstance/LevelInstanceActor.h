// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Containers/Array.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceActorImpl.h"
#include "LevelInstance/LevelInstanceActorGuid.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "LevelInstanceActor.generated.h"

UCLASS(MinimalAPI)
class ALevelInstance : public AActor, public ILevelInstanceInterface
{
	GENERATED_BODY()

public:
	ENGINE_API ALevelInstance();

protected:
#if WITH_EDITORONLY_DATA
	/** Level LevelInstance */
	UPROPERTY(EditAnywhere, Category = Level, Meta = (NoCreate, DisplayName="Level", DisableLevelInstancePropertyOverride))
	TSoftObjectPtr<UWorld> WorldAsset;

	UPROPERTY(VisibleAnywhere, Category = Override, meta = (EditInline, NoResetToDefault, EditCondition="PropertyOverrides != nullptr", EditConditionHides))
	TObjectPtr<ULevelInstancePropertyOverrideAsset> PropertyOverrides;
#endif
	
	UPROPERTY(VisibleAnywhere, Category = Default)
	TObjectPtr<ULevelInstanceComponent> LevelInstanceComponent;

	UPROPERTY(Replicated)
	TSoftObjectPtr<UWorld> CookedWorldAsset;

	UPROPERTY(Transient, Replicated)
	FGuid LevelInstanceSpawnGuid;

	FLevelInstanceActorGuid LevelInstanceActorGuid;
	FLevelInstanceActorImpl LevelInstanceActorImpl;

#if WITH_EDITORONLY_DATA
//Force the usage of SetDesiredRuntimeBehavior to change the behavior
private:
	UPROPERTY(EditAnywhere, Category = Level, AdvancedDisplay, Meta = (DisplayName="Level Behavior", DisableLevelInstancePropertyOverride))
	ELevelInstanceRuntimeBehavior DesiredRuntimeBehavior;
#endif

public:
	// Begin ILevelInstanceInterface
	ENGINE_API const FLevelInstanceID& GetLevelInstanceID() const override;
	ENGINE_API bool HasValidLevelInstanceID() const override;
	ENGINE_API virtual const FGuid& GetLevelInstanceGuid() const override;
	ENGINE_API virtual const TSoftObjectPtr<UWorld>& GetWorldAsset() const override;
	
	ENGINE_API virtual void OnLevelInstanceLoaded() override;
	ENGINE_API virtual bool IsLoadingEnabled() const override;
	// End ILevelInstanceInterface

	ENGINE_API virtual void PostRegisterAllComponents() override;
	ENGINE_API virtual void PostUnregisterAllComponents() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	ENGINE_API virtual void PostNetInit() override;

#if WITH_EDITOR
	// Begin ILevelInstanceInterface
	ENGINE_API virtual ULevelInstanceComponent* GetLevelInstanceComponent() const override;
	ENGINE_API virtual bool SetWorldAsset(TSoftObjectPtr<UWorld> WorldAsset) override;
	virtual ELevelInstanceRuntimeBehavior GetDesiredRuntimeBehavior() const override { return DesiredRuntimeBehavior; }
	ENGINE_API void SetDesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior NewBehavior) override;
	virtual ELevelInstanceRuntimeBehavior GetDefaultRuntimeBehavior() const override { return ELevelInstanceRuntimeBehavior::Partitioned; }
	ENGINE_API virtual TSubclassOf<AActor> GetEditorPivotClass() const override;
	ENGINE_API virtual bool SupportsPartialEditorLoading() const override;

	ENGINE_API virtual bool SupportsPropertyOverrides() const override;
	ENGINE_API ULevelInstancePropertyOverrideAsset* GetPropertyOverrideAsset() const override;
	// End ILevelInstanceInterface
			
	// UObject overrides
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& Context) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists) override;

	// AActor overrides
	ENGINE_API virtual bool CanEditChangeComponent(const UActorComponent* InComponent, const FProperty* InProperty) const override;
	ENGINE_API virtual void CheckForErrors() override;
	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	ENGINE_API virtual bool CanDeleteSelectedActor(FText& OutReason) const override;
	ENGINE_API virtual void SetIsTemporarilyHiddenInEditor(bool bIsHidden) override;
	ENGINE_API virtual bool SetIsHiddenEdLayer(bool bIsHiddenEdLayer) override;
	ENGINE_API virtual void EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const override;
	ENGINE_API virtual void PushSelectionToProxies() override;
	ENGINE_API virtual void PushLevelInstanceEditingStateToProxies(bool bInEditingState) override;
	ENGINE_API virtual FBox GetComponentsBoundingBox(bool bNonColliding = false, bool bIncludeFromChildActors = false) const override;
	ENGINE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
	ENGINE_API virtual bool IsLockLocation() const override;
	ENGINE_API virtual bool IsActorLabelEditable() const override;
	ENGINE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	ENGINE_API virtual bool GetSoftReferencedContentObjects(TArray<FSoftObjectPath>& SoftObjects) const override;
	ENGINE_API virtual bool OpenAssetEditor() override;
	ENGINE_API virtual bool EditorCanAttachFrom(const AActor* InChild, FText& OutReason) const override;	
	ENGINE_API virtual bool IsUserManaged() const override;
	ENGINE_API virtual bool ShouldExport() override;
	virtual bool SupportsSubRootSelection() const override { return true; }
	ENGINE_API virtual bool IsHLODRelevant() const override;
	ENGINE_API virtual bool HasHLODRelevantComponents() const override;
	// End of AActor interface

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLevelInstanceActorPostLoad, ALevelInstance*);
	static ENGINE_API FOnLevelInstanceActorPostLoad OnLevelInstanceActorPostLoad;

private:
	ENGINE_API virtual void SetPropertyOverrideAsset(ULevelInstancePropertyOverrideAsset* InPropertyOverrideAsset) override;
	ENGINE_API virtual bool ShouldCookWorldAsset() const;
#endif
};

DEFINE_ACTORDESC_TYPE(ALevelInstance, FLevelInstanceActorDesc);
