// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Serialization/Archive.h"
#include "Misc/Guid.h"

class AActor;

/**
 * Helper struct that allows serializing the ActorGuid for runtime use.
 */
struct FLevelInstanceActorGuid
{
	// Exists only to support 'FVTableHelper' Actor constructors
	FLevelInstanceActorGuid() : FLevelInstanceActorGuid(nullptr) {}

	FLevelInstanceActorGuid(AActor* InActor) : Actor(InActor) {}

#if !WITH_EDITOR
	ENGINE_API void AssignIfInvalid();
#endif

	ENGINE_API bool IsValid() const;
	ENGINE_API const FGuid& GetGuid() const;

	TObjectPtr<AActor> Actor = nullptr;
	FGuid ActorGuid;

	ENGINE_API friend FArchive& operator<<(FArchive& Ar, FLevelInstanceActorGuid& LevelInstanceActorGuid);

private:
	ENGINE_API const FGuid& GetGuid_Internal() const;
};
