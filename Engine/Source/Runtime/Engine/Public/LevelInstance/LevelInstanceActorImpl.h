// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Templates/UnrealTemplate.h"

/**
 * Provides base implementation for AActor classes implementing the ILevelInstanceInterface
 *
 * Should be used as a member of the AActor class implementing ILevelInstanceInterface
 *
 * @see ILevelInstanceInterface, ALevelInstance
 */
class FLevelInstanceActorImpl
{
private:
	ILevelInstanceInterface* LevelInstance;
	FLevelInstanceID LevelInstanceID;

#if WITH_EDITOR
	FLevelInstanceID CachedLevelInstanceID;
	bool bCachedIsTemporarilyHiddenInEditor;
	bool bGuardLoadUnload;
	bool bAllowPartialLoading;
public:
	TSoftObjectPtr<UWorld> CachedWorldAsset;
#endif
public:
	// Exists only to support 'FVTableHelper' Actor constructors
	FLevelInstanceActorImpl()
		: FLevelInstanceActorImpl(nullptr)
	{
	}

	FLevelInstanceActorImpl(ILevelInstanceInterface* InLevelInstance)
		: LevelInstance(InLevelInstance)
#if WITH_EDITOR
		, bCachedIsTemporarilyHiddenInEditor(false)
		, bGuardLoadUnload(false)
		, bAllowPartialLoading(true)
#endif
	{
	}

	virtual ~FLevelInstanceActorImpl() = default;

	ENGINE_API virtual void RegisterLevelInstance();
	ENGINE_API virtual void UnregisterLevelInstance();

	/**
	 * Begin ILevelInstanceInterface Implementaion 
	 * 
	 */
	ENGINE_API const FLevelInstanceID& GetLevelInstanceID() const;
	ENGINE_API bool HasValidLevelInstanceID() const;
	ENGINE_API virtual bool IsLoadingEnabled() const;
	ENGINE_API virtual void OnLevelInstanceLoaded();

#if WITH_EDITOR
	ENGINE_API virtual bool SupportsPropertyOverrides() const;
	ENGINE_API virtual bool SupportsPartialEditorLoading() const;
	ENGINE_API virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists);
#endif

	/**
	 * Begin AActor Implementation
	 *
	 * Actor implementing ILevelInstanceInterface should be overriding most of those methods and forward the call their member FLevelInstanceActorImpl
	 * 
	 * @see ALevelInstance as an example
	 */
#if WITH_EDITOR
	ENGINE_API virtual void PreEditUndo(TFunctionRef<void()> SuperCall);
	ENGINE_API virtual void PostEditUndo(TFunctionRef<void()> SuperCall);
	ENGINE_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation, TFunctionRef<void(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)> SuperCall);
	ENGINE_API virtual void PostEditImport(TFunctionRef<void()> SuperCall);
	ENGINE_API virtual bool CanEditChange(const FProperty* Property) const;
	ENGINE_API virtual bool CanEditChangeComponent(const UActorComponent* Component, const FProperty* InProperty) const;
	ENGINE_API virtual void PreEditChange(FProperty* Property, bool bWorldAssetChange, TFunctionRef<void(FProperty*)> SuperCall);
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent, bool bWorldAssetChange, TFunction<void(FPropertyChangedEvent&)> SuperCall);
	ENGINE_API virtual bool CanDeleteSelectedActor(FText& OutReason) const;
	ENGINE_API virtual void SetIsTemporarilyHiddenInEditor(bool bIsHidden, TFunctionRef<void(bool)> SuperCall);
	ENGINE_API virtual bool SetIsHiddenEdLayer(bool bIsHiddenEdLayer, TFunctionRef<bool(bool)> SuperCall);
	ENGINE_API virtual void EditorGetUnderlyingActors(TSet<AActor*>& OutUnderlyingActors) const;
	ENGINE_API virtual bool IsLockLocation() const;
	ENGINE_API virtual bool IsActorLabelEditable() const;
	ENGINE_API virtual bool IsUserManaged() const;
	ENGINE_API virtual bool ShouldExport() const;
	ENGINE_API virtual bool GetBounds(FBox& OutBounds) const;
	ENGINE_API virtual bool GetEditorBounds(FBox& OutBounds) const;
	ENGINE_API virtual void PushSelectionToProxies();
	ENGINE_API virtual void PushLevelInstanceEditingStateToProxies(bool bInEditingState);
	ENGINE_API virtual void CheckForErrors();
	// End AActor Implementation
private:
	ENGINE_API virtual bool IsLockedActor() const;
	ENGINE_API virtual void PostEditUndoInternal();
#endif
};
