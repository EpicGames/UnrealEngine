// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "LevelInstance/LevelInstanceTypes.h"

#include "LevelInstanceComponent.generated.h"

/**
 * ULevelInstanceComponent subclasses USceneComponent for Editing purposes so that we can have a proxy to the LevelInstanceActor's RootComponent transform without attaching to it.
 *
 * It is responsible for updating the transform of the ALevelInstanceEditorInstanceActor which is created when loading a LevelInstance Instance Level
 *
 * We use this method to avoid attaching the Instance Level Actors to the ILevelInstanceInterface. (Cross level attachment and undo/redo pain)
 * 
 * The LevelInstance Level Actors are attached to this ALevelInstanceEditorInstanceActor keeping the attachment local to the Instance Level and shielded from the transaction buffer.
 *
 * Avoiding those Level Actors from being part of the Transaction system allows us to unload that level without clearing the transaction buffer. It also allows BP Reinstancing without having to update attachements.
 */
UCLASS(MinimalAPI)
class ULevelInstanceComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()
public:
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#endif
#if WITH_EDITOR
	// Those are the methods that need overriding to be able to properly update the AttachComponent
	ENGINE_API virtual void OnRegister() override;
	ENGINE_API virtual void PreEditUndo() override;
	ENGINE_API virtual void PostEditUndo() override;
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;

	ENGINE_API void UpdateEditorInstanceActor();
	ENGINE_API void OnEdit();
	ENGINE_API void OnEditOverrides();
	ENGINE_API void OnCommit();
	ENGINE_API void OnCommitOverrides();

	const FWorldPartitionActorFilter& GetFilter() const { return IsEditFilter() ? EditFilter : Filter; }
	ENGINE_API void SetFilter(const FWorldPartitionActorFilter& InFilter, bool bNotify = true);
	ENGINE_API const TMap<FActorContainerID, TSet<FGuid>>& GetFilteredActorsPerContainer() const;
	ENGINE_API void UpdateEditFilter();
	ENGINE_API void ClearCachedFilter();
private:
	ENGINE_API bool ShouldShowSpriteComponent() const;
	ENGINE_API void OnFilterChanged();
	ENGINE_API void SetActiveFilter(const FWorldPartitionActorFilter& InFilter);
	ENGINE_API bool IsEditFilter() const;

	friend class FLevelInstanceActorImpl;
	TWeakObjectPtr<AActor> CachedEditorInstanceActorPtr;
#endif

#if WITH_EDITORONLY_DATA
public:
	//Filters are only allowed on Embedded Level Instances Editor\Private LI per design.
	UPROPERTY()
	ELevelInstanceRuntimeBehavior ShadowDesiredRuntimeBehavior;

private:
	//, EditCondition="bFiltersAllowed"))
	UPROPERTY(EditAnywhere, Category = Filter, meta=(LevelInstanceFilter, DisableLevelInstancePropertyOverride, EditCondition = "ShadowDesiredRuntimeBehavior==ELevelInstanceRuntimeBehavior::Partitioned"))
	FWorldPartitionActorFilter Filter;

	UPROPERTY(EditAnywhere, Transient, Category = Filter, meta=(LevelInstanceEditFilter))
	FWorldPartitionActorFilter EditFilter;

	FWorldPartitionActorFilter UndoRedoCachedFilter;

	mutable FWorldPartitionActorFilter CachedFilter;
	mutable TOptional<TMap<FActorContainerID, TSet<FGuid>>> CachedFilteredActorsPerContainer;

	// Used to cancel the package getting dirty when editing the EditFilter which is transient
	bool bWasDirtyBeforeEditFilterChange;
#endif
};
