// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstanceEditorInstanceActor.generated.h"

class ILevelInstanceInterface;
class ULevel;
class USceneComponent;

/**
 * Editor Only Actor that is spawned inside every LevelInstance Instance Level so that we can update its Actor Transforms through the ILevelInstanceInterface's (ULevelInstanceComponent)
 * @see ULevelInstanceComponent
 */
UCLASS(transient, notplaceable, MinimalAPI)
class ALevelInstanceEditorInstanceActor : public AActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	static ENGINE_API ALevelInstanceEditorInstanceActor* Create(ILevelInstanceInterface* LevelInstance, ULevel* LoadedLevel);
	
	void SetLevelInstanceID(const FLevelInstanceID& InLevelInstanceID) { LevelInstanceID = InLevelInstanceID; }
	const FLevelInstanceID& GetLevelInstanceID() const { return LevelInstanceID; }
	
	virtual bool IsSelectionParentOfAttachedActors() const override { return true; }
	virtual bool IsSelectionChild() const override { return true; }
	ENGINE_API virtual AActor* GetSelectionParent() const override;

private:
	virtual bool ActorTypeIsMainWorldOnly() const override { return true; }
	friend class ULevelInstanceComponent;
	ENGINE_API void UpdateWorldTransform(const FTransform& WorldTransform);

	FLevelInstanceID LevelInstanceID;
#endif

#if WITH_EDITORONLY_DATA
private:
	// Level-space transforms for child actors whose root component has absolute flags.
	// Populated on the first UpdateWorldTransform call (before LevelTransform is modified)
	// so all actors are present and abs flags are set. Used on subsequent calls to avoid
	// floating-point error drift across repeated moves of the level instance.
	UPROPERTY(Transient)
	TMap<TObjectPtr<USceneComponent>, FTransform> LevelSpaceTransformCache;
	bool bCacheBuilt = false;
#endif
};
