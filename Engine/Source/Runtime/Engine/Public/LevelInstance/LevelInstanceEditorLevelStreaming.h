// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "LevelInstanceEditorLevelStreaming.generated.h"

class ILevelInstanceInterface;

UCLASS(Transient, MinimalAPI)
class ULevelStreamingLevelInstanceEditor : public ULevelStreamingAlwaysLoaded
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual bool ShowInLevelCollection() const override { return true; }
	virtual bool IsUserManaged() const override { return false; }
	ILevelInstanceInterface* GetLevelInstance() const;
	const FLevelInstanceID& GetLevelInstanceID() const { return LevelInstanceID; }
	FBox GetBounds() const;

	virtual TOptional<FFolder::FRootObject> GetFolderRootObject() const override;
protected:
	void OnLevelActorAdded(AActor* InActor);
	void OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors);
	void OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance);

	friend class ULevelInstanceSubsystem;

	static ULevelStreamingLevelInstanceEditor* Load(ILevelInstanceInterface* LevelInstance);
	static void Unload(ULevelStreamingLevelInstanceEditor* LevelStreaming);

	virtual void OnLevelLoadedChanged(ULevel* Level) override;
private:
	FLevelInstanceID LevelInstanceID;

	// When creating a new LevelInstance initialize UActorDescContainerInstance using those values
	UActorDescContainerInstance* ParentContainerInstance = nullptr;
	FGuid ParentContainerGuid;
#endif
};