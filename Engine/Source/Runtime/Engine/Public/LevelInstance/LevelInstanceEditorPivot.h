// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "LevelInstance/LevelInstanceTypes.h"

class ILevelInstanceEditorPivotInterface;
class ILevelInstanceInterface;
class ULevelStreaming;
class AActor;

class FLevelInstanceEditorPivotHelper
{
public:
	static ENGINE_API ILevelInstanceEditorPivotInterface* Create(ILevelInstanceInterface* LevelInstance, ULevelStreaming* LevelStreaming);
	static ENGINE_API void SetPivot(ILevelInstanceEditorPivotInterface* PivotInterface, ELevelInstancePivotType PivotType, AActor* PivotToActor);
	static ENGINE_API FVector GetPivot(ILevelInstanceEditorPivotInterface* PivotInterface, ELevelInstancePivotType PivotType, AActor* PivotToActor);
	static ENGINE_API void ShowPivotLocation(const FVector& PivotLocation);
};

#endif
