// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstance/LevelInstanceEditorPivotInterface.h"
#include "LevelInstanceEditorPivotActor.generated.h"

UCLASS(transient, notplaceable, hidecategories=(Rendering,Replication,Collision,Partition,Input,HLOD,Actor,Cooking), MinimalAPI)
class ALevelInstancePivot : public AActor, public ILevelInstanceEditorPivotInterface
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	virtual bool CanDeleteSelectedActor(FText& OutReason) const override { return false; }
	ENGINE_API virtual void PostEditMove(bool bFinished) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditUndo() override;
	virtual bool ShouldExport() override { return false; }
private:
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
	virtual bool ActorTypeSupportsExternalDataLayer() const override { return false; }
#endif
};
