// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "LevelInstanceEditorPivotInterface.generated.h"

UINTERFACE(MinimalAPI)
class ULevelInstanceEditorPivotInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Interface to be implemented by Actor classes to implement support for LevelInstance Editor Pivot
 */
class ILevelInstanceEditorPivotInterface
{
	GENERATED_IINTERFACE_BODY()
#if WITH_EDITOR
public:
	struct FInitState
	{
		FLevelInstanceID LevelInstanceID;
		FTransform ActorTransform;
		FVector LevelOffset;
	};

	void SetInitState(const FInitState& InInitState) { InitState = InInitState; }
	const FLevelInstanceID& GetLevelInstanceID() const { return InitState.LevelInstanceID; }

	ENGINE_API virtual void UpdateOffset();
protected:
	FInitState InitState;
#endif
};
