// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UObject/SoftObjectPtr.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "LevelInstanceInterface.generated.h"

class ULevelInstanceComponent;
class ULevelStreamingLevelInstance;

UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class ULevelInstanceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()

};

/**
 * Interface to be implemented by Actor classes to implement support for LevelInstances
 *
 * Interface exists to allow integration of the LevelInstance workflow on existing Actor class hierarchies.
 * 
 * @see FLevelInstanceActorImpl to use as a member or base class member of the implementing class. It provides most of the boiler plate code so that the implementing class behaves properly in editor (Undo/Redo/Delete/Selection/Imporrt/etc)
 */

class ILevelInstanceInterface
{
	GENERATED_IINTERFACE_BODY()
			
	// Pure Interface Start
	virtual const FLevelInstanceID& GetLevelInstanceID() const = 0;

	virtual bool HasValidLevelInstanceID() const = 0;

	virtual const FGuid& GetLevelInstanceGuid() const = 0;

	
	UFUNCTION(BlueprintCallable, Category = Default)
	virtual const TSoftObjectPtr<UWorld>& GetWorldAsset() const = 0;

	virtual bool IsLoadingEnabled() const = 0;

#if WITH_EDITOR
	virtual bool SupportsPropertyOverrides() const { return false; }
		
	virtual ULevelInstancePropertyOverrideAsset* GetPropertyOverrideAsset() const { return nullptr; }

	virtual ULevelInstanceComponent* GetLevelInstanceComponent() const = 0;

	virtual ELevelInstanceRuntimeBehavior GetDesiredRuntimeBehavior() const = 0;

	virtual ELevelInstanceRuntimeBehavior GetDefaultRuntimeBehavior() const = 0;

	//Default assumes it's a constant Behavior. This is needed to allow\deny the editor of actor layers at the component level. You must implement this method if your LevelInstance child support multiple runtime behavior. 
	virtual void SetDesiredRuntimeBehavior(ELevelInstanceRuntimeBehavior NewBehavior) { checkf(false, TEXT("SetDesiredRuntimeBehavior should not be called on this child of ILevelInstanceInterface. Please implement an override if required.")); }

	virtual TSubclassOf<AActor> GetEditorPivotClass() const { return nullptr; }
	
	virtual void AdjustPivotOnCreation() { }

	ENGINE_API virtual bool SupportsPartialEditorLoading() const;
#endif
	// Pure Interface End
		
	virtual void OnLevelInstanceLoaded() {}
		
	FString GetWorldAssetPackage() const { return GetWorldAsset().GetUniqueID().GetLongPackageName(); }

	bool IsWorldAssetValid() const { return GetWorldAsset().GetUniqueID().IsValid(); }

	ENGINE_API virtual ULevelInstanceSubsystem* GetLevelInstanceSubsystem() const;

	UFUNCTION(BlueprintCallable, Category = Default)
	ENGINE_API virtual bool IsLoaded() const;
	
	UFUNCTION(BlueprintCallable, Category = Default)
	ENGINE_API virtual void LoadLevelInstance();

	UFUNCTION(BlueprintCallable, Category = Default)
	ENGINE_API virtual void UnloadLevelInstance();

	virtual bool IsInitiallyVisible() const { return true; }

	ENGINE_API virtual ULevelStreamingLevelInstance* GetLevelStreaming() const;

	ENGINE_API virtual TSubclassOf<ULevelStreamingLevelInstance> GetLevelStreamingClass() const;

	UFUNCTION(BlueprintCallable, Category = Default)
	ENGINE_API virtual ULevel* GetLoadedLevel() const;

#if WITH_EDITOR
	ENGINE_API virtual void OnEdit();
	ENGINE_API virtual void OnEditPropertyOverrides();

	virtual void OnEditChild() {}

	ENGINE_API virtual void OnCommit(bool bChanged);
	ENGINE_API virtual void OnCommitPropertyOverrides(bool bChanged);

	virtual void OnCommitChild(bool bChanged) {}
	
	bool IsInAnyEditMode() const { return IsEditing() || IsEditingPropertyOverrides(); }

	ENGINE_API bool IsEditingPropertyOverrides() const;

	ENGINE_API virtual bool IsEditing() const;
	
	ENGINE_API virtual bool HasChildEdit() const;

	ENGINE_API virtual bool HasParentEdit() const;
	
	ENGINE_API virtual bool IsDirty() const;
	
	ENGINE_API virtual bool HasDirtyChildren() const;
	
	ENGINE_API virtual bool CanEnterEdit(FText* OutReason = nullptr) const;

	ENGINE_API bool CanEnterEditPropertyOverrides(FText* OutReason = nullptr) const;
	
	ENGINE_API virtual bool EnterEdit(AActor* ContextActor = nullptr);

	ENGINE_API bool EnterEditPropertyOverrides(AActor* ContextActor = nullptr);
	
	ENGINE_API virtual bool CanExitEdit(bool bDiscardEdits = false, FText* OutReason = nullptr) const;

	ENGINE_API bool CanExitEditPropertyOverrides(bool bDiscardEdits = false, FText* OutReason = nullptr) const;
	
	ENGINE_API virtual bool ExitEdit(bool bDiscardEdits = false);

	ENGINE_API bool ExitEditPropertyOverrides(bool bDiscardEdits = false);
			
	ENGINE_API virtual bool SetCurrent();
	
	ENGINE_API virtual bool MoveActorsTo(const TArray<AActor*>& ActorsToMove);

	ENGINE_API virtual const FWorldPartitionActorFilter& GetFilter() const;

	ENGINE_API virtual const TMap<FActorContainerID, TSet<FGuid>>& GetFilteredActorsPerContainer() const;

	ENGINE_API virtual void SetFilter(const FWorldPartitionActorFilter& InFilter, bool bNotify = true);

	virtual void OnFilterChanged() {}

	// Return supported filter types when using filter for loading actors
	ENGINE_API virtual EWorldPartitionActorFilterType GetLoadingFilterTypes() const;

	// Return supported filter types when setting filter through details panel
	ENGINE_API virtual EWorldPartitionActorFilterType GetDetailsFilterTypes() const;

	// Whether the level instance is HLOD relevant. If not, all it's actors will be excluded from HLOD.
	ENGINE_API virtual bool IsLevelInstanceHLODRelevant() const;
	
	ENGINE_API virtual bool HasStandaloneHLOD() const;
#endif

	/** Sets the UWorld asset reference when loading a LevelInstance */
	UFUNCTION(BlueprintCallable, Category = Default)
	virtual bool SetWorldAsset(TSoftObjectPtr<UWorld> WorldAsset)
	{
		return false;
	}

	ENGINE_API virtual void UpdateLevelInstanceFromWorldAsset();

#if WITH_EDITOR
private:
	friend class ULevelInstanceSubsystem;
	virtual void SetPropertyOverrideAsset(ULevelInstancePropertyOverrideAsset* InPropertyOverride) {}
#endif
};
