// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/LevelStreamingDynamic.h"
#include "LevelInstance/LevelInstanceTypes.h"

#if WITH_EDITOR
#include "WorldPartition/ActorDescContainerInstance.h"
#endif

#include "LevelInstanceLevelStreaming.generated.h"

class ILevelInstanceInterface;
class ALevelInstanceEditorInstanceActor;

UCLASS(Transient, MinimalAPI)
class ULevelStreamingLevelInstance : public ULevelStreamingDynamic
{
	GENERATED_UCLASS_BODY()

public:
	ENGINE_API ILevelInstanceInterface* GetLevelInstance() const;

#if WITH_EDITOR
	virtual bool ShowInLevelCollection() const override { return false; }
	virtual bool IsUserManaged() const override { return false; }

	ENGINE_API FBox GetBounds() const;

	ENGINE_API virtual TOptional<FFolder::FRootObject> GetFolderRootObject() const override;
#endif
	
protected:
	static ENGINE_API ULevelStreamingLevelInstance* LoadInstance(ILevelInstanceInterface* LevelInstanceActor);
	static ENGINE_API void UnloadInstance(ULevelStreamingLevelInstance* LevelStreaming);

	ENGINE_API virtual void OnLevelLoadedChanged(ULevel* Level) override;

	friend class ULevelInstanceSubsystem;

	const FLevelInstanceID& GetLevelInstanceID() const { return LevelInstanceID; }
private:
#if WITH_EDITOR
	void ResetLevelInstanceLoaders();
	
	ENGINE_API virtual void OnLoadedActorsAddedToLevelPreEvent(const TArray<AActor*>& InActors);
	ENGINE_API virtual void OnLoadedActorsAddedToLevelPostEvent(const TArray<AActor*>& InActors);
	ENGINE_API virtual void OnLoadedActorsRemovedFromLevelPostEvent(const TArray<AActor*>& InActors);
	ENGINE_API virtual void OnCurrentStateChanged(ELevelStreamingState InPrevState, ELevelStreamingState InNewState) override;
	ENGINE_API void OnPreInitializeContainerInstance(UActorDescContainerInstance::FInitializeParams& InInitParams, UActorDescContainerInstance* InContainerInstance);
	void InitializeActors(const TArray<AActor*>& InActors);
	void OnActorReplacedEvent(FWorldPartitionActorDescInstance* InActorDescInstance);

	enum EApplyPropertieOverrideType
	{
		PreConstruction,
		PostConstruction,
		PreAndPostConstruction
	};

	void ApplyPropertyOverrides(const TArray<AActor*>& InActor, bool bInAlreadyAppliedTransformOnActors, EApplyPropertieOverrideType InApplyPropertyOverrideType);

	bool IsEditorWorldMode() const;

	TWeakObjectPtr<ALevelInstanceEditorInstanceActor> LevelInstanceEditorInstanceActor;

	mutable FTransform CachedTransform;
	mutable FBox CachedBounds;
	bool bResetLoadersCalled;
#endif
	FLevelInstanceID LevelInstanceID;
};
