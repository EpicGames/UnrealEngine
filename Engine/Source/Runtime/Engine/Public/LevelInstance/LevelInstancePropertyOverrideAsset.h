// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"
#include "LevelInstance/LevelInstanceTypes.h"

#include "LevelInstancePropertyOverrideAsset.generated.h"

class ULevelStreamingLevelInstanceEditorPropertyOverride;
class ILevelInstanceInterface;
class ULevelStreamingLevelInstanceEditorPropertyOverride;
class FLevelInstancePropertyOverrideDesc;

struct FLevelInstanceActorPropertyOverride
{
	FLevelInstanceActorPropertyOverride(const FLevelInstanceID& InLevelInstanceID, const FActorPropertyOverride* InActorPropertyOverride)
		: LevelInstanceID(InLevelInstanceID)
		, ActorPropertyOverride(InActorPropertyOverride) {}

	FLevelInstanceID LevelInstanceID;
	const FActorPropertyOverride* ActorPropertyOverride = nullptr;
};

UCLASS(MinimalAPI, NotBlueprintable)
class ULevelInstancePropertyOverrideAsset : public UWorldPartitionPropertyOverride
{
	GENERATED_BODY()
public:
	ULevelInstancePropertyOverrideAsset() {}
	virtual ~ULevelInstancePropertyOverrideAsset() = default;
		
#if WITH_EDITOR
	const TSoftObjectPtr<UWorld>& GetWorldAsset() const { return WorldAsset; }

	// Remove the container path from overrides.
	void RemoveContainerOverride(const FActorContainerPath& InContainerPath) { PropertyOverridesPerContainer.Remove(InContainerPath); }

	// Remove ActorOverrides under InContainerPath whose actor GUIDs are not present in InActorDescs.
	// If pruning leaves no remaining ActorOverrides at the path, the container entry itself is removed.
	void PruneStaleActorOverrides(const FActorContainerPath& InContainerPath, const TMap<FGuid, TSharedPtr<class FWorldPartitionActorDesc>>& InActorDescs);

private:
	// Begin UWorldPartitionPropertyOverride Interface
	using UWorldPartitionPropertyOverride::ApplyPropertyOverrides;
	// End UWorldPartitionPropertyOverride Interface

	static bool SerializeActorPropertyOverrides(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming, AActor* InActor, bool bForReset, FActorPropertyOverride& OutActorPropertyOverrides, const TMap<UObject*, UObject*>* InInstanceToArchetypeMap = nullptr, const TMap<UObject*, UObject*>* InArchetypeToInstanceMap = nullptr);

	void Initialize(const TSoftObjectPtr<UWorld> InWorldAsset) { WorldAsset = InWorldAsset; }
	void SerializePropertyOverrides(ILevelInstanceInterface* InLevelInstanceOverrideOwner, ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreamingInterface);
	void ResetPropertyOverridesForActor(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreamingInterface, AActor* InActor);
	
	// Return non instanced SoftObjectPtr to this Object
	TSoftObjectPtr<ULevelInstancePropertyOverrideAsset> GetSourceAssetPtr() const;

	// Return FActorContainerPath to InChild relative to InParent
	FActorContainerPath GetContainerPropertyOverridePath(ILevelInstanceInterface* InParent, ILevelInstanceInterface* InChild);

	friend class ULevelInstanceSubsystem;
	friend class ULevelStreamingLevelInstance;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
	friend class FLevelInstancePropertyOverrideDesc;
#endif

#if WITH_EDITORONLY_DATA
	// Not editable for now, user can reset the property overrides on its owning level instance to change the loaded world
	UPROPERTY(VisibleAnywhere, Category = Level, meta = (NoCreate, DisplayName = "Level", ToolTip = "Reset Property Override to change level"))
	TSoftObjectPtr<UWorld> WorldAsset;

	UPROPERTY(Transient)
	bool bSavingOverrideEdit = false;
#endif
};




