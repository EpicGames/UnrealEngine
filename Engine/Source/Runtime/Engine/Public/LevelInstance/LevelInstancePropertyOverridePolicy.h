// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "WorldPartition/WorldPartitionPropertyOverride.h"
#include "LevelInstancePropertyOverridePolicy.generated.h"

UCLASS(Abstract, NotBlueprintable)
class ULevelInstancePropertyOverridePolicy : public UWorldPartitionPropertyOverridePolicy
{
	GENERATED_BODY()
public:
	ENGINE_API ULevelInstancePropertyOverridePolicy(FVTableHelper& Helper);
	ULevelInstancePropertyOverridePolicy() = default;
	virtual ~ULevelInstancePropertyOverridePolicy() = default;

#if WITH_EDITOR
	ENGINE_API virtual bool CanOverrideProperty(const FProperty* InProperty) const override final;

protected:
	virtual bool CanOverridePropertyImpl(const FProperty* InProperty) const { return true; }
#endif
};
