// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreFwd.h"
#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "LevelInstance/LevelInstancePropertyOverridePolicy.h"
#include "LevelInstanceSettings.generated.h"

#define UE_API ENGINE_API

UCLASS(MinimalAPI, config = Engine, defaultconfig)
class ULevelInstanceSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULevelInstanceSettings();
	
	static ULevelInstanceSettings* Get() { return CastChecked<ULevelInstanceSettings>(ULevelInstanceSettings::StaticClass()->GetDefaultObject()); }

#if WITH_EDITOR
	UE_API bool IsLevelInstancePropertyOverrideSettingEnabled() const;
	UE_API bool IsPropertyOverridePolicyDefined() const;
	UE_API bool IsPropertyOverrideEnabled() const;

	UE_API bool IsLevelInstanceEditCompatibleWithLandscapeEdit() const;

	UE_DEPRECATED(5.5, "Use SetIsLevelInstanceDisabled instead.")
	void DisableLevelInstanceSupport() { bIsLevelInstanceDisabled = true; }
	void SetIsLevelInstanceDisabled(bool bValue) { bIsLevelInstanceDisabled = bValue; }
	bool IsLevelInstanceDisabled() const { return bIsLevelInstanceDisabled; }
private:
	friend class ULevelInstanceSubsystem;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
		
	UE_API void UpdatePropertyOverridePolicy();
	ULevelInstancePropertyOverridePolicy* GetPropertyOverridePolicy() const { return PropertyOverridePolicy; }
#endif

protected:
	// Keep out of WITH_EDITORONLY_DATA so that it can be properly set in -game
	UPROPERTY(config)
	FString PropertyOverridePolicyClass;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<ULevelInstancePropertyOverridePolicy> PropertyOverridePolicy;

	UPROPERTY(Transient)
	bool bIsLevelInstanceDisabled;
#endif
};

#undef UE_API
