// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "UObject/GCObject.h"
#include "Folder.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "LevelInstance/LevelInstanceTypes.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "LevelInstance/LevelInstancePropertyOverrideAsset.h"
#include "Streaming/StreamingWorldSubsystemInterface.h"

#if WITH_EDITOR
#include "EditorLevelUtils.h"
#include "Internationalization/Text.h"
#endif

#include "LevelInstanceSubsystem.generated.h"

class ILevelInstanceInterface;
class ULevelInstanceEditorObject;
class ULevelStreamingLevelInstance;
class ULevelStreamingLevelInstanceEditor;
class ULevelStreamingLevelInstanceEditorPropertyOverride;
class UWorldPartitionSubsystem;
class UBlueprint;
struct FActorPropertyOverride;

enum class ELevelInstanceBreakFlags : uint8
{
	None = 0,

	/**
	 * The actors will be placed inside the folder the LI is inside of, under a subfolder with the name of the
	 * Level Instance, and also keeping their original folder structure.
	 * So if i.e. the Level Instance Actor is called "Desert/LI_House2", and an actor inside is named "Lights/Light_Sun",
	 * the actor will be moved to "Desert/LI_House2/Lights/Light_Sun" in the outer level.
	 *
	 * If this flag is not set, actors will be placed either in the root folder of the outer level (but their original
	 * folders from the LI kept), or, if context folder is set, they'll be moved there without any subfolders.
	 */
	KeepFolders = 1 << 0,
};
ENUM_CLASS_FLAGS(ELevelInstanceBreakFlags);

/**
 * ULevelInstanceSubsystem
 */
UCLASS(MinimalAPI)
class ULevelInstanceSubsystem : public UWorldSubsystem, public IStreamingWorldSubsystemInterface
{
	GENERATED_BODY()

public:
	ENGINE_API ULevelInstanceSubsystem();
	ENGINE_API virtual ~ULevelInstanceSubsystem();

	//~ Begin UObject Interface
	static ENGINE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface
		
	//~ Begin USubsystem Interface.
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize();
	ENGINE_API virtual bool DoesSupportWorldType(EWorldType::Type WorldType) const override;
	//~ End USubsystem Interface.
	
	//~Begin IStreamingWorldSubsystemInterface
	ENGINE_API virtual void OnUpdateStreamingState() override;
	//~End IStreamingWorldSubsystemInterface

	ENGINE_API ILevelInstanceInterface* GetLevelInstance(const FLevelInstanceID& LevelInstanceID) const;
	ENGINE_API ILevelInstanceInterface* GetOwningLevelInstance(const ULevel* Level) const;
	ENGINE_API FLevelInstanceID RegisterLevelInstance(ILevelInstanceInterface* LevelInstance);
	ENGINE_API static ULevel* GetOwningLevel(const ULevel* Level, bool bFollowChainToNonLevelInstanceOwningLevel = false);
	ENGINE_API void UnregisterLevelInstance(ILevelInstanceInterface* LevelInstance);
	ENGINE_API void RequestLoadLevelInstance(ILevelInstanceInterface* LevelInstance, bool bUpdate);
	ENGINE_API void RequestUnloadLevelInstance(ILevelInstanceInterface* LevelInstance);
	ENGINE_API bool IsLoaded(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API bool IsLoading(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API void ForEachLevelInstanceAncestors(const AActor* Actor, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const;
	ENGINE_API void ForEachLevelInstanceAncestors(AActor* Actor, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const;
	ENGINE_API void ForEachLevelInstanceAncestorsAndSelf(AActor* Actor, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const;
	/** Runs a lambda operation along the ancestors that own the LevelInstance. Primarily for capturing inclusive true/false by using lambda captures */
	ENGINE_API void ForEachLevelInstanceAncestorsAndSelf(const AActor* Actor, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const;
	ENGINE_API ULevelStreamingLevelInstance* GetLevelInstanceLevelStreaming(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API void ForEachActorInLevelInstance(const ILevelInstanceInterface* LevelInstance, TFunctionRef<bool(AActor* LevelActor)> Operation) const;
	ENGINE_API ULevel* GetLevelInstanceLevel(const ILevelInstanceInterface* LevelInstance) const;
	/** Checks if a WorldAsset can/should be used in a LevelInstance */
	static ENGINE_API bool CanUseWorldAsset(const ILevelInstanceInterface* LevelInstance, TSoftObjectPtr<UWorld> WorldAsset, FString* OutReason);
	/**
	Lambda expr format that checks ancestor owners of a LevelInstance.
	Detects if the WorldAsset shares the same package as the current LevelInstance or any of its ancestors owning the LevelInstance.
	Used in conjunction with ForEachLevelInstanceAncestorsAndSelf to find a loop using a boolean lambda capture.
	*/
	static ENGINE_API bool CheckForLoop(const ILevelInstanceInterface* LevelInstance, TSoftObjectPtr<UWorld> WorldAsset, TArray<TPair<FText, TSoftObjectPtr<UWorld>>>* LoopInfo = nullptr, const ILevelInstanceInterface** LoopStart = nullptr);

#if WITH_EDITOR
	ENGINE_API void Tick();
	ENGINE_API void OnExitEditorMode();
	ENGINE_API void OnTryExitEditorMode();

	UE_DEPRECATED(5.3, "Use FPackedLevelActorUtils::PackAllLoadedActors")
	void PackAllLoadedActors() {}
	UE_DEPRECATED(5.3, "Use FPackedLevelActorUtils::CanPackAllLoadedActors")
	bool CanPackAllLoadedActors() const { return false;}

	ENGINE_API ILevelInstanceInterface* GetEditingLevelInstance() const;
	ENGINE_API bool CanEditLevelInstance(const ILevelInstanceInterface* LevelInstance, FText* OutReason = nullptr) const;
	ENGINE_API bool CanCommitLevelInstance(const ILevelInstanceInterface* LevelInstance, bool bDiscardEdits = false, FText* OutReason = nullptr) const;
	ENGINE_API void EditLevelInstance(ILevelInstanceInterface* LevelInstance, TWeakObjectPtr<AActor> ContextActorPtr = nullptr);
	ENGINE_API bool CommitLevelInstance(ILevelInstanceInterface* LevelInstance, bool bDiscardEdits = false, TSet<FName>* DirtyPackages = nullptr);
	ENGINE_API bool IsEditingLevelInstanceDirty(const ILevelInstanceInterface* LevelInstance) const;
	bool IsEditingLevelInstance(const ILevelInstanceInterface* LevelInstance) const { return GetLevelInstanceEdit(LevelInstance) != nullptr; }
	
	ENGINE_API bool GetLevelInstanceEditorBounds(const ILevelInstanceInterface* LevelInstance, FBox& OutBounds) const;
	ENGINE_API bool GetLevelInstanceBounds(const ILevelInstanceInterface* LevelInstance, FBox& OutBounds) const;
	static ENGINE_API bool GetLevelInstanceBoundsFromPackage(const FTransform& InstanceTransform, FName LevelPackage, FBox& OutBounds);
	
	ENGINE_API void ForEachLevelInstanceChild(const ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const;
	ENGINE_API void ForEachLevelInstanceChild(ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const;
	ENGINE_API bool HasDirtyChildrenLevelInstances(const ILevelInstanceInterface* LevelInstance) const;
	
	ENGINE_API void SetIsHiddenEdLayer(ILevelInstanceInterface* LevelInstance, bool bIsHiddenEdLayer);
	ENGINE_API void SetIsTemporarilyHiddenInEditor(ILevelInstanceInterface* LevelInstance, bool bIsHidden);

	ENGINE_API bool SetCurrent(ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API bool IsCurrent(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API ILevelInstanceInterface* CreateLevelInstanceFrom(const TArray<AActor*>& ActorsToMove, const FNewLevelInstanceParams& CreationParams);
	ENGINE_API bool CanCreateLevelInstanceFrom(const TArray<AActor*>& ActorsToMove, FText* OutReason = nullptr);
	ENGINE_API bool MoveActorsToLevel(const TArray<AActor*>& ActorsToRemove, ULevel* DestinationLevel, TArray<AActor*>* OutActors = nullptr) const;
	ENGINE_API bool MoveActorsTo(ILevelInstanceInterface* LevelInstance, const TArray<AActor*>& ActorsToMove, TArray<AActor*>* OutActors = nullptr);
	ENGINE_API bool BreakLevelInstance(ILevelInstanceInterface* LevelInstance, uint32 Levels = 1, TArray<AActor*>* OutMovedActors = nullptr, ELevelInstanceBreakFlags Flags = ELevelInstanceBreakFlags::None);
	ENGINE_API bool CanBreakLevelInstance(const ILevelInstanceInterface* LevelInstance) const;

	ENGINE_API bool CanMoveActorToLevel(const AActor* Actor, FText* OutReason = nullptr) const;
	ENGINE_API void OnActorDeleted(AActor* Actor);

	ENGINE_API bool LevelInstanceHasLevelScriptBlueprint(const ILevelInstanceInterface* LevelInstance) const;

	ENGINE_API ILevelInstanceInterface* GetParentLevelInstance(const AActor* Actor) const;
	ENGINE_API void BlockLoadLevelInstance(ILevelInstanceInterface* LevelInstance);
	ENGINE_API void BlockUnloadLevelInstance(ILevelInstanceInterface* LevelInstance);
		
	ENGINE_API bool HasChildEdit(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API bool HasParentEdit(const ILevelInstanceInterface* LevelInstance) const;
	
	ENGINE_API TArray<ILevelInstanceInterface*> GetLevelInstances(const FString& WorldAssetPackage) const;
		
	// Returns the upper chain of level instance actors for the specified level starting with the level instance referencing the level
	ENGINE_API void ForEachLevelInstanceActorAncestors(const ULevel* Level, TFunctionRef<bool(AActor*)> Operation) const;
	ENGINE_API TArray<AActor*> GetParentLevelInstanceActors(const ULevel* Level) const;
	ENGINE_API FString PrefixWithParentLevelInstanceActorLabels(const FString& ActorLabel, const ULevel* Level) const;

	static ENGINE_API bool CheckForLoop(const ILevelInstanceInterface* LevelInstance, TArray<TPair<FText, TSoftObjectPtr<UWorld>>>* LoopInfo = nullptr, const ILevelInstanceInterface** LoopStart = nullptr);
	
	UE_DEPRECATED(5.3, "CanUsePackage is deprecated.")
	static bool CanUsePackage(FName InPackageName) { return true;  }

	/** Editor-only event triggered when level instance is committed with changes */
	DECLARE_EVENT_OneParam(ULevelInstanceSubsystem, FLevelInstanceChanged, FName);
	FLevelInstanceChanged& OnLevelInstanceChanged() { return LevelInstanceChangedEvent; }

	/** Editor-only event triggered when level instances are reloaded after a change */
	DECLARE_EVENT_OneParam(ULevelInstanceSubsystem, FLevelInstancesUpdated, const TArray<ILevelInstanceInterface*>&);
	FLevelInstancesUpdated& OnLevelInstancesUpdated() { return LevelInstancesUpdatedEvent; }

	DECLARE_EVENT_TwoParams(ULevelInstanceSubsystem, FLevelInstanceEditCancelled, ILevelInstanceInterface*, bool);
	FLevelInstanceEditCancelled& OnLevelInstanceEditCancelled() { return LevelInstanceEditCancelled; }

	static ENGINE_API void ResetLoadersForWorldAsset(const FString& WorldAsset);

	ENGINE_API bool PassLevelInstanceFilter(UWorld* World, const FWorldPartitionHandle& Actor) const;

	ENGINE_API bool IsEditingLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance) const;

	ENGINE_API bool IsSubSelectionEnabled() const;
private:
	friend class FLevelInstanceEditorModule;
	friend struct FLevelInstanceActorDetailsHelper;
	friend struct FLevelInstanceEditorModeToolkitHelper;
	friend struct FLevelInstanceMenuUtils;
	friend class ILevelInstanceInterface;
	friend class ULevelInstanceEditorMode;
	friend class ULevelStreamingLevelInstanceEditor;
	friend class ULevelStreamingLevelInstanceEditorPropertyOverride;
	friend class ULevelInstancePropertyOverrideAsset;

	// EditPropertyOverrides
	ENGINE_API TArray<ILevelInstanceInterface*> GetLevelInstances(const TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>& PropertyOverrideAsset) const;
	ENGINE_API bool HasParentPropertyOverridesEdit(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API ILevelInstanceInterface* GetEditingPropertyOverridesLevelInstance() const;
	ENGINE_API bool CanCommitLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance, bool bDiscardEdits = false, FText* OutReason = nullptr) const;
	ENGINE_API bool CanEditLevelInstancePropertyOverrides(const ILevelInstanceInterface* LevelInstance, FText* OutReason = nullptr) const;
	ENGINE_API void EditLevelInstancePropertyOverrides(ILevelInstanceInterface* LevelInstance, AActor* ContextActor);
	ENGINE_API bool CommitLevelInstancePropertyOverrides(ILevelInstanceInterface* LevelInstance, bool bDiscardEdits = false);
	ENGINE_API bool CanResetPropertyOverridesForActor(AActor* Actor) const;
	ENGINE_API void ResetPropertyOverridesForActor(AActor* Actor);
	ENGINE_API bool CanResetPropertyOverrides(ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API void ResetPropertyOverrides(ILevelInstanceInterface* LevelInstance);

	ENGINE_API static void RegisterPrimitiveColorHandler();
	ENGINE_API static void UnregisterPrimitiveColorHandler();

	ENGINE_API bool GetLevelInstanceBoundsInternal(const ILevelInstanceInterface* LevelInstance, bool bIsEditorBounds, FBox& OutBounds) const;
#endif
	friend class ULevelStreamingLevelInstance;

private:
	ENGINE_API void UpdateStreamingStateInternal();
	ENGINE_API void BlockOnLoading();
	ENGINE_API void LoadLevelInstance(ILevelInstanceInterface* LevelInstance);
	ENGINE_API void UnloadLevelInstance(const FLevelInstanceID& LevelInstanceID);
	ENGINE_API void ForEachActorInLevel(ULevel* Level, TFunctionRef<bool(AActor * LevelActor)> Operation) const;
	ENGINE_API void RegisterLoadedLevelStreamingLevelInstance(ULevelStreamingLevelInstance* LevelStreaming);

#if WITH_EDITOR
	ENGINE_API ULevelStreamingLevelInstanceEditor* CreateNewStreamingLevelForWorld(UWorld& InWorld, const EditorLevelUtils::FCreateNewStreamingLevelForWorldParams& InParams);

	ENGINE_API void ResetLoadersForWorldAssetInternal(const FString& WorldAsset);
	ENGINE_API void OnAssetsPreDelete(const TArray<UObject*>& Objects);
	ENGINE_API void OnPreSaveWorldWithContext(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext);
	ENGINE_API void OnPreWorldRename(UWorld* InWorld, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename);
	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);

	ENGINE_API void RegisterLoadedLevelStreamingLevelInstanceEditor(ULevelStreamingLevelInstanceEditor* LevelStreaming);
	
	ENGINE_API void OnEditChild(const FLevelInstanceID& LevelInstanceID);
	ENGINE_API void OnCommitChild(const FLevelInstanceID& LevelInstanceID, bool bChildChanged);

	ENGINE_API bool ForEachLevelInstanceChildImpl(const ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(const ILevelInstanceInterface*)> Operation) const;
	ENGINE_API bool ForEachLevelInstanceChildImpl(ILevelInstanceInterface* LevelInstance, bool bRecursive, TFunctionRef<bool(ILevelInstanceInterface*)> Operation) const;

	ENGINE_API void BreakLevelInstance_Impl(ILevelInstanceInterface* LevelInstance, uint32 Levels, TArray<AActor*>& OutMovedActors, ELevelInstanceBreakFlags Flags);

	void BroadcastOnLevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstance, ILevelInstanceEditorModule::ELevelInstanceEditMode NewMode);

	static ENGINE_API bool ShouldIgnoreDirtyPackage(UPackage* DirtyPackage, const UWorld* EditingWorld);

	class FLevelInstanceEdit
	{
	public:
		TObjectPtr<ULevelStreamingLevelInstanceEditor> LevelStreaming;
		TObjectPtr<ULevelInstanceEditorObject> EditorObject;
		TObjectPtr<AActor> LevelInstanceActor;

		FLevelInstanceEdit(ULevelStreamingLevelInstanceEditor* InLevelStreaming, ILevelInstanceInterface* InLevelInstance);
		~FLevelInstanceEdit();

		UWorld* GetEditWorld() const;
		ILevelInstanceInterface* GetLevelInstance() const;

		void AddReferencedObjects(FReferenceCollector& Collector);

		void GetPackagesToSave(TArray<UPackage*>& OutPackagesToSave) const;
		bool CanDiscard(FText* OutReason = nullptr) const;
		bool HasCommittedChanges() const;
		void MarkCommittedChanges();
	};

	class FPropertyOverrideEdit
	{
	public:
		FPropertyOverrideEdit(ULevelStreamingLevelInstanceEditorPropertyOverride* InLevelStreaming);
		~FPropertyOverrideEdit();

		ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming;
		
		ILevelInstanceInterface* GetLevelInstance() const;
		bool CanDiscard(FText* OutReason = nullptr) const { return true; }
		bool IsDirty() const;
		bool Save(ILevelInstanceInterface* InLevelInstanceOverrideOwner) const;
	};
		
	ENGINE_API FActorContainerID GetLevelInstancePropertyOverridesContext(ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API bool HasEditableLevelInstancePropertyOverrides(TArray<FLevelInstanceActorPropertyOverride>& InPropertyOverrides) const;
	ENGINE_API bool GetLevelInstancePropertyOverridesForActor(const AActor* Actor, FActorContainerID PropertyOverrideContext, TArray<FLevelInstanceActorPropertyOverride>& OutPropertyOverrides) const;
	
	ENGINE_API ILevelInstanceInterface* GetLevelInstancePropertyOverridesEditOwner(ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API const ILevelInstanceInterface* GetLevelInstancePropertyOverridesEditOwner(const ILevelInstanceInterface* LevelInstance) const;
	
	ENGINE_API bool EditLevelInstanceInternal(ILevelInstanceInterface* LevelInstance, TWeakObjectPtr<AActor> ContextActorPtr, const FString& InActorNameToSelect, bool bRecursive);
	ENGINE_API bool CommitLevelInstanceInternal(TUniquePtr<FLevelInstanceEdit>& LevelInstanceEdit, bool bDiscardEdits = false, bool bDiscardOnFailure = false, TSet<FName>* DirtyPackages = nullptr);
	ENGINE_API bool CommitLevelInstancePropertyOverridesInternal(TUniquePtr<FPropertyOverrideEdit>& InPropertyOverrideEdit, bool bDiscardEdits);
	bool CanEditLevelInstanceCommon(const ILevelInstanceInterface* LevelInstance, FText* OutReason = nullptr) const;

	void OnExitEditorModeInternal(bool bForceExit);
	bool TryCommitLevelInstanceEdit(bool bForceExit);
	bool TryCommitLevelInstancePropertyOverrideEdit(bool bForceExit);

	ENGINE_API const FLevelInstanceEdit* GetLevelInstanceEdit(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API bool IsLevelInstanceEditDirty(const FLevelInstanceEdit* LevelInstanceEdit) const;
	ENGINE_API bool PromptUserForCommit(const FLevelInstanceEdit* InLevelInstanceEdit, bool& bOutDiscard, bool bForceCommit = false) const;
	ENGINE_API bool PromptUserForCommitPropertyOverrides(const FPropertyOverrideEdit* InPropertyOverrideEdit, bool& bOutDiscard, bool bForceCommit = false) const;

	ENGINE_API const FPropertyOverrideEdit* GetLevelInstancePropertyOverrideEdit(const ILevelInstanceInterface* LevelInstance) const;
	ENGINE_API void RegisterLoadedLevelStreamingPropertyOverride(ULevelStreamingLevelInstanceEditorPropertyOverride* LevelStreaming);
	void UpdateLevelInstancesFromPropertyOverrideAsset(const TSoftObjectPtr<ULevelInstancePropertyOverrideAsset>& PreviousAssetPath, ULevelInstancePropertyOverrideAsset* NewAsset);

	void ForEachLevelStreaming(TFunctionRef<bool(ULevelStreaming*)> Operation) const;

	FString GetActorNameToSelectFromContext(const ILevelInstanceInterface* LevelInstance, const AActor* ContextActor, const FString& DefaultName = FString()) const;
	void SelectActorFromActorName(ILevelInstanceInterface* LevelInstance, const FString& ActorName) const;

	struct FLevelsToRemoveScope
	{
		FLevelsToRemoveScope(ULevelInstanceSubsystem* InOwner);
		~FLevelsToRemoveScope();
		bool IsValid() const { return !bIsBeingDestroyed; }

		TArray<ULevel*> Levels;
		TWeakObjectPtr<ULevelInstanceSubsystem> Owner;
		bool bResetTrans = false;
		bool bIsBeingDestroyed = false;
	};
	
	ENGINE_API void RemoveLevelsFromWorld(const TArray<ULevel*>& Levels, bool bResetTrans = true);

private:
#endif

#if WITH_EDITOR
	bool bIsCreatingLevelInstance;
	bool bIsCommittingLevelInstance;

	FLevelInstanceChanged LevelInstanceChangedEvent;
	FLevelInstancesUpdated LevelInstancesUpdatedEvent;
	FLevelInstanceEditCancelled LevelInstanceEditCancelled;

	static bool bPrimitiveColorHandlerRegistered;
#endif

	struct FLevelInstance
	{
		ULevelStreamingLevelInstance* LevelStreaming = nullptr;
	};

	TMap<ILevelInstanceInterface*, bool> LevelInstancesToLoadOrUpdate;
	TSet<FLevelInstanceID> LevelInstancesToUnload;
	TSet<FLevelInstanceID> LoadingLevelInstances;
	TMap<FLevelInstanceID, FLevelInstance> LoadedLevelInstances;
	TMap<FLevelInstanceID, ILevelInstanceInterface*> RegisteredLevelInstances;

#if WITH_EDITORONLY_DATA
	// Optional scope to accelerate level unload by batching them
	TUniquePtr<FLevelsToRemoveScope> LevelsToRemoveScope;

	TUniquePtr<FLevelInstanceEdit> LevelInstanceEdit;
	TUniquePtr<FPropertyOverrideEdit> PropertyOverrideEdit;

	TMap<FLevelInstanceID, int32> ChildEdits;

	FWorldPartitionReference CurrentEditLevelInstanceActor;
#endif
};
