// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "WorldPartition/WorldPartitionActorContainerID.h"
#include "LevelInstanceTypes.generated.h"

// FLevelInstanceID is a runtime unique id that is computed from the Hash of LevelInstance Actor Guid and all its ancestor LevelInstance Actor Guids.
// Resulting in a different ID for all instances whether they load the same level or not.
struct FLevelInstanceID
{
	FLevelInstanceID() {}
	FLevelInstanceID(class ULevelInstanceSubsystem* LevelInstanceSubsystem, class ILevelInstanceInterface* LevelInstance);

	inline friend uint32 GetTypeHash(const FLevelInstanceID& Key)
	{
		return ::GetTypeHash(Key.GetHash());
	}

	inline bool operator!=(const FLevelInstanceID& Other) const
	{
		return !(*this == Other);
	}

	inline bool operator==(const FLevelInstanceID& Other) const
	{
		return Hash == Other.Hash && ContainerID == Other.ContainerID && ActorName == Other.ActorName && PackageShortName == Other.PackageShortName;
	}

	inline bool IsValid() const { return !ContainerID.IsMainContainer(); }

	inline uint64 GetHash() const { return Hash; }

	inline const FActorContainerID& GetContainerID() const { return ContainerID; }

private:
	uint64 Hash = 0;
	FActorContainerID ContainerID;
	
	// Hashed only for Loaded LevelInstances 
	// Spawned LevelInstances have a unique Guid which is enough.
	
	// Name allows distinguishing between LevelInstances with embedded parent LevelInstances because embedded actors have a guaranteed unique name (ContainerID suffix)
	FName ActorName;
	// PackageShortName allows distinguising between instanced LevelInstances of the same source level.
	// - Loading /Game/Path/WorldA.WorldA as /Game/Path/WorldA_LevelInstance1.WorldA & /Game/Path/WorldA_LevelInstance".WorldA
	//   with the source WorldA containing one of many LevelInstance actors. 
	//   Those actors would end up with the same hash. We use PackageShortName (WorldA_LevelInstance1 & WorldA_LevelInstance2) to distinguish them.
	FString PackageShortName;
};

UENUM()
enum class ELevelInstanceRuntimeBehavior : uint8
{
	None UMETA(Hidden),
	// Deprecated exists only to avoid breaking Actor Desc serialization
	Embedded_Deprecated UMETA(Hidden),
	// Move level instance actors to the main world partition
	Partitioned UMETA(DisplayName = "Embedded"),
	// Use level streaming to load level instance actors
	LevelStreaming UMETA(DisplayName = "Standalone"),
};

UENUM()
enum class ELevelInstanceCreationType : uint8
{
	LevelInstance,
	PackedLevelActor
};

UENUM()
enum class ELevelInstancePivotType : uint8
{
	CenterMinZ,
	Center,
	Actor,
	WorldOrigin
};

USTRUCT()
struct FNewLevelInstanceParams
{
	GENERATED_USTRUCT_BODY()
			
	UPROPERTY(EditAnywhere, Category = Default, meta = (EditCondition = "!bHideCreationType", EditConditionHides, HideEditConditionToggle))
	ELevelInstanceCreationType Type = ELevelInstanceCreationType::LevelInstance;

	UPROPERTY(EditAnywhere, Category = Pivot)
	ELevelInstancePivotType PivotType = ELevelInstancePivotType::CenterMinZ;

	UPROPERTY(EditAnywhere, Category = Pivot)
	TObjectPtr<AActor> PivotActor = nullptr;

	UPROPERTY()
	bool bAlwaysShowDialog = true;

	UPROPERTY()
	TObjectPtr<UWorld> TemplateWorld = nullptr;
		
	UPROPERTY()
	FString LevelPackageName;

	UPROPERTY()
	bool bPromptForSave = false;

	UPROPERTY()
	TSubclassOf<AActor> LevelInstanceClass;

	UPROPERTY()
	bool bEnableStreaming = false;

private:
	UPROPERTY(EditAnywhere, Category = Default, meta = (EditCondition = "!bForceExternalActors", EditConditionHides, HideEditConditionToggle))
	bool bExternalActors = true;
	
	UPROPERTY()
	bool bForceExternalActors = false;

	UPROPERTY()
	bool bHideCreationType = false;

public:
	void HideCreationType() { bHideCreationType = true; }
	void SetForceExternalActors(bool bInForceExternalActors) { bForceExternalActors = bInForceExternalActors; }
	void SetExternalActors(bool bInExternalActors) { bExternalActors = bInExternalActors; }
	bool UseExternalActors() const { return bForceExternalActors || bExternalActors; }
};