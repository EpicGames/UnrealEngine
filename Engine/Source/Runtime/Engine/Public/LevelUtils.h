// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class ULevel;
class ULevelStreaming;

/**
 * A set of static methods for common editor operations that operate on ULevel objects.
 */
class FLevelUtils
{
public:
	///////////////////////////////////////////////////////////////////////////
	// Given a ULevel, find the corresponding ULevelStreaming.

	/**
	 * Returns the streaming level corresponding to the specified ULevel, or NULL if none exists.
	 *
	 * @param		Level		The level to query.
	 * @return					The level's streaming level, or NULL if none exists.
	 */
	static ENGINE_API ULevelStreaming* FindStreamingLevel(const ULevel* Level);

	/**
	 * Returns the streaming level by package FName, or NULL if none exists.
	 *
	 * @param		InWorld			World to look in for the streaming level
	 * @param		PackageFName	FName of the package containing the ULevel to query
	 * @return						The level's streaming level, or NULL if none exists.
	 */
	static ENGINE_API ULevelStreaming* FindStreamingLevel(UWorld* InWorld, const FName PackageName);

	/**
	 * Returns the streaming level by package name, or NULL if none exists.
	 *
	 * @param		InWorld			World to look in for the streaming level
	 * @param		PackageName		Name of the package containing the ULevel to query
	 * @return						The level's streaming level, or NULL if none exists.
	 */
	static ENGINE_API ULevelStreaming* FindStreamingLevel(UWorld* InWorld, const TCHAR* PackageName);

	/**
	 * Returns whether the given package is referenced by one of the world streaming levels or not.
	 *
	 * @param		InWorld			World to look in for the streaming level
	 * @param		InPackageName	Name of the package containing the ULevel to query
	 * @return						True if the given package is referenced by one of
	 *								the world streaming levels, else False.
	 */
	static ENGINE_API bool IsValidStreamingLevel(UWorld* InWorld, const TCHAR* InPackageName);

	/**
	 * Returns whether the given package is part of the world server visible streaming levels or not.
	 *
	 * @param		InWorld			World to look in for the streaming level
	 * @param		InPackageName	Name of the package containing the ULevel to query
	 * @return						True if the given package is referenced by one of
	 *								the world server visible streaming levels, else False.
	 */
	static ENGINE_API bool IsServerStreamingLevelVisible(UWorld* InWorld, const FName& InPackageName);

	/**
	 * Returns the streaming level by package name if visible on server, or NULL if none exists.
	 *
	 * @param		InWorld			World to look in for the streaming level
	 * @param		PackageName		Name of the package containing the ULevel to query
	 * @return						The level's streaming level if visible on server, or NULL if none exists.
	 */
	static ENGINE_API ULevelStreaming* GetServerVisibleStreamingLevel(UWorld* InWorld, const FName& InPackageName);

	/** Returns whether the world supports for a client to use "making visible" transaction requests to the server. */
	static ENGINE_API bool SupportsMakingVisibleTransactionRequests(UWorld* InWorld);

	/** Returns whether the world supports for a client to use "making invisible" transaction requests to the server. */
	static ENGINE_API bool SupportsMakingInvisibleTransactionRequests(UWorld* InWorld);

	///////////////////////////////////////////////////////////////////////////
	// Locking/unlocking levels for edit.

#if WITH_EDITOR
	/**
	 * Returns true if the specified level is locked for edit, false otherwise.
	 *
	 * @param	Level		The level to query.
	 * @return				true if the level is locked, false otherwise.
	 */
	static ENGINE_API bool IsLevelLocked(ULevel* Level);
	static ENGINE_API bool IsLevelLocked(AActor* Actor);

	/**
	 * Sets a level's edit lock.
	 *
	 * @param	Level		The level to modify.
	 */
	static ENGINE_API void ToggleLevelLock(ULevel* Level);
#endif

	///////////////////////////////////////////////////////////////////////////
	// Controls whether the level is loaded in editor.

	/**
	 * Returns true if the level is currently loaded in the editor, false otherwise.
	 *
	 * @param	Level		The level to query.
	 * @return				true if the level is loaded, false otherwise.
	 */
	static ENGINE_API bool IsLevelLoaded(ULevel* Level);


	///////////////////////////////////////////////////////////////////////////
	// Level visibility.

	/**
	 * Returns true if the specified level is visible in the editor, false otherwise.
	 *
	 * @param	StreamingLevel		The level to query.
	 */
#if WITH_EDITORONLY_DATA
	static ENGINE_API bool IsStreamingLevelVisibleInEditor(const ULevelStreaming* StreamingLevel);
#endif

	/**
	 * Returns true if the specified level is visible in the editor, false otherwise.
	 *
	 * @param	Level		The level to query.
	 */
	static ENGINE_API bool IsLevelVisible(const ULevel* Level);


	struct FApplyLevelTransformParams
	{
		FApplyLevelTransformParams(ULevel* InLevel, const FTransform& InLevelTransform)
			: Level(InLevel)
			, LevelTransform(InLevelTransform)
		{
		}

		// The level to Transform.
		ULevel* Level;

		// If the actor is non null, directly transform only this actor
		AActor* Actor = nullptr;

		// How to Transform the level.
		const FTransform& LevelTransform;

		// Whether to call SetRelativeTransform or update the Location and Rotation in place without any other updating
		bool bSetRelativeTransformDirectly = false;

#if WITH_EDITOR
		// Whether to call PostEditMove on actors after transforming
		bool bDoPostEditMove = true;
#endif
	};

	/** Transforms the level to a new world space */
	static ENGINE_API void ApplyLevelTransform(const FApplyLevelTransformParams& TransformParams);

#if WITH_EDITOR
	///////////////////////////////////////////////////////////////////////////
	// Level - editor transforms.

	/**
	 * Calls PostEditMove on all the actors in the level
	 *
	 * @param	Level		The level.
	 */
	static ENGINE_API void ApplyPostEditMove( ULevel* Level );

	/**
	 * Sets a new LevelEditorTransform on a streaming level .
	 *
	 * @param	StreamingLevel		The level.
	 * @param	Transform			The new transform.
	 * @param	bDoPostEditMove		Whether to call PostEditMove on actors after transforming
	 */
	static ENGINE_API void SetEditorTransform(ULevelStreaming* StreamingLevel, const FTransform& Transform, bool bDoPostEditMove = true);

	/**
	 * Apply the LevelEditorTransform on a level.
	 *
	 * @param	StreamingLevel		The level.
	 * @param   bDoPostEditMove		Whether to call PostEditMove on actors after transforming
	 * @param	Actor				Optional actor on which to apply the transform instead of the full level.
	 */
	static ENGINE_API void ApplyEditorTransform(const ULevelStreaming* StreamingLevel, bool bDoPostEditMove = true, AActor* Actor = nullptr);

	/**
	 * Remove the LevelEditorTransform from a level.
	 *
	 * @param	StreamingLevel		The level.
	 * @param	bDoPostEditMove		Whether to call PostEditMove on actors after transforming
	 * @param	Actor				Optional actor on which to apply the transform instead of the full level.
	 */
	static ENGINE_API void RemoveEditorTransform(const ULevelStreaming* StreamingLevel, bool bDoPostEditMove = true, AActor* Actor = nullptr);

	/**
	* Returns true if we are moving a level
	*/
	static ENGINE_API bool IsMovingLevel();
	static ENGINE_API bool IsApplyingLevelTransform();

private:

	// Flag to mark if we are currently finalizing a level offset
	static ENGINE_API bool bMovingLevel;
	static ENGINE_API bool bApplyingLevelTransform;

#endif // WITH_EDITOR
};

