// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/TypeHash.h"

/**
 * Class used to identify ULightComponents on the rendering thread without having to pass the pointer around,
 * Which would make it easy for people to access game thread variables from the rendering thread.
 */
class FLightComponentId
{
public:
	FLightComponentId()
		: IDValue(0)
	{
	}

	inline bool IsValid() const
	{
		return IDValue > 0;
	}

	inline bool operator==(FLightComponentId OtherId) const
	{
		return IDValue == OtherId.IDValue;
	}
	
	inline bool operator<(FLightComponentId OtherId) const
	{
		return IDValue < OtherId.IDValue;
	}

	friend uint32 GetTypeHash(FLightComponentId Id)
	{
		return GetTypeHash(Id.IDValue);
	}

	ENGINE_API static FLightComponentId GetNextId();

private:
	uint32 IDValue;

	static ENGINE_API FThreadSafeCounter NextLightId;

	FLightComponentId(uint32 InIDValue)
	: IDValue(InIDValue)
	{
	}
};