// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMap.h: Light-map definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/ChunkedArray.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "RenderDeferredCleanup.h"
#include "Serialization/BulkData.h"
#include "SceneManagement.h"
#include "Engine/LightMapTexture2D.h"

class FLightMap2D;
class UInstancedStaticMeshComponent;
class UMapBuildDataRegistry;
class UPrimitiveComponent;
struct FQuantizedLightmapData;
class ULightComponent;
class UVirtualTexture;
class FShadowMapData2D;
class FFourDistanceFieldSamples;

/** Whether to use bilinear filtering on lightmaps */
extern ENGINE_API bool GUseBilinearLightmaps;

/** Whether to allow padding around mappings. Old-style lighting doesn't use this. */
extern ENGINE_API bool GAllowLightmapPadding;

/** The quality level of the current lighting build */
extern ENGINE_API ELightingBuildQuality GLightingBuildQuality;

#if WITH_EDITOR
/** Debug options for Lightmass */
extern ENGINE_API FLightmassDebugOptions GLightmassDebugOptions;
#endif

extern ENGINE_API FColor GTexelSelectionColor;

extern ENGINE_API bool IsTexelDebuggingEnabled();

/**
 * The abstract base class of 1D and 2D light-maps.
 */
class FLightMap : private FDeferredCleanupInterface
{
public:
	enum
	{
		LMT_None = 0,
		LMT_1D = 1,
		LMT_2D = 2,
	};

	/** The GUIDs of lights which this light-map stores. */
	TArray<FGuid> LightGuids;

	/** Default constructor. */
	ENGINE_API FLightMap();

	/** Destructor. */
	virtual ~FLightMap() { check(NumRefs == 0); }

	/**
	 * Checks if a light is stored in this light-map.
	 * @param	LightGuid - The GUID of the light to check for.
	 * @return	True if the light is stored in the light-map.
	 */
	bool ContainsLight(const FGuid& LightGuid) const
	{
		return LightGuids.Find(LightGuid) != INDEX_NONE;
	}

	// FLightMap interface.
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) {}
	ENGINE_API virtual void Serialize(FArchive& Ar);
	virtual FLightMapInteraction GetInteraction(ERHIFeatureLevel::Type InFeatureLevel) const = 0;
	virtual FShadowMapInteraction GetShadowInteraction(ERHIFeatureLevel::Type InFeatureLevel) const { return FShadowMapInteraction::None(); }

	// Runtime type casting.
	virtual FLightMap2D* GetLightMap2D() { return NULL; }
	virtual const FLightMap2D* GetLightMap2D() const { return NULL; }

	// Reference counting.
	void AddRef()
	{
		check(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		NumRefs++;
	}
	void Release()
	{
		check(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread());
		checkSlow(NumRefs > 0);
		if(--NumRefs == 0)
		{
			Cleanup();
		}
	}

	/**
	* @return true if high quality lightmaps are allowed
	*/
	inline bool AllowsHighQualityLightmaps() const
	{
		return bAllowHighQualityLightMaps;
	}

protected:

	/** Indicates whether the lightmap is being used for directional or simple lighting. */
	bool bAllowHighQualityLightMaps;

	/**
	 * Called when the light-map is no longer referenced.  Should release the lightmap's resources.
	 */
	ENGINE_API virtual void Cleanup();

private:
	int32 NumRefs;
};



/**
 * Lightmap reference serializer
 * Intended to be used by TRefCountPtr's serializer, not called directly
 */
extern ENGINE_API FArchive& operator<<(FArchive& Ar, FLightMap*& R);

/** 
 * Incident lighting for a single sample, as produced by a lighting build. 
 * FGatheredLightSample is used for gathering lighting instead of this format as FLightSample is not additive.
 */
struct FLightSample
{
	/** 
	 * Coefficients[0] stores the normalized average color, 
	 * Coefficients[1] stores the maximum color component in each lightmap basis direction,
	 * and Coefficients[2] stores the simple lightmap which is colored incident lighting along the vertex normal.
	 */
	float Coefficients[NUM_STORED_LIGHTMAP_COEF][3];

	/** True if this sample maps to a valid point on a triangle.  This is only meaningful for texture lightmaps. */
	bool bIsMapped;

	/** Initialization constructor. */
	FLightSample():
		bIsMapped(false)
	{
		FMemory::Memzero(Coefficients,sizeof(Coefficients));
	}
};

/**
 * The raw data which is used to construct a 2D light-map.
 */
class FLightMapData2D
{
public:

	/** The GUIDs of lights which this light-map stores. */
	TArray<FGuid> LightGuids;

	/**
	 * Minimal initialization constructor.
	 */
	FLightMapData2D(uint32 InSizeX,uint32 InSizeY):
		Data(InSizeX * InSizeY),
		SizeX(InSizeX),
		SizeY(InSizeY)
	{}

	// Accessors.
	const FLightSample& operator()(uint32 X,uint32 Y) const { return Data(SizeX * Y + X); }
	FLightSample& operator()(uint32 X,uint32 Y) { return Data(SizeX * Y + X); }
	uint32 GetSizeX() const { return SizeX; }
	uint32 GetSizeY() const { return SizeY; }

private:

	/** The incident light samples for a 2D array of points on the surface. */
	TChunkedArray<FLightSample> Data;

	/** The width of the light-map. */
	uint32 SizeX;

	/** The height of the light-map. */
	uint32 SizeY;
};

/**
 * A 2D array of incident lighting data.
 */
class FLightMap2D : public FLightMap
{
public:

	ENGINE_API FLightMap2D();
	ENGINE_API FLightMap2D(bool InAllowHighQualityLightMaps);

	// FLightMap2D interface.

	/**
	 * Returns the texture containing the RGB coefficients for a specific basis.
	 * @param	BasisIndex - The basis index.
	 * @return	The RGB coefficient texture.
	 */
	ENGINE_API const UTexture2D* GetTexture(uint32 BasisIndex) const;
	ENGINE_API UTexture2D* GetTexture(uint32 BasisIndex);

	void GetReferencedTextures(TArray<UTexture2D*>& OutTextures) const
	{
		for (int32 BasisIndex = 0; BasisIndex < UE_ARRAY_COUNT(Textures); BasisIndex++)
		{
			if (Textures[BasisIndex])
			{
				OutTextures.Add(Textures[BasisIndex]);
			}
		}

		if (GetSkyOcclusionTexture())
		{
			OutTextures.Add(GetSkyOcclusionTexture());
		}

		if (GetAOMaterialMaskTexture())
		{
			OutTextures.Add(GetAOMaterialMaskTexture());
		}
	}

	/**
	 * Returns SkyOcclusionTexture.
	 * @return	The SkyOcclusionTexture.
	 */
	ENGINE_API UTexture2D* GetSkyOcclusionTexture() const;

	ENGINE_API UTexture2D* GetAOMaterialMaskTexture() const;

	ULightMapVirtualTexture2D* GetVirtualTexture(uint32 BasisIndex) const { return VirtualTextures[BasisIndex]; }

	ENGINE_API bool IsVirtualTextureValid() const;

	/**
	 * Returns whether the specified basis has a valid lightmap texture or not.
	 * @param	BasisIndex - The basis index.
	 * @return	true if the specified basis has a valid lightmap texture, otherwise false
	 */
	ENGINE_API bool IsValid(uint32 BasisIndex) const;

	const FVector2D& GetCoordinateScale() const { return CoordinateScale; }
	const FVector2D& GetCoordinateBias() const { return CoordinateBias; }

	// FLightMap interface.
	ENGINE_API virtual void AddReferencedObjects( FReferenceCollector& Collector );

	ENGINE_API virtual void Serialize(FArchive& Ar);
	ENGINE_API virtual FLightMapInteraction GetInteraction(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API virtual FShadowMapInteraction GetShadowInteraction(ERHIFeatureLevel::Type InFeatureLevel) const;

	// Runtime type casting.
	virtual const FLightMap2D* GetLightMap2D() const { return this; }
	virtual FLightMap2D* GetLightMap2D() { return this; }

	/**
	 * Allocates texture space for the light-map and stores the light-map's raw data for deferred encoding.
	 * If the light-map has no lights in it, it will return NULL.
	 * SourceQuantizedData will be deleted by this function.
	 * @param	LightMapOuter - The package to create the light-map and textures in.
	 * @param	SourceQuantizedData - If the data is already quantized, the values will be in here, and not in RawData. 
	 * @param	SourceShadowMapData - Shadow map data to be combined into the lightmap atlas, used when creating VT lightmaps
	 * @param	Bounds - The bounds of the primitive the light-map will be rendered on.  Used as a hint to pack light-maps on nearby primitives in the same texture.
	 * @param	InPaddingType - the method for padding the lightmap.
	 * @param	LightmapFlags - flags that determine how the lightmap is stored (e.g. streamed or not)
	 */
	static ENGINE_API TRefCountPtr<FLightMap2D> AllocateLightMap(UObject* LightMapOuter,
		FQuantizedLightmapData*& SourceQuantizedData,
		const TMap<ULightComponent*, FShadowMapData2D*>& SourceShadowMapData,
		const FBoxSphereBounds& Bounds, ELightMapPaddingType InPaddingType, ELightMapFlags InLightmapFlags );

	/**
	 * Allocates texture space for the light-map and stores the light-map's raw data for deferred encoding.
	 * If the light-map has no lights in it, it will return NULL.
	 * SourceQuantizedData will be deleted by this function.
	 * @param	Component - The component that owns this lightmap
	 * @param	SourceQuantizedData - If the data is already quantized, the values will be in here, and not in RawData.
	 * @param	Bounds - The bounds of the primitive the light-map will be rendered on.  Used as a hint to pack light-maps on nearby primitives in the same texture.
	 * @param	InPaddingType - the method for padding the lightmap.
	 * @param	LightmapFlags - flags that determine how the lightmap is stored (e.g. streamed or not)
	 */
	static ENGINE_API TRefCountPtr<FLightMap2D> AllocateInstancedLightMap(UObject* LightMapOuter, UInstancedStaticMeshComponent* Component,
		TArray<TUniquePtr<FQuantizedLightmapData>> SourceQuantizedData,
		TArray<TMap<ULightComponent*, TUniquePtr<FShadowMapData2D>>>&& InstancedShadowMapData,
		UMapBuildDataRegistry* Registry, FGuid MapBuildDataId, const FBoxSphereBounds& Bounds, ELightMapPaddingType InPaddingType, ELightMapFlags LightmapFlags);

	/**
	 * Executes all pending light-map encoding requests.
	 * @param	bLightingSuccessful	Whether the lighting build was successful or not.
	 * @param	bForceCompletion	Force all encoding to be fully completed (they may be asynchronous).
	 */
	static ENGINE_API void EncodeTextures(const FStaticLightingBuildContext* LightingContext, bool bLightingSuccessful, bool bMultithreadedEncode = false );

#if WITH_EDITOR
	/**
	 * Constructs mip maps for a single shadowmap texture.
	 */
	static ENGINE_API int32 EncodeShadowTexture(const FStaticLightingBuildContext* LightingContext, struct FLightMapPendingTexture& PendingTexture, TArray<TArray<FFourDistanceFieldSamples>>& MipData);
#endif // WITH_EDITOR

	/** Call to enable/disable status update of LightMap encoding */
	static void SetStatusUpdate(bool bInEnable)
	{
		bUpdateStatus = bInEnable;
	}

	static bool GetStatusUpdate()
	{
		return bUpdateStatus;
	}
	
	/** The textures containing the light-map data. */
	TObjectPtr<ULightMapTexture2D> Textures[2];

	TObjectPtr<ULightMapTexture2D> SkyOcclusionTexture;

	TObjectPtr<ULightMapTexture2D> AOMaterialMaskTexture;

	TObjectPtr<UShadowMapTexture2D> ShadowMapTexture;

	/** The virtual textures containing the light-map data. */
	TObjectPtr<ULightMapVirtualTexture2D >VirtualTextures[2];

	/** A scale to apply to the coefficients. */
	FVector4f ScaleVectors[NUM_STORED_LIGHTMAP_COEF];

	/** Bias value to apply to the coefficients. */
	FVector4f AddVectors[NUM_STORED_LIGHTMAP_COEF];

	/** The scale which is applied to the light-map coordinates before sampling the light-map textures. */
	FVector2D CoordinateScale;

	/** The bias which is applied to the light-map coordinates before sampling the light-map textures. */
	FVector2D CoordinateBias;

	/** Stores the inverse of the penumbra size, normalized.  Stores 1 to interpret the shadowmap as a shadow factor directly, instead of as a distance field. */
	FVector4f InvUniformPenumbraSize;

	/** Tracks which of the 4 channels has valid texture data. */
	bool bShadowChannelValid[4];

protected:
	ENGINE_API FLightMap2D(const TArray<FGuid>& InLightGuids);

	/** If true, update the status when encoding light maps */
	static ENGINE_API bool bUpdateStatus;
};

struct FLegacyQuantizedDirectionalLightSample
{
	/** The lighting coefficients, colored. */
	FColor	Coefficients[2];
};

struct FLegacyQuantizedSimpleLightSample
{
	/** The lighting coefficients, colored. */
	FColor	Coefficients[1];
};

/**
 * The light incident for a point on a surface in three directions, stored as bytes representing values from 0-1.
 *
 * @warning BulkSerialize: FQuantizedDirectionalLightSample is serialized as memory dump
 * See TArray::BulkSerialize for detailed description of implied limitations.
 */
struct FQuantizedDirectionalLightSample
{
	/** The lighting coefficients, colored. */
	FColor	Coefficients[NUM_HQ_LIGHTMAP_COEF];
};

/**
* The light incident for a point on a surface along the surface normal, stored as bytes representing values from 0-1.
*
* @warning BulkSerialize: FQuantizedSimpleLightSample is serialized as memory dump
* See TArray::BulkSerialize for detailed description of implied limitations.
*/
struct FQuantizedSimpleLightSample
{
	/** The lighting coefficients, colored. */
	FColor	Coefficients[NUM_LQ_LIGHTMAP_COEF];
};

// These act like a POD as FColor is also set to act like POD
template <> struct TIsPODType<FQuantizedDirectionalLightSample> { enum { Value = true }; };
template <> struct TIsPODType<FQuantizedSimpleLightSample> { enum { Value = true }; };

/**
 * Bulk data array of FQuantizedLightSamples
 */
template<class QuantizedLightSampleType>
struct TQuantizedLightSampleBulkData : public TBulkData<QuantizedLightSampleType>
{
	typedef QuantizedLightSampleType SampleType;
};

/** A 1D array of incident lighting data. */
class FLegacyLightMap1D : public FLightMap
{
public:

	FLegacyLightMap1D()
	{
		bAllowHighQualityLightMaps = false;
	}

	// FLightMap interface.
	virtual void Serialize(FArchive& Ar);

	virtual FLightMapInteraction GetInteraction(ERHIFeatureLevel::Type InFeatureLevel) const { return FLightMapInteraction::None(); }
};

/** Stores debug information for a lightmap sample. */
struct FSelectedLightmapSample
{
	UPrimitiveComponent* Component;
	int32 NodeIndex;
	FLightMapRef Lightmap;
	FVector Position;
	/** Position in the texture mapping */
	int32 LocalX;
	int32 LocalY;
	int32 MappingSizeX;
	int32 MappingSizeY;
	
	/** Default ctor */
	FSelectedLightmapSample() :
		Component(NULL),
		NodeIndex(INDEX_NONE),
		Lightmap(NULL),
		Position(FVector::ZeroVector),
		LocalX(-1),
		LocalY(-1),
		MappingSizeX(-1),
		MappingSizeY(-1)
	{}

	/** Constructor used for a texture lightmap sample */
	FSelectedLightmapSample(
		UPrimitiveComponent* InComponent, 
		int32 InNodeIndex,
		FLightMapRef& InLightmap, 
		FVector InPosition, 
		int32 InLocalX,
		int32 InLocalY,
		int32 InMappingSizeX,
		int32 InMappingSizeY) 
		:
		Component(InComponent),
		NodeIndex(InNodeIndex),
		Lightmap(InLightmap),
		Position(InPosition),
		LocalX(InLocalX),
		LocalY(InLocalY),
		MappingSizeX(InMappingSizeX),
		MappingSizeY(InMappingSizeY)
	{}
};

class FDebugShadowRay
{
public:
	FVector Start;
	FVector End;
	int32 bHit;

	FDebugShadowRay(const FVector& InStart, const FVector& InEnd, bool bInHit) :
		Start(InStart),
		End(InEnd),
		bHit(bInHit)
	{}
};

/**
 * The quantized coefficients for a single light-map texel.
 */
struct FLightMapCoefficients
{
	uint8 Coverage;
	uint8 Coefficients[NUM_STORED_LIGHTMAP_COEF][4];
	uint8 SkyOcclusion[4];
	uint8 AOMaterialMask;
};

struct FQuantizedLightmapData
{
	/** Width or a 2D lightmap, or number of samples for a 1D lightmap */
	uint32 SizeX;

	/** Height of a 2D lightmap */
	uint32 SizeY;

	/** The quantized coefficients */
	TArray<FLightMapCoefficients> Data;

	/** The scale to apply to the quantized coefficients when expanding */
	float Scale[NUM_STORED_LIGHTMAP_COEF][4];

	/** Bias value to apply to the coefficients. */
	float Add[NUM_STORED_LIGHTMAP_COEF][4];

	/** The GUIDs of lights which this light-map stores. */
	TArray<FGuid> LightGuids;

	bool bHasSkyShadowing;

	FQuantizedLightmapData() :
		bHasSkyShadowing(false)
	{}

	ENGINE_API bool HasNonZeroData() const;

	ENGINE_API void Serialize(FArchive& Ar);
};

/**
 * Checks if a lightmap texel is mapped or not.
 *
 * @param MappingData	Array of lightmap texels
 * @param X				X-coordinate for the texel to check
 * @param Y				Y-coordinate for the texel to check
 * @param Pitch			Number of texels per row
 * @return				true if the texel is mapped
 */
inline bool IsTexelMapped( const TArray<FLightMapCoefficients>& MappingData, int32 X, int32 Y, int32 Pitch )
{
	const FLightMapCoefficients& Sample = MappingData[Y * Pitch + X];
	bool bIsMapped = (Sample.Coverage > 0);
	return bIsMapped;
}

/**
 * Calculates the minimum rectangle that encompasses all mapped texels.
 *
 * @param MappingData	Array of lightmap/shadowmap texels
 * @param SizeX			Number of texels along the X-axis
 * @param SizeY			Number of texels along the Y-axis
 * @param CroppedRect	[out] Upon return, contains the minimum rectangle that encompasses all mapped texels
 */
template <class TMappingData>
void CropUnmappedTexels( const TMappingData& MappingData, int32 SizeX, int32 SizeY, FIntRect& CroppedRect )
{
	int32 StartX = 0;
	int32 StartY = 0;
	int32 EndX = SizeX;
	int32 EndY = SizeY;

	CroppedRect.Min.X = EndX;
	CroppedRect.Min.Y = EndY;
	CroppedRect.Max.X = StartX - 1;
	CroppedRect.Max.Y = StartY - 1;

	for ( int32 Y = StartY; Y < EndY; ++Y )
	{
		bool bIsMappedRow = false;

		// Scan for first mapped texel in this row (also checks whether the row contains a mapped texel at all).
		for ( int32 X = StartX; X < EndX; ++X )
		{
			if ( IsTexelMapped( MappingData, X, Y, SizeX ) )
			{
				CroppedRect.Min.X = FMath::Min<int32>(CroppedRect.Min.X, X);
				bIsMappedRow = true;
				break;
			}
		}

		// Scan for the last mapped texel in this row.
		for ( int32 X = EndX-1; X > CroppedRect.Max.X; --X )
		{
			if ( IsTexelMapped( MappingData, X, Y, SizeX ) )
			{
				CroppedRect.Max.X = X;
				break;
			}
		}

		if ( bIsMappedRow )
		{
			CroppedRect.Min.Y = FMath::Min<int32>(CroppedRect.Min.Y, Y);
			CroppedRect.Max.Y = FMath::Max<int32>(CroppedRect.Max.Y, Y);
		}
	}

	CroppedRect.Max.X = CroppedRect.Max.X + 1;
	CroppedRect.Max.Y = CroppedRect.Max.Y + 1;
	CroppedRect.Min.X = FMath::Min<int32>(CroppedRect.Min.X, CroppedRect.Max.X);
	CroppedRect.Min.Y = FMath::Min<int32>(CroppedRect.Min.Y, CroppedRect.Max.Y);
}

/** 
 * A bundle of lightmap resources which are referenced by multiple components. 
 */
class FLightmapResourceCluster : public FRenderResource
{
public:
	FLightmapResourceCluster() : AllocatedVT(nullptr) {}
	ENGINE_API virtual ~FLightmapResourceCluster();

	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList);
	ENGINE_API virtual void ReleaseRHI();

	ENGINE_API void SetFeatureLevelAndInitialize(const FStaticFeatureLevel InFeatureLevel);
	ENGINE_API void TryInitializeUniformBuffer();
	ENGINE_API void UpdateUniformBuffer();
	
	/**
	 * Allocates virtual texture on demand and returns it, may return nullptr if not using virtual texture
	 * 'const' method setting 'mutable' member is not amazing, but this is required to work around ordering of render commands submitted from main thread,
	 * it's possible for commands that want to access AllocatedVT from here execute before this has a chance to run InitRHI()
	 */
	ENGINE_API const IAllocatedVirtualTexture* GetAllocatedVT() const;
	ENGINE_API void ConditionalCreateAllocatedVT();
	ENGINE_API void ReleaseAllocatedVT();
	ENGINE_API bool GetUseVirtualTexturing() const;

	FLightmapClusterResourceInput Input;
	mutable IAllocatedVirtualTexture* AllocatedVT;
	FUniformBufferRHIRef UniformBuffer;
};
