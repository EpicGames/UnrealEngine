// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightSceneProxy.h"

struct FLightSceneDesc
{
	const FTransform& GetTransform() const
	{
		return Transform;
	}

	FVector4 GetPosition() const
	{
		return FVector4(GetTransform().GetLocation(), 1);
	}

	FLightSceneProxy* GetSceneProxy() const
	{
		return SceneProxy;
	}

	FTransform Transform = FTransform::Identity;
	FLightSceneProxy* SceneProxy = nullptr;
};