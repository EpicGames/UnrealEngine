// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "LightComponentId.h"
#include "UObject/NameTypes.h"

class ULightComponent;
class FMaterialRenderProxy;
class FSceneView;
class FSceneViewFamily;
class FWholeSceneProjectedShadowInitializer;
class FPerObjectProjectedShadowInitializer;
class FShadowCascadeSettings;
class FRHICommandList;
class FLightSceneInfo;
class UTextureLightProfile;
class FTexture;
struct FViewMatrices;
struct FLightRenderParameters;
struct FLightSceneProxyDesc;

enum ELightShaderParameterFlags
{
	RectAsSpotLight=1,
};

// Hacky base class to avoid 8 bytes of padding after the vtable
class FLightSceneProxyFixLayout
{
public:
	virtual ~FLightSceneProxyFixLayout() = default;
};

/**
 * Encapsulates the data which is used to render a light by the rendering thread.
 * The constructor is called from the game thread, and after that the rendering thread owns the object.
 * FLightSceneProxy is in the engine module and is subclassed to implement various types of lights.
 */
class FLightSceneProxy : public FLightSceneProxyFixLayout
{
public:

	/** Initialization constructors. */
	ENGINE_API FLightSceneProxy(const ULightComponent* InLightComponent);
	ENGINE_API FLightSceneProxy(const FLightSceneProxyDesc& Desc);
	ENGINE_API virtual ~FLightSceneProxy();

	/**
	 * Tests whether the light affects the given bounding volume.
	 * @param Bounds - The bounding volume to test.
	 * @return True if the light affects the bounding volume
	 */
	virtual bool AffectsBounds(const FBoxSphereBounds& Bounds) const
	{
		return true;
	}

	ENGINE_API void SetLightFunctionMaterial_GameThread(const ULightComponent* Component);
	ENGINE_API void SetTransmission_GameThread(const ULightComponent* Component);

	ENGINE_API virtual FSphere GetBoundingSphere() const;

	/** @return radius of the light */
	virtual float GetRadius() const { return FLT_MAX; }
	virtual float GetOuterConeAngle() const { return 0.0f; }
	virtual float GetSourceRadius() const { return 0.0f; }
	virtual bool IsInverseSquared() const { return true; }
	virtual bool IsRectLight() const { return false; }
	virtual bool  IsLocalLight() const { return false; }
	virtual bool HasSourceTexture() const { return false; }
	virtual float GetLightSourceAngle() const { return 0.0f; }
	virtual float GetShadowSourceAngleFactor() const { return 1.0f; }
	virtual float GetTraceDistance() const { return 0.0f; }

	// TODO: refactor this to move into the shadow scene renderer
	virtual float GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices, const FIntPoint& CameraViewRectSize) const { return 0.0f; }

	UE_DEPRECATED(5.1, "The GetEffectiveScreenRadius() that uses the screen-percentage scaled view rect (above) is used now.")
	virtual float GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices) const { return 0.0f; }

	/** Accesses parameters needed for rendering the light. */
	virtual void GetLightShaderParameters(FLightRenderParameters& OutLightParameters, uint32 Flags=0) const {}

	virtual FVector2D GetDirectionalLightDistanceFadeParameters(ERHIFeatureLevel::Type InFeatureLevel, bool bPrecomputedLightingIsValid, int32 MaxNearCascades) const
	{
		return FVector2D(0, 0);
	}

	virtual int32 GetDirectionalLightForwardShadingPriority() const
	{
		return 0;
	}

	virtual bool GetLightShaftOcclusionParameters(float& OutOcclusionMaskDarkness, float& OutOcclusionDepthRange) const
	{
		OutOcclusionMaskDarkness = 0;
		OutOcclusionDepthRange = 1;
		return false;
	}

	virtual FVector GetLightPositionForLightShafts(FVector ViewOrigin) const
	{
		return GetPosition();
	}

	/**
	 * Sets up a projected shadow initializer for shadows from the entire scene.
	 * @return True if the whole-scene projected shadow should be used.
	 */
	virtual bool GetWholeSceneProjectedShadowInitializer(const FSceneViewFamily& ViewFamily, TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> >& OutInitializers) const
	{
		return false;
	}

	/** Whether this light should create per object shadows for dynamic objects. */
	ENGINE_API virtual bool ShouldCreatePerObjectShadowsForDynamicObjects() const;

	/** Whether this light should create CSM for dynamic objects only (forward renderer) */
	ENGINE_API virtual bool UseCSMForDynamicObjects() const;

	float GetVSMTexelDitherScale() const
	{
		return VSMTexelDitherScale;
	}

	float GetVSMResolutionLodBias() const
	{
		return VSMResolutionLodBias;
	}
	
	/** Returns the number of view dependent shadows this light will create, not counting distance field shadow cascades. */
	virtual uint32 GetNumViewDependentWholeSceneShadows(const FSceneView& View, bool bPrecomputedLightingIsValid) const { return 0; }

	/**
	 * Sets up a projected shadow initializer that's dependent on the current view for shadows from the entire scene.
	 * @param InCascadeIndex cascade index or INDEX_NONE for the distance field cascade
	 * @return True if the whole-scene projected shadow should be used.
	 */
	virtual bool GetViewDependentWholeSceneProjectedShadowInitializer(
		const FSceneView& View,
		int32 InCascadeIndex,
		bool bPrecomputedLightingIsValid,
		FWholeSceneProjectedShadowInitializer& OutInitializer) const
	{
		return false;
	}

	/**
	 * Sets up a projected shadow initializer for the given subject.
	 * @param SubjectBounds - The bounding volume of the subject.
	 * @param OutInitializer - Upon successful return, contains the initialization parameters for the shadow.
	 * @return True if a projected shadow should be cast by this subject-light pair.
	 */
	virtual bool GetPerObjectProjectedShadowInitializer(const FBoxSphereBounds& SubjectBounds, FPerObjectProjectedShadowInitializer& OutInitializer) const
	{
		return false;
	}

	// @param InCascadeIndex cascade index or INDEX_NONE for the distance field cascade
	// @param OutCascadeSettings can be 0
	virtual FSphere GetShadowSplitBounds(const FSceneView& View, int32 InCascadeIndex, bool bPrecomputedLightingIsValid, FShadowCascadeSettings* OutCascadeSettings) const { return FSphere(FVector::ZeroVector, 0); }
	virtual FSphere GetShadowSplitBoundsDepthRange(const FSceneView& View, FVector ViewOrigin, float SplitNear, float SplitFar, FShadowCascadeSettings* OutCascadeSettings) const { return FSphere(FVector::ZeroVector, 0); }

	// Max world-space distance for CSM receivers (near + far cascades combined).
	virtual float GetMaxShadowReceiverDistance(const FSceneView& View, bool bPrecomputedLightingIsValid) const { return 0.0f; }

	ENGINE_API virtual bool GetScissorRect(FIntRect& ScissorRect, const FSceneView& View, const FIntRect& ViewRect) const;

	// @param OutScissorRect the scissor rect used if one is set
	// @return whether a scissor rect is set
	virtual bool SetScissorRect(FRHICommandList& RHICmdList, const FSceneView& View, const FIntRect& ViewRect, FIntRect* OutScissorRect = nullptr) const
	{
		return false;
	}

	virtual bool ShouldCreateRayTracedCascade(ERHIFeatureLevel::Type Type, bool bPrecomputedLightingIsValid, int32 MaxNearCascades) const { return false; }

	// Accessors.
	float GetUserShadowBias() const { return ShadowBias; }
	float GetUserShadowSlopeBias() const { return ShadowSlopeBias; }

	/**
	 * Note: The Rendering thread must not dereference UObjects!
	 * The game thread owns UObject state and may be writing to them at any time.
	 * Mirror the data in the scene proxy and access that instead.
	 */
	UE_DEPRECATED(5.8, "Use GetLightComponentId instead.")
	inline const ULightComponent* GetLightComponent() const { return LightComponent; }
	inline const FLightComponentId& GetLightComponentId() const { return LightComponentId; }
	inline FSceneInterface* GetSceneInterface() const { return SceneInterface; }
	inline FLightSceneInfo* GetLightSceneInfo() const { return LightSceneInfo; }
	inline const FMatrix& GetWorldToLight() const { return WorldToLight; }
	inline const FMatrix& GetLightToWorld() const { return LightToWorld; }
	inline FVector GetDirection() const { return FVector(WorldToLight.M[0][0], WorldToLight.M[1][0], WorldToLight.M[2][0]); }
	inline FVector GetOrigin() const { return LightToWorld.GetOrigin(); }
	inline FVector4 GetPosition() const { return Position; }
	inline const FLinearColor& GetColor() const { return Color; }
	inline float GetIndirectLightingScale() const { return IndirectLightingScale; }
	inline float GetVolumetricScatteringIntensity() const { return VolumetricScatteringIntensity; }
	inline float GetRayEndBias() const { return RayEndBias; }
	inline float GetShadowResolutionScale() const { return ShadowResolutionScale; }
	inline FGuid GetLightGuid() const { return LightGuid; }
	inline float GetShadowSharpen() const { return ShadowSharpen; }
	inline float GetContactShadowLength() const { return ContactShadowLength; }
	inline bool IsContactShadowLengthInWS() const { return bContactShadowLengthInWS; }
	inline float GetContactShadowCastingIntensity() const { return ContactShadowCastingIntensity; }
	inline float GetContactShadowNonCastingIntensity() const { return ContactShadowNonCastingIntensity; }
	inline float GetSpecularScale() const { return SpecularScale; }
	inline float GetDiffuseScale() const { return DiffuseScale; }
	inline FVector GetLightFunctionScale() const { return LightFunctionScale; }
	inline float GetLightFunctionFadeDistance() const { return LightFunctionFadeDistance; }
	inline float GetLightFunctionDisabledBrightness() const { return LightFunctionDisabledBrightness; }
	inline UTextureLightProfile* GetIESTexture() const { return IESTexture; }
	ENGINE_API FTexture* GetIESTextureResource() const;
	inline const FMaterialRenderProxy* GetLightFunctionMaterial() const { return LightFunctionMaterial; }
	inline bool IsMovable() const { return bMovable; }
	inline bool HasStaticLighting() const { return bStaticLighting; }
	inline bool HasStaticShadowing() const { return bStaticShadowing; }
	inline bool CastsDynamicShadow() const { return bCastDynamicShadow; }
	inline bool CastsStaticShadow() const { return bCastStaticShadow; }
	inline bool AllowMegaLights() const { return bAllowMegaLights; }
	TEnumAsByte<EMegaLightsShadowMethod::Type> GetMegaLightsShadowMethod() const { return MegaLightsShadowMethod; }
	inline bool CastsTranslucentShadows() const { return bCastTranslucentShadows; }
	inline bool CastsVolumetricShadow() const { return bCastVolumetricShadow; }
	inline bool CastsHairStrandsDeepShadow() const { return bCastHairStrandsDeepShadow; }
	inline TEnumAsByte<ECastRayTracedShadow::Type> CastsRaytracedShadow() const { return CastRaytracedShadow; }
	inline bool AffectReflection() const { return bAffectReflection; }
	inline bool AffectGlobalIllumination() const { return bAffectGlobalIllumination; }
	inline bool CastsShadowsFromCinematicObjectsOnly() const { return bCastShadowsFromCinematicObjectsOnly; }
	inline bool CastsModulatedShadows() const { return bCastModulatedShadows; }
	inline const FLinearColor& GetModulatedShadowColor() const { return ModulatedShadowColor; }
	inline const float GetShadowAmount() const { return ShadowAmount; }
	inline bool AffectsTranslucentLighting() const { return bAffectTranslucentLighting; }
	inline bool Transmission() const { return bTransmission; }
	inline bool UseRayTracedDistanceFieldShadows() const { return bUseRayTracedDistanceFieldShadows; }
	inline bool UseVirtualShadowMaps() const { return bUseVirtualShadowMaps; }
	inline float GetRayStartOffsetDepthScale() const { return RayStartOffsetDepthScale; }
	inline uint8 GetLightType() const { return LightType; }
	inline uint8 GetLightingChannelMask() const { return LightingChannelMask; }
	inline uint8 GetViewLightingChannelMask() const { return ViewLightingChannelMask; }
	inline FName GetComponentFName() const { return ComponentName; }

	inline bool IsSelected() const { return bSelected; }
	inline void SetSelected(bool bNewSelected) { bSelected = bNewSelected; }

	// Necessary to build FLightSceneInfo
	inline bool IsPrecomputedLightingValid() const { return bIsPrecomputedLightingValid; }
	inline bool IsLightShaftBloomEnabled() const { return bEnableLightShaftBloom; }
	inline float GetBloomScale() const { return BloomScale; }
	inline float GetBloomThreshold() const { return BloomThreshold; }
	inline float GetBloomMaxBrightness() const { return BloomMaxBrightness; }
	inline const FColor& GetBloomTint() const { return BloomTint; }

	/**
	 * Use to get the owning actor label (or component name as fallback, if the owner is null or ENABLE_DEBUG_LABELS is off) for diagnostic messages, debug or profiling.
	 * The actor label is what is shown in the UI (as opposed to the the FName).
	 */
#if ACTOR_HAS_LABELS
	inline const FString& GetOwnerNameOrLabel() const { return OwnerNameOrLabel; }
#else 
	inline FString GetOwnerNameOrLabel() const { return ComponentName.ToString(); }
#endif 

	inline FName GetLevelName() const { return LevelName; }
	inline TStatId GetStatId() const
	{
		return StatId;
	}
	inline int32 GetShadowMapChannel() const { return ShadowMapChannel; }
	inline int32 GetPreviewShadowMapChannel() const { return PreviewShadowMapChannel; }

	inline const class FStaticShadowDepthMap* GetStaticShadowDepthMap() const { return StaticShadowDepthMap; }

	inline bool GetForceCachedShadowsForMovablePrimitives() const { return bForceCachedShadowsForMovablePrimitives; }

	inline uint32 GetSamplesPerPixel() const { return SamplesPerPixel; }
	inline float GetDeepShadowLayerDistribution() const { return DeepShadowLayerDistribution; }
	/**
	 * Shifts light position and all relevant data by an arbitrary delta.
	 * Called on world origin changes
	 * @param InOffset - The delta to shift by
	 */
	ENGINE_API virtual void ApplyWorldOffset(FVector InOffset);

	virtual float GetMaxDrawDistance() const { return 0.0f; }
	virtual float GetFadeRange() const { return 0.0f; }

	// Atmosphere / Fog related functions.

	inline bool IsUsedAsAtmosphereSunLight() const { return bUsedAsAtmosphereSunLight; }
	inline uint8 GetAtmosphereSunLightIndex() const { return AtmosphereSunLightIndex; }
	inline FLinearColor GetAtmosphereSunDiskColorScale() const { return AtmosphereSunDiskColorScale; }
	virtual void SetAtmosphereRelatedProperties(FLinearColor TransmittanceTowardSunIn, FLinearColor SunDiscOuterSpaceLuminanceIn) {}
	virtual FLinearColor GetOuterSpaceLuminance() const { return FLinearColor::White; }
	virtual FLinearColor GetOuterSpaceIlluminance() const { return GetColor(); }
	virtual FLinearColor GetAtmosphereTransmittanceTowardSun() const { return FLinearColor::White; }
	virtual FLinearColor GetSunIlluminanceOnGroundPostTransmittance() const { return GetColor(); }
	virtual FLinearColor GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance() const { return GetColor(); }
	virtual bool GetPerPixelTransmittanceEnabled() const { return false; }
	static float GetSunOnEarthHalfApexAngleRadian()
	{
		const float SunOnEarthApexAngleDegree = 0.545f;	// Apex angle == angular diameter
		return 0.5f * SunOnEarthApexAngleDegree * UE_PI / 180.0f;
	}
	/**
	 * @return the light half apex angle (half angular diameter) in radian.
	 */
	virtual float GetSunLightHalfApexAngleRadian() const { return GetSunOnEarthHalfApexAngleRadian(); }

	virtual bool GetCastShadowsOnClouds() const { return false; }
	virtual bool GetCastShadowsOnAtmosphere() const { return false; }
	virtual bool GetCastCloudShadows() const { return false; }
	virtual float GetCloudShadowExtent() const { return 1.0f; }
	virtual float GetCloudShadowMapResolutionScale() const { return 1.0f; }
	virtual float GetCloudShadowRaySampleCountScale() const { return 1.0f; }
	virtual float GetCloudShadowStrength() const { return 1.0f; }
	virtual float GetCloudShadowOnAtmosphereStrength() const { return 1.0f; }
	virtual float GetCloudShadowOnSurfaceStrength() const { return 1.0f; }
	virtual float GetCloudShadowDepthBias() const { return 0.0f; }
	virtual FLinearColor GetCloudScatteredLuminanceScale() const { return FLinearColor::White; }
	virtual bool GetUsePerPixelAtmosphereTransmittance() const { return false; }

	inline void  SetLightFunctionAtlasIndices(uint8 LightIndex) { LightFunctionAtlasLightIndex = LightIndex; }
	inline bool  HasValidLightFunctionAtlasSlot() const { return LightFunctionAtlasLightIndex != 0; }
	inline uint8 GetLightFunctionAtlasLightIndex() const { return LightFunctionAtlasLightIndex; }

protected:

	friend class FScene;
	friend class FSceneRenderer;
	friend class FLightSceneInfo;

	/** The light component. (Only necessary for FScene::GetRelevantLights) */
	const ULightComponent* LightComponent;

	/** The runtime unique id used to identify a light component. */
	const FLightComponentId LightComponentId;

	/** The scene the primitive is in. */
	FSceneInterface* SceneInterface;

	/** The light's scene info. */
	class FLightSceneInfo* LightSceneInfo;

	/** The homogeneous position of the light. */
	FVector4 Position;

	/** The light color. */
	FLinearColor Color;

	/** A transform from world space into light space. */
	FMatrix WorldToLight;

	/** A transform from light space into world space. */
	FMatrix LightToWorld;

	/** Scale for indirect lighting from this light.  When 0, indirect lighting is disabled. */
	float IndirectLightingScale;

	/** Scales this light's intensity for volumetric scattering. */
	float VolumetricScatteringIntensity;

	/** Bias applied to end of shadow rays (-1 if global value should be used) */
	float RayEndBias;

	float ShadowResolutionScale;

	/** User setting from light component, 0:no bias, 0.5:reasonable, larger object might appear to float */
	float ShadowBias;

	/** User setting from light component, 0:no bias, 0.5:reasonable, larger object might appear to float */
	float ShadowSlopeBias;

	/** Sharpen shadow filtering */
	float ShadowSharpen;

	/** Length of screen space ray trace for sharp contact shadows. */
	float ContactShadowLength;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" enabled. 0 = no shadow, 1 (default) = fully shadowed. */
	float ContactShadowCastingIntensity;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" disabled. 0 (default) = no shadow, 1 = fully shadowed. */
	float ContactShadowNonCastingIntensity;

	/** Specular scale */
	float SpecularScale;

	/** Diffuse scale */
	float DiffuseScale;

	/** The light's persistent shadowing GUID. */
	FGuid LightGuid;

	/**
	 * Shadow map channel which is used to match up with the appropriate static shadowing during a deferred shading pass.
	 * This is generated during a lighting build.
	 */
	int32 ShadowMapChannel;

	/** Transient shadowmap channel used to preview the results of stationary light shadowmap packing. */
	int32 PreviewShadowMapChannel;

	float RayStartOffsetDepthScale;

	const class FStaticShadowDepthMap* StaticShadowDepthMap;

	/** Light function parameters. */
	FVector	LightFunctionScale;
	float LightFunctionFadeDistance;
	float LightFunctionDisabledBrightness;
	const FMaterialRenderProxy* LightFunctionMaterial;

	/**
	 * IES texture (light profiles from real world measured data)
	 * We are safe to store a U pointer as those objects get deleted deferred, storing an FTexture pointer would crash if we recreate the texture
	 */
	UTextureLightProfile* IESTexture;

	/** True: length of screen space ray trace for sharp contact shadows is in world space. False: in screen space. */
	uint8 bContactShadowLengthInWS : 1;

	/* True if the light's Mobility is set to Movable. */
	const uint8 bMovable : 1;

	/**
	 * Return True if a light's parameters as well as its position is static during gameplay, and can thus use static lighting.
	 * A light with HasStaticLighting() == true will always have HasStaticShadowing() == true as well.
	 */
	const uint8 bStaticLighting : 1;

	/**
	 * Whether the light has static direct shadowing.
	 * The light may still have dynamic brightness and color.
	 * The light may or may not also have static lighting.
	 */
	uint8 bStaticShadowing : 1;

	/** True if the light casts dynamic shadows. */
	uint8 bCastDynamicShadow : 1;

	/** True if the light casts static shadows. */
	const uint8 bCastStaticShadow : 1;

	/** Whether the light is allowed to cast dynamic shadows from translucency. */
	const uint8 bCastTranslucentShadows : 1;

	/** Whether light from this light transmits through surfaces with subsurface scattering profiles. Requires light to be movable. */
	uint8 bTransmission : 1;

	const uint8 bCastVolumetricShadow : 1;
	const uint8 bCastHairStrandsDeepShadow : 1;
	const uint8 bCastShadowsFromCinematicObjectsOnly : 1;

	const uint8 bForceCachedShadowsForMovablePrimitives : 1;

	/** Whether the light shadows are computed with shadow-mapping or ray-tracing (when available). */
	const TEnumAsByte<ECastRayTracedShadow::Type> CastRaytracedShadow;

	/** Whether the light affects objects in reflections, when ray-traced reflection is enabled. */
	const uint8 bAffectReflection : 1;

	/** Whether the light affects global illumination, when ray-traced global illumination is enabled. */
	const uint8 bAffectGlobalIllumination : 1;

	/** Whether the light affects translucency or not.  Disabling this can save GPU time when there are many small lights. */
	const uint8 bAffectTranslucentLighting : 1;

	/** Whether to consider light as a sunlight for atmospheric scattering and exponential height fog. */
	const uint8 bUsedAsAtmosphereSunLight : 1;

	/** Whether to use ray traced distance field area shadows. */
	const uint8 bUseRayTracedDistanceFieldShadows : 1;

	/** Whether to use virtual shadow maps. */
	uint8 bUseVirtualShadowMaps : 1;

	/** Whether the light will cast modulated shadows when using the forward renderer (mobile). */
	uint8 bCastModulatedShadows : 1;

	/** Whether to render csm shadows for movable objects only (mobile). */
	uint8 bUseWholeSceneCSMForMovableObjects : 1;

    /** Is the light selected in the editor? */
	uint8 bSelected : 1;

	/** Whether the light should be rendered with MegaLights. */
	const uint8 bAllowMegaLights : 1;

	/** Whether the precomputed lighting is valid. */
	uint8 bIsPrecomputedLightingValid : 1;

	/** Whether to render light shaft bloom from this light. */
	uint8 bEnableLightShaftBloom : 1;

	/** Scales the additive color. */
	float BloomScale = 1.0f;

	/** Scene color must be larger than this to create bloom in the light shafts. */
	float BloomThreshold = 1.0f;

	/** After exposure is applied, scene color brightness larger than BloomMaxBrightness will be rescaled down to BloomMaxBrightness. */
	float BloomMaxBrightness = 1.0f;

	/** Multiplies against scene color to create the bloom color. */
	FColor BloomTint = FColor(EForceInit::ForceInitToZero);

	/** Whether the light shadows are computed with shadow-mapping or ray-tracing (when available). */
	const TEnumAsByte<EMegaLightsShadowMethod::Type> MegaLightsShadowMethod;

	/**
	* The light index in order to be able to read matrix and parameters when reading the light function atlas for that light.
	* A value of 0 means this is the default identity light function and no light function sampling will be done in shader.
	*/
	uint8 LightFunctionAtlasLightIndex;

	/** The index of the atmospheric light. Multiple lights can be considered when computing the sky/atmospheric scattering. */
	const uint8 AtmosphereSunLightIndex;

	FLinearColor AtmosphereSunDiskColorScale;

	/** The light type (ELightComponentType) */
	const uint8 LightType;

	uint8 LightingChannelMask;
	uint8 ViewLightingChannelMask;

	/** Used for dynamic stats */
	TStatId StatId;

	/** The name of the light component. */
	FName ComponentName;

	/** The name of the level the light is in. */
	FName LevelName;

	/** Used to control the amount of additional dither filtering applied to shadows for each light. */
	float VSMTexelDitherScale;
	/** Used to control shadow resolution for each light. */
	float VSMResolutionLodBias;

	/** Only for whole scene directional lights, if FarShadowCascadeCount > 0 and FarShadowDistance >= WholeSceneDynamicShadowRadius, where far shadow cascade should end. */
	float FarShadowDistance;

	/** Only for whole scene directional lights, 0: no FarShadowCascades, otherwise the count of cascades between WholeSceneDynamicShadowRadius and FarShadowDistance that are covered by distant shadow cascades. */
	uint32 FarShadowCascadeCount;

	/** Modulated shadow color. */
	FLinearColor ModulatedShadowColor;

	/** Control the amount of shadow occlusion. */
	float ShadowAmount;

	/** Samples per pixel for ray tracing */
	uint32 SamplesPerPixel;

	/** Deep shadow layer distribution. */
	float DeepShadowLayerDistribution;

	/** IES texture atlas id. */
	uint32 IESAtlasId;

	/** Lookup for cached shadow atlas information for timesliced runtime reflection captures. */
	uint8 TimesliceAtlasIndex[4];		// Index in array of cached atlased shadows, by ProjectionIndex
	uint32 TimesliceAtlasFrame;			// Frame number to track if AtlasIndex is valid (matches frame number of current active capture)

	/**
	 * Updates the light proxy's cached transforms.
	 * @param InLightToWorld - The new light-to-world transform.
	 * @param InPosition - The new position of the light.
	 */
	ENGINE_API void SetTransform(const FMatrix& InLightToWorld, const FVector4& InPosition);

	/** Updates the light's color. */
	ENGINE_API void SetColor(const FLinearColor& InColor);

private:
#if ACTOR_HAS_LABELS
	// May store the label or name of the actor containing the component or if there is no actor the name of the component itself
	FString OwnerNameOrLabel;
#endif
};