// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightComponentId.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Math/Color.h"
#include "UObject/ObjectPtr.h"

class FSceneInterface;
class ULightComponent;
class FStaticShadowDepthMap;

struct FLightSceneProxyDesc
{
	FLightSceneProxyDesc() = default;
	virtual ~FLightSceneProxyDesc() = default;
	ENGINE_API virtual void Serialize(FArchive& Ar);
	ENGINE_API const FStaticShadowDepthMap* GetStaticShadowDepthMap() const;
	ENGINE_API TStatId GetStatID() const;

	//
	// Transient properties
	//

	/** The light component. */
	const ULightComponent* LightComponent = nullptr;

	/** A runtime unique id that can be used to identify light source **/
	FLightComponentId LightComponentId;

	/** The scene the light is in. */
	FSceneInterface* SceneInterface = nullptr;

#if ACTOR_HAS_LABELS
	// May store the label or name of the actor containing the component or if there is no actor the name of the component itself
	FString OwnerNameOrLabel;
#endif

	/** Whether the light is selected in the editor. */
	uint8 bSelected : 1 = false;

	//
	// Persistent properties
	// 

	/** True: length of screen space ray trace for sharp contact shadows is in world space. False: in screen space. */
	uint8 bContactShadowLengthInWS : 1 = false;

	/* True if the light's Mobility is set to Movable. */
	uint8 bMovable : 1 = false;

	/** True if a light's parameters as well as its position is static during gameplay, and can thus use static lighting. */
	uint8 bStaticLighting : 1 = false;

	/** Whether the light has static direct shadowing. */
	uint8 bStaticShadowing : 1 = false;

	/** True if the light casts dynamic shadows. */
	uint8 bCastDynamicShadow : 1 = false;

	/** True if the light casts static shadows. */
	uint8 bCastStaticShadow : 1 = false;

	/** Whether the light is allowed to cast dynamic shadows from translucency. */
	uint8 bCastTranslucentShadows : 1 = false;

	/** Whether light from this light transmits through surfaces with subsurface scattering profiles. Requires light to be movable. */
	uint8 bTransmission : 1 = false;

	/** Whether the light shadows volumetric fog.  Disabling this can save GPU time. */
	uint8 bCastVolumetricShadow : 1 = false;

	/** Whether the light should cast high quality hair-strands self-shadowing. When this option is enabled, an extra GPU cost for this light. */
	uint8 bCastHairStrandsDeepShadow : 1 = false;

	/** Whether the light should only cast shadows from components marked as bCastCinematicShadows. */
	uint8 bCastShadowsFromCinematicObjectsOnly : 1 = false;

	/** Enables cached shadows for movable primitives for this light even if r.shadow.cachedshadowscastfrommovableprimitives is 0. */
	uint8 bForceCachedShadowsForMovablePrimitives : 1 = false;

	/** Whether the light affects objects in reflections, when ray-traced reflection is enabled. */
	uint8 bAffectReflection : 1 = false;

	/** Whether the light affects global illumination, when ray-traced global illumination is enabled. */
	uint8 bAffectGlobalIllumination : 1 = false;

	/** Whether the light affects translucency or not.  Disabling this can save GPU time when there are many small lights. */
	uint8 bAffectTranslucentLighting : 1 = false;

	/** Whether to consider light as a sunlight for atmospheric scattering and exponential height fog. */
	uint8 bUsedAsAtmosphereSunLight : 1 = false;

	/** Whether to use ray traced distance field area shadows. */
	uint8 bUseRayTracedDistanceFieldShadows : 1 = false;

	/** Whether the light should be rendered with MegaLights. */
	uint8 bAllowMegaLights : 1 = false;

	/** Whether the precomputed lighting is valid. */
	uint8 bIsPrecomputedLightingValid : 1 = false;

	/** Whether to render light shaft bloom from this light. */
	uint8 bEnableLightShaftBloom : 1 = false;

	/** The light color. */
	FLinearColor Color = FLinearColor(EForceInit::ForceInitToZero);

	/** Scale for indirect lighting from this light.  When 0, indirect lighting is disabled. */
	float IndirectLightingScale = 0.0f;

	/** Scales this light's intensity for volumetric scattering. */
	float VolumetricScatteringIntensity = 1.0f;

	/** Bias applied to end of shadow rays (-1 if global value should be used) */
	float RayEndBias = 0.0f;

	float ShadowResolutionScale = 1.0f;

	/** User setting from light component, 0:no bias, 0.5:reasonable, larger object might appear to float */
	float ShadowBias = 0.0f;

	/** User setting from light component, 0:no bias, 0.5:reasonable, larger object might appear to float */
	float ShadowSlopeBias = 0.0f;

	/** Sharpen shadow filtering */
	float ShadowSharpen = 0.0f;

	/** Length of screen space ray trace for sharp contact shadows. */
	float ContactShadowLength = 0.0f;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" enabled. 0 = no shadow, 1 (default) = fully shadowed. */
	float ContactShadowCastingIntensity = 1.0f;

	/** Intensity of the shadows cast by primitives with "cast contact shadow" disabled. 0 (default) = no shadow, 1 = fully shadowed. */
	float ContactShadowNonCastingIntensity = 0.0f;

	/** Specular scale */
	float SpecularScale = 1.0f;

	/** Diffuse scale */
	float DiffuseScale = 1.0f;

	/** The light's persistent shadowing GUID. */
	FGuid LightGuid;

	/** Shadow map channel which is used to match up with the appropriate static shadowing during a deferred shading pass. This is generated during a lighting build. */
	int32 MapBuildDataShadowMapChannel = INDEX_NONE;

	/** Transient shadowmap channel used to preview the results of stationary light shadowmap packing. */
	int32 PreviewShadowMapChannel = INDEX_NONE;

	/** Controls how large of an offset ray traced shadows have from the receiving surface as the camera gets further away. */
	float RayStartOffsetDepthScale = 0.0f;

	/** Light function parameters. */
	FVector	LightFunctionScale = FVector::ZeroVector;

	/** Distance at which the light function should be completely faded to DisabledBrightness. */
	float LightFunctionFadeDistance = 0.0f;

	/** Brightness factor applied to the light when the light function is specified but disabled, for example in scene captures that use SceneCapView_LitNoShadows. */
	float LightFunctionDisabledBrightness = 0.0f;

	/** The light function material to be applied to this light. */
	TObjectPtr<class UMaterialInterface> LightFunctionMaterial = nullptr;

	/** IES texture (light profiles from real world measured data) */
	TObjectPtr<class UTextureLightProfile> IESTexture = nullptr;

	/** Whether the light shadows are computed with shadow-mapping or ray-tracing (when available). */
	TEnumAsByte<ECastRayTracedShadow::Type> CastRaytracedShadow = ECastRayTracedShadow::Disabled;

	/** Scales the additive color. */
	float BloomScale = 1.0f;

	/** Scene color must be larger than this to create bloom in the light shafts. */
	float BloomThreshold = 1.0f;

	/** After exposure is applied, scene color brightness larger than BloomMaxBrightness will be rescaled down to BloomMaxBrightness. */
	float BloomMaxBrightness = 1.0f;

	/** Multiplies against scene color to create the bloom color. */
	FColor BloomTint = FColor(EForceInit::ForceInitToZero);

	/** Whether the light shadows are computed with shadow-mapping or ray-tracing (when available). */
	TEnumAsByte<EMegaLightsShadowMethod::Type> MegaLightsShadowMethod = EMegaLightsShadowMethod::Default;

	/** The index of the atmospheric light. Multiple lights can be considered when computing the sky/atmospheric scattering. */
	uint8 AtmosphereSunLightIndex = 0xFF;

	FLinearColor AtmosphereSunDiskColorScale = FLinearColor(EForceInit::ForceInitToZero);

	/** The light type (ELightComponentType) */
	uint8 LightType = 0xFF;

	/** Channels that this light should affect. */
	uint8 LightingChannelMask = 0;

	/** View / light masking support.  Controls which views this light should affect. */
	uint8 ViewLightingChannelMask = 0;

	/** The name of the light component. */
	FName ComponentName = NAME_None;

	/** The name of the level the light is in. */
	FName LevelName = NAME_None;

	/** Samples per pixel for ray tracing */
	uint32 SamplesPerPixel = 1;

	/** Deep shadow layer distribution. */
	float DeepShadowLayerDistribution = 0.5f;
};
