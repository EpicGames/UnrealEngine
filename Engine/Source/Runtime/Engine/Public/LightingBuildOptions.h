// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class ULevel;

/**
 * A set of parameters specifying how static lighting is rebuilt.
 */
class FLightingBuildOptions
{
public:
	FLightingBuildOptions()
	:	bUseErrorColoring(false)
	,	bDumpBinaryResults(false)
	,	bOnlyBuildSelected(false)
	,	bOnlyBuildCurrentLevel(false)
	,	bOnlyBuildSelectedLevels(false)
	,	bOnlyBuildVisibility(false)
	,	bShowLightingBuildInfo(false)
	,	bVolumetricLightmapFinalizerPass(false)
	,	bApplyDeferedActorMappingPass(false)
	,	QualityLevel(Quality_Preview)
	,	NumUnusedLocalCores(1)
	{}

	/**
	 * @return true if the lighting should be built for the level, given the current set of lighting build options.
	 */
	ENGINE_API bool ShouldBuildLightingForLevel(ULevel* Level) const;

	/** Whether to color problem objects (wrapping uvs, etc.)						*/
	bool					bUseErrorColoring;
	/** Whether to dump binary results or not										*/
	bool					bDumpBinaryResults;
	/** Whether to only build lighting for selected actors/brushes/surfaces			*/
	bool					bOnlyBuildSelected;
	/** Whether to only build lighting for current level							*/
	bool					bOnlyBuildCurrentLevel;
	/** Whether to only build lighting for levels selected in the Level Browser.	*/
	bool					bOnlyBuildSelectedLevels;
	/** Whether to only build visibility, and leave lighting untouched.				*/
	bool					bOnlyBuildVisibility;
	/** Whether to display the lighting build info following a build.				*/
	bool					bShowLightingBuildInfo;
	/** Indicates to is the volumetric lightmaps finalizing pass (for WP maps)    	*/
	bool					bVolumetricLightmapFinalizerPass;
	/** Indicates to is the lightmaps finalizing pass (for WP maps)    	*/
	bool					bApplyDeferedActorMappingPass;
	/** The quality level to use for the lighting build. (0-3)						*/
	ELightingBuildQuality	QualityLevel;
	/** The quality level to use for half-resolution lightmaps (not exposed)		*/
	static ELightingBuildQuality	HalfResolutionLightmapQualityLevel;
	/** The number of cores to leave 'unused'										*/
	int32						NumUnusedLocalCores;
	/** The set of levels selected in the Level Browser.							*/
	TArray<ULevel*>			SelectedLevels;	
	/** The directory that'll be used to store the deferred mappings				*/
	FString					MappingsDirectory;
	
	/** The custom filter to invoke to know if we should build lighting for an Actor
	* 
	*  void ShouldBuildLigthing(const AActor*, bool& bBuildLightingForActor, bool& bIncludeActorInLightingScene, bool& bDeferActorMapping)
	* 
	*/	
	TFunction<void(const AActor*, bool&, bool&, bool& )>	ShouldBuildLighting;
};

