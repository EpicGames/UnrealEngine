// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"
#include "Containers/StaticArray.h"

class FLightCacheInterface;

/** The maximum value between NUM_LQ_LIGHTMAP_COEF and NUM_HQ_LIGHTMAP_COEF. */
inline constexpr int32 MAX_NUM_LIGHTMAP_COEF = 2;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FPrecomputedLightingUniformParameters, ENGINE_API)
SHADER_PARAMETER(FVector4f, StaticShadowMapMasks) // TDistanceFieldShadowsAndLightMapPolicy
SHADER_PARAMETER(FVector4f, InvUniformPenumbraSizes) // TDistanceFieldShadowsAndLightMapPolicy
SHADER_PARAMETER(FVector4f, LightMapCoordinateScaleBias) // TLightMapPolicy
SHADER_PARAMETER(FVector4f, ShadowMapCoordinateScaleBias) // TDistanceFieldShadowsAndLightMapPolicy
SHADER_PARAMETER_ARRAY_EX(FVector4f, LightMapScale, [MAX_NUM_LIGHTMAP_COEF], EShaderPrecisionModifier::Half) // TLightMapPolicy
SHADER_PARAMETER_ARRAY_EX(FVector4f, LightMapAdd, [MAX_NUM_LIGHTMAP_COEF], EShaderPrecisionModifier::Half) // TLightMapPolicy
SHADER_PARAMETER_ARRAY(FUintVector4, LightmapVTPackedPageTableUniform, [2]) // VT (1 page table, 2x uint4)
SHADER_PARAMETER_ARRAY(FUintVector4, LightmapVTPackedUniform, [5]) // VT (5 layers, 1x uint4 per layer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

ENGINE_API void GetDefaultPrecomputedLightingParameters(FPrecomputedLightingUniformParameters& Parameters);

ENGINE_API void GetPrecomputedLightingParameters(
	ERHIFeatureLevel::Type FeatureLevel,
	FPrecomputedLightingUniformParameters& Parameters,
	const FLightCacheInterface* LCI);

struct FLightmapSceneShaderData
{
	// Must match usf
	enum { DataStrideInFloat4s = 15 };

	TStaticArray<FVector4f, DataStrideInFloat4s> Data;

	FLightmapSceneShaderData()
		: Data(InPlace, NoInit)
	{
		FPrecomputedLightingUniformParameters ShaderParameters;
		GetDefaultPrecomputedLightingParameters(ShaderParameters);
		Setup(ShaderParameters);
	}

	explicit FLightmapSceneShaderData(const FPrecomputedLightingUniformParameters& ShaderParameters)
		: Data(InPlace, NoInit)
	{
		Setup(ShaderParameters);
	}

	ENGINE_API FLightmapSceneShaderData(const class FLightCacheInterface* LCI, ERHIFeatureLevel::Type FeatureLevel);

	ENGINE_API void Setup(const FPrecomputedLightingUniformParameters& ShaderParameters);
};
