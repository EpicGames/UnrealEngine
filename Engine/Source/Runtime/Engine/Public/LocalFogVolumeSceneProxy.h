// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Color.h"

class ULocalFogVolumeComponent;

/** Represents a UVolumetricCloudComponent to the rendering thread, created game side from the component. */
class FLocalFogVolumeSceneProxy
{
public:

	// Initialization constructor.
	ENGINE_API FLocalFogVolumeSceneProxy(const ULocalFogVolumeComponent* InComponent);
	ENGINE_API ~FLocalFogVolumeSceneProxy();

	void UpdateComponentTransform(const FTransform& Transform);

	FTransform FogTransform;

	float RadialFogExtinction;
	float HeightFogExtinction;
	float HeightFogFalloff;
	float HeightFogOffset;
	float FogUniformScale;

	uint8 FogSortPriority;
	float FogStartDistance;

	float FogPhaseG;
	FLinearColor FogAlbedo;
	FLinearColor FogEmissive;
private:
};
