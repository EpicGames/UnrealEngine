// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalLightSceneProxy.h: Local light scene info definition.
=============================================================================*/

#pragma once

#include "LightSceneProxy.h"

class ULocalLightComponent;
struct FLocalLightSceneProxyDesc;

/** The parts of the point light scene info that aren't dependent on the light policy type. */
class FLocalLightSceneProxy : public FLightSceneProxy
{
public:

	/** The light radius. */
	float Radius;

	/** One over the light's radius. */
	float InvRadius;

	/** Initialization constructors. */
	FLocalLightSceneProxy(const ULocalLightComponent* Component);
	ENGINE_API FLocalLightSceneProxy(const FLocalLightSceneProxyDesc& LightDesc);

	//~Begin FLightSceneProxy Interface
	ENGINE_API virtual float GetMaxDrawDistance() const final override;
	ENGINE_API virtual float GetFadeRange() const final override;
	ENGINE_API virtual float GetRadius() const override;
	ENGINE_API virtual bool AffectsBounds(const FBoxSphereBounds& Bounds) const override;
	ENGINE_API virtual bool GetScissorRect(FIntRect& ScissorRect, const FSceneView& View, const FIntRect& ViewRect) const override;
	ENGINE_API virtual bool SetScissorRect(FRHICommandList& RHICmdList, const FSceneView& View, const FIntRect& ViewRect, FIntRect* OutScissorRect = nullptr) const override;
	ENGINE_API virtual FSphere GetBoundingSphere() const;
	ENGINE_API virtual float GetEffectiveScreenRadius(const FViewMatrices& ShadowViewMatrices, const FIntPoint& CameraViewRectSize) const override;
	ENGINE_API virtual bool GetPerObjectProjectedShadowInitializer(const FBoxSphereBounds& SubjectBounds, class FPerObjectProjectedShadowInitializer& OutInitializer) const override;
	ENGINE_API virtual bool IsLocalLight() const override;
	//~End FLightSceneProxy Interface

	/**
	 * Called on the light scene info after it has been passed to the rendering thread to update the rendering thread's cached info when
	 * the light's radius changes.
	 */
	void UpdateRadius_GameThread(float Radius);

	virtual FVector GetPerObjectProjectedShadowProjectionPoint(const FBoxSphereBounds& SubjectBounds) const;

protected:

	/** Updates the light scene info's radius from the component. */
	ENGINE_API void UpdateRadius(float ComponentRadius);

	float MaxDrawDistance;
	float FadeRange;
	float InverseExposureBlend;
};