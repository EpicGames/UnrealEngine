// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightSceneProxyDesc.h"

struct FLocalLightSceneProxyDesc : public FLightSceneProxyDesc
{
	typedef FLightSceneProxyDesc Super;

	FLocalLightSceneProxyDesc() = default;
	virtual ~FLocalLightSceneProxyDesc() = default;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** World distance to where the light stops rendering (lighting & shadows). 0.0 or less means no max draw distance. */
	float MaxDrawDistance = 0.0f;

	/** Fade the light out as it approaches MaxDrawDistance. */
	float MaxDistanceFadeRange = 0.0f;

	/** Blend Factor used to blend between Intensity and Intensity/Exposure. */
	float InverseExposureBlend = 0.0f;

	/** Bounds the light's visible influence. */
	float AttenuationRadius = 0.0f;
};