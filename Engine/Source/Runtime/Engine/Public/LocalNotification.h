// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLocalNotification, Log, All);

/**
 * Interface for local notification modules
 */

class ILocalNotificationService
{
public:

	/** Clear all pending local notifications. Typically this will be done before scheduling new notifications when going into the background */
	virtual void ClearAllLocalNotifications() = 0;

	/** Schedule a local notification at a specific time, inLocalTime specifies the current local time or if UTC time should be used 
	 * @param FireDateTime The time at which to fire the local notification
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param Title The title of the notification
	 * @param Body The more detailed description of the notification
	 * @param Action The text to be displayed on the slider controller
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	 * @return A unique identifier used to cancel the notification
	*/
	virtual int32 ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent) = 0;

	/** Schedule a local notification at a specific time with a specific Id, inLocalTime specifies the current local time or if UTC time should be used
	 * Currently only works for Android.
	 * @param FireDateTime The time at which to fire the local notification
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param Title The title of the notification
	 * @param Body The more detailed description of the notification
	 * @param Action The text to be displayed on the slider controller
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	 * @param IdOverride A int that is passed to pin the notification to an Id. By doing this it will only show the latest notification sent from that ID.
	 * @return A unique identifier used to cancel the notification
	*/
	virtual int32 ScheduleLocalNotificationAtTimeOverrideId(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent, int32 IdOverride) = 0;

	/** Schedule a local notification badge at a specific time, inLocalTime specifies the current local time or if UTC time should be used
	 * @param FireDateTime The time at which to fire the local notification
	 * @param LocalTime If true the provided time is in the local timezone, if false it is in UTC
	 * @param ActivationEvent A string that is passed in the delegate callback when the app is brought into the foreground from the user activating the notification
	 * @return A unique identifier used to cancel the notification
	*/
	virtual int32 ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool LocalTime, const FString& ActivationEvent) = 0;

	/** Get the local notification that was used to launch the app
	 * @param NotificationLaunchedApp Return true if a notification was used to launch the app
	 * @param ActivationEvent Returns the name of the ActivationEvent if a notification was used to launch the app
	 * @param FireDate Returns the time the notification was activated
	*/
	virtual void GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate) = 0;

	/** Cancel a local notification given the ActivationEvent
	 * @param ActivationEvent The string passed into the Schedule call for the notification to be cancelled
	*/
	virtual void CancelLocalNotification(const FString& ActivationEvent) = 0;

	/** Cancel a local notification given the ActivationEvent
	 * @param NotificationId The Id returned from ScheduleLocalNotificationAtTime*
	 */
	virtual void CancelLocalNotification(int32 NotificationId) = 0;


	/** Set the local notification that was used to launch the app
	 * @param ActivationEvent Returns the name of the ActivationEvent if a notification was used to launch the app
	 * @param FireDate Returns the time the notification was activated
	*/
	virtual void SetLaunchNotification(FString const& ActivationEvent, int32 FireDate) = 0;

};

/** Defines the interface of a module implementing a local notification server. */
class ILocalNotificationModule : public IModuleInterface
{
public:

	/** Gets the one true local notification service. */
	virtual ILocalNotificationService* GetLocalNotificationService() = 0;
};
