// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "ShaderParameterMacros.h"
#include "Components.h"
#include "VertexFactory.h"
#include "GlobalRenderResources.h"

class FMaterial;
class FSceneView;
struct FMeshBatchElement;

/*=============================================================================
	LocalVertexFactory.h: Local vertex factory definitions.
=============================================================================*/

BEGIN_SHADER_PARAMETER_STRUCT(FLocalVertexFactoryInstanceDitherParameters, ENGINE_API)
	SHADER_PARAMETER(FVector4f, InstancingViewZCompareZero)
	SHADER_PARAMETER(FVector4f, InstancingViewZCompareOne)
	SHADER_PARAMETER(FVector4f, InstancingViewZConstant)
	SHADER_PARAMETER(FVector4f, InstancingTranslatedWorldViewOriginZero)
	SHADER_PARAMETER(FVector4f, InstancingTranslatedWorldViewOriginOne)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLocalVertexFactoryUniformShaderParameters,ENGINE_API)
	SHADER_PARAMETER(FIntVector4,VertexFetch_Parameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FLocalVertexFactoryInstanceDitherParameters, InstanceDitherParameters)
	SHADER_PARAMETER(int32, PreSkinBaseVertexIndex)
	SHADER_PARAMETER(uint32,LODLightmapDataIndex)
	SHADER_PARAMETER(uint32,bIsGPUSkinPassThrough)
	SHADER_PARAMETER_SRV(Buffer<float2>, VertexFetch_TexCoordBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, VertexFetch_PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, VertexFetch_PreSkinPositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_PackedTangentsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexFetch_ColorComponentsBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

ENGINE_API FLocalVertexFactoryInstanceDitherParameters MakeNullLocalVertexFactoryInstanceDitherParameters();

extern ENGINE_API void GetLocalVFUniformShaderParameters(
	FLocalVertexFactoryUniformShaderParameters& UniformParameters,
	const class FLocalVertexFactory* VertexFactory, 
	uint32 LODLightmapDataIndex, 
	class FColorVertexBuffer* OverrideColorVertexBuffer, 
	int32 BaseVertexIndex,
	int32 PreSkinBaseVertexIndex
	);

extern ENGINE_API TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters> CreateLocalVFUniformBuffer(
	const class FLocalVertexFactory* VertexFactory, 
	uint32 LODLightmapDataIndex, 
	class FColorVertexBuffer* OverrideColorVertexBuffer, 
	int32 BaseVertexIndex,
	int32 PreSkinBaseVertexIndex,
	const FLocalVertexFactoryInstanceDitherParameters* OptionalInstanceDitherParameters = nullptr,
	EUniformBufferUsage UniformBufferUsage = UniformBuffer_MultiFrame
	);

extern ENGINE_API void AddStaticMeshTextureCoordinateElements(
	int32 BaseTexCoordAttribute, 
	const TArrayView<FVertexStreamComponent>& TextureCoordinates, 
	FVertexDeclarationElementList& Elements,
	FVertexFactory::FVertexStreamList& InOutStreams);

/**
 */
ENGINE_API bool DoesPlatformUseInstancedStaticMeshVertexFactory(const FStaticShaderPlatform Platform);

/**
 * A vertex factory which simply transforms explicit vertex attributes from local to world space.
 */
class FLocalVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FLocalVertexFactory, ENGINE_API);
public:

	ENGINE_API FLocalVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, const char* InDebugName);

	ENGINE_API virtual ~FLocalVertexFactory();

	struct FDataType : public FStaticMeshDataType
	{
		FVertexStreamComponent PreSkinPositionComponent;
		FRHIShaderResourceView* PreSkinPositionComponentSRV = nullptr;
	#if WITH_EDITORONLY_DATA
		const class UStaticMesh* StaticMesh = nullptr;
		bool bIsCoarseProxy = false;
	#endif
	};

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static ENGINE_API void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);

	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, EVertexInputStreamType InputStreamType, bool bSupportsManualVertexFetch, FDataType& Data, FVertexDeclarationElementList& Elements);

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	ENGINE_API void SetData(FRHICommandListBase& RHICmdList, const FDataType& InData);

	UE_DEPRECATED(5.4, "SetData requires a command list.")
	ENGINE_API void SetData(const FDataType& InData);

	/**
	* Copy the data from another vertex factory
	* @param Other - factory to copy from
	*/
	ENGINE_API void Copy(const FLocalVertexFactory& Other);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override
	{
		UniformBuffer.SafeRelease();
		FVertexFactory::ReleaseRHI();
	}

	UE_DEPRECATED(5.5, "SetColorOverrideStream is no longer used")
	inline void SetColorOverrideStream(FRHICommandList& RHICmdList, const FVertexBuffer* ColorVertexBuffer) const
	{
		checkf(ColorVertexBuffer->IsInitialized(), TEXT("Color Vertex buffer was not initialized! Name %s"), *ColorVertexBuffer->GetFriendlyName());
		checkf(IsInitialized() && EnumHasAnyFlags(EVertexStreamUsage::Overridden, Data.ColorComponent.VertexStreamUsage) && ColorStreamIndex > 0, TEXT("Per-mesh colors with bad stream setup! Name %s"), *ColorVertexBuffer->GetFriendlyName());
		RHICmdList.SetStreamSource(ColorStreamIndex, ColorVertexBuffer->VertexBufferRHI, 0);
	}

	void GetColorOverrideStream(const FVertexBuffer* ColorVertexBuffer, FVertexInputStreamArray& VertexStreams) const
	{
		checkf(ColorVertexBuffer->IsInitialized(), TEXT("Color Vertex buffer was not initialized! Name %s"), *ColorVertexBuffer->GetFriendlyName());
		checkf(IsInitialized() && EnumHasAnyFlags(EVertexStreamUsage::Overridden, Data.ColorComponent.VertexStreamUsage) && ColorStreamIndex > 0, TEXT("Per-mesh colors with bad stream setup! Name %s"), *ColorVertexBuffer->GetFriendlyName());

		VertexStreams.Add(FVertexInputStream(ColorStreamIndex, 0, ColorVertexBuffer->VertexBufferRHI));
	}

	virtual FShaderResourceViewRHIRef GetTriangleSortingPositionSRV() const override
	{
		return GetPositionsSRV();
	}

	inline FRHIShaderResourceView* GetPositionsSRV() const
	{
		return Data.PositionComponentSRV;
	}

	inline FRHIShaderResourceView* GetPreSkinPositionSRV() const
	{
		return Data.PreSkinPositionComponentSRV ? Data.PreSkinPositionComponentSRV : GNullColorVertexBuffer.VertexBufferSRV.GetReference();
	}

	inline FRHIShaderResourceView* GetTangentsSRV() const
	{
		return Data.TangentsSRV;
	}

	inline FRHIShaderResourceView* GetTextureCoordinatesSRV() const
	{
		return Data.TextureCoordinatesSRV;
	}

	inline FRHIShaderResourceView* GetColorComponentsSRV() const
	{
		return Data.ColorComponentsSRV;
	}

	inline const uint32 GetColorIndexMask() const
	{
		return Data.ColorIndexMask;
	}

	inline const int GetLightMapCoordinateIndex() const
	{
		return Data.LightMapCoordinateIndex;
	}

	inline const int GetNumTexcoords() const
	{
		return Data.NumTexCoords;
	}

	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer.GetReference();
	}

	inline const bool IsGPUSkinPassThrough() const
	{
		return bGPUSkinPassThrough;
	}

#if WITH_EDITORONLY_DATA
	virtual bool IsCoarseProxyMesh() const override { return Data.bIsCoarseProxy; }

	inline const class UStaticMesh* GetStaticMesh() const { return Data.StaticMesh; }
#endif
		
protected:
	friend class FLocalVertexFactoryShaderParameters;
	friend class FSkeletalMeshSceneProxy;

	const FDataType& GetData() const { return Data; }
	
	static ENGINE_API void GetVertexElements(
		ERHIFeatureLevel::Type FeatureLevel, 
		EVertexInputStreamType InputStreamType, 
		bool bSupportsManualVertexFetch,
		FDataType& Data, 
		FVertexDeclarationElementList& Elements, 
		FVertexStreamList& InOutStreams, 
		int32& OutColorStreamIndex);

	FDataType Data;
	TUniformBufferRef<FLocalVertexFactoryUniformShaderParameters> UniformBuffer;
	
	int32 ColorStreamIndex;

	bool bGPUSkinPassThrough = false;

	struct FDebugName
	{
		FDebugName(const char* InDebugName)
#if !UE_BUILD_SHIPPING
			: DebugName(InDebugName)
#endif
		{}
	private:
#if !UE_BUILD_SHIPPING
		const char* DebugName;
#endif
	} DebugName;
};

/**
 * Shader parameters for all LocalVertexFactory derived classes.
 */
class FLocalVertexFactoryShaderParametersBase : public FVertexFactoryShaderParameters
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FLocalVertexFactoryShaderParametersBase, ENGINE_API, NonVirtual);
public:
	ENGINE_API void Bind(const FShaderParameterMap& ParameterMap);

	ENGINE_API void GetElementShaderBindingsBase(
		const class FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		FRHIUniformBuffer* VertexFactoryUniformBuffer,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
		) const;

	FLocalVertexFactoryShaderParametersBase()
		: bAnySpeedTreeParamIsBound(false)
	{
	}

	// SpeedTree LOD parameter
	LAYOUT_FIELD(FShaderParameter, LODParameter);

	// True if LODParameter is bound, which puts us on the slow path in GetElementShaderBindings
	LAYOUT_FIELD(bool, bAnySpeedTreeParamIsBound);
};

/** Shader parameter class used by FLocalVertexFactory only - no derived classes. */
class FLocalVertexFactoryShaderParameters : public FLocalVertexFactoryShaderParametersBase
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FLocalVertexFactoryShaderParameters, ENGINE_API, NonVirtual);
public:
	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const; 
};
