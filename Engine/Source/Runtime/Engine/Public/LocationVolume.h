// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Volume.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

#include "LocationVolume.generated.h"

class FLoaderAdapterActor;

/**
 * A volume representing a location in the world. Used for World Partition loading regions.
 */
UCLASS(BlueprintType, Blueprintable, HideCategories=(Mobility, HLOD, Collision, Navigation, BrushSettings, Tags, Cooking, Replication, Networking, Actor, Mobile, RayTracing, AssetUserData, LevelInstance, DataLayers), MinimalAPI)
class ALocationVolume : public AVolume, public IWorldPartitionActorLoaderInterface
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
#endif
	//~ End UObject Interface

	//~ Begin AActor Interface
	virtual bool IsEditorOnly() const override { return true; }
#if WITH_EDITOR
	ENGINE_API virtual void PostRegisterAllComponents();
	ENGINE_API virtual void UnregisterAllComponents(bool bForReregister) override;
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
	//~ End AActor Interface

	//~ Begin IWorldPartitionActorLoaderInterface interface
	ENGINE_API virtual ILoaderAdapter* GetLoaderAdapter() override;
	//~ End IWorldPartitionActorLoaderInterface interface
#endif

	/** Load this location volume */
	UFUNCTION(BlueprintCallable, CallInEditor, Category=WorldPartition)
	ENGINE_API void Load();

	/** Unload this location volume */
	UFUNCTION(BlueprintCallable, CallInEditor, Category=WorldPartition)
	ENGINE_API void Unload();

	/** Return if this location volume is loaded */
	UFUNCTION(BlueprintCallable, Category=WorldPartition)
	ENGINE_API bool IsLoaded() const;

	UPROPERTY(EditAnywhere, Category=LocationVolume)
	FColor DebugColor;

#if WITH_EDITORONLY_DATA
	/* To support per-user last loaded location volumes */
	uint8 bIsAutoLoad : 1;
#endif

#if WITH_EDITOR
private:
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
	FLoaderAdapterActor* WorldPartitionActorLoader;
#endif
};
