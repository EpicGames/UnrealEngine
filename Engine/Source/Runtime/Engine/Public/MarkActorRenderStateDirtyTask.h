// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "Components/ActorComponent.h"

class FMarkActorRenderStateDirtyTask
{
public:
	explicit FMarkActorRenderStateDirtyTask(UActorComponent* InActorComponent)
		: ActorComponent(InActorComponent)
	{
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (UActorComponent* AC = ActorComponent.Get())
		{
			AC->MarkRenderStateDirty();
		}
	}

public:

	TWeakObjectPtr<UActorComponent> ActorComponent;

	static ESubsequentsMode::Type	GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type				GetDesiredThread() { return ENamedThreads::GameThread; }
	inline TStatId				GetStatId() const { return TStatId(); }
};