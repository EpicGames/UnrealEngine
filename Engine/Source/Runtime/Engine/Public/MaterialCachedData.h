// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialLayersFunctions.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "SceneTypes.h"
#include "MaterialCachedData.generated.h"

class UTexture;
class UTextureCollection;
class UCurveLinearColor;
class UCurveLinearColorAtlas;
class UFont;
class UMaterial;
class UMaterialExpression;
class UMaterialExpressionExternalCodeBase;
class UMaterialParameterCollection;
class URuntimeVirtualTexture;
class ULandscapeGrassType;
class UMaterialCacheVirtualTextureTag;
class UMaterialFunctionInterface;
class UMaterialInterface;
struct FStaticParameterSet;
class IMaterialEnumerationProvider;

/** Stores information about a function that this material references, used to know when the material needs to be recompiled. */
USTRUCT()
struct FMaterialFunctionInfo
{
	GENERATED_USTRUCT_BODY()

	/** Id that the function had when this material was last compiled. */
	UPROPERTY()
	FGuid StateId;

	/** The function which this material has a dependency on. */
	UPROPERTY()
	TObjectPtr<UMaterialFunctionInterface> Function = nullptr;
};
inline bool operator==(const FMaterialFunctionInfo& Lhs, const FMaterialFunctionInfo& Rhs)
{
	return Lhs.Function == Rhs.Function;
}
inline bool operator!=(const FMaterialFunctionInfo& Lhs, const FMaterialFunctionInfo& Rhs)
{
	return !operator==(Lhs, Rhs);
}

/** Stores information about a parameter collection that this material references, used to know when the material needs to be recompiled. */
USTRUCT()
struct FMaterialParameterCollectionInfo
{
	GENERATED_USTRUCT_BODY()

	/** Id that the collection had when this material was last compiled. */
	UPROPERTY()
	FGuid StateId;

	/** The collection which this material has a dependency on. */
	UPROPERTY()
	TObjectPtr<UMaterialParameterCollection> ParameterCollection = nullptr;

	bool operator==(const FMaterialParameterCollectionInfo& Other) const
	{
		return StateId == Other.StateId && ParameterCollection == Other.ParameterCollection;
	}
};

USTRUCT()
struct FMaterialCachedParameterEditorInfo
{
	GENERATED_USTRUCT_BODY()

	FMaterialCachedParameterEditorInfo() = default;
	FMaterialCachedParameterEditorInfo(const FGuid& InGuid) : ExpressionGuid(InGuid) {}
	FMaterialCachedParameterEditorInfo(const FGuid& InGuid, const FString& InDescription, const FName& InGroup, int32 InSortPriority)
		: Description(InDescription)
		, Group(InGroup)
		, SortPriority(InSortPriority)
		, ExpressionGuid(InGuid)
	{}

	UPROPERTY()
	FString Description;

	UPROPERTY()
	FName Group;

	UPROPERTY()
	int32 SortPriority = 0;

	UPROPERTY()
	FGuid ExpressionGuid;
};

USTRUCT()
struct FMaterialCachedParameterEntry
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API static const FMaterialCachedParameterEntry EmptyData;

	// This is used to map FMaterialParameterInfos to indices, which are then used to index various TArrays containing values for each type of parameter
	// (ExpressionGuids and Overrides, along with ScalarValues, VectorValues, etc)
	UPROPERTY()
	TSet<FMaterialParameterInfo> ParameterInfoSet;
};

USTRUCT()
struct FMaterialCachedParameterEditorEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FMaterialCachedParameterEditorInfo> EditorInfo;
};

struct FMaterialCachedExpressionContext
{
	const UMaterialFunctionInterface* CurrentFunction = nullptr;
	const FMaterialLayersFunctions* LayerOverrides = nullptr;
	bool bUpdateFunctionExpressions = true;
};

USTRUCT()
struct FParameterAssetPathsEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TMap<FString, int32> AssetPaths;

	ENGINE_API FString ToString() const;
};

USTRUCT()
struct FMaterialCachedExpressionEditorOnlyData
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API static const FMaterialCachedExpressionEditorOnlyData EmptyData;

	UPROPERTY()
	FMaterialCachedParameterEntry EditorOnlyEntries[NumMaterialEditorOnlyParameterTypes];

	UPROPERTY()
	FMaterialCachedParameterEditorEntry EditorEntries[NumMaterialParameterTypes];

	UPROPERTY()
	TArray<bool> StaticSwitchValues_DEPRECATED;

	UPROPERTY()
	TArray<FStaticComponentMaskValue> StaticComponentMaskValues;

	UPROPERTY()
	TArray<FVector2D> ScalarMinMaxValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UObject>> ScalarEnumerationValues;
	
	UPROPERTY()
	TArray<int32> ScalarEnumerationIndexValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UCurveLinearColor>> ScalarCurveValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UCurveLinearColorAtlas>> ScalarCurveAtlasValues;

	UPROPERTY()
	TArray<FParameterChannelNames> VectorChannelNameValues;

	UPROPERTY()
	TArray<bool> VectorUsedAsChannelMaskValues;

	UPROPERTY()
	TArray<FParameterChannelNames> TextureChannelNameValues;

	UPROPERTY()
	FMaterialLayersFunctionsEditorOnlyData MaterialLayers;

	UPROPERTY()
	TMap<FName, FParameterAssetPathsEntry> AssetPathsMap;

	UPROPERTY()
	TArray<FName> LandscapeLayerNames;

	UPROPERTY()
	TSet<FString> ExpressionIncludeFilePaths;

	UPROPERTY()
	TSet<FName> UserSceneTextureInputs;

	UPROPERTY()
	TArray<FName> ShaderTags;

	/** All the default textures present in the nodes of the material this represents. */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> EditorReferencedDefaultTextures;
};

USTRUCT()
struct FMaterialCachedExpressionData
{
	GENERATED_USTRUCT_BODY()
	
	ENGINE_API static const FMaterialCachedExpressionData EmptyData;

	ENGINE_API FMaterialCachedExpressionData();

#if WITH_EDITOR
	void UpdateForExpressions(const FMaterialCachedExpressionContext& Context, TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, EMaterialParameterAssociation Association, int32 ParameterIndex);
	void UpdatePropertyConnections(UMaterial& Material);
	void UpdateForFunction(const FMaterialCachedExpressionContext& Context, UMaterialFunctionInterface* Function, EMaterialParameterAssociation Association, int32 ParameterIndex);
	void UpdateForLayerFunctions(const FMaterialCachedExpressionContext& Context, const FMaterialLayersFunctions& LayerFunctions);
	void AnalyzeMaterial(UMaterial& Material, const FMaterialLayersFunctions* LayersOverrides = nullptr);

	void Validate(const UMaterialInterface& Material);

	/** Adds a parameter. If this returns false, a parameter with identical name has already been added but it was set to a different value. */
	void AddParameter(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& ParameterMeta, const FString& InAssetPath ,UObject*& OutReferencedTexture, UTextureCollection*& OutReferencedTextureCollection, UMaterialExpression* InExpression);

	void AddParameter(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& ParameterMeta, const FString& InAssetPath, UObject*& OutReferencedTexture, UMaterialExpression* InExpression)
	{
		UTextureCollection* UnusedReferencedTextureCollection = nullptr;
		return AddParameter(ParameterInfo, ParameterMeta, InAssetPath, OutReferencedTexture, UnusedReferencedTextureCollection, InExpression);
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	inline FMaterialCachedParameterEntry& GetParameterTypeEntry(EMaterialParameterType Type)
	{
		const int32 TypeIndex = (int32)Type;
		if (TypeIndex < NumMaterialRuntimeParameterTypes)
		{
			return RuntimeEntries[TypeIndex];
		}
		check(EditorOnlyData);
		return EditorOnlyData->EditorOnlyEntries[TypeIndex - NumMaterialRuntimeParameterTypes];
	}
	inline FMaterialCachedParameterEditorEntry& GetParameterEditorTypeEntry(EMaterialParameterType Type)
	{
		check(EditorOnlyData);
		return EditorOnlyData->EditorEntries[(int32)Type];
	}

	inline const FMaterialCachedParameterEntry& GetParameterTypeEntry(EMaterialParameterType Type) const
	{
		return const_cast<FMaterialCachedExpressionData*>(this)->GetParameterTypeEntry(Type);
	}
	inline const FMaterialCachedParameterEditorEntry& GetParameterEditorTypeEntry(EMaterialParameterType Type) const
	{
		check(EditorOnlyData);
		return EditorOnlyData->EditorEntries[(int32)Type];
	}
#else
	inline const FMaterialCachedParameterEntry& GetParameterTypeEntry(EMaterialParameterType Type) const { return Type >= EMaterialParameterType::NumRuntime ? FMaterialCachedParameterEntry::EmptyData : RuntimeEntries[static_cast<int32>(Type)]; }
#endif

	inline int32 GetNumParameters(EMaterialParameterType Type) const { return GetParameterTypeEntry(Type).ParameterInfoSet.Num(); }
	int32 FindParameterIndex(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& HashedParameterInfo) const;
	ENGINE_API bool GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutResult) const;
	void GetParameterValueByIndex(EMaterialParameterType Type, int32 ParameterIndex, FMaterialParameterMetadata& OutResult) const;
	const FGuid& GetExpressionGuid(EMaterialParameterType Type, int32 Index) const;
	void GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const;
	void GetAllParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	void GetAllGlobalParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const;
	void GetAllGlobalParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;

#if WITH_EDITORONLY_DATA
	/** @return The index at which the given texture sits in the default textures array or INDEX_NONE if not present. */
	int32 GetDefaultTextureIdx(UObject* InTexture) const;
#endif

	ENGINE_API void AddReferencedObjects(FReferenceCollector& Collector);

	/** Returns an array of the guids of functions used, with the call hierarchy flattened. */
	void AppendReferencedFunctionIdsTo(TArray<FGuid>& OutIds) const;

	/** Returns an array of the guids of parameter collections used. */
	void AppendReferencedParameterCollectionIdsTo(TArray<FGuid>& OutIds) const;

	/** Returns a hash over all external code references from this cache. */
	uint64 GetExternalCodeReferencesHash() const;

	bool IsPropertyConnected(EMaterialProperty Property) const
	{
		return ((PropertyConnectedMask >> (uint64)Property) & 0x1) != 0;
	}

	void SetPropertyConnected(EMaterialProperty Property)
	{
		PropertyConnectedMask |= (1ull << (uint64)Property);
	}

	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);

private:
#if WITH_EDITOR
	void DoPostAnalyzeChecks(const UMaterial& BaseMat);
#endif
	bool CheckDuplicateParameterValuesEqual(const int32 InExistingParamIdx, const FMaterialParameterMetadata& DuplicateParameterMeta);
	FString GetParameterValueAsText(const FMaterialParameterMetadata& ParameterMeta);
	void InsertParameterValue(const FMaterialParameterMetadata& ParameterMeta, const int32 Index, UObject*& OutReferencedTexture, UTextureCollection*& OutReferencedTextureCollection);

public:

#if WITH_EDITORONLY_DATA
	TSharedPtr<FMaterialCachedExpressionEditorOnlyData> EditorOnlyData;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	FMaterialCachedParameterEntry RuntimeEntries[NumMaterialRuntimeParameterTypes];

	UPROPERTY()
	TArray<int32> ScalarPrimitiveDataIndexValues;

	UPROPERTY()
	TArray<int32> VectorPrimitiveDataIndexValues;

	UPROPERTY()
	TArray<float> ScalarValues;

	UPROPERTY()
	TArray<bool> StaticSwitchValues;

	UPROPERTY()
	TArray<bool> DynamicSwitchValues;
	
	UPROPERTY()
	TArray<FLinearColor> VectorValues;

	UPROPERTY()
	TArray<FVector4d> DoubleVectorValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UTexture>> TextureValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UTextureCollection>> TextureCollectionValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UMaterialParameterCollection>> ParameterCollectionValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<UFont>> FontValues;

	UPROPERTY()
	TArray<int32> FontPageValues;
	
	UPROPERTY()
	TArray<TSoftObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextureValues;

	UPROPERTY()
	TArray<TSoftObjectPtr<USparseVolumeTexture>> SparseVolumeTextureValues;

	UPROPERTY()
	TArray<TObjectPtr<UTextureCollection>> ReferencedTextureCollections;

	/** Array of all material expression classes that contain external code declarations. */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> ReferencedExternalCodeExpressionClasses;

	/** Array of all functions this material depends on. */
	UPROPERTY()
	TArray<FMaterialFunctionInfo> FunctionInfos;

	/** CRC of the FunctionInfos StateIds. */
	UPROPERTY()
	uint32 FunctionInfosStateCRC;

	/** Array of all parameter collections this material depends on. */
	UPROPERTY()
	TArray<FMaterialParameterCollectionInfo> ParameterCollectionInfos;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<ULandscapeGrassType>> GrassTypes_DEPRECATED;
#endif // WITH_EDITORONLY_DATA	

	UPROPERTY()
	TMap<FName, TObjectPtr<ULandscapeGrassType>> NamedGrassTypes;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialCacheVirtualTextureTag>> MaterialCacheTags;

	UPROPERTY()
	FMaterialLayersFunctionsRuntimeData MaterialLayers;

	UPROPERTY()
	TArray<FName> DynamicParameterNames;

	UPROPERTY()
	TArray<bool> QualityLevelsUsed;

	UPROPERTY()
	uint64 MaterialCacheUVCoordinatesUsedMask = 0x0;

	UPROPERTY()
	uint32 bHasMaterialLayers : 1;

	UPROPERTY()
	uint32 bHasRuntimeVirtualTextureOutput : 1;

	UPROPERTY()
	uint32 bHasFirstPersonOutput : 1;

	UPROPERTY()
	uint32 bUsesTemporalResponsiveness : 1;

	UPROPERTY()
	uint32 bUsesMotionVectorWorldOffset : 1;

	UPROPERTY()
	uint32 bSamplesMaterialCache : 1;

	UPROPERTY()
	uint32 bHasMaterialCacheOutput : 1;

	UPROPERTY()
	uint32 bMaterialCacheHasNonUVDerivedExpression : 1;

	UPROPERTY()
	uint32 bHasSceneColor : 1;

	UPROPERTY()
	uint32 bHasPerInstanceCustomData : 1;

	UPROPERTY()
	uint32 bHasPerInstanceRandom : 1;

	UPROPERTY()
	uint32 bHasVertexInterpolator : 1;

	UPROPERTY()
	uint32 bHasCustomizedUVs : 1;

	UPROPERTY()
	uint32 bHasMeshPaintTexture : 1;

	UPROPERTY()
	uint32 bHasWorldPosition : 1;

	UPROPERTY()
	uint32 PropertyConnectedBitmask_DEPRECATED = 0;

	/** Each bit corresponds to EMaterialProperty connection status. */
	UPROPERTY()
	uint64 PropertyConnectedMask = 0;

#if WITH_EDITOR
	struct FParamExpressionsValues
	{
		TArray<TPair<TObjectPtr<class UMaterialExpression>, FString>> ExpressionsValues;
		bool bHasDuplicatesWithDifferentValues = false;
	};
	/** Stores Parameters to expresions and string values. Can be used for error logging duplicates.*/
	TMap<FName, FParamExpressionsValues> ParameterExpressionsAndValues;

	struct FParsedExpressionData
	{
		UMaterialExpression* Expression = nullptr;

		// These are optional in case the expression is a Parameter
		int32 ParameterIdx = INDEX_NONE;
		FName ParameterName;
		
		// This could theoretically be gathered from Expression 
		// but being in a Material Function can override values
		FMaterialParameterMetadata ParameterMeta;
	};

	// Used for any post analyze checks
	// Should be cleared after AnalyzeMaterial
	TArray<FParsedExpressionData> ParsedExpressions;
#endif
};

template<>
struct TStructOpsTypeTraits<FMaterialCachedExpressionData> : public TStructOpsTypeTraitsBase2<FMaterialCachedExpressionData>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};
