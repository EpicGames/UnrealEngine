// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialCompiler.h: Material compiler interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "MaterialShared.h"
#include "Materials/MaterialExpressionCollectionTransform.h"
#include "Materials/MaterialExpressionSpeedTree.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialExpressionLocalPosition.h"
#include "Materials/MaterialExpressionWorldPosition.h"
#include "Materials/MaterialExpressionTextureProperty.h"
#include "Materials/MaterialExpressionViewProperty.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"

// This interface is needed so that the SubstrateTranslatorData can be passed around in material expression
struct FSubstrateTranslatorDataInterface
{
	virtual void Errorf(const FString& Format) = 0;

	virtual FMaterialShadingModelField GetMaterialShadingModels() = 0;


	virtual FGuid SubstrateTreeStackGetPathUniqueId() = 0;
	virtual FGuid SubstrateTreeStackGetParentPathUniqueId() = 0;
	virtual bool GetSubstrateTreeOutOfStackDepthOccurred() = 0;

	virtual int32 SubstrateThicknessStackGetThicknessIndex() = 0;
	virtual int32 SubstrateThicknessStackPush(class UMaterialExpression* Expression, FScalarMaterialInput* Input) = 0;
	virtual int32 SubstrateThicknessStackPush(class UMaterialExpression* Expression, FExpressionInput* Input) = 0;
	virtual void SubstrateThicknessStackPop() = 0;
	struct FSubstrateThicknessExpression
	{
		FExpressionInput* ExpressionInput = nullptr;
		FScalarMaterialInput* MaterialInput = nullptr;
	};
	virtual FSubstrateThicknessExpression* SubstrateThicknessStackGetExpression(int32 Index) = 0;

	virtual FGuid SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex) = 0;
	virtual void SubstrateTreeStackPop() = 0;

	virtual FSubstrateOperator& SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, class UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending = false) = 0;
	virtual FSubstrateOperator& SubstrateGetDefaultBSDFOperator(FGuid MaterialExpressionGuid, class UMaterialExpression* Parent) = 0;
	virtual FSubstrateOperator& SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid) = 0;
	virtual FSubstrateOperator* SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex) = 0;

	virtual class FMaterialCompiler* GetCompilerLegacy() = 0;
	virtual class MIR::FEmitter* GetCompilerNew() = 0;

	virtual void PushFunction(class FMaterialFunctionCompileState* FunctionState) = 0;
	virtual void PopFunction() = 0;
};

namespace UE
{
namespace Shader
{
struct FValue;
enum class EValueType : uint8;
}
}

class Error;
class UMaterialParameterCollection;
class URuntimeVirtualTexture;
class UTexture;
class UTextureCollection;
struct FMaterialParameterInfo;
class USparseVolumeTexture;
struct FMaterialExternalCodeDeclaration;
struct FMaterialCacheTagLayout;

enum EMaterialForceCastFlags
{
	MFCF_ForceCast		= 1<<0,	// Used by caller functions as a helper
	MFCF_ExactMatch		= 1<<2, // If flag set skips the cast on an exact match, else skips on a compatible match
	MFCF_ReplicateValue	= 1<<3	// Replicates a Float1 value when up-casting, else appends zero
};

/** What type of compiler is this? Used by material expressions that select input based on compile context */
enum class EMaterialCompilerType
{
	Standard, /** Standard HLSL translator */
	Lightmass, /** Lightmass proxy compiler */
	MaterialProxy, /** Flat material proxy compiler */
};


/** Whether we need some data export from a Substrate material from spatially varying properties, e.g. diffuse color for Lighmass to generate lightmaps. */
enum ESubstrateMaterialExport : uint8
{
	SME_None					= 0,
	SME_BaseColorPostCoverage	= 1,
	SME_BaseColor				= 2,
	SME_Normal					= 3,
	SME_Emissive				= 4,
	SME_Specular				= 5,
	SME_Roughness				= 6,
	SME_Anisotropy				= 7,
	SME_Metallic				= 8,
	SME_Transmittance			= 9,
	SME_Opacity					= 10,
	SME_OpacityMask				= 11,
	SME_CustomData0				= 12,
	SME_CustomData1				= 13,
	SME_SubsurfaceColor			= 14,
	SME_Tangent					= 15,
	SME_ShadingModel			= 16,
	SME_MaterialPreview			= 17
};

/** Exported materials are all opaque unlit. This is used to give some context to the export logic.*/
enum ESubstrateMaterialExportContext : uint8
{
	SMEC_Opaque = 0,
	SMEC_Translucent = 1
};

enum class EMaterialCustomOutputFlags : uint8
{
	None = 0u,
	AllowAttributeConnection = (1u << 0)
};
ENUM_CLASS_FLAGS(EMaterialCustomOutputFlags);

/** Additional parameters used in TransformPosition / TransformVector. */
struct FTransformParameters
{
	/** Code index for the scale of the tiles used in Periodic World Space. */
	int32 PeriodicWorldTileSizeIndex = INDEX_NONE;

	/** Code index for the interpolation alpha value used in First Person Space. */
	int32 FirstPersonInterpolationAlphaIndex = INDEX_NONE;
};

/** 
 * The interface used to translate material expressions into executable code. 
 * Note: Most member functions should be pure virtual to force a FProxyMaterialCompiler override!
 */
class FMaterialCompiler
{
public:
	virtual ~FMaterialCompiler() = default;

	/** Whether material translation should abort */
	virtual bool ShouldStopTranslating() const = 0;

	// sets internal state CurrentShaderFrequency 
	// @param OverrideShaderFrequency SF_NumFrequencies to not override
	virtual void SetMaterialProperty(EMaterialProperty InProperty, EShaderFrequency OverrideShaderFrequency = SF_NumFrequencies, bool bUsePreviousFrameTime = false) = 0;
	
	/** Pushes a material attributes property onto the stack. Called as we begin compiling a property through a MaterialAttributes pin. */
	virtual void PushMaterialAttribute(const FGuid& InAttributeID) = 0;
	/** Pops a MaterialAttributes property off the stack. Called as we finish compiling a property through a MaterialAttributes pin. */
	virtual FGuid PopMaterialAttribute() = 0;
	/** Gets the current top of the MaterialAttributes property stack. */
	virtual const FGuid GetMaterialAttribute() = 0;
	/** Sets the bottom MaterialAttributes property of the stack. */
	virtual void SetBaseMaterialAttribute(const FGuid& InAttributeID) = 0;

	virtual class UMaterialExpressionCustomOutput* GetTopCustomOutput() = 0;

	/** Pushes a parameter owner onto the stack. Called as we begin compiling each layer function of MaterialAttributeLayers. */
	virtual void PushParameterOwner(const FMaterialParameterInfo& InOwnerInfo) = 0;
	/** Pops a parameter owner off the stack. Called as we finish compiling each layer function of MaterialAttributeLayers. */
	virtual FMaterialParameterInfo PopParameterOwner() = 0;

	// gets value stored by SetMaterialProperty()
	virtual EShaderFrequency GetCurrentShaderFrequency() const = 0;
	//
	virtual int32 Error(const TCHAR* Text) = 0;
	ENGINE_API int32 Errorf(const TCHAR* Format,...);
	virtual void AppendExpressionError(UMaterialExpression* Expression, const TCHAR* Text) = 0;

	virtual int32 CallExpression(FMaterialExpressionKey ExpressionKey,FMaterialCompiler* InCompiler) = 0;

	virtual int32 CallExpressionExec(UMaterialExpression* Expression) = 0;

	virtual EMaterialCompilerType GetCompilerType() const { return EMaterialCompilerType::Standard; }
	inline bool IsMaterialProxyCompiler() const { return GetCompilerType() == EMaterialCompilerType::MaterialProxy; }
	inline bool IsLightmassCompiler() const { return GetCompilerType() == EMaterialCompilerType::Lightmass; }

	inline void SetSubstrateMaterialExportType(ESubstrateMaterialExport InSubstrateMaterialExport, ESubstrateMaterialExportContext InSubstrateMaterialExportContext, uint8 InSubstrateMaterialExportLegacyBlendMode)
	{
		SubstrateMaterialExport = InSubstrateMaterialExport; 
		SubstrateMaterialExportContext = InSubstrateMaterialExportContext;
		SubstrateMaterialExportLegacyBlendMode = InSubstrateMaterialExportLegacyBlendMode;
	}
	inline ESubstrateMaterialExport GetSubstrateMaterialExportType() const { return SubstrateMaterialExport; }
	inline ESubstrateMaterialExportContext GetSubstrateMaterialExportContext() const { return SubstrateMaterialExportContext; }
	inline uint8 GetSubstrateMaterialExportLegacyBlendMode() const { return SubstrateMaterialExportLegacyBlendMode; }

	inline bool IsVertexInterpolatorBypass() const
	{
		const EMaterialCompilerType Type = GetCompilerType();
		return Type == EMaterialCompilerType::Lightmass;
	}

	virtual EMaterialValueType GetType(int32 Code) = 0;

	virtual EMaterialQualityLevel::Type GetQualityLevel() = 0;

	virtual ERHIFeatureLevel::Type GetFeatureLevel() = 0;

	virtual EShaderPlatform GetShaderPlatform() = 0;

	virtual const ITargetPlatform* GetTargetPlatform() const = 0;

	virtual bool IsTangentSpaceNormal() const = 0;

	virtual FMaterialShadingModelField GetMaterialShadingModels() const = 0;
	
	/** Get the shading models that were encountered when compiling a material's Shading Model attribute graph.
	 *	This will represent the actually used Shading Models for a particular material given its static switches, feature levels and material quality for example.  
	 *	Will return all Shading Models in the material if this is called before that attribute has been compiled. 
	 */
	virtual FMaterialShadingModelField GetCompiledShadingModels() const = 0;

	virtual EMaterialValueType GetParameterType(int32 Index) const = 0;

	virtual FMaterialUniformExpression* GetParameterUniformExpression(int32 Index) const = 0;

	virtual bool GetTextureForExpression(int32 Index, int32& OutTextureIndex, EMaterialSamplerType& OutSamplerType, TOptional<FName>& OutParameterName) const = 0;
	virtual bool GetTextureCollectionForExpression(int32 Index, int32& OutTextureCollectionIndex, TOptional<FName>& OutParameterName) const = 0;

	virtual bool IsMaterialPropertyUsed(EMaterialProperty Property, int32 CodeChunkIdx) const = 0;
	
	/** 
	 * Casts the passed in code to DestType, or generates a compile error if the cast is not valid. 
	 * This will truncate a type (float4 -> float3) but not add components (float2 -> float3), however a float1 can be cast to any float type by replication. 
	 */
	virtual int32 ValidCast(int32 Code,EMaterialValueType DestType) = 0;
	virtual int32 ForceCast(int32 Code,EMaterialValueType DestType,uint32 ForceCastFlags = 0) = 0;

	/** Cast shading model integer to float value */
	virtual int32 CastShadingModelToFloat(int32 Code) = 0;

	virtual int32 TruncateLWC(int32 Code) = 0;

	/** Pushes a function onto the compiler's function stack, which indicates that compilation is entering a function. */
	virtual void PushFunction(FMaterialFunctionCompileState* FunctionState) = 0;

	/** Pops a function from the compiler's function stack, which indicates that compilation is leaving a function. */
	virtual FMaterialFunctionCompileState* PopFunction() = 0;

	virtual int32 GetCurrentFunctionStackDepth() = 0;

	virtual int32 AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex) = 0;	
	virtual int32 CollectionTransform(int32 InputIndex, const TStaticArray<int32, 5>& CollectionParameters, EParameterCollectionTransformType TransformType) = 0;

	virtual int32 NumericParameter(EMaterialParameterType ParameterType, FName ParameterName, const UE::Shader::FValue& DefaultValue) = 0;

	ENGINE_API int32 ScalarParameter(FName ParameterName, float DefaultValue);
	ENGINE_API int32 VectorParameter(FName ParameterName, const FLinearColor& DefaultValue);

	virtual int32 Constant(float X) = 0;
	virtual int32 Constant2(float X,float Y) = 0;
	virtual int32 Constant3(float X,float Y,float Z) = 0;
	virtual int32 Constant4(float X,float Y,float Z,float W) = 0;
	virtual int32 GenericConstant(const UE::Shader::FValue& Value) = 0;

	virtual	int32 ViewProperty(EMaterialExposedViewProperty Property, bool InvProperty = false) = 0;
	virtual int32 IsOrthographic() = 0;

	virtual int32 GameTime(bool bPeriodic, float Period) = 0;
	virtual int32 RealTime(bool bPeriodic, float Period) = 0;
	virtual int32 DeltaTime() = 0;
	virtual int32 PeriodicHint(int32 PeriodicCode) { return PeriodicCode; }

	virtual int32 Sine(int32 X) = 0;
	virtual int32 Cosine(int32 X) = 0;
	virtual int32 Tangent(int32 X) = 0;
	virtual int32 Arcsine(int32 X) = 0;
	virtual int32 ArcsineFast(int32 X) = 0;
	virtual int32 Arccosine(int32 X) = 0;
	virtual int32 ArccosineFast(int32 X) = 0;
	virtual int32 Arctangent(int32 X) = 0;
	virtual int32 ArctangentFast(int32 X) = 0;
	virtual int32 Arctangent2(int32 Y, int32 X) = 0;
	virtual int32 Arctangent2Fast(int32 Y, int32 X) = 0;

	virtual int32 Floor(int32 X) = 0;
	virtual int32 Ceil(int32 X) = 0;
	virtual int32 Round(int32 X) = 0;
	virtual int32 Truncate(int32 X) = 0;
	virtual int32 Sign(int32 X) = 0;
	virtual int32 Frac(int32 X) = 0;
	virtual int32 Fmod(int32 A, int32 B) = 0;
	virtual int32 Modulo(int32 A, int32 B) = 0;
	virtual int32 Abs(int32 X) = 0;

	virtual int32 ReflectionVector() = 0;
	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) = 0;
	virtual int32 CameraVector() = 0;
	virtual int32 LightVector() = 0;

	virtual int32 GetViewportUV() = 0;
	virtual int32 GetPixelPosition() = 0;
	virtual int32 LocalPosition(EPositionIncludedOffsets IncludedOffsets, ELocalPositionOrigin OriginType) = 0;
	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) = 0;
	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) = 0;
	virtual int32 ObjectRadius() = 0;
	virtual int32 ObjectBounds() = 0;
	virtual int32 ObjectLocalBounds(int32 OutputIndex) = 0;
	virtual int32 InstanceLocalBounds(int32 OutputIndex) = 0;
	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) = 0;
	virtual int32 DistanceCullFade() = 0;
	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) = 0;
	virtual int32 ParticleMacroUV() = 0;
	virtual int32 ParticleSubUV(int32 TextureIndex, EMaterialSamplerType SamplerType, int32 MipValue0Index, int32 MipValue1Index, ETextureMipValueMode MipValueMode, bool bBlend) = 0;
	virtual int32 ParticleSubUVProperty(int32 PropertyIndex) = 0;
	virtual int32 ParticleColor() = 0;
	virtual int32 ParticlePosition(EPositionOrigin OriginType) = 0;
	virtual int32 ParticleRadius() = 0;
	virtual int32 SphericalParticleOpacity(int32 Density) = 0;
	virtual int32 ParticleRelativeTime() = 0;
	virtual int32 ParticleMotionBlurFade() = 0;
	virtual int32 ParticleRandom() = 0;
	virtual int32 ParticleDirection() = 0;
	virtual int32 ParticleSpeed() = 0;
	virtual int32 ParticleSize() = 0;
	virtual int32 ParticleSpriteRotation() = 0;

	virtual int32 DynamicBranch(int32 Condition, int32 A, int32 B) = 0;
	virtual int32 If(int32 A,int32 B,int32 AGreaterThanB,int32 AEqualsB,int32 ALessThanB,int32 Threshold) = 0;
	virtual int32 Switch(int32 SwitchValueInput, int32 DefaultInput, TArray<int32>& CompiledInputs) = 0;

	virtual int32 TextureCoordinate(uint32 CoordinateIndex, bool UnMirrorU, bool UnMirrorV) = 0;
	virtual void  SetPotentiallyManipulateTexCoords() = 0;
	virtual int32 TextureSample(int32 Texture,int32 Coordinate,enum EMaterialSamplerType SamplerType,int32 MipValue0Index=INDEX_NONE,int32 MipValue1Index=INDEX_NONE,ETextureMipValueMode MipValueMode=TMVM_None,ESamplerSourceMode SamplerSource=SSM_FromTextureAsset,ETextureGatherMode GatherMode=TGM_None,int32 TextureReferenceIndex=INDEX_NONE, bool AutomaticViewMipBias=false, bool AdaptiveVirtualTexture=false, bool EnableFeedback = true) = 0;
	virtual int32 TextureProperty(int32 InTexture, EMaterialExposedTextureProperty Property) = 0;

	virtual int32 TextureDecalMipmapLevel(int32 TextureSizeInput) = 0;
	virtual int32 TextureDecalDerivative(bool bDDY) = 0;
	virtual int32 DecalColor() = 0;
	virtual int32 DecalLifetimeOpacity() = 0;

	virtual int32 Texture(UTexture* Texture,int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource=SSM_FromTextureAsset,ETextureMipValueMode MipValueMode=TMVM_None) = 0;
	virtual int32 TextureParameter(FName ParameterName,UTexture* DefaultTexture,int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource=SSM_FromTextureAsset) = 0;

	virtual int32 TextureCollection(UTextureCollection* TextureCollection, int32& TextureCollectionReferenceIndex) = 0;
	virtual int32 TextureCollectionParameter(FName ParameterName, UTextureCollection* DefaultValue, int32& TextureCollectionReferenceIndex) = 0;
	virtual int32 TextureCollectionCount(int32 InTextureCollectionCodeIndex) = 0;
	virtual int32 TextureFromCollection(int32 TextureCollectionCodeIndex, int32 IndexIntoCollectionCodeIndex, EMaterialValueType ResultTextureType) = 0;

	virtual int32 TextureStreamingInfo(int32 TextureReferenceIndex, int32 TextureIndex, int32 CoordinateIndex) = 0;

	virtual int32 MaterialCacheTextureDescriptor(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, uint32 LayerIndex) = 0;
	virtual int32 SampleMaterialCache(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, int32 TexCoordIndex) = 0;
	virtual int32 DefaultMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout) = 0;
	virtual int32 SetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex, int32 Value) = 0;
	virtual int32 GetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex) = 0;
	virtual int32 MaterialCacheOutput(UMaterialExpressionCustomOutput* CustomOutput, const FMaterialCacheTagLayout& Layout, int32 AttributeSet) = 0;
	
	virtual int32 VirtualTexture(URuntimeVirtualTexture* InTexture, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) = 0;
	virtual int32 VirtualTextureParameter(FName ParameterName, URuntimeVirtualTexture* DefaultValue, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) = 0;
	virtual int32 VirtualTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) = 0;
	virtual int32 VirtualTextureUniform(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) = 0;
	virtual int32 VirtualTextureWorldToUV(int32 WorldPositionIndex, int32 P0, int32 P1, int32 P2, EPositionOrigin PositionOrigin) = 0;
	virtual int32 VirtualTextureUnpack(int32 CodeIndex0, int32 CodeIndex1, int32 CodeIndex2, int32 P0, EVirtualTextureUnpackType UnpackType) = 0;
	virtual int32 VirtualTextureCustomData() = 0;

	virtual int32 ExternalTexture(const FGuid& ExternalTextureGuid) = 0;
	virtual int32 ExternalTexture(UTexture* InTexture, int32& TextureReferenceIndex) = 0;
	virtual int32 ExternalTextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex) = 0;
	virtual int32 ExternalTextureCoordinateScaleRotation(int32 TextureReferenceIndex, TOptional<FName> ParameterName) = 0;
	virtual int32 ExternalTextureCoordinateScaleRotation(const FGuid& ExternalTextureGuid) = 0;
	virtual int32 ExternalTextureCoordinateOffset(int32 TextureReferenceIndex, TOptional<FName> ParameterName) = 0;
	virtual int32 ExternalTextureCoordinateOffset(const FGuid& ExternalTextureGuid) = 0;

	// Could be called sub texture and used to support multiple texture samples from a single node?
	// Making it clear for now and setting explicitly a USparseVolumeTexture as object.

	/**
	 * Register a sparse volume texture to be sampled.
	 * @param Texture The sparse volume texture to sample.
	 * @param TextureReferenceIndex Output the index of the texture in the referenced textures of the material.
	 * @param SamplerType The sampler type
	 * @return The code chunk index of the texture.
	 */
	virtual int32 SparseVolumeTexture(USparseVolumeTexture* Texture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) = 0;
	/**
	 * Register a parameterized sparse volume texture to be sampled.
	 * @param ParameterName The sparse volume texture parameter name.
	 * @param InDefaultTexture The default static sparse volume texture to sample.
	 * @param TextureReferenceIndex Output the index of the texture in the referenced textures of the material.
	 * @param SamplerType The sampler type
	 * @return The code chunk index of the texture parameter.
	 */
	virtual int32 SparseVolumeTextureParameter(FName ParameterName, USparseVolumeTexture* InDefaultTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) = 0;
	/**
	 * Register a uniform parameter required to be able to sample a sparse volume texture.
	 * @param TextureIndex The TextureReferenceIndex.
	 * @param VectorIndex The index of the vector in the list of vector to bind.
	 * @param Type The type of the vector.
	 * @return The code chunk index of the uniform.
	 */
	virtual int32 SparseVolumeTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) = 0;
	/**
	 * Register a uniform parameter required to be able to sample a parameterizes sparse volume texture.
	 * @param ParameterName The sparse volume texture parameter name.
	 * @param TextureIndex The TextureReferenceIndex.
	 * @param VectorIndex The index of the vector in the list of vector to bind.
	 * @param Type The type of the vector.
	 * @return The code chunk index of the uniform.
	 */
	virtual int32 SparseVolumeTextureUniformParameter(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) = 0;
	/**
	 * Sample a sparse volume texture page table.
	 * @param SparseVolumeTextureIndex The code chunk index of the texture.
	 * @param UVWIndex The UVW coordinate at which to sample the texture.
	 * @param MipLevelIndex The mip level at which to sample the texture.
	 * @param SamplerSource The type of sampler to use to sample the texture.
	 * @param bIsManualLinearMipMapSecondSample is true when this is the second/higher sample for manual linear mip map interpolation.
	 * @return The code chunk index of the result of the texture sample.
	 */
	virtual int32 SparseVolumeTextureSamplePageTable(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipLevelIndex, ESamplerSourceMode SamplerSource, bool bIsManualLinearMipMapSecondSample) = 0;
	/**
	 * Sample a sparse volume texture physical tile data texture.
	 * @param SparseVolumeTextureIndex The code chunk index of the texture.
	 * @param VoxelCoordIndex The coordinate at which to sample the texture.
	 * @param PhysicalTileDataIdxIndex The code chunk of the index (0 or 1) of the physical tile data texture to sample.
	 * @param bIsManualLinearMipMapSecondSample is true when this is the second/higher sample for manual linear mip map interpolation.
	 * @return The code chunk index of the result of the texture sample.
	 */
	virtual int32 SparseVolumeTextureSamplePhysicalTileData(int32 SparseVolumeTextureIndex, int32 VoxelCoordIndex, int32 PhysicalTileDataIdxIndex, bool bIsManualLinearMipMapSecondSample) = 0;

	virtual int32 SparseVolumeTextureSample(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipValue0Index, int32 MipValue1Index, int32 PhysicalTileDataIdxIndex, ETextureMipValueMode MipValueMode, ESamplerSourceMode SamplerSource) = 0;

	virtual UObject* GetReferencedTexture(int32 Index)
	{
		return nullptr;
	}

	virtual UTextureCollection* GetReferencedTextureCollection(int32 Index)
	{
		return nullptr;
	}

	int32 Texture(UTexture* InTexture, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource=SSM_FromTextureAsset)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return Texture(InTexture, TextureReferenceIndex, SamplerType, SamplerSource);
	}

	int32 SparseVolumeTexture(USparseVolumeTexture* Texture, EMaterialSamplerType SamplerType)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return SparseVolumeTexture(Texture, TextureReferenceIndex, SamplerType);
	}

	int32 VirtualTexture(URuntimeVirtualTexture* InTexture, int32 TextureLayerIndex, int32 PageTableLayerIndex, EMaterialSamplerType SamplerType)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return VirtualTexture(InTexture, TextureLayerIndex, PageTableLayerIndex, TextureReferenceIndex, SamplerType);
	}

	int32 VirtualTextureParameter(FName ParameterName, URuntimeVirtualTexture* DefaultValue, int32 TextureLayerIndex, int32 PageTableLayerIndex, EMaterialSamplerType SamplerType)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return VirtualTextureParameter(ParameterName, DefaultValue, TextureLayerIndex, PageTableLayerIndex, TextureReferenceIndex, SamplerType);
	}

	int32 ExternalTexture(UTexture* DefaultTexture)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return ExternalTexture(DefaultTexture, TextureReferenceIndex);
	}

	int32 TextureParameter(FName ParameterName,UTexture* DefaultTexture, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource=SSM_FromTextureAsset)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return  TextureParameter(ParameterName, DefaultTexture, TextureReferenceIndex, SamplerType, SamplerSource);
	}

	int32 ExternalTextureParameter(FName ParameterName, UTexture* DefaultTexture)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		return ExternalTextureParameter(ParameterName, DefaultTexture, TextureReferenceIndex);
	}

	int32 TextureCollection(UTextureCollection* InTextureCollection)
	{
		int32 TextureCollectionReferenceIndex = INDEX_NONE;
		return TextureCollection(InTextureCollection, TextureCollectionReferenceIndex);
	}

	int32 TextureCollectionParameter(FName ParameterName, UTextureCollection* DefaultValue)
	{
		int32 TextureCollectionReferenceIndex = INDEX_NONE;
		return TextureCollectionParameter(ParameterName, DefaultValue, TextureCollectionReferenceIndex);
	}

	virtual	int32 PixelDepth()=0;
	virtual int32 SceneDepth(int32 Offset, int32 ViewportUV, bool bUseOffset) = 0;
	virtual int32 SceneColor(int32 Offset, int32 ViewportUV, bool bUseOffset) = 0;
	// @param SceneTextureId of type ESceneTextureId e.g. PPI_SubsurfaceColor
	virtual int32 SceneTextureLookup(int32 ViewportUV, uint32 SceneTextureId, bool bFiltered, bool bClamped, bool bUnused) = 0;
	virtual int32 GetSceneTextureViewSize(int32 SceneTextureId, bool InvProperty) = 0;
	virtual int32 FindOrAddUserSceneTexture(FName UserSceneTextureName) = 0;
	virtual int32 DBufferTextureLookup(int32 ViewportUV, uint32 DBufferTextureIndex) = 0;
	virtual int32 PathTracingBufferTextureLookup(int32 ViewportUV, uint32 PathTracingBufferTextureIndex) = 0;

	virtual int32 StaticBool(bool Value) = 0;
	virtual int32 StaticBoolParameter(FName ParameterName,bool bDefaultValue) = 0;
	virtual int32 DynamicBoolParameter(FName ParameterName,bool bDefaultValue) = 0;
	virtual int32 StaticComponentMask(int32 Vector,FName ParameterName,bool bDefaultR,bool bDefaultG,bool bDefaultB,bool bDefaultA) = 0;
	virtual const FMaterialLayersFunctions* GetMaterialLayers() = 0;

	virtual bool GetStaticBoolValue(int32 BoolIndex, bool& bSucceeded) = 0;
	virtual int32 StaticTerrainLayerWeight(FName ParameterName,int32 Default, bool bTextureArray = false) = 0;

	virtual int32 FontSignedDistanceData() = 0;
	// Feedback the instanced material layers data after the UMaterialExpressionMaterialAttributeLayers is assembled
	virtual void FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers) {} // TODO: SHould be pure virtual

	/** Adds code to the material that is defined by an external function definition. */
	virtual int32 ExternalCode(const FMaterialExternalCodeDeclaration& InExternalCode) = 0;

	virtual int32 VertexColor() = 0;
	virtual int32 MeshPaintTextureCoordinateIndex() = 0;
	virtual int32 MeshPaintTextureDescriptor() = 0;
	virtual int32 MeshPaintTextureReplace(int32 Invalid, int32 Valid) = 0;
	
	virtual int32 PreSkinnedPosition() = 0;
	virtual int32 PreSkinnedNormal() = 0;

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) = 0;

#if WITH_EDITOR
	virtual int32 MaterialBakingWorldPosition() = 0;
#endif

	virtual int32 Add(int32 A,int32 B) = 0;
	virtual int32 Sub(int32 A,int32 B) = 0;
	virtual int32 Mul(int32 A,int32 B) = 0;
	virtual int32 Div(int32 A,int32 B) = 0;
	virtual int32 Dot(int32 A,int32 B) = 0;
	virtual int32 Cross(int32 A,int32 B) = 0;
		
	virtual int32 Power(int32 Base,int32 Exponent) = 0;
	virtual int32 Exponential(int32 X) = 0;
	virtual int32 Exponential2(int32 X) = 0;
	virtual int32 Logarithm(int32 X) = 0;
	virtual int32 Logarithm2(int32 X) = 0;
	virtual int32 Logarithm10(int32 X) = 0;
	virtual int32 SquareRoot(int32 X) = 0;
	virtual int32 Length(int32 X) = 0;
	virtual int32 Normalize(int32 X) = 0;
	virtual int32 HsvToRgb(int32 X) = 0;
	virtual int32 RgbToHsv(int32 X) = 0;

	virtual int32 Lerp(int32 X,int32 Y,int32 A) = 0;
	virtual int32 Min(int32 A,int32 B) = 0;
	virtual int32 Max(int32 A,int32 B) = 0;
	virtual int32 Clamp(int32 X,int32 A,int32 B) = 0;
	virtual int32 Saturate(int32 X) = 0;

	virtual int32 SmoothStep(int32 X,int32 Y,int32 A) = 0;
	virtual int32 Step(int32 Y,int32 X) = 0;
	virtual int32 InvLerp(int32 X,int32 Y,int32 A) = 0;

	virtual int32 ComponentMask(int32 Vector,bool R,bool G,bool B,bool A) = 0;
	virtual int32 VectorComponent(int32 Vector, int32 Component) = 0;
	virtual int32 BitfieldExtract(int32 Value, int32 BitOffset, int32 BitCount) = 0;
	virtual int32 AppendVector(int32 A,int32 B) = 0;
	virtual int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) = 0;
	virtual int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) = 0;
	int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, int32 A)
	{
		FTransformParameters Parameters {};
		return TransformVector(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, int32 A)
	{
		FTransformParameters Parameters {};
		return TransformPosition(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	virtual int32 TransformNormalFromRequestedBasisToWorld(int32 NormalCodeChunk) = 0;

	virtual int32 DynamicParameter(FLinearColor& DefaultValue, uint32 ParameterIndex = 0) = 0;
	virtual int32 LightmapUVs() = 0;
	virtual int32 PrecomputedAOMask()  = 0;

	virtual int32 GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect) = 0;
	virtual int32 ShadowReplace(int32 Default, int32 Shadow) = 0;
	virtual int32 NaniteReplace(int32 Default, int32 Nanite) = 0;
	virtual int32 MaterialCache(int32 Default, int32 MaterialCache) = 0;
	virtual int32 RayTracingQualitySwitchReplace(int32 Normal, int32 RayTraced) = 0;
	virtual int32 PathTracingQualitySwitchReplace(int32 Normal, int32 PathTraced) = 0;
	virtual int32 PathTracingRayTypeSwitch(int32 Main, int32 Shadow, int32 IndirectDiffuse, int32 IndirectSpecular, int32 IndirectVolume) = 0;
	virtual int32 LightmassReplace(int32 Realtime, int32 Lightmass) = 0;
	virtual int32 ReflectionCapturePassSwitch(int32 Default, int32 Reflection) = 0;

	virtual int32 ObjectOrientation() = 0;
	virtual int32 RotateAboutAxis(int32 NormalizedRotationAxisAndAngleIndex, int32 PositionOnAxisIndex, int32 PositionIndex) = 0;
	virtual int32 TwoSidedSign() = 0;
	virtual int32 VertexNormal() = 0;
	virtual int32 VertexTangent() = 0;
	virtual int32 PixelNormalWS() = 0;
	virtual int32 IsFirstPerson() = 0;

	virtual int32 CustomExpression(class UMaterialExpressionCustom* Custom, int32 OutputIndex, TArray<int32>& CompiledInputs) = 0;
	virtual int32 CustomOutput(class UMaterialExpressionCustomOutput* Custom, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags = EMaterialCustomOutputFlags::None) = 0;
	
	virtual int32 PushRuntimeVirtualTextureOutput() = 0;
	virtual int32 PopRuntimeVirtualTextureOutput(uint8 AttributeMask) = 0;
	virtual bool IsInRuntimeVirtualTextureOutput() const = 0;

	virtual int32 DDX(int32 X) = 0;
	virtual int32 DDY(int32 X) = 0;
	
	virtual int32 PerInstanceRandom() = 0;
	virtual int32 PerInstanceFadeAmount() = 0;
	virtual int32 PerInstanceCustomData(int32 DataIndex, int32 DefaultValueIndex) = 0;
	virtual int32 PerInstanceCustomData3Vector(int32 DataIndex, int32 DefaultValueIndex) = 0;
	virtual int32 AntialiasedTextureMask(int32 Tex, int32 UV, float Threshold, uint8 Channel) = 0;

	virtual int32 Sobol(int32 Cell, int32 Index, int32 Seed) = 0;
	virtual int32 TemporalSobol(int32 Index, int32 Seed) = 0;
	virtual int32 Noise(int32 Position, EPositionOrigin PositionOrigin, float Scale, int32 Quality, uint8 NoiseFunction, bool bTurbulence, int32 Levels, float OutputMin, float OutputMax, float LevelScale, int32 FilterWidth, bool bTiling, uint32 RepeatSize) = 0;
	virtual int32 VectorNoise(int32 Position, EPositionOrigin PositionOrigin, int32 Quality, uint8 NoiseFunction, bool bTiling, uint32 RepeatSize) = 0;
	virtual int32 ScalarBlueNoise() = 0;

	virtual int32 BlackBody( int32 Temp ) = 0;
	virtual int32 DistanceToNearestSurface(int32 PositionArg, EPositionOrigin PositionOrigin) = 0;
	virtual int32 DistanceFieldGradient(int32 PositionArg, EPositionOrigin PositionOrigin) = 0;
	virtual int32 DistanceFieldApproxAO(int32 PositionArg, EPositionOrigin PositionOrigin, int32 NormalArg, int32 BaseDistanceArg, int32 RadiusArg, uint32 NumSteps, float StepScale) = 0;
	virtual int32 ProcessPhysicsField(int32 PositionArg, EPositionOrigin PositionOrigin, const int32 OutputType, const int32 TargetIndex, const bool bSampleField) = 0;
	virtual int32 DepthOfFieldFunction(int32 Depth, int32 FunctionValueIndex) = 0;
	virtual int32 PostVolumeUserFlagTestFunction(int32 Input) = 0;
	virtual int32 AtmosphericFogColor(int32 WorldPosition, EPositionOrigin PositionOrigin) = 0;
	virtual int32 RotateScaleOffsetTexCoords(int32 TexCoordCodeIndex, int32 RotationScale, int32 Offset) = 0;
	virtual int32 SpeedTree(int32 GeometryArg, int32 WindArg, int32 LODArg, float BillboardThreshold, bool bAccurateWindVelocities, bool bExtraBend, int32 ExtraBendArg) = 0;
	virtual int32 EyeAdaptation() = 0;
	virtual int32 EyeAdaptationInverse(int32 LightValueArg, int32 AlphaArg) = 0;
	virtual int32 AtmosphericLightVector() = 0;
	virtual int32 AtmosphericLightColor() = 0;

	virtual int32 SkyAtmosphereLightIlluminance(int32 WorldPosition, EPositionOrigin PositionOrigin, int32 LightIndex) = 0;
	virtual int32 SkyAtmosphereLightIlluminanceOnGround(int32 LightIndex) = 0;
	virtual int32 SkyAtmosphereLightDirection(int32 LightIndex) = 0;
	virtual int32 SkyAtmosphereLightDiskLuminance(int32 LightIndex, int32 OverrideAtmosphereLightDiscCosHalfApexAngle) = 0;
	virtual int32 SkyAtmosphereViewLuminance(int32 WorldDirectionOverrideCodeChunk) = 0;
	virtual int32 SkyAtmosphereAerialPerspective(int32 WorldPosition, EPositionOrigin PositionOrigin) = 0;
	virtual int32 SkyAtmosphereDistantLightScatteredLuminance() = 0;
	virtual int32 SkyLightEnvMapSample(int32 DirectionCodeChunk, int32 RoughnessCodeChunk) = 0;
	virtual int32 MainDirectionalLightIlluminance() = 0;
	virtual int32 MainDirectionalLightDirection() = 0;

	virtual int32 GetCloudSampleAltitude() = 0;
	virtual int32 GetCloudSampleAltitudeInLayer() = 0;
	virtual int32 GetCloudSampleNormAltitudeInLayer() = 0;
	virtual int32 GetCloudSampleShadowSampleDistance() = 0;
	virtual int32 GetVolumeSampleConservativeDensity() = 0;

	virtual int32 GetCloudEmptySpaceSkippingSphereCenterWorldPosition() = 0;
	virtual int32 GetCloudEmptySpaceSkippingSphereRadius() = 0;

	virtual int32 GetHairUV() = 0;
	virtual int32 GetHairDimensions() = 0;
	virtual int32 GetHairSeed() = 0;
	virtual int32 GetHairClumpID() = 0;
	virtual int32 GetHairTangent(bool bUseTangentSpace) = 0;
	virtual int32 GetHairRootUV() = 0;
	virtual int32 GetHairBaseColor() = 0;
	virtual int32 GetHairRoughness() = 0;
	virtual int32 GetHairAO() = 0;
	virtual int32 GetHairDepth() = 0;
	virtual int32 GetHairCoverage() = 0;
	virtual int32 GetHairAuxilaryData() = 0;
	virtual int32 GetHairAtlasUVs() = 0;
	virtual int32 GetHairGroupIndex() = 0;
	virtual int32 GetHairColorFromMelanin(int32 Melanin, int32 Redness, int32 DyeColor) = 0;
	virtual int32 CustomPrimitiveData(int32 OutputIndex, EMaterialValueType Type) = 0;
	virtual int32 ShadingModel(EMaterialShadingModel InSelectedShadingModel) = 0;

	// Material attributes
	virtual int32 DefaultMaterialAttributes() = 0;
	virtual int32 SetMaterialAttribute(int32 MaterialAttributes, int32 Value, const FGuid& AttributeID) = 0;

	// Exec
	virtual int32 BeginScope() = 0;
	virtual int32 BeginScope_If(int32 Condition) = 0;
	virtual int32 BeginScope_Else() = 0;
	virtual int32 BeginScope_For(const UMaterialExpression* Expression, int32 StartIndex, int32 EndIndex, int32 IndexStep) = 0;
	virtual int32 EndScope() = 0;
	virtual int32 ForLoopIndex(const UMaterialExpression* Expression) = 0;
	virtual int32 ReturnMaterialAttributes(int32 MaterialAttributes) = 0;
	virtual int32 SetLocal(const FName& LocalName, int32 Value) = 0;
	virtual int32 GetLocal(const FName& LocalName) = 0;

	// Neural network nodes
	virtual int32 NeuralOutput(int32 ViewportUV, uint32 NeuralIndexType) = 0;
	
	// Substrate
	virtual int32 SubstrateCreateAndRegisterNullMaterial() = 0;
	virtual int32 SubstrateSlabBSDF(
		int32 DiffuseAlbedo, int32 F0, int32 F90,
		int32 Roughness, int32 Anisotropy,
		int32 SSSProfileId, int32 SSSMFP, int32 SSSMFPScale, int32 SSSPhaseAniso, int32 SSSType,
		int32 EmissiveColor, 
		int32 SecondRoughness, int32 SecondRoughnessWeight, int32 SecondRoughnessAsSimpleClearCoat, int32 ClearCoatBottomNormal,
		int32 FuzzAmount, int32 FuzzColor, int32 FuzzRoughness,
		int32 Thickness,
		int32 GlintValue, int32 GlintUV,
		int32 SpecularProfileId,
		bool bIsAtTheBottomOfTopology,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateConversionFromLegacy(
		bool bHasDynamicShadingModels,
		int32 BaseColor, int32 Specular, int32 Metallic,
		int32 Roughness, int32 Anisotropy,
		int32 SubSurfaceColor, int32 SubSurfaceProfileId,
		int32 ClearCoat, int32 ClearCoatRoughness,
		int32 EmissiveColor,
		int32 Opacity,
		int32 ThinTranslucentTransmittanceColor,
		int32 ThinTranslucentSurfaceCoverage,
		int32 WaterScatteringCoefficients, int32 WaterAbsorptionCoefficients, int32 WaterPhaseG, int32 ColorScaleBehindWater,
		int32 ShadingModel,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		int32 ClearCoat_Normal, int32 ClearCoat_Tangent, const FString& ClearCoat_SharedLocalBasisIndexMacro,
		int32 CustomTangent_Tangent,
		FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateVolumetricFogCloudBSDF(int32 Albedo, int32 Extinction, int32 EmissiveColor, int32 AmbientOcclusion, bool bEmissiveOnly) = 0;
	virtual int32 SubstrateUnlitBSDF(int32 EmissiveColor, int32 TransmittanceColor, int32 Normal, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateUIBSDF(int32 EmissiveColor, int32 Opacity, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateHairBSDF(int32 BaseColor, int32 Scatter, int32 Specular, int32 Roughness, int32 Backlit, int32 EmissiveColor, int32 Tangent, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateEyeBSDF(int32 DiffuseAlbedo, int32 Roughness, int32 IrisMask, int32 IrisDistance, int32 IrisNormal, int32 IrisPlaneNormal, int32 SSSProfileId, int32 EmissiveColor, int32 CorneaNormal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateSingleLayerWaterBSDF(
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness, int32 EmissiveColor, int32 TopMaterialOpacity, 
		int32 WaterAlbedo, int32 WaterExtinction, int32 WaterPhaseG, int32 ColorScaleBehindWater, int32 Normal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateToonBSDF(
		int32 ToonProfileId,
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness,
		int32 EmissiveColor, int32 Normal,
		int32 PatternUVs,
		int32 Anisotropy, int32 Tangent,
		const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateHorizontalMixing(int32 Background, int32 Foreground, int32 Mix, int OperatorIndex, uint32 MaxDistanceFromLeaves) = 0;
	virtual int32 SubstrateHorizontalMixingParameterBlending(int32 Background, int32 Foreground, int32 HorizontalMixCodeChunk, int32 NormalMixCodeChunk, const FString& SharedLocalBasisIndexMacro, int32 BackgroundBSDFNormalCodeChunk, int32 ForegroundBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateVerticalLayering(int32 Top, int32 Base, int32 Thickness, int OperatorIndex, uint32 MaxDistanceFromLeaves) = 0;
	virtual int32 SubstrateVerticalLayeringParameterBlending(int32 Top, int32 Base, int32 Thickness, const FString& SharedLocalBasisIndexMacro, int32 TopBSDFNormalCodeChunk, int32 BaseBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateAdd(int32 A, int32 B, int OperatorIndex, uint32 MaxDistanceFromLeaves) = 0;
	virtual int32 SubstrateAddParameterBlending(int32 A, int32 B, int32 AMixWeight, const FString& SharedLocalBasisIndexMacro, int32 ABSDFNormalCodeChunk, int32 BBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateWeight(int32 A, int32 Weight, int OperatorIndex, uint32 MaxDistanceFromLeaves) = 0;
	virtual int32 SubstrateWeightParameterBlending(int32 A, int32 Weight, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateSelectParameterBlending(int32 A, int32 B, int32 SelectValue, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) = 0;
	virtual int32 SubstrateTransmittanceToMFP(int32 TransmittanceColor, int32 DesiredThickness, int32 OutputIndex) = 0;
	virtual int32 SubstrateMetalnessToDiffuseAlbedoF0(int32 BaseColor, int32 Specular, int32 Metallic, int32 OutputIndex) = 0;
	virtual int32 SubstrateHazinessToSecondaryRoughness(int32 BaseRoughness, int32 Haziness, int32 OutputIndex) = 0;
	virtual int32 SubstrateThinFilm(int32 NormalCodeChunk, int32 SpecularColorCodeChunk, int32 EdgeSpecularColorCodeChunk, int32 ThicknessCodeChunk, int32 IORCodeChunk, int32 OutputIndex) = 0;
	virtual int32 SubstrateCompilePreview(int32 SubstrateDataCodeChunk) = 0;

	/**
	 * This is dedicated to skip evaluating any opacity input when a material instance toggles the translucent blend mode to opaque.
	 */
	virtual bool SubstrateSkipsOpacityEvaluation() = 0;

	/**
	 * Pushes a node of the Substrate tree being walked. A node is caracterised by an expression and its input path taken.
	 */
	virtual FGuid SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex) = 0;
	/**
	 * Returns the unique id of the Substrate tree path, identifying the current node we have reached. 
	 * That is used as a unique identifier for each BSDF, recover their Substrate tree operator and apply some simplifications if required.
	 */
	virtual FGuid SubstrateTreeStackGetPathUniqueId() = 0;
	/**
	 * Returns the unique id of the Substrate tree path for the parent node of the current node position.
	 */
	virtual FGuid SubstrateTreeStackGetParentPathUniqueId() = 0;
	/**
	 * Pops a node node of the Substrate tree being walked. Used when walking up the tree back to its root level.
	 */
	virtual void SubstrateTreeStackPop() = 0;
	/**
	 * This can be used to know if the Substrate tree we are trying to build is too deep and we should stop the compilation.
	 * This can be used by all nodes taking as input SubstrateData in order to detect node re-entry leading to cyclic graph we cannot handle and compile internally: we must fail the compilation.
	 */
	virtual bool GetSubstrateTreeOutOfStackDepthOccurred() = 0;

	virtual int32 SubstrateThicknessStackGetThicknessIndex() = 0;
	virtual int32 SubstrateThicknessStackGetThicknessCode(int32 Index) = 0;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input) = 0;
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input) = 0;
	virtual void SubstrateThicknessStackPop() = 0;

	virtual struct FSubstrateTranslatorDataInterface& GetSubstrateTranslatorData() = 0;

	/**
	 * Register an operator of the tree representation the Substrate material and its topology.
	 */
	virtual FSubstrateOperator& SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending = false) = 0;
	/**
	 * Return the operator information for a given expression.
	 */
	virtual FSubstrateOperator& SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid) = 0;
	/**
	 * Return the operator information for a given index.
	 */
	virtual FSubstrateOperator* SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex) = 0;

	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk) = 0;
	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk, int32 TangentCodeChunk) = 0;
	virtual FString GetSubstrateSharedLocalBasisIndexMacro(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis) = 0;
	virtual int32 SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 ACodeChunk, int32 BCodeChunk) = 0;
	virtual int32 SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 TopCodeChunk) = 0;
	virtual int32 SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 BackgroundCodeChunk, int32 ForegroundCodeChunk, int32 HorizontalMixCodeChunk) = 0;

	// Water
	virtual int32 SceneDepthWithoutWater(int32 Offset, int32 ViewportUV, bool bUseOffset, float FallbackDepth) = 0;

	virtual int32 MapARPassthroughCameraUV(int32 UV) = 0;
	// The compiler can run in a different state and this affects caching of sub expression, Expressions are different (e.g. ResolvedView.PrevWorldViewOrigin) when using previous frame's values
	// If possible we should re-factor this to avoid having to deal with compiler state
	virtual bool IsCurrentlyCompilingForPreviousFrame() const { return false; }
	virtual bool IsDevelopmentFeatureEnabled(const FName& FeatureName) const { return true; }

	// Validation mode to allow different handling of material expressions during translation.
	void EnableValidationMode(bool bEnable) { bIsValidating = bEnable; }
	bool IsValidating() const { return bIsValidating; }

protected:

private:
	ESubstrateMaterialExport SubstrateMaterialExport = SME_None;
	ESubstrateMaterialExportContext SubstrateMaterialExportContext = SMEC_Opaque;
	uint8 SubstrateMaterialExportLegacyBlendMode = 0;
	bool bIsValidating = false;
};

/** 
 * A proxy for the material compiler interface which by default passes all function calls unmodified. 
 * Note: Any functions of FMaterialCompiler that change the internal compiler state must be routed!
 */
class FProxyMaterialCompiler : public FMaterialCompiler
{
public:

	// Constructor.
	FProxyMaterialCompiler(FMaterialCompiler* InCompiler) :
		Compiler(InCompiler)
	{}

	// Simple pass through all other material operations unmodified.
	
	virtual bool ShouldStopTranslating() const override { return false; }
	virtual FMaterialShadingModelField GetMaterialShadingModels() const { return Compiler->GetMaterialShadingModels(); }
	virtual FMaterialShadingModelField GetCompiledShadingModels() const { return Compiler->GetCompiledShadingModels(); }
	virtual EMaterialValueType GetParameterType(int32 Index) const { return Compiler->GetParameterType(Index); }
	virtual FMaterialUniformExpression* GetParameterUniformExpression(int32 Index) const { return Compiler->GetParameterUniformExpression(Index); }
	virtual bool GetTextureForExpression(int32 Index, int32& OutTextureIndex, EMaterialSamplerType& OutSamplerType, TOptional<FName>& OutParameterName) const override { return Compiler->GetTextureForExpression(Index, OutTextureIndex, OutSamplerType, OutParameterName); }
	virtual bool GetTextureCollectionForExpression(int32 Index, int32& OutTextureCollectionIndex, TOptional<FName>& OutParameterName) const override { return Compiler->GetTextureCollectionForExpression(Index, OutTextureCollectionIndex, OutParameterName); }
	virtual void SetMaterialProperty(EMaterialProperty InProperty, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) override { Compiler->SetMaterialProperty(InProperty, OverrideShaderFrequency, bUsePreviousFrameTime); }
	virtual void PushMaterialAttribute(const FGuid& InAttributeID) override { Compiler->PushMaterialAttribute(InAttributeID); }
	virtual FGuid PopMaterialAttribute() override { return Compiler->PopMaterialAttribute(); }
	virtual const FGuid GetMaterialAttribute() override { return Compiler->GetMaterialAttribute(); }
	virtual void SetBaseMaterialAttribute(const FGuid& InAttributeID) override { Compiler->SetBaseMaterialAttribute(InAttributeID); }

	virtual UMaterialExpressionCustomOutput* GetTopCustomOutput() override { return Compiler->GetTopCustomOutput(); }

	virtual bool IsTangentSpaceNormal() const override { return Compiler->IsTangentSpaceNormal(); }

	virtual void PushParameterOwner(const FMaterialParameterInfo& InOwnerInfo) override { Compiler->PushParameterOwner(InOwnerInfo); }
	virtual FMaterialParameterInfo PopParameterOwner() override { return Compiler->PopParameterOwner(); }

	virtual EShaderFrequency GetCurrentShaderFrequency() const override { return Compiler->GetCurrentShaderFrequency(); }
	virtual int32 Error(const TCHAR* Text) override { return Compiler->Error(Text); }
	virtual void AppendExpressionError(UMaterialExpression* Expression, const TCHAR* Text) override { return Compiler->AppendExpressionError(Expression, Text); }

	virtual int32 CallExpression(FMaterialExpressionKey ExpressionKey, FMaterialCompiler* InCompiler) override { return Compiler->CallExpression(ExpressionKey, InCompiler); }

	virtual int32 CallExpressionExec(UMaterialExpression* Expression) { return Compiler->CallExpressionExec(Expression); }

	virtual void PushFunction(FMaterialFunctionCompileState* FunctionState) override { Compiler->PushFunction(FunctionState); }
	virtual FMaterialFunctionCompileState* PopFunction() override { return Compiler->PopFunction(); }
	virtual int32 GetCurrentFunctionStackDepth() override { return Compiler->GetCurrentFunctionStackDepth(); }

	virtual EMaterialValueType GetType(int32 Code) override { return Compiler->GetType(Code); }
	virtual EMaterialQualityLevel::Type GetQualityLevel() override { return Compiler->GetQualityLevel(); }
	virtual ERHIFeatureLevel::Type GetFeatureLevel() override { return Compiler->GetFeatureLevel(); }
	virtual EShaderPlatform GetShaderPlatform() override { return Compiler->GetShaderPlatform(); }
	virtual const ITargetPlatform* GetTargetPlatform() const override { return Compiler->GetTargetPlatform(); }
	virtual bool IsMaterialPropertyUsed(EMaterialProperty Property, int32 CodeChunkIdx) const override { return Compiler->IsMaterialPropertyUsed(Property, CodeChunkIdx); }
	virtual int32 ValidCast(int32 Code, EMaterialValueType DestType) override { return Compiler->ValidCast(Code, DestType); }
	virtual int32 ForceCast(int32 Code, EMaterialValueType DestType, uint32 ForceCastFlags = 0) override { return Compiler->ForceCast(Code, DestType, ForceCastFlags); }
	virtual int32 CastShadingModelToFloat(int32 Code) override { return Compiler->CastShadingModelToFloat(Code); }
	virtual int32 TruncateLWC(int32 Code) override { return Compiler->TruncateLWC(Code); }

	virtual int32 AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex) override { return Compiler->AccessCollectionParameter(ParameterCollection, ParameterIndex, ComponentIndex); }
	virtual int32 CollectionTransform(int32 InputIndex, const TStaticArray<int32, 5>& CollectionParameters, EParameterCollectionTransformType TransformType) override { return Compiler->CollectionTransform(InputIndex, CollectionParameters, TransformType); }

	virtual int32 NumericParameter(EMaterialParameterType ParameterType, FName ParameterName, const UE::Shader::FValue& DefaultValue) override { return Compiler->NumericParameter(ParameterType, ParameterName, DefaultValue); }

	virtual int32 Constant(float X) override { return Compiler->Constant(X); }
	virtual int32 Constant2(float X, float Y) override { return Compiler->Constant2(X, Y); }
	virtual int32 Constant3(float X, float Y, float Z) override { return Compiler->Constant3(X, Y, Z); }
	virtual int32 Constant4(float X, float Y, float Z, float W) override { return Compiler->Constant4(X, Y, Z, W); }
	virtual int32 GenericConstant(const UE::Shader::FValue& Value) override { return Compiler->GenericConstant(Value); }

	virtual	int32 ViewProperty(EMaterialExposedViewProperty Property, bool InvProperty) override { return Compiler->ViewProperty(Property, InvProperty); }
	virtual int32 IsOrthographic() override { return Compiler->IsOrthographic(); }

	virtual int32 GameTime(bool bPeriodic, float Period) override { return Compiler->GameTime(bPeriodic, Period); }
	virtual int32 RealTime(bool bPeriodic, float Period) override { return Compiler->RealTime(bPeriodic, Period); }
	virtual int32 DeltaTime() override { return Compiler->DeltaTime(); }

	virtual int32 PeriodicHint(int32 PeriodicCode) override { return Compiler->PeriodicHint(PeriodicCode); }

	virtual int32 Sine(int32 X) override { return Compiler->Sine(X); }
	virtual int32 Cosine(int32 X) override { return Compiler->Cosine(X); }
	virtual int32 Tangent(int32 X) override { return Compiler->Tangent(X); }
	virtual int32 Arcsine(int32 X) override { return Compiler->Arcsine(X); }
	virtual int32 ArcsineFast(int32 X) override { return Compiler->ArcsineFast(X); }
	virtual int32 Arccosine(int32 X) override { return Compiler->Arccosine(X); }
	virtual int32 ArccosineFast(int32 X) override { return Compiler->ArccosineFast(X); }
	virtual int32 Arctangent(int32 X) override { return Compiler->Arctangent(X); }
	virtual int32 ArctangentFast(int32 X) override { return Compiler->ArctangentFast(X); }
	virtual int32 Arctangent2(int32 Y, int32 X) override { return Compiler->Arctangent2(Y, X); }
	virtual int32 Arctangent2Fast(int32 Y, int32 X) override { return Compiler->Arctangent2Fast(Y, X); }

	virtual int32 Floor(int32 X) override { return Compiler->Floor(X); }
	virtual int32 Ceil(int32 X) override { return Compiler->Ceil(X); }
	virtual int32 Round(int32 X) override { return Compiler->Round(X); }
	virtual int32 Truncate(int32 X) override { return Compiler->Truncate(X); }
	virtual int32 Sign(int32 X) override { return Compiler->Sign(X); }
	virtual int32 Frac(int32 X) override { return Compiler->Frac(X); }
	virtual int32 Fmod(int32 A, int32 B) override { return Compiler->Fmod(A, B); }
	virtual int32 Modulo(int32 A, int32 B) override { return Compiler->Modulo(A, B); }
	virtual int32 Abs(int32 X) override { return Compiler->Abs(X); }

	virtual int32 ReflectionVector() override { return Compiler->ReflectionVector(); }
	virtual int32 CameraVector() override { return Compiler->CameraVector(); }
	virtual int32 LightVector() override { return Compiler->LightVector(); }

	virtual int32 GetViewportUV() override { return Compiler->GetViewportUV(); }
	virtual int32 GetPixelPosition() override { return Compiler->GetPixelPosition(); }
	virtual int32 LocalPosition(EPositionIncludedOffsets IncludedOffsets, ELocalPositionOrigin OriginType) { return Compiler->LocalPosition(IncludedOffsets, OriginType); }
	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) override { return Compiler->WorldPosition(WorldPositionIncludedOffsets); }
	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) override { return Compiler->ObjectWorldPosition(OriginType); }
	virtual int32 ObjectRadius() override { return Compiler->ObjectRadius(); }
	virtual int32 ObjectBounds() override { return Compiler->ObjectBounds(); }
	virtual int32 ObjectLocalBounds(int32 OutputIndex) override { return Compiler->ObjectLocalBounds(OutputIndex); }
	virtual int32 InstanceLocalBounds(int32 OutputIndex) override { return Compiler->InstanceLocalBounds(OutputIndex); }
	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) override { return Compiler->PreSkinnedLocalBounds(OutputIndex); }
	virtual int32 DistanceCullFade() override { return Compiler->DistanceCullFade(); }
	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) override { return Compiler->ActorWorldPosition(OriginType); }
	virtual int32 ParticleMacroUV() override { return Compiler->ParticleMacroUV(); }
	virtual int32 ParticleSubUV(int32 TextureIndex, EMaterialSamplerType SamplerType, int32 MipValue0Index, int32 MipValue1Index, ETextureMipValueMode MipValueMode, bool bBlend) override 
	{
		return Compiler->ParticleSubUV(TextureIndex, SamplerType, MipValue0Index, MipValue1Index, MipValueMode,	bBlend);
	}
	virtual int32 ParticleSubUVProperty(int32 PropertyIndex) override { return Compiler->ParticleSubUVProperty(PropertyIndex); }
	virtual int32 ParticleColor() override { return Compiler->ParticleColor(); }
	virtual int32 ParticlePosition(EPositionOrigin OriginType) override { return Compiler->ParticlePosition(OriginType); }
	virtual int32 ParticleRadius() override { return Compiler->ParticleRadius(); }
	virtual int32 SphericalParticleOpacity(int32 Density) override { return Compiler->SphericalParticleOpacity(Density); }

	virtual int32 DynamicBranch(int32 Condition, int32 A, int32 B) override { return Compiler->DynamicBranch(Condition, A, B); };
	virtual int32 If(int32 A, int32 B, int32 AGreaterThanB, int32 AEqualsB, int32 ALessThanB, int32 Threshold) override { return Compiler->If(A, B, AGreaterThanB, AEqualsB, ALessThanB, Threshold); }
	virtual int32 Switch(int32 SwitchValueInput, int32 DefaultInput, TArray<int32>& CompiledInputs) override { return Compiler->Switch(SwitchValueInput, DefaultInput, CompiledInputs); };
	
	virtual int32 TextureSample(int32 InTexture, int32 Coordinate, enum EMaterialSamplerType SamplerType, int32 MipValue0Index, int32 MipValue1Index, ETextureMipValueMode MipValueMode, ESamplerSourceMode SamplerSource, ETextureGatherMode GatherMode, int32 TextureReferenceIndex, bool AutomaticViewMipBias, bool AdaptiveVirtualTexture, bool EnableFeedback) override
	{
		return Compiler->TextureSample(InTexture, Coordinate, SamplerType, MipValue0Index, MipValue1Index, MipValueMode, SamplerSource, GatherMode, TextureReferenceIndex, AutomaticViewMipBias, AdaptiveVirtualTexture);
	}
	virtual int32 TextureProperty(int32 InTexture, EMaterialExposedTextureProperty Property) override
	{
		return Compiler->TextureProperty(InTexture, Property);
	}

	virtual int32 TextureCoordinate(uint32 CoordinateIndex, bool UnMirrorU, bool UnMirrorV) override { return Compiler->TextureCoordinate(CoordinateIndex, UnMirrorU, UnMirrorV); }
	virtual void SetPotentiallyManipulateTexCoords() override { Compiler->SetPotentiallyManipulateTexCoords(); }

	virtual int32 TextureDecalMipmapLevel(int32 TextureSizeInput) override { return Compiler->TextureDecalMipmapLevel(TextureSizeInput); }
	virtual int32 TextureDecalDerivative(bool bDDY) override { return Compiler->TextureDecalDerivative(bDDY); }
	virtual int32 DecalColor() override { return Compiler->DecalColor(); }
	virtual int32 DecalLifetimeOpacity() override { return Compiler->DecalLifetimeOpacity(); }

	virtual int32 Texture(UTexture* InTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource = SSM_FromTextureAsset, ETextureMipValueMode MipValueMode = TMVM_None) override { return Compiler->Texture(InTexture, TextureReferenceIndex, SamplerType, SamplerSource, MipValueMode); }
	virtual int32 TextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType, ESamplerSourceMode SamplerSource = SSM_FromTextureAsset) override { return Compiler->TextureParameter(ParameterName, DefaultValue, TextureReferenceIndex, SamplerType, SamplerSource); }

	virtual int32 TextureCollection(UTextureCollection* TextureCollection, int32& TextureCollectionReferenceIndex) override
	{
		return Compiler->TextureCollection(TextureCollection, TextureCollectionReferenceIndex);
	}
	virtual int32 TextureCollectionParameter(FName ParameterName, UTextureCollection* DefaultValue, int32& TextureCollectionReferenceIndex) override
	{
		return Compiler->TextureCollectionParameter(ParameterName, DefaultValue, TextureCollectionReferenceIndex);
	}
	virtual int32 TextureCollectionCount(int32 InTextureCollectionCodeIndex) override
	{
		return Compiler->TextureCollectionCount(InTextureCollectionCodeIndex);
	}
	virtual int32 TextureFromCollection(int32 TextureCollectionCodeIndex, int32 IndexIntoCollectionCodeIndex, EMaterialValueType ResultTextureType) override
	{
		return Compiler->TextureFromCollection(TextureCollectionCodeIndex, IndexIntoCollectionCodeIndex, ResultTextureType);
	}

	virtual int32 TextureStreamingInfo(int32 TextureReferenceIndex, int32 TextureIndex, int32 CoordinateIndex) override
	{
		return Compiler->TextureStreamingInfo(TextureReferenceIndex, TextureIndex, CoordinateIndex);
	}
	
	virtual int32 MaterialCacheTextureDescriptor(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, uint32 LayerIndex) override
	{
		return Compiler->MaterialCacheTextureDescriptor(Layout, PrimitiveIDIndex, LayerIndex);
	}
	
	virtual int32 SampleMaterialCache(const FMaterialCacheTagLayout& Layout, int32 PrimitiveIDIndex, int32 TexCoordIndex) override
	{
		return Compiler->SampleMaterialCache(Layout, PrimitiveIDIndex, TexCoordIndex);
	}
	
	virtual int32 DefaultMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout) override
	{
		return Compiler->DefaultMaterialCacheAttribute(Layout);
	}

	virtual int32 SetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex, int32 Value) override
	{
		return Compiler->SetMaterialCacheAttribute(Layout, AttributeSet, AttributeIndex, Value);
	}
	
	virtual int32 GetMaterialCacheAttribute(const FMaterialCacheTagLayout& Layout, int32 AttributeSet, int32 AttributeIndex) override
	{
		return Compiler->GetMaterialCacheAttribute(Layout, AttributeSet, AttributeIndex);
	}

	virtual int32 MaterialCacheOutput(UMaterialExpressionCustomOutput* CustomOutput, const FMaterialCacheTagLayout& Layout, int32 AttributeSet) override
	{
		return Compiler->MaterialCacheOutput(CustomOutput, Layout, AttributeSet);
	}
	
	virtual int32 VirtualTexture(URuntimeVirtualTexture* InTexture, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override
	{
		return Compiler->VirtualTexture(InTexture, TextureLayerIndex, PageTableLayerIndex, TextureReferenceIndex, SamplerType);
	}
	virtual int32 VirtualTextureParameter(FName ParameterName, URuntimeVirtualTexture* DefaultValue, int32 TextureLayerIndex, int32 PageTableLayerIndex, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override
	{
		return Compiler->VirtualTextureParameter(ParameterName, DefaultValue, TextureLayerIndex, PageTableLayerIndex, TextureReferenceIndex, SamplerType);
	}
	virtual int32 VirtualTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override { return Compiler->VirtualTextureUniform(TextureIndex, VectorIndex, Type); }
	virtual int32 VirtualTextureUniform(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override { return Compiler->VirtualTextureUniform(ParameterName, TextureIndex, VectorIndex, Type); }
	virtual int32 VirtualTextureWorldToUV(int32 WorldPositionIndex, int32 P0, int32 P1, int32 P2, EPositionOrigin PositionOrigin) override { return Compiler->VirtualTextureWorldToUV(WorldPositionIndex, P0, P1, P2, PositionOrigin); }
	virtual int32 VirtualTextureUnpack(int32 CodeIndex0, int32 CodeIndex1, int32 CodeIndex2, int32 P0, EVirtualTextureUnpackType UnpackType) override { return Compiler->VirtualTextureUnpack(CodeIndex0, CodeIndex1, CodeIndex2, P0, UnpackType); }
	virtual int32 VirtualTextureCustomData() override { return Compiler->VirtualTextureCustomData(); }

	virtual int32 ExternalTexture(const FGuid& ExternalTextureGuid) override { return Compiler->ExternalTexture(ExternalTextureGuid); }
	virtual int32 ExternalTexture(UTexture* InTexture, int32& TextureReferenceIndex) override { return Compiler->ExternalTexture(InTexture, TextureReferenceIndex); }
	virtual int32 ExternalTextureParameter(FName ParameterName, UTexture* DefaultValue, int32& TextureReferenceIndex) override { return Compiler->ExternalTextureParameter(ParameterName, DefaultValue, TextureReferenceIndex); }
	virtual int32 ExternalTextureCoordinateScaleRotation(int32 TextureReferenceIndex, TOptional<FName> ParameterName) override { return Compiler->ExternalTextureCoordinateScaleRotation(TextureReferenceIndex, ParameterName); }
	virtual int32 ExternalTextureCoordinateScaleRotation(const FGuid& ExternalTextureGuid) override { return Compiler->ExternalTextureCoordinateScaleRotation(ExternalTextureGuid); }
	virtual int32 ExternalTextureCoordinateOffset(int32 TextureReferenceIndex, TOptional<FName> ParameterName) override { return Compiler->ExternalTextureCoordinateOffset(TextureReferenceIndex, ParameterName); }
	virtual int32 ExternalTextureCoordinateOffset(const FGuid& ExternalTextureGuid) override { return Compiler->ExternalTextureCoordinateOffset(ExternalTextureGuid); }

	virtual int32 SparseVolumeTexture(USparseVolumeTexture* Texture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType) override																									{ return Compiler->SparseVolumeTexture(Texture, TextureReferenceIndex, SamplerType); }
	virtual int32 SparseVolumeTextureParameter(FName ParameterName, USparseVolumeTexture* InDefaultTexture, int32& TextureReferenceIndex, EMaterialSamplerType SamplerType)	override															{ return Compiler->SparseVolumeTextureParameter(ParameterName, InDefaultTexture, TextureReferenceIndex, SamplerType); }
	virtual int32 SparseVolumeTextureUniform(int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override																														{ return Compiler->SparseVolumeTextureUniform(TextureIndex, VectorIndex, Type); }
	virtual int32 SparseVolumeTextureUniformParameter(FName ParameterName, int32 TextureIndex, int32 VectorIndex, UE::Shader::EValueType Type) override																							{ return Compiler->SparseVolumeTextureUniformParameter(ParameterName, TextureIndex, VectorIndex, Type); }
	virtual int32 SparseVolumeTextureSamplePageTable(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipLevelIndex, ESamplerSourceMode SamplerSource, bool bIsManualLinearMipMapSecondSample) override									{ return Compiler->SparseVolumeTextureSamplePageTable(SparseVolumeTextureIndex, UVWIndex, MipLevelIndex, SamplerSource, bIsManualLinearMipMapSecondSample); }
	virtual int32 SparseVolumeTextureSamplePhysicalTileData(int32 SparseVolumeTextureIndex, int32 VoxelCoordIndex, int32 PhysicalTileDataIdxIndex, bool bIsManualLinearMipMapSecondSample) override												{ return Compiler->SparseVolumeTextureSamplePhysicalTileData(SparseVolumeTextureIndex, VoxelCoordIndex, PhysicalTileDataIdxIndex, bIsManualLinearMipMapSecondSample); }
	virtual int32 SparseVolumeTextureSample(int32 SparseVolumeTextureIndex, int32 UVWIndex, int32 MipValue0Index, int32 MipValue1Index, int32 PhysicalTileDataIdxIndex, ETextureMipValueMode MipValueMode, ESamplerSourceMode SamplerSource)	{ return Compiler->SparseVolumeTextureSample(SparseVolumeTextureIndex, UVWIndex, MipValue0Index, MipValue1Index, PhysicalTileDataIdxIndex, MipValueMode, SamplerSource); }

	virtual UObject* GetReferencedTexture(int32 Index) override
	{
		return Compiler->GetReferencedTexture(Index);
	}

	virtual UTextureCollection* GetReferencedTextureCollection(int32 Index) override
	{
		return Compiler->GetReferencedTextureCollection(Index);
	}

	virtual	int32 PixelDepth() override { return Compiler->PixelDepth(); }
	virtual int32 SceneDepth(int32 Offset, int32 ViewportUV, bool bUseOffset) override { return Compiler->SceneDepth(Offset, ViewportUV, bUseOffset); }
	virtual int32 SceneColor(int32 Offset, int32 ViewportUV, bool bUseOffset) override { return Compiler->SceneColor(Offset, ViewportUV, bUseOffset); }
	virtual int32 SceneTextureLookup(int32 ViewportUV, uint32 InSceneTextureId, bool bFiltered, bool bClamped, bool bUnused) override { return Compiler->SceneTextureLookup(ViewportUV, InSceneTextureId, bFiltered, bClamped, bUnused); }
	virtual int32 GetSceneTextureViewSize(int32 SceneTextureId, bool InvProperty) override { return Compiler->GetSceneTextureViewSize(SceneTextureId, InvProperty); }
	virtual int32 FindOrAddUserSceneTexture(FName UserSceneTextureName) override { return Compiler->FindOrAddUserSceneTexture(UserSceneTextureName); }
	virtual int32 DBufferTextureLookup(int32 ViewportUV, uint32 DBufferTextureIndex) override { return Compiler->DBufferTextureLookup(ViewportUV, DBufferTextureIndex); }
	virtual int32 PathTracingBufferTextureLookup(int32 ViewportUV, uint32 PathTracingBufferTextureIndex) override { return Compiler->PathTracingBufferTextureLookup(ViewportUV, PathTracingBufferTextureIndex); }

	virtual int32 StaticBool(bool Value) override { return Compiler->StaticBool(Value); }
	virtual int32 StaticBoolParameter(FName ParameterName, bool bDefaultValue) override { return Compiler->StaticBoolParameter(ParameterName, bDefaultValue); }
	virtual int32 DynamicBoolParameter(FName ParameterName, bool bDefaultValue) override { return Compiler->DynamicBoolParameter(ParameterName, bDefaultValue); }
	virtual int32 StaticComponentMask(int32 Vector, FName ParameterName, bool bDefaultR, bool bDefaultG, bool bDefaultB, bool bDefaultA) override { return Compiler->StaticComponentMask(Vector, ParameterName, bDefaultR, bDefaultG, bDefaultB, bDefaultA); }
	virtual const FMaterialLayersFunctions* GetMaterialLayers() override { return Compiler->GetMaterialLayers(); }
	virtual bool GetStaticBoolValue(int32 BoolIndex, bool& bSucceeded) override { return Compiler->GetStaticBoolValue(BoolIndex, bSucceeded); }
	virtual int32 StaticTerrainLayerWeight(FName ParameterName, int32 Default, bool bTextureArray = false) override { return Compiler->StaticTerrainLayerWeight(ParameterName, Default, bTextureArray); }

	virtual int32 ExternalCode(const FMaterialExternalCodeDeclaration& InExternalCode) override { return Compiler->ExternalCode(InExternalCode); }

	virtual int32 FontSignedDistanceData() override { return Compiler->FontSignedDistanceData(); }

	virtual int32 VertexColor() override { return Compiler->VertexColor(); }
	virtual int32 MeshPaintTextureCoordinateIndex() override { return Compiler->MeshPaintTextureCoordinateIndex(); }
	virtual int32 MeshPaintTextureDescriptor() override { return Compiler->MeshPaintTextureDescriptor(); }
	virtual int32 MeshPaintTextureReplace(int32 Invalid, int32 Valid) override { return Compiler->MeshPaintTextureReplace(Invalid, Valid); }

	virtual int32 PreSkinnedPosition() override { return Compiler->PreSkinnedPosition(); }
	virtual int32 PreSkinnedNormal() override { return Compiler->PreSkinnedNormal(); }

	virtual int32 Add(int32 A, int32 B) override { return Compiler->Add(A, B); }
	virtual int32 Sub(int32 A, int32 B) override { return Compiler->Sub(A, B); }
	virtual int32 Mul(int32 A, int32 B) override { return Compiler->Mul(A, B); }
	virtual int32 Div(int32 A, int32 B) override { return Compiler->Div(A, B); }
	virtual int32 Dot(int32 A, int32 B) override { return Compiler->Dot(A, B); }
	virtual int32 Cross(int32 A, int32 B) override { return Compiler->Cross(A, B); }

	virtual int32 Power(int32 Base, int32 Exponent) override { return Compiler->Power(Base, Exponent); }
	virtual int32 Exponential(int32 X) override { return Compiler->Exponential(X); }
	virtual int32 Exponential2(int32 X) override { return Compiler->Exponential2(X); }
	virtual int32 Logarithm(int32 X) override { return Compiler->Logarithm(X); }
	virtual int32 Logarithm2(int32 X) override { return Compiler->Logarithm2(X); }
	virtual int32 Logarithm10(int32 X) override { return Compiler->Logarithm10(X); }
	virtual int32 SquareRoot(int32 X) override { return Compiler->SquareRoot(X); }
	virtual int32 Length(int32 X) override { return Compiler->Length(X); }
	virtual int32 Normalize(int32 X) override { return Compiler->Normalize(X); }
	virtual int32 HsvToRgb(int32 X) override { return Compiler->HsvToRgb(X); }
	virtual int32 RgbToHsv(int32 X) override { return Compiler->RgbToHsv(X); }

	virtual int32 Lerp(int32 X, int32 Y, int32 A) override { return Compiler->Lerp(X, Y, A); }
	virtual int32 Min(int32 A, int32 B) override { return Compiler->Min(A, B); }
	virtual int32 Max(int32 A, int32 B) override { return Compiler->Max(A, B); }
	virtual int32 Clamp(int32 X, int32 A, int32 B) override { return Compiler->Clamp(X, A, B); }
	virtual int32 Saturate(int32 X) override { return Compiler->Saturate(X); }

	virtual int32 SmoothStep(int32 X, int32 Y, int32 A) override { return Compiler->SmoothStep(X, Y, A); }
	virtual int32 Step(int32 Y, int32 X) override { return Compiler->Step(Y, X); }
	virtual int32 InvLerp(int32 X, int32 Y, int32 A) override { return Compiler->InvLerp(X, Y, A); }

	virtual int32 ComponentMask(int32 Vector, bool R, bool G, bool B, bool A) override { return Compiler->ComponentMask(Vector, R, G, B, A); }
	virtual int32 VectorComponent(int32 Vector, int32 Component) override { return Compiler->VectorComponent(Vector, Component); }
	virtual int32 BitfieldExtract(int32 Value, int32 BitOffset, int32 BitCount) override { return Compiler->BitfieldExtract(Value, BitOffset, BitCount); }
	virtual int32 AppendVector(int32 A, int32 B) override { return Compiler->AppendVector(A, B); }
	virtual int32 TransformVector(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		return Compiler->TransformVector(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	virtual int32 TransformPosition(EMaterialCommonBasis SourceCoordBasis, EMaterialCommonBasis DestCoordBasis, FTransformParameters& Parameters, int32 A) override
	{
		return Compiler->TransformPosition(SourceCoordBasis, DestCoordBasis, Parameters, A);
	}
	virtual int32 TransformNormalFromRequestedBasisToWorld(int32 NormalCodeChunk) override { return Compiler->TransformNormalFromRequestedBasisToWorld(NormalCodeChunk); }

	virtual int32 DynamicParameter(FLinearColor& DefaultValue, uint32 ParameterIndex = 0) override { return Compiler->DynamicParameter(DefaultValue, ParameterIndex); }
	virtual int32 LightmapUVs() override { return Compiler->LightmapUVs(); }
	virtual int32 PrecomputedAOMask() override { return Compiler->PrecomputedAOMask(); }

	virtual int32 GIReplace(int32 Direct, int32 StaticIndirect, int32 DynamicIndirect) override { return Compiler->GIReplace(Direct, StaticIndirect, DynamicIndirect); }
	virtual int32 ShadowReplace(int32 Default, int32 Shadow) override { return Compiler->ShadowReplace(Default, Shadow); }
	virtual int32 NaniteReplace(int32 Default, int32 Nanite) override { return Compiler->NaniteReplace(Default, Nanite); }
	virtual int32 MaterialCache(int32 Default, int32 MaterialCache) override { return Compiler->MaterialCache(Default, MaterialCache); }
	virtual int32 RayTracingQualitySwitchReplace(int32 Normal, int32 RayTraced) override { return Compiler->RayTracingQualitySwitchReplace(Normal, RayTraced); }
	virtual int32 PathTracingQualitySwitchReplace(int32 Normal, int32 PathTraced) override { return Compiler->PathTracingQualitySwitchReplace(Normal, PathTraced); }
	virtual int32 PathTracingRayTypeSwitch(int32 Main, int32 Shadow, int32 IndirectDiffuse, int32 IndirectSpecular, int32 IndirectVolume) override { return Compiler->PathTracingRayTypeSwitch(Main, Shadow, IndirectDiffuse, IndirectSpecular, IndirectVolume); }
	virtual int32 LightmassReplace(int32 Realtime, int32 Lightmass) override { return Compiler->LightmassReplace(Realtime, Lightmass); }
	virtual int32 ReflectionCapturePassSwitch(int32 Default, int32 Reflection) override { return Compiler->ReflectionCapturePassSwitch(Default, Reflection); }

	virtual int32 ObjectOrientation() override { return Compiler->ObjectOrientation(); }
	virtual int32 RotateAboutAxis(int32 NormalizedRotationAxisAndAngleIndex, int32 PositionOnAxisIndex, int32 PositionIndex) override
	{
		return Compiler->RotateAboutAxis(NormalizedRotationAxisAndAngleIndex, PositionOnAxisIndex, PositionIndex);
	}
	virtual int32 TwoSidedSign() override { return Compiler->TwoSidedSign(); }
	virtual int32 VertexNormal() override { return Compiler->VertexNormal(); }
	virtual int32 VertexTangent() override { return Compiler->VertexTangent(); }
	virtual int32 PixelNormalWS() override { return Compiler->PixelNormalWS(); }
	virtual int32 IsFirstPerson() override { return Compiler->IsFirstPerson(); }

	virtual int32 CustomExpression(class UMaterialExpressionCustom* Custom, int32 OutputIndex, TArray<int32>& CompiledInputs) override { return Compiler->CustomExpression(Custom, OutputIndex, CompiledInputs); }
	virtual int32 CustomOutput(class UMaterialExpressionCustomOutput* Custom, int32 OutputIndex, int32 OutputCode, EMaterialCustomOutputFlags Flags) override { return Compiler->CustomOutput(Custom, OutputIndex, OutputCode, Flags); }
	
	virtual int32 PushRuntimeVirtualTextureOutput() override { return Compiler->PushRuntimeVirtualTextureOutput(); }
	virtual int32 PopRuntimeVirtualTextureOutput(uint8 AttributeMask) override { return Compiler->PopRuntimeVirtualTextureOutput(AttributeMask); }
	virtual bool IsInRuntimeVirtualTextureOutput() const override { return Compiler->IsInRuntimeVirtualTextureOutput(); }

	virtual int32 DDX(int32 X) override { return Compiler->DDX(X); }
	virtual int32 DDY(int32 X) override { return Compiler->DDY(X); }

	virtual int32 AntialiasedTextureMask(int32 Tex, int32 UV, float Threshold, uint8 Channel) override
	{
		return Compiler->AntialiasedTextureMask(Tex, UV, Threshold, Channel);
	}

	virtual int32 Sobol(int32 Cell, int32 Index, int32 Seed) override { return Compiler->Sobol(Cell, Index, Seed); }
	virtual int32 TemporalSobol(int32 Index, int32 Seed) override { return Compiler->TemporalSobol(Index, Seed); }
	virtual int32 Noise(int32 Position, EPositionOrigin PositionOrigin, float Scale, int32 Quality, uint8 NoiseFunction, bool bTurbulence, int32 Levels, float OutputMin, float OutputMax, float LevelScale, int32 FilterWidth, bool bTiling, uint32 TileSize) override
	{
		return Compiler->Noise(Position, PositionOrigin, Scale, Quality, NoiseFunction, bTurbulence, Levels, OutputMin, OutputMax, LevelScale, FilterWidth, bTiling, TileSize);
	}
	virtual int32 VectorNoise(int32 Position, EPositionOrigin PositionOrigin, int32 Quality, uint8 NoiseFunction, bool bTiling, uint32 TileSize) override
	{
		return Compiler->VectorNoise(Position, PositionOrigin, Quality, NoiseFunction, bTiling, TileSize);
	}
	virtual int32 ScalarBlueNoise() override { return Compiler->ScalarBlueNoise(); }

	virtual int32 BlackBody(int32 Temp) override { return Compiler->BlackBody(Temp); }
	virtual int32 DistanceToNearestSurface(int32 PositionArg, EPositionOrigin PositionOrigin) override { return Compiler->DistanceToNearestSurface(PositionArg, PositionOrigin); }
	virtual int32 DistanceFieldGradient(int32 PositionArg, EPositionOrigin PositionOrigin) override { return Compiler->DistanceFieldGradient(PositionArg, PositionOrigin); }
	virtual int32 DistanceFieldApproxAO(int32 PositionArg, EPositionOrigin PositionOrigin, int32 NormalArg, int32 BaseDistanceArg, int32 RadiusArg, uint32 NumSteps, float StepScale) override { return Compiler->DistanceFieldApproxAO(PositionArg, PositionOrigin, NormalArg, BaseDistanceArg, RadiusArg, NumSteps, StepScale); }
	virtual int32 ProcessPhysicsField(int32 PositionArg, EPositionOrigin PositionOrigin, const int32 OutputType, const int32 TargetIndex, const bool bSampleField)  override { return Compiler->ProcessPhysicsField(PositionArg, PositionOrigin, OutputType, TargetIndex, bSampleField); }
	virtual int32 PerInstanceRandom() override { return Compiler->PerInstanceRandom(); }
	virtual int32 PerInstanceFadeAmount() override { return Compiler->PerInstanceFadeAmount(); }
	virtual int32 PerInstanceCustomData(int32 DataIndex, int32 DefaultValueIndex) override { return Compiler->PerInstanceCustomData(DataIndex, DefaultValueIndex); }
	virtual int32 PerInstanceCustomData3Vector(int32 DataIndex, int32 DefaultValueIndex) override { return Compiler->PerInstanceCustomData3Vector(DataIndex, DefaultValueIndex); }
	virtual int32 DepthOfFieldFunction(int32 Depth, int32 FunctionValueIndex) override
	{
		return Compiler->DepthOfFieldFunction(Depth, FunctionValueIndex);
	}

	virtual int32 PostVolumeUserFlagTestFunction(int32 Input) override { return Compiler->PostVolumeUserFlagTestFunction(Input); }
	virtual int32 GetHairUV() override { return Compiler->GetHairUV(); }
	virtual int32 GetHairDimensions() override { return Compiler->GetHairDimensions(); }
	virtual int32 GetHairSeed() override { return Compiler->GetHairSeed(); }
	virtual int32 GetHairClumpID() override { return Compiler->GetHairClumpID(); }
	virtual int32 GetHairTangent(bool bUseTangentSpace) override { return Compiler->GetHairTangent(bUseTangentSpace); }
	virtual int32 GetHairRootUV() override { return Compiler->GetHairRootUV(); }
	virtual int32 GetHairBaseColor() override { return Compiler->GetHairBaseColor(); }
	virtual int32 GetHairRoughness() override { return Compiler->GetHairRoughness(); }
	virtual int32 GetHairAO() override { return Compiler->GetHairAO(); }
	virtual int32 GetHairDepth() override { return Compiler->GetHairDepth(); }
	virtual int32 GetHairCoverage() override { return Compiler->GetHairCoverage(); }
	virtual int32 GetHairAuxilaryData() override { return Compiler->GetHairAuxilaryData(); }
	virtual int32 GetHairAtlasUVs() override { return Compiler->GetHairAtlasUVs(); }
	virtual int32 GetHairGroupIndex() override { return Compiler->GetHairGroupIndex(); }
	virtual int32 GetHairColorFromMelanin(int32 Melanin, int32 Redness, int32 DyeColor) override { return Compiler->GetHairColorFromMelanin(Melanin, Redness, DyeColor); }

	virtual int32 RotateScaleOffsetTexCoords(int32 TexCoordCodeIndex, int32 RotationScale, int32 Offset) override
	{
		return Compiler->RotateScaleOffsetTexCoords(TexCoordCodeIndex, RotationScale, Offset);
	}

	virtual int32 SpeedTree(int32 GeometryArg, int32 WindArg, int32 LODArg, float BillboardThreshold, bool bAccurateWindVelocities, bool bExtraBend, int32 ExtraBendArg) override
	{
		return Compiler->SpeedTree(GeometryArg, WindArg, LODArg, BillboardThreshold, bAccurateWindVelocities, bExtraBend, ExtraBendArg);
	}

	virtual int32 AtmosphericFogColor(int32 WorldPosition, EPositionOrigin PositionOrigin) override
	{
		return Compiler->AtmosphericFogColor(WorldPosition, PositionOrigin);
	}

	virtual int32 AtmosphericLightVector() override
	{
		return Compiler->AtmosphericLightVector();
	}

	virtual int32 AtmosphericLightColor() override
	{
		return Compiler->AtmosphericLightColor();
	}

	virtual int32 SkyAtmosphereLightIlluminance(int32 WorldPosition, EPositionOrigin PositionOrigin, int32 LightIndex) override
	{
		return Compiler->SkyAtmosphereLightIlluminance(WorldPosition, PositionOrigin, LightIndex);
	}

	virtual int32 SkyAtmosphereLightIlluminanceOnGround(int32 LightIndex) override
	{
		return Compiler->SkyAtmosphereLightIlluminanceOnGround(LightIndex);
	}

	virtual int32 SkyAtmosphereLightDirection(int32 LightIndex) override
	{
		return Compiler->SkyAtmosphereLightDirection(LightIndex);
	}

	virtual int32 SkyAtmosphereLightDiskLuminance(int32 LightIndex, int32 OverrideAtmosphereLightDiscCosHalfApexAngle) override
	{
		return Compiler->SkyAtmosphereLightDiskLuminance(LightIndex, OverrideAtmosphereLightDiscCosHalfApexAngle);
	}

	virtual int32 SkyAtmosphereViewLuminance(int32 WorldDirectionOverrideCodeChunk) override
	{
		return Compiler->SkyAtmosphereViewLuminance(WorldDirectionOverrideCodeChunk);
	}

	virtual int32 SkyAtmosphereAerialPerspective(int32 WorldPosition, EPositionOrigin PositionOrigin) override
	{
		return Compiler->SkyAtmosphereAerialPerspective(WorldPosition, PositionOrigin);
	}

	virtual int32 SkyAtmosphereDistantLightScatteredLuminance() override
	{
		return Compiler->SkyAtmosphereDistantLightScatteredLuminance();
	}

	virtual int32 SkyLightEnvMapSample(int32 DirectionCodeChunk, int32 RoughnessCodeChunk) override
	{
		return Compiler->SkyLightEnvMapSample(DirectionCodeChunk, RoughnessCodeChunk);
	}

	virtual int32 MainDirectionalLightIlluminance() override
	{
		return Compiler->MainDirectionalLightIlluminance();
	}

	virtual int32 MainDirectionalLightDirection() override
	{
		return Compiler->MainDirectionalLightDirection();
	}

	virtual int32 GetCloudSampleAltitude() override
	{
		return Compiler->GetCloudSampleAltitude();
	}

	virtual int32 GetCloudSampleAltitudeInLayer() override
	{
		return Compiler->GetCloudSampleAltitudeInLayer();
	}

	virtual int32 GetCloudSampleNormAltitudeInLayer() override
	{
		return Compiler->GetCloudSampleNormAltitudeInLayer();
	}

	virtual int32 GetCloudSampleShadowSampleDistance() override
	{
		return Compiler->GetCloudSampleShadowSampleDistance();
	}

	virtual int32 GetVolumeSampleConservativeDensity() override
	{
		return Compiler->GetVolumeSampleConservativeDensity();
	}

	virtual int32 GetCloudEmptySpaceSkippingSphereCenterWorldPosition() override
	{
		return Compiler->GetCloudEmptySpaceSkippingSphereCenterWorldPosition();
	}

	virtual int32 GetCloudEmptySpaceSkippingSphereRadius() override
	{
		return Compiler->GetCloudEmptySpaceSkippingSphereRadius();
	}

	virtual int32 SceneDepthWithoutWater(int32 Offset, int32 ViewportUV, bool bUseOffset, float FallbackDepth) override
	{
		return Compiler->SceneDepthWithoutWater(Offset, ViewportUV, bUseOffset, FallbackDepth);
	}

	virtual int32 CustomPrimitiveData(int32 OutputIndex, EMaterialValueType Type) override
	{
		return Compiler->CustomPrimitiveData(OutputIndex, Type);
	}

	virtual int32 ShadingModel(EMaterialShadingModel InSelectedShadingModel) override
	{
		return Compiler->ShadingModel(InSelectedShadingModel);
	}

	virtual int32 MapARPassthroughCameraUV(int32 UV) override
	{
		return Compiler->MapARPassthroughCameraUV(UV);
	}

	virtual int32 EyeAdaptation() override
	{
		return Compiler->EyeAdaptation();
	}

	virtual int32 EyeAdaptationInverse(int32 LightValueArg, int32 AlphaArg) override
	{
		return Compiler->EyeAdaptationInverse(LightValueArg, AlphaArg);
	}

	virtual bool IsDevelopmentFeatureEnabled(const FName& FeatureName) const override
	{
		return Compiler->IsDevelopmentFeatureEnabled(FeatureName);
	}

	virtual int32 DefaultMaterialAttributes() override
	{
		return Compiler->DefaultMaterialAttributes();
	}

	virtual int32 SetMaterialAttribute(int32 MaterialAttributes, int32 Value, const FGuid& AttributeID)
	{
		return Compiler->SetMaterialAttribute(MaterialAttributes, Value, AttributeID);
	}

	virtual int32 BeginScope() override
	{
		return Compiler->BeginScope();
	}

	virtual int32 BeginScope_If(int32 Condition) override
	{
		return Compiler->BeginScope_If(Condition);
	}

	virtual int32 BeginScope_Else() override
	{
		return Compiler->BeginScope_Else();
	}

	virtual int32 BeginScope_For(const UMaterialExpression* Expression, int32 StartIndex, int32 EndIndex, int32 IndexStep) override
	{
		return Compiler->BeginScope_For(Expression, StartIndex, EndIndex, IndexStep);
	}

	virtual int32 EndScope() override
	{
		return Compiler->EndScope();
	}

	virtual int32 ForLoopIndex(const UMaterialExpression* Expression) override
	{
		return Compiler->ForLoopIndex(Expression);
	}

	virtual int32 ReturnMaterialAttributes(int32 MaterialAttributes) override
	{
		return Compiler->ReturnMaterialAttributes(MaterialAttributes);
	}

	virtual int32 SetLocal(const FName& LocalName, int32 Value) override
	{
		return Compiler->SetLocal(LocalName, Value);
	}

	virtual int32 GetLocal(const FName& LocalName) override
	{
		return Compiler->GetLocal(LocalName);
	}

	virtual int32 NeuralOutput(int32 ViewportUV, uint32 NeuralIndexType) override
	{
		return Compiler->NeuralOutput(ViewportUV, NeuralIndexType);
	}

	virtual int32 SubstrateCreateAndRegisterNullMaterial() override
	{
		return Compiler->SubstrateCreateAndRegisterNullMaterial();
	}

	virtual int32 SubstrateSlabBSDF(
		int32 DiffuseAlbedo, int32 F0, int32 F90,
		int32 Roughness, int32 Anisotropy,
		int32 SSSProfileId, int32 SSSMFP, int32 SSSMFPScale, int32 SSSPhaseAniso, int32 SSSType,
		int32 EmissiveColor, 
		int32 SecondRoughness, int32 SecondRoughnessWeight, int32 SecondRoughnessAsSimpleClearCoat, int32 ClearCoatBottomNormal,
		int32 FuzzAmount, int32 FuzzColor, int32 FuzzRoughness,
		int32 Thickness,
		int32 GlintValue, int32 GlintUV,
		int32 SpecularProfileId,
		bool bIsAtTheBottomOfTopology,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateSlabBSDF(
			DiffuseAlbedo, F0, F90,
			Roughness, Anisotropy,
			SSSProfileId, SSSMFP, SSSMFPScale, SSSPhaseAniso, SSSType,
			EmissiveColor, 
			SecondRoughness, SecondRoughnessWeight, SecondRoughnessAsSimpleClearCoat, ClearCoatBottomNormal,
			FuzzAmount, FuzzColor, FuzzRoughness,
			Thickness,
			GlintValue, GlintUV,
			SpecularProfileId,
			bIsAtTheBottomOfTopology,
			Normal, Tangent, SharedLocalBasisIndexMacro,
			PromoteToOperator);
	}

	virtual int32 SubstrateConversionFromLegacy(
		bool bHasDynamicShadingModels,
		int32 BaseColor, int32 Specular, int32 Metallic,
		int32 Roughness, int32 Anisotropy,
		int32 SubSurfaceColor, int32 SubSurfaceProfileId,
		int32 ClearCoat, int32 ClearCoatRoughness,
		int32 EmissiveColor,
		int32 Opacity,
		int32 ThinTranslucentTransmittanceColor,
		int32 ThinTranslucentSurfaceCoverage,
		int32 WaterScatteringCoefficients, int32 WaterAbsorptionCoefficients, int32 WaterPhaseG, int32 ColorScaleBehindWater,
		int32 ShadingModel,
		int32 Normal, int32 Tangent, const FString& SharedLocalBasisIndexMacro,
		int32 ClearCoat_Normal, int32 ClearCoat_Tangent, const FString& ClearCoat_SharedLocalBasisIndexMacro,
		int32 CustomTangent_Tangent,
		FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateConversionFromLegacy(
			bHasDynamicShadingModels,
			BaseColor, Specular, Metallic,
			Roughness, Anisotropy,
			SubSurfaceColor, SubSurfaceProfileId,
			ClearCoat, ClearCoatRoughness,
			EmissiveColor,
			Opacity,
			ThinTranslucentTransmittanceColor,
			ThinTranslucentSurfaceCoverage,
			WaterScatteringCoefficients, WaterAbsorptionCoefficients, WaterPhaseG, ColorScaleBehindWater,
			ShadingModel,
			Normal, Tangent, SharedLocalBasisIndexMacro,
			ClearCoat_Normal, ClearCoat_Tangent, ClearCoat_SharedLocalBasisIndexMacro,
			CustomTangent_Tangent,
			PromoteToOperator);
	}

	virtual int32 SubstrateVolumetricFogCloudBSDF(int32 Albedo, int32 Extinction, int32 EmissiveColor, int32 AmbientOcclusion, bool bEmissiveOnly) override
	{
		return Compiler->SubstrateVolumetricFogCloudBSDF(Albedo, Extinction, EmissiveColor, AmbientOcclusion, bEmissiveOnly);
	}

	virtual int32 SubstrateUnlitBSDF(int32 EmissiveColor, int32 TransmittanceColor, int32 Normal, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateUnlitBSDF(EmissiveColor, TransmittanceColor, Normal, PromoteToOperator);
	}

	virtual int32 SubstrateUIBSDF(int32 EmissiveColor, int32 Opacity, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateUIBSDF(EmissiveColor, Opacity, PromoteToOperator);
	}

	virtual int32 SubstrateHairBSDF(int32 BaseColor, int32 Scatter, int32 Specular, int32 Roughness, int32 Backlit, int32 EmissiveColor, int32 Tangent, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateHairBSDF(BaseColor, Scatter, Specular, Roughness, Backlit, EmissiveColor, Tangent, SharedLocalBasisIndexMacro, PromoteToOperator);
	}

	virtual int32 SubstrateEyeBSDF(int32 DiffuseAlbedo, int32 Roughness, int32 IrisMask, int32 IrisDistance, int32 IrisNormal, int32 IrisPlaneNormal, int32 SSSProfileId, int32 EmissiveColor, int32 CorneaNormal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateEyeBSDF(DiffuseAlbedo, Roughness, IrisMask, IrisDistance, IrisNormal, IrisPlaneNormal, SSSProfileId, EmissiveColor, CorneaNormal, SharedLocalBasisIndexMacro, PromoteToOperator);
	}

	virtual int32 SubstrateSingleLayerWaterBSDF(
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness, int32 EmissiveColor, int32 TopMaterialOpacity, 
		int32 WaterAlbedo, int32 WaterExtinction, int32 WaterPhaseG, int32 ColorScaleBehindWater, int32 Normal, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateSingleLayerWaterBSDF(
			BaseColor, Metallic, Specular, Roughness, EmissiveColor, TopMaterialOpacity, 
			WaterAlbedo, WaterExtinction, WaterPhaseG, ColorScaleBehindWater, Normal, SharedLocalBasisIndexMacro, PromoteToOperator);
	}

	virtual int32 SubstrateToonBSDF(
		int32 ToonProfileId,
		int32 BaseColor, int32 Metallic, int32 Specular, int32 Roughness,
		int32 EmissiveColor, int32 Normal,
		int32 PatternUVs,
		int32 Anisotropy, int32 Tangent,
		const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateToonBSDF(
			ToonProfileId,
			BaseColor, Metallic, Specular, Roughness,
			EmissiveColor, Normal,
			PatternUVs,
			Anisotropy, Tangent,
			SharedLocalBasisIndexMacro, PromoteToOperator);
	}

	virtual int32 SubstrateHorizontalMixing(int32 Background, int32 Foreground, int32 Mix, int OperatorIndex, uint32 MaxDistanceFromLeaves) override
	{
		return Compiler->SubstrateHorizontalMixing(Background, Foreground, Mix, OperatorIndex, MaxDistanceFromLeaves);
	}

	virtual int32 SubstrateHorizontalMixingParameterBlending(int32 Background, int32 Foreground, int32 HorizontalMixCodeChunk, int32 NormalMixCodeChunk, const FString& SharedLocalBasisIndexMacro, int32 BackgroundBSDFNormalCodeChunk, int32 ForegroundBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateHorizontalMixingParameterBlending(Background, Foreground, HorizontalMixCodeChunk, NormalMixCodeChunk, SharedLocalBasisIndexMacro, BackgroundBSDFNormalCodeChunk, ForegroundBSDFNormalCodeChunk, PromoteToOperator);
	}

	virtual int32 SubstrateVerticalLayering(int32 Top, int32 Base, int32 Thickness, int OperatorIndex, uint32 MaxDistanceFromLeaves) override
	{
		return Compiler->SubstrateVerticalLayering(Top, Base, Thickness, OperatorIndex, MaxDistanceFromLeaves);
	}

	virtual int32 SubstrateVerticalLayeringParameterBlending(int32 Top, int32 Base, int32 Thickness, const FString& SharedLocalBasisIndexMacro, int32 TopBSDFNormalCodeChunk, int32 BaseBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateVerticalLayeringParameterBlending(Top, Base, Thickness, SharedLocalBasisIndexMacro, TopBSDFNormalCodeChunk, BaseBSDFNormalCodeChunk, PromoteToOperator);
	}

	virtual int32 SubstrateAdd(int32 A, int32 B, int OperatorIndex, uint32 MaxDistanceFromLeaves) override
	{
		return Compiler->SubstrateAdd(A, B, OperatorIndex, MaxDistanceFromLeaves);
	}

	virtual int32 SubstrateAddParameterBlending(int32 A, int32 B, int32 AMixWeight, const FString& SharedLocalBasisIndexMacro, int32 ABSDFNormalCodeChunk, int32 BBSDFNormalCodeChunk, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateAddParameterBlending(A, B, AMixWeight, SharedLocalBasisIndexMacro, ABSDFNormalCodeChunk, BBSDFNormalCodeChunk, PromoteToOperator);
	}

	virtual int32 SubstrateWeight(int32 A, int32 Weight, int OperatorIndex, uint32 MaxDistanceFromLeaves) override
	{
		return Compiler->SubstrateWeight(A, Weight, OperatorIndex, MaxDistanceFromLeaves);
	}
	virtual int32 SubstrateThinFilm(int32 NormalCodeChunk, int32 SpecularColorCodeChunk, int32 EdgeSpecularColorCodeChunk, int32 ThicknessCodeChunk, int32 IORCodeChunk, int32 OutputIndex) override
	{
		return Compiler->SubstrateThinFilm(NormalCodeChunk, SpecularColorCodeChunk, EdgeSpecularColorCodeChunk, ThicknessCodeChunk, IORCodeChunk, OutputIndex);
	}

	virtual int32 SubstrateWeightParameterBlending(int32 A, int32 Weight, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateWeightParameterBlending(A, Weight, PromoteToOperator);
	}

	virtual int32 SubstrateTransmittanceToMFP(int32 TransmittanceColor, int32 DesiredThickness, int32 OutputIndex) override
	{
		return Compiler->SubstrateTransmittanceToMFP(TransmittanceColor, DesiredThickness, OutputIndex);
	}

	virtual int32 SubstrateSelectParameterBlending(int32 A, int32 B, int32 SelectValue, const FString& SharedLocalBasisIndexMacro, FSubstrateOperator* PromoteToOperator) override
	{
		return Compiler->SubstrateSelectParameterBlending(A, B, SelectValue, SharedLocalBasisIndexMacro, PromoteToOperator);
	}
	
	virtual int32 SubstrateMetalnessToDiffuseAlbedoF0(int32 BaseColor, int32 Specular, int32 Metallic, int32 OutputIndex) override
	{
		return Compiler->SubstrateMetalnessToDiffuseAlbedoF0(BaseColor, Specular, Metallic, OutputIndex);
	}

	virtual int32 SubstrateHazinessToSecondaryRoughness(int32 BaseRoughness, int32 Haziness, int32 OutputIndex) override
	{
		return Compiler->SubstrateHazinessToSecondaryRoughness(BaseRoughness, Haziness, OutputIndex);
	}

	virtual int32 SubstrateCompilePreview(int32 SubstrateDataCodeChunk) override
	{
		return Compiler->SubstrateCompilePreview(SubstrateDataCodeChunk);
	}

	virtual bool SubstrateSkipsOpacityEvaluation() override
	{
		return Compiler->SubstrateSkipsOpacityEvaluation();
	}

	virtual FGuid SubstrateTreeStackPush(UMaterialExpression* Expression, uint32 InputIndex) override
	{
		return Compiler->SubstrateTreeStackPush(Expression, InputIndex);
	}
	virtual FGuid SubstrateTreeStackGetPathUniqueId() override
	{
		return Compiler->SubstrateTreeStackGetPathUniqueId();
	}
	virtual FGuid SubstrateTreeStackGetParentPathUniqueId() override
	{
		return Compiler->SubstrateTreeStackGetParentPathUniqueId();
	}
	virtual void SubstrateTreeStackPop() override
	{
		Compiler->SubstrateTreeStackPop();
	}
	virtual bool GetSubstrateTreeOutOfStackDepthOccurred() override
	{
		return Compiler->GetSubstrateTreeOutOfStackDepthOccurred();
	}

	virtual int32 SubstrateThicknessStackGetThicknessIndex() override
	{
		return Compiler->SubstrateThicknessStackGetThicknessIndex();
	}
	virtual int32 SubstrateThicknessStackGetThicknessCode(int32 Index) override
	{
		return Compiler->SubstrateThicknessStackGetThicknessCode(Index);
	}
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FScalarMaterialInput* Input) override
	{
		return Compiler->SubstrateThicknessStackPush(Expression, Input);
	}
	virtual int32 SubstrateThicknessStackPush(UMaterialExpression* Expression, FExpressionInput* Input) override
	{
		return Compiler->SubstrateThicknessStackPush(Expression, Input);
	}
	virtual void SubstrateThicknessStackPop() override
	{
		Compiler->SubstrateThicknessStackPop();
	}
	virtual struct FSubstrateTranslatorDataInterface& GetSubstrateTranslatorData() override
	{
		return Compiler->GetSubstrateTranslatorData();
	}

	virtual FSubstrateOperator& SubstrateCompilationRegisterOperator(int32 OperatorType, FGuid SubstrateExpressionGuid, FGuid ChildMaterialExpressionGuid, UMaterialExpression* Parent, FGuid SubstrateParentExpressionGuid, bool bUseParameterBlending = false) override
	{
		return Compiler->SubstrateCompilationRegisterOperator(OperatorType, SubstrateExpressionGuid, ChildMaterialExpressionGuid, Parent, SubstrateParentExpressionGuid, bUseParameterBlending);
	}

	virtual FSubstrateOperator& SubstrateCompilationGetOperator(FGuid SubstrateExpressionGuid) override
	{
		return Compiler->SubstrateCompilationGetOperator(SubstrateExpressionGuid);
	}

	virtual FSubstrateOperator* SubstrateCompilationGetOperatorFromIndex(int32 OperatorIndex) override
	{
		return Compiler->SubstrateCompilationGetOperatorFromIndex(OperatorIndex);
	}


	virtual int32 SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 ACodeChunk, int32 BCodeChunk) override
	{
		return Compiler->SubstrateAddParameterBlendingBSDFCoverageToNormalMixCodeChunk(ACodeChunk, BCodeChunk);
	}

	virtual int32 SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 TopCodeChunk) override
	{
		return Compiler->SubstrateVerticalLayeringParameterBlendingBSDFCoverageToNormalMixCodeChunk(TopCodeChunk);
	}

	virtual int32 SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(int32 BackgroundCodeChunk, int32 ForegroundCodeChunk, int32 HorizontalMixCodeChunk) override
	{
		return Compiler->SubstrateHorizontalMixingParameterBlendingBSDFCoverageToNormalMixCodeChunk(BackgroundCodeChunk, ForegroundCodeChunk, HorizontalMixCodeChunk);
	}

	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk) override
	{
		return Compiler->SubstrateCompilationInfoRegisterSharedLocalBasis(NormalCodeChunk);
	}

	virtual FSubstrateRegisteredSharedLocalBasis SubstrateCompilationInfoRegisterSharedLocalBasis(int32 NormalCodeChunk, int32 TangentCodeChunk) override
	{
		return Compiler->SubstrateCompilationInfoRegisterSharedLocalBasis(NormalCodeChunk, TangentCodeChunk);
	}

	virtual FString GetSubstrateSharedLocalBasisIndexMacro(const FSubstrateRegisteredSharedLocalBasis& SharedLocalBasis) override
	{
		return Compiler->GetSubstrateSharedLocalBasisIndexMacro(SharedLocalBasis);
	}

protected:
		
	FMaterialCompiler* Compiler;
};

// Helper class to handle MaterialAttribute changes on the compiler stack
class FScopedMaterialCompilerAttribute
{
public:
	FScopedMaterialCompilerAttribute(FMaterialCompiler* InCompiler, const FGuid& InAttributeID)
	: Compiler(InCompiler)
	, AttributeID(InAttributeID)
	{
		check(Compiler);
		Compiler->PushMaterialAttribute(AttributeID);
	}

	~FScopedMaterialCompilerAttribute()
	{
		verify(AttributeID == Compiler->PopMaterialAttribute());
	}

private:
	FMaterialCompiler*	Compiler;
	FGuid				AttributeID;
};
