// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Containers/UnrealString.h"

#include "MaterialDomain.generated.h"

/** Defines the domain of a material. */
UENUM()
enum EMaterialDomain : int
{
	/** The material's attributes describe a 3d surface. */
	MD_Surface UMETA(DisplayName = "Surface"),
	/** The material's attributes describe a deferred decal, and will be mapped onto the decal's frustum. */
	MD_DeferredDecal UMETA(DisplayName = "Deferred Decal"),
	/** The material's attributes describe a light's distribution. */
	MD_LightFunction UMETA(DisplayName = "Light Function"),
	/** The material's attributes describe a 3d volume. */
	MD_Volume UMETA(DisplayName = "Volume"),
	/** The material will be used in a custom post process pass. */
	MD_PostProcess UMETA(DisplayName = "Post Process"),
	/** The material will be used for UMG or Slate UI */
	MD_UI UMETA(DisplayName = "User Interface"),
	/** The material will be used for runtime virtual texture (Deprecated). */
	MD_RuntimeVirtualTexture UMETA(Hidden),

	MD_MAX
};

ENGINE_API FString MaterialDomainString(EMaterialDomain MaterialDomain);
