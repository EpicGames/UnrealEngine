// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneTypes.h"
#include "Math/MathFwd.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"
#include "UObject/StructOpsTypeTraits.h"

struct FExpressionOutput;

namespace UE::Shader
{
enum class EValueType : uint8;
struct FValue;
}

//
//	FExpressionInput
//

//@warning: FExpressionInput is mirrored in MaterialExpression.h and manually "subclassed" in Material.h (FMaterialInput)
struct FExpressionInput
{
	/** 
	 * Material expression that this input is connected to, or NULL if not connected. 
	 * If you want to be safe when checking against dangling Reroute nodes, please use GetTracedInput before accessing this property.
	*/
	class UMaterialExpression* Expression;

	/** 
	 * Index into Expression's outputs array that this input is connected to.
	 * If you want to be safe when checking against dangling Reroute nodes, please use GetTracedInput before accessing this property.
	*/
	int32 OutputIndex;

	/** 
	 * Optional name of the input.  
	 * Note that this is the only member which is not derived from the output currently connected. 
	 */
	FName InputName;
	
	/**
	 * Input value component mask.
	 */
	int32 Mask, MaskR, MaskG, MaskB, MaskA;

	FExpressionInput()
		: OutputIndex(0)
		, Mask(0)
		, MaskR(0)
		, MaskG(0)
		, MaskB(0)
		, MaskA(0)
	{
		Expression = nullptr;
	}

	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);

#if WITH_EDITOR
	ENGINE_API int32 Compile(class FMaterialCompiler* Compiler);

	/**
	 * Tests if the input has a material expression connected to it
	 *
	 * @return	true if an expression is connected, otherwise false
	 */
	bool IsConnected() const 
	{ 
		return (nullptr != Expression);
	}

	/** Is the input connected to a constant value */
	bool IsConstant() const { return false; }

	/** Connects output of InExpression to this input */
	ENGINE_API void Connect( int32 InOutputIndex, class UMaterialExpression* InExpression );

	/** If this input goes through reroute nodes or other paths that should not affect code, trace back on the input chain.*/
	ENGINE_API FExpressionInput GetTracedInput() const;

	/** Helper for setting component mask. */
	void SetMask(int32 UseMask, int32 R, int32 G, int32 B, int32 A)
	{
		Mask = UseMask;
		MaskR = R;
		MaskG = G;
		MaskB = B;
		MaskA = A;
	}

	/** Returns the connected expression output if this input is connected to one,	nullptr otherwise. */
	FExpressionOutput* GetConnectedOutput() const;

#endif // WITH_EDITOR
};

template<>
struct TStructOpsTypeTraits<FExpressionInput>
	: public TStructOpsTypeTraitsBase2<FExpressionInput>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

//
//	FExpressionOutput
//

struct FExpressionOutput
{
	FName	OutputName;
	int32	Mask,
		MaskR,
		MaskG,
		MaskB,
		MaskA;

	FExpressionOutput(int32 InMask = 0, int32 InMaskR = 0, int32 InMaskG = 0, int32 InMaskB = 0, int32 InMaskA = 0)
		: Mask(InMask)
		, MaskR(InMaskR)
		, MaskG(InMaskG)
		, MaskB(InMaskB)
		, MaskA(InMaskA)
	{}

	FExpressionOutput(FName InOutputName, int32 InMask = 0, int32 InMaskR = 0, int32 InMaskG = 0, int32 InMaskB = 0, int32 InMaskA = 0)
		: OutputName(InOutputName)
		, Mask(InMask)
		, MaskR(InMaskR)
		, MaskG(InMaskG)
		, MaskB(InMaskB)
		, MaskA(InMaskA)
	{}

	/** Helper for setting component mask. */
	void SetMask(int32 UseMask, int32 R, int32 G, int32 B, int32 A)
	{
		Mask = UseMask;
		MaskR = R;
		MaskG = G;
		MaskB = B;
		MaskA = A;
	}
};

//
//	FMaterialInput
//

template<class InputType> struct FMaterialInput : FExpressionInput
{
	FMaterialInput()
	{
		UseConstant = 0;
		Constant = {};
	}

#if WITH_EDITOR
	bool IsConstant() const { return UseConstant; }
#endif

	uint32	UseConstant : 1;
	InputType	Constant;
};

struct FColorMaterialInput : FMaterialInput<FLinearColor>
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
	ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FColorMaterialInput>
	: public TStructOpsTypeTraitsBase2<FColorMaterialInput>
{
	enum
	{
		WithSerializer = true,
	};

	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

struct FScalarMaterialInput : FMaterialInput<float>
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
	ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FScalarMaterialInput>
	: public TStructOpsTypeTraitsBase2<FScalarMaterialInput>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

struct FShadingModelMaterialInput : FMaterialInput<uint32>
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
	ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FShadingModelMaterialInput>
	: public TStructOpsTypeTraitsBase2<FShadingModelMaterialInput>
{
	enum
	{
		WithSerializer = true,
	};
};

struct FSubstrateMaterialInput : FMaterialInput<uint32> // Still giving it a default type
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
    ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FSubstrateMaterialInput>
	: public TStructOpsTypeTraitsBase2<FSubstrateMaterialInput>
{
	enum
	{
		WithSerializer = true,
	};
};

struct FVectorMaterialInput : FMaterialInput<FVector3f>
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
	ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FVectorMaterialInput>
	: public TStructOpsTypeTraitsBase2<FVectorMaterialInput>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

struct FVector2MaterialInput : FMaterialInput<FVector2f>
{
#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, EMaterialProperty Property);
#endif  // WITH_EDITOR
	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);
	ENGINE_API void DefaultValueChanged(const FString& DefaultValue);
	ENGINE_API FString GetDefaultValue() const;
};

template<>
struct TStructOpsTypeTraits<FVector2MaterialInput>
	: public TStructOpsTypeTraitsBase2<FVector2MaterialInput>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

struct FMaterialAttributesInput : FExpressionInput
{
	FMaterialAttributesInput() 
	: PropertyConnectedMask(0)
	{ 
		// Ensure PropertyConnectedMask can contain all properties.
		static_assert((uint64)(MP_MaterialAttributes)-1 < (8 * sizeof(PropertyConnectedMask)), "PropertyConnectedMask cannot contain entire EMaterialProperty enumeration.");
	}

#if WITH_EDITOR
	ENGINE_API int32 CompileWithDefault(class FMaterialCompiler* Compiler, const FGuid& AttributeID);
	inline bool IsConnected(EMaterialProperty Property) { return ((PropertyConnectedMask >> (uint64)Property) & 0x1) != 0; }
	inline bool IsConnected() const { return FExpressionInput::IsConnected(); }
	inline void SetConnectedProperty(EMaterialProperty Property, bool bIsConnected)
	{
		PropertyConnectedMask = bIsConnected ? PropertyConnectedMask | (1ull << (uint64)Property) : PropertyConnectedMask & ~(1ull << (uint64)Property);
	}
#endif  // WITH_EDITOR

	/** ICPPStructOps interface */
	ENGINE_API bool Serialize(FArchive& Ar);

	// Each bit corresponds to EMaterialProperty connection status.
	uint64 PropertyConnectedMask;
};

template<>
struct TStructOpsTypeTraits<FMaterialAttributesInput>
	: public TStructOpsTypeTraitsBase2<FMaterialAttributesInput>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
