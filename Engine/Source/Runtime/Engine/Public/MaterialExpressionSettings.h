// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/TopLevelAssetPath.h"

#if WITH_EDITOR

/**
 * Singleton for the material expression settings and permissions
 */
class FMaterialExpressionSettings
{
public:
	/** Gets singleton instance */
	static ENGINE_API FMaterialExpressionSettings* Get();

	/** Delegate to filter class paths from permissions lists */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsClassPathAllowed, const FTopLevelAssetPath& /*InClassPath*/);

	ENGINE_API void RegisterIsClassPathAllowedDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate);
	ENGINE_API void UnregisterIsClassPathAllowedDelegate(const FName OwnerName);
	ENGINE_API bool IsClassPathAllowed(const FTopLevelAssetPath& InClassPath) const;
	ENGINE_API bool HasClassPathFiltering() const;

private:
	FMaterialExpressionSettings() = default;
	~FMaterialExpressionSettings() = default;

	/** Delegates called to determine whether a class type is allowed to be processed in the material translator */
	TMap<FName, FOnIsClassPathAllowed> IsClassPathAllowedDelegates;
};

#endif // WITH_EDITOR

/**
 * Returns whether the specified class of material expression is permitted.
 * For instance, custom expressions are not permitted in certain UE editor configurations for client generated materials.
 */
bool IsExpressionClassPermitted(const UClass* const Class);
