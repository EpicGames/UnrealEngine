// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "MaterialKeyIncludeEnum.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/StringBuilder.h"
#include "RHIShaderPlatform.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

/**
 * Output class passed to RecordAndEmit functions for material shader data. It receives function calls that either
 * save/load variables to compact binary or emits those variables to an FShaderKeyGenerator to construct the key.
 * It is written to by the function RecordOrEmitMaterialShaderMapKey and its helper functions, to create the
 * shadermap's DDC key, to hash the Material's cook dependencies for incremental cooks, and to save/load the inputs
 * for those cook dependencies to cook metadata, with a single function definition that lists all the relevant
 * variables. This reduces the amount of boilerplate that has to be written and maintained for each variable that
 * can affect the DDC key and cook dependencies.
 * 
 * The Context additionally holds flags for which portion of data should be included in its output; these flags
 * are set differently by the calling function depending on whether we are saving a DDC key or material dependencies,
 * and on what kind of DDC key we are saving.
 * 
 * The Context additionally holds some common arguments - ShaderPlatform and ShaderFormat - that are provided to the
 * functions from the calling functions.
 */
class FMaterialKeyGeneratorContext
{
public:
	/** Which mode the GeneratorContext is operating in; these modes are mutually exclusive, not flags. */
	enum class EMode : uint8
	{
		Emitting,
		Saving,
		Loading,
	};

public:
	/**
	 * Construct a Context that is emitting variables into a binary-formatted shader DDC key or into a material
	 * dependencies hash function.
	 */
	ENGINE_API FMaterialKeyGeneratorContext(TUniqueFunction<void(const void* Data, uint64 Size)>&& HashFunction,
		EShaderPlatform InShaderPlatform);
	/** Construct a Context that is emitting variables into a text-formatted shader DDC key. */
	ENGINE_API FMaterialKeyGeneratorContext(FString& InResultString, EShaderPlatform InShaderPlatform);
	/** Construct a Context that is saving data to compact binary. */
	ENGINE_API FMaterialKeyGeneratorContext(FCbWriter& InWriter, EShaderPlatform InShaderPlatform);
	/** Construct a Context that is loading data from compact binary. */
	ENGINE_API FMaterialKeyGeneratorContext(FCbObjectView LoadRoot, EShaderPlatform InShaderPlatform);
	ENGINE_API ~FMaterialKeyGeneratorContext();

	/** Add include flags. @see EMaterialKeyInclude. */
	inline void AddFlags(EMaterialKeyInclude Flags);
	/** Remove include flags. @see EMaterialKeyInclude. */
	inline void RemoveFlags(EMaterialKeyInclude Flags);
	/** Add or remove include flags, depending on the value of bIncluded. @see EMaterialKeyInclude. */
	inline void SetFlags(EMaterialKeyInclude Flags, bool bIncluded);
	/** Report whether all requested flags are included. @see EMaterialKeyInclude. */
	inline bool HasAllFlags(EMaterialKeyInclude Flags) const;
	/** Return the list of include flags. @see EMaterialKeyInclude. */
	inline EMaterialKeyInclude GetFlags() const;

	/** Return the mode the Context is operating in. See also the Is functions that returns true when in each mode. */
	inline EMode GetMode() const;
	/** True iff the Context is in emit mode - creating DDC key or hashing material dependencies. */
	inline bool IsEmitting() const;
	/** True iff the Context is in either IsSaving or IsLoading mode. */
	inline bool IsRecording() const;
	/** True iff the Context is saving to compact binary. */
	inline bool IsSaving() const;
	/** True iff the Context is loading from compact binary. */
	inline bool IsLoading() const;
	/** The ShaderPlatform provided by the caller. */
	inline EShaderPlatform GetShaderPlatform() const;
	/** The ShaderFormat that corresponds to the ShaderPlatform provided by the caller. */
	inline FName GetShaderFormat() const;

	/**
	 * Output function used by all three modes. The given data is either saved, loaded, or emitted. If saving or
	 * loading, the Name is used as the id for the data. If emitting the Name is not used.
	 * The name is scoped by the object level defined by RecordObjectStart/RecordObjectEnd.
	 */
	template <typename T>
	inline void RecordAndEmit(FUtf8StringView Name, T&& Data);

	/**
	 * When saving or loading, save the given Data with the given name. When emitting, this function is a noop.
	 * The name is scoped by the object level defined by RecordObjectStart/RecordObjectEnd.
	 */
	template <typename T>
	inline void Record(FUtf8StringView Name, T&& Data);
	/**
	 * When saving or loading, start a new object scope with the given name. When emitting, this function is a noop.
	 * The name is scoped in the parent object level defined by previous calls to RecordObjectStart/RecordObjectEnd.
	 */
	ENGINE_API void RecordObjectStart(FUtf8StringView Name);
	/**
	 * When saving or loading, end the object scope started by the last call to RecordObjectStart, and return to the
	 * parent object scope. When emitting, this function is a noop.
	 */
	ENGINE_API void RecordObjectEnd();
	/** When loading, execute the given function. In all other modes, this function is a noop. */
	inline void PostLoad(TFunctionRef<void()> Action);

	/**
	 * When emitting, append the given Data to the ShaderKeyGenerator, using operator<<. In all other modes, this
	 * function is a noop.
	 */
	template <typename T>
	inline void Emit(T&& Data);
	/** When emitting, call Callback with the ShaderKeyGenerator. In all other modes, this function is a noop. */
	inline void EmitFunc(TFunctionRef<void(FShaderKeyGenerator&)> Callback);
	/**  When emitting, call the ShaderKeyGenerator's AppendDebugText. In all other modes, this function is a noop. */
	inline void EmitDebugText(FStringView Data);
	/**  When emitting, call the ShaderKeyGenerator's AppendSeparator. In all other modes, this function is a noop. */
	inline void EmitSeparator();
	/**  When emitting, call the ShaderKeyGenerator's AppendBoolInt. In all other modes, this function is a noop. */
	inline void EmitBoolInt(bool Data);
	/** When emitting, return pointer to the ShaderKeyGenerator. In all other modes, returns nullptr. */
	inline FShaderKeyGenerator* GetKeyGenIfEmitting();
	/** Must only be called when emitting; fails assertion if not. Returns reference to the ShaderKeyGenerator. */
	inline FShaderKeyGenerator& GetKeyGenIfEmittingChecked();

	/** When loading, report whether any Record function reported an error. In all other modes, always returns true. */
	inline bool HasLoadError() const;

private:
	FCbObjectView GetCurrentObject();

private:
	union
	{
		FShaderKeyGenerator KeyGen;
		TArray<FCbObjectView, TInlineAllocator<2>> ObjectStack;
		FCbWriter* Writer;
	};
	FName ShaderFormat;
	EShaderPlatform ShaderPlatform = EShaderPlatform::SP_NumPlatforms;
	EMode Mode = EMode::Emitting;
	EMaterialKeyInclude IncludeFlags = EMaterialKeyInclude::All;
	bool bHasLoadError = false;
};

ENUM_CLASS_FLAGS(EMaterialKeyInclude);

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline void FMaterialKeyGeneratorContext::AddFlags(EMaterialKeyInclude Flags)
{
	EnumAddFlags(IncludeFlags, Flags);
}

inline void FMaterialKeyGeneratorContext::RemoveFlags(EMaterialKeyInclude Flags)
{
	EnumRemoveFlags(IncludeFlags, Flags);
}

inline void FMaterialKeyGeneratorContext::SetFlags(EMaterialKeyInclude Flags, bool bIncluded)
{
	if (bIncluded)
	{
		AddFlags(Flags);
	}
	else
	{
		RemoveFlags(Flags);
	}
}

inline bool FMaterialKeyGeneratorContext::HasAllFlags(EMaterialKeyInclude Flags) const
{
	return EnumHasAllFlags(IncludeFlags, Flags);
}

inline EMaterialKeyInclude FMaterialKeyGeneratorContext::GetFlags() const
{
	return IncludeFlags;
}

FMaterialKeyGeneratorContext::EMode FMaterialKeyGeneratorContext::GetMode() const
{
	return Mode;
}

inline bool FMaterialKeyGeneratorContext::IsEmitting() const
{
	return Mode == EMode::Emitting;
}

inline bool FMaterialKeyGeneratorContext::IsRecording() const
{
	return (Mode == EMode::Saving) | (Mode == EMode::Loading);
}

inline bool FMaterialKeyGeneratorContext::IsSaving() const
{
	return Mode == EMode::Saving;
}

inline bool FMaterialKeyGeneratorContext::IsLoading() const
{
	return Mode == EMode::Loading;
}

inline EShaderPlatform FMaterialKeyGeneratorContext::GetShaderPlatform() const
{
	return ShaderPlatform;
}

inline FName FMaterialKeyGeneratorContext::GetShaderFormat() const
{
	return ShaderFormat;
}

template <typename T>
inline void FMaterialKeyGeneratorContext::RecordAndEmit(FUtf8StringView Name, T&& Data)
{
	Record(Name, Data);
	Emit(Data);
}

template <typename T>
inline void FMaterialKeyGeneratorContext::Record(FUtf8StringView Name, T&& Data)
{
	switch (Mode)
	{
	case EMode::Emitting:
		break;
	case EMode::Saving:
		*Writer << Name << Data;
		break;
	case EMode::Loading:
		if (!LoadFromCompactBinary(GetCurrentObject()[Name], Data))
		{
			bHasLoadError = true;
		}
		break;
	default:
		checkNoEntry();
		break;
	}
}

inline void FMaterialKeyGeneratorContext::PostLoad(TFunctionRef<void()> Action)
{
	if (Mode == EMode::Loading)
	{
		Action();
	}
}

template <typename T>
inline void FMaterialKeyGeneratorContext::Emit(T&& Data)
{
	if (Mode == EMode::Emitting)
	{
		KeyGen << Data;
	}
}

inline void FMaterialKeyGeneratorContext::EmitFunc(TFunctionRef<void(FShaderKeyGenerator&)> Callback)
{
	if (Mode == EMode::Emitting)
	{
		Callback(KeyGen);
	}
}

inline void FMaterialKeyGeneratorContext::EmitDebugText(FStringView Data)
{
	if (Mode == EMode::Emitting)
	{
		KeyGen.AppendDebugText(Data);
	}
}

inline void FMaterialKeyGeneratorContext::EmitSeparator()
{
	if (Mode == EMode::Emitting)
	{
		KeyGen.AppendSeparator();
	}
}

inline void FMaterialKeyGeneratorContext::EmitBoolInt(bool Data)
{
	if (Mode == EMode::Emitting)
	{
		KeyGen.AppendBoolInt(Data);
	}
}

inline FShaderKeyGenerator* FMaterialKeyGeneratorContext::GetKeyGenIfEmitting()
{
	if (Mode == EMode::Emitting)
	{
		return &KeyGen;
	}
	return nullptr;
}

inline FShaderKeyGenerator& FMaterialKeyGeneratorContext::GetKeyGenIfEmittingChecked()
{
	check(Mode == EMode::Emitting);
	return KeyGen;
}

inline bool FMaterialKeyGeneratorContext::HasLoadError() const
{
	return Mode != EMode::Loading || bHasLoadError;
}

#endif