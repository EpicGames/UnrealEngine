// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/**
 * Categories of fields which should be included in a ShaderMapId or in the output of a FMaterialKeyGeneratorContext.
 * GetShaderMapId, and code recording or emitting fields to a FMaterialKeyGeneratorContext, calls HasAllFlags for the
 * flags relevant to a given field before writing it to the ShaderMapId's ShaderTypes or to the
 * FMaterialKeyGeneratorContext's RecordAndEmit functions.
 */
enum class EMaterialKeyInclude : uint32
{
	/** shadertype, shaderpipelinetype, vertexfactortype dependencies are included in the output */
	ShaderDependencies = 0x1,
	/** shader hlsl file hashes are included in the output */
	SourceAndMaterialState = 0x2,
	/**
	 * Global data that applies to many or all Materials is included in the output.
	 * For calculating the ShaderMap Id, this includes
	 *    The ShaderTypes, ShaderPipelineTypes, and VertexFactoryTypes in the Material's FMaterialShaderMapLayout.
	 * For the output of FMaterialKeyGeneratorContext this includes globals (cvars, project settings) that impact
	 *     The Material's FMaterialShaderMapLayout via GetShaderTypeLayoutHash.
	 *     The ShaderPlatform being emitted via ShaderMapAppendKey.
	 *     All Materials via FMaterialAttributeDefinitionMap::AppendDDCKey.
	 */
	Globals = 0x4,
	/** data stored in UObject exports in .uasset and .umap files is included in the output */
	UObjectData = 0x8,

	All = ShaderDependencies | SourceAndMaterialState | Globals | UObjectData,
};
