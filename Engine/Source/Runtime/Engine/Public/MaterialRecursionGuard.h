// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShared.h: Shared material definitions.
=============================================================================*/

#pragma once

template<typename T>
struct TMaterialRecursionGuard
{
	inline TMaterialRecursionGuard() = default;
	inline TMaterialRecursionGuard(const TMaterialRecursionGuard& Parent)
		: Value(nullptr)
		, PreviousLink(&Parent)
	{
	}

	inline void Set(const T* InValue)
	{
		check(Value == nullptr);
		Value = InValue;
	}

	inline bool Contains(const T* InValue)
	{
		TMaterialRecursionGuard const* Link = this;
		do
		{
			if (Link->Value == InValue)
			{
				return true;
			}
			Link = Link->PreviousLink;
		} while (Link);
		return false;
	}

private:
	const T* Value = nullptr;
	TMaterialRecursionGuard const* PreviousLink = nullptr;
};
