// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialSceneTextureId.generated.h"

/** like EPassInputId but can expose more e.g. GBuffer */
UENUM()
enum ESceneTextureId : int
{
	/** Scene color, normal post process passes should use PostProcessInput0 */
	PPI_SceneColor UMETA(DisplayName="SceneColor"),
	/** Scene depth, single channel, contains the linear depth of the opaque objects */
	PPI_SceneDepth UMETA(DisplayName="SceneDepth"),
	/** Material diffuse, RGB color (computed from GBuffer) */
	PPI_DiffuseColor UMETA(DisplayName="DiffuseColor"),
	/** Material specular, RGB color (computed from GBuffer) */
	PPI_SpecularColor UMETA(DisplayName="SpecularColor"),
	/** Material subsurface, RGB color (GBuffer, only for some ShadingModels) */
	PPI_SubsurfaceColor UMETA(DisplayName="SubsurfaceColor"),
	/** Material base, RGB color (GBuffer), can be modified on read by the ShadingModel, consider StoredBasedColor */
	PPI_BaseColor UMETA(DisplayName="BaseColor (for lighting)"),
	/** Material specular, single channel (GBuffer), can be modified on read by the ShadingModel, consider StoredSpecular */
	PPI_Specular UMETA(DisplayName="Specular (for lighting)"),
	/** Material metallic, single channel (GBuffer) */
	PPI_Metallic UMETA(DisplayName="Metallic"),
	/** Normal, RGB in -1..1 range, not normalized (GBuffer) */
	PPI_WorldNormal UMETA(DisplayName="WorldNormal"),
	/** Not yet supported */
	PPI_SeparateTranslucency UMETA(DisplayName="SeparateTranslucency"),
	/** Material opacity, single channel (GBuffer) */
	PPI_Opacity UMETA(DisplayName="Opacity"),
	/** Material roughness, single channel (GBuffer) */
	PPI_Roughness UMETA(DisplayName="Roughness"),
	/** Material ambient occlusion, single channel (GBuffer) */
	PPI_MaterialAO UMETA(DisplayName="MaterialAO"),
	/** Scene depth, single channel, contains the linear depth of the opaque objects rendered with CustomDepth (mesh property) */
	PPI_CustomDepth UMETA(DisplayName="CustomDepth"),
	/** Input #0 of this postprocess pass, usually the only one hooked up */
	PPI_PostProcessInput0 UMETA(DisplayName="PostProcessInput0"),
	/** Input #1 of this postprocess pass, usually not used */
	PPI_PostProcessInput1 UMETA(DisplayName="PostProcessInput1"),
	/** Input #2 of this postprocess pass, usually not used */
	PPI_PostProcessInput2 UMETA(DisplayName="PostProcessInput2"),
	/** Input #3 of this postprocess pass, usually not used */
	PPI_PostProcessInput3 UMETA(DisplayName="PostProcessInput3"),
	/** Input #4 of this postprocess pass, usually not used */
	PPI_PostProcessInput4 UMETA(DisplayName="PostProcessInput4"),
	/** Input #5 of this postprocess pass, usually not used */
	PPI_PostProcessInput5 UMETA(DisplayName="PostProcessInput5"),
	/** Input #6 of this postprocess pass, usually not used */
	PPI_PostProcessInput6 UMETA(DisplayName="PostProcessInput6"),
	/** Decal Mask, single bit (was moved to stencil for better performance, not accessible at the moment) */
	PPI_DecalMask UMETA(DisplayName="Decal Mask"),
	/** Shading model */
	PPI_ShadingModelColor UMETA(DisplayName="Shading Model Color"),
	/** Shading model ID */
	PPI_ShadingModelID UMETA(DisplayName="Shading Model ID"),
	/** Ambient Occlusion, single channel */
	PPI_AmbientOcclusion UMETA(DisplayName="Ambient Occlusion"),
	/** Scene stencil, contains CustomStencil mesh property of the opaque objects rendered with CustomDepth */
	PPI_CustomStencil UMETA(DisplayName="CustomStencil"),
	/** Material base, RGB color (GBuffer) */
	PPI_StoredBaseColor UMETA(DisplayName="BaseColor (as stored in GBuffer)"),
	/** Material specular, single channel (GBuffer) */
	PPI_StoredSpecular UMETA(DisplayName="Specular (as stored in GBuffer)"),
	/** Scene Velocity */
	PPI_Velocity UMETA(DisplayName="Velocity"),
	/** Tangent, RGB in -1..1 range, not normalized (GBuffer) */
	PPI_WorldTangent UMETA(DisplayName = "WorldTangent"),
	/** Material anisotropy, single channel (GBuffer) */
	PPI_Anisotropy UMETA(DisplayName = "Anisotropy"),
	/** Whether the pixel is marked as belonging to an opaque first person primitive, single channel (GBuffer) */
	PPI_IsFirstPerson UMETA(DisplayName = "IsFirstPerson"),

	/** Used internally for user scene texture inputs.  Later remapped to PPI_PostProcessInput0-6 in the shader via procedurally generated defines. */
	PPI_UserSceneTexture0 = 32 UMETA(Hidden),
	PPI_UserSceneTexture1 UMETA(Hidden),
	PPI_UserSceneTexture2 UMETA(Hidden),
	PPI_UserSceneTexture3 UMETA(Hidden),
	PPI_UserSceneTexture4 UMETA(Hidden),
	PPI_UserSceneTexture5 UMETA(Hidden),
	PPI_UserSceneTexture6 UMETA(Hidden),
};
