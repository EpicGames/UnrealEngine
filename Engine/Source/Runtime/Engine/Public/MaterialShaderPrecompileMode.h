// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

enum class EMaterialShaderPrecompileMode
{
	None,
	Background,
	Synchronous,

	Default = Background,
};
