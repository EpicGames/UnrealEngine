// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShader.h: Material shader definitions.
=============================================================================*/

#pragma once

#include "Shader.h"
#include "GlobalShader.h"

/** A macro to implement material shaders. */
#define IMPLEMENT_MATERIAL_SHADER_TYPE(TemplatePrefix,ShaderClass,SourceFilename,FunctionName,Frequency) \
	IMPLEMENT_SHADER_TYPE( \
		TemplatePrefix, \
		ShaderClass, \
		SourceFilename, \
		FunctionName, \
		Frequency \
		);

class FMaterial;
class FMaterialShaderMap;
class FMaterialShaderMapId;
class FShaderCommonCompileJob;
class FShaderCompileJob;
class FShaderKeyGenerator;
class FUniformExpressionSet;
class FVertexFactoryType;
struct FMaterialShaderParameters;
struct FConservativeShaderParameters;
struct FMaterialShadingModelField;
enum EBlendMode : int;
enum EMaterialShadingModel : int;
enum class EShaderCompileJobPriority : uint8;
class FMaterialCompilationOutput;
#if WITH_EDITOR
class FMaterialKeyGeneratorContext;
#endif


DECLARE_DELEGATE_RetVal_OneParam(FString, FShadingModelToStringDelegate, EMaterialShadingModel)

/** Converts an EMaterialShadingModel to a string description. */
extern ENGINE_API FString GetShadingModelString(EMaterialShadingModel ShadingModel);

/** Converts an FMaterialShadingModelField to a string description, base on the passed in delegate. */
extern ENGINE_API FString GetShadingModelFieldString(FMaterialShadingModelField ShadingModels, const FShadingModelToStringDelegate& Delegate, const FString& Delimiter = " ");

/** Converts an FMaterialShadingModelField to a string description, base on a default function. */
extern ENGINE_API FString GetShadingModelFieldString(FMaterialShadingModelField ShadingModels);

/** Converts an EBlendMode to a string description. */
extern ENGINE_API FString GetBlendModeString(EBlendMode BlendMode);

#if WITH_EDITOR
extern ENGINE_API FString GetMaterialShaderMapKeyString(
	const FMaterialShaderMapId& ShaderMapId,
	const FConservativeShaderParameters& ShaderParameters,
	EShaderPlatform Platform,
	bool bIncludeKeyStringShaderDependencies = true);

extern ENGINE_API void RecordOrEmitMaterialShaderMapKey(
	FMaterialKeyGeneratorContext& Context,
	FMaterialShaderMapId& ShaderMapId,
	FConservativeShaderParameters& ShaderParameters);

// Alternate arguments for RecordOrEmitMaterialShaderMapKey: Support being called with const&
// when emitting or saving.
extern ENGINE_API void RecordOrEmitMaterialShaderMapKey(
	FMaterialKeyGeneratorContext& Context,
	const FMaterialShaderMapId& ShaderMapId,
	const FConservativeShaderParameters& ShaderParameters);
#endif

/** Called for every material shader to update the appropriate stats. */
extern void UpdateMaterialShaderCompilingStats(const FMaterial* Material);

/**
 * Dump material stats for a given platform.
 * 
 * @param	Platform	Platform to dump stats for.
 */
extern ENGINE_API void DumpMaterialStats( EShaderPlatform Platform );

/**
 * A shader meta type for material-linked shaders.
 */
class FMaterialShaderType : public FShaderType
{
public:
	struct CompiledShaderInitializerType : FGlobalShaderType::CompiledShaderInitializerType
	{
		const FUniformExpressionSet& UniformExpressionSet;
		const FString DebugDescription;

		CompiledShaderInitializerType(
			const FShaderType* InType,
			int32 InPermutationId,
			const FShaderCompilerOutput& CompilerOutput,
			const FUniformExpressionSet& InUniformExpressionSet,
			uint64 InMaterialShaderMapHash,
			const FShaderPipelineType* InShaderPipeline,
			const FVertexFactoryType* InVertexFactoryType,
			const FString& InDebugDescription
			)
		: FGlobalShaderType::CompiledShaderInitializerType(
			InType, 
			nullptr, 
			InPermutationId,
			CompilerOutput,
			InMaterialShaderMapHash,
			InShaderPipeline,
			InVertexFactoryType
			)
		, UniformExpressionSet(InUniformExpressionSet)
		, DebugDescription(InDebugDescription)
		{}
	};

	IMPLEMENT_SHADER_TYPE_CONSTRUCTOR(FMaterialShaderType, EShaderTypeForDynamicCast::Material, TEXT("material shader"));

#if WITH_EDITOR
	/**
	 * Enqueues a compilation for a new shader of this type.
	 * @param Material - The material to link the shader with.
	 */
	void BeginCompileShader(
		EShaderCompileJobPriority Priority,
		uint32 ShaderMapJobId,
		int32 PermutationId,
		const FMaterial* Material,
		const FMaterialCompilationOutput& CompilationOutput,
		FSharedShaderCompilerEnvironment* MaterialEnvironment,
		EShaderPlatform Platform,
		EShaderPermutationFlags PermutationFlags,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs,
		const FString& DebugGroupName,
		const TCHAR* DebugDescription,
		const TCHAR* DebugExtension
	) const;

	static void BeginCompileShaderPipeline(
		EShaderCompileJobPriority Priority,
		uint32 ShaderMapJobId,
		EShaderPlatform Platform,
		EShaderPermutationFlags PermutationFlags,
		const FMaterial* Material,
		const FMaterialCompilationOutput& CompilationOutput,
		FSharedShaderCompilerEnvironment* MaterialEnvironment,
		const FShaderPipelineType* ShaderPipeline,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs,
		const FString& DebugGroupName,
		const TCHAR* DebugDescription,
		const TCHAR* DebugExtension
	);

	/**
	 * Either creates a new instance of this type or returns an equivalent existing shader.
	 * @param Material - The material to link the shader with.
	 * @param CurrentJob - Compile job that was enqueued by BeginCompileShader.
	 */
	FShader* FinishCompileShader(
		const FUniformExpressionSet& UniformExpressionSet,
		uint64 MaterialShaderMapHash,
		const FShaderCompileJob& CurrentJob,
		const FShaderPipelineType* ShaderPipeline,
		const FString& InDebugDescription
		) const;
#endif // WITH_EDITOR

	/**
	 * Checks if the shader type should be cached for a particular platform and material.
	 * @param Platform - The platform to check.
	 * @param Material - The material to check.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCompilePermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, int32 PermutationId, EShaderPermutationFlags Flags) const;

	static bool ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, EShaderPermutationFlags Flags);

#if WITH_EDITOR
	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param Platform - Platform to compile for.
	 * @param Environment - The shader compile environment that the function modifies.
	 */
	void SetupCompileEnvironment(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, int32 PermutationId, EShaderPermutationFlags Flags, FShaderCompilerEnvironment& Environment) const;
#endif // WITH_EDITOR
};

struct FMaterialShaderTypes
{
	const FShaderPipelineType* PipelineType;
	const FShaderType* ShaderType[SF_NumFrequencies];
	int32 PermutationId[SF_NumFrequencies];

	inline FMaterialShaderTypes() : PipelineType(nullptr) { FMemory::Memzero(ShaderType); FMemory::Memzero(PermutationId); }

	inline const FShaderType* AddShaderType(const FShaderType* InType, int32 InPermutationId = 0)
	{
		const EShaderFrequency Frequency = InType->GetFrequency();
		check(ShaderType[Frequency] == nullptr);
		ShaderType[Frequency] = InType;
		PermutationId[Frequency] = InPermutationId;
		return InType;
	}

	template<typename ShaderType>
	inline const FShaderType* AddShaderType(int32 InPermutationId = 0)
	{
		return AddShaderType(&ShaderType::GetStaticType(), InPermutationId);
	}
};

struct FMaterialShaders
{
	const FShaderMapBase* ShaderMap;
	FShaderPipeline* Pipeline;
	FShader* Shaders[SF_NumFrequencies];

	inline FMaterialShaders() : ShaderMap(nullptr), Pipeline(nullptr) { FMemory::Memzero(Shaders); }

	bool TryGetPipeline(FShaderPipelineRef& OutPipeline) const
	{
		if (Pipeline)
		{
			OutPipeline = FShaderPipelineRef(Pipeline, *ShaderMap);
			return true;
		}
		return false;
	}

	template<typename ShaderType>
	inline bool TryGetShader(EShaderFrequency InFrequency, TShaderRef<ShaderType>& OutShader) const
	{
		FShader* Shader = Shaders[InFrequency];
		if (Shader)
		{
			checkSlow(Shader->GetFrequency() == InFrequency);
			OutShader = TShaderRef<ShaderType>(static_cast<ShaderType*>(Shader), *ShaderMap);
			// FTypeLayoutDesc::operator== doesn't work correctly in all cases, when dealing with inline/templated types
			//checkfSlow(OutShader.GetType()->GetLayout().IsDerivedFrom(StaticGetTypeLayoutDesc<ShaderType>()), TEXT("Invalid cast of shader type '%s' to '%s'"),
			//	OutShader.GetType()->GetName(),
			//	StaticGetTypeLayoutDesc<ShaderType>().Name);
			return true;
		}
		return false;
	}

	template<typename ShaderType>
	inline bool TryGetShader(EShaderFrequency InFrequency, TShaderRef<ShaderType>* OutShader) const
	{
		FShader* Shader = Shaders[InFrequency];
		if (Shader)
		{
			checkSlow(Shader->GetFrequency() == InFrequency);
			check(OutShader); // make sure output isn't null, if we have the shader
			*OutShader = TShaderRef<ShaderType>(static_cast<ShaderType*>(Shader), *ShaderMap);
			// FTypeLayoutDesc::operator== doesn't work correctly in all cases, when dealing with inline/templated types
			//checkfSlow(OutShader.GetType()->GetLayout().IsDerivedFrom(StaticGetTypeLayoutDesc<ShaderType>()), TEXT("Invalid cast of shader type '%s' to '%s'"),
			//	OutShader.GetType()->GetName(),
			//	StaticGetTypeLayoutDesc<ShaderType>().Name);
			return true;
		}
		return false;
	}

	template<typename ShaderType> inline bool TryGetVertexShader(TShaderRef<ShaderType>& OutShader) const { return TryGetShader(SF_Vertex, OutShader); }
	template<typename ShaderType> inline bool TryGetPixelShader(TShaderRef<ShaderType>& OutShader) const { return TryGetShader(SF_Pixel, OutShader); }
	template<typename ShaderType> inline bool TryGetGeometryShader(TShaderRef<ShaderType>& OutShader) const { return TryGetShader(SF_Geometry, OutShader); }
	template<typename ShaderType> inline bool TryGetMeshShader(TShaderRef<ShaderType>& OutShader) const { return TryGetShader(SF_Mesh, OutShader); }
	template<typename ShaderType> inline bool TryGetComputeShader(TShaderRef<ShaderType>& OutShader) const { return TryGetShader(SF_Compute, OutShader); }

	template<typename ShaderType> inline bool TryGetVertexShader(TShaderRef<ShaderType>* OutShader) const { return TryGetShader(SF_Vertex, OutShader); }
	template<typename ShaderType> inline bool TryGetPixelShader(TShaderRef<ShaderType>* OutShader) const { return TryGetShader(SF_Pixel, OutShader); }
	template<typename ShaderType> inline bool TryGetGeometryShader(TShaderRef<ShaderType>* OutShader) const { return TryGetShader(SF_Geometry, OutShader); }
	template<typename ShaderType> inline bool TryGetMeshShader(TShaderRef<ShaderType>* OutShader) const { return TryGetShader(SF_Mesh, OutShader); }
	template<typename ShaderType> inline bool TryGetComputeShader(TShaderRef<ShaderType>* OutShader) const { return TryGetShader(SF_Compute, OutShader); }
};
