// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShared.h: Shared material definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Containers/ArrayView.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "Misc/SecureHash.h"
#include "RHIDefinitions.h"
#include "RHIUniformBufferLayoutInitializer.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"
#include "UniformBuffer.h"
#include "Shader.h"
#include "VertexFactory.h"
#include "SceneTypes.h"
#include "StaticParameterSet.h"
#include "Misc/Optional.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ArchiveProxy.h"
#include "MaterialKeyIncludeEnum.h"
#include "MaterialSceneTextureId.h"
#include "MaterialShaderPrecompileMode.h"
#include "MaterialValueType.h"
#include "VirtualTexturingFwd.h"
#include "Templates/UnrealTemplate.h"
#include "PSOPrecacheFwd.h"
#include "UObject/ObjectMacros.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "Shader/Preshader.h"
#include "Shader/ShaderTypes.h"
#include "RenderUtils.h"

#include <atomic>

#include "MaterialShared.generated.h"

struct FExpressionInput;
struct FExtraShaderCompilerSettings;
class FCbFieldView;
class FCbWriter;
class FMaterial;
class FMaterialCompiler;
class FMaterialRenderProxy;
class FMaterialShaderType;
struct FMaterialShaderParameters;
class FMaterialUniformExpression;
class FMaterialUniformExpressionTexture;
struct FUniformExpressionCache;
class FUniformExpressionSet;
class FMeshMaterialShaderType;
class FSceneView;
class FShaderCommonCompileJob;
class FShaderKeyGenerator;
enum class EShaderCompileJobPriority : uint8;
class FVirtualTexture2DResource;
class IAllocatedVirtualTexture;
class UMaterial;
class UMaterialExpression;
class UMaterialExpressionMaterialFunctionCall;
class UMaterialInstance;
class UMaterialInterface;
class URuntimeVirtualTexture;
class USparseVolumeTexture;
class USubsurfaceProfile;
class UTexture;
class UTexture2D;
class FMaterialTextureParameterInfo;
class FMaterialExternalTextureParameterInfo;
class FMeshMaterialShaderMapLayout;
struct FMaterialShaderTypes;
struct FMaterialShaders;
struct FMaterialCachedExpressionData;
class FShaderCompileJobStatus;
class FShaderMapLayout;
namespace MIR { struct FDebugDumpIRState; }
#if WITH_EDITOR
class FMaterialKeyGeneratorContext;
#endif

enum EMaterialDomain : int;

namespace UE
{
	class FEmitter;
}

namespace UE::Cook
{
	class FODSCClientData;
}

template <class ElementType> class TLinkedList;

#define ME_CAPTION_HEIGHT		18
#define ME_STD_VPADDING			16
#define ME_STD_HPADDING			32
#define ME_STD_BORDER			8
#define ME_STD_THUMBNAIL_SZ		96
#define ME_PREV_THUMBNAIL_SZ	256
#define ME_STD_LABEL_PAD		16
#define ME_STD_TAB_HEIGHT		21

// disallow debug data in shipping or on non-desktop Test
#define ALLOW_SHADERMAP_DEBUG_DATA (!(UE_BUILD_SHIPPING || (UE_BUILD_TEST && !PLATFORM_DESKTOP)))

// Adds various checks to track FMaterial lifetime, may add some overhead
#define UE_CHECK_FMATERIAL_LIFETIME PLATFORM_WINDOWS

DECLARE_LOG_CATEGORY_EXTERN(LogMaterial, Log, All);

/** Creates a string that represents the given quality level. */
extern void GetMaterialQualityLevelName(EMaterialQualityLevel::Type InMaterialQualityLevel, FString& OutName);
extern FName GetMaterialQualityLevelFName(EMaterialQualityLevel::Type InMaterialQualityLevel);

inline bool IsSubsurfaceShadingModel(FMaterialShadingModelField ShadingModel)
{
	return ShadingModel.HasShadingModel(MSM_Subsurface) || ShadingModel.HasShadingModel(MSM_PreintegratedSkin) ||
		ShadingModel.HasShadingModel(MSM_SubsurfaceProfile) || ShadingModel.HasShadingModel(MSM_TwoSidedFoliage) ||
		ShadingModel.HasShadingModel(MSM_Cloth) || ShadingModel.HasShadingModel(MSM_Eye);
}

inline bool UseSubsurfaceProfile(FMaterialShadingModelField ShadingModel)
{
	return ShadingModel.HasShadingModel(MSM_SubsurfaceProfile) || ShadingModel.HasShadingModel(MSM_Eye);
}

inline uint32 GetUseSubsurfaceProfileShadingModelMask()
{
	return (1u << MSM_SubsurfaceProfile) | (1u << MSM_Eye) | (Substrate::IsSubstrateEnabled() ? (1u << MSM_PreintegratedSkin) : 0u);
}

UE_DEPRECATED(5.7, "Please use AllowDitheredLODTransition with a EShaderPlatform argument")
ENGINE_API bool AllowDitheredLODTransition(ERHIFeatureLevel::Type FeatureLevel);

/** Whether to allow dithered LOD transitions for a specific shader platform. */
ENGINE_API bool AllowDitheredLODTransition(EShaderPlatform ShaderPlatform);

/** Blend modes utility functions */
ENGINE_API bool IsOpaqueBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsOpaqueBlendMode(const FMaterial& In);
ENGINE_API bool IsOpaqueBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsOpaqueBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsOpaqueOrMaskedBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsOpaqueOrMaskedBlendMode(const FMaterial& In);
ENGINE_API bool IsOpaqueOrMaskedBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsOpaqueOrMaskedBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsMaskedBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsMaskedBlendMode(const FMaterial& In);
ENGINE_API bool IsMaskedBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsMaskedBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsTranslucentOnlyBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsTranslucentOnlyBlendMode(const FMaterial& In);
ENGINE_API bool IsTranslucentOnlyBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsTranslucentOnlyBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsTranslucentBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsTranslucentBlendMode(const FMaterial& In);
ENGINE_API bool IsTranslucentBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsTranslucentBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsAlphaHoldoutBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsAlphaHoldoutBlendMode(const FMaterial& In);
ENGINE_API bool IsAlphaHoldoutBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsAlphaHoldoutBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsModulateBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsModulateBlendMode(const FMaterial& In);
ENGINE_API bool IsModulateBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsModulateBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsAdditiveBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsAdditiveBlendMode(const FMaterial& In);
ENGINE_API bool IsAdditiveBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsAdditiveBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsAlphaCompositeBlendMode(EBlendMode BlendMode);
ENGINE_API bool IsAlphaCompositeBlendMode(const FMaterial& In);
ENGINE_API bool IsAlphaCompositeBlendMode(const UMaterialInterface& In);
ENGINE_API bool IsAlphaCompositeBlendMode(const FMaterialShaderParameters& In);

ENGINE_API bool IsUsingNewMaterialTranslatorPrototype();

#if WITH_EDITOR

void GetCmdLineFilterShaderFormats(TArray<FName>& InOutShderFormats);

/** 
* What material quality level should we explicitly cook for? 
* @returns Quality level or INDEX_NONE if the switch was not specified.
* 
* @note: -CacheMaterialQuality=
*/
int32 GetCmdLineMaterialQualityToCache();

#endif

/** @return the number of components in a vector type. */
inline uint32 GetNumComponents(EMaterialValueType Type)
{
	switch (Type)
	{
	case MCT_UInt:
	case MCT_UInt1:
	case MCT_Float:
	case MCT_Float1: return 1;
	case MCT_UInt2:
	case MCT_Float2: return 2;
	case MCT_UInt3:
	case MCT_Float3: return 3;
	case MCT_UInt4:
	case MCT_Float4: return 4;
	case MCT_LWCScalar: return 1;
	case MCT_LWCVector2: return 2;
	case MCT_LWCVector3: return 3;
	case MCT_LWCVector4: return 4;
	case MCT_StaticBool: return 1;
	case MCT_Bool: return 1;
	default: return 0;
	}
}

inline bool IsMaterialValueType(EMaterialValueType InType, EMaterialValueType InCompare)
{
	return (InType & InCompare) != 0;
}

inline bool IsLWCType(EMaterialValueType InType)
{
	return IsMaterialValueType(InType, MCT_LWCType);
}

inline bool IsFloatNumericType(EMaterialValueType InType)
{
	return IsMaterialValueType(InType, EMaterialValueType(MCT_Float | MCT_LWCType));
}

inline bool IsUIntNumericType(EMaterialValueType InType)
{
	return IsMaterialValueType(InType, MCT_UInt);
}

inline bool IsNumericType(EMaterialValueType InType)
{
	// 'ShadingModel' is considered an 'int' 
	return IsMaterialValueType(InType, EMaterialValueType(MCT_Float | MCT_LWCType | MCT_UInt | MCT_ShadingModel));
}

inline bool IsPrimitiveType(EMaterialValueType InType)
{
	// 'ShadingModel' is considered an 'int' 
	return IsMaterialValueType(InType, EMaterialValueType(MCT_Float | MCT_LWCType | MCT_UInt | MCT_ShadingModel | MCT_Bool | MCT_StaticBool));
}

inline EMaterialValueType MakeNonLWCType(EMaterialValueType Type)
{
	switch (Type)
	{
	case MCT_LWCScalar: return MCT_Float1;
	case MCT_LWCVector2: return MCT_Float2;
	case MCT_LWCVector3: return MCT_Float3;
	case MCT_LWCVector4: return MCT_Float4;
	default: return Type;
	}
}

inline EMaterialValueType MakeLWCType(EMaterialValueType Type)
{
	switch (Type)
	{
	case MCT_Float: return MCT_LWCScalar;
	case MCT_Float1: return MCT_LWCScalar;
	case MCT_Float2: return MCT_LWCVector2;
	case MCT_Float3: return MCT_LWCVector3;
	case MCT_Float4: return MCT_LWCVector4;
	default: return Type;
	}
}

UENUM()
enum class ELWCFunctionKind
{
	Constructor,
	Promote,
	Demote,
	Add,
	Subtract,
	Divide,
	MultiplyVectorVector,
	MultiplyVectorMatrix,
	MultiplyMatrixMatrix,
	Other,
	Max
};

/**
 * The common bases of material
 */
enum EMaterialCommonBasis
{
	MCB_Tangent,
	MCB_Local,
	MCB_TranslatedWorld,
	MCB_World,
	MCB_View,
	MCB_Camera,
	MCB_MeshParticle,
	MCB_Instance,
	MCB_PeriodicWorld,
	MCB_FirstPerson,
	MCB_MAX,
};

//when setting deferred scene resources whether to throw warnings when we fall back to defaults.
enum struct EDeferredParamStrictness
{
	ELoose, // no warnings
	EStrict, // throw warnings
};

/** Fully describes a material compilation target */
struct FMaterialCompileTargetParameters
{
	FMaterialCompileTargetParameters(EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, const ITargetPlatform* InTargetPlatform)
		: ShaderPlatform(InShaderPlatform), FeatureLevel(InFeatureLevel), TargetPlatform(InTargetPlatform)
	{}

	EShaderPlatform ShaderPlatform;
	ERHIFeatureLevel::Type FeatureLevel;
	const ITargetPlatform* TargetPlatform;
};

/**
 * The context of a material being rendered.
 */
struct FMaterialRenderContext
{
	/** material instance used for the material shader */
	const FMaterialRenderProxy* MaterialRenderProxy;
	/** Material resource to use. */
	const FMaterial& Material;

	/** Whether or not selected objects should use their selection color. */
	bool bShowSelection;

	/** 
	* Constructor
	*/
	ENGINE_API FMaterialRenderContext(
		const FMaterialRenderProxy* InMaterialRenderProxy,
		const FMaterial& InMaterial,
		const FSceneView* InView);

	FMaterialRenderContext(
		const FMaterialRenderProxy* InMaterialRenderProxy,
		const FMaterial& InMaterial,
		bool bInShowSelection)
		: MaterialRenderProxy(InMaterialRenderProxy)
		, Material(InMaterial)
		, bShowSelection(bInShowSelection)
	{}

	ENGINE_API void GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const UTexture*& OutValue) const;
	ENGINE_API void GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const URuntimeVirtualTexture*& OutValue) const;
	ENGINE_API void GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureIndex, const USparseVolumeTexture*& OutValue) const;
	ENGINE_API void GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, int32 TextureCollectionIndex, const UTextureCollection*& OutValue) const;
	ENGINE_API FGuid GetExternalTextureGuid(const FGuid& ExternalTextureGuid, const FName& ParameterName, int32 SourceTextureIndex) const;
};

using EShaderFrequencyMask = uint16;

class FMaterialVirtualTextureStack
{
	DECLARE_TYPE_LAYOUT(FMaterialVirtualTextureStack, NonVirtual);
public:
	FMaterialVirtualTextureStack();
	/** Construct with a texture index when this references a preallocated VT stack (for example when we are using a URuntimeVirtualTexture). */
	FMaterialVirtualTextureStack(int32 InPreallocatedStackTextureIndex);

	/** Add space for a layer in the stack. Returns an index that can be used for SetLayer(). */
	uint32 AddLayer();
	/** Set an expression index at a layer in the stack. */
	uint32 SetLayer(int32 LayerIndex, int32 UniformExpressionIndex);
	/** Get the number of layers allocated in the stack. */
	inline uint32 GetNumLayers() const { return NumLayers; }
	/** Returns true if we have allocated the maximum number of layers for this stack. */
	inline bool AreLayersFull() const { return NumLayers == VIRTUALTEXTURE_SPACE_MAXLAYERS; }
	/** Find the layer in the stack that was set with this expression index. */
	int32 FindLayer(int32 UniformExpressionIndex) const;

	/** Returns true if this is a stack that with a preallocated layout of layers (for example when we are using a URuntimeVirtualTexture). */
	inline bool IsPreallocatedStack() const { return PreallocatedStackTextureIndex != INDEX_NONE; }
	/** Get the array of UTexture objects for the expressions that in the layers of this stack. Can return nullptr objects for layers that don't hold UTexture2D references. */
	void GetTextureValues(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, UTexture const** OutValues) const;
	/** Get the URuntimeVirtualTexture object if one was used to initialize this stack. */
	void GetTextureValue(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const URuntimeVirtualTexture*& OutValue) const;
	/** Get the USparseVolumeTexture object if one was used to initialize this stack. */
	void GetTextureValue(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const USparseVolumeTexture * &OutValue) const;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMaterialVirtualTextureStack& Stack)
	{
		Stack.Serialize(Ar);
		return Ar;
	}

	friend bool operator==(const FMaterialVirtualTextureStack& Lhs, const FMaterialVirtualTextureStack& Rhs)
	{
		if (Lhs.PreallocatedStackTextureIndex != Rhs.PreallocatedStackTextureIndex || Lhs.NumLayers != Rhs.NumLayers)
		{
			return false;
		}
		for (uint32 i = 0u; i < Lhs.NumLayers; ++i)
		{
			if (Lhs.LayerUniformExpressionIndices[i] != Rhs.LayerUniformExpressionIndices[i])
			{
				return false;
			}
		}
		// Note: ShaderFrequencyMask is intentionally ignored — it only affects the editor-side decision to include the parameter in the VS compact uniform buffer.
		return true;
	}

	friend inline bool operator!=(const FMaterialVirtualTextureStack& Lhs, const FMaterialVirtualTextureStack& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	/** Number of layers that have been allocated in this stack. */
	LAYOUT_FIELD(uint32, NumLayers);
	/** Indices of the expressions that were set to layers in this stack. */
	LAYOUT_ARRAY(int32, LayerUniformExpressionIndices, VIRTUALTEXTURE_SPACE_MAXLAYERS);
	/** Index of a texture reference if we create a stack from a single known texture that has it's own layer stack. */
	LAYOUT_FIELD(int32, PreallocatedStackTextureIndex);
	LAYOUT_FIELD_EDITORONLY(EShaderFrequencyMask, ShaderFrequencyMask);
};

class FMaterialCacheTagStack
{
	DECLARE_TYPE_LAYOUT(FMaterialCacheTagStack, NonVirtual);

public:
	friend bool operator==(const FMaterialCacheTagStack& Lhs, const FMaterialCacheTagStack& Rhs)
	{
		// Note: ShaderFrequencyMask is intentionally ignored — it only affects the editor-side decision to include the parameter in the VS compact uniform buffer.
		return Lhs.TagGuid == Rhs.TagGuid;
	}

	friend inline bool operator!=(const FMaterialCacheTagStack& Lhs, const FMaterialCacheTagStack& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
	
	LAYOUT_FIELD(FGuid, TagGuid);
	LAYOUT_FIELD_EDITORONLY(EShaderFrequencyMask, ShaderFrequencyMask);
};

/**
* Represents the evaluation of a material parameter directly to a word offset in the preshader uniform buffer.
* A UniformParameterEvaluation avoids the overhead of executing an entire preshader when all it does is 
* evaluate a parameter with a single instruction, and is also used to efficiently preload parameters.
* 
* Note that during material translation, BufferOffset may contain a unique "register" index, which is later
* fixed up to the buffer offset by a call to FixupNumericParameterEvaluations.
*/
class FMaterialUniformParameterEvaluation
{
	DECLARE_TYPE_LAYOUT(FMaterialUniformParameterEvaluation, NonVirtual);
public:
	LAYOUT_FIELD(uint16, ParameterIndex);	/** Index of the parameter for UniformExpressionSet::GetNumericParameter() */
	LAYOUT_FIELD(uint16, BufferOffset); 	/** Offset in words where result will be written. */
	LAYOUT_FIELD(uint16, ComponentCount); 	/** Number of components to write */
};

class FMaterialUniformPreshaderHeader
{
	DECLARE_TYPE_LAYOUT(FMaterialUniformPreshaderHeader, NonVirtual);
public:
	friend inline bool operator==(const FMaterialUniformPreshaderHeader& Lhs, const FMaterialUniformPreshaderHeader& Rhs)
	{
		return Lhs.Type == Rhs.Type &&
			Lhs.BufferOffset == Rhs.BufferOffset &&
			Lhs.OpcodeOffset == Rhs.OpcodeOffset &&
			Lhs.OpcodeSize == Rhs.OpcodeSize;
	}
	friend inline bool operator!=(const FMaterialUniformPreshaderHeader& Lhs, const FMaterialUniformPreshaderHeader& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	LAYOUT_FIELD(UE::Shader::EValueType, Type);
	LAYOUT_FIELD(uint16, BufferOffset); /** Offset in the uniform buffer where result will be written */
	LAYOUT_FIELD(uint32, OpcodeOffset); /** Offset of the preshader opcodes, within the material's buffer */
	LAYOUT_FIELD(uint32, OpcodeSize); /** Size of the preshader opcodes */
};

class FMaterialNumericParameterInfo
{
	DECLARE_TYPE_LAYOUT(FMaterialNumericParameterInfo, NonVirtual);
public:

	friend inline bool operator==(const FMaterialNumericParameterInfo& Lhs, const FMaterialNumericParameterInfo& Rhs)
	{
		return Lhs.ParameterInfo == Rhs.ParameterInfo && Lhs.ParameterType == Rhs.ParameterType && Lhs.DefaultValueOffset == Rhs.DefaultValueOffset;
	}
	friend inline bool operator!=(const FMaterialNumericParameterInfo& Lhs, const FMaterialNumericParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	LAYOUT_FIELD(FHashedMaterialParameterInfo, ParameterInfo);
	LAYOUT_FIELD(EMaterialParameterType, ParameterType);
	LAYOUT_FIELD(uint32, DefaultValueOffset);
};

/** Must invalidate ShaderVersion.ush when changing */
enum class EMaterialTextureParameterType : uint32
{
	Standard2D,
	Cube,
	Array2D,
	ArrayCube,
	Volume,
	Virtual,
	SparseVolume,

	Count,
};
inline constexpr uint32 NumMaterialTextureParameterTypes = (uint32)EMaterialTextureParameterType::Count;

class FMaterialTextureParameterInfo
{
	DECLARE_TYPE_LAYOUT(FMaterialTextureParameterInfo, NonVirtual);
public:
	friend inline bool operator==(const FMaterialTextureParameterInfo& Lhs, const FMaterialTextureParameterInfo& Rhs)
	{
        // Note: ShaderFrequencyMask is intentionally ignored — it only affects the editor-side decision to include the parameter in the VS compact uniform buffer.
		return Lhs.ParameterInfo == Rhs.ParameterInfo && Lhs.TextureIndex == Rhs.TextureIndex && Lhs.SamplerSource == Rhs.SamplerSource && Lhs.VirtualTextureLayerIndex == Rhs.VirtualTextureLayerIndex;
	}
	friend inline bool operator!=(const FMaterialTextureParameterInfo& Lhs, const FMaterialTextureParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	inline FName GetParameterName() const { return ScriptNameToName(ParameterInfo.Name); }

	ENGINE_API void GetGameThreadTextureValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTexture*& OutValue) const;
	ENGINE_API void GetGameThreadTextureValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, USparseVolumeTexture*& OutValue) const;

	LAYOUT_FIELD(FHashedMaterialParameterInfo, ParameterInfo);
	LAYOUT_FIELD_INITIALIZED(int32, TextureIndex, INDEX_NONE);
	LAYOUT_FIELD(TEnumAsByte<ESamplerSourceMode>, SamplerSource);
	LAYOUT_FIELD_INITIALIZED(uint8, VirtualTextureLayerIndex, 0u);
	LAYOUT_FIELD_EDITORONLY(EShaderFrequencyMask, ShaderFrequencyMask);
};

class FMaterialTextureCollectionParameterInfo
{
	DECLARE_TYPE_LAYOUT(FMaterialTextureCollectionParameterInfo, NonVirtual);
public:
	friend inline bool operator==(const FMaterialTextureCollectionParameterInfo& Lhs, const FMaterialTextureCollectionParameterInfo& Rhs)
	{
        // Note: ShaderFrequencyMask is intentionally ignored — it only affects the editor-side decision to include the parameter in the VS compact uniform buffer.
		return Lhs.ParameterInfo == Rhs.ParameterInfo && Lhs.TextureCollectionIndex == Rhs.TextureCollectionIndex && Lhs.bIsVirtualCollection == Rhs.bIsVirtualCollection;
	}
	friend inline bool operator!=(const FMaterialTextureCollectionParameterInfo& Lhs, const FMaterialTextureCollectionParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	ENGINE_API void GetGameThreadTextureCollectionValue(const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTextureCollection*& OutValue) const;
	void GetTextureCollection(const FMaterialRenderContext& Context, const UTextureCollection*& OutTextureCollection) const;

	LAYOUT_FIELD(FHashedMaterialParameterInfo, ParameterInfo);
	LAYOUT_FIELD(int32, TextureCollectionIndex);
	LAYOUT_FIELD(bool, bIsVirtualCollection);
	LAYOUT_FIELD_EDITORONLY(EShaderFrequencyMask, ShaderFrequencyMask);
};

class FMaterialExternalTextureParameterInfo
{
	DECLARE_TYPE_LAYOUT(FMaterialExternalTextureParameterInfo, NonVirtual);
public:
	friend inline bool operator==(const FMaterialExternalTextureParameterInfo& Lhs, const FMaterialExternalTextureParameterInfo& Rhs)
	{
        // Note: ShaderFrequencyMask is intentionally ignored — it only affects the editor-side decision to include the parameter in the VS compact uniform buffer.
		return Lhs.SourceTextureIndex == Rhs.SourceTextureIndex && Lhs.ExternalTextureGuid == Rhs.ExternalTextureGuid && Lhs.ParameterName == Rhs.ParameterName;
	}
	friend inline bool operator!=(const FMaterialExternalTextureParameterInfo& Lhs, const FMaterialExternalTextureParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	bool GetExternalTexture(const FMaterialRenderContext& Context, FTextureRHIRef& OutTextureRHI, FSamplerStateRHIRef& OutSamplerStateRHI) const;

	LAYOUT_FIELD(FScriptName, ParameterName);
	LAYOUT_FIELD(FGuid, ExternalTextureGuid);
	LAYOUT_FIELD(int32, SourceTextureIndex);
	LAYOUT_FIELD_EDITORONLY(EShaderFrequencyMask, ShaderFrequencyMask);
};

class FUniformParameterOverrides
{
public:
	void SetNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value);
	void ClearNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo);
	bool GetNumericOverride(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, UE::Shader::FValue& OutValue) const;

	void SetTextureOverride(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo, UTexture* Texture);
	UTexture* GetTextureOverride_GameThread(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo) const;
	UTexture* GetTextureOverride_RenderThread(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& ParameterInfo) const;

private:
	struct FNumericParameterKey
	{
		FHashedMaterialParameterInfo ParameterInfo;
		EMaterialParameterType ParameterType;

		friend inline bool operator==(const FNumericParameterKey& Lhs, const FNumericParameterKey& Rhs)
		{
			return Lhs.ParameterInfo == Rhs.ParameterInfo && Lhs.ParameterType == Rhs.ParameterType;
		}
		friend inline bool operator!=(const FNumericParameterKey& Lhs, const FNumericParameterKey& Rhs)
		{
			return !operator==(Lhs, Rhs);
		}
		friend inline uint32 GetTypeHash(const FNumericParameterKey& Value)
		{
			return HashCombine(GetTypeHash(Value.ParameterInfo), GetTypeHash(Value.ParameterType));
		}
	};

	TMap<FNumericParameterKey, UE::Shader::FValue> NumericOverrides;

	struct FTextureParameterKey
	{
		FHashedMaterialParameterInfo ParameterInfo;
		int32 TextureIndex;

		friend inline bool operator==(const FTextureParameterKey& Lhs, const FTextureParameterKey& Rhs)
		{
			return Lhs.ParameterInfo == Rhs.ParameterInfo && Lhs.TextureIndex == Rhs.TextureIndex;
		}
		friend inline bool operator!=(const FTextureParameterKey& Lhs, const FTextureParameterKey& Rhs)
		{
			return !operator==(Lhs, Rhs);
		}
		friend inline uint32 GetTypeHash(const FTextureParameterKey& Value)
		{
			return HashCombine(GetTypeHash(Value.ParameterInfo), GetTypeHash(Value.TextureIndex));
		}
	};

	TMap<FTextureParameterKey, UTexture*> GameThreadTextureOverides[NumMaterialTextureParameterTypes];
	TMap<FTextureParameterKey, UTexture*> RenderThreadTextureOverrides[NumMaterialTextureParameterTypes];
};

/** Metadata allowing to create a dense uniform buffer from the generic one*/
class FCompactionMetaDataSegment
{
	DECLARE_TYPE_LAYOUT(FCompactionMetaDataSegment, NonVirtual);
public:
	LAYOUT_FIELD(uint16, SrcOffset);
	LAYOUT_FIELD(uint16, DestOffset);
	LAYOUT_FIELD(uint16, SizeToCopy);
};


/** Stores information per frequency about the values stored in FUniformExpressionSet*/
class FCompactUniformExpressionSet
{
	DECLARE_TYPE_LAYOUT(FCompactUniformExpressionSet, NonVirtual);
public:
	FCompactUniformExpressionSet() : UniformPreshaderBufferSize(0u) {}

	LAYOUT_FIELD(FRHIUniformBufferLayoutInitializer, UniformBufferLayoutInitializer);
	LAYOUT_FIELD(TMemoryImageArray<FCompactionMetaDataSegment>, CompactionSegments);
	LAYOUT_FIELD(uint32, UniformPreshaderBufferSize);
};

/** Stores all uniform expressions for a material generated from a material translation. */
class FUniformExpressionSet
{
	DECLARE_TYPE_LAYOUT(FUniformExpressionSet, NonVirtual);
public:
	FUniformExpressionSet() : UniformPreshaderBufferSize(0u) {}

	bool IsEmpty() const;
	bool operator==(const FUniformExpressionSet& ReferenceSet) const;
	ENGINE_API FString GetSummaryString() const;

#if WITH_EDITOR
	TArray<FShaderParametersMetadata*> CreateBufferStructs(EShaderPrecisionModifier::Type ParameterPrecision);
#endif

	void SetParameterCollections(TConstArrayView<const class UMaterialParameterCollection*> Collections);

	ENGINE_API void FillUniformBuffer(const FMaterialRenderContext& MaterialRenderContext, const FUniformExpressionCache& UniformExpressionCache, const FRHIUniformBufferLayout* UniformBufferLayout, uint8* TempBuffer, int TempBufferSize) const;

	ENGINE_API void FillUniformBuffer(const FMaterialRenderContext& MaterialRenderContext, TConstArrayView<IAllocatedVirtualTexture*> AllocatedVTs, const FRHIUniformBufferLayout* UniformBufferLayout, uint8* TempBuffer, int TempBufferSize) const;

	static void CompactUniformBuffer(uint8* DestBuffer, int32 DestBufferSize, const uint8* SrcBuffer, int32 SrcBufferSize, const FCompactUniformExpressionSet& CompactUniforms);

	// Get a combined hash of all referenced Texture2D's underlying RHI textures, going through TextureReferences. Can be used to tell if any texture has gone through texture streaming mip changes recently.
	ENGINE_API uint32 GetReferencedTexture2DRHIHash(const FMaterialRenderContext& MaterialRenderContext) const;

	inline bool HasExternalTextureExpressions() const
	{
		return UniformExternalTextureParameters.Num() > 0;
	}

	const FRHIUniformBufferLayoutInitializer& GetUniformBufferLayoutInitializer() const
	{
		return UniformBufferLayoutInitializer;
	}

	UE::Shader::FValue GetDefaultParameterValue(EMaterialParameterType Type, uint32 Offset) const
	{
		return UE::Shader::FValue::FromMemoryImage(GetShaderValueType(Type), DefaultValues.GetData() + Offset);
	}

	const uint8* GetDefaultParameterData(uint32 Offset) const
	{
		return DefaultValues.GetData() + Offset;
	}

	inline const FMaterialNumericParameterInfo& GetNumericParameter(uint32 Index) const { return UniformNumericParameters[Index]; }
	inline const FMaterialTextureParameterInfo& GetTextureParameter(EMaterialTextureParameterType Type, int32 Index) const { return UniformTextureParameters[(uint32)Type][Index]; }
	const FMaterialTextureCollectionParameterInfo& GetTextureCollectionParameter(int32 Index) const { return UniformTextureCollectionParameters[Index]; }

	inline int32 GetNumTextures(EMaterialTextureParameterType Type) const { return UniformTextureParameters[(uint32)Type].Num(); }
#if WITH_EDITORONLY_DATA
	void AddTextureShaderFrequencyMask(EMaterialTextureParameterType Type, int32 Index, EShaderFrequencyMask Mask)
	{
		UniformTextureParameters[(int32)Type][Index].ShaderFrequencyMask |= Mask;
	}
#endif
	ENGINE_API void GetGameThreadTextureValue(EMaterialTextureParameterType Type, int32 Index, const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTexture*& OutValue, bool bAllowOverride = true) const;
	ENGINE_API void GetTextureValue(EMaterialTextureParameterType Type, int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const UTexture*& OutValue) const;
	ENGINE_API void GetTextureValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const URuntimeVirtualTexture*& OutValue) const;
	ENGINE_API void GetTextureValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const USparseVolumeTexture*& OutValue) const;

	int32 GetNumTextureCollections() const { return UniformTextureCollectionParameters.Num(); }
	ENGINE_API void GetGameThreadTextureCollectionValue(int32 Index, const UMaterialInterface* MaterialInterface, const FMaterial& Material, UTextureCollection*& OutValue) const;
	ENGINE_API void GetTextureCollectionValue(int32 Index, const FMaterialRenderContext& Context, const FMaterial& Material, const UTextureCollection*& OutValue) const;

	int32 FindOrAddTextureParameter(EMaterialTextureParameterType Type, const FMaterialTextureParameterInfo& Info);
	int32 FindOrAddTextureCollectionParameter(const FMaterialTextureCollectionParameterInfo& Info);
	int32 FindOrAddExternalTextureParameter(const FMaterialExternalTextureParameterInfo& Info);
	int32 FindOrAddNumericParameter(EMaterialParameterType Type, const FMaterialParameterInfo& ParameterInfo, uint32 DefaultValueOffset);
	uint32 AddDefaultParameterValue(const UE::Shader::FValue& Value);
	
	/// Adds a new direct parameter evaluation for parameter with specified index (returned by FindOrAddNumericParameter)
	/// BufferOffset is the location in the uniform buffer in which to evaluate the parameter (in multiple of float components).
	void AddNumericParameterEvaluation(uint32 ParameterIndex, uint32 BufferOffset, uint32 ComponentCount);
	
	// Fixes up numeric parameter evaluation indices to offsets in the output buffer, using the given remapping table.
	// Also, optionally fixes up component counts.
	void FixupNumericParameterEvaluations(TConstArrayView<uint16> Remap, TConstArrayView<uint8> UniformComponentCount);

	/// Allocates the specified number of float4s in the uniform buffer and returns the index
	/// of the first newly allocated float4 in the buffer.
	/// Note: multiply returned value by sizeof(float4) to know the byte offset in the buffer.
	uint32 AllocateFromUniformBuffer(uint32 NumOfFloat4s);

	// Returns size of uniform preshader buffer in float4s.
	uint32 GetUniformBufferSize() const { return UniformPreshaderBufferSize; }

	// Returns the uniform preshader data.
	UE::Shader::FPreshaderData& GetPreshaderData() { return UniformPreshaderData; }
	const UE::Shader::FPreshaderData& GetPreshaderData() const { return UniformPreshaderData; }

	// Returns size of temporary preshader buffer in float4s.  Temporary space should be added to the CPU side uniform buffer.
	uint32 GetTemporaryBufferSize() const { return UniformPreshaderData.Preshader2TemporarySize; }
	
	TConstArrayView<FMaterialCacheTagStack> GetMaterialCacheTagStacks() const { return MaterialCacheTagStacks; }
	
	TConstArrayView<FMaterialVirtualTextureStack> GetVTStacks() const { return VTStacks; }
	const FMaterialVirtualTextureStack& GetVTStack(int32 Index) const { return VTStacks[Index]; }
	int32 AddVTStack(int32 InPreallocatedStackTextureIndex);
	int32 AddVTLayer(int32 StackIndex, int32 TextureIndex);
	void SetVTLayer(int32 StackIndex, int32 VTLayerIndex, int32 TextureIndex);

	/// Writes a new uniform preshader entry.
	/// @param BufferOffset Specifies the location in the preshader buffer where the result of the preshader instruction will be written to. See FMaterialUniformPreshaderHeader::BufferOffset.
	/// @param EntryType Specifies the value type of the preshader instruction output. See FMaterialUniformPreshaderHeader::Type.
	/// @param EntryGenerator Callback that writes the preshader opcodes.
	void WriteUniformPreshaderEntry(uint32 BufferOffset, UE::Shader::EValueType EntryType, const TFunction<void(UE::Shader::FPreshaderData&)>& EntryGenerator);

	void CountTextureCollections(uint32& BindlessCollections, uint32& VirtualCollections) const;

	const FCompactUniformExpressionSet* GetCompactUniformsVS() const 
	{ 
		check(CompactUniformsVSOptional.Num() <= 1); 
		return CompactUniformsVSOptional.IsEmpty() ? nullptr : &CompactUniformsVSOptional[0]; 
	}

	FCompactUniformExpressionSet* GetCompactUniformsVS() 
	{ 
		check(CompactUniformsVSOptional.Num() <= 1); 
		return CompactUniformsVSOptional.IsEmpty() ? nullptr : &CompactUniformsVSOptional[0]; 
	}

	FCompactUniformExpressionSet* CreateCompactUniformsVS()
	{
		CompactUniformsVSOptional.Empty();
		CompactUniformsVSOptional.Add(FCompactUniformExpressionSet());
		return &CompactUniformsVSOptional[0];
	}

	void EmptyCompactUniformsVS()
	{
		CompactUniformsVSOptional.Empty();
	}

protected:
	union FVTPackedStackAndLayerIndex
	{
		inline FVTPackedStackAndLayerIndex(uint16 InStackIndex, uint16 InLayerIndex) : StackIndex(InStackIndex), LayerIndex(InLayerIndex) {}

		uint32 PackedValue;
		struct
		{
			uint16 StackIndex;
			uint16 LayerIndex;
		};
	};

	FVTPackedStackAndLayerIndex GetVTStackAndLayerIndex(int32 UniformExpressionIndex) const;

	friend class FMaterial;
	friend class FHLSLMaterialTranslator;
	friend class FMaterialShaderMap;
	friend class FMaterialShader;
	friend class FMaterialRenderProxy;
	friend class FMaterialVirtualTextureStack;
	friend class FDebugUniformExpressionSet;
	friend struct MIR::FDebugDumpIRState;
	LAYOUT_FIELD(TMemoryImageArray<FMaterialUniformParameterEvaluation>, UniformParameterEvaluations);
	LAYOUT_FIELD(TMemoryImageArray<FMaterialUniformPreshaderHeader>, UniformPreshaders);
	LAYOUT_FIELD(TMemoryImageArray<FMaterialNumericParameterInfo>, UniformNumericParameters);
	LAYOUT_ARRAY(TMemoryImageArray<FMaterialTextureParameterInfo>, UniformTextureParameters, NumMaterialTextureParameterTypes);
	LAYOUT_FIELD(TMemoryImageArray<FMaterialExternalTextureParameterInfo>, UniformExternalTextureParameters);
	LAYOUT_FIELD(TMemoryImageArray<FMaterialTextureCollectionParameterInfo>, UniformTextureCollectionParameters);
	LAYOUT_FIELD(uint32, UniformPreshaderBufferSize);

	LAYOUT_FIELD(UE::Shader::FPreshaderData, UniformPreshaderData);
	LAYOUT_FIELD(TMemoryImageArray<uint8>, DefaultValues);

	/** Virtual texture stacks found during compilation */
	LAYOUT_FIELD(TMemoryImageArray<FMaterialVirtualTextureStack>, VTStacks);

	/** Material cache tag stacks found during translation */
	LAYOUT_FIELD(TMemoryImageArray<FMaterialCacheTagStack>, MaterialCacheTagStacks)

	/** Ids of parameter collections referenced by the material that was translated. */
	LAYOUT_FIELD(TMemoryImageArray<FGuid>, ParameterCollections);

	LAYOUT_FIELD(FRHIUniformBufferLayoutInitializer, UniformBufferLayoutInitializer);
	
	// Using TMemoryImageArray here because CompactUniformsVS is optional for now, and we don't want to bloat FUniformExpressionSet when disabled
	// FUniformExpressionSet are getting copied with shadermaps, so things are usually copied by value, so having a pointer makes things trickier
	LAYOUT_FIELD(TMemoryImageArray<FCompactUniformExpressionSet>, CompactUniformsVSOptional);

};

/**
Stores outputs from the material compile that need to be saved.
Updates to layout fields in this class must be accompanied with a GUID bump to MaterialTranslationDDCVersion entry in FDevSystemGuids.
*/
class FMaterialCompilationOutput
{
	DECLARE_TYPE_LAYOUT(FMaterialCompilationOutput, NonVirtual);
public:
	FMaterialCompilationOutput() :
		UsedSceneTextures(0),
		UsedPathTracingBufferTextures(0),
#if WITH_EDITOR
		EstimatedNumTextureSamplesVS(0),
		EstimatedNumTextureSamplesPS(0),
		EstimatedNumVirtualTextureLookups(0),
		NumUsedUVScalars(0),
		NumUsedCustomInterpolatorScalars(0),
		NumVirtualTextureFeedbackRequests(0),
#endif
		SubstrateMaterialCompilationOutput(),
		UsedDBufferTextures(0),
		RuntimeVirtualTextureOutputAttributeMask(0),
		bNeedsSceneTextures(false),
		bUsesDBufferTextureLookup(false),
		bUsesEyeAdaptation(false),
		bModifiesMeshPosition(false),
		bUsesWorldPositionOffset(false),
		bUsesTemporalResponsiveness(false),
		bUsesMotionVectorWorldOffset(false),
		bUsesGlobalDistanceField(false),
		bUsesPixelDepthOffset(false),
		bUsesDistanceCullFade(false),
		bUsesPerInstanceCustomData(false),
		bUsesVertexInterpolator(false),
		bHasRuntimeVirtualTextureOutputNode(false),
		bUsesAnisotropy(false),
		bIsLightFunctionAtlasCompatible(false),
		bUsesDisplacement(false),
		bUsedWithNeuralNetworks(false),
		bUsesCustomizedUVs(false),
		bUsesOpacityMask(false),
		bUsesCustomHLSLPixelDiscard(false),
		bUsesOpacity(false)
	{
#if WITH_EDITOR
		FMemory::Memzero(EstimatedLWCFuncUsagesVS);
		FMemory::Memzero(EstimatedLWCFuncUsagesPS);
		FMemory::Memzero(EstimatedLWCFuncUsagesCS);
#endif
	}

	bool IsSceneTextureUsed(ESceneTextureId TexId) const { return (UsedSceneTextures & (1 << TexId)) != 0; }

	void SetIsSceneTextureUsed(ESceneTextureId TexId)
	{
		// User scene textures aren't added to used bits -- those are stored separately in UserSceneTextureInputs
		if (TexId < PPI_UserSceneTexture0)
		{
			UsedSceneTextures |= (1 << TexId);
		}
	}

#if WITH_EDITOR
	// Returns Id for a user scene texture input, or INDEX_NONE if allocation fails.
	int32 FindOrAddUserSceneTexture(FName UserSceneTexture);

	// Returns Id for a user scene texture input, or INDEX_NONE if not found.
	int32 FindUserSceneTexture(FName UserSceneTexture) const;

	// Returns number of post process inputs used, between PostProcessInput SceneTexture nodes and UserSceneTexture nodes.  If this exceeds
	// kPostProcessMaterialInputCountMax, compile should be failed (number is returned rather than bool for error reporting purposes).
	int32 GetNumPostProcessInputsUsed() const;
#endif  // WITH_EDITOR

	void SetIsDBufferTextureUsed(int32 TextureIndex) { UsedDBufferTextures |= (1 << TextureIndex); }
	void SetIsDBufferTextureLookupUsed(bool bValue) { bUsesDBufferTextureLookup = bValue; }

	void SetIsPathTracingBufferTextureUsed(int32 TexId) { UsedPathTracingBufferTextures |= (1 << TexId); }

	/** Indicates whether the material uses scene color. */
	bool RequiresSceneColorCopy() const { return IsSceneTextureUsed(PPI_SceneColor); }

	/** Returns mask for all scene textures that are from the GBuffer */
	static uint32 GetGBufferMask()
	{
		return
			(1u << PPI_DiffuseColor) |
			(1u << PPI_SpecularColor) |
			(1u << PPI_SubsurfaceColor) |
			(1u << PPI_BaseColor) |
			(1u << PPI_Specular) |
			(1u << PPI_Metallic) |
			(1u << PPI_WorldNormal) |
			(1u << PPI_WorldTangent) |
			(1u << PPI_Opacity) |
			(1u << PPI_Roughness) |
			(1u << PPI_Anisotropy) |
			(1u << PPI_MaterialAO) |
			(1u << PPI_DecalMask) |
			(1u << PPI_ShadingModelColor) |
			(1u << PPI_ShadingModelID) |
			(1u << PPI_StoredBaseColor) |
			(1u << PPI_StoredSpecular) |
			(1u << PPI_Velocity) |
			(1u << PPI_IsFirstPerson);
	}

	/** true if the material uses any GBuffer textures */
	bool NeedsGBuffer() const
	{
		return UsedSceneTextures & GetGBufferMask();
	}

	/** true if the material uses the SceneDepth lookup */
	bool UsesSceneDepthLookup() const { return IsSceneTextureUsed(PPI_SceneDepth); }

	/** true if the material uses the Velocity SceneTexture lookup */
	bool UsesVelocitySceneTexture() const { return IsSceneTextureUsed(PPI_Velocity); }

	LAYOUT_FIELD(FUniformExpressionSet, UniformExpressionSet);

	/** User scene texture inputs to this material, these will use slots not taken up by the UsedSceneTextures bitfield below */
	LAYOUT_FIELD(TMemoryImageArray<FScriptName>, UserSceneTextureInputs);

	/** Bitfield of the ESceneTextures used */
	LAYOUT_FIELD(uint32, UsedSceneTextures);

	/** Bitfield of the EPathTracingBufferTextureId used */
	LAYOUT_FIELD(uint8, UsedPathTracingBufferTextures);

	/** Number of times SampleTexture is called, excludes custom nodes. */
	LAYOUT_FIELD_EDITORONLY(uint16, EstimatedNumTextureSamplesVS);
	LAYOUT_FIELD_EDITORONLY(uint16, EstimatedNumTextureSamplesPS);

	/** Estimate of the number of times each LWC operator occurs in the material shader code */
	LAYOUT_ARRAY_EDITORONLY(uint16, EstimatedLWCFuncUsagesVS, (int)ELWCFunctionKind::Max)
	LAYOUT_ARRAY_EDITORONLY(uint16, EstimatedLWCFuncUsagesPS, (int)ELWCFunctionKind::Max)
	LAYOUT_ARRAY_EDITORONLY(uint16, EstimatedLWCFuncUsagesCS, (int)ELWCFunctionKind::Max)
	
	/** Number of virtual texture lookups performed, excludes direct invocation in shaders (for example VT lightmaps) */
	LAYOUT_FIELD_EDITORONLY(uint16, EstimatedNumVirtualTextureLookups);

	/** Number of used custom UV scalars. */
	LAYOUT_FIELD_EDITORONLY(uint8, NumUsedUVScalars);

	/** Number of used custom vertex interpolation scalars. */
	LAYOUT_FIELD_EDITORONLY(uint8, NumUsedCustomInterpolatorScalars);

	/** Number of feedback requests used for virtual texturing. */
	LAYOUT_FIELD_EDITORONLY(uint8, NumVirtualTextureFeedbackRequests);

	/** The Substrate material layout */
	LAYOUT_FIELD(FSubstrateMaterialCompilationOutput, SubstrateMaterialCompilationOutput);

	/** Bitfield of used DBuffer textures . */
	LAYOUT_FIELD(uint8, UsedDBufferTextures);

	/** Bitfield of runtime virtual texture output attributes. */
	LAYOUT_FIELD(uint8, RuntimeVirtualTextureOutputAttributeMask);

	/** true if the material needs the scene texture lookups. */
	LAYOUT_BITFIELD(uint8, bNeedsSceneTextures, 1);

	/** true if the material uses DBuffer texture lookups. */
	LAYOUT_BITFIELD(uint8, bUsesDBufferTextureLookup, 1);

	/** true if the material uses the EyeAdaptationLookup */
	LAYOUT_BITFIELD(uint8, bUsesEyeAdaptation, 1);

	/** true if the material modifies the the mesh position. */
	LAYOUT_BITFIELD(uint8, bModifiesMeshPosition, 1);

	/** Whether the material uses world position offset. */
	LAYOUT_BITFIELD(uint8, bUsesWorldPositionOffset, 1);

	/** true if the material uses temporal responsiveness. */
	LAYOUT_BITFIELD(uint8, bUsesTemporalResponsiveness, 1);

	/** true if the material uses motion vector offset. */
	LAYOUT_BITFIELD(uint8, bUsesMotionVectorWorldOffset, 1);

	/** true if material uses the global distance field */
	LAYOUT_BITFIELD(uint8, bUsesGlobalDistanceField, 1);

	/** true if the material writes a pixel depth offset */
	LAYOUT_BITFIELD(uint8, bUsesPixelDepthOffset, 1);

	/** true if the material uses distance cull fade */
	LAYOUT_BITFIELD(uint8, bUsesDistanceCullFade, 1);

	/** true if the material uses per-instance custom data */
	LAYOUT_BITFIELD(uint8, bUsesPerInstanceCustomData, 1);

	/** true if the material uses vertex interpolator */
	LAYOUT_BITFIELD(uint8, bUsesVertexInterpolator, 1);

	/** true if the material writes to a runtime virtual texture custom output node. */
	LAYOUT_BITFIELD(uint8, bHasRuntimeVirtualTextureOutputNode, 1);

	/** true if the material uses non 0 anisotropy value */
	LAYOUT_BITFIELD(uint8, bUsesAnisotropy, 1);

	/** true if the material is compatible with the light function atlas */
	LAYOUT_BITFIELD(uint8, bIsLightFunctionAtlasCompatible, 1);
	
	/** Whether the material uses scalar displacement. */
	LAYOUT_BITFIELD(uint8, bUsesDisplacement, 1);

	/** Whether the material uses NNE. */
	LAYOUT_BITFIELD(uint8, bUsedWithNeuralNetworks, 1);
	
	/** Whether the material uses customized UVs. */
	LAYOUT_BITFIELD(uint8, bUsesCustomizedUVs, 1);

	/** True if the opacity mask isn't constant 1.0 (after constant folding). */
	LAYOUT_BITFIELD(uint8, bUsesOpacityMask, 1);

	/** True if any UMaterialExpressionCustom compiled into this material uses clip() or discard to conditionally discard pixels, making it behave like an opacity mask. */
	LAYOUT_BITFIELD(uint8, bUsesCustomHLSLPixelDiscard, 1);

	/** True if the opacity isn't constant 1.0 (after constant folding). */
	LAYOUT_BITFIELD(uint8, bUsesOpacity, 1);
};

struct FDebugShaderPipelineInfo
{
	const FShaderPipelineType* Pipeline = nullptr;
	TArray<FShaderType*> ShaderTypes;
};

struct FDebugShaderTypeInfo
{
	FVertexFactoryType* VFType = nullptr;
	TArray<FShaderType*> ShaderTypes;
	TArray<FDebugShaderPipelineInfo> Pipelines;
};

/** 
 * Usage options for a shader map.
 * The purpose of EMaterialShaderMapUsage is to allow creating a unique yet deterministic (no appCreateGuid) Id,
 * For a shader map corresponding to any UMaterial or UMaterialInstance, for different use cases.
 * As an example, when exporting a material to Lightmass we want to compile a shader map with FLightmassMaterialProxy,
 * And generate a FMaterialShaderMapId for it that allows reuse later, so it must be deterministic.
 */
namespace EMaterialShaderMapUsage
{
	enum Type
	{
		Default,
		LightmassExportEmissive,
		LightmassExportDiffuse,
		LightmassExportOpacity,
		LightmassExportNormal,
		MaterialExportBaseColor,
		MaterialExportSpecular,
		MaterialExportNormal,
		MaterialExportTangent,
		MaterialExportMetallic,
		MaterialExportRoughness,
		MaterialExportAnisotropy,
		MaterialExportAO,
		MaterialExportEmissive,
		MaterialExportOpacity,
		MaterialExportOpacityMask,
		MaterialExportSubSurfaceColor,
		MaterialExportClearCoat,
		MaterialExportClearCoatRoughness,
		MaterialExportCustomOutput,
		MaterialExportShadingModel,
		MaterialExportRefraction,
	};
}

struct FMaterialShaderParametersBase
{
	EMaterialDomain MaterialDomain;
	FMaterialShadingModelField ShadingModels;
	union
	{
		// Room for up to 16 flags in previous padding space
		uint16 PackedFlags16;
		struct
		{
			uint16 bIsUsedWithStaticMesh : 1;
			uint16 bIsTranslucentSurfaceLighting : 1;
			uint16 bHasFirstPersonInterpolation : 1;
			uint16 bIsSubstrateMaterial : 1;
			uint16 bUsesTemporalResponsiveness : 1;
			uint16 bUsesMotionVectorWorldOffset : 1;
			uint16 bSamplesMaterialCache : 1;
			uint16 bHasMaterialCacheOutput : 1;
			uint16 bIsUsedWithMeshDeformer : 1;
			uint16 bIsUsedWithInstancedSkinnedMesh : 1;
		};
	};
	EBlendMode BlendMode;
	ERHIFeatureLevel::Type FeatureLevel;
	EMaterialQualityLevel::Type QualityLevel;
	uint16 PreshaderGap;
	int32 BlendableLocation;
	int32 NumCustomizedUVs;
	uint32 StencilCompare;
	int8 NumMaterialCacheTags;
	union
	{
		// These flags are full, add more flags above
		uint64 PackedFlags;
		struct
		{
			uint64 bIsDefaultMaterial : 1;
			uint64 bIsSpecialEngineMaterial : 1;
			uint64 bIsMasked : 1;
			uint64 bIsDitherMasked : 1;
			uint64 bIsTwoSided : 1;
			uint64 bIsThinSurface : 1;
			uint64 bIsDistorted : 1;
			uint64 bShouldCastDynamicShadows : 1;
			uint64 bWritesEveryPixel : 1;
			uint64 bWritesEveryPixelShadowPass : 1;
			uint64 bHasDiffuseAlbedoConnected : 1;
			uint64 bHasF0Connected : 1;
			uint64 bHasBaseColorConnected : 1;
			uint64 bHasNormalConnected : 1;
			uint64 bHasRoughnessConnected : 1;
			uint64 bHasSpecularConnected : 1;
			uint64 bHasMetallicConnected : 1;
			uint64 bHasEmissiveColorConnected : 1;
			uint64 bHasAmbientOcclusionConnected : 1;
			uint64 bHasAnisotropyConnected : 1;
			uint64 bIsTessellationEnabled : 1;
			uint64 bHasDisplacementConnected : 1;
			uint64 bHasVertexPositionOffsetConnected : 1;
			uint64 bHasPixelDepthOffsetConnected : 1;
			uint64 bMaterialMayModifyMeshPosition : 1;
			uint64 bIsUsedWithStaticLighting : 1;
			uint64 bIsUsedWithParticleSprites : 1;
			uint64 bIsUsedWithMeshParticles : 1;
			uint64 bIsUsedWithNiagaraSprites : 1;
			uint64 bIsUsedWithNiagaraMeshParticles : 1;
			uint64 bIsUsedWithNiagaraRibbons : 1;
			uint64 bIsUsedWithLandscape : 1;
			uint64 bIsUsedWithBeamTrails : 1;
			uint64 bIsUsedWithSplineMeshes : 1;
			uint64 bIsUsedWithSkeletalMesh : 1;
			uint64 bIsUsedWithMorphTargets : 1;
			uint64 bIsUsedWithAPEXCloth : 1;
			uint64 bIsUsedWithGeometryCache : 1;
			uint64 bIsUsedWithGeometryCollections : 1;
			uint64 bIsUsedWithHairStrands : 1;
			uint64 bIsUsedWithWater : 1;
			uint64 bIsTranslucencyWritingVelocity : 1;
			uint64 bIsTranslucencyWritingCustomDepth : 1;
			uint64 bIsDitheredLODTransition : 1;
			uint64 bIsUsedWithInstancedStaticMeshes : 1;
			uint64 bHasPerInstanceCustomData : 1;
			uint64 bHasPerInstanceRandom : 1;
			uint64 bHasVertexInterpolator : 1;
			uint64 bHasRuntimeVirtualTextureOutput : 1;
			uint64 bIsUsedWithLidarPointCloud : 1;
			uint64 bIsUsedWithVirtualHeightfieldMesh : 1;
			uint64 bIsUsedWithNeuralNetworks : 1;
			uint64 bIsUsedWithNanite : 1;
			uint64 bIsUsedWithVoxels : 1;
			uint64 bIsUsedWithCurves : 1;
			uint64 bIsStencilTestEnabled : 1;
			uint64 bIsTranslucencySurface : 1;
			uint64 bShouldDisableDepthTest : 1;
			uint64 bHasRenderTracePhysicalMaterialOutput : 1;
			uint64 bIsUsedWithVolumetricCloud : 1;
			uint64 bIsUsedWithHeterogeneousVolumes : 1;
			uint64 bIsMobileSeparateTranslucencyEnabled : 1;
			uint64 bAlwaysEvaluateWorldPositionOffset : 1;
			uint64 bDisablePreExposureScale : 1;
			uint64 bAllowVariableRateShading : 1;
		};
	};

protected:
	FMaterialShaderParametersBase(const FMaterial* InMaterial = nullptr);

private:
#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	ENGINE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FMaterialShaderParametersBase& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend ENGINE_API bool LoadFromCompactBinary(FCbFieldView Field, FMaterialShaderParametersBase& OutValue);
#endif
};

// FMaterialShaderParametersBase split out to ease serializing of raw data
struct FMaterialShaderParameters : public FMaterialShaderParametersBase
{
	TArray<FName> MaterialShaderTags;

	UE_DEPRECATED(5.8, "This will be moved under protected. Use FCompilationShaderParameters or FConservativeShaderParameters to create Shader Parameters.")
	ENGINE_API explicit FMaterialShaderParameters(const FMaterial* InMaterial = nullptr);

protected:
	void InitializeShaderLayoutParamsFromMaterial(const FMaterial* InMaterial);
	void InitializeShaderLayoutParamsFromCompOutput(const FMaterial* InMaterial, const FMaterialCompilationOutput& InCompilationOutput);
};

// For Shader Compilation. Built based on Translation output.
struct FCompilationShaderParameters : public FMaterialShaderParameters
{
	ENGINE_API explicit FCompilationShaderParameters(const FMaterial* InMaterial, const FMaterialCompilationOutput& InCompilationOutput);
};

// For DDC Key/Iterative cooking. Built based on asset data.
struct FConservativeShaderParameters : public FMaterialShaderParameters
{
	ENGINE_API FConservativeShaderParameters();
	ENGINE_API explicit FConservativeShaderParameters(const FMaterial* InMaterial);

#if WITH_EDITOR
	void Record(FMaterialKeyGeneratorContext& Context);
#endif
};

/** Contains all the information needed to uniquely identify a FMaterialShaderMap. */
class FMaterialShaderMapId
{
public:
	uint64 CookedShaderMapIdHash = 0u;

#if WITH_EDITOR
	/** 
	 * The base material's StateId.  
	 * This guid represents all the state of a UMaterial that is not covered by the other members of FMaterialShaderMapId.
	 * Any change to the UMaterial that modifies that state (for example, adding an expression) must modify this guid.
	 */
	FGuid BaseMaterialId;
#endif

	/** 
	 * Quality level that this shader map is going to be compiled at.  
	 * Can be a value of EMaterialQualityLevel::Num if quality level doesn't matter to the compiled result.
	 */
	EMaterialQualityLevel::Type QualityLevel;

	/** Feature level that the shader map is going to be compiled for. */
	ERHIFeatureLevel::Type FeatureLevel;

#if WITH_EDITOR
	/** 
	 * Indicates what use case this shader map will be for.
	 * This allows the same UMaterial / UMaterialInstance to be compiled with multiple FMaterial derived classes,
	 * While still creating an Id that is deterministic between runs (no appCreateGuid used).
	 */
	EMaterialShaderMapUsage::Type Usage;

	/** 
	 * Name of which specific custom output this shader map will be a use case for. Only used if Usage is MaterialExportCustomOutput.
	 */
	FString UsageCustomOutput;

private:
	/** Was the shadermap Id loaded in from a cooked resource. */
	bool bIsCookedId;

	/** Relevant portions of StaticParameterSet from material. */
	TArray<FStaticSwitchParameter> StaticSwitchParameters;
	TArray<FStaticComponentMaskParameter> StaticComponentMaskParameters;
	TArray<FStaticTerrainLayerWeightParameter> TerrainLayerWeightParameters;
	TOptional<FMaterialLayersFunctions::ID> MaterialLayersId;
public:
	/** Guids of any functions the material was dependent on. */
	TArray<FGuid> ReferencedFunctions;

	/** Guids of any Parameter Collections the material was dependent on. */
	TArray<FGuid> ReferencedParameterCollections;

	/** Shader types of shaders that are inlined in this shader map in the DDC. */
	TArray<FShaderTypeDependency> ShaderTypeDependencies;

	/** Shader pipeline types of shader pipelines that are inlined in this shader map in the DDC. */
	TArray<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies;

	/** Vertex factory types of shaders that are inlined in this shader map in the DDC. */
	TArray<FVertexFactoryTypeDependency> VertexFactoryTypeDependencies;

	/** 
	 * Hash of the textures referenced by the uniform expressions in the shader map.
	 * This is stored in the shader map Id to gracefully handle situations where code changes
	 * that generates the array of textures that the uniform expressions use to link up after being loaded from the DDC.
	 */
	uint64 TextureReferencesHash = 0u;

	/** A hash of the content of files (dynamically) included by material expressions. */
	uint64 ExpressionIncludesHash = 0u;
	
	/** A hash of the base property overrides for this material instance. */
	uint64 BasePropertyOverridesHash = 0u;

	/** A hash of all external code declarations referenced in the shader map. */
	uint64 ExternalCodeReferencesHash = 0u;

	/** A hash of the base material's full path */
	uint64 BaseMaterialPathHash = 0u;

	/** Is the material using the new HLSL generator? */
	bool bUsingNewHLSLGenerator;

	/**
	 * Preferred format for the shader byte code the shader map is going to be compiled for.
	 * This is used for preview platforms and affects all shaders, so make it part of the shader map Id.
	 */
	FName PreferredShaderCodeFormat;

	/** The Substrate configuration used when compiling this material, can be tweaked in the material editor for live visualization of simplification. */
	FSubstrateCompilationConfig SubstrateCompilationConfig;

#endif // WITH_EDITOR

	/*
	 * Type layout parameters of the memory image
	 */
	FPlatformTypeLayoutParameters LayoutParams;

	FMaterialShaderMapId()
		: QualityLevel(EMaterialQualityLevel::High)
		, FeatureLevel(ERHIFeatureLevel::SM5)
#if WITH_EDITOR
		, Usage(EMaterialShaderMapUsage::Default)
		, bIsCookedId(false)
		, bUsingNewHLSLGenerator(false)
		, SubstrateCompilationConfig()
#endif
	{ }

	~FMaterialShaderMapId() = default;

#if WITH_EDITOR
	ENGINE_API void SetShaderDependencies(const TArray<FShaderType*>& ShaderTypes, const TArray<const FShaderPipelineType*>& ShaderPipelineTypes, const TArray<FVertexFactoryType*>& VFTypes, EShaderPlatform ShaderPlatform);
#endif

	void Serialize(FArchive& Ar, bool bLoadingCooked);

	bool IsCookedId() const
	{
#if WITH_EDITOR
		return bIsCookedId;
#else
		return true;
#endif
	}

	bool IsValid() const
	{
#if WITH_EDITOR
		return !IsCookedId() ? BaseMaterialId.IsValid() : (CookedShaderMapIdHash != 0u);
#else
		return (CookedShaderMapIdHash != 0u);
#endif
	}

	friend uint32 GetTypeHash(const FMaterialShaderMapId& Ref)
	{
#if WITH_EDITOR
		return !Ref.IsCookedId() ? Ref.BaseMaterialId.A : (uint32)(Ref.CookedShaderMapIdHash);
#else
		// Using the a truncated form of the cooked shadermap hash value directly, this is fine assuming a good quality hash
		return (uint32)Ref.CookedShaderMapIdHash;
#endif
	}

	SIZE_T GetSizeBytes() const
	{
		return sizeof(*this)
#if WITH_EDITOR
			+ ReferencedFunctions.GetAllocatedSize()
			+ ReferencedParameterCollections.GetAllocatedSize()
			+ ShaderTypeDependencies.GetAllocatedSize()
			+ ShaderPipelineTypeDependencies.GetAllocatedSize()
			+ VertexFactoryTypeDependencies.GetAllocatedSize()
#endif
			;
	}

#if WITH_EDITOR
	/** Hashes the material-specific part of this shader map Id. */
	UE_DEPRECATED(5.8, "Use version returning uint64 (SHA1 hashes no longer used)")
	inline void GetMaterialHash(FSHAHash& OutHash, bool bWithStaticParameters) const 
	{ 
		uint64 NewHash = GetMaterialHash(bWithStaticParameters);
		OutHash = FSHA1::HashBuffer(&NewHash, sizeof(NewHash));
	}

	ENGINE_API uint64 GetMaterialHash(bool bWithStaticParameters) const;

	uint64 GetMaterialHash() const
	{
		return GetMaterialHash(true);
	}
#endif

	ENGINE_API bool Equals(const FMaterialShaderMapId& ReferenceSet, bool bWithStaticParameters) const;

	/** 
	* Tests this set against another for equality
	* 
	* @param ReferenceSet	The set to compare against
	* @return				true if the sets are equal
	*/
	bool operator==(const FMaterialShaderMapId& ReferenceSet) const
	{
		return Equals(ReferenceSet, true);
	}

	bool operator!=(const FMaterialShaderMapId& ReferenceSet) const
	{
		return !Equals(ReferenceSet, true);
	}

	/** Ensure content is valid - for example overrides are set deterministically for serialization and sorting */
	bool IsContentValid() const;

	inline EShaderPermutationFlags GetPermutationFlags() const { return GetShaderPermutationFlags(LayoutParams); }

#if WITH_EDITOR
	/** Updates the Id's static parameter set data. Reset the override parameters for deterministic serialization *and* comparison */
	void UpdateFromParameterSet(const FStaticParameterSet& StaticParameters);

	/** Appends string representations of this Id to a key string. */
	void AppendKeyString(FString& KeyString, bool bIncludeSourceAndMaterialState = true,
		bool bIncludeKeyStringShaderDependencies = true) const;
	void RecordAndEmit(FMaterialKeyGeneratorContext& Context);

	const TArray<FStaticSwitchParameter> &GetStaticSwitchParameters() const 					{ return StaticSwitchParameters; }
	const TArray<FStaticComponentMaskParameter> &GetStaticComponentMaskParameters() const 		{ return StaticComponentMaskParameters; }
	const TArray<FStaticTerrainLayerWeightParameter> &GetTerrainLayerWeightParameters() const 	{ return TerrainLayerWeightParameters; }
	const TOptional<FMaterialLayersFunctions::ID>& GetMaterialLayersId() const					{ return MaterialLayersId; }

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	bool ContainsShaderType(const FShaderType* ShaderType, int32 PermutationId) const;

	/** Returns true if the requested shader type is a dependency of this shader map Id. */
	bool ContainsShaderPipelineType(const FShaderPipelineType* ShaderPipelineType) const;

	/** Returns true if the requested vertex factory type is a dependency of this shader map Id. */
	bool ContainsVertexFactoryType(const FVertexFactoryType* VFType) const;
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	void AppendStaticParameters(FShaderKeyGenerator& KeyGen) const;
#endif
};

/**
 * The shaders which the render the material on a mesh generated by a particular vertex factory type.
 */
class FMeshMaterialShaderMap : public FShaderMapContent
{
	DECLARE_TYPE_LAYOUT(FMeshMaterialShaderMap, NonVirtual);
public:
	FMeshMaterialShaderMap(EShaderPlatform InPlatform, const FHashedName& InVertexFactoryTypeName) 
		: FShaderMapContent(InPlatform)
		, VertexFactoryTypeName(InVertexFactoryTypeName)
	{}

	// Accessors.
	inline const FHashedName& GetVertexFactoryTypeName() const { return VertexFactoryTypeName; }

private:
	/** The vertex factory type these shaders are for. */
	LAYOUT_FIELD(FHashedName, VertexFactoryTypeName);
};

struct FMaterialProcessedSource
{
	DECLARE_TYPE_LAYOUT(FMaterialProcessedSource, NonVirtual);
public:
	FMaterialProcessedSource() {}
	FMaterialProcessedSource(const FHashedName& InName, const TCHAR* InSource) : Name(InName), Source(InSource) {}

	LAYOUT_FIELD(FHashedName, Name);
	LAYOUT_FIELD(FMemoryImageString, Source);
};

class FMaterialShaderMapContent : public FShaderMapContent
{
	friend class FMaterialShaderMap;
	DECLARE_TYPE_LAYOUT(FMaterialShaderMapContent, NonVirtual);
public:
	using Super = FShaderMapContent;

	inline explicit FMaterialShaderMapContent(EShaderPlatform InPlatform = EShaderPlatform::SP_NumPlatforms) : FShaderMapContent(InPlatform) {}
	~FMaterialShaderMapContent();

	inline uint32 GetNumShaders() const
	{
		uint32 NumShaders = Super::GetNumShaders();
		for (FMeshMaterialShaderMap* MeshShaderMap : OrderedMeshShaderMaps)
		{
			NumShaders += MeshShaderMap->GetNumShaders();
		}
		return NumShaders;
	}

	inline uint32 GetNumShaderPipelines() const
	{
		uint32 NumPipelines = Super::GetNumShaderPipelines();
		for (FMeshMaterialShaderMap* MeshShaderMap : OrderedMeshShaderMaps)
		{
			NumPipelines += MeshShaderMap->GetNumShaderPipelines();
		}
		return NumPipelines;
	}

	void Finalize(const FShaderMapResourceCode* Code);

private:
	struct FProjectMeshShaderMapToKey
	{
		inline const FHashedName& operator()(const FMeshMaterialShaderMap* InShaderMap) { return InShaderMap->GetVertexFactoryTypeName(); }
	};

	ENGINE_API const FMeshMaterialShaderMap* GetMeshShaderMap(const FHashedName& VertexFactoryTypeName) const;
	ENGINE_API FMeshMaterialShaderMap* AcquireMeshShaderMap(const FHashedName& VertexFactoryTypeName);

	void AddMeshShaderMap(const FHashedName& VertexFactoryTypeName, FMeshMaterialShaderMap* MeshShaderMap);
	void RemoveMeshShaderMap(const FHashedName& VertexFactoryTypeName);

	/** The material's mesh shader maps, indexed by VFType->GetId(), for fast lookup at runtime. */
	LAYOUT_FIELD(TMemoryImageArray<TMemoryImagePtr<FMeshMaterialShaderMap>>, OrderedMeshShaderMaps);

	/** Uniform expressions generated from the material compile. */
	LAYOUT_FIELD(FMaterialCompilationOutput, MaterialCompilationOutput);

	LAYOUT_FIELD(FShaderHash, ShaderContentHash);

	LAYOUT_FIELD(FScriptName, UserSceneTextureOutput);
	LAYOUT_FIELD_INITIALIZED(int32, UserTextureDivisorX, 0);
	LAYOUT_FIELD_INITIALIZED(int32, UserTextureDivisorY, 0);
	LAYOUT_FIELD(FScriptName, ResolutionRelativeToInput);

	LAYOUT_FIELD_EDITORONLY(TMemoryImageArray<FMaterialProcessedSource>, ShaderProcessedSource);
	LAYOUT_FIELD_EDITORONLY(FMemoryImageString, FriendlyName);
	LAYOUT_FIELD_EDITORONLY(FMemoryImageString, DebugDescription);
	LAYOUT_FIELD_EDITORONLY(FMemoryImageString, MaterialPath);
};

/**
 * The set of material shaders for a single material.
 */
class FMaterialShaderMap : public TShaderMap<FMaterialShaderMapContent, FShaderMapPointerTable>, public FDeferredCleanupInterface
{
public:
	using Super = TShaderMap<FMaterialShaderMapContent, FShaderMapPointerTable>;

	/**
	 * Finds the shader map for a material.
	 * @param ShaderMapId - The static parameter set and other properties identifying the shader map
	 * @param Platform - The platform to lookup for
	 * @return NULL if no cached shader map was found.
	 */
	static TRefCountPtr<FMaterialShaderMap> FindId(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform);

#if WITH_EDITOR
	ENGINE_API static FMaterialShaderMap* FindCompilingShaderMap(uint32 CompilingId);

	/** Gets outdated types from all loaded material shader maps */
	static void GetAllOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes);

	/** 
	 * Attempts to load the shader map for the given material from the Derived Data Cache.
	 * If InOutShaderMap is valid, attempts to load the individual missing shaders instead.
	 * Returns (via OutDDCKeyDesc parameter) a helpful string to debug the DDC key and parameters
	 */
	static void LoadFromDerivedDataCache(const FMaterial* Material, const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TRefCountPtr<FMaterialShaderMap>& InOutShaderMap, FString& OutDDCKeyDesc);

	/** A context returned by BeginLoadFromDerivedDataCache that can be queried to know the state of the async request. */
	struct FAsyncLoadContext
	{
		virtual ~FAsyncLoadContext() = default;
		virtual bool IsReady() const = 0;
		virtual TRefCountPtr<FMaterialShaderMap> Get() = 0;
	};

	/**
	 * Begin an attempts to load the shader map for the given material from the Derived Data Cache.
	 * If InOutShaderMap is valid, attempts to load the individual missing shaders instead.
	 * Returns (via OutDDCKeyDesc parameter) a helpful string to debug the DDC key and parameters
	 */
	static TSharedRef<FAsyncLoadContext> BeginLoadFromDerivedDataCache(const FMaterial* Material, const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TRefCountPtr<FMaterialShaderMap>& InOutShaderMap, FString& OutDDCKeyDesc);
#endif

	FMaterialShaderMap();
	virtual ~FMaterialShaderMap();
	
#if WITH_EDITOR
	FMaterialShaderMap* AcquireFinalizedClone();
	FMaterialShaderMap* GetFinalizedClone() const;
#endif // WITH_EDITOR

	// ShaderMap interface
	TShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId = 0) const
	{
		FShader* Shader = GetContent()->GetShader(ShaderType, PermutationId);
		return Shader ? TShaderRef<FShader>(Shader, *this) : TShaderRef<FShader>();
	}
	template<typename ShaderType> TShaderRef<ShaderType> GetShader(int32 PermutationId = 0) const
	{
		return TShaderRef<ShaderType>::Cast(GetShader(&ShaderType::GetStaticType(), PermutationId));
	}
	template<typename ShaderType> TShaderRef<ShaderType> GetShader(const typename ShaderType::FPermutationDomain& PermutationVector) const
	{
		return TShaderRef<ShaderType>::Cast(GetShader(&ShaderType::GetStaticType(), PermutationVector.ToDimensionValueId()));
	}

	uint32 GetMaxNumInstructionsForShader(FShaderType* ShaderType) const { return GetContent()->GetMaxNumInstructionsForShader(*this, ShaderType); }

#if WITH_EDITOR

	/** Submits compile jobs for this shadermap, returns number of jobs submitted. */
	int32 SubmitCompileJobs(uint32 CompilingShaderMapId,
		const FMaterial* Material,
		const TRefCountPtr<FSharedShaderCompilerEnvironment>& MaterialEnvironment,
		EShaderCompileJobPriority Priority) const;

	/**
	 * Compiles the shaders for a material and caches them in this shader map.
	 * @param Material - The material to compile shaders for.
	 * @param ShaderMapId - the set of static parameters to compile for
	 * @param Platform - The platform to compile to
	 */
	void Compile(
		FMaterial* Material,
		const FMaterialShaderMapId& ShaderMapId, 
		const TRefCountPtr<FSharedShaderCompilerEnvironment>& MaterialEnvironment,
		const FMaterialCompilationOutput& InMaterialCompilationOutput,
		EShaderPlatform Platform,
		EMaterialShaderPrecompileMode PrecompileMode
		);

	/** Sorts the incoming compiled jobs into the appropriate mesh shader maps */
	void ProcessCompilationResults(const TArray<TRefCountPtr<FShaderCommonCompileJob>>& ICompilationResults, int32& InOutJobIndex, float& TimeBudget);
#endif

	/**
	 * Checks whether the material shader map is missing any shader types necessary for the given material.
	 * @param Material - The material which is checked.
	 * @return True if the shader map has all of the shader types necessary.
	 */
	bool IsComplete(const FMaterial* Material, bool bSilent);

	/**
	 * Collect all possible PSO's which can be used with this material shader map for given parameters
	 */
	FPSOPrecacheDataArray CollectPSOPrecacheData(const FMaterialPSOPrecacheParams& PrecacheParams);

#if WITH_EDITOR
	ENGINE_API const FMemoryImageString* GetShaderSource(const FVertexFactoryType* VertexFactoryType, const FShaderType* ShaderType, int32 PermutationId) const;
#endif

	/** Builds a list of the shaders in a shader map. */
	ENGINE_API void GetShaderList(TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;

	/** Builds a list of the shaders in a shader map. Key is FShaderType::TypeName */
	ENGINE_API virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	
#if !UE_BUILD_SHIPPING
	virtual void DumpShaderList(FShaderListReport& Out) const;
#endif // !UE_BUILD_SHIPPING

	/** Builds a list of the shader pipelines in a shader map. */
	ENGINE_API virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;


	/** Number of Shaders in Shadermap */
	ENGINE_API uint32 GetShaderNum() const;

	/** Registers a material shader map in the global map so it can be used by materials. */
	void Register(EShaderPlatform InShaderPlatform);

	/** Registers a material shader map in the global map so it can be used by materials.
		Since the shader maps content can change this will overwrite entries in the global map.
	*/
	void RegisterForODSC(EShaderPlatform InShaderPlatform);

	// Reference counting.
	ENGINE_API void AddRef();
	ENGINE_API void Release();

	/** Serializes the shader map. */
	bool Serialize(FShaderSerializeContext& Ctx);

#if WITH_EDITOR
	/** Saves this shader map to the derived data cache. */
	void SaveToDerivedDataCache(const FConservativeShaderParameters& ShaderParameters);
#endif

	/** Serializes a shader map to an archive (used with recompiling shaders for a remote console) */
	ENGINE_API static void SaveForRemoteRecompile(FArchive& Ar, const TMap<FString, TArray<TRefCountPtr<FMaterialShaderMap> > >& CompiledShaderMaps);
	ENGINE_API static void LoadForRemoteRecompile(FArchive& Ar, EShaderPlatform ShaderPlatform, TArray<UMaterialInterface*>& OutLoadedMaterials);

#if WITH_EDITOR
	/** Returns the maximum number of texture samplers used by any shader in this shader map. */
	uint32 GetMaxTextureSamplers() const;

	void GetOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const;
	void SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& SaveKeyVal);
#endif

	// Accessors.
	const FMeshMaterialShaderMap* GetMeshShaderMap(const FVertexFactoryType* VertexFactoryType) const { return GetContent()->GetMeshShaderMap(VertexFactoryType->GetHashedName()); }
	const FMeshMaterialShaderMap* GetMeshShaderMap(const FHashedName& VertexFactoryTypeName) const { return GetContent()->GetMeshShaderMap(VertexFactoryTypeName); }
	FMeshMaterialShaderMap* AcquireMeshShaderMap(const FVertexFactoryType* VertexFactoryType) { return GetMutableContent()->AcquireMeshShaderMap(VertexFactoryType->GetHashedName()); }
	FMeshMaterialShaderMap* AcquireMeshShaderMap(const FHashedName& VertexFactoryTypeName) { return GetMutableContent()->AcquireMeshShaderMap(VertexFactoryTypeName); }
	const FMaterialShaderMapId& GetShaderMapId() const { return ShaderMapId; }
	const FShaderHash& GetShaderContentHash() const { return GetContent()->ShaderContentHash; }

#if WITH_EDITOR
	uint32 AcquireCompilingId(const TRefCountPtr<FSharedShaderCompilerEnvironment>& InMaterialEnvironment);
	void ReleaseCompilingId();
	uint32 GetCompilingId() const { return CompilingId; }
	const TRefCountPtr<FSharedShaderCompilerEnvironment>& GetPendingCompilerEnvironment() const { return PendingCompilerEnvironment; }
#endif // WITH_EDITOR

	bool IsCompilationFinalized() const { return bCompilationFinalized; }
	bool CompiledSuccessfully() const { return bCompiledSuccessfully; }
#if WITH_EDITOR
	void SetCompiledSuccessfully(bool bSuccess) { bCompiledSuccessfully = bSuccess; }
	void AddCompilingDependency(FMaterial* Material);
	void RemoveCompilingDependency(FMaterial* Material);
#endif // WITH_EDITOR

#if WITH_ODSC
	bool IsFromODSC() const { return bIsFromODSC; }
	void SetIsFromODSC(bool bInIsFromODSC) { bIsFromODSC = bInIsFromODSC; }
#endif

#if WITH_EDITORONLY_DATA
	const TCHAR* GetFriendlyName() const { return *GetContent()->FriendlyName; }
	const TCHAR* GetDebugDescription() const { return *GetContent()->DebugDescription; }
	const TCHAR* GetMaterialPath() const { return *GetContent()->MaterialPath; }
#else
	const TCHAR* GetFriendlyName() const { return TEXT(""); }
	const TCHAR* GetDebugDescription() const { return TEXT(""); }
	const TCHAR* GetMaterialPath() const { return TEXT(""); }
#endif
	const FMaterialCompilationOutput& GetCompilationOutput() const { return GetContent()->MaterialCompilationOutput; }
	bool RequiresSceneColorCopy() const { return GetContent()->MaterialCompilationOutput.RequiresSceneColorCopy(); }
	bool NeedsSceneTextures() const { return GetContent()->MaterialCompilationOutput.bNeedsSceneTextures; }
	bool UsesGlobalDistanceField() const { return GetContent()->MaterialCompilationOutput.bUsesGlobalDistanceField; }
	bool UsesWorldPositionOffset() const { return GetContent()->MaterialCompilationOutput.bUsesWorldPositionOffset; }
	bool UsesTemporalResponsiveness() const { return GetContent()->MaterialCompilationOutput.bUsesTemporalResponsiveness; }
	bool UsesMotionVectorWorldOffset() const { return GetContent()->MaterialCompilationOutput.bUsesMotionVectorWorldOffset; }
	bool UsesDisplacement() const { return GetContent()->MaterialCompilationOutput.bUsesDisplacement; }
	bool NeedsGBuffer() const { return GetContent()->MaterialCompilationOutput.NeedsGBuffer(); }
	bool UsesEyeAdaptation() const { return GetContent()->MaterialCompilationOutput.bUsesEyeAdaptation; }
	bool ModifiesMeshPosition() const { return GetContent()->MaterialCompilationOutput.bModifiesMeshPosition; }
	bool UsesPixelDepthOffset() const { return GetContent()->MaterialCompilationOutput.bUsesPixelDepthOffset; }
	bool UsesSceneDepthLookup() const { return GetContent()->MaterialCompilationOutput.UsesSceneDepthLookup(); }
	bool UsesVelocitySceneTexture() const { return GetContent()->MaterialCompilationOutput.UsesVelocitySceneTexture(); }
	bool UsesDistanceCullFade() const { return GetContent()->MaterialCompilationOutput.bUsesDistanceCullFade; }
	bool UsesAnisotropy() const { return GetContent()->MaterialCompilationOutput.bUsesAnisotropy; }
	bool UsesVertexInterpolator() const { return GetContent()->MaterialCompilationOutput.bUsesVertexInterpolator; }
	bool IsLightFunctionAtlasCompatible() const { return GetContent()->MaterialCompilationOutput.bIsLightFunctionAtlasCompatible; }

	const FSubstrateMaterialCompilationOutput& GetSubstrateMaterialCompilationOutput() const { return GetContent()->MaterialCompilationOutput.SubstrateMaterialCompilationOutput; }
	uint8 GetSubstrateMaterialType() const { return GetSubstrateMaterialCompilationOutput().SubstrateMaterialType; }
	uint8 GetSubstrateClosureCount() const { return GetSubstrateMaterialCompilationOutput().SubstrateClosureCount; }
	uint8 GetSubstrateUintPerPixel() const { return GetSubstrateMaterialCompilationOutput().SubstrateUintPerPixel; }
	ESubstrateBsdfFeature GetSubstrateMaterialBsdfFeatures() const { return ESubstrateBsdfFeature(GetSubstrateMaterialCompilationOutput().SubstrateMaterialBsdfFeatures); }
	ESubstrateTileType GetSubstrateTileType() const { return GetSubstrateTileTypeFromMaterialType(GetSubstrateMaterialCompilationOutput().SubstrateMaterialType); }

#if WITH_EDITOR
	uint32 GetNumUsedUVScalars() const { return GetContent()->MaterialCompilationOutput.NumUsedUVScalars; }
	uint32 GetNumUsedCustomInterpolatorScalars() const { return GetContent()->MaterialCompilationOutput.NumUsedCustomInterpolatorScalars; }
	void GetEstimatedNumTextureSamples(uint32& VSSamples, uint32& PSSamples) const { VSSamples = GetContent()->MaterialCompilationOutput.EstimatedNumTextureSamplesVS; PSSamples = GetContent()->MaterialCompilationOutput.EstimatedNumTextureSamplesPS; }
	uint32 GetEstimatedNumVirtualTextureLookups() const { return GetContent()->MaterialCompilationOutput.EstimatedNumVirtualTextureLookups; }
	using FLWCUsagesArray = TStaticArray<uint16, (int)ELWCFunctionKind::Max>;
	void GetEstimatedLWCFuncUsages(FLWCUsagesArray& UsagesVS, FLWCUsagesArray& UsagesPS, FLWCUsagesArray& UsagesCS) const
	{
		CopyAssignItems(UsagesVS.GetData(), GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesVS, (int)ELWCFunctionKind::Max);
		CopyAssignItems(UsagesPS.GetData(), GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesPS, (int)ELWCFunctionKind::Max);
		CopyAssignItems(UsagesCS.GetData(), GetContent()->MaterialCompilationOutput.EstimatedLWCFuncUsagesCS, (int)ELWCFunctionKind::Max);
	}
	ENGINE_API void GetEstimatedLWCFuncUsageComplexity(uint32& LWCComplexityVS, uint32& LWCComplexityPS, uint32& LWCComplexityCS) const;
	uint32 GetNumPreshaders() const { return GetContent()->MaterialCompilationOutput.UniformExpressionSet.UniformPreshaders.Num(); }
#endif
	uint32 GetNumVirtualTextureStacks() const { return GetContent()->MaterialCompilationOutput.UniformExpressionSet.VTStacks.Num(); }
	uint8 GetRuntimeVirtualTextureOutputAttributeMask() const { return GetContent()->MaterialCompilationOutput.RuntimeVirtualTextureOutputAttributeMask; }
	bool UsesSceneTexture(uint32 TexId) const { return (GetContent()->MaterialCompilationOutput.UsedSceneTextures & (1ull << TexId)) != 0; }
	TConstArrayView<FScriptName> GetUserSceneTextureInputs() const { return GetContent()->MaterialCompilationOutput.UserSceneTextureInputs; }
	FScriptName GetUserSceneTextureOutput() const { return GetContent()->UserSceneTextureOutput; }
	FIntPoint GetUserTextureDivisor() const { return FIntPoint(GetContent()->UserTextureDivisorX, GetContent()->UserTextureDivisorY); }
	FScriptName GetResolutionRelativeToInput() const { return GetContent()->ResolutionRelativeToInput; }

	bool UsesPathTracingBufferTexture(uint32 TexId) const { return (GetContent()->MaterialCompilationOutput.UsedPathTracingBufferTextures & (1ull << TexId)) != 0;}

	bool IsValidForRendering(bool bFailOnInvalid = false) const
	{
#if 0
		// Any material being used for rendering should be frozen, and successfully compiled
		// Compiliation may not be finalized yet, if the shader map is still compiling
		const bool bValid = bCompiledSuccessfully && GetFrozenContentSize() > 0u;
		// && !bDeletedThroughDeferredCleanup; //deferred actually deletion will prevent the material to go away before we finish rendering
#endif
		const bool bValid = GetFrozenContentSize() > 0u;

		checkf(bValid || !bFailOnInvalid, TEXT("FMaterialShaderMap %s invalid for rendering: bCompilationFinalized: %i, bCompiledSuccessfully: %i, bDeletedThroughDeferredCleanup: %i, FrozenContentSize: %d"), GetFriendlyName(),
			bCompilationFinalized, bCompiledSuccessfully, bDeletedThroughDeferredCleanup ? 1 : 0, GetFrozenContentSize());
		return bValid;
	}

	const FUniformExpressionSet& GetUniformExpressionSet() const { return GetContent()->MaterialCompilationOutput.UniformExpressionSet; }
	const FRHIUniformBufferLayout* GetUniformBufferLayout() const { return UniformBufferLayout; }
	const FRHIUniformBufferLayout* GetUniformBufferLayoutVS() const { return UniformBufferLayoutVS; }

	int32 GetNumRefs() const { return NumRefs; }
	int32 GetRefCount() const { return NumRefs; }

	void CountNumShaders(int32& NumShaders, int32& NumPipelines) const
	{
		NumShaders = GetContent()->GetNumShaders();
		NumPipelines = GetContent()->GetNumShaderPipelines();

		for (FMeshMaterialShaderMap* MeshShaderMap : GetContent()->OrderedMeshShaderMaps)
		{
			if (MeshShaderMap)
			{
				NumShaders += MeshShaderMap->GetNumShaders();
				NumPipelines += MeshShaderMap->GetNumShaderPipelines();
			}
		}
	}
	void DumpDebugInfo(FOutputDevice& OutputDevice) const;

#if WITH_EDITOR
	void InitializeForODSC(EShaderPlatform TargetShaderPlatform, const FMaterialCompilationOutput& NewCompilationOutput);

	void AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const;
#endif

protected:
#if WITH_EDITOR
	inline void AddCompilingMaterialExternalDependency() { check(IsInGameThread()); ++CompilingMaterialNumExternalDependencies; }
	inline void RemoveCompilingMaterialExternalDependency() { check(IsInGameThread()); --CompilingMaterialNumExternalDependencies; CheckReleaseCompilingId();  }
	ENGINE_API void CheckReleaseCompilingId();
#endif

	void PostFinalizeContent() override;

private:
	/** 
	 * A global map from a material's static parameter set to any shader map cached for that material. 
	 * Note: this does not necessarily contain all material shader maps in memory.  Shader maps with the same key can evict each other.
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TMap<FMaterialShaderMapId,FMaterialShaderMap*> GIdToMaterialShaderMap[SP_NumPlatforms];
	static FCriticalSection GIdToMaterialShaderMapCS;

#if ALLOW_SHADERMAP_DEBUG_DATA
	/** 
	 * All material shader maps in memory. 
	 * No ref counting needed as these are removed on destruction of the shader map.
	 */
	static TArray<FMaterialShaderMap*> AllMaterialShaderMaps;

	/** Guards access to AllMaterialShaderMaps, which can be written to from an async loading thread. */
	static FCriticalSection AllMaterialShaderMapsGuard;
#endif

#if WITH_EDITOR
	TRefCountPtr<FMaterialShaderMap> FinalizedClone;
	TRefCountPtr<FSharedShaderCompilerEnvironment> PendingCompilerEnvironment;
	TArray<TRefCountPtr<FMaterial>> CompilingMaterialDependencies;
	int32 CompilingMaterialNumExternalDependencies = 0;
#endif // WITH_EDITOR

	FUniformBufferLayoutRHIRef UniformBufferLayout;
	FUniformBufferLayoutRHIRef UniformBufferLayoutVS;

#if ALLOW_SHADERMAP_DEBUG_DATA
	float CompileTime;
#endif

	/** The static parameter set that this shader map was compiled with and other parameters unique to this shadermap */
	FMaterialShaderMapId ShaderMapId;

	/** Tracks material resources and their shader maps that need to be compiled but whose compilation is being deferred. */
	//static TMap<TRefCountPtr<FMaterialShaderMap>, TArray<FMaterial*> > ShaderMapsBeingCompiled;

#if WITH_EDITOR
	/** Uniquely identifies this shader map during compilation, needed for deferred compilation where shaders from multiple shader maps are compiled together. */
	uint32 CompilingId = 0;
#endif // WITH_EDITOR

	mutable std::atomic<int32> NumRefs;

	/** Used to catch errors where the shader map is deleted directly. */
	bool bDeletedThroughDeferredCleanup;

	/** Indicates whether this shader map has been registered in GIdToMaterialShaderMap */
	uint32 bRegistered : 1;

	/** 
	 * Indicates whether this shader map has had ProcessCompilationResults called after Compile.
	 * The shader map must not be used on the rendering thread unless bCompilationFinalized is true.
	 */
	uint32 bCompilationFinalized : 1;

	uint32 bCompiledSuccessfully : 1;

	/** Indicates whether the shader map should be stored in the shader cache. */
	uint32 bIsPersistent : 1;

	/** Indicates whether the shader map is from ODSC */
	uint32 bIsFromODSC : 1;

#if WITH_EDITOR
	FShader* ProcessCompilationResultsForSingleJob(class FShaderCompileJob* SingleJob, const FShaderPipelineType* ShaderPipeline, uint64 MaterialShaderMapHash);
#endif

#if WITH_EDITOR && ALLOW_SHADERMAP_DEBUG_DATA
	// InFlightJobs is only populated during cooking as its current use is to track cook hangs.
	// if we need to generalize this later threadsafety will need to be added to make this work with ODSC
	// where the same shadermap can be accessed simultaneously both on GT and RT
	mutable TMap<uint32, TPair<FString, TSharedPtr<FShaderCompileJobStatus>>> InFlightJobs;
#endif

	/** Iterate over global MaterialShaderMap cache and remove MaterialShaderMap which reference FShaderMapResource argument */
	static void ShaderMapResourceExplicitRelease(const FShaderMapResource* ShaderMapResource);

	friend ENGINE_API void DumpMaterialStats( EShaderPlatform Platform );
	friend class FShaderCompilingManager;
	friend class UE::Cook::FODSCClientData;
};

/** Flags that define the behavior of material translation validation. */
enum class EMaterialTranslateValidationFlags : uint32
{
	None = 0,
	Strict_RuntimeVirtualTexture = 1 << 0,
};
ENUM_CLASS_FLAGS(EMaterialTranslateValidationFlags)

/** 
 * Enum that contains entries for the ways that material properties need to be compiled.
 * This 'inherits' from EMaterialProperty in the sense that all of its values start after the values in EMaterialProperty.
 * Each material property is compiled once for its usual shader frequency, determined by GetShaderFrequency(),
 * And then this enum contains entries for extra compiles of a material property with a different shader frequency.
 * This is necessary for material properties which need to be evaluated in multiple shader frequencies.
 */
enum ECompiledMaterialProperty
{
	CompiledMP_EmissiveColorCS = MP_MAX,
	CompiledMP_PrevWorldPositionOffset,
	CompiledMP_MAX
};

/** 
* Enum that contains entries for the ways that material properties can be compiled with partial derivative calculations.
* Standard material shaders using the automatic hardware FiniteDifferences, Nanite uses Analytic, and we will likely
* need to add a separate one for raytracing at a later date.
*/
enum ECompiledPartialDerivativeVariation
{
	CompiledPDV_FiniteDifferences,
	CompiledPDV_Analytic,
	CompiledPDV_MAX
};

/**
 * Uniquely identifies a material expression output. 
 * Used by the material compiler to keep track of which output it is compiling.
 */
class FMaterialExpressionKey
{
public:
	UMaterialExpression* Expression;
	int32 OutputIndex;
	/** Attribute currently being compiled through a MatterialAttributes connection. */
	FGuid MaterialAttributeID;
	// Expressions are different (e.g. ResolvedView.PrevWorldViewOrigin) when using previous frame's values, value if from FHLSLMaterialTranslator::bCompilingPreviousFrame
	bool bCompilingPreviousFrameKey;

	FMaterialExpressionKey(UMaterialExpression* InExpression, int32 InOutputIndex) :
		Expression(InExpression),
		OutputIndex(InOutputIndex),
		MaterialAttributeID(FGuid(0,0,0,0)),
		bCompilingPreviousFrameKey(false)
	{}

	FMaterialExpressionKey(UMaterialExpression* InExpression, int32 InOutputIndex, const FGuid& InMaterialAttributeID, bool bInCompilingPreviousFrameKey) :
		Expression(InExpression),
		OutputIndex(InOutputIndex),
		MaterialAttributeID(InMaterialAttributeID),
		bCompilingPreviousFrameKey(bInCompilingPreviousFrameKey)
	{}


	friend bool operator==(const FMaterialExpressionKey& X, const FMaterialExpressionKey& Y)
	{
		return X.Expression == Y.Expression && X.OutputIndex == Y.OutputIndex && X.MaterialAttributeID == Y.MaterialAttributeID && X.bCompilingPreviousFrameKey == Y.bCompilingPreviousFrameKey;
	}

	friend uint32 GetTypeHash(const FMaterialExpressionKey& ExpressionKey)
	{
		return PointerHash(ExpressionKey.Expression);
	}
};

/** Function specific compiler state. */
class FMaterialFunctionCompileState
{
public:
	explicit FMaterialFunctionCompileState(UMaterialExpressionMaterialFunctionCall* InFunctionCall)
		: FunctionCall(InFunctionCall)
	{}

	~FMaterialFunctionCompileState()
	{
		ClearSharedFunctionStates();
	}

	FMaterialFunctionCompileState* FindOrAddSharedFunctionState(FMaterialExpressionKey& ExpressionKey, class UMaterialExpressionMaterialFunctionCall* SharedFunctionCall)
	{
		if (FMaterialFunctionCompileState** ExistingState = SharedFunctionStates.Find(ExpressionKey))
		{
			return *ExistingState;
		}
		return SharedFunctionStates.Add(ExpressionKey, new FMaterialFunctionCompileState(SharedFunctionCall));
	}

	void ClearSharedFunctionStates()
	{
		for (auto SavedStateIt = SharedFunctionStates.CreateIterator(); SavedStateIt; ++SavedStateIt)
		{
			FMaterialFunctionCompileState* SavedState = SavedStateIt.Value();
			SavedState->ClearSharedFunctionStates();
			delete SavedState;
		}
		SharedFunctionStates.Empty();
	}

	void Reset()
	{
		ExpressionStack.Empty();
		ExpressionCodeMap.Empty();
		ClearSharedFunctionStates();
	}

	class UMaterialExpressionMaterialFunctionCall* FunctionCall;

	// Stack used to avoid re-entry within this function
	TArray<FMaterialExpressionKey> ExpressionStack;

	/** A map from material expression to the index into CodeChunks of the code for the material expression. */
	TMap<FMaterialExpressionKey,int32> ExpressionCodeMap;

	TMap<UMaterialExpression*, int32> ExecExpressionCodeMap;

private:
	/** Cache of MaterialFunctionOutput CodeChunks.  Allows for further reuse than just the ExpressionCodeMap */
	TMap<FMaterialExpressionKey, FMaterialFunctionCompileState*> SharedFunctionStates;
};

/** Returns whether the given expression class is allowed. */
UE_DEPRECATED(5.8, "Should call UMaterialExpression::IsAllowedIn on the Class' DefaultObject")
extern ENGINE_API bool IsAllowedExpressionType(const UClass* Class, const bool bMaterialFunction);

/** Parses a string into multiple lines, for use with tooltips. */
extern ENGINE_API void ConvertToMultilineToolTip(const FString& InToolTip, const int32 TargetLineLength, TArray<FString>& OutToolTip);

/** Given a combination of EMaterialValueType flags, get text descriptions of all types */
UE_DEPRECATED(5.8, "Use the GetMaterialValueTypeDescriptions(EMaterialValueType...) overload instead")
extern ENGINE_API void GetMaterialValueTypeDescriptions(const uint32 MaterialValueType, TArray<FText>& OutDescriptions);
extern ENGINE_API void GetMaterialValueTypeDescriptions(const EMaterialValueType MaterialValueType, TArray<FText>& OutDescriptions);

/** Check whether a combination of EMaterialValueType flags can be connected */
UE_DEPRECATED(5.8, "Use the CanConnectMaterialValueTypes(EMaterialValueType, EMaterialValueType) overload instead")
extern ENGINE_API bool CanConnectMaterialValueTypes(const uint32 InputType, const uint32 OutputType);
extern ENGINE_API bool CanConnectMaterialValueTypes(const EMaterialValueType InputType, const EMaterialValueType OutputType);

/**
 * FMaterial serves 3 intertwined purposes:
 *   Represents a material to the material compilation process, and provides hooks for extensibility (CompileProperty, etc)
 *   Represents a material to the renderer, with functions to access material properties
 *   Stores a cached shader map, and other transient output from a compile, which is necessary with async shader compiling
 *      (when a material finishes async compilation, the shader map and compile errors need to be stored somewhere)
 */
class FMaterial
{
public:	
#if UE_CHECK_FMATERIAL_LIFETIME
	ENGINE_API uint32 AddRef() const;
	ENGINE_API uint32 Release() const;
	inline uint32 GetRefCount() const { return uint32(NumDebugRefs.GetValue()); }

	mutable FThreadSafeCounter NumDebugRefs;
#else
	inline uint32 AddRef() const { return 0u; }
	inline uint32 Release() const { return 0u; }
	inline uint32 GetRefCount() const { return 0u; }
#endif

	/** Sets shader maps on the specified materials without blocking. */
	ENGINE_API static void SetShaderMapsOnMaterialResources(const TMap<TRefCountPtr<FMaterial>, TRefCountPtr<FMaterialShaderMap>>& MaterialsToUpdate);

	ENGINE_API static void DeferredDelete(FMaterial* Material);

	ENGINE_API static void DeleteMaterialsOnRenderThread(TArray<TRefCountPtr<FMaterial>>& MaterialsRenderThread);

	template<typename TMaterial>
	static void DeferredDeleteArray(TArray<TRefCountPtr<TMaterial>>& Materials)
	{
		if (Materials.Num() > 0)
		{
			TArray<TRefCountPtr<FMaterial>> MaterialsRenderThread;
			for (TRefCountPtr<TMaterial>& Material : Materials)
			{
				FMaterial* MaterialToDestroy = Material.GetReference();
				MaterialToDestroy->PrepareDestroy_GameThread();
				MaterialsRenderThread.Emplace(MoveTemp(Material));
			}

			Materials.Empty();
			DeleteMaterialsOnRenderThread(MaterialsRenderThread);
		}
	}

	template<typename TMaterial>
	static void DeferredDeleteArray(TArray<TMaterial*>& Materials)
	{
		if (Materials.Num() > 0)
		{
			TArray<TRefCountPtr<FMaterial>> MaterialsRenderThread;
			for (FMaterial* Material : Materials)
			{
				Material->PrepareDestroy_GameThread();
				MaterialsRenderThread.Emplace(Material);
			}

			Materials.Empty();
			DeleteMaterialsOnRenderThread(MaterialsRenderThread);
		}
	}

	/**
	 * Minimal initialization constructor.
	 */
	FMaterial():
		RenderingThreadShaderMap(NULL),
		QualityLevel(EMaterialQualityLevel::Num),
		FeatureLevel(ERHIFeatureLevel::Num),
#if WITH_EDITOR
		ShaderPlatform(EShaderPlatform::SP_NumPlatforms),
#endif
		bContainsInlineShaders(false),
		bLoadedCookedShaderMapId(false),
		bAllowPixelDepthOffset(true)
	{
#if UE_CHECK_FMATERIAL_LIFETIME
		bOwnerBeginDestroyed = false;
#endif
	}

	/**
	 * Destructor
	 */
	ENGINE_API virtual ~FMaterial();

	/**
	 * Prepares to destroy the material, must be called from game thread
	 * Returns 'true' if PrepareDestroy_RenderThread() is required
	 */
	ENGINE_API virtual bool PrepareDestroy_GameThread();

	/**
	  * Prepares to destroy the material, must be called from render thread, only if PrepareDestroy_GameThread() returned true
	  */
	ENGINE_API virtual void PrepareDestroy_RenderThread();

	/**
	 * Caches the material shaders for this material on the given platform.
	 * This is used by material resources of UMaterials.
	 */
	UE_DEPRECATED(5.7, "Call CacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API bool CacheShaders(EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);
	ENGINE_API bool CacheShaders(EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);

	/**
	 * Caches the material shaders for the given static parameter set and platform.
	 * This is used by material resources of UMaterialInstances.
	 */
	UE_DEPRECATED(5.7, "Call CacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API bool CacheShaders(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);
	ENGINE_API bool CacheShaders(const FMaterialShaderMapId& ShaderMapId, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);

#if WITH_EDITOR

	/**
	 * Begins caching the material shaders for the given static parameter set and platform.
	 * This is used by material resources of UMaterials.
	 */
	UE_DEPRECATED(5.7, "Call BeginCacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API void BeginCacheShaders(EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback = nullptr);
	ENGINE_API void BeginCacheShaders(EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback = nullptr);
	
	/**
	 * Begins caching the material shaders for the given static parameter set and platform.
	 * This is used by material resources of UMaterialInstances.
	 */
	UE_DEPRECATED(5.7, "Call BeginCacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API void BeginCacheShaders(const FMaterialShaderMapId& ShaderMapId, EShaderPlatform Platform, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback = nullptr);
	ENGINE_API void BeginCacheShaders(const FMaterialShaderMapId& ShaderMapId, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr, TUniqueFunction<void(bool bSuccess)>&& CompletionCallback = nullptr);

	/**
	 * Returns whether or not a material caching is still pending.
	 */
	ENGINE_API bool IsCachingShaders() const;

	/**
	 * Finishes any pending material caching.
	 */
	ENGINE_API bool FinishCacheShaders() const;

	/**
	 * Submits local compile jobs for the exact given shader types and vertex factory type combination.
	 * @note CacheShaders() should be called first to prepare the resource for compilation.
	 * @note The arrays of shader types, pipeline types and vertex factory types must match.
	 * @note Entries in the PipelineTypes and ShaderTypes arrays can contain null entries.
	 * @note These compile jobs are submitted async and it is up to client code to block on results if needed.
	 */
	UE_DEPRECATED(5.7, "Call CacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API void CacheGivenTypes(EShaderPlatform ShaderPlatform, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*>& PipelineTypes, const TArray<const FShaderType*>& ShaderTypes);
	ENGINE_API void CacheGivenTypes(const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*>& PipelineTypes, const TArray<const FShaderType*>& ShaderTypes);

	void AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const;
#endif

	/**
	 * Collect all possible PSO's  which can be used with this material for given parameters - PSOs will be async precached
	 */
	UE_DEPRECATED(5.7, "Call CollectPSOs without ERHIFeatureLevel::Type as its now cached inside the FMaterial")
	ENGINE_API FGraphEventArray CollectPSOs(ERHIFeatureLevel::Type InFeatureLevel, const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs);
	ENGINE_API FGraphEventArray CollectPSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs);

	/**
	 * Collect all PSO request information already done for this material
	 */
	ENGINE_API TArray<FMaterialPSOPrecacheRequestID> GetMaterialPSOPrecacheRequestIDs() const;

	/**
	 * Clear all cached PSO data because manager has been reset
	 */
	void ClearPrecachedPSORequestIDs();

	/**
	 * Should the shader for this material with the given platform, shader type and vertex 
	 * factory type combination be compiled
	 *
	 * @param ShaderType	Which shader is being compiled
	 * @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	 *
	 * @return true if the shader should be compiled
	 */
	UE_DEPRECATED(5.7, "Call ShouldCachePipeline without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API virtual bool ShouldCache(EShaderPlatform Platform, const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const final;
	ENGINE_API virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const;

	UE_DEPRECATED(5.7, "Call ShouldCachePipeline without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API bool ShouldCachePipeline(EShaderPlatform Platform, const FShaderPipelineType* PipelineType, const FVertexFactoryType* VertexFactoryType) const;
	ENGINE_API bool ShouldCachePipeline(const FShaderPipelineType* PipelineType, const FVertexFactoryType* VertexFactoryType) const;

	/** Serializes the material. */
	ENGINE_API virtual void LegacySerialize(FArchive& Ar);

	/** Serializes the shader map inline in this material, including any shader dependencies. */
	void SerializeInlineShaderMap(FArchive& Ar, const FName& SerializingAsset = NAME_None);

	/** Serializes the shader map inline in this material, including any shader dependencies. */
	void RegisterInlineShaderMap(bool bLoadingCooked);

	/** Releases this material's shader map.  Must only be called on materials not exposed to the rendering thread! */
	void ReleaseShaderMap();

	/** Discards loaded shader maps if the application can't render */
	void DiscardShaderMap();

	// Material properties.

	struct FBuildShaderMapIdArgs
	{
		FMaterialShaderMapId* OutId = nullptr; // Guaranteed non-null in BuildShaderMapIdOverride

		EShaderPlatform Platform = EShaderPlatform::SP_NumPlatforms;
		const ITargetPlatform* TargetPlatform = nullptr;
		EMaterialKeyInclude IncludeFlags = EMaterialKeyInclude::All;
		FString BaseMaterialPath;
	};

	UE_DEPRECATED(5.7, "Call BuildShaderMapId instead")
	virtual void GetShaderMapId(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform,
		FMaterialShaderMapId& OutId) const
	{
		BuildShaderMapId(OutId, TargetPlatform);
	}

	/** Returns the ShaderMapId used in DDC and IncrementalCook for the ShaderMap created for this FMaterial. */
	UE_DEPRECATED(5.7, "Call BuildShaderMapId without EShaderPlatform as its now cached inside the FMaterial")
	void BuildShaderMapId(FMaterialShaderMapId& OutId, EShaderPlatform Platform, const ITargetPlatform* TargetPlatform,
		EMaterialKeyInclude IncludeFlags = EMaterialKeyInclude::All) const
	{
		BuildShaderMapId(OutId, TargetPlatform, IncludeFlags);
	}
	
	void BuildShaderMapId(FMaterialShaderMapId& OutId, const ITargetPlatform* TargetPlatform,
		EMaterialKeyInclude IncludeFlags = EMaterialKeyInclude::All) const
	{
		BuildShaderMapIdOverride({
			.OutId = &OutId, .Platform = GetShaderPlatform(), .TargetPlatform = TargetPlatform, .IncludeFlags = IncludeFlags, .BaseMaterialPath = GetBaseMaterialPathName()}
			);
	}

protected:
	/**
	 * The function that should be override by subclasses that want to modify either the DDC key or the
	 * IncrementalCook key. */
	ENGINE_API virtual void BuildShaderMapIdOverride(const FBuildShaderMapIdArgs& Args) const;

public:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Call GetStaticParameterSet without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API	virtual void GetStaticParameterSet(EShaderPlatform Platform, FStaticParameterSet& OutSet) const final;
	ENGINE_API virtual void GetStaticParameterSet(FStaticParameterSet& OutSet) const;
#endif // WITH_EDITORONLY_DATA
	virtual EMaterialDomain GetMaterialDomain() const = 0; // See EMaterialDomain.
	virtual bool IsTwoSided() const = 0;
	virtual bool IsThinSurface() const = 0;
	virtual bool IsDitheredLODTransition() const = 0;
	virtual bool IsTranslucencyWritingCustomDepth() const { return false; }
	virtual bool IsTranslucencyWritingVelocity() const { return false; }
	virtual bool IsTranslucencyVelocityFromDepth() const{ return false; }
	virtual bool IsTranslucencyWritingFrontLayerTransparency() const { return false; }
	virtual bool IsTangentSpaceNormal() const { return false; }
	virtual bool ShouldGenerateSphericalParticleNormals() const { return false; }
	virtual	bool ShouldDisableDepthTest() const { return false; }
	virtual	bool ShouldWriteOnlyAlpha() const { return false; }
	virtual	bool ShouldEnableResponsiveAA() const { return false; }
	virtual bool ShouldDoSSR() const { return false; }
	virtual bool ShouldDoContactShadows() const { return false; }
	virtual	bool HasPixelAnimation() const { return false; }
	virtual	bool UsesTemporalResponsiveness() const { return false; }
	virtual	bool UsesMotionVectorWorldOffset() const { return false; }
	virtual bool IsLightFunction() const = 0;
	virtual bool IsUsedWithEditorCompositing() const { return false; }
	virtual bool IsDeferredDecal() const = 0;
	virtual bool IsUIMaterial() const = 0;
	virtual bool IsVolumetricPrimitive() const = 0;
	virtual bool IsWireframe() const = 0;
	virtual bool IsPostProcessMaterial() const { return false; }
	virtual bool IsSpecialEngineMaterial() const = 0;
	virtual bool IsUsedWithSkeletalMesh() const { return false; }
	virtual bool IsUsedWithLandscape() const { return false; }
	virtual bool IsUsedWithParticleSystem() const { return false; }
	virtual bool IsUsedWithParticleSprites() const { return false; }
	virtual bool IsUsedWithBeamTrails() const { return false; }
	virtual bool IsUsedWithMeshParticles() const { return false; }
	virtual bool IsUsedWithNiagaraSprites() const { return false; }
	virtual bool IsUsedWithNiagaraRibbons() const { return false; }
	virtual bool IsUsedWithNiagaraMeshParticles() const { return false; }
	virtual bool IsUsedWithStaticLighting() const { return false; }
	virtual	bool IsUsedWithMorphTargets() const { return false; }
	virtual bool IsUsedWithSplineMeshes() const { return false; }
	virtual bool IsUsedWithInstancedStaticMeshes() const { return false; }
	virtual bool IsUsedWithGeometryCollections() const { return false; }
	virtual bool IsUsedWithAPEXCloth() const { return false; }
	virtual bool IsUsedWithNanite() const { return false; }
	virtual bool IsUsedWithVoxels() const { return false; }
	virtual bool IsUsedWithCurves() const { return false; }
	virtual bool IsUsedWithVolumetricCloud() const { return false; }
	virtual bool IsUsedWithHeterogeneousVolumes() const { return false; }
	virtual bool SamplesMaterialCache() const { return false; }
	virtual bool HasMaterialCacheOutput() const { return false; }
	virtual bool IsUsedWithUI() const { return false; }
	virtual bool IsUsedWithGeometryCache() const { return false; }
	virtual bool IsUsedWithWater() const { return false; }
	virtual bool IsUsedWithHairStrands() const { return false; }
	virtual bool IsUsedWithLidarPointCloud() const { return false; }
	virtual bool IsUsedWithVirtualHeightfieldMesh() const { return false; }
	virtual bool IsUsedWithNeuralNetworks() const { return false; }
	virtual bool IsUsedWithMeshDeformer() const { return false; }
	virtual bool IsUsedWithInstancedSkinnedMesh() const { return false; }
	virtual bool IsUsedWithStaticMesh() const { return true; }
	virtual bool HasAnyUsage() const { return true; }
	virtual bool IsFullyRough() const { return false; }
	virtual bool GetForceCompatibleWithLightFunctionAtlas() const { return false; }
	virtual bool UseNormalCurvatureToRoughness() const { return false; }
	virtual enum EMaterialFloatPrecisionMode GetMaterialFloatPrecisionMode() const { return EMaterialFloatPrecisionMode::MFPM_Default; };
	virtual bool IsUsingAlphaToCoverage() const { return false; }
	virtual bool IsUsingPreintegratedGFForSimpleIBL() const { return false; }
	virtual bool IsUsingHQForwardReflections() const { return false; }
	virtual bool GetForwardBlendsSkyLightCubemaps() const { return false; }
	virtual bool IsUsingPlanarForwardReflections() const { return false; }
	virtual bool IsNonmetal() const { return false; }
	virtual bool UseLmDirectionality() const { return true; }
	virtual bool IsMasked() const = 0;
	virtual bool IsDitherMasked() const { return false; }
	virtual bool AllowNegativeEmissiveColor() const { return false; }
	virtual enum EBlendMode GetBlendMode() const = 0;
	ENGINE_API virtual enum ERefractionMode GetRefractionMode() const;
	UE_DEPRECATED(5.7, "This is not longer valid since even when not plugged in, the default value on the root node can be used")
	virtual bool GetRootNodeOverridesDefaultRefraction() const = 0;
	virtual FMaterialShadingModelField GetShadingModels() const = 0;
	virtual bool IsShadingModelFromMaterialExpression() const = 0;
	virtual enum ETranslucencyLightingMode GetTranslucencyLightingMode() const { return TLM_VolumetricNonDirectional; };
	virtual float GetOpacityMaskClipValue() const = 0;
	virtual bool GetCastDynamicShadowAsMasked() const = 0;
	virtual bool IsDistorted() const { return false; };
	virtual ERefractionCoverageMode GetRefractionCoverageMode() const { return RCM_CoverageIgnored; }
	virtual EPixelDepthOffsetMode GetPixelDepthOffsetMode() const { return PDOM_Legacy; }
	virtual float GetTranslucencyDirectionalLightingIntensity() const { return 1.0f; }
	virtual float GetTranslucentShadowDensityScale() const { return 1.0f; }
	virtual float GetTranslucentSelfShadowDensityScale() const { return 1.0f; }
	virtual float GetTranslucentSelfShadowSecondDensityScale() const { return 1.0f; }
	virtual float GetTranslucentSelfShadowSecondOpacity() const { return 1.0f; }
	virtual float GetTranslucentBackscatteringExponent() const { return 1.0f; }
	virtual FDisplacementScaling GetDisplacementScaling() const { return FDisplacementScaling(); }
	virtual bool IsDisplacementFadeEnabled() const { return false; }
	virtual FDisplacementFadeRange GetDisplacementFadeRange() const { return FDisplacementFadeRange(); }
	virtual bool IsTranslucencyAfterDOFEnabled() const { return false; }
	virtual bool IsTranslucencyAfterMotionBlurEnabled() const { return false; }
	UE_DEPRECATED(5.7, "Call IsDualBlendingEnabled without EShaderPlatform as its now cached inside the FMaterial")
	virtual bool IsDualBlendingEnabled(EShaderPlatform Platform) const final { return false; }
	virtual bool IsDualBlendingEnabled() const { return false; }
	virtual bool IsMobileSeparateTranslucencyEnabled() const { return false; }
	virtual FLinearColor GetTranslucentMultipleScatteringExtinction() const { return FLinearColor::White; }
	virtual float GetTranslucentShadowStartOffset() const { return 0.0f; }
	virtual float GetRefractionDepthBiasValue() const { return 0.0f; }
	virtual bool ShouldApplyFogging() const { return false; }
	virtual bool ShouldApplyCloudFogging() const { return false; }
	virtual bool ShouldAlwaysEvaluateWorldPositionOffset() const { return false; }
	virtual bool ComputeFogPerPixel() const { return false; }
	virtual bool IsSky() const { return false; }
	virtual bool HasSubstrateRoughnessTracking() const { return true; }
	virtual bool AllowTranslucentLocalLightShadow() const { return false;  }
	virtual float GetTranslucentLocalLightShadowQuality() const { return 0.f;  }
	virtual float GetTranslucentDirectionalLightShadowQuality() const { return 0.f;  }
	virtual FString GetFriendlyName() const = 0;
	/** Similar to GetFriendlyName, but but avoids historical behavior of the former, returning the exact asset name for material instances instead of just the material. */
	virtual FString GetAssetName() const { return GetFriendlyName(); }

#if WITH_EDITOR
	/** Similar to GetFriendlyName, but appends a hash of shader map ID portion of the DDC key (excluding source file hashes) to differentiate materials/material 
		instances that would collide when considering the base material name alone (this name is used when creating debug output paths for disambiguation). */
	UE_DEPRECATED(5.7, "Call GetUniqueAssetName without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API FString GetUniqueAssetName(EShaderPlatform Platform, const FMaterialShaderMapId& ShaderMapId) const;
	ENGINE_API FString GetUniqueAssetName(const FMaterialShaderMapId& ShaderMapId) const;
#else
	/** GetUniqueAssetName just returns GetAssetName in non-editor builds */
	UE_DEPRECATED(5.7, "Call GetUniqueAssetName without EShaderPlatform as its now cached inside the FMaterial")
	FString GetUniqueAssetName(EShaderPlatform Platform, const FMaterialShaderMapId& ShaderMapId) const { return GetAssetName(); }
	FString GetUniqueAssetName(const FMaterialShaderMapId& ShaderMapId) const { return GetAssetName(); }
#endif
	virtual bool HasVertexPositionOffsetConnected() const { return false; }
	virtual bool HasPixelDepthOffsetConnected() const { return false; }
	virtual uint32 GetMaterialDecalResponse() const { return 0; }
	virtual bool HasBaseColorConnected() const { return false; }
	virtual bool HasNormalConnected() const { return false; }
	virtual bool HasRoughnessConnected() const { return false; }
	virtual bool HasSpecularConnected() const { return false; }
	virtual bool HasMetallicConnected() const { return false; }
	virtual bool HasEmissiveColorConnected() const { return false; }
	virtual bool HasAnisotropyConnected() const { return false; }
	virtual bool HasAmbientOcclusionConnected() const { return false; }
	virtual bool HasMaterialPropertyConnected(EMaterialProperty In) const { return false; }
	virtual bool HasDisplacementConnected() const { return false; }
	virtual bool IsSubstrateMaterial() const { return false; }
	virtual bool RequiresSynchronousCompilation() const { return false; };
	virtual bool IsDefaultMaterial() const { return false; };
	virtual int32 GetNumCustomizedUVs() const { return 0; }
	virtual int32 GetNumMaterialCacheTags() const { return 0; }
	virtual int32 GetBlendableLocation() const { return 0; }
	virtual int32 GetBlendablePriority() const { return 0; }
	virtual bool GetBlendableOutputAlpha() const { return false; }
	virtual bool GetDisablePreExposureScale() const { return false; }
	virtual bool IsStencilTestEnabled() const { return false; }
	virtual uint32 GetStencilRefValue() const { return 0; }
	virtual uint32 GetStencilCompare() const { return 0; }
	virtual bool HasPerInstanceCustomData() const { return false; }
	virtual bool HasPerInstanceRandom() const { return false; }
	virtual bool HasVertexInterpolator() const { return false; }
	virtual bool HasRuntimeVirtualTextureOutput() const { return false; }
	virtual bool HasFirstPersonOutput() const { return false; }
	virtual bool CastsRayTracedShadows() const { return true; }
	virtual bool IsTessellationEnabled() const { return false; }
	virtual bool HasRenderTracePhysicalMaterialOutputs() const { return false; }
	virtual uint16 GetPreshaderGap() const { return 0; }
	virtual EMaterialShadingRate GetShadingRate() const { return MSR_1x1; }
	virtual int32 GetNeuralProfileId() const { return INDEX_NONE; }
	virtual bool IsVariableRateShadingAllowed() const { return true; }
	virtual void GetShaderTags(TArray<FName>& OutShaderTags) const {};
	virtual EMaterialTranslateValidationFlags GetMaterialTranslateValidationFlags() const { return EMaterialTranslateValidationFlags::None; }

	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	virtual bool IsPersistent() const = 0;
	virtual UMaterialInterface* GetMaterialInterface() const { return NULL; }
	/** For tracking ownership data */
	FName GetOwnerFName() const;

	virtual bool IsPreview() const { return false; }

	ENGINE_API const FMaterialCachedExpressionData& GetCachedExpressionData() const;

	/** Is the material required to be complete? Default materials, special engine materials, etc */
	ENGINE_API bool IsRequiredComplete() const;

#if WITH_EDITOR
	/**
	* Called when compilation of an FMaterial finishes, after the GameThreadShaderMap is set and the render command to set the RenderThreadShaderMap is queued
	*/
	virtual void NotifyCompilationFinished() { }

	/**
	* Cancels all outstanding compilation jobs for this material.
	*/
	ENGINE_API void CancelCompilation();

	/** 
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	ENGINE_API void FinishCompilation();

	/** 
	 * Blocks until compilation has completed. Returns immediately if a compilation is not outstanding.
	 */
	ENGINE_API static void FinishCompilation(const TCHAR* MaterialName, const TArray<FMaterial*>& MaterialsToCompile);

	/**
	 * Checks if the compilation for this shader is finished
	 * 
	 * @return returns true if compilation is complete false otherwise
	 */
	ENGINE_API bool IsCompilationFinished() const;

	/** Is the material using the new (WIP) HLSL generator? */
	ENGINE_API virtual bool IsUsingNewHLSLGenerator() const;

	/** Get to the Substrate compilation config */
	ENGINE_API virtual const FSubstrateCompilationConfig& GetSubstrateCompilationConfig() const;
	/** Set the Substrate compilation config */
	ENGINE_API virtual void SetSubstrateCompilationConfig(FSubstrateCompilationConfig& SubstrateCompilationConfig);

#endif // WITH_EDITOR

	/**
	* Checks if there is a valid GameThreadShaderMap, that is, the material can be rendered as intended.
	*
	* @return returns true if there is a GameThreadShaderMap.
	*/
	ENGINE_API bool HasValidGameThreadShaderMap() const;

	/** Returns whether this material should be considered for casting dynamic shadows. */
	inline bool ShouldCastDynamicShadows() const
	{
		const EBlendMode BlendMode = GetBlendMode();
		return !GetShadingModels().HasOnlyShadingModel(MSM_SingleLayerWater) &&
				(IsOpaqueBlendMode(BlendMode)
 				 || IsMaskedBlendMode(BlendMode)
  				 || (IsTranslucentOnlyBlendMode(BlendMode) && GetCastDynamicShadowAsMasked()));
	}


	inline EMaterialQualityLevel::Type GetQualityLevel() const
	{
		return QualityLevel;
	}

#if WITH_EDITOR
	FUniformParameterOverrides TransientOverrides;
#endif // WITH_EDITOR

	// Accessors.
	ENGINE_API const FUniformExpressionSet& GetUniformExpressions() const;
	ENGINE_API TArrayView<const FMaterialTextureParameterInfo> GetUniformTextureExpressions(EMaterialTextureParameterType Type) const;
	ENGINE_API TConstArrayView<FMaterialTextureCollectionParameterInfo> GetUniformTextureCollectionExpressions() const;
	ENGINE_API TArrayView<const FMaterialNumericParameterInfo> GetUniformNumericParameterExpressions() const;

	inline TArrayView<const FMaterialTextureParameterInfo> GetUniform2DTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::Standard2D); }
	inline TArrayView<const FMaterialTextureParameterInfo> GetUniformCubeTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::Cube); }
	inline TArrayView<const FMaterialTextureParameterInfo> GetUniform2DArrayTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::Array2D); }
	inline TArrayView<const FMaterialTextureParameterInfo> GetUniformVolumeTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::Volume); }
	inline TArrayView<const FMaterialTextureParameterInfo> GetUniformVirtualTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::Virtual); }
	inline TArrayView<const FMaterialTextureParameterInfo> GetUniformSparseVolumeTextureExpressions() const { return GetUniformTextureExpressions(EMaterialTextureParameterType::SparseVolume); }

#if WITH_EDITOR
	const TArray<FString>& GetCompileErrors() const { return CompileErrors; }
	void SetCompileErrors(const TArray<FString>& InCompileErrors) { CompileErrors = InCompileErrors; }
	const TArray<UMaterialExpression*>& GetErrorExpressions() const { return ObjectPtrDecay(ErrorExpressions); }
	const FGuid& GetLegacyId() const { return Id_DEPRECATED; }
	const FString& GetDebugGroupName() const { return DebugGroupName; }
#endif // WITH_EDITOR

	inline const FStaticFeatureLevel GetFeatureLevel() const { checkSlow(FeatureLevel != ERHIFeatureLevel::Num); return FeatureLevel; }
	inline const EShaderPlatform GetShaderPlatform() const 
	{ 
#if WITH_EDITOR
		checkSlow(ShaderPlatform != EShaderPlatform::SP_NumPlatforms); return ShaderPlatform; 
#else
		return GMaxRHIShaderPlatform;
#endif
	}
	bool GetUsesDynamicParameter() const 
	{ 
		//@todo - remove non-dynamic parameter particle VF and always support dynamic parameter
		return true; 
	}
	ENGINE_API bool RequiresSceneColorCopy_GameThread() const;
	ENGINE_API bool RequiresSceneColorCopy_RenderThread() const;
	ENGINE_API bool NeedsSceneTextures() const;
	ENGINE_API bool NeedsGBuffer() const;
	ENGINE_API bool UsesEyeAdaptation() const;	
	ENGINE_API bool UsesGlobalDistanceField_GameThread() const;

	/** Does the material use world position offset. */
	ENGINE_API bool MaterialUsesWorldPositionOffset_RenderThread() const;
	ENGINE_API bool MaterialUsesWorldPositionOffset_GameThread() const;

	/** Does the material use displacement. */
	ENGINE_API bool MaterialUsesDisplacement_RenderThread() const;
	ENGINE_API bool MaterialUsesDisplacement_GameThread() const;

	/** Does the material use a pixel depth offset. */
	ENGINE_API bool MaterialUsesPixelDepthOffset_RenderThread() const;
	ENGINE_API bool MaterialUsesPixelDepthOffset_GameThread() const;

	/** Does the material use temporal responsiveness. */
	ENGINE_API bool MaterialUsesTemporalResponsiveness_RenderThread() const;
	ENGINE_API bool MaterialUsesTemporalResponsiveness_GameThread() const;

	/** Does the material use a motion vector offset. */
	ENGINE_API bool MaterialUsesMotionVectorWorldOffset_RenderThread() const;
	ENGINE_API bool MaterialUsesMotionVectorWorldOffset_GameThread() const;

	/** Does the material modify the mesh position. */
	ENGINE_API bool MaterialModifiesMeshPosition_RenderThread() const;
	ENGINE_API bool MaterialModifiesMeshPosition_GameThread() const;

	/** Does the material use a distance cull fade. */
	ENGINE_API bool MaterialUsesDistanceCullFade_GameThread() const;

	/** Does the material use a SceneDepth lookup. */
	ENGINE_API bool MaterialUsesSceneDepthLookup_RenderThread() const;
	ENGINE_API bool MaterialUsesSceneDepthLookup_GameThread() const;

	/** The material usage mask of CustomDepth and CustomStencil*/
	ENGINE_API uint8 GetCustomDepthStencilUsageMask_GameThread() const;

	/** Note: This function is only intended for use in deciding whether or not shader permutations are required before material translation occurs. */
	ENGINE_API bool MaterialMayModifyMeshPosition() const;

	/** Get the runtime virtual texture output attribute mask for the material. */
	ENGINE_API uint8 GetRuntimeVirtualTextureOutputAttibuteMask_GameThread() const;
	ENGINE_API uint8 GetRuntimeVirtualTextureOutputAttibuteMask_RenderThread() const;

	ENGINE_API bool MaterialUsesAnisotropy_GameThread() const;
	ENGINE_API bool MaterialUsesAnisotropy_RenderThread() const;

	/** Returns whether the material uses any Vertex Interpolators(not just contains). */
	ENGINE_API bool MaterialUsesVertexInterpolator_GameThread() const;
	ENGINE_API bool MaterialUsesVertexInterpolator_RenderThread() const;

	/** Get whether or not the material is compatible with the light function atlas. */
	ENGINE_API bool MaterialIsLightFunctionAtlasCompatible_GameThread() const;
	ENGINE_API bool MaterialIsLightFunctionAtlasCompatible_RenderThread() const;

	/** Get Substrate material type (single, single, complex slab). */
	ENGINE_API uint8 MaterialGetSubstrateMaterialType_GameThread() const;
	ENGINE_API uint8 MaterialGetSubstrateMaterialType_RenderThread() const;

	/** Get Substrate material BSDF count. */
	ENGINE_API uint8 MaterialGetSubstrateClosureCount_GameThread() const;
	ENGINE_API uint8 MaterialGetSubstrateClosureCount_RenderThread() const;

	/** Get Substrate material uint count per pixel. */
	ENGINE_API uint8 MaterialGetSubstrateUintPerPixel_GameThread() const;
	ENGINE_API uint8 MaterialGetSubstrateUintPerPixel_RenderThread() const;

	/** Get Substrate tile type requirement. */
	ENGINE_API ESubstrateTileType MaterialGetSubstrateTileType_GameThread() const;
	ENGINE_API ESubstrateTileType MaterialGetSubstrateTileType_RenderThread() const;

	/** Get Substrate material BSDFs features. */
	ENGINE_API ESubstrateBsdfFeature MaterialGetSubstrateMaterialBsdfFeatures_GameThread() const;
	ENGINE_API ESubstrateBsdfFeature MaterialGetSubstrateMaterialBsdfFeatures_RenderThread() const;

	class FMaterialShaderMap* GetGameThreadShaderMap() const 
	{ 
		checkSlow(IsInGameThread() || IsInParallelGameThread() || IsInAsyncLoadingThread());
		return GameThreadShaderMap; 
	}

	ENGINE_API void SetGameThreadShaderMap(FMaterialShaderMap* InMaterialShaderMap);
	ENGINE_API void SetInlineShaderMap(FMaterialShaderMap* InMaterialShaderMap);
	ENGINE_API void UpdateInlineShaderMapIsComplete();

	ENGINE_API class FMaterialShaderMap* GetRenderingThreadShaderMap() const;

	inline bool IsGameThreadShaderMapComplete() const
	{
		checkSlow(IsInGameThread() || IsInParallelGameThread());
		return bGameThreadShaderMapIsComplete.load(std::memory_order_relaxed);
	}

	inline bool IsRenderingThreadShaderMapComplete() const
	{
		checkSlow(IsInParallelRenderingThread());
		return bRenderingThreadShaderMapIsComplete.load(std::memory_order_relaxed);
	}

	ENGINE_API void SetRenderingThreadShaderMap(TRefCountPtr<FMaterialShaderMap>& InMaterialShaderMap);

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector);

	virtual TArrayView<const TObjectPtr<UObject>> GetReferencedTextures() const = 0;
	virtual TObjectPtr<UObject> GetMatDefaultTextureAtIdx(const int32 inIdx) const = 0;

#if WITH_EDITOR
	virtual int32 GetMatDefaultTextureIdx(TObjectPtr<UObject> InTexture) const = 0;
	virtual const TArray<TObjectPtr<UObject>>& GetMatDefaultTextures() const = 0;
#endif

	virtual TConstArrayView<TObjectPtr<UTextureCollection>> GetReferencedTextureCollections() const = 0;

	/**
	 * Finds the shader matching the template type and the passed in vertex factory, asserts if not found.
	 * Note - Only implemented for FMeshMaterialShaderTypes
	 */
	template<typename ShaderType>
	TShaderRef<ShaderType> GetShader(FVertexFactoryType* VertexFactoryType, const typename ShaderType::FPermutationDomain& PermutationVector, bool bFatalIfMissing = true) const
	{
		return GetShader<ShaderType>(VertexFactoryType, PermutationVector.ToDimensionValueId(), bFatalIfMissing);
	}

	template <typename ShaderType>
	TShaderRef<ShaderType> GetShader(FVertexFactoryType* VertexFactoryType, int32 PermutationId = 0, bool bFatalIfMissing = true) const
	{
		return TShaderRef<ShaderType>::Cast(GetShader(&ShaderType::GetStaticType(), VertexFactoryType, PermutationId, bFatalIfMissing));
	}

	ENGINE_API FShaderPipelineRef GetShaderPipeline(class FShaderPipelineType* ShaderPipelineType, FVertexFactoryType* VertexFactoryType, bool bFatalIfNotFound = true) const;

	ENGINE_API bool TryGetShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType, FMaterialShaders& OutShaders) const;

	ENGINE_API bool HasShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const;

	UE_DEPRECATED(5.7, "Call ShouldCacheShaders without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API bool ShouldCacheShaders(const EShaderPlatform ShaderPlatform, const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const;
	ENGINE_API bool ShouldCacheShaders(const FMaterialShaderTypes& InTypes, const FVertexFactoryType* InVertexFactoryType) const;

#if WITH_EDITOR
	ENGINE_API void SubmitCompileJobs_GameThread(EShaderCompileJobPriority Priority);
	ENGINE_API void SubmitCompileJobs_RenderThread(EShaderCompileJobPriority Priority) const;
#endif // WITH_EDITOR

	/** Returns a string that describes the material's usage for debugging purposes. */
	virtual FString GetMaterialUsageDescription() const = 0;

	/** Returns true if this material is allowed to make development shaders via the global CVar CompileShadersForDevelopment. */
	virtual bool GetAllowDevelopmentShaderCompile()const{ return true; }

	/** Returns which shadermap this material is bound to. */
	virtual EMaterialShaderMapUsage::Type GetMaterialShaderMapUsage() const { return EMaterialShaderMapUsage::Default; }

	/**
	* Get user source code for the material, with a list of code snippets to highlight representing the code for each MaterialExpression
	* @param OutSource - generated source code
	* @param OutHighlightMap - source code highlight list
	* @return - true on Success
	*/
	ENGINE_API bool GetMaterialExpressionSource(FString& OutSource);

	/** Returns summary statistics for preshaders */
	ENGINE_API void GetPreshaderStats(uint32& TotalParameters, uint32& TotalOps) const;

	/* Helper function to look at both IsMasked and IsDitheredLODTransition to determine if it writes every pixel */
	ENGINE_API bool WritesEveryPixel(bool bShadowPass = false) const;
	ENGINE_API bool WritesEveryPixel(bool bShadowPass, bool bVFTypeSupportsNullPixelShader) const;
	ENGINE_API bool WritesEveryPixel(bool bShadowPass, bool bVFTypeSupportsNullPixelShader, bool bOpacityMaskIsOpaque) const;

	/** call during shader compilation jobs setup to fill additional settings that may be required by classes who inherit from this */
	UE_DEPRECATED(5.7, "Call SetupExtraCompilationSettings without EShaderPlatform as its now cached inside the FMaterial")
	virtual void SetupExtraCompilationSettings(const EShaderPlatform Platform, FExtraShaderCompilerSettings& Settings) const final { SetupExtraCompilationSettings(Settings); }
	virtual void SetupExtraCompilationSettings(FExtraShaderCompilerSettings& Settings) const {}

	void DumpDebugInfo(FOutputDevice& OutputDevice);
	UE_DEPRECATED(5.7, "Call SaveShaderStableKeys without EShaderPlatform as its now cached inside the FMaterial")
	void SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, struct FStableShaderKeyAndValue& SaveKeyVal); // arg is non-const, we modify it as we go
	void SaveShaderStableKeys(struct FStableShaderKeyAndValue& SaveKeyVal); // arg is non-const, we modify it as we go

#if WITH_EDITOR
	/**
	*	Gathers a list of shader types sorted by vertex factory types that should be cached for this material.  Avoids doing expensive material
	*	and shader compilation to acquire this information.
	*
	*	@param	Platform		The shader platform to get info for.
	*	@param	OutShaderInfo	Array of results sorted by vertex factory type, and shader type.
	*
	*/
	UE_DEPRECATED(5.7, "Call GetShaderTypes without EShaderPlatform and with specialized ShaderParameters")
	ENGINE_API void GetShaderTypes(EShaderPlatform Platform, const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const;

	UE_DEPRECATED(5.8, "Call GetShaderTypes with specialized ShaderParameters")
	/* Returns shader types based on conservative Shader Parameters. */
	ENGINE_API void GetShaderTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const;

	/* Returns shader types based on given Shader Parameters. */
	ENGINE_API void GetShaderTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FDebugShaderTypeInfo>& OutShaderInfo, const FMaterialShaderParameters& InParameters) const;

	UE_DEPRECATED(5.7, "Call GetShaderTypesForLayout without EShaderPlatform as its now cached inside the FMaterial")
	void GetShaderTypesForLayout(EShaderPlatform Platform, const FShaderMapLayout& Layout, FVertexFactoryType* VertexFactory, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const;
	void GetShaderTypesForLayout(const FShaderMapLayout& Layout, FVertexFactoryType* VertexFactory, TArray<FDebugShaderTypeInfo>& OutShaderInfo) const;
#endif // WITH_EDITOR

#if WITH_EDITOR
	/** 
	 * Adds an FMaterial to the global list.
	 * Any FMaterials that don't belong to a UMaterialInterface need to be registered in this way to work correctly with runtime recompiling of outdated shaders.
	 */
	static void AddEditorLoadedMaterialResource(FMaterial* Material)
	{
		EditorLoadedMaterialResources.Add(Material);
	}

	/** Recompiles any materials in the EditorLoadedMaterialResources list if they are not complete. */
	static void UpdateEditorLoadedMaterialResources(EShaderPlatform InShaderPlatform);

	/** Allows to associate the shader resources with the asset for load order. */
	virtual FName GetAssetPath() const { return NAME_None; };

#endif // WITH_EDITOR

	virtual FString GetFullPath() const { return TEXT(""); };

#if WITH_EDITOR
	virtual void BeginAllowCachingStaticParameterValues() {};
	virtual void EndAllowCachingStaticParameterValues() {};
#endif // WITH_EDITOR

#if UE_CHECK_FMATERIAL_LIFETIME
	void SetOwnerBeginDestroyed() { bOwnerBeginDestroyed = true; }
	bool IsOwnerBeginDestroyed() const { return bOwnerBeginDestroyed; }
#else
	inline void SetOwnerBeginDestroyed() {}
	inline bool IsOwnerBeginDestroyed() const { return false; }
#endif

#if WITH_EDITORONLY_DATA
	/* Gather any UMaterialExpressionCustomOutput expressions they can be compiled in turn */
	virtual void GatherCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const {}

	/* Gather any UMaterialExpressionCustomOutput expressions in the material and referenced function calls */
	virtual void GatherExpressionsForCustomInterpolators(TArray<class UMaterialExpression*>& OutExpressions) const {}
#endif // WITH_EDITORONLY_DATA

#if WITH_ODSC
	uint8 GetODSCMetaData() const { return ODSCMetaData.load(std::memory_order_relaxed); };
	void SetODSCMetaData(uint8 InODSCMetaData) const { ODSCMetaData.store(InODSCMetaData, std::memory_order_relaxed); };
#endif

	virtual void FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers) {}
	virtual const FMaterialLayersFunctions* GetMaterialLayers() const { return nullptr; }

protected:
	// shared code needed for GetUniformScalarParameterExpressions, GetUniformVectorParameterExpressions, GetUniformCubeTextureExpressions..
	// @return can be 0
	const FMaterialShaderMap* GetShaderMapToUse() const;

#if WITH_EDITOR
	/**
	* Fills the passed array with IDs of shader maps unfinished compilation jobs.
	*/
	void AddShaderMapIDsWithUnfinishedCompilation(TArray<int32>& ShaderMapIds);

	uint32 GetGameThreadCompilingShaderMapId() const { return GameThreadCompilingShaderMapId; }

	/**
	 * Checks that no pre-compilation errors have been detected and if so it reports them using specified compiler.
	 * @return whether no errors occurred.
	 */
	virtual bool CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const { return true; }
#endif // WITH_EDITOR

	/**
	 * Entry point for compiling a specific material property.  This must call SetMaterialProperty. 
	 * @param OverrideShaderFrequency SF_NumFrequencies to not override
	 * @return cases to the proper type e.g. Compiler->ForceCast(Ret, GetValueType(Property));
	 */
	virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, class FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency = SF_NumFrequencies, bool bUsePreviousFrameTime = false) const = 0;
	
	/** Used to translate code for custom output attributes such as ClearCoatBottomNormal  */
	virtual int32 CompileCustomAttribute(const FGuid& AttributeID, class FMaterialCompiler* Compiler) const {return INDEX_NONE;}

	/** Retrieves the full path of the base UMaterial associated with this FMaterial */
	virtual FString GetBaseMaterialPathName() const { return TEXT(""); }
	/** Get a debug name useful for debugging, just the base material path by default */
	virtual FString GetDebugName() const { return GetBaseMaterialPathName(); }

	UE_DEPRECATED(5.7, "Please use SetQualityLevelProperties with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	void SetQualityLevelProperties(ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQualityLevel = EMaterialQualityLevel::Num)
	{
		SetQualityLevelProperties(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel), InQualityLevel);
	}
	
	void SetQualityLevelProperties(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type InQualityLevel = EMaterialQualityLevel::Num)
	{
		QualityLevel = InQualityLevel;
		
#if WITH_EDITOR
		FeatureLevel = FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(InShaderPlatform);
		ShaderPlatform = InShaderPlatform;
#else
		FeatureLevel = GMaxRHIFeatureLevel;
#endif
	}

	/** 
	 * Gets the shader map usage of the material, which will be included in the DDC key.
	 * This mechanism allows derived material classes to create different DDC keys with the same base material.
	 * For example lightmass exports diffuse and emissive, each of which requires a material resource with the same base material.
	 */
	virtual EMaterialShaderMapUsage::Type GetShaderMapUsage() const { return EMaterialShaderMapUsage::Default; }

	/** Gets the Guid that represents this material. */
	virtual FGuid GetMaterialId() const = 0;
	
	/** Produces arrays of any shader and vertex factory type that this material is dependent on, using conservative parameters. */
	UE_DEPRECATED(5.7, "Call GetDependentShaderAndVFTypes without EShaderPlatform as its now cached inside the FMaterial")
	ENGINE_API void GetDependentShaderAndVFTypes(EShaderPlatform Platform, const FPlatformTypeLayoutParameters& LayoutParams, TArray<FShaderType*>& OutShaderTypes, TArray<const FShaderPipelineType*>& OutShaderPipelineTypes, TArray<FVertexFactoryType*>& OutVFTypes) const;
	ENGINE_API void GetDependentShaderAndVFTypes(const FPlatformTypeLayoutParameters& LayoutParams, TArray<FShaderType*>& OutShaderTypes, TArray<const FShaderPipelineType*>& OutShaderPipelineTypes, TArray<FVertexFactoryType*>& OutVFTypes) const;

	bool GetLoadedCookedShaderMapId() const { return bLoadedCookedShaderMapId; }

#if WITH_EDITOR
	void SetCompilingShaderMap(FMaterialShaderMap* InMaterialShaderMap);

	/**
	 * The returned value is not const because the expression is used to build the Substrate tree and this requires to execute multiple such as link function caller, compile expression, etc.
	 * See HLSLTranslator.cpp for the single use case.
	 * @return Nullptr if this is not a material used to generate a preview of a node of a material graph.
	 */
	virtual UMaterialExpression* GetMaterialGraphNodePreviewExpression() const { return nullptr; }
#endif

	void SetAllowPixelDepthOffset(bool bEnable) { bAllowPixelDepthOffset = bEnable; }
	bool AllowPixelDepthOffset() const { return bAllowPixelDepthOffset; }
	
private:
#if WITH_EDITOR
	bool ReleaseGameThreadCompilingShaderMap();
#endif
	void ReleaseRenderThreadCompilingShaderMap();

#if WITH_EDITOR
	/** 
	 * Tracks FMaterials without a corresponding UMaterialInterface in the editor, for example FExpressionPreviews.
	 * Used to handle the 'recompileshaders changed' command in the editor.
	 * This doesn't have to use a reference counted pointer because materials are removed on destruction.
	 */
	ENGINE_API static TSet<FMaterial*> EditorLoadedMaterialResources;

	TArray<FString> CompileErrors;

	/** List of material expressions which generated a compiler error during the last compile. */
	TArray<TObjectPtr<UMaterialExpression>> ErrorExpressions;

	TSharedPtr<FMaterialShaderMap::FAsyncLoadContext> CacheShadersPending;
	TUniqueFunction<bool ()> CacheShadersCompletion;

	uint32 GameThreadCompilingShaderMapId = 0;
	uint32 RenderingThreadCompilingShaderMapId = 0;

	TRefCountPtr<FSharedShaderCompilerEnvironment> GameThreadPendingCompilerEnvironment;
	TRefCountPtr<FSharedShaderCompilerEnvironment> RenderingThreadPendingCompilerEnvironment;

	FString DebugGroupName;
#endif // WITH_EDITOR

	/** 
	 * Used to prevent submitting the material more than once during CacheMeshDrawCommands unless priority has been increased. 
	 */
	mutable std::atomic<int8> RenderingThreadShaderMapSubmittedPriority { -1 };

	/**
	 * Used to prevent submitting the material more than once.  This is accessed by just the game thread so it doesn't need to be an atomic.
	 */
	EShaderCompileJobPriority GameThreadShaderMapSubmittedPriority = EShaderCompileJobPriority::None;

	/** 
	 * Game thread tracked shader map, which is ref counted and manages shader map lifetime. 
	 * The shader map uses deferred deletion so that the rendering thread has a chance to process a release command when the shader map is no longer referenced.
	 * Code that sets this is responsible for updating RenderingThreadShaderMap in a thread safe way.
	 * During an async compile, this will be NULL and will not contain the actual shader map until compilation is complete.
	 */
	TRefCountPtr<FMaterialShaderMap> GameThreadShaderMap;

	/** 
	 * Shader map for this material resource which is accessible by the rendering thread. 
	 * This must be updated along with GameThreadShaderMap, but on the rendering thread.
	 */
	TRefCountPtr<FMaterialShaderMap> RenderingThreadShaderMap;

#if WITH_EDITOR
	/** 
	 * Legacy unique identifier of this material resource.
	 * This functionality is now provided by UMaterial::StateId.
	 */
	FGuid Id_DEPRECATED;

	/** 
	 * Contains the compiling id of this shader map when it is being compiled asynchronously. 
	 * This can be used to access the shader map during async compiling, since GameThreadShaderMap will not have been set yet.
	 */
	//TArray<int32, TInlineAllocator<1> > OutstandingCompileShaderMapIds;
#endif // WITH_EDITOR

	/** Quality level that this material is representing, may be EMaterialQualityLevel::Num if material doesn't depend on current quality level */
	EMaterialQualityLevel::Type QualityLevel;

#if WITH_ODSC
	mutable std::atomic<uint8> ODSCMetaData = {};
#endif

	/** Feature level that this material is representing. */
	ERHIFeatureLevel::Type FeatureLevel;

#if WITH_EDITOR
	/** Shader Platform that this material is representing. */
	EShaderPlatform ShaderPlatform;
#endif

	/** 
	 * Whether this material was loaded with shaders inlined. 
	 * If true, GameThreadShaderMap will contain a reference to the inlined shader map between Serialize and PostLoad.
	 */
	uint32 bContainsInlineShaders : 1;
	uint32 bLoadedCookedShaderMapId : 1;

	uint32 bAllowPixelDepthOffset : 1;

#if UE_CHECK_FMATERIAL_LIFETIME
	/** Set when the owner of this FMaterial (typically a UMaterial or UMaterialInstance) has had BeginDestroy() called */
	uint32 bOwnerBeginDestroyed : 1;
#endif // UE_CHECK_FMATERIAL_LIFETIME
	
	// These are touched by different threads and need to be outside the bitfield above.
	std::atomic<bool> bGameThreadShaderMapIsComplete{ false };
	std::atomic<bool> bRenderingThreadShaderMapIsComplete{ false };

	/** MaterialPSO request IDs currently active for this material */
	TArray<FMaterialPSOPrecacheRequestID> PrecachedPSORequestIDs;

	/** Shared critical section on PrecachedPSORequestIDs because contention should be very limited */
	static FCriticalSection PrecachedPSORequestIDsCS;

#if WITH_EDITOR
	/**
	* Compiles this material for Platform, storing the result in OutShaderMap if the compile was synchronous
	*/
	bool BeginCompileShaderMap(
		const FMaterialShaderMapId& ShaderMapId,
		const FStaticParameterSet &StaticParameterSet,
		EMaterialShaderPrecompileMode PrecompileMode,
		const ITargetPlatform* TargetPlatform = nullptr);

	bool Translate_Legacy(const FMaterialShaderMapId& InShaderMapId,
		const FStaticParameterSet& InStaticParameters,
		const ITargetPlatform* InTargetPlatform,
		FMaterialCompilationOutput& OutCompilationOutput,
		TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment);

	bool Translate_New(const FMaterialShaderMapId& InShaderMapId,
		const FStaticParameterSet& InStaticParameters,
		const ITargetPlatform* InTargetPlatform,
		FMaterialCompilationOutput& OutCompilationOutput,
		TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment);

	bool Translate(const FMaterialShaderMapId& InShaderMapId,
		const FStaticParameterSet& InStaticParameters,
		const ITargetPlatform* InTargetPlatform,
		FMaterialCompilationOutput& OutCompilationOutput,
		TRefCountPtr<FSharedShaderCompilerEnvironment>& OutMaterialEnvironment);

	/** Populates OutEnvironment with defines needed to compile shaders for this material. */
	void SetupMaterialEnvironment(
		TConstArrayView<FShaderParametersMetadata*> InUniformBufferStructs,
		const FUniformExpressionSet& InUniformExpressionSet,
		FShaderCompilerEnvironment& OutEnvironment
		) const;
#endif // WITH_EDITOR

	/** Get the float precision mode for the material and shader taking into account any project wide precision values */
	static void GetOutputPrecision(EMaterialFloatPrecisionMode FloatPrecisionMode, bool& bFullPrecisionInPS, bool& bFullPrecisionInMaterial);

	/**
	 * Finds the shader matching the template type and the passed in vertex factory, asserts if not found.
	 */
	ENGINE_API TShaderRef<FShader> GetShader(class FMeshMaterialShaderType* ShaderType, FVertexFactoryType* VertexFactoryType, int32 PermutationId, bool bFatalIfMissing = true) const;

#if WITH_EDITOR
	uint64 GetReferencedTexturesHash() const;

	uint64 GetExpressionIncludesHash() const;

	uint64 GetExternalCodeReferencesHash() const;
#endif // WITH_EDITOR

	friend class FMaterialShaderMap;
	friend class FShaderCompilingManager;
	friend class FHLSLMaterialTranslator;
	friend class FMaterialHLSLErrorHandler;
	friend class UE::Cook::FODSCClientData;
};



/**
 * Implementation of the FMaterial interface for a UMaterial or UMaterialInstance.
 */
class FMaterialResource : public FMaterial
{
public:

	ENGINE_API FMaterialResource();
	ENGINE_API virtual ~FMaterialResource();

	void SetMaterial(UMaterial* InMaterial, UMaterialInstance* InInstance, EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type InQualityLevel = EMaterialQualityLevel::Num)
	{
		Material = InMaterial;
		MaterialInstance = InInstance;
		SetQualityLevelProperties(InShaderPlatform, InQualityLevel);
	}

#if WITH_EDITOR
	/** Returns the number of samplers used in this material, or -1 if the material does not have a valid shader map (compile error or still compiling). */
	ENGINE_API int32 GetSamplerUsage() const;
#endif

#if WITH_EDITOR
	ENGINE_API void GetUserInterpolatorUsage(uint32& NumUsedUVScalars, uint32& NumUsedCustomInterpolatorScalars) const;
	ENGINE_API void GetEstimatedNumTextureSamples(uint32& VSSamples, uint32& PSSamples) const;
	ENGINE_API uint32 GetEstimatedNumVirtualTextureLookups() const;
	using FLWCUsagesArray = TStaticArray<uint16, (int)ELWCFunctionKind::Max>;
	ENGINE_API void GetEstimatedLWCFuncUsages(FLWCUsagesArray& UsagesVS, FLWCUsagesArray& UsagesPS, FLWCUsagesArray& UsagesCS) const;
#endif
	ENGINE_API uint32 GetNumVirtualTextureStacks() const;

	ENGINE_API virtual FString GetMaterialUsageDescription() const override;

	// FMaterial interface.
protected:
	ENGINE_API virtual void BuildShaderMapIdOverride(const FBuildShaderMapIdArgs& Args) const override;
public:
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual void GetStaticParameterSet(FStaticParameterSet& OutSet) const override;
#endif
#if WITH_EDITOR
	ENGINE_API virtual void BeginAllowCachingStaticParameterValues() override;
	ENGINE_API virtual void EndAllowCachingStaticParameterValues() override;
#endif // WITH_EDITOR
	ENGINE_API virtual EMaterialDomain GetMaterialDomain() const override;
	ENGINE_API virtual bool IsTwoSided() const override;
	ENGINE_API virtual bool IsThinSurface() const override;
	ENGINE_API virtual bool IsDitheredLODTransition() const override;
	ENGINE_API virtual bool IsTranslucencyWritingCustomDepth() const override;
	ENGINE_API virtual bool IsTranslucencyWritingVelocity() const override;
	ENGINE_API virtual bool IsTranslucencyVelocityFromDepth() const override;
	ENGINE_API virtual bool IsTranslucencyWritingFrontLayerTransparency() const override;
	ENGINE_API virtual bool IsTangentSpaceNormal() const override;
	ENGINE_API virtual bool ShouldGenerateSphericalParticleNormals() const override;
	ENGINE_API virtual bool ShouldDisableDepthTest() const override;
	ENGINE_API virtual bool ShouldWriteOnlyAlpha() const override;
	ENGINE_API virtual bool ShouldEnableResponsiveAA() const override;
	ENGINE_API virtual bool ShouldDoSSR() const override;
	ENGINE_API virtual bool ShouldDoContactShadows() const override;
	ENGINE_API virtual bool HasPixelAnimation() const override;
	ENGINE_API virtual bool UsesTemporalResponsiveness() const override;
	ENGINE_API virtual bool UsesMotionVectorWorldOffset() const override;
	ENGINE_API virtual bool IsLightFunction() const override;
	ENGINE_API virtual bool IsUsedWithEditorCompositing() const override;
	ENGINE_API virtual bool IsDeferredDecal() const override;
	ENGINE_API virtual bool IsVolumetricPrimitive() const override;
	ENGINE_API virtual bool IsWireframe() const override;
	ENGINE_API virtual bool IsVariableRateShadingAllowed() const override;
	ENGINE_API virtual EMaterialShadingRate  GetShadingRate() const override;
	ENGINE_API virtual bool IsUIMaterial() const override;
	ENGINE_API virtual bool IsPostProcessMaterial() const override;
	ENGINE_API virtual bool IsSpecialEngineMaterial() const override;
	ENGINE_API virtual bool IsUsedWithSkeletalMesh() const override;
	ENGINE_API virtual bool IsUsedWithLandscape() const override;
	ENGINE_API virtual bool IsUsedWithParticleSystem() const override;
	ENGINE_API virtual bool IsUsedWithParticleSprites() const override;
	ENGINE_API virtual bool IsUsedWithBeamTrails() const override;
	ENGINE_API virtual bool IsUsedWithMeshParticles() const override;
	ENGINE_API virtual bool IsUsedWithNiagaraSprites() const override;
	ENGINE_API virtual bool IsUsedWithNiagaraRibbons() const override;
	ENGINE_API virtual bool IsUsedWithNiagaraMeshParticles() const override;
	ENGINE_API virtual bool IsUsedWithStaticLighting() const override;
	ENGINE_API virtual bool IsUsedWithMorphTargets() const override;
	ENGINE_API virtual bool IsUsedWithSplineMeshes() const override;
	ENGINE_API virtual bool IsUsedWithInstancedStaticMeshes() const override;
	ENGINE_API virtual bool IsUsedWithGeometryCollections() const override;
	ENGINE_API virtual bool IsUsedWithAPEXCloth() const override;
	ENGINE_API virtual bool IsUsedWithGeometryCache() const override;
	ENGINE_API virtual bool IsUsedWithWater() const override;
	ENGINE_API virtual bool IsUsedWithHairStrands() const override;
	ENGINE_API virtual bool IsUsedWithLidarPointCloud() const override;
	ENGINE_API virtual bool IsUsedWithVirtualHeightfieldMesh() const override;
	ENGINE_API virtual bool IsUsedWithNeuralNetworks() const override;
	ENGINE_API virtual bool IsUsedWithMeshDeformer() const override;
	ENGINE_API virtual bool IsUsedWithInstancedSkinnedMesh() const override;
	ENGINE_API virtual bool IsUsedWithNanite() const override;
	ENGINE_API virtual bool IsUsedWithVoxels() const override;
	ENGINE_API virtual bool IsUsedWithCurves() const override;
	ENGINE_API virtual bool IsUsedWithVolumetricCloud() const override;
	ENGINE_API virtual bool IsUsedWithHeterogeneousVolumes() const override;
	ENGINE_API virtual bool IsUsedWithStaticMesh() const override;
	ENGINE_API virtual bool HasAnyUsage() const override;
	ENGINE_API virtual bool SamplesMaterialCache() const override;
	ENGINE_API virtual bool HasMaterialCacheOutput() const override;
	ENGINE_API virtual bool IsFullyRough() const override;
	ENGINE_API virtual bool GetForceCompatibleWithLightFunctionAtlas() const override;
	ENGINE_API virtual bool UseNormalCurvatureToRoughness() const override;
	ENGINE_API virtual enum EMaterialFloatPrecisionMode GetMaterialFloatPrecisionMode() const override;
	ENGINE_API virtual bool IsUsingAlphaToCoverage() const override;
	ENGINE_API virtual bool IsUsingPreintegratedGFForSimpleIBL() const override;
	ENGINE_API virtual bool IsUsingHQForwardReflections() const override;
	ENGINE_API virtual bool GetForwardBlendsSkyLightCubemaps() const override;
	ENGINE_API virtual bool IsUsingPlanarForwardReflections() const override;
	ENGINE_API virtual bool IsNonmetal() const override;
	ENGINE_API virtual bool UseLmDirectionality() const override;
	ENGINE_API virtual enum EBlendMode GetBlendMode() const override;
	ENGINE_API virtual enum ERefractionMode GetRefractionMode() const override;
	UE_DEPRECATED(5.7, "This is not longer valid since even when not plugged in, the default value on the root node can be used")
	ENGINE_API virtual bool GetRootNodeOverridesDefaultRefraction() const override;
	ENGINE_API virtual uint32 GetMaterialDecalResponse() const override;
	ENGINE_API virtual bool HasBaseColorConnected() const override;
	ENGINE_API virtual bool HasNormalConnected() const override;
	ENGINE_API virtual bool HasRoughnessConnected() const override;
	ENGINE_API virtual bool HasSpecularConnected() const override;
	ENGINE_API virtual bool HasMetallicConnected() const override;
	ENGINE_API virtual bool HasEmissiveColorConnected() const override;
	ENGINE_API virtual bool HasAnisotropyConnected() const override;
	ENGINE_API virtual bool HasAmbientOcclusionConnected() const override;
	ENGINE_API virtual bool HasDisplacementConnected() const override;
	ENGINE_API virtual bool IsSubstrateMaterial() const override;
	ENGINE_API virtual bool HasMaterialPropertyConnected(EMaterialProperty In) const override;
	ENGINE_API virtual FMaterialShadingModelField GetShadingModels() const override;
	ENGINE_API virtual bool IsShadingModelFromMaterialExpression() const override;
	ENGINE_API virtual enum ETranslucencyLightingMode GetTranslucencyLightingMode() const override;
	ENGINE_API virtual float GetOpacityMaskClipValue() const override;
	ENGINE_API virtual bool GetCastDynamicShadowAsMasked() const override;
	ENGINE_API virtual bool IsDistorted() const override;
	ENGINE_API virtual ERefractionCoverageMode GetRefractionCoverageMode() const override;
	ENGINE_API virtual EPixelDepthOffsetMode GetPixelDepthOffsetMode() const override;
	ENGINE_API virtual float GetTranslucencyDirectionalLightingIntensity() const override;
	ENGINE_API virtual float GetTranslucentShadowDensityScale() const override;
	ENGINE_API virtual float GetTranslucentSelfShadowDensityScale() const override;
	ENGINE_API virtual float GetTranslucentSelfShadowSecondDensityScale() const override;
	ENGINE_API virtual float GetTranslucentSelfShadowSecondOpacity() const override;
	ENGINE_API virtual float GetTranslucentBackscatteringExponent() const override;
	ENGINE_API virtual bool IsTranslucencyAfterDOFEnabled() const override;
	ENGINE_API virtual bool IsTranslucencyAfterMotionBlurEnabled() const override;
	ENGINE_API virtual bool IsDualBlendingEnabled() const override;
	ENGINE_API virtual bool IsMobileSeparateTranslucencyEnabled() const override;
	ENGINE_API virtual FDisplacementScaling GetDisplacementScaling() const override;
	ENGINE_API virtual bool IsDisplacementFadeEnabled() const override;
	ENGINE_API virtual FDisplacementFadeRange GetDisplacementFadeRange() const override;
	ENGINE_API virtual FLinearColor GetTranslucentMultipleScatteringExtinction() const override;
	ENGINE_API virtual float GetTranslucentShadowStartOffset() const override;
	ENGINE_API virtual bool IsMasked() const override;
	ENGINE_API virtual bool IsDitherMasked() const override;
	ENGINE_API virtual bool AllowNegativeEmissiveColor() const override;
	ENGINE_API virtual FString GetFriendlyName() const override;
	ENGINE_API virtual FString GetAssetName() const override;
	ENGINE_API virtual bool RequiresSynchronousCompilation() const override;
	ENGINE_API virtual bool IsDefaultMaterial() const override;
	ENGINE_API virtual int32 GetNumCustomizedUVs() const override;
	ENGINE_API virtual int32 GetNumMaterialCacheTags() const override;
	ENGINE_API virtual int32 GetBlendableLocation() const override;
	ENGINE_API virtual int32 GetBlendablePriority() const override;
	ENGINE_API virtual bool GetBlendableOutputAlpha() const override;
	ENGINE_API virtual bool GetDisablePreExposureScale() const override;
	ENGINE_API virtual bool IsStencilTestEnabled() const override;
	ENGINE_API virtual uint32 GetStencilRefValue() const override;
	ENGINE_API virtual uint32 GetStencilCompare() const override;
	ENGINE_API virtual float GetRefractionDepthBiasValue() const override;
	ENGINE_API virtual bool ShouldApplyFogging() const override;
	ENGINE_API virtual bool ShouldApplyCloudFogging() const override;
	ENGINE_API virtual bool ShouldAlwaysEvaluateWorldPositionOffset() const override;
	ENGINE_API virtual bool IsSky() const override;
	ENGINE_API virtual bool ComputeFogPerPixel() const override;
	ENGINE_API virtual bool HasPerInstanceCustomData() const override;
	ENGINE_API virtual bool HasPerInstanceRandom() const override;
	ENGINE_API virtual bool HasVertexInterpolator() const override;
	ENGINE_API virtual bool HasRuntimeVirtualTextureOutput() const override;
	ENGINE_API virtual bool HasFirstPersonOutput() const override;
	ENGINE_API virtual bool CastsRayTracedShadows() const override;
	ENGINE_API virtual bool IsTessellationEnabled() const override;
	ENGINE_API virtual bool HasRenderTracePhysicalMaterialOutputs() const override;
	ENGINE_API virtual uint16 GetPreshaderGap() const override;
	ENGINE_API virtual UMaterialInterface* GetMaterialInterface() const override;
	ENGINE_API virtual int32 GetNeuralProfileId() const override;
	ENGINE_API virtual bool HasSubstrateRoughnessTracking() const override;
	ENGINE_API virtual bool AllowTranslucentLocalLightShadow() const override;
	ENGINE_API virtual float GetTranslucentLocalLightShadowQuality() const override;
	ENGINE_API virtual float GetTranslucentDirectionalLightShadowQuality() const override;

	/**
	 * Should shaders compiled for this material be saved to disk?
	 */
	ENGINE_API virtual bool IsPersistent() const override;
	ENGINE_API virtual FGuid GetMaterialId() const override;

#if WITH_EDITOR
	ENGINE_API void GetShaderTags(TArray<FName>& OutShaderTags) const override;
	ENGINE_API EMaterialTranslateValidationFlags GetMaterialTranslateValidationFlags() const override;
	ENGINE_API virtual void NotifyCompilationFinished() override;
	/** Allows to associate the shader resources with the asset for load order. */
	ENGINE_API virtual FName GetAssetPath() const override;
	ENGINE_API virtual bool IsUsingNewHLSLGenerator() const override;
	ENGINE_API virtual bool CheckInValidStateForCompilation(FMaterialCompiler* Compiler) const override;

	/** @return The index at which the given texture sits in the default textures array for this material. */
	ENGINE_API virtual int32 GetMatDefaultTextureIdx(TObjectPtr<UObject> InTexture) const override;

	/** @return All default textures present in this material. */
	ENGINE_API virtual const TArray<TObjectPtr<UObject>>& GetMatDefaultTextures() const override;
#endif // WITH_EDITOR

	ENGINE_API virtual FString GetFullPath() const override;

	ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	ENGINE_API virtual void LegacySerialize(FArchive& Ar) override;

	/**
	 * Returns the relevant texture for the given Idx.
	 * In editor, it looks through all default textures. 
	 * In non-editor, it has access to default textures based on cook output, if ShouldStripUnusedDefaultTextures returns true.
	 * @return The texture present at the given Idx or nullptr if not available.
	 */
	ENGINE_API virtual TObjectPtr<UObject> GetMatDefaultTextureAtIdx(const int32 InIdx) const override;

	UE_DEPRECATED(5.8, "Please use the GetMatDefaultTextures for editor or GetMatDefaultTextureAtIdx for non-editor.")
	ENGINE_API virtual TArrayView<const TObjectPtr<UObject>> GetReferencedTextures() const override;
	ENGINE_API virtual TConstArrayView<TObjectPtr<UTextureCollection>> GetReferencedTextureCollections() const override;

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	ENGINE_API virtual bool GetAllowDevelopmentShaderCompile() const override;

	inline const UMaterial* GetMaterial() const { return Material; }
	inline const UMaterialInstance* GetMaterialInstance() const { return MaterialInstance; }
	inline void SetMaterial(UMaterial* InMaterial) { Material = InMaterial; }
	inline void SetMaterialInstance(UMaterialInstance* InMaterialInstance) { MaterialInstance = InMaterialInstance; }

	ENGINE_API virtual void FeedbackMaterialLayersInstancedGraphFromCompilation(const FMaterialLayersFunctions* InLayers) override;
	ENGINE_API virtual const FMaterialLayersFunctions* GetMaterialLayers() const override;
protected:
	TObjectPtr<UMaterial> Material;
	TObjectPtr<UMaterialInstance> MaterialInstance;

	FMaterialLayersFunctions MaterialLayersFunctions;

	/** Entry point for compiling a specific material property.  This must call SetMaterialProperty. */
	ENGINE_API virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, class FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const override;
	/** Used to translate code for custom output attributes such as ClearCoatBottomNormal  */
	ENGINE_API virtual int32 CompileCustomAttribute(const FGuid& AttributeID, FMaterialCompiler* Compiler) const override;

#if WITH_EDITORONLY_DATA
	/* Gives the material a chance to compile any custom output nodes it has added */
	ENGINE_API virtual void GatherCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const override;
	ENGINE_API virtual void GatherExpressionsForCustomInterpolators(TArray<class UMaterialExpression*>& OutExpressions) const override;
#endif // WITH_EDITORONLY_DATA

	ENGINE_API virtual bool HasVertexPositionOffsetConnected() const override;
	ENGINE_API virtual bool HasPixelDepthOffsetConnected() const override;

	/** Useful for debugging. */
	ENGINE_API virtual FString GetBaseMaterialPathName() const override;
	ENGINE_API virtual FString GetDebugName() const override;
};

/**
 * This class takes care of all of the details you need to worry about when modifying a UMaterial
 * on the main thread. This class should *always* be used when doing so!
 */
class FMaterialUpdateContext
{
	/** UMaterial parents of any UMaterialInterfaces updated within this context. */
	TSet<UMaterial*> UpdatedMaterials;
	/** Materials updated within this context. */
	TSet<UMaterialInterface*> UpdatedMaterialInterfaces;
	/** Active global component reregister context, if any. */
	TUniquePtr<class FGlobalComponentReregisterContext> ComponentReregisterContext;
	/** Active global component render state recreation context, if any. */
	TUniquePtr<class FGlobalComponentRecreateRenderStateContext> ComponentRecreateRenderStateContext;
	/** The shader platform that was being processed - can control if we need to update components */
	EShaderPlatform ShaderPlatform;
	/** True if the SyncWithRenderingThread option was specified. */
	bool bSyncWithRenderingThread;
	/** True if the SkipRecacheStaticPermutations option was specified. */
	bool bSkipRecacheStaticPermutations;
	/** True if the SkipUpdateStaticDrawLists option was specified. */
	bool bSkipUpdateStaticDrawLists;

public:

	/** Options controlling what is done before/after the material is updated. */
	struct EOptions
	{
		enum Type
		{
			/** Reregister all components while updating the material. */
			ReregisterComponents = 0x1,
			/**
			 * Sync with the rendering thread. This is necessary when modifying a
			 * material exposed to the rendering thread. You may omit this flag if
			 * you have already flushed rendering commands.
			 */
			SyncWithRenderingThread = 0x02,
			/* Recreates only the render state for all components (mutually exclusive with ReregisterComponents) */
			RecreateRenderStates = 0x04,
			/** Skip recache of material instance permutations. Only safe to use if you know that no shader code or static switches are changing. */
			SkipRecacheStaticPermutations = 0x08,
			/**
			 * Skip updating static draw lists.
			 * Has no meaning if ReregisterComponents or RecreateRenderStates is set as it is already skipped.
			 * Only safe to use this if you know that no static draw lists could be using the material.
			*/
			SkipUpdateStaticDrawLists = 0x10,
			/** Default options: Recreate render state, sync with rendering thread. */
			Default = RecreateRenderStates | SyncWithRenderingThread,
		};
	};

	/** Initialization constructor. */
	explicit ENGINE_API FMaterialUpdateContext(uint32 Options = EOptions::Default, EShaderPlatform InShaderPlatform = GMaxRHIShaderPlatform);

	/** Destructor. */
	ENGINE_API ~FMaterialUpdateContext();

	/** Add a material that has been updated to the context. */
	ENGINE_API void AddMaterial(UMaterial* Material);

	/** Adds a material instance that has been updated to the context. */
	ENGINE_API void AddMaterialInstance(UMaterialInstance* Instance);

	/** Adds a material interface that has been updated to the context. */
	ENGINE_API void AddMaterialInterface(UMaterialInterface* Instance);

	inline const TSet<UMaterialInterface*>& GetUpdatedMaterials() const { return UpdatedMaterialInterfaces; }
};

/**
 * Check whether the specified texture is needed to render the material instance.
 * @param Texture	The texture to check.
 * @return bool - true if the material uses the specified texture.
 */
ENGINE_API bool DoesMaterialUseTexture(const UMaterialInterface* Material,const UTexture* CheckTexture);

#if WITH_EDITORONLY_DATA
/** TODO - This can be removed whenever VER_UE4_MATERIAL_ATTRIBUTES_REORDERING is no longer relevant. */
ENGINE_API void DoMaterialAttributeReorder(FExpressionInput* Input, const FPackageFileVersion&, int32 RenderObjVer, int32 UE5MainVer);
#endif // WITH_EDITORONLY_DATA

struct FMaterialResourceLocOnDisk
{
	// Relative offset to package (uasset/umap + uexp) beginning
	uint32 Offset;
	// ERHIFeatureLevel::Type
	uint8 FeatureLevel;
	// EMaterialQualityLevel::Type
	uint8 QualityLevel;
};

inline FArchive& operator<<(FArchive& Ar, FMaterialResourceLocOnDisk& Loc)
{
	Ar << Loc.Offset;
	Ar << Loc.FeatureLevel;
	Ar << Loc.QualityLevel;
	return Ar;
}

class FMaterialResourceMemoryWriter final : public FMemoryWriter
{
public:
	FMaterialResourceMemoryWriter(FArchive& Ar);

	virtual ~FMaterialResourceMemoryWriter();

	FMaterialResourceMemoryWriter(const FMaterialResourceMemoryWriter&) = delete;
	FMaterialResourceMemoryWriter(FMaterialResourceMemoryWriter&&) = delete;
	FMaterialResourceMemoryWriter& operator=(const FMaterialResourceMemoryWriter&) = delete;
	FMaterialResourceMemoryWriter& operator=(FMaterialResourceMemoryWriter&&) = delete;

	virtual FArchive& operator<<(class FName& Name) override;

	virtual const FCustomVersionContainer& GetCustomVersions() const override { return ParentAr->GetCustomVersions(); }

	virtual FString GetArchiveName() const override { return TEXT("FMaterialResourceMemoryWriter"); }

	inline void BeginSerializingMaterialResource()
	{
		Locs.AddUninitialized();
		int64 ResourceOffset = this->Tell();
		Locs.Last().Offset = IntCastChecked<uint32>(ResourceOffset);
	}

	inline void EndSerializingMaterialResource(const FMaterialResource& Resource)
	{
		static_assert(ERHIFeatureLevel::Num <= 256, "ERHIFeatureLevel doesn't fit into a byte");
		static_assert(EMaterialQualityLevel::Num <= 256, "EMaterialQualityLevel doesn't fit into a byte");
		check(Resource.GetMaterialInterface());
		Locs.Last().FeatureLevel = uint8(Resource.GetFeatureLevel());
		Locs.Last().QualityLevel = uint8(Resource.GetQualityLevel());
	}

private:
	TArray<uint8> Bytes;
	TArray<FMaterialResourceLocOnDisk> Locs;
	TMap<FNameEntryId, int32> Name2Indices;
	FArchive* ParentAr;

	void SerializeToParentArchive();
};

class FMaterialResourceWriteScope final
{
public:
	FMaterialResourceWriteScope(
		FMaterialResourceMemoryWriter* InAr,
		const FMaterialResource& InResource) :
		Ar(InAr),
		Resource(InResource)
	{
		check(Ar);
		Ar->BeginSerializingMaterialResource();
	}

	~FMaterialResourceWriteScope()
	{
		Ar->EndSerializingMaterialResource(Resource);
	}

private:
	FMaterialResourceMemoryWriter* Ar;
	const FMaterialResource& Resource;
};

class FMaterialResourceProxyReader final : private TUniquePtr<FArchive>, public FArchiveProxy
{
public:
	FMaterialResourceProxyReader(
		FArchive& Ar,
		ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num,
		EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num);

	FMaterialResourceProxyReader(
		const TCHAR* Filename,
		uint32 NameMapOffset,
		ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num,
		EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num);

	virtual ~FMaterialResourceProxyReader();

	FMaterialResourceProxyReader(const FMaterialResourceProxyReader&) = delete;
	FMaterialResourceProxyReader(FMaterialResourceProxyReader&&) = delete;
	FMaterialResourceProxyReader& operator=(const FMaterialResourceProxyReader&) = delete;
	FMaterialResourceProxyReader& operator=(FMaterialResourceProxyReader&&) = delete;

	virtual int64 Tell() override
	{
		return InnerArchive.Tell() - OffsetToFirstResource;
	}

	virtual void Seek(int64 InPos) override
	{
		InnerArchive.Seek(OffsetToFirstResource + InPos);
	}

	virtual FArchive& operator<<(class FName& Name) override;

	virtual FString GetArchiveName() const override { return TEXT("FMaterialResourceProxyReader"); }

private:
	TArray<FName> Names;
	int64 OffsetToFirstResource;
	int64 OffsetToEnd;

	void Initialize(
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		bool bSeekToEnd = false);
};

ENGINE_API uint8 GetRayTracingMaskFromMaterial(const EBlendMode BlendMode);

//

#if WITH_EDITOR
/** An FMaterialResource and other information needed for cooking it. */
struct FMaterialResourceForCooking
{
	TRefCountPtr<FMaterialResource> Resource;
	EShaderPlatform Platform;
};

class FEditorMaterialResourceBase : public FMaterial
{
	ENGINE_API virtual const TArray<TObjectPtr<UObject>>& GetMatDefaultTextures() const override;
	ENGINE_API virtual int32 GetMatDefaultTextureIdx(TObjectPtr<UObject> InTexture) const override;
	ENGINE_API virtual TObjectPtr<UObject> GetMatDefaultTextureAtIdx(const int32 InIdx) const override;

	ENGINE_API virtual TArrayView<const TObjectPtr<UObject>> GetReferencedTextures() const override;
	ENGINE_API virtual TConstArrayView<TObjectPtr<UTextureCollection>> GetReferencedTextureCollections() const override;
protected:
	TArray<TObjectPtr<UObject>> ReferencedTextures;
	TArray<TObjectPtr<UTextureCollection>> ReferencedTextureCollections;
};

/**
 * Utility method for getting the shaders for the current platform, using FCompilationShaderParameters.
 * In case the ShaderMap is available, it uses the Compilation output within to generate the shaders.
 * In case it is not, it handles creating that output efficiently, without triggering shader compilation.
 */
ENGINE_API TArray<FDebugShaderTypeInfo> GetShadersUsingCompilationParameters_GameThread(UMaterialInterface* InMaterial, const EShaderPlatform InShaderPlatform = GMaxRHIShaderPlatform);
#endif

inline bool ShouldIncludeMaterialInDefaultOpaquePass(const FMaterial& Material)
{
	return !Material.IsSky()
		&& !Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater);
}
