// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

UE_DEPRECATED_HEADER(5.6, "Materials/MaterialParameters.h")