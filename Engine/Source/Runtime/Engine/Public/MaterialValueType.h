// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialValueType.h
=============================================================================*/

#pragma once

#include "HAL/Platform.h"

/**
 * The types which can be used by materials.
 */
enum EMaterialValueType : uint64
{
	/** 
	 * A scalar float type.  
	 * Note that MCT_Float1 will not auto promote to any other float types, 
	 * So use MCT_Float instead for scalar expression return types.
	 */
	MCT_Float1                = 1u << 0,
	MCT_Float2                = 1u << 1,
	MCT_Float3                = 1u << 2,
	MCT_Float4                = 1u << 3,

	/** 
	 * Any size float type by definition, but this is treated as a scalar which can auto convert (by replication) to any other size float vector.
	 * Use this as the type for any scalar expressions.
	 */
	MCT_Texture2D	          = 1u << 4,
	MCT_TextureCube	          = 1u << 5,
	MCT_Texture2DArray		  = 1u << 6,
	MCT_TextureCubeArray      = 1u << 7,
	MCT_VolumeTexture         = 1u << 8,
	MCT_StaticBool            = 1u << 9,
	MCT_Unknown               = 1u << 10,
	MCT_MaterialAttributes	  = 1u << 11,
	MCT_TextureExternal       = 1u << 12,
	MCT_TextureVirtual        = 1u << 13,
	MCT_SparseVolumeTexture   = 1u << 14,

	/** Used internally when sampling from virtual textures */
	MCT_VTPageTableResult     = 1u << 15,
	
	MCT_ShadingModel		  = 1u << 16,
	MCT_Substrate			  = 1u << 17,

	MCT_LWCScalar			  = 1u << 18,
	MCT_LWCVector2			  = 1u << 19,
	MCT_LWCVector3			  = 1u << 20,
	MCT_LWCVector4			  = 1u << 21,

	MCT_Execution             = 1u << 22,

	/** Used for code chunks that are statements with no value, rather than expressions */
	MCT_VoidStatement         = 1u << 23,

	/** Non-static bool, only used in new HLSL translator */
	MCT_Bool                  = 1u << 24,

	/** Unsigned int types */
	MCT_UInt1                 = 1u << 25,
	MCT_UInt2                 = 1u << 26,
	MCT_UInt3                 = 1u << 27,
	MCT_UInt4                 = 1u << 28,

	MCT_TextureCollection     = 1u << 29,
	MCT_TextureMeshPaint      = 1u << 30,
	MCT_TextureMaterialCache  = 1u << 31,

	/* Matrix types */
	MCT_Float3x3              = 1ull << 32,
	MCT_Float4x4              = 1ull << 33,
	MCT_LWCMatrix             = 1ull << 34,

	/** Material cache */
	MCT_MaterialCacheABuffer  = 1ull << 35,

	/** An internal type */
	MCT_Unexposed             = 1ull << 36,

	/** Reserved range for licensee modifications, any bits in the (inclusive) range is free for modifications */
	MCT_LicenseeReservedBegin     = 1ull << 48,
	MCT_LicenseeReservedEnd       = 1ull << 63,

	/** MCT_SparseVolumeTexture is intentionally not (yet) included here because it differs a lot from the other texture types and may not be supported/appropriate for all MCT_Texture use cases. */
	MCT_Texture = MCT_Texture2D | MCT_TextureCube | MCT_Texture2DArray | MCT_TextureCubeArray | MCT_VolumeTexture | MCT_TextureExternal | MCT_TextureVirtual | MCT_TextureMeshPaint | MCT_TextureMaterialCache,

	MCT_Float   = MCT_Float1 | MCT_Float2 | MCT_Float3 | MCT_Float4,
	MCT_UInt    = MCT_UInt1 | MCT_UInt2 | MCT_UInt3 | MCT_UInt4,
	MCT_LWCType = MCT_LWCScalar | MCT_LWCVector2 | MCT_LWCVector3 | MCT_LWCVector4,
	MCT_Numeric = MCT_Float | MCT_LWCType | MCT_Bool,
};
