// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "HAL/ThreadSafeBool.h"
#include "RenderCommandFence.h"
#include "Materials/MaterialInterface.h"

#if WITH_EDITOR
#include "RHIDefinitions.h"
#endif

#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialLayersFunctions.h"
#include "Materials/MaterialOverrideNanite.h"
#include "Templates/UniquePtr.h"
#include "Shader/ShaderTypes.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Material.generated.h"

enum EMaterialDomain : int;
enum class EMaterialTranslateValidationFlags : uint32;
class ITargetPlatform;
class UMaterialExpressionComment;
class UPhysicalMaterial;
class UPhysicalMaterialMask;
class USubsurfaceProfile;
class USpecularProfile;
class UToonProfile;
class UTexture;
struct FODSCRequestPayload;

#if WITH_EDITOR

/** struct used for annotations when a materials 'used with' flags have changed and need saving */
struct FMaterialsWithDirtyUsageFlags
{
	/** store the flags that have been changed since last save, each bit represents a flag */
	uint32 MaterialFlagsThatHaveChanged;

	FMaterialsWithDirtyUsageFlags():
	MaterialFlagsThatHaveChanged(0)
	{
	}

	/**
	 * Determine if this annotation is the default
	 * @return true is this is a default annotation
	 */
	inline bool IsDefault()
	{
		return MaterialFlagsThatHaveChanged == DefaultAnnotation.MaterialFlagsThatHaveChanged;
	}

	/** Mark the specificed flag as changed in this annotation */
	void MarkUsageFlagDirty(EMaterialUsage UsageFlag);

	/** Query the annotation to see if the specified flag has been changed */	
	bool IsUsageFlagDirty(EMaterialUsage UsageFlag);

	/** Default state for annotations (no flags changed)*/
	static const FMaterialsWithDirtyUsageFlags DefaultAnnotation;
};
#endif

/** Defines how the GBuffer channels are getting manipulated by a decal material pass. Actual index is used to control shader parameters so don't change order. */
UENUM()
enum EDecalBlendMode : int
{
	/** Blend full material, updating the GBuffer, does not work for baked lighting. */
	DBM_Translucent UMETA(DisplayName="Translucent"),
	/** Modulate BaseColor, blend rest, updating the GBuffer, does not work for baked lighting. Does not work in DBuffer mode (approximated as Translucent). */
	DBM_Stain UMETA(DisplayName="Stain"),
	/** Only blend normal, updating the GBuffer, does not work for baked lighting. */
	DBM_Normal UMETA(DisplayName="Normal"),
	/** Additive emissive only. */
	DBM_Emissive UMETA(DisplayName="Emissive"),
	/** Put into DBuffer to work for baked lighting as well (becomes DBM_TranslucentNormal if normal is not hooked up). */
	DBM_DBuffer_ColorNormalRoughness UMETA(DisplayName="DBuffer Translucent Color,Normal,Roughness"),
	/** Put into DBuffer to work for baked lighting as well. */
	DBM_DBuffer_Color UMETA(DisplayName="DBuffer Translucent Color"),
	/** Put into DBuffer to work for baked lighting as well (becomes DBM_DBuffer_Color if normal is not hooked up). */
	DBM_DBuffer_ColorNormal UMETA(DisplayName="DBuffer Translucent Color,Normal"),
	/** Put into DBuffer to work for baked lighting as well. */
	DBM_DBuffer_ColorRoughness UMETA(DisplayName="DBuffer Translucent Color,Roughness"),
	/** Put into DBuffer to work for baked lighting as well. */
	DBM_DBuffer_Normal UMETA(DisplayName="DBuffer Translucent Normal"),
	/** Put into DBuffer to work for baked lighting as well (becomes DBM_DBuffer_Roughness if normal is not hooked up). */
	DBM_DBuffer_NormalRoughness UMETA(DisplayName="DBuffer Translucent Normal,Roughness"),
	/** Put into DBuffer to work for baked lighting as well. */
	DBM_DBuffer_Roughness UMETA(DisplayName="DBuffer Translucent Roughness"),

	/** Internal DBffer decal blend modes used for auto-converted decals */
	DBM_DBuffer_Emissive UMETA(DisplayName = "DBuffer Emissive", Hidden),
	DBM_DBuffer_AlphaComposite UMETA(DisplayName = "DBuffer AlphaComposite (Premultiplied Alpha)", Hidden),
	DBM_DBuffer_EmissiveAlphaComposite UMETA(DisplayName = "DBuffer Emissive AlphaComposite (Premultiplied Alpha)", Hidden),

	/** Output signed distance in Opacity depending on LightVector. Note: Can be costly, no shadow casting but receiving, no per pixel normal yet, no quality settings yet */
	DBM_Volumetric_DistanceFunction UMETA(DisplayName="Volumetric Distance Function (experimental)"),
	
	/** Blend with existing scene color. Decal color is already pre-multiplied by alpha. */
	DBM_AlphaComposite UMETA(DisplayName ="AlphaComposite (Premultiplied Alpha)"),

	/** Ambient occlusion. */
	DBM_AmbientOcclusion UMETA(DisplayName = "Ambient Occlusion"),

	DBM_MAX,
};

/** Defines how the material reacts on DBuffer decals, later we can expose more variants between None and Default. */
UENUM()
enum EMaterialDecalResponse : int
{
	/** Do not receive decals (Later we still can read the DBuffer channels to customize the effect, this frees up some interpolators). */
	MDR_None UMETA(DisplayName="None"),

	/** Receive Decals, applies all DBuffer channels. */
	MDR_ColorNormalRoughness UMETA(DisplayName="Color Normal Roughness"),
	/** Receive Decals, applies color DBuffer channels. */
	MDR_Color UMETA(DisplayName="Color"),
	/** Receive Decals, applies color and normal DBuffer channels. */
	MDR_ColorNormal UMETA(DisplayName="Color Normal"),
	/** Receive Decals, applies color, roughness, specular and metallic DBuffer channels. */
	MDR_ColorRoughness UMETA(DisplayName="Color Roughness"),
	/** Receive Decals, applies normal DBuffer channels. */
	MDR_Normal UMETA(DisplayName="Normal"),
	/** Receive Decals, applies normal, roughness, specular and metallic DBuffer channels. */
	MDR_NormalRoughness UMETA(DisplayName="Normal Roughness"),
	/** Receive Decals, applies roughness, specular and metallic DBuffer channels. */
	MDR_Roughness UMETA(DisplayName="Roughness"),
	MDR_MAX
};

/** Specifies which separate translucency pass to render in. */
UENUM()
enum EMaterialTranslucencyPass : int
{
	/** Render before depth of field. */
	MTP_BeforeDOF UMETA(DisplayName="Before DOF"),
	/** Render after depth of field. */
	MTP_AfterDOF UMETA(DisplayName="After DOF"),
	/** Render after motion blur. Disables depth test (the reconstruction post MB and TSR would otherwise flicker due to the TSR camera jittering making the depth buffer unstable). Because of that Lumen high quality reflections is also disabled to avoid visual discrepancy at depth intersection. */
	MTP_AfterMotionBlur UMETA(DisplayName="After Motion Blur"),
	MTP_MAX
};

// Material input structs.
//@warning: manually mirrored in MaterialShared.h
#if !CPP      //noexport struct
USTRUCT(noexport)
struct FMaterialInput
{
	/** Material expression that this input is connected to, or NULL if not connected. */
	UPROPERTY()
	TObjectPtr<class UMaterialExpression> Expression;

	/** Index into Expression's outputs array that this input is connected to. */
	UPROPERTY()
	int32 OutputIndex;

	/** 
	 * Optional name of the input.  
	 * Note that this is the only member which is not derived from the output currently connected. 
	 */
	UPROPERTY()
	FName InputName;

	UPROPERTY()
	int32 Mask;

	UPROPERTY()
	int32 MaskR;

	UPROPERTY()
	int32 MaskG;

	UPROPERTY()
	int32 MaskB;

	UPROPERTY()
	int32 MaskA;
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FColorMaterialInput : public FMaterialInput
{
	UPROPERTY()
	uint32 UseConstant : 1;

	UPROPERTY()
	FLinearColor Constant;

	FColorMaterialInput() :
		UseConstant(0) {}
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FScalarMaterialInput : public FMaterialInput
{
	UPROPERTY()
	uint32 UseConstant : 1;

	UPROPERTY()
	float Constant;

	FScalarMaterialInput() :
		UseConstant(0) {}
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FShadingModelMaterialInput : public FMaterialInput
{
	// No support for constant
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FSubstrateMaterialInput : public FMaterialInput
{
	// No support for constant
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FVectorMaterialInput : public FMaterialInput
{
	UPROPERTY()
	uint32 UseConstant : 1;

	UPROPERTY()
	FVector3f Constant;

	FVectorMaterialInput:
		UseConstant(0) {}
};
#endif

#if !CPP      //noexport struct
USTRUCT(noexport)
struct FVector2MaterialInput : public FMaterialInput
{
	UPROPERTY()
	uint32 UseConstant : 1;

	UPROPERTY()
	float ConstantX=0.0f;

	UPROPERTY()
	float ConstantY=0.0f;

	FVector2MaterialInput:
		UseConstant(0) {}
};
#endif

struct FMaterialInputDescription
{
	FExpressionInput* Input = nullptr;
	UE::Shader::EValueType Type = UE::Shader::EValueType::Void;
	UE::Shader::FValue ConstantValue;
	bool bUseConstant = false;
	bool bHidden = false;
};

USTRUCT()
struct FParameterGroupData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Group Sorting")
	FString GroupName;

	UPROPERTY(EditAnywhere, Category = "Group Sorting")
	int32 GroupSortPriority;

	FParameterGroupData()
	{
		GroupName = FString();
		GroupSortPriority = 0;
	}

	FParameterGroupData(const FString& InString)
		: GroupName(InString),
		GroupSortPriority(0)
	{
	}

	FParameterGroupData(const FString& InString, int32 InSortPriority)
	:	GroupName(InString),
		GroupSortPriority(InSortPriority)
	{
	}

	bool operator<(const FParameterGroupData& Other) const
	{
		return GroupName < Other.GroupName;
	}

	friend uint32 GetTypeHash(const FParameterGroupData& InGroupData)
	{
		return GetTypeHash(InGroupData.GroupName);
	}

	bool operator==(const FParameterGroupData& Other) const
	{
		return GroupName == Other.GroupName;
	}
};

UCLASS(MinimalAPI, Optional)
class UMaterialEditorOnlyData : public UMaterialInterfaceEditorOnlyData
{
	GENERATED_BODY()
public:
	UMaterialEditorOnlyData();

	// Reflection
	UPROPERTY()
	FColorMaterialInput BaseColor;

	UPROPERTY()
	FScalarMaterialInput Metallic;

	UPROPERTY()
	FScalarMaterialInput Specular;

	UPROPERTY()
	FScalarMaterialInput Roughness;

	UPROPERTY()
	FScalarMaterialInput Anisotropy;

	UPROPERTY()
	FVectorMaterialInput Normal;

	UPROPERTY()
	FVectorMaterialInput Tangent;

	UPROPERTY()
	FColorMaterialInput EmissiveColor;

	// Transmission.
	UPROPERTY()
	FScalarMaterialInput Opacity;

	UPROPERTY()
	FScalarMaterialInput OpacityMask;

	/** Adds to world position in the vertex shader. */
	UPROPERTY()
	FVectorMaterialInput WorldPositionOffset;

	UPROPERTY()
	FScalarMaterialInput Displacement;

	/** Inner material color, only used for ShadingModel=Subsurface */
	UPROPERTY()
	FColorMaterialInput SubsurfaceColor;

	/**  */
	UPROPERTY()
	FScalarMaterialInput ClearCoat;

	/**  */
	UPROPERTY()
	FScalarMaterialInput ClearCoatRoughness;

	/** output ambient occlusion to the GBuffer */
	UPROPERTY()
	FScalarMaterialInput AmbientOcclusion;

	/**
	 * output refraction index for translucent rendering
	 * Air:1.0 Water:1.333 Ice:1.3 Glass:~1.6 Diamond:2.42
	 */
	UPROPERTY()
	FScalarMaterialInput Refraction;

	/**
	 * These inputs are evaluated in the vertex shader and allow artists to do arbitrary vertex shader operations and access them in the pixel shader.
	 * When unconnected or hidden they default to passing through the vertex UVs.
	 */
	UPROPERTY()
	FVector2MaterialInput CustomizedUVs[8];

	UPROPERTY()
	FMaterialAttributesInput MaterialAttributes;

	UPROPERTY()
	FScalarMaterialInput PixelDepthOffset;

	UPROPERTY()
	FShadingModelMaterialInput ShadingModelFromMaterialExpression;

	UPROPERTY()
	FScalarMaterialInput SurfaceThickness;

	UPROPERTY()
	FSubstrateMaterialInput FrontMaterial;

	UPROPERTY()
	FMaterialExpressionCollection ExpressionCollection;

	/** Controls where this parameter group is displayed in a material instance parameter list.  The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Group Sorting")
	TArray<FParameterGroupData> ParameterGroupData;

	/** Store the version of the Substrate's auto-conversion applied to this material if any (default, no conversion =-1). */
	UPROPERTY()
	int32 SubstrateConversionVersion;

	/** Store the version of the Substrate code used by this material (default, no version =-1). */
	UPROPERTY()
	int32 SubstrateVersion;

	/** Whether the material has been auto - converted at load time, and hasn't been saved. */
	UPROPERTY()
	bool bIsSubsutrateAutoConvertedAndNotSaved = false;

	/** Reset the version of the Substrate's auto-conversion applied to this material */
	ENGINE_API void ResetSubstrateConversionVersion();

	/** Return true if the Substrate's auto-conversion applied to this material */
	ENGINE_API bool IsSubstrateAutoConverted() const;
};

/**
 * A Material is an asset which can be applied to a mesh to control the visual look of the scene. 
 * When light from the scene hits the surface, the shading model of the material is used to calculate how that light interacts with the surface. 
 *
 * Warning: Creating new materials directly increases shader compile times!  Consider creating a Material Instance off of an existing material instead.
 */
UCLASS(hidecategories=Object, MinimalAPI, BlueprintType, PrioritizeCategories = ("Material", "Nanite", "Translucency", "TranslucencySelfShadowing", "Refraction", "WorldPositionOffset", "PostProcessMaterial",  "Mobile", "ForwardShading", "PhysicalMaterial", "PhysicalMaterialMask", "Usage", "Lightmass", "Previewing", "ImportSettings"))
class UMaterial : public UMaterialInterface
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	virtual const UClass* GetEditorOnlyDataClass() const override { return UMaterialEditorOnlyData::StaticClass(); }

	virtual UMaterialEditorOnlyData* GetEditorOnlyData() override { return CastChecked<UMaterialEditorOnlyData>(Super::GetEditorOnlyData(), ECastCheckedType::NullAllowed); }
	virtual const UMaterialEditorOnlyData* GetEditorOnlyData() const override { return CastChecked<UMaterialEditorOnlyData>(Super::GetEditorOnlyData(), ECastCheckedType::NullAllowed); }
#endif // WITH_EDITORONLY_DATA

	// Physics.
	
	/** Physical material to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, Category=PhysicalMaterial)
	TObjectPtr<class UPhysicalMaterial> PhysMaterial;

	/** Physical material mask to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TObjectPtr<class UPhysicalMaterialMask> PhysMaterialMask;

	/** Physical material mask map to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, Category = PhysicalMaterialMask)
	TObjectPtr<class UPhysicalMaterial> PhysicalMaterialMap[EPhysicalMaterialMaskColor::MAX];
	

	UPROPERTY()
	TArray<TObjectPtr<class UPhysicalMaterial>> RenderTracePhysicalMaterialOutputs;

	/** 
	 * The domain that the material's attributes will be evaluated in. 
	 * Certain pieces of material functionality are only valid in certain domains, for example vertex normal is only valid on a surface.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Material, AssetRegistrySearchable)
	TEnumAsByte<EMaterialDomain> MaterialDomain;

	/** Determines how the material's color is blended with background colors. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Material, AssetRegistrySearchable)
	TEnumAsByte<EBlendMode> BlendMode;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "No longer used."))
	TEnumAsByte<EDecalBlendMode> DecalBlendMode;

	/** 
	 * Defines how the material reacts on DBuffer decals (Affects look, performance and texture/sample usage).
	 * Non DBuffer Decals can be disabled on the primitive (e.g. static mesh)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Material, AdvancedDisplay, meta=(DisplayName = "Decal Response (DBuffer)"), AssetRegistrySearchable)
	TEnumAsByte<EMaterialDecalResponse> MaterialDecalResponse;

	/** An override material which will be used instead of this one when rendering with Nanite. */
	UPROPERTY(EditAnywhere, Category=Nanite, meta = (EditInline, ShowOnlyInnerProperties))
	FMaterialOverrideNanite NaniteOverrideMaterial;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Nanite, meta = (DisplayName = "Displacement", EditCondition="bEnableTessellation", DisplayAfter="bEnableTessellation"))
	FDisplacementScaling DisplacementScaling;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Nanite, meta = (DisplayName = "Displacement Fade", EditCondition="bEnableDisplacementFade", DisplayAfter="DisplacementScaling"))
	FDisplacementFadeRange DisplacementFadeRange;

private:
	/** Determines how inputs are combined to create the material's final color. */
	UPROPERTY(EditAnywhere, Category=Material, AssetRegistrySearchable)
	TEnumAsByte<EMaterialShadingModel> ShadingModel; 

public:
	/** Whether the material should cast shadows as masked even though it has a translucent blend mode. */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay)
	uint8 bCastDynamicShadowAsMasked : 1;

private:
	UPROPERTY(AssetRegistrySearchable)
	FMaterialShadingModelField ShadingModels;

#if WITH_EDITORONLY_DATA
	/** These are the shading models present in this material. Note that all these shading models might not be used in all feature levels and quality levels. */
	UPROPERTY(VisibleAnywhere, Transient, Category=Material)
	FString UsedShadingModels;
#endif

public:

	/**
	 * If BlendMode is BLEND_Masked, the surface is not rendered where OpacityMask < OpacityMaskClipValue.
	 * If BlendMode is BLEND_Translucent, BLEND_Additive, or BLEND_Modulate, and "Output Depth and Velocity" is enabled,
	 * the object velocity is not rendered where Opacity < OpacityMaskClipValue.
	 */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay)
	float OpacityMaskClipValue;

private:
	/** Deprecated. Use TranslucencyPass instead. */
	UPROPERTY()
	uint8 bEnableSeparateTranslucency_DEPRECATED : 1;

public:
	/**
	 * Indicates that the material should be rendered using responsive anti-aliasing. Improves sharpness of small moving particles such as sparks.
	 * Only use for small moving features because it will cause aliasing of the background.
	 */
	UPROPERTY(EditAnywhere, Category=Translucency, meta=(DisplayName = "Responsive AA"), AdvancedDisplay)
	uint8 bEnableResponsiveAA : 1;

	/** SSR on translucency */
	UPROPERTY(EditAnywhere, Category=Translucency, meta=(DisplayName = "Screen Space Reflections"))
	uint8 bScreenSpaceReflections : 1;

	/** Contact shadows on translucency */
	UPROPERTY(EditAnywhere, Category = Translucency, meta = (DisplayName = "Contact Shadows"))
	uint8 bContactShadows : 1;

	/** Indicates that the material should be rendered without backface culling and the normal should be flipped for backfaces. */
	UPROPERTY(EditAnywhere, Category=Material)
	uint8 TwoSided : 1;

	/** Indicates that the material should be rendered as a thin surface (i.e., without inner volume). Only used by Substrate materials. Enabling ThinSurface will disable subsurface profiles. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bIsThinSurface : 1;

	/** Whether meshes rendered with the material should support dithered LOD transitions. */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay, meta = (DisplayName = "Dithered LOD Transition"))
	uint8 DitheredLODTransition : 1;

	/** Dither opacity mask. When combined with Temporal AA this can be used as a form of limited translucency which supports all lighting features. */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 DitherOpacityMask : 1;

	/** Whether the material should allow outputting negative emissive color values.  Only allowed on unlit materials. */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 bAllowNegativeEmissiveColor : 1;
	
	/** Whether the opaque material has any pixel animations happening, that isn't included in the geometric velocities.
	 * This allows to disable renderer's heuristics that assumes animation is fully described with motion vector, such as TSR's anti-flickering heuristic.
	 */
	UPROPERTY(EditAnywhere, Category=Material, meta=(DisplayName = "Has Pixel Animation"), AdvancedDisplay)
	uint8 bHasPixelAnimation : 1;
	
	/** Whether tessellation is enabled on the material. NOTE: Required for displacement to work. */
	UPROPERTY(EditAnywhere, Category=Nanite, meta = (Keywords = "Displacement"))
	uint8 bEnableTessellation : 1;

	/** Enables fading out and disabling of dynamic displacement in the distance, as displacement becomes unnoticeable */
	UPROPERTY(EditAnywhere, Category=Nanite, meta=(EditCondition="bEnableTessellation", InlineEditConditionToggle))
	uint8 bEnableDisplacementFade : 1;

	/**
	 * Specifies the separate pass in which to render translucency.
	 * This can be used to avoid artifacts caused by certain post processing effects.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Translucency)
	TEnumAsByte<EMaterialTranslucencyPass> TranslucencyPass;

	/** Sets the lighting mode that will be used on this material if it is translucent. */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category=Translucency, meta=(DisplayName = "Lighting Mode"))
	TEnumAsByte<ETranslucencyLightingMode> TranslucencyLightingMode;

	/** Indicates that the translucent material should not be affected by bloom or DOF. (Note: Depth testing is not available) */
	UPROPERTY(EditAnywhere, Category = Translucency, meta = (DisplayName = "Mobile Separate Translucency"), AdvancedDisplay)
	uint8 bEnableMobileSeparateTranslucency : 1;

	/** Number of customized UV inputs to display.  Unconnected customized UV inputs will just pass through the vertex UVs. */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay, meta = (ClampMin = 0))
	int32 NumCustomizedUVs;

	/** 
	 * Useful for artificially increasing the influence of the normal on the lighting result for translucency.
	 * A value larger than 1 increases the influence of the normal, a value smaller than 1 makes the lighting more ambient.
	 */
	UPROPERTY(EditAnywhere, Category=Translucency, meta=(DisplayName = "Directional Lighting Intensity"))
	float TranslucencyDirectionalLightingIntensity;

	/** Scale used to make translucent shadows more or less opaque than the material's actual opacity. */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Shadow Density Scale"))
	float TranslucentShadowDensityScale;

	/** 
	 * Scale used to make translucent self-shadowing more or less opaque than the material's shadow on other objects. 
	 * This is only used when the object is casting a volumetric translucent shadow.
	 */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Self Shadow Density Scale"))
	float TranslucentSelfShadowDensityScale;

	/** Used to make a second self shadow gradient, to add interesting shading in the shadow of the first. */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Second Density Scale"))
	float TranslucentSelfShadowSecondDensityScale;

	/** Controls the strength of the second self shadow gradient. */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Second Opacity"))
	float TranslucentSelfShadowSecondOpacity;

	/** 
	 * Controls how diffuse the material's backscattering is when using the MSM_Subsurface shading model.
	 * Larger exponents give a less diffuse look (smaller, brighter backscattering highlight).
	 * This is only used when the object is casting a volumetric translucent shadow from a directional light.
	 */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Backscattering Exponent"))
	float TranslucentBackscatteringExponent;

	/** 
	 * Colored extinction factor used to approximate multiple scattering in dense volumes. 
	 * This is only used when the object is casting a volumetric translucent shadow.
	 */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Multiple Scattering Extinction"))
	FLinearColor TranslucentMultipleScatteringExtinction;

	/** Local space distance to bias the translucent shadow.  Positive values move the shadow away from the light. */
	UPROPERTY(EditAnywhere, Category=TranslucencySelfShadowing, meta=(DisplayName = "Start Offset"))
	float TranslucentShadowStartOffset;

	/** Whether to draw on top of opaque pixels even if behind them. This only has meaning for translucency. */
	UPROPERTY(EditAnywhere, Category=Translucency, AdvancedDisplay)
	uint8 bDisableDepthTest : 1;

	/** Whether the transluency pass should write its alpha, and only the alpha, into the framebuffer */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay)
	uint8 bWriteOnlyAlpha : 1;

	/** Whether to generate spherical normals for particles that use this material. */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 bGenerateSphericalParticleNormals : 1;

	/**
	 * Whether the material takes a tangent space normal or a world space normal as input.
	 * (TangentSpace requires extra instructions but is often more convenient).
	 */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 bTangentSpaceNormal : 1;

	/**
	 * If enabled, the material's emissive colour is injected into the LightPropagationVolume
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Material, meta=(DisplayName = "Emissive (Dynamic Area Light)"), AdvancedDisplay)
	uint8 bUseEmissiveForDynamicAreaLighting : 1;

	/** 
	 * This is a special usage flag that allows a material to be assignable to any primitive type.
	 * This is useful for materials used by code to implement certain viewmodes, for example the default material or lighting only material.
	 * The cost is that nearly 20x more shaders will be compiled for the material than the average material, which will greatly increase shader compile time and memory usage.
	 * This flag should only be enabled when absolutely necessary, and is purposefully not exposed to the UI to prevent abuse.
	 */
	UPROPERTY(duplicatetransient)
	uint8 bUsedAsSpecialEngineMaterial : 1;

	/**
	 * This can be used to disable roughness tracking. When this is done, top layers roughness will no longer impact bottom layer roughness.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta = (DisplayName = "Roughness Tracking"), Category = Substrate)
	uint8 bSubstrateRoughnessTracking : 1;

	/** 
	 * Indicates that the material and its instances can be used with skeletal meshes.  
	 * This will result in the shaders required to support skeletal meshes being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithSkeletalMesh : 1;

	/**
	 * Indicates that the material and its instances can be use with mesh deformers
	 * This will result in the shaders required to support mesh deformers being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithMeshDeformer : 1;

	/** 
	 * Indicates that the material and its instances can be used with editor compositing  
	 * This will result in the shaders required to support editor compositing being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithEditorCompositing : 1;

	/** 
	 * Indicates that the material and its instances can be used with particle sprites 
	 * This will result in the shaders required to support particle sprites being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithParticleSprites : 1;

	/** 
	 * Indicates that the material and its instances can be used with beam trails
	 * This will result in the shaders required to support beam trails being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithBeamTrails : 1;

	/** 
	 * Indicates that the material and its instances can be used with mesh particles
	 * This will result in the shaders required to support mesh particles being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithMeshParticles : 1;

	/**
	* Indicates that the material and its instances can be used with Niagara sprites (meshes and ribbons, respectively)
	* This will result in the shaders required to support Niagara sprites being compiled which will increase shader compile time and memory usage.
	*/
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithNiagaraSprites : 1;

	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithNiagaraRibbons : 1;

	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithNiagaraMeshParticles : 1;

	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithGeometryCache : 1;
	
	/** 
	 * Indicates that the material and its instances can be used with static lighting
	 * This will result in the shaders required to support static lighting being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithStaticLighting : 1;

	/** 
	 * Indicates that the material and its instances can be used with morph targets
	 * This will result in the shaders required to support morph targets being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithMorphTargets : 1;

	/** 
	 * Indicates that the material and its instances can be used with spline meshes
	 * This will result in the shaders required to support spline meshes being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithSplineMeshes : 1;

	/** 
	 * Indicates that the material and its instances can be used with instanced static meshes
	 * This will result in the shaders required to support instanced static meshes being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithInstancedStaticMeshes : 1;

	/**
	 * Indicates that the material and its instances can be use with geometry collections
	 * This will result in the shaders required to support geometry collections being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithGeometryCollections : 1;

	/** 
	 * Indicates that the material and its instances can be used with distortion
	 * This will result in the shaders required to support distortion being compiled which will increase shader compile time and memory usage.
	 */
	UPROPERTY()
	uint8 bUsesDistortion : 1;

	/** 
	 * Indicates that the material do not use the material physically based refraction (e.g. IOR from reflectivity F0), but overrides it for artistic purposes.
	 */
	 UPROPERTY()
	uint8 bRootNodeOverridesDefaultDistortion_DEPRECATED : 1;

	/** 
	 * Indicates that the material and its instances can be used with clothing
	 * This will result in the shaders required to support clothing being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage)
	uint8 bUsedWithClothing : 1;

	/**
	 * Indicates that the material and its instances can be use with water
	 * This will result in the shaders required to support water meshes being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithWater : 1;

	/**
	 * Indicates that the material and its instances can be use with hair strands
	 * This will result in the shaders required to support hair strands geometries being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithHairStrands : 1;

	/**
	 * Indicates that the material and its instances can be use with LiDAR Point Clouds
	 * This will result in the shaders required to support LiDAR Point Cloud geometries being compiled which will increase shader compile time and memory usage.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithLidarPointCloud : 1;

	/**
	 * Indicates that the material and its instances can be used with Virtual Heightfield Mesh.
	 * This will result in the shaders required to support Virtual Heightfield Mesh geometries being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithVirtualHeightfieldMesh : 1;
	
	/**
	 * Indicates that the material and its instances can be used with Nanite meshes.
	 * This will result in the shaders required to support Nanite geometries being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithNanite : 1;

	/**
	 * Indicates that the material and its instances can be used with Nanite voxel meshes.
	 * This will result in the shaders required to support Nanite voxels being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithVoxels : 1;

	/**
	 * Indicates that the material and its instances can be used with Nanite curve meshes.
	 * This will result in the shaders required to support Nanite curves being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint8 bUsedWithCurves : 1;

	/** 
	 * Indicates that the material and its instances with volumetric cloud. Without that flag, it can only be used on volumetric fog.
	 * This will result in the shaders required to support Volumetric Cloud rendering being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithVolumetricCloud : 1;

	/**
	 * Indicates that the material and its instances with heterogeneous volumes. Without that flag, it can only be used on volumetric fog.
	 * This will result in the shaders required to support Heterogeneous Volumes rendering being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithHeterogeneousVolumes : 1;

	/**
	 * Indicates that the material and its instances can be used with instanced skinned meshes.
	 * This will result in the shaders required to support instanced skinned meshes being compiled which will increase shader compile time and memory usage.
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage)
	uint32 bUsedWithInstancedSkinnedMesh : 1;

	/**
	 * Indicates that the material and its instances can be used on StaticMeshes, which is true by default on all Materials
	 * This can be used to remove shaders which are not required, reducing shader compile time and memory usage
	 */
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Usage, AdvancedDisplay)
	uint32 bUsedWithStaticMesh : 1;

	/**
	 * Indicates that the material and its instances can be used with Slate UI and UMG
	 * This will result in the shaders required to support UI materials being compiled which will increase shader compile time and memory usage.
	 */
	UPROPERTY()
	uint8 bUsedWithUI_DEPRECATED : 1;

	/** 
	 * Indicates that this material is safe to use with the light function atlas:
	 *  - texture coordinates are manipulated in a way that works.
	 *  - world position and depth are not used in the graph or do work with the atlas.
	 * Forces all deferred lights to go the fast batched lighting code path (no screen shadow mask).
	 */
	UPROPERTY(EditAnywhere, Category = LightFunctionMaterial, meta = (DisplayName = "Compatible With Light Function Atlas"))
	uint8 bForceCompatibleWithLightFunctionAtlas : 1;

	/** 
	 * Whether to automatically set usage flags on this material or its instances based on what they are applied to in the editor.
	 * Note that adding usage flags accidentally can add unwanted shader permutations.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Usage, AdvancedDisplay)
	uint8 bAutomaticallySetUsageInEditor : 1;

	/* Forces the material to be completely rough. Saves a number of instructions and one sampler. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Material, AdvancedDisplay)
	uint8 bFullyRough : 1;

	/**
	 *	Deprecated. Use FloatPrecisionMode instead.
	 *  Forces this material to use full (highp) precision in the pixel shader.
	 *	This is slower than the default (mediump) but can be used to work around precision-related rendering errors.
	 *	This setting has no effect on older mobile devices that do not support high precision.
	 */
	UPROPERTY()
	uint8 bUseFullPrecision_DEPRECATED : 1;

	/**
	 *	Per Material override to change floating point precision in the pixel shader (ALU and uniforms precision)
	 *	Full-precision (highp) is slower than (half-precision/mediump) but can be used to work around precision-related rendering errors.
	 *  Use:
	 *  'Default' to take the precision set by the project wide cvar r.Mobile.FloatPrecisionMode. It defaults to 0 (half precision)
	 *	'Full-precision for MaterialExpressions only' if you still want to keep half precision in.ush/.usf shaders (global shaders)
	 *	'Full-precision for every float' to force every float and uniform to full-precision
	 *	'Half-precision' to try to use half precision ALU and uniforms where appropriate
	 *	'Half-precision including shader parameters' same behaviour as 'Half-precision' but it also uses half precision for uniforms with custom parameters like PreshaderBuffer uniform
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Mobile)
	TEnumAsByte<EMaterialFloatPrecisionMode> FloatPrecisionMode;

	/* Use lightmap directionality and per pixel normals. If disabled, lighting from lightmaps will be flat but cheaper. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mobile)
	uint8 bUseLightmapDirectionality : 1;

	/* Use alpha to coverage for masked material on mobile, make sure MSAA is enabled as well. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Mobile, AdvancedDisplay, meta = (EditCondition = "BlendMode != EBlendMode::BLEND_Opaque"))
	uint8 bUseAlphaToCoverage : 1;

	/* Forward (including mobile) renderer: use preintegrated GF lut for simple IBL, but will use one more sampler. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ForwardShading, meta = (DisplayName = "PreintegratedGF For Simple IBL"))
	uint32 bForwardRenderUsePreintegratedGFForSimpleIBL : 1;

	/* 
	 * Forward renderer: enables multiple parallax-corrected reflection captures that blend together.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ForwardShading, meta = (DisplayName = "High Quality Reflections"))
	uint8 bUseHQForwardReflections : 1;

	/* 
	 * Enables blending of sky light cubemap textures. When disabled, the secondary cubemap is only visible when the blend factor is 1.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ForwardShading, meta = (DisplayName = "Blend Sky Light Cubemaps"))
	uint8 bForwardBlendsSkyLightCubemaps : 1;

	/* Enables planar reflection when using the forward renderer or mobile. Enabling this setting reduces the number of samplers available to the material as one more sampler will be used for the planar reflection. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ForwardShading, meta = (DisplayName = "Planar Reflections"))
	uint8 bUsePlanarForwardReflections : 1;

	/* Reduce roughness based on screen space normal changes. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Material, AdvancedDisplay)
	uint8 bNormalCurvatureToRoughness : 1;

	/** Allows a translucent material to be used with custom depth writing by compiling additional shaders. */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay, meta = (DisplayName = "Allow Custom Depth Writes"))
	uint8 AllowTranslucentCustomDepthWrites : 1;

	/** Allows a translucent material to be used with Front Layer Translucency by compiling additional shaders. Useful for controlling what should be included in Front Layer Translucency. */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay, meta = (DisplayName = "Allow Front Layer Translucency"))
	uint8 bAllowFrontLayerTranslucency : 1;

	/** When true, allows to share Lumen Cards between different instances even when material uses world position or per instance data, which may change material look per instance. All materials on a component needs this flag set for sharing to work. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Material, AdvancedDisplay)
	uint8 bCompatibleWithLumenCardSharing : 1;

	/** Enables a wireframe view of the mesh the material is applied to.  */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 Wireframe : 1;

	/** Allows a translucent material to receive local light shadows. */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay, meta = (DisplayName = "Allow Local Light Shadow"))
	uint8 bAllowTranslucentLocalLightShadow : 1;

	/** Set local light shadow quality received by translucent material. */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay, meta = (DisplayName = "Local Light Shadow Quality", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", SliderExponent = 1.0))
	float TranslucentLocalLightShadowQuality = 0;

	/** Set directional light shadow quality received by translucent material. */
	UPROPERTY(EditAnywhere, Category = Translucency, AdvancedDisplay, meta = (DisplayName = "Directional Light Shadow Quality", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", SliderExponent = 1.0))
	float TranslucentDirectionalLightShadowQuality = 0;

	/**
	 * Select what shading rate to apply, on platforms that support variable rate shading.
	 * Non-1x1 rates will reduce the rasterization fidelity for the material; they will not super-sample the material.
	 * This can save GPU performance on materials where reduced fidelity is acceptable.
	 */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay)
	TEnumAsByte<EMaterialShadingRate> ShadingRate;

	/** Allows the use of variable rate shading when evaluating this material. This will only apply to the base/translucency pass. */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay)
	uint8 bAllowVariableRateShading : 1;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 EditorX;

	UPROPERTY()
	int32 EditorY;

	UPROPERTY()
	int32 EditorPitch;

	UPROPERTY()
	int32 EditorYaw;
#endif // WITH_EDITORONLY_DATA

	/** true if this Material can be assumed Opaque when set to masked. */
	UPROPERTY()
	uint8 bCanMaskedBeAssumedOpaque : 1;

	/** true if Material is masked and uses custom opacity */
	UPROPERTY()
	uint8 bIsMasked_DEPRECATED : 1;

	/** true if Material is the preview material used in the material editor. */
	UPROPERTY(transient, duplicatetransient)
	uint8 bIsPreviewMaterial : 1;

	/** true if Material is the function preview material used in the material instance editor. */
	UPROPERTY(transient, duplicatetransient)
	uint8 bIsFunctionPreviewMaterial : 1;

	/** when true, the material attributes pin is used instead of the regular pins. */
	UPROPERTY(EditAnywhere, Category=Material)
	uint8 bUseMaterialAttributes : 1;

	UE_DEPRECATED(5.8, "Experimental bEnableExecWire is no longer supported.")
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bEnableExecWire : 1;
	
	/** when true, the new material translator is used to translate this material */
	UPROPERTY(EditAnywhere, Category = Material, meta=(DisplayName="Enable new material translator"))
	uint8 bEnableNewHLSLGenerator : 1;

	/** when true, the material casts ray tracing shadows. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bCastRayTracedShadows : 1;

	/** When true, translucent materials are fogged. Defaults to true. */
	UPROPERTY(EditAnywhere, Category=Translucency, meta=(DisplayName = "Apply Fogging"))
	uint8 bUseTranslucencyVertexFog : 1;

	/** When true, translucent materials receive cloud contribution as part of the fog evaluation, per vertex or per pixel according to the other selected options. This is a rough approximation but can help in some cases. Defaults to false. Fog is applied on clouds, so Apply Fogging must be true to use this feature. */
	UPROPERTY(EditAnywhere, Category=Translucency, meta=(DisplayName = "Apply Cloud Fogging"))
	uint8 bApplyCloudFogging : 1;

	/** Unlit and Opaque materials can be used as sky material on a sky dome mesh. When IsSky is true, these meshes will not receive any contribution from the aerial perspective. Height and Volumetric fog effects will still be applied. */
	UPROPERTY(EditAnywhere, Category=Material, AdvancedDisplay)
	uint8 bIsSky : 1;

	/** When true, translucent materials have fog computed for every pixel, which costs more but fixes artifacts due to low tessellation. */
	UPROPERTY(EditAnywhere, Category=Translucency)
	uint8 bComputeFogPerPixel : 1;

	/** When true, translucent materials will output motion vectors and write to depth buffer in velocity pass. */
	UPROPERTY(EditAnywhere, Category = Translucency, meta = (DisplayName = "Output Depth and Velocity"))
	uint8 bOutputTranslucentVelocity : 1;

	/** 
	 * When true, translucent materials that write velocity will calculate it only based on the written depth and camera movement. 
	 * This can be useful for translucent volume raycasts which write pixel depth offset and want to calculate velocity based on the output depth.
	 */
	UPROPERTY(EditAnywhere, Category = Translucency, meta = (DisplayName = "Velocity From Depth Only", editcondition = "bOutputTranslucentVelocity"))
	uint8 bIsTranslucencyVelocityFromDepth : 1;

	/** If true the compilation environment will be changed to remove the global COMPILE_SHADERS_FOR_DEVELOPMENT flag. */
	UPROPERTY(transient, duplicatetransient)
	uint8 bAllowDevelopmentShaderCompile : 1;

	/** true if this is a special material used for stats by the material editor. */
	UPROPERTY(transient, duplicatetransient)
	uint8 bIsMaterialEditorStatsMaterial : 1;

	/** 
	 * Relax restrictions on Runtime Virtual Texture output compilation.
	 * Normally we give a compilation error whenever a Runtime Virtual Texture Output reads from a Virtual Texture. 
	 * This is because there is no feedback from Runtime Virtual Texture Output passes and no guarantee that Virtual Textures will be ready.
	 * If allowed then Runtime Virtual Texture pages can contain stale low resolution data that is not evicted until the pages are flushed.
	 */
	UPROPERTY(EditAnywhere, Category = VirtualTexture)
	uint8 bRelaxRuntimeVirtualTextureRestrictions : 1;
	uint8 bRelaxRuntimeVirtualTextureRestrictionsForOldContent : 1;
	
	/** Where the node is inserted in the (post processing) graph, only used if domain is PostProcess */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PostProcessMaterial, meta=(DisplayName = "Blendable Location"))
	TEnumAsByte<EBlendableLocation> BlendableLocation;

	/** If this is enabled, the blendable will output alpha */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "Output Alpha"))
	uint8 BlendableOutputAlpha : 1;

	/** Specify a user generated scene texture as output, overriding the default output implied by "Blendable Location", only used if domain is PostProcess */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "User Scene Texture"))
	FName UserSceneTexture;

	/** A divisor for the resolution of the User Scene Texture above, allowing an intermediate with reduced resolution */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "User Texture Divisor"))
	FIntPoint UserTextureDivisor = FIntPoint(0, 0);

	/** Set output resolution relative to the given User Scene Texture input.  Negative User Texture Divisors upsample (clamped at full resolution). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "Resolution Relative To Input"))
	FName ResolutionRelativeToInput;

	/**
	 * Disable pre-exposure scale in post process materials (multiply by View.OneOverPreExposure on inputs, View.PreExposure on output).  Useful for
	 * materials that don't care about the absolute intensity of SceneColor (for example, a blur), simplifying custom HLSL and saving some performance.
	 * Or useful for non-color UserSceneTextures (for example, mask, matte, modulation, offset, or ID textures), where pre-exposure scale is
	 * undesirable.  Pre-exposure scale can be manually reapplied via custom HLSL if needed on specific inputs or the output.
	 */
	UPROPERTY(EditAnywhere, Category = PostProcessMaterial)
	uint8 bDisablePreExposureScale : 1;

	/**
	* Indicates that the material and its instances can be used with neural network engine.
	* This will result in the shaders required to support neural network engine being compiled which will increase shader compile time and memory usage.
	* In addition, an additional pass will run before the postprocess pass for neural network engine, which will increase the rendering cost due to
	* buffer copy and inference.
	*/
	UE_DEPRECATED(5.8, "Always use the GetUsageByFlag() accessor.")
	UPROPERTY(EditAnywhere, Category = PostProcessMaterial)
	uint8 bUsedWithNeuralNetworks : 1;

	/** 
	 * Selectively execute post process material only for pixels that pass the stencil test against the Custom Depth/Stencil buffer. 
	 * Pixels that fail the stencil test are filled with the previous post process material output or scene color.
	 */
	UPROPERTY(EditAnywhere, Category = PostProcessMaterial, AdvancedDisplay)
	uint8 bEnableStencilTest : 1;

	UPROPERTY(EditAnywhere, Category = PostProcessMaterial, AdvancedDisplay, meta = (EditCondition = "bEnableStencilTest"))
	TEnumAsByte<EMaterialStencilCompare> StencilCompare;

	UPROPERTY(EditAnywhere, Category = PostProcessMaterial, AdvancedDisplay, meta = (EditCondition = "bEnableStencilTest"))
	uint8 StencilRefValue = 0;

	/**
	* Set by reference object cannot be modified.
	*/
	UPROPERTY(EditAnywhere, Category = PostProcessMaterial, AdvancedDisplay, meta = (EditCondition = "false"))
	int8 NeuralProfileId = INDEX_NONE;

	UPROPERTY()
	TEnumAsByte<ERefractionMode> RefractionMode_DEPRECATED;

	/** Controls how the Refraction input is interpreted and how the refraction offset into scene color is computed for this material. */
	UPROPERTY(EditAnywhere, Category=Refraction)
	TEnumAsByte<ERefractionMode> RefractionMethod;

	/** Controls whether refraction takes into account the material surface coverage, or not. This is only for when substrate FrontMaterial is plugged in, not legacy root node material for the sake of compatibility. */
	UPROPERTY(EditAnywhere, Category=Refraction)
	TEnumAsByte<ERefractionCoverageMode> RefractionCoverageMode;

	/** Controls whether refraction takes into account the material surface coverage, or not. */
	UPROPERTY(EditAnywhere, Category=PixelDepthOffset)
	TEnumAsByte<EPixelDepthOffsetMode> PixelDepthOffsetMode;

	/** If multiple nodes with the same  type are inserted at the same point, this defined order and if they get combined, only used if domain is PostProcess */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "Blendable Priority"))
	int32 BlendablePriority;

	/** Allows blendability to be turned off, only used if domain is PostProcess */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PostProcessMaterial, meta = (DisplayName = "Is Blendable"))
	uint8 bIsBlendable : 1;

	/** If non-zero, overrides r.Material.PreshaderGapInterval for this material.  Workaround for a platform specific register overflow bug. */
	UPROPERTY(EditAnywhere, Category = Material, AdvancedDisplay)
	uint16 PreshaderGap;

	/** This is the refraction depth bias, larger values offset distortion to prevent closer objects from rendering into the distorted surface at acute viewing angles but increases the disconnect between surface and where the refraction starts. */
	UPROPERTY(EditAnywhere, Category=Refraction)
	float RefractionDepthBias;

	/** 
	 * Specifies the max world position offset of the material. Use this value to resolve issues with culling and self-occlusion caused by
	 * World Position Offset, and/or to restrict how much offset is permitted (i.e. values are clamped on each axis).
	 * NOTE: A value of 0.0 effectively means "no maximum", and will not clamp the offsets, however it will also not extend culling bounds.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPositionOffset, meta = (ClampMin = 0.0f))
	float MaxWorldPositionOffsetDisplacement;

	/**
	 * Forces World Position Offset to always be evaluated for this material, even if the primitive it's applied to has disabled it
	 * via "Evaluate World Position Offset" or "World Position Offset Disable Distance".
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = WorldPositionOffset)
	bool bAlwaysEvaluateWorldPositionOffset;

	/** Not a UPROPERTY, used to propagate editor only Substrate material simplification options for preview. */
	FSubstrateCompilationConfig SubstrateCompilationConfig;

	/** 
	 * Guid that uniquely identifies this material. 
	 * Any changes to the state of the material that do not appear separately in the shadermap DDC keys must cause this guid to be regenerated!
	 * For example, a modification to the Expressions array.
	 * Code changes that cause the guid to be regenerated on load should be avoided, as that requires a resave of the content to stop recompiling every load.
	 */
	UPROPERTY()
	FGuid StateId;

	/** 
	 * FMaterialRenderProxy derivative that represent this material to the renderer, when the renderer needs to fetch parameter values.
	 */
	class FDefaultMaterialInstance* DefaultMaterialInstance;

#if WITH_EDITORONLY_DATA
	/** Used to detect duplicate parameters.  Does not contain parameters in referenced functions! */
	TMap<FName, TArray<UMaterialExpression*> > EditorParameters;

	/** EdGraph based representation of the Material */
	TObjectPtr<class UMaterialGraph>	MaterialGraph;
#endif //WITH_EDITORONLY_DATA

private:
	/** Inline material resources serialized from disk. To be processed on game thread in PostLoad. */
	TArray<FMaterialResource> LoadedMaterialResources;

	/** 
	 * Material resources used for rendering this material.
	 * There need to be as many entries in this array as can be used simultaneously for rendering.  
	 * For example the material needs to support being rendered at different quality levels and feature levels within the same process.
	 * These are always valid and non-null, but only the entries affected by CacheResourceShadersForRendering are actually valid for rendering.
	 */
	TArray<FMaterialResource*> MaterialResources;

	/** Flag used to guarantee that the RT is finished using various resources in this UMaterial before cleanup. */
	FThreadSafeBool ReleasedByRT;

	/** This instance is caching uniform expressions (the render thread hasn't canceled its creation). */
	mutable bool bCachingUniformExpressions = false;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bSavedCachedExpressionData_DEPRECATED;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FGuid> ReferencedTextureGuids;

#endif // WITH_EDITORONLY_DATA
public:

	ENGINE_API UMaterial();
	ENGINE_API UMaterial(FVTableHelper& Helper);

	//~ Begin UMaterialInterface Interface.
	ENGINE_API virtual UMaterial* GetMaterial() override;
	ENGINE_API virtual const UMaterial* GetMaterial() const override;
	ENGINE_API virtual const UMaterial* GetMaterial_Concurrent(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	ENGINE_API virtual void GetMaterialInheritanceChain(FMaterialInheritanceChain& OutChain) const override;
	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue, EMaterialGetParameterValueFlags Flags = EMaterialGetParameterValueFlags::Default) const override;

	ENGINE_API virtual bool GetRefractionSettings(float& OutBiasValue) const override;
	ENGINE_API virtual EBlendableLocation GetBlendableLocation(const UMaterial* Base) const override;
	ENGINE_API virtual int32 GetBlendablePriority(const UMaterial* Base) const override;
	ENGINE_API virtual void GetDependencies(TSet<UMaterialInterface*>& Dependencies) override;
	ENGINE_API virtual FMaterialRenderProxy* GetRenderProxy() const override;
	ENGINE_API virtual UPhysicalMaterial* GetPhysicalMaterial() const override;
	ENGINE_API virtual UPhysicalMaterialMask* GetPhysicalMaterialMask() const override;
	ENGINE_API virtual UPhysicalMaterial* GetPhysicalMaterialFromMap(int32 Index) const override;
	ENGINE_API virtual UMaterialInterface* GetNaniteOverride(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	ENGINE_API virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, TOptional<EMaterialQualityLevel::Type> InSpecificQualityLevel = TOptional<EMaterialQualityLevel::Type>(), TOptional<EShaderPlatform> InSpecificShaderPlatform = TOptional<EShaderPlatform>()) const override;
	ENGINE_API virtual void GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const override;
	ENGINE_API virtual void OverrideTexture(const UTexture* InTextureToOverride, UTexture* OverrideTexture, ERHIFeatureLevel::Type InFeatureLevel) override;
	ENGINE_API virtual void SetNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value, EShaderPlatform InShaderPlatform) override;
	ENGINE_API virtual void ClearNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, EShaderPlatform InShaderPlatform) override;
	ENGINE_API virtual uint32 GetUsageFlags() const override;
	ENGINE_API virtual bool GetUsageByFlag(EMaterialUsage Usage) const override;
	ENGINE_API virtual bool SetMaterialUsage(EMaterialUsage Usage) override;
	ENGINE_API virtual FMaterialResource* AllocateResource();
	ENGINE_API virtual FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) override;
	ENGINE_API virtual const FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) const override;
	ENGINE_API virtual bool GetMaterialLayers(FMaterialLayersFunctions& OutLayers, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	ENGINE_API virtual bool UpdateLightmassTextureTracking() override;
#if WITH_EDITOR
	ENGINE_API virtual bool GetGroupSortPriority(const FString& InGroupName, int32& OutSortPriority) const override;
	ENGINE_API virtual bool GetTexturesInPropertyChain(EMaterialProperty InProperty, TArray<UTexture*>& OutTextures, 
		TArray<FName>* OutTextureParamNames, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality) override;
#endif
	ENGINE_API virtual void RecacheUniformExpressions(bool bRecreateUniformBuffer) const override;
	
	ENGINE_API virtual float GetOpacityMaskClipValue() const override;
	ENGINE_API virtual bool GetCastDynamicShadowAsMasked() const override;
	ENGINE_API virtual EBlendMode GetBlendMode() const override;
	ENGINE_API virtual FMaterialShadingModelField GetShadingModels() const override;
	ENGINE_API virtual bool IsShadingModelFromMaterialExpression() const override;
	ENGINE_API virtual bool IsTwoSided() const override;
	ENGINE_API virtual bool IsThinSurface() const override;
	ENGINE_API virtual bool IsDitheredLODTransition() const override;
	ENGINE_API virtual bool IsTranslucencyWritingCustomDepth() const override;
	ENGINE_API virtual bool IsTranslucencyWritingVelocity() const override;
	ENGINE_API virtual bool IsTranslucencyVelocityFromDepth() const override;
	ENGINE_API virtual bool IsTranslucencyWritingFrontLayerTransparency() const override;
	ENGINE_API virtual bool IsMasked() const override;
	ENGINE_API virtual bool IsDeferredDecal() const override;
	ENGINE_API virtual bool IsUIMaterial() const;
	ENGINE_API virtual bool IsPostProcessMaterial() const;
	ENGINE_API virtual bool IsLightFunctionMaterial() const;
	ENGINE_API bool IsPostProcessMaterialOutputingAlpha() const;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfile_Internal() const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal() const override;
	ENGINE_API virtual uint32 NumSubsurfaceProfileRoot_Internal() const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal(uint32 Index) const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileOverride_Internal() const override;
	ENGINE_API virtual uint32 NumSpecularProfile_Internal() const override;
	ENGINE_API virtual USpecularProfile* GetSpecularProfile_Internal(uint32 Index) const override;
	ENGINE_API virtual UNeuralProfile* GetNeuralProfile_Internal() const override;
	ENGINE_API virtual uint32 NumToonProfile_Internal() const override;
	ENGINE_API virtual UToonProfile* GetToonProfile_Internal(uint32 Index) const override;
	ENGINE_API virtual bool CastsRayTracedShadows() const override;
	ENGINE_API virtual bool IsTessellationEnabled() const override;
	ENGINE_API virtual FDisplacementScaling GetDisplacementScaling() const override;
	ENGINE_API virtual bool IsDisplacementFadeEnabled() const override;
	ENGINE_API virtual FDisplacementFadeRange GetDisplacementFadeRange() const override;
	ENGINE_API virtual float GetMaxWorldPositionOffsetDisplacement() const override;
	ENGINE_API virtual bool ShouldAlwaysEvaluateWorldPositionOffset() const override;
	ENGINE_API virtual bool HasPixelAnimation() const override;
	ENGINE_API virtual bool HasVertexInterpolator() const override;
	ENGINE_API virtual bool HasCustomizedUVs() const override;
	ENGINE_API virtual bool WritesToRuntimeVirtualTexture() const override;
	ENGINE_API virtual bool HasMeshPaintTexture() const override;
	ENGINE_API virtual bool HasCustomPrimitiveData() const override;
	ENGINE_API virtual bool HasSubstrateRoughnessTracking() const override;
	ENGINE_API virtual bool IsCompatibleWithLumenCardSharing() const override;

	ENGINE_API bool IsMobileSeparateTranslucencyEnabled() const;

	ENGINE_API virtual FGraphEventArray PrecachePSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs) override;

	ENGINE_API void SetShadingModel(EMaterialShadingModel NewModel);

	/** Checks to see if an input property should be active, based on the state of the material */
	ENGINE_API virtual bool IsPropertyActive(EMaterialProperty InProperty) const override;

#if WITH_EDITORONLY_DATA
	/** Return true if the material support shading model override */
	ENGINE_API bool SupportsShadingModelOverride() const;
#endif

#if WITH_EDITOR
	/** 
	* Like IsPropertyActive(), but should be used in context of editor.
	* For example, there is an optimization that transforms masked materials into opaque materials in certain situations.  If this optimization is active,
	* the opacity mask input will no longer be active normally (since blend mode will be reported as opaque),
	* but we still want to be able to connect this input from within the material editor.
	*/
	ENGINE_API bool IsPropertyActiveInEditor(EMaterialProperty InProperty) const;
#endif

	/** Like IsPropertyActive(), but considers any state overriden by DerivedMaterial */
	ENGINE_API bool IsPropertyActiveInDerived(EMaterialProperty InProperty, const UMaterialInterface* DerivedMaterial) const;

#if WITH_EDITOR
	/** Allows material properties to be compiled with the option of being overridden by the material attributes input. */
	ENGINE_API virtual int32 CompilePropertyEx( class FMaterialCompiler* Compiler, const FGuid& AttributeID ) override;
	ENGINE_API virtual bool ShouldForcePlanePreview() override;
	ENGINE_API virtual void ForceRecompileForRendering(EMaterialShaderPrecompileMode CompileMode = EMaterialShaderPrecompileMode::Default) override;

	/* returns whether valid compilation output exists. Outputs all default textures used by the material, based on the compilation output. */
	ENGINE_API virtual bool TryGetUsedTexturesInCookCache(TArray<TObjectPtr<UObject>>& OutTextures) override;
#endif // WITH_EDITOR
	//~ End UMaterialInterface Interface.

	//~ Begin UObject Interface
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
#if WITH_EDITOR
	ENGINE_API virtual void BeginCacheForCookedPlatformData( const ITargetPlatform *TargetPlatform ) override;
	ENGINE_API virtual bool IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) override;
	ENGINE_API virtual void ClearCachedCookedPlatformData( const ITargetPlatform *TargetPlatform ) override;
	ENGINE_API virtual void ClearAllCachedCookedPlatformData() override;
#endif
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	ENGINE_API virtual bool CanBeClusterRoot() const override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~ End UObject Interface

	enum class EPostEditChangeEffectOnShaders
	{
		Default,
		DoesNotInvalidate
	};

	ENGINE_API void PostEditChangePropertyInternal(FPropertyChangedEvent& PropertyChangedEvent, const EPostEditChangeEffectOnShaders EffectOnShaders);

#if WITH_EDITOR
	/** Cancels any currently outstanding compilation jobs for this material. Useful in the material editor when some edits superceds existing, in flight compilation jobs.*/
	ENGINE_API virtual void CancelOutstandingCompilation();
#endif // WITH_EDITOR

	/**
	 * Return the default material, loading it if necessary
	 */
	ENGINE_API static UMaterial* GetDefaultMaterial(EMaterialDomain Domain);

	/**
	 * Returns true if the material is one of the default materials.
	 */
	ENGINE_API bool IsDefaultMaterial() const;

	/** 
	 * Releases rendering resources used by this material. 
	 * This should only be called directly if the material will not be deleted through the GC system afterward.
	 * FlushRenderingCommands() must have been called before this.
	 */
	ENGINE_API void ReleaseResources();

	/** Checks to see if the Usage flag has an annotation marking it as needing to be saved */
	ENGINE_API bool IsUsageFlagDirty(EMaterialUsage Usage);
	
	UE_DEPRECATED(5.7, "Please use IsCompilingOrHadCompileError with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API bool IsCompilingOrHadCompileError(ERHIFeatureLevel::Type InFeatureLevel);

	/** Useful to customize rendering if that case (e.g. hide the object) */
	ENGINE_API bool IsCompilingOrHadCompileError(EShaderPlatform InShaderPlatform);

#if WITH_EDITORONLY_DATA
	ENGINE_API TConstArrayView<TObjectPtr<UMaterialExpression>> GetExpressions() const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API TConstArrayView<TObjectPtr<UMaterialExpressionComment>> GetEditorComments() const;
	ENGINE_API const FMaterialExpressionCollection& GetExpressionCollection() const;
	ENGINE_API FMaterialExpressionCollection& GetExpressionCollection();
	ENGINE_API void AssignExpressionCollection(const FMaterialExpressionCollection& InCollection);

	ENGINE_API bool SetParameterValueEditorOnly(const FName& ParameterName, const FMaterialParameterMetadata& Meta);
	ENGINE_API bool SetVectorParameterValueEditorOnly(FName ParameterName, FLinearColor InValue);
	ENGINE_API bool SetScalarParameterValueEditorOnly(FName ParameterName, float InValue);
	ENGINE_API bool SetTextureParameterValueEditorOnly(FName ParameterName, class UTexture* InValue);
	ENGINE_API bool SetTextureCollectionParameterValueEditorOnly(FName ParameterName, class UTextureCollection* InValue);
	ENGINE_API bool SetRuntimeVirtualTextureParameterValueEditorOnly(FName ParameterName, class URuntimeVirtualTexture* InValue);
	ENGINE_API bool SetSparseVolumeTextureParameterValueEditorOnly(FName ParameterName, class USparseVolumeTexture* InValue);
	ENGINE_API bool SetFontParameterValueEditorOnly(FName ParameterName, class UFont* InFontValue, int32 InFontPage);
	ENGINE_API bool SetStaticComponentMaskParameterValueEditorOnly(FName ParameterName, bool R, bool G, bool B, bool A, FGuid OutExpressionGuid);
	ENGINE_API bool SetStaticSwitchParameterValueEditorOnly(FName ParameterName, bool OutValue, FGuid OutExpressionGuid);
#endif // WITH_EDITOR

	/** Tracking of in-flight uniform expression cache update operations for the DefaultMaterialInstance, for thread safety destroying the resource. */
	void StartCacheUniformExpressions() const;
	void FinishCacheUniformExpressions() const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/**
	 * Output to the log which materials and textures are used by this material.
	 * @param Indent	Number of tabs to put before the log.
	 */
	ENGINE_API virtual void LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const override;
#endif

	/**
	 *	Returns all the Guids related to this material. For material instances, this includes the parent hierarchy.
	 *  Used for versioning as parent changes don't update the child instance Guids.
	 *
	 *	@param	bIncludeTextures	Whether to include the referenced texture Guids.
	 *	@param	OutGuids			The list of all resource guids affecting the precomputed lighting system and texture streamer.
	 */
	ENGINE_API virtual void GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const override;

#if WITH_EDITOR
	/**
	 *	Returns a CRC of all the Guids related to this material's shader.
	 *  This includes Guids from the parent hierarchy (for material instances),
	 *  material function state, and parameter assignment state.
	 *  NOTE this function does NOT include texture content state or texture streaming state.
	 *  Used as a faster way to detect changes than GetLightingGuidChain.
	 */
	ENGINE_API virtual uint32 ComputeAllStateCRC() const override;
#endif // WITH_EDITOR

	/**
	 * Checks that no pre-compilation errors have been detected and if so it reports them using specified compiler.
	 * @return whether no errors occurred.
	 */
	ENGINE_API bool CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const;
	
private:
	void SetInitialValues();

	/** Handles setting up an annotation for this object if a flag has changed value */
	void MarkUsageFlagDirty(EMaterialUsage Usage, bool CurrentValue, bool NewValue);
	
	/** to share code for PostLoad() and PostEditChangeProperty(), and UMaterialInstance::InitResources(), needs to be refactored */
	void PropagateDataToMaterialProxy();

#if WITH_EDITOR
	void BackwardsCompatibilityInputConversion();
	void BackwardsCompatibilityVirtualTextureOutputConversion();
	void BackwardsCompatibilityDecalConversion();
	bool ConvertMaterialToSubstrateMaterial(bool bAllowEmptyMaterialUpdate, bool bRequestedByUser = false);

	/** Marks the material's package dirty in order to make a material usage change set during map load persistent. 
	  * This couldn't be done during map load as loading cannot mark packages dirty. Invoked manually by the user 
	  * from the Map Check message log.
	  */
	void FixupMaterialUsageAfterLoad();
#endif

public:

#if WITH_EDITOR
	ENGINE_API bool UserRequestsConvertMaterialToSubstrateMaterial();
#endif

	/** Directly set the usage flag on this material. This doesn't validate the usage flag for the material, and it doesn't trigger recompilation. */
	ENGINE_API void SetUsageByFlag(EMaterialUsage Usage, bool NewValue);

	UE_DEPRECATED(5.8, "Use the new (virtual) implementation")
	bool SetMaterialUsage(bool& bNeedsRecompile, EMaterialUsage Usage, UMaterialInterface* MaterialInstance = nullptr) { return SetMaterialUsage(Usage); }

	ENGINE_API virtual void CacheShaders(EMaterialShaderPrecompileMode CompileMode) override;
#if WITH_EDITOR
	ENGINE_API virtual void CacheGivenTypesForCooking(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type QualityLevel, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*> PipelineTypes, const TArray<const FShaderType*>& ShaderTypes) override;
#endif
	ENGINE_API virtual bool IsComplete() const override;

#if WITH_EDITOR
	ENGINE_API virtual bool IsCompiling() const override;
#endif

#if WITH_EDITORONLY_DATA
	ENGINE_API virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const override;
	ENGINE_API virtual void GetDependentFunctions(TArray<class UMaterialFunctionInterface*>& DependentFunctions) const override;
#endif // WITH_EDITORONLY_DATA

	/** Returns the material's decal response mode */
	uint32 GetMaterialDecalResponse() const { return MaterialDecalResponse; }

#if WITH_EDITORONLY_DATA
	/**
	 * Attempt to find a expression by its GUID.
	 */
	template<typename ExpressionType>
	ExpressionType* FindExpressionByGUID(const FGuid &InGUID)
	{
		if (InGUID.IsValid())
		{
			return FindExpressionByGUIDRecursive<ExpressionType>(InGUID, GetExpressions());
		}
		return nullptr;
	}

	/** Builds a map with Guids as keys and Expressions as Values */
	ENGINE_API void BuildExpressionParametersGuidsMap(TMap<FGuid, TObjectPtr<UMaterialExpression>>& OutExpressionMap);

	/* Get all expressions of the requested type */
	template<typename ExpressionType>
	void GetAllExpressionsOfType(TArray<const ExpressionType*>& OutExpressions) const
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{
			ExpressionType* ExpressionPtr = Cast<ExpressionType>(Expression);
			if (ExpressionPtr)
			{
				OutExpressions.Add(ExpressionPtr);
			}
		}
	}

	static ENGINE_API UMaterialFunctionInterface* GetExpressionFunctionPointer(const UMaterialExpression* Expression);

	struct FLayersInterfaces
	{
		TConstArrayView<UMaterialFunctionInterface*> Layers;
		TConstArrayView<UMaterialFunctionInterface*> Blends;
	};
	static ENGINE_API TOptional<FLayersInterfaces> GetExpressionLayers(const UMaterialExpression* Expression);

	/** Get all expressions of the requested type, recursing through any function expressions in the material */
	template<typename ExpressionType>
	void GetAllExpressionsInMaterialAndFunctionsOfType(TArray<ExpressionType*>& OutExpressions) const
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{
			ExpressionType* ExpressionOfType = Cast<ExpressionType>(Expression);
			if (ExpressionOfType)
			{
				OutExpressions.Add(ExpressionOfType);
			}

			if (UMaterialFunctionInterface* MaterialFunction = GetExpressionFunctionPointer(Expression))
			{
				MaterialFunction->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
			}
			else if (TOptional<FLayersInterfaces> LayersInterfaces = GetExpressionLayers(Expression))
			{
				for (UMaterialFunctionInterface* Layer : LayersInterfaces->Layers)
				{
					if (Layer)
					{
						Layer->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
					}
				}

				for (UMaterialFunctionInterface* Blend : LayersInterfaces->Blends)
				{
					if (Blend)
					{
						Blend->GetAllExpressionsOfType<ExpressionType>(OutExpressions);
					}
				}
			}
		}
	}

	/** Checks if the material contains an expression of the requested type, recursing through any function expressions in the material */
	template<typename ExpressionType>
	bool HasAnyExpressionsInMaterialAndFunctionsOfType() const
	{
		for (UMaterialExpression* Expression : GetExpressions())
		{
			ExpressionType* ExpressionOfType = Cast<ExpressionType>(Expression);
			if (ExpressionOfType)
			{
				return true;
			}

			if (UMaterialFunctionInterface* MaterialFunction = GetExpressionFunctionPointer(Expression))
			{
				if (MaterialFunction->HasAnyExpressionsOfType<ExpressionType>())
				{
					return true;
				}
			}
			else if (TOptional<FLayersInterfaces> LayersInterfaces = GetExpressionLayers(Expression))
			{
				for (auto* Layer : LayersInterfaces->Layers)
				{
					if (Layer && Layer->HasAnyExpressionsOfType<ExpressionType>())
					{
						return true;
					}
				}

				for (auto* Blend : LayersInterfaces->Blends)
				{
					if (Blend && Blend->HasAnyExpressionsOfType<ExpressionType>())
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	ENGINE_API void UpdateTransientExpressionData();
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API void UpdateCachedExpressionData();
#endif

	/** Try to get an existing sorting group with the given name. Nullptr if none is found. */
	ENGINE_API const FParameterGroupData* GetSortingGroup(const FString& InGroupName);

	/** Attempts to add a new group name to the Group Data struct. True if new name was added. */
	ENGINE_API bool AttemptInsertNewGroupName(const FString& InNewName);

private:
	/**
	 * Flush existing resource shader maps and combines the material resource's Ids with another unique guid that represent the transformation applied so we can stay deterministic.
	 * If the guid given is non-valid (default value) we generate an entirely new guid for the resource.
	 * @param	TransformationId	If valid, combine this guid with the current material resource's Ids yielding a new but deterministic Ids.
	 */
	ENGINE_API virtual void ReleaseResourcesAndMutateDDCKey(const FGuid& TransformationId = FGuid());
	
	/** 
	 * Cache resource shaders for rendering. 
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 * The results will be applied to this FMaterial in the renderer when they are finished compiling.
	 * Note: This modifies material variables used for rendering and is assumed to be called within a FMaterialUpdateContext!
	 */
	void CacheResourceShadersForRendering(bool bRegenerateId, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default);

#if WITH_EDITOR
	/**
	 * Cache resource shaders for cooking on the given shader platform.
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 * This does not apply completed results to the renderer scenes.
	 * Caller is responsible for deleting OutCachedMaterialResources.
	 * Note: This modifies material variables used for rendering and is assumed to be called within a FMaterialUpdateContext!
	 */
	void CacheResourceShadersForCooking(EShaderPlatform Platform, TArray<FMaterialResourceForCooking>& OutCachedMaterialResources, 
		const ITargetPlatform* TargetPlatform = nullptr, bool bBlocking = false);
#endif

	void GetNewResources(EShaderPlatform ShaderPlatform, TArray<FMaterialResource*>& NewResourcesToCache);

	/** Caches shader maps for an array of material resources. */
	void CacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);

#if WITH_EDITOR

	/** Begins caching shader maps for an array of material resources. */
	void BeginCacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);

	/**
	 * Retrieves a hash of all texture references which should force a recompile of this material
	 */
	uint64 GetForceRecompileTextureIdsHash() const;
#endif // WITH_EDITOR

public:
#if WITH_EDITOR
	ENGINE_API bool IsTextureForceRecompileCacheRessource(UTexture *Texture);

	/* Recompute the ddc cache key and reload the material in case the key is not the same.
	 * It will also make sure lightmass texture reference are up to date
	 */
	ENGINE_API void UpdateMaterialShaderCacheAndTextureReferences();
#endif

	/**
	 * Go through every material, flush the specified types and re-initialize the material's shader maps.
	 */
	ENGINE_API static void UpdateMaterialShaders(TArray<const FShaderType*>& ShaderTypesToFlush, TArray<const FShaderPipelineType*>& ShaderPipelineTypesToFlush, TArray<const FVertexFactoryType*>& VFTypesToFlush, EShaderPlatform ShaderPlatform);

#if WITH_EDITOR
	/** Builds a map from UMaterialInterface name to the shader maps that are needed for rendering on the given platform. */
	ENGINE_API static void CompileMaterialsForRemoteRecompile(
		const TArray<UMaterialInterface*>& MaterialsToCompile,
		EShaderPlatform ShaderPlatform,
		class ITargetPlatform* TargetPlatform,
		TMap<FString, TArray<TRefCountPtr<class FMaterialShaderMap> > >& OutShaderMaps);

	ENGINE_API static void CompileODSCMaterialsForRemoteRecompile(TArray<FODSCRequestPayload> ShadersToRecompile, TMap<FString, TArray<TRefCountPtr<class FMaterialShaderMap>>>& OutShaderMaps, TFunction<UMaterialInterface*(const FString&)> ODSCCustomLoadMaterial = {});

	/**
	 * Add an expression node that represents a parameter to the list of material parameters.
	 * @param	Expression	Pointer to the node that is going to be inserted if it's a parameter type.
	 */
	ENGINE_API virtual bool AddExpressionParameter(UMaterialExpression* Expression, TMap<FName, TArray<UMaterialExpression*> >& ParameterTypeMap);

	/**
	 * Removes an expression node that represents a parameter from the list of material parameters.
	 * @param	Expression	Pointer to the node that is going to be removed if it's a parameter type.
	 */
	ENGINE_API virtual bool RemoveExpressionParameter(UMaterialExpression* Expression);

	/**
	 * A parameter with duplicates has to update its peers so that they all have the same value. If this step isn't performed then
	 * the expression nodes will not accurately display the final compiled material.
	 * @param	Parameter	Pointer to the expression node whose state needs to be propagated.
	 */
	ENGINE_API virtual void PropagateExpressionParameterChanges(const UMaterialExpression* Parameter);

	/**
	 * A parameter with duplicates has to update its peers so that they all have the same value. If this step isn't performed then
	 * the expression nodes will not accurately display the final compiled material.
	 * @param	ParameterName	Name of the parameter to update
	 * @param	Meta			New value of the parameter
	 */
	ENGINE_API virtual void PropagateExpressionParameterChanges(const FName& ParameterName, const FMaterialParameterMetadata& Meta);

	/**
	 * Remove the expression from the editor parameters list (if it exists) and then re-adds it.
	 * @param	Expression	The expression node that represents a parameter that needs updating.
	 */
	ENGINE_API virtual void UpdateExpressionParameterName(UMaterialExpression* Expression);

	/**
	 * Iterate through all of the expression nodes in the material and finds any parameters to put in EditorParameters.
	 */
	ENGINE_API virtual void BuildEditorParameterList();

	/**
	 * Return whether the provided expression parameter has duplicates.
	 * @param	Expression	The expression parameter to check for duplicates.
	 */
	ENGINE_API virtual bool HasDuplicateParameters(const UMaterialExpression* Expression);

	/**
	 * Return whether the provided expression dynamic parameter has duplicates.
	 * @param	Expression	The expression dynamic parameter to check for duplicates.
	 */
	ENGINE_API virtual bool HasDuplicateDynamicParameters(const UMaterialExpression* Expression);

	/**
	 * Iterate through all of the expression nodes and fix up changed properties on
	 * matching dynamic parameters when a change occurs.
	 *
	 * @param	Expression	The expression dynamic parameter.
	 */
	ENGINE_API virtual void UpdateExpressionDynamicParameters(const UMaterialExpression* Expression);

	/** Collect all material expressions fomr this material and all its functions and figure out which possible shading models exist in this material */
	ENGINE_API void RebuildShadingModelField();

	/**
	 * Get the name of a parameter.
	 * @param	Expression	The expression to retrieve the name from.
	 * @param	OutName		The variable that will hold the parameter name.
	 * @return	true if the expression is a parameter with a name.
	 */
	static bool GetExpressionParameterName(const UMaterialExpression* Expression, FName& OutName);

	/**
	 * Return whether the provided expression node is a parameter.
	 *
	 * @param	Expression	The expression node to inspect.
	 */
	ENGINE_API static bool IsParameter(const UMaterialExpression* Expression);

	/**
	 * Return whether the provided expression node is a dynamic parameter.
	 *
	 * @param	Expression	The expression node to inspect.
	 */
	ENGINE_API static bool IsDynamicParameter(const UMaterialExpression* Expression);
#endif // WITH_EDITOR

	/* Helper functions for text output of properties. */
	static const TCHAR* GetMaterialShadingModelString(EMaterialShadingModel InMaterialShadingModel);
	static EMaterialShadingModel GetMaterialShadingModelFromString(const TCHAR* InMaterialShadingModelStr);
	static const TCHAR* GetBlendModeString(EBlendMode InBlendMode);
	static EBlendMode GetBlendModeFromString(const TCHAR* InBlendModeStr);

#if WITH_EDITOR
	/**
	*	Get the expression input for the given property
	*
	*	@param	InProperty				The material property chain to inspect, such as MP_BaseColor.
	*
	*	@return	FExpressionInput*		A pointer to the expression input of the property specified, 
	*									or NULL if an invalid property was requested (some properties have been removed from UI, those return NULL).
	*/
	ENGINE_API FExpressionInput* GetExpressionInputForProperty(EMaterialProperty InProperty);

	ENGINE_API bool GetExpressionInputDescription(EMaterialProperty InProperty, FMaterialInputDescription& OutDescription);
#endif

#if WITH_EDITORONLY_DATA
	/* Returns any UMaterialExpressionFunctionOutput expressions */
	ENGINE_API void GetAllFunctionOutputExpressions(TArray<class UMaterialExpressionFunctionOutput*>& OutFunctionOutputs) const;
	/* Returns any UMaterialExpressionCustomOutput expressions */
	ENGINE_API void GetAllCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const;
	ENGINE_API void GetAllExpressionsForCustomInterpolators(TArray<class UMaterialExpression*>& OutExpressions) const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/**
	 *	Get all referenced expressions (returns the chains for all properties).
	 *
	 *	@param	OutExpressions						The array to fill in all of the expressions.
	 *	@param	InStaticParameterSet				Optional static parameter set - if supplied only walk the StaticSwitch branches according to it.
	 *	@Param	InFeatureLevel						Optional feature level - if supplied, only walk FeatureLevelSwitch branches according to it.
	 *	@Param	InQuality							Optional quality switch - if supplied, only walk QualitySwitch branches according to it.
	 *	@Param	InShadingPath						Optional shading path switch - if supplied, only walk ShadingPathSwitch branches according to it.
	 *  @Param	bInRecurseIntoMaterialFunctions		Optional enable recursion into Material Functions - if true, will also return expressions from inside of MaterialFunctions.
	 *  @Param	InMobileCustomOutputExpressionTypesToQuery	Optional a list of material expression custom output types to include in the query on mobile devices.  On non-mobile devices, all custom output types are queried.
	 *
	 *	@return	bool					true if successful, false if not.
	 */
	ENGINE_API virtual bool GetAllReferencedExpressions(TArray<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel = ERHIFeatureLevel::Num, EMaterialQualityLevel::Type InQuality = EMaterialQualityLevel::Num, ERHIShadingPath::Type InShadingPath = ERHIShadingPath::Num, const bool bInRecurseIntoMaterialFunctions = false, TSet<UClass*>* InMobileCustomOutputExpressionTypesToQuery = nullptr);


	/**
	 *	Get the expression chain for the given property (ie fill in the given array with all expressions in the chain).
	 *
	 *	@param	InProperty							The material property chain to inspect, such as MP_BaseColor.
	 *	@param	OutExpressions						The array to fill in all of the expressions.
	 *	@param	InStaticParameterSet				Optional static parameter set - if supplied only walk the StaticSwitch branches according to it.
	 *	@Param	InFeatureLevel						Optional feature level - if supplied, only walk FeatureLevelSwitch branches according to it.
	 *	@Param	InQuality							Optional quality switch - if supplied, only walk QualitySwitch branches according to it.
	 *	@Param	InShadingPath						Optional shading path switch - if supplied, only walk ShadingPathSwitch branches according to it.
	 *  @Param	bInRecurseIntoMaterialFunctions		Optional enable recursion into Material Functions - if true, will also return expressions from inside of MaterialFunctions.
	 *
	 *	@return	bool					true if successful, false if not.
	 */
	ENGINE_API virtual bool GetExpressionsInPropertyChain(EMaterialProperty InProperty, 
		TArray<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel = ERHIFeatureLevel::Num, EMaterialQualityLevel::Type InQuality = EMaterialQualityLevel::Num, ERHIShadingPath::Type InShadingPath = ERHIShadingPath::Num, const bool bInRecurseIntoMaterialFunctions = false);

#endif // WITH_EDITOR

protected:

#if WITH_EDITOR
	/**
	 *	Recursively retrieve the expressions contained in the chain of the given expression.
	 *
	 *	@param	InExpression						The expression to start at.
	 *	@param	InOutProcessedInputs				A set of processed expression inputs. (To avoid circular loops causing infinite recursion)
	 *	@param	OutExpressions						The set to fill in all of the expressions.
	 *	@param	InStaticParameterSet				Optional static parameter set - if supplied only walk the StaticSwitch branches according to it.
	 *	@Param	InFeatureLevel						Optional feature level - if supplied, only walk FeatureLevelSwitch branches according to it.
	 *	@Param	InQuality							Optional quality switch - if supplied, only walk QualitySwitch branches according to it.
	 *	@Param	InShadingPath						Optional shading path switch - if supplied, only walk ShadingPathSwitch branches according to it.
	 *	@Param	InShaderFrequency					Optional shader frequency - if supplied, only walk ShaderFrequencySwitch branches according to it.
	 *  @Param	bInRecurseIntoMaterialFunctions		Optional enable recursion into Material Functions - if true, will also return expressions from inside of MaterialFunctions.
	 *
	 *	@return	bool					true if successful, false if not.
	 */
	ENGINE_API virtual bool RecursiveGetExpressionChain(UMaterialExpression* InExpression, TSet<FExpressionInput*>& InOutProcessedInputs, 
		TSet<UMaterialExpression*>& OutExpressions, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel = ERHIFeatureLevel::Num,
		EMaterialQualityLevel::Type InQuality = EMaterialQualityLevel::Num,
		ERHIShadingPath::Type InShadingPath = ERHIShadingPath::Num,
		EShaderFrequency InShaderFrequency = SF_NumFrequencies, 
		EMaterialProperty InProperty = MP_MAX,
		const bool bInRecurseIntoMaterialFunctions = false);

	/**
	*	Recursively update the bRealtimePreview for each expression based on whether it is connected to something that is time-varying.
	*	This is determined based on the result of UMaterialExpression::NeedsRealtimePreview();
	*
	*	@param	InExpression				The expression to start at.
	*	@param	InOutExpressionsToProcess	Array of expressions we still need to process.
	*
	*/
	void RecursiveUpdateRealtimePreview(UMaterialExpression* InExpression, TArray<UMaterialExpression*>& InOutExpressionsToProcess);
#endif

public:
	/**
	 * Looks through the ReferencedDefaultTextures array for the texture present at the given Idx.
	 * At editor time, it looks through all default textures. 
	 * At runtime, it has access to default textures based on cook output, if ShouldStripUnusedDefaultTextures returns true.
	 * @return The default texture that is relevant to the given Idx or nullptr if not present.
	 */
	ENGINE_API virtual TObjectPtr<UObject> GetDefaultTextureAtIdx(const int32 InIdx, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) override;
	ENGINE_API virtual void DumpDebugInfo(FOutputDevice& OutputDevice) const override;
	void SaveShaderStableKeys(const class ITargetPlatform* TP);
	ENGINE_API virtual void SaveShaderStableKeysInner(const class ITargetPlatform* TP, const struct FStableShaderKeyAndValue& SaveKeyVal) override;

#if WITH_EDITOR
	ENGINE_API virtual void GetShaderTypes(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FDebugShaderTypeInfo>& OutShaderInfo) override;

	ENGINE_API virtual void AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const override;
#endif // WITH_EDITOR

	ENGINE_API bool IsPropertyConnected(EMaterialProperty Property) const;
	ENGINE_API bool HasBaseColorConnected() const;
	ENGINE_API bool HasRoughnessConnected() const;
	ENGINE_API bool HasAmbientOcclusionConnected() const;
	ENGINE_API bool HasNormalConnected() const;
	ENGINE_API bool HasSpecularConnected() const;
	ENGINE_API bool HasMetallicConnected() const;
	ENGINE_API bool HasEmissiveColorConnected() const;
	ENGINE_API bool HasAnisotropyConnected() const;
	ENGINE_API bool HasSurfaceThicknessConnected() const;
	ENGINE_API bool HasSubstrateFrontMaterialConnected() const;
	ENGINE_API bool HasVertexPositionOffsetConnected() const;
	ENGINE_API bool HasDisplacementConnected() const;
	ENGINE_API bool HasPixelDepthOffsetConnected() const;

	// Return true if the property is supported
	ENGINE_API bool IsPropertySupported(EMaterialProperty InProperty) const;

	/** Returns true if the material property is relevant for mobile platforms */
	ENGINE_API static bool IsPropertyRelevantForMobile(EMaterialProperty InProperty);

#if WITH_EDITOR
	static void NotifyCompilationFinished(UMaterialInterface* Material);

	DECLARE_EVENT_OneParam( UMaterial, FMaterialCompilationFinished, UMaterialInterface* );
	ENGINE_API static FMaterialCompilationFinished& OnMaterialCompilationFinished();
#endif // WITH_EDITOR

	// For all materials, UMaterial::CacheResourceShadersForRendering
	ENGINE_API static void AllMaterialsCacheResourceShadersForRendering(bool bUpdateProgressDialog = false, bool bCacheAllRemainingShaders = true);

#if WITH_EDITORONLY_DATA
	/**
	 * Flip the X coordinates of a material's expressions and space them out more
	 *
	 * @param	Expressions		Array of material expressions
	 * @param	Comments		Array of material expression comments
	 * @param	bScaleCoords	Whether to scale the coordinates to space out nodes
	 * @param	Material		The Material to flip its home coords (optional)
	 */
	static void FlipExpressionPositions(TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, TConstArrayView<TObjectPtr<UMaterialExpressionComment>> Comments, bool bScaleCoords, UMaterial* Material = NULL);

	/**
	 * Shifts the positions of comments so that they are aligned correctly with other expressions
	 *
	 * @param	Comments	Array of comments to fix
	 */
	static void FixCommentPositions(TConstArrayView<TObjectPtr<UMaterialExpressionComment>> Comments);

	/**
	 * Checks whether a Material is arranged in the old style, with inputs flowing from right to left
	 */
	bool HasFlippedCoordinates();

	/** Fixup feature level nodes that haven't set the SM6 pin. Returns true if a fixup was needed. */
	static bool FixFeatureLevelNodesForSM6(TArray<UMaterialExpression*> const& InExpressions);

#endif //WITH_EDITORONLY_DATA
	/**
	 * Return the array of physical materials associated with the RenderTrace material outputs.
	 * @return The physical material array.
	 */
	ENGINE_API TArrayView<const TObjectPtr<UPhysicalMaterial>> GetRenderTracePhysicalMaterialOutputs() const;

	/**
	 * Set the array of physical materials associated with the RenderTrace material outputs.
	 */
	ENGINE_API void SetRenderTracePhysicalMaterialOutputs(TArrayView<TObjectPtr<UPhysicalMaterial>> PhysicalMaterials);

#if WITH_EDITOR
	/** Get collection of shader tags from this material's expressions. */
	ENGINE_API void GetShaderTags(TArray<FName>& OutShaderTags) const;
	/** Get user flags on the material that influence translation validation errors. */
	ENGINE_API EMaterialTranslateValidationFlags GetMaterialTranslateValidationFlags() const;
#endif

private:
#if WITH_EDITOR
	static FMaterialCompilationFinished MaterialCompilationFinishedEvent;

	bool IsRefractionPinPluggedIn(const UMaterialEditorOnlyData* EditorOnly);
#endif // WITH_EDITOR

	friend class FLightmassMaterialProxy;
	/** Class that knows how to update Materials */
	friend class FMaterialUpdateContext;
	friend class FMaterialResource;
	friend class FMaterialEditor;
	friend class FMaterialDetailCustomization;

	// DO NOT CALL outside of FMaterialEditor!
	ENGINE_API static void ForceNoCompilationInPostLoad(bool bForceNoCompilation);

#if WITH_EDITORONLY_DATA
	/* Helper function to help finding expression GUID taking into account UMaterialExpressionMaterialFunctionCall */
	template<typename ExpressionType>
	ExpressionType* FindExpressionByGUIDRecursive(const FGuid& InGUID, TConstArrayView<TObjectPtr<UMaterialExpression>> InMaterialExpression)
	{
		for (int32 ExpressionIndex = 0; ExpressionIndex < InMaterialExpression.Num(); ExpressionIndex++)
		{
			UMaterialExpression* ExpressionPtr = InMaterialExpression[ExpressionIndex];

			if (ExpressionPtr && ExpressionPtr->GetParameterExpressionId() == InGUID)
			{
				check(ExpressionPtr->bIsParameterExpression);
				return Cast<ExpressionType>(ExpressionPtr);
			}
			else if (UMaterialFunctionInterface* MaterialFunction = GetExpressionFunctionPointer(ExpressionPtr))
			{
				if (ExpressionType* Expression = FindExpressionByGUIDRecursive<ExpressionType>(InGUID, MaterialFunction->GetExpressions()))
				{
					return Expression;
				}
			}
			else if (TOptional<FLayersInterfaces> LayersInterfaces = UMaterial::GetExpressionLayers(ExpressionPtr))
			{
				for (const auto* Layer : LayersInterfaces->Layers)
				{
					if (Layer)
					{
						if (ExpressionType* Expression = FindExpressionByGUIDRecursive<ExpressionType>(InGUID, Layer->GetExpressions()))
						{
							return Expression;
						}
					}
				}

				for (const auto* Blend : LayersInterfaces->Blends)
				{
					if (Blend)
					{
						if (ExpressionType* Expression = FindExpressionByGUIDRecursive<ExpressionType>(InGUID, Blend->GetExpressions()))
						{
							return Expression;
						}
					}
				}
			}
		}

		return nullptr;
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<class UMaterialExpressionComment>> EditorComments_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<class UMaterialExpression>> Expressions_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "GetEditorOnlyData().ParameterGroupData"))
	TArray<FParameterGroupData> ParameterGroupData_DEPRECATED;

	UPROPERTY()
	FColorMaterialInput DiffuseColor_DEPRECATED;

	UPROPERTY()
	FColorMaterialInput SpecularColor_DEPRECATED;

	UPROPERTY()
	FColorMaterialInput BaseColor_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Metallic_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Specular_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Roughness_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Anisotropy_DEPRECATED;

	UPROPERTY()
	FVectorMaterialInput Normal_DEPRECATED;

	UPROPERTY()
	FVectorMaterialInput Tangent_DEPRECATED;

	UPROPERTY()
	FColorMaterialInput EmissiveColor_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Opacity_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput OpacityMask_DEPRECATED;

	UPROPERTY()
	FVectorMaterialInput WorldPositionOffset_DEPRECATED;

	UPROPERTY()
	FColorMaterialInput SubsurfaceColor_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput ClearCoat_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput ClearCoatRoughness_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput AmbientOcclusion_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput Refraction_DEPRECATED;

	UPROPERTY()
	FVector2MaterialInput CustomizedUVs_DEPRECATED[8];

	UPROPERTY()
	FMaterialAttributesInput MaterialAttributes_DEPRECATED;

	UPROPERTY()
	FScalarMaterialInput PixelDepthOffset_DEPRECATED;

	UPROPERTY()
	FShadingModelMaterialInput ShadingModelFromMaterialExpression_DEPRECATED;

	UPROPERTY()
	FSubstrateMaterialInput FrontMaterial_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};



