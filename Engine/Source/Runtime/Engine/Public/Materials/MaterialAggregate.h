// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialValueType.h"
#include "Engine/DataAsset.h"
#include "SceneTypes.h"
#include "MaterialAggregate.generated.h"

class UMaterialAggregate;

// Specifies the type of an individual material aggregate attribute.
UENUM()
enum class EMaterialAggregateAttributeType
{
	Bool1,
	Bool2,
	Bool3,
	Bool4,
	
	UInt1,
	UInt2,
	UInt3,
	UInt4,

	Float1,
	Float2,
	Float3,
	Float4,

	// This is needed because of the way materials and material attributes sadly work. The normal attribute
	// default value is automatically assigned to (0, 0, 1) or VertexNormal depending on whether the material
	// is in tangent or world space normals. If one day this mechanism changes or is removed, remove this entrys 
	// and make Normals simple float3s.
	Normal,

	// Attribute is a shading mode. Used in by MaterialAttributes aggregate.
	ShadingModel,

	// Attribute is of type MaterialAttributes aggregate.
	MaterialAttributes,

	// Attribute is of type SubstrateData aggregate.
	SubstrateData,

	// Attribute is of specified user aggregate type.
	Aggregate,
};

// Represents a single attribute within a material aggregate.
USTRUCT(MinimalAPI)
struct FMaterialAggregateAttribute
{
	GENERATED_BODY()

	// The name of the attribute.
	UPROPERTY(EditAnywhere, Category=MaterialAggregateAttribute)
	FName Name{};

	// The type of the attribute.
	UPROPERTY(EditAnywhere, Category=MaterialAggregateAttribute)
	EMaterialAggregateAttributeType Type = EMaterialAggregateAttributeType::Float4;

	// The nested aggregate reference, used only if TypeKind is Aggregate.
	UPROPERTY(EditAnywhere, Category=MaterialAggregateAttribute, meta=(EditCondition="Type == EMaterialAggregateAttributeType::Aggregate", EditConditionHides))
	TObjectPtr<UMaterialAggregate> Aggregate{};

	// This attribute default value, assigned when the parent aggregate is constructed without an assignment to this attribute.
	UPROPERTY(EditAnywhere, Category=MaterialAggregateAttribute)
	FVector4f DefaultValue;

#if WITH_EDITOR
	// Converts this attribute's type to a corresponding material value type.
	EMaterialValueType ToMaterialValueType() const;
#endif
};

// It defines a collection of arithmetic material values to be bundled together.
// A material aggregate works similarly to a struct in C/C++. Each attribute has a name and specifies a type, either a
// primitive one like float4 or another aggregate (for nested structures).
UCLASS(MinimalAPI)
class UMaterialAggregate : public UDataAsset
{
	GENERATED_BODY()

public:
	// List of material aggregate attributes.
	UPROPERTY(EditAnywhere, Category=MaterialAggregate)
	TArray<FMaterialAggregateAttribute> Attributes;

public:
#if WITH_EDITOR
	// Finds the index of an attribute in the aggregate by name.
	int32 FindAttributeIndexByName(FName Name) const;

	virtual void PostLoad() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // #if WITH_EDITOR
};


#if WITH_EDITOR

// Aggregate APIs specific for the MaterialAttributes data type.
namespace MaterialAttributesAggregate
{

// Returns an array of all the MaterialAttributes properties.
const TConstArrayView<EMaterialProperty> GetMaterialProperties();

// Returns a reference to the global material attributes aggregate used by the engine.
const UMaterialAggregate* Get();

// Returns a single cached aggregate for the specified material property.
const FMaterialAggregateAttribute* GetAttribute(EMaterialProperty Property);

// Maps a material property to the corresponding attribute index in the aggregate.
int32 MaterialPropertyToAttributeIndex(EMaterialProperty Property);

// Maps an attribute index back to the corresponding material property.
EMaterialProperty AttributeIndexToMaterialProperty(int32 MaterialAttributeIndex);

// Returns whether specified material Property is enabled in a MaterialAttributes aggregate with given number of custom UVs enabled.
inline bool IsPropertyEnabled(EMaterialProperty Property)
{
	return MaterialPropertyToAttributeIndex(Property) != INDEX_NONE;
}

} // MaterialAttributesAggregate
#endif // #if WITH_EDITOR
