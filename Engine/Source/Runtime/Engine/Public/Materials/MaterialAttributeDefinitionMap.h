// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "MaterialValueType.h"
#include "Math/Vector4.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Templates/UnrealTemplate.h"

enum EMaterialProperty : int;
enum EMaterialShadingModel : int;
enum EShaderFrequency : uint8;

class FMaterialCompiler;
class FShaderKeyGenerator;
class UMaterial;

struct FMaterialShadingModelField;

/**
 * Custom attribute blend functions
 */
typedef int32(*MaterialAttributeBlendFunction)(FMaterialCompiler* Compiler, int32 A, int32 B, int32 Alpha);

/**
 * Attribute data describing a material property
 */
class FMaterialAttributeDefintion
{
public:
	FMaterialAttributeDefintion(const FGuid& InGUID, const FString& AttributeName, EMaterialProperty InProperty,
		EMaterialValueType InValueType, const FVector4& InDefaultValue, EShaderFrequency InShaderFrequency,
		int32 InTexCoordIndex = INDEX_NONE, bool bInIsHidden = false, MaterialAttributeBlendFunction InBlendFunction = nullptr);

	ENGINE_API int32 CompileDefaultValue(FMaterialCompiler* Compiler) const;

	bool operator==(const FMaterialAttributeDefintion& Other) const
	{
		return (AttributeID == Other.AttributeID);
	}

	FGuid				AttributeID;
	FVector4			DefaultValue;
	FString				AttributeName;
	EMaterialProperty	Property;
	EMaterialValueType	ValueType;
	EShaderFrequency	ShaderFrequency;
	int32				TexCoordIndex;

	// Optional function pointer for custom blend behavior
	MaterialAttributeBlendFunction BlendFunction;

	// Hidden from auto-generated lists but valid for manual material creation
	bool				bIsHidden;
};

/**
 * Attribute data describing a material property used for a custom output
 */
class FMaterialCustomOutputAttributeDefintion : public FMaterialAttributeDefintion
{
public:
	FMaterialCustomOutputAttributeDefintion(const FGuid& InGUID, UClass* InExpressionClass, const FString& InAttributeName, const FString& InFunctionName, int32 InOutputIndex, bool bInNeedsPreviousFrameEvaluation, EMaterialProperty InProperty,
		EMaterialValueType InValueType, const FVector4& InDefaultValue, EShaderFrequency InShaderFrequency, MaterialAttributeBlendFunction InBlendFunction = nullptr);

	bool operator==(const FMaterialCustomOutputAttributeDefintion& Other) const
	{
		return (AttributeID == Other.AttributeID);
	}

	UClass*				CustomOutputExpressionStaticClass;
	FString				FunctionName; // Name of function used to access attribute in shader code
	int32				OutputIndex;
	bool				bNeedsPreviousFrameEvaluation;
};

/**
 * Material property to attribute data mappings
 */
class FMaterialAttributeDefinitionMap
{
public:
	ENGINE_API FMaterialAttributeDefinitionMap();

	/** Compiles the default expression for a material attribute */
	static ENGINE_API int32 CompileDefaultExpression(FMaterialCompiler* Compiler, EMaterialProperty Property);

	/** Compiles the default expression for a material attribute */
	static ENGINE_API int32 CompileDefaultExpression(FMaterialCompiler* Compiler, const FGuid& AttributeID);

	/** Returns the display name of a material attribute */
	static ENGINE_API const FString& GetAttributeName(EMaterialProperty Property);

	/** Returns the display name of a material attribute */
	static ENGINE_API const FString& GetAttributeName(const FGuid& AttributeID);

	/** Returns the display name of a material attribute, accounting for overrides based on properties of a given material */
	static ENGINE_API FText GetDisplayNameForMaterial(EMaterialProperty Property, UMaterial* Material);

	/** Returns the display name of a material attribute, accounting for overrides based on properties of a given material */
	static ENGINE_API FText GetDisplayNameForMaterial(const FGuid& AttributeID, UMaterial* Material);

	/** Returns the value type of a material attribute */
	static ENGINE_API EMaterialValueType GetValueType(EMaterialProperty Property);

	/** Returns the value type of a material attribute */
	static ENGINE_API EMaterialValueType GetValueType(const FGuid& AttributeID);

	/** Returns the default value of a material property */
	static ENGINE_API FVector4f GetDefaultValue(EMaterialProperty Property);

	/** Returns the default value of a material attribute */
	static ENGINE_API FVector4f GetDefaultValue(const FGuid& AttributeID);

	/** Returns the shader frequency of a material attribute */
	static ENGINE_API EShaderFrequency GetShaderFrequency(EMaterialProperty Property);

	/** Returns the shader frequency of a material attribute */
	static ENGINE_API EShaderFrequency GetShaderFrequency(const FGuid& AttributeID);

	/** Returns the bit index of this property in the bitmasks returned by GetBitmask(), GetConnectedMaterialAttributesBitmask() */
	static ENGINE_API int32 GetIndex(EMaterialProperty Property);

	/** Returns the bit index of this attribute in the bitmasks returned by GetBitmask(), GetConnectedMaterialAttributesBitmask() and GetCustomOutputNodesBitmask() */
	static ENGINE_API int32 GetIndex(const FGuid& AttributeID);

	/** Returns a bitmask with just the bit corresponding to this property set */
	static ENGINE_API uint64 GetBitmask(EMaterialProperty Property);

	/** Returns a bitmask with just the bit corresponding to this attribute set */
	static ENGINE_API uint64 GetBitmask(const FGuid& AttributeID);

	/** Checks if the bit corresponding to the property/attribute is set in the bitmask */
	static ENGINE_API bool IsAttributeInBitmask(uint64 Bitmask, EMaterialProperty Property);

	/** Checks if the bit corresponding to the property/attribute is set in the bitmask */
	static ENGINE_API bool IsAttributeInBitmask(uint64 Bitmask, const FGuid& AttributeID);

	/** Returns the attribute ID for a matching material property */
	static ENGINE_API FGuid GetID(EMaterialProperty Property);

	/** Returns a the material property matching the specified attribute AttributeID */
	static ENGINE_API EMaterialProperty GetProperty(const FGuid& AttributeID);

	/** Returns the custom blend function of a material attribute */
	static ENGINE_API MaterialAttributeBlendFunction GetBlendFunction(const FGuid& AttributeID);

	/** Returns a default attribute AttributeID */
	static ENGINE_API FGuid GetDefaultID();

	/** Appends a hash of the property map intended for use with the DDC key */
	static ENGINE_API void AppendDDCKeyString(FString& String);
	static ENGINE_API void AppendDDCKey(FShaderKeyGenerator& KeyGen);

	/** Appends a new attribute definition to the custom output list */
	static ENGINE_API void AddCustomAttribute(const FGuid& AttributeID, UClass* ExpressionClass,
		const FString& AttributeName, const FString& FunctionName, int32 OutputIndex, EMaterialValueType ValueType,
		const FVector4& DefaultValue, EShaderFrequency ShaderFrequency, bool bNeedsPreviousFrameEvaluation = false,
		MaterialAttributeBlendFunction BlendFunction = nullptr);

	/** Returns the first custom attribute ID that has the specificed attribute name */
	static ENGINE_API FGuid GetCustomAttributeID(const FString& AttributeName);

	/**
	 * Returns the first custom attribute definition that has the specificed attribute name.
	 * This function is not threadsafe, do not use. Use GetCustomAttributeCopy instead.
	 * TODO: Deprecate this function.
	 */
	static ENGINE_API const FMaterialCustomOutputAttributeDefintion* GetCustomAttribute(const FString& AttributeName);
	/** Returns the first custom attribute definition that has the specificed attribute name. */
	static ENGINE_API TOptional<FMaterialCustomOutputAttributeDefintion> GetCustomAttributeCopy(const FString& AttributeName);

	/**
	 * Returns the first custom attribute definition that has the specificed attribute AttributeID.
	 * This function is not threadsafe, do not use. Use GetCustomAttributeCopy instead.
	 * TODO: Deprecate this function.
	 */
	static ENGINE_API const FMaterialCustomOutputAttributeDefintion* GetCustomAttribute(const FGuid& AttributeID);
	/** Returns the first custom attribute definition that has the specificed attribute AttributeID. */
	static ENGINE_API TOptional<FMaterialCustomOutputAttributeDefintion> GetCustomAttributeCopy(const FGuid& AttributeID);

	/** Returns a list of registered custom attributes */
	static ENGINE_API void GetCustomAttributeList(TArray<FMaterialCustomOutputAttributeDefintion>& CustomAttributeList);

	/**
	 * This function is not threadsafe, do not use. Use GetReadLockedOrderedVisibleAttributeList instead.
	 * TODO: Deprecate this function.
	 */
	static ENGINE_API const TArray<FGuid>& GetOrderedVisibleAttributeList();
	
	struct FReadLockedOrderVisibleAttributeList : public FNoncopyable
	{
	public:
		ENGINE_API ~FReadLockedOrderVisibleAttributeList();
		const TArray<FGuid>& List;
	private:
		FReadLockedOrderVisibleAttributeList();
		FReadLockedOrderVisibleAttributeList(FReadLockedOrderVisibleAttributeList&&) = delete;
		FReadLockedOrderVisibleAttributeList& operator=(FReadLockedOrderVisibleAttributeList&&) = delete;
		friend FMaterialAttributeDefinitionMap;
	};
	static ENGINE_API FReadLockedOrderVisibleAttributeList GetReadLockedOrderedVisibleAttributeList();

#if WITH_EDITOR
	/** Returns a bitmask of all attributes/properties that are set via Set/MakeMaterialAttribute expressions, taking material function calls into account */
	static ENGINE_API uint64 GetConnectedMaterialAttributesBitmask(TConstArrayView<TObjectPtr<class UMaterialExpression>> Expressions);
	/** Returns a bitmask of all attributes/properties that are set via Set/MakeMaterialAttribute expressions, taking material function calls into account */
	static ENGINE_API uint64 GetConnectedMaterialAttributesBitmask(TConstArrayView<class UMaterialExpression*> Expressions);
	/** Returns a bitmask of all supported custom output attributes in the given array of custom output expressions */
	static ENGINE_API uint64 GetCustomOutputNodesBitmask(TConstArrayView<class UMaterialExpressionCustomOutput*> Expressions);
#endif

	/** Returns a list of display names and their associated GUIDs for material properties */
	static ENGINE_API void GetAttributeNameToIDList(TArray<TPair<FString, FGuid>>& NameToIDList);

private:

	// Internal map management
	void InitializeAttributeMap();

	void Add(const FGuid& AttributeID, const FString& AttributeName, EMaterialProperty Property,
		EMaterialValueType ValueType, const FVector4& DefaultValue, EShaderFrequency ShaderFrequency,
		int32 TexCoordIndex = INDEX_NONE, bool bIsHidden = false, MaterialAttributeBlendFunction BlendFunction = nullptr);
	void AddCustomAttributeInner(const FGuid& AttributeID, UClass* ExpressionClass, const FString& AttributeName,
		const FString& FunctionName, int32 OutputIndex, EMaterialValueType ValueType, const FVector4& DefaultValue, 
		EShaderFrequency ShaderFrequency, bool bNeedsPreviousFrameEvaluation = false,
		MaterialAttributeBlendFunction BlendFunction = nullptr);


	FMaterialAttributeDefintion* Find(const FGuid& AttributeID);
	FMaterialAttributeDefintion* Find(EMaterialProperty Property);

	// Helper functions to determine display name based on shader model, material domain, etc.
	static FText GetAttributeOverrideForMaterialInLock(const FGuid& AttributeID, UMaterial* Material);
	static FString GetPinNameFromShadingModelFieldInLock(FMaterialShadingModelField InShadingModels,
		const TArray<TKeyValuePair<EMaterialShadingModel, FString>>& InCustomShadingModelPinNames, 
		const FString& InDefaultPinName);

#if WITH_EDITOR
	template<typename ExpressionsArrayType>
	static uint64 GetConnectedMaterialAttributesBitmask_Internal(const ExpressionsArrayType& Expressions);
#endif

	// Functions that we call internally from other functions on the class, the caller guarantees that the readlock
	// is already held.
	static int32 GetIndexInLock(EMaterialProperty Property);
	static int32 GetIndexInLock(const FGuid& AttributeID);
	static uint64 GetBitmaskInLock(EMaterialProperty Property);
	static uint64 GetBitmaskInLock(const FGuid& AttributeID);

	static FMaterialAttributeDefinitionMap GMaterialPropertyAttributesMap;

	TMap<EMaterialProperty, FMaterialAttributeDefintion>	AttributeMap; // Fixed map of compile-time definitions
	TArray<FMaterialCustomOutputAttributeDefintion>			CustomAttributes; // Array of custom output definitions
	TArray<FGuid>											OrderedVisibleAttributeList; // List used for consistency with e.g. combobox filling

	FString													AttributeDDCString;
	bool bIsInitialized;
};
