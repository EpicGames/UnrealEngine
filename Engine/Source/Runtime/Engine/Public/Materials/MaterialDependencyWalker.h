// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

class UMaterialExpression;

struct FMaterialDependencySearchMetadata
{
	FMaterialDependencySearchMetadata() :
		bHasTextureInput(0)
	{
	}

	uint32 bHasTextureInput : 1;
};

enum EMaterialDependencySearchFlags
{
	MDSF_TextureDependencyOnly = (1 << 0),
};


// Walks the dependency graph (i.e. all expression inputs) of the specified material expression.
// Returns the maxmium depth the search went or INDEX_NONE if the limit has reached before finding anything.
// @param SearchFlags can be a bitwise OR combination of EMaterialDependencySearchFlags.
ENGINE_API int32 WalkMaterialDependencyGraph(const UMaterialExpression* Expression, int32 MaxDepth, uint32 SearchFlags, FMaterialDependencySearchMetadata& OutMetaData);


#endif // WITH_EDITOR
