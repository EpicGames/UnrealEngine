// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "UObject/Interface.h"
#include "MaterialEnumeration.generated.h"

class IMaterialEnumerationProvider
{
	GENERATED_BODY()

public:
	// Tries to resolve the value for the given name.
	// Returns true if found, false otherwise. Writes result to OutValue.
	virtual bool ResolveValue(FName EntryName, int32& OutValue, int32 DefaultValue = 0) const = 0;

	// Iterates over all name-value entries.
	virtual void ForEachEntry(TFunctionRef<void (FName Name, int32 Value)> Iterator) const = 0;

	// Helper to get the value for a name, or return default if not found.
	ENGINE_API int32 GetValueOrDefault(FName EntryName, int32 DefaultValue = 0) const;
};

UINTERFACE(MinimalAPI)
class UMaterialEnumerationProvider : public UInterface
{
	GENERATED_BODY()
};
