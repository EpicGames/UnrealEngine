// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
//#include "MaterialCompiler.h"

#include "MaterialExpression.generated.h"

class UEdGraphNode;
class UMaterial;
class UTexture;
class UTextureCollection;
struct FGraphActionMenuBuilder;
struct FPropertyChangedEvent;
struct FMaterialParameterMetadata;
struct FMaterialShadingModelField;
struct FSubstrateMaterialInfo;
struct FSubstrateOperator;

class UMaterialExpression;
class UMaterialExpressionComment;
class FMaterialExpressionKey;

class FFindInMaterialResult;

enum EEdGraphPinDirection : int;
enum class EMaterialParameterType : uint8;

namespace MIR
{
	class FEmitter;
	struct FValue;
}

struct FSubstrateTranslatorDataInterface;

//@warning: FExpressionInput is mirrored in MaterialShared.h and manually "subclassed" in Material.h (FMaterialInput)
#if !CPP      //noexport struct
USTRUCT(noexport)
struct FExpressionInput
{
	/** UMaterial expression that this input is connected to, or NULL if not connected. */
	UPROPERTY()
	TObjectPtr<class UMaterialExpression> Expression;

	/** Index into Expression's outputs array that this input is connected to. */
	UPROPERTY()
	int32 OutputIndex;

	/** 
	 * optional FName of the input.  
	 * Note that this is the only member which is not derived from the output currently connected. 
	 */
	UPROPERTY()
	FName InputName;


	UPROPERTY()
	int32 Mask;

	UPROPERTY()
	int32 MaskR;

	UPROPERTY()
	int32 MaskG;

	UPROPERTY()
	int32 MaskB;

	UPROPERTY()
	int32 MaskA;
};

USTRUCT(noexport)
struct FMaterialAttributesInput : public FExpressionInput
{
	UPROPERTY(transient)
	int64 PropertyConnectedMask;
};

#endif

#if !CPP      //noexport struct
/** Struct that represents an expression's output. */
USTRUCT(noexport)
struct FExpressionOutput
{
	UPROPERTY()
	FName OutputName;

	UPROPERTY()
	int32 Mask;

	UPROPERTY()
	int32 MaskR;

	UPROPERTY()
	int32 MaskG;

	UPROPERTY()
	int32 MaskB;

	UPROPERTY()
	int32 MaskA;
};
#endif

enum class EMaterialExpressionSetParameterValueFlags : uint32
{
	None = 0u,
	SendPostEditChangeProperty = (1u << 0), // Send PostEditChangeProperty events for all properties that are modified
	NoUpdateExpressionGuid = (1u << 1), // By default ExpressionGUI will be updated for static parameters
	AssignGroupAndSortPriority  = (1u << 2), //  Update the Group and SortPriority along with parameter value
};
ENUM_CLASS_FLAGS(EMaterialExpressionSetParameterValueFlags);

USTRUCT()
struct FMaterialExpressionCollection
{
	GENERATED_BODY()

	ENGINE_API void AddExpression(UMaterialExpression* InExpression);
	ENGINE_API void RemoveExpression(UMaterialExpression* InExpression);
	ENGINE_API void AddComment(UMaterialExpressionComment* InExpression);
	ENGINE_API void RemoveComment(UMaterialExpressionComment* InExpression);
	ENGINE_API void Empty();

	/** Array of material expressions, excluding Comments.  Used by the material editor. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialExpression>> Expressions;
	
	/** Array of comments associated with this material; viewed in the material editor. */
	UPROPERTY()
	TArray<TObjectPtr<UMaterialExpressionComment>> EditorComments;

	/** The execution begin expression, if material is using an exec wire */
	UPROPERTY()
	TObjectPtr<UMaterialExpression> ExpressionExecBegin = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialExpression> ExpressionExecEnd = nullptr;
};

UCLASS(abstract, Optional, BlueprintType, Config=MaterialExpressions, hidecategories=Object, MinimalAPI)
class UMaterialExpression : public UObject
{
	GENERATED_UCLASS_BODY()

	static constexpr int32 CompileExecutionOutputIndex = -2;

	UPROPERTY(BlueprintReadWrite, Category = MaterialEditing)
	int32 MaterialExpressionEditorX;

	UPROPERTY(BlueprintReadWrite, Category = MaterialEditing)
	int32 MaterialExpressionEditorY;

	/** Expression's Graph representation */
	UPROPERTY(transient)
	TObjectPtr<UEdGraphNode>	GraphNode;

	/** If exists, expresssion containing this expression within its subgraph. */
	UPROPERTY()
	TObjectPtr<UMaterialExpression> SubgraphExpression;

	/** Text of last error for this expression */
	FString LastErrorText;

	/** GUID to uniquely identify this node, to help the tutorials out */
	UPROPERTY()
	FGuid MaterialExpressionGuid;

	/** 
	 * The material that this expression is currently being compiled in.  
	 * This is not necessarily the object which owns this expression, for example a preview material compiling a material function's expressions.
	 */
	UPROPERTY()
	TObjectPtr<class UMaterial> Material;

	/** 
	 * The material function that this expression is being used with, if any.
	 * This will be NULL if the expression belongs to a function that is currently being edited, 
	 */
	UPROPERTY()
	TObjectPtr<class UMaterialFunction> Function;

	/** A description that level designers can add (shows in the material editor UI). */
	UPROPERTY(EditAnywhere, Category=MaterialExpression, meta=(MultiLine=true, DisplayAfter = "SortPriority"))
	FString Desc;

	/** Set to true by RecursiveUpdateRealtimePreview() if the expression's preview needs to be updated in realtime in the material editor. */
	UPROPERTY()
	uint32 bRealtimePreview:1;

	/** If true, we should update the preview next render. This is set when changing bRealtimePreview. */
	UPROPERTY(transient)
	uint32 bNeedToUpdatePreview:1;

	/** Indicates that this is a 'parameter' type of expression and should always be loaded (ie not cooked away) because we might want the default parameter. */
	UPROPERTY()
	uint8 bIsParameterExpression : 1;

	/** If true, the comment bubble will be visible in the graph editor */
	UPROPERTY()
	uint32 bCommentBubbleVisible:1;

	/** If true, use the output name as the label for the pin */
	UPROPERTY()
	uint32 bShowOutputNameOnPin:1;

	/** If true, changes the pin color to match the output mask */
	UPROPERTY()
	uint32 bShowMaskColorsOnPin:1;

	/** If true, do not render the preview window for the expression */
	UPROPERTY()
	uint32 bHidePreviewWindow:1;

	/** If true, show a collapsed version of the node */
	UPROPERTY()
	uint32 bCollapsed:1;

	/** Whether the node represents an input to the shader or not.  Used to color the node's background. */
	UPROPERTY(config)
	uint32 bShaderInputData:1;

	/** Whether to draw the expression's inputs. */
	UPROPERTY()
	uint32 bShowInputs:1;

	/** Whether to draw the expression's outputs. */
	UPROPERTY()
	uint32 bShowOutputs:1;

	/** Localized categories to sort this expression into... */
	UPROPERTY(config)
	TArray<FText> MenuCategories;

	/** The expression's outputs, which are set in default properties by derived classes. */
	UPROPERTY()
	TArray<FExpressionOutput> Outputs;

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditImport() override;
	ENGINE_API virtual bool CanEditChange( const FProperty* InProperty ) const override;
	
	ENGINE_API virtual bool Modify( bool bAlwaysMarkDirty=true ) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void Serialize( FStructuredArchive::FRecord Record ) override;
	//~ End UObject Interface.

	ENGINE_API UObject* GetAssetOwner() const;
	ENGINE_API FString GetAssetPathName() const;

#if WITH_EDITOR
	/**
	 * Create the new shader code chunk needed for the Abs expression
	 *
	 * @param	Compiler - UMaterial compiler that knows how to handle this expression.
	 * @return	Index to the new FMaterialCompiler::CodeChunk entry for this expression
	 */	
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex);
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) { return Compile(Compiler, OutputIndex); }

	ENGINE_API bool IsUsingNewHLSLGenerator() const;

	/**
	 * Builds this expression's material IR.
	 * 
	 * This function is responsible for doing the semantic analysis on this error, error checking and reporting and
	 * implementing this expression's logic by emitting appropriate material IR values.
	 */
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter);

	/* Material expressions can add shader tags that to a material.  Adding a shader tag helps to identify materials
	 *  that use certain material expressions, and can be used to limit shader generation to only materials that have
	 *  use that material expression
	 */
	virtual void GetShaderTags(TArray<FName>& ShaderTagsOut) {}

	virtual TSharedPtr<class SGraphNodeMaterialBase> CreateCustomGraphNodeWidget() { return nullptr; }
#endif // WITH_EDITOR

	/**
	* Returns whether the creation of this expression is allowed in a given material or material function
	*
	* @param MaterialOrFunction - The material or function for this expression to test against
	* @return True if this expression can be created in this material or function, false otherwise.
	*/
	ENGINE_API virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const;


	/**
	* Fill the array with all textures dependence that should trig a recompile of the material.
	*/
	virtual void GetTexturesForceMaterialRecompile(TArray<UTexture *> &Textures) const { }


	/** 
	 * To get any texture references this expression emits.
	 * This is used to link the compiled uniform expressions with their default texture values. 
	 * Any UMaterialExpression whose compilation creates a texture uniform expression (eg Compiler->Texture, Compiler->TextureParameter) must implement this.
	 */
	virtual UObject* GetReferencedTexture() const { return nullptr; }

	using ReferencedTextureArray = TArray<UObject*, TInlineAllocator<4>>; 
	virtual ReferencedTextureArray GetReferencedTextures() const { return { GetReferencedTexture() }; }
	
	/** Returns true if GetReferencedTexture() / GetReferencedTextures() can ever return a valid pointer(s). */
	virtual bool CanReferenceTexture() const { return false; }

	virtual UTextureCollection* GetReferencedTextureCollection() const { return nullptr; }

#if WITH_EDITOR

	/** Counts the number of inputs this expression has. Default implementation has O(n) complexity. */
	ENGINE_API virtual int32 CountInputs() const;

	/**
	 * Returns the output at specified index is valid
	 */
	FExpressionOutput* GetOutput(int32 OutputIndex) { return Outputs.IsValidIndex(OutputIndex) ? &Outputs[OutputIndex] : nullptr; }

	/**
	 * Returns the input at index InputIndex if valid, nullptr otherwise.
	 * Note: Implementations are required to return nullptr for invalid input indices.
	 */
	ENGINE_API virtual FExpressionInput* GetInput(int32 InputIndex);
	const FExpressionInput* GetInput(int32 InputIndex) const { return const_cast<UMaterialExpression*>(this)->GetInput(InputIndex); }

	UE_DEPRECATED(5.5, "Use FExpressionInputIterator instead or GetInput() directly.")
	ENGINE_API virtual TArrayView<FExpressionInput*> GetInputsView();

	ENGINE_API virtual FName GetInputName(int32 InputIndex) const;
	
	ENGINE_API virtual bool IsInputConnectionRequired(int32 InputIndex) const;
		
	UE_DEPRECATED(5.6, "Use EMaterialValueType GetInputValueType(...) instead")
	ENGINE_API virtual uint32 GetInputType(int32 InputIndex);

	ENGINE_API virtual TArray<FExpressionOutput>& GetOutputs();

	UE_DEPRECATED(5.6, "Use EMaterialValueType GetOutputValueType(...) instead")
	ENGINE_API virtual uint32 GetOutputType(int32 OutputIndex);
	
	virtual EMaterialValueType GetInputValueType(int32 InputIndex)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return static_cast<EMaterialValueType>(GetInputType(InputIndex));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return static_cast<EMaterialValueType>(GetOutputType(OutputIndex));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual bool IsStaticExpression() const { return false; }
	virtual bool CanUserDeleteExpression() const { return true; }

	/** Find the property that is associated with the input pin. */
	ENGINE_API virtual TArray<FProperty*> GetInputPinProperty(int32 PinIndex);
	ENGINE_API virtual FName GetInputPinSubCategory(int32 PinIndex);
	ENGINE_API virtual UObject* GetInputPinSubCategoryObject(int32 PinIndex);
	ENGINE_API virtual void PinDefaultValueChanged(int32 PinIndex, const FString& DefaultValue);
	ENGINE_API virtual void ForcePropertyValueChanged(FProperty* Property, bool bUpdatePreview = true);
	ENGINE_API virtual void RefreshNode(bool bUpdatePreview = true);
	ENGINE_API virtual FString GetInputPinDefaultValue(int32 PinIndex);
	ENGINE_API virtual TArray<FProperty*> GetPropertyInputs() const;

	ENGINE_API virtual FText GetCreationDescription() const;
	ENGINE_API virtual FText GetCreationName() const;

	/**
	 *	Get the width required by this expression (in the material editor).
	 *
	 *	@return	int32			The width in pixels.
	 */
	ENGINE_API virtual int32 GetWidth() const;
	ENGINE_API virtual int32 GetHeight() const;
	ENGINE_API virtual bool UsesLeftGutter() const;
	ENGINE_API virtual bool UsesRightGutter() const;

	/**
	 *	Returns the text to display on the material expression (in the material editor).
	 *
	 *	@return	FString		The text to display.
	 */
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const;
	/** Get a single line description of the material expression (used for lists) */
	ENGINE_API virtual FString GetDescription() const;
	/** Get a tooltip for the specified connector. */
	ENGINE_API virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip);

	/** Get a tooltip for the expression itself. */
	ENGINE_API virtual void GetExpressionToolTip(TArray<FString>& OutToolTip);
	/**
	 *	Returns the amount of padding to use for the label.
	 *
	 *	@return int32			The padding (in pixels).
	 */
	virtual int GetLabelPadding() { return 0; }
	ENGINE_API virtual int32 CompilerError(class FMaterialCompiler* Compiler, const TCHAR* pcMessage);
#endif // WITH_EDITOR

	/**
	 * @return whether the expression preview needs realtime update
	 */
#if WITH_EDITOR
	virtual bool NeedsRealtimePreview() { return false; }

	/**
	 * @return text overlaid over the preview in the material editor
	 */
	virtual FText GetPreviewOverlayText() const { return FText(); }

	/**
	 * MatchesSearchQuery: Check this expression to see if it matches the search query
	 * @param SearchQuery - User's search query (never blank)
	 * @return true if the expression matches the search query
	 */
	ENGINE_API virtual bool MatchesSearchQuery( const TCHAR* SearchQuery );

	/**
	 * MatchesSearchQuery: Check this expression and, if applicable, it's subobjects,to see if it matches the search query.
	 * Also provides an FFindInMaterialResult object to return additional data for the result. 
	 * @param SearchQuery - User's search query (never blank)
	 * @param SearchResult - Result data
	 * @return true if the expression matches the search query
	 */
	ENGINE_API virtual bool DeepSearchQuery(const TCHAR* SearchQuery, TSharedPtr<FFindInMaterialResult> SearchResult);

	/**
	 * Copy the SrcExpressions into the specified material, preserving internal references.
	 * New material expressions are created within the specified material.
	 */
	static ENGINE_API void CopyMaterialExpressions(const TArray<class UMaterialExpression*>& SrcExpressions, const TArray<class UMaterialExpressionComment*>& SrcExpressionComments, 
		class UMaterial* Material, class UMaterialFunction* Function, TArray<class UMaterialExpression*>& OutNewExpressions, TArray<class UMaterialExpression*>& OutNewComments);

	/**
	 * Marks certain expression types as outputting material attributes. Allows the material editor preview material to know if it should use its material attributes pin.
	 */
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) { return false; }

	/**
	 * Marks certain expression types as outputting Substrate material. Allows the material functions to directly return a Substrate material as output pin.
	 */
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) { return false; }

	/**
	 * Returns specific EMaterialValueType when the result is MaterialAttributes or Substrate Material, MCT_Unknown otherwise
	 */
	EMaterialValueType GetCompositeOutput(int32 OutputIndex);

	/**
	 * Recursively parse nodes outputing Substrate material in order to gather all the possible shading models used in a material graph output a Substrate material.
	 */
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) { }

	UE_DEPRECATED(5.8, "Please use SubstrateGenerateMaterialTopologyTree taking the FSubstrateTranslatorDataInterface parameter.")
	ENGINE_API virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(class FMaterialCompiler* Compiler, class UMaterialExpression* Parent, int32 OutputIndex);

	/**
	 * A Substrate material is a tree with FrontMaterial being its root and BSDF being leaves, with operators in the middle.
	 * This recursively parse nodes outputing Substrate material in order to gather the maximum distance to any leaves.
	 * This is used to drive the bottom up order processing of those nodes. 
	 */
	ENGINE_API virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex);

	/**
	 * If true, discards the output index when caching this expression which allows more cases to re-use the output instead of adding a separate instruction
	 */
	virtual bool CanIgnoreOutputIndex() { return false; }

	/**
	 * Connects the specified output to the passed material for previewing. 
	 */
	ENGINE_API void ConnectToPreviewMaterial(UMaterial* InMaterial, int32 OutputIndex);
#endif // WITH_EDITOR

#if WITH_EDITOR
	/**
	 * Check if input exppresion is directly connected to the material.
	 */
	ENGINE_API virtual bool IsExpressionConnected(FExpressionInput* Input, int32 OutputIndex);

	/**
	 * Connects the specified input expression to the specified output of this expression.
	 */
	ENGINE_API virtual void ConnectExpression(FExpressionInput* Input, int32 OutputIndex);
#endif // WITH_EDITOR

	/** 
	* Generates a GUID for the parameter expression if one doesn't already exist and we are one.
	*
	* @param bForceGeneration	Whether we should generate a GUID even if it is already valid.
	*/
	ENGINE_API void UpdateParameterGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty);

	/** Callback to access derived classes' parameter expression id. */
	virtual FGuid& GetParameterExpressionId()
	{
		checkf(!bIsParameterExpression, TEXT("Expressions with bIsParameterExpression==true must implement their own GetParameterExpressionId!"));
		static FGuid Dummy;
		return Dummy;
	}

	/**
	* Generates a GUID for this expression if one doesn't already exist.
	*
	* @param bForceGeneration	Whether we should generate a GUID even if it is already valid.
	*/
	ENGINE_API void UpdateMaterialExpressionGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty);
	
	/** Return the material expression guid. */
	virtual FGuid& GetMaterialExpressionId()
	{		
#if WITH_EDITORONLY_DATA
		return MaterialExpressionGuid;
#else
		static FGuid Dummy;
		return Dummy;
#endif
	}

	/** Asserts if the expression is not contained by its Material or Function's expressions array. */
	ENGINE_API void ValidateState();

#if WITH_EDITOR

	/** Returns whether or not this expression should show the preview widget */
	ENGINE_API bool ShouldShowPreview() const;

	/** Returns the keywords that should be used when searching for this expression */
	virtual FText GetKeywords() const {return FText::GetEmpty();}

	/**
	 * Recursively gets a list of all expressions that are connected to this
	 * Checks for repeats so that it can't end up in an infinite loop
	 *
	 * @param InputExpressions Array to contain/pass on expressions
	 *
	 * @return Whether a repeat was found while getting expressions
	 */
	ENGINE_API bool GetAllInputExpressions(TArray<UMaterialExpression*>& InputExpressions);

	/**
	 * Can this node be renamed?
	 */
	ENGINE_API virtual bool CanRenameNode() const;

	/**
	 * Returns the current 'name' of the node (typically a parameter name).
	 * Only valid to call on a node that previously returned CanRenameNode() = true.
	 */
	ENGINE_API virtual FString GetEditableName() const;

	/**
	 * Sets the current 'name' of the node (typically a parameter name)
	 * Only valid to call on a node that previously returned CanRenameNode() = true.
	 */
	ENGINE_API virtual void SetEditableName(const FString& NewName);

	/** 
	* Parameter Name functions, this is requires as multiple class have ParameterName
	* but are not UMaterialExpressionParameter due to class hierarchy. */
	virtual bool HasAParameterName() const { return false; }
	ENGINE_API virtual void ValidateParameterName(const bool bAllowDuplicateName = true);
	ENGINE_API virtual bool HasClassAndNameCollision(UMaterialExpression* OtherExpression) const;

	ENGINE_API EMaterialParameterType GetParameterType() const;

	virtual FName GetParameterName() const { return NAME_None; }
	virtual void SetParameterName(const FName& Name) {}
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const { return false; }
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None) { return false; }

	virtual void GetLandscapeLayerNames(TArray<FName>& OutLayers) const {}
	virtual void GetIncludeFilePaths(TSet<FString>& OutIncludeFilePaths) const {}

	/**
	 * Called after a node copy, once the Material and Function properties are set correctly and that all new expressions are added to Material->Expressions
	 * @param	CopiedExpressions	The expressions copied in this copy
	 */
	virtual void PostCopyNode(const TArray<UMaterialExpression*>& CopiedExpressions) {}

	ENGINE_API virtual bool HasConnectedOutputs() const;

	/** Checks whether any inputs to this expression create a loop */
	ENGINE_API bool ContainsInputLoop(const bool bStopOnFunctionCall = true);

	/** This overload accepts the set of visited expressions to avoid visiting them again when checking at once whether multiple expressions contain a loop. */
	ENGINE_API bool ContainsInputLoop(TSet<UMaterialExpression*>& VisitedExpressions, const bool bStopOnFunctionCall = true);

	/** 
	 * Checks if a pin of specified PinDirection and PinIndex can be deleted 
	 * @param PinDirection direction of the pin
	 * @param PinIndex index of the pin 
	 * @return returns true if the pin can be deleted
	 */
	virtual bool CanDeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) const { return false; }

	/**
	 * Deletes the pin of specified PinDirection and PinIndex. Should only be used if CanDeletePin returns true for that pin
	 * @param PinDirection direction of the pin
	 * @param PinIndex index of the pin
	 */
	virtual void DeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) {}

	/**
	 * Allows Material Expressions to register additional actions in the context menu
	 * @param ActionMenuBuilder action menu builder to add actions to
	 * @param CategoryName category name for actions
	 */
	virtual void RegisterAdditionalMenuActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FText& CategoryName) {}

protected:
	/** Caches the list of fixed expression inputs this expression has. */
	TArray<FExpressionInput*> CachedInputs;

private:
	/**
	 * Helper struct to represent nodes the trail of nodes we're coming from when visiting a new
	 * expression input node.
	 */
	struct FContainsInputLoopInternalExpressionStack;

	/**
	 * Checks whether any inputs to this expression create a loop by recursively
	 * calling itself and keeping a list of inputs as expression keys.
	 *
	 * @param ExpressionStack    Stack of expressions that have been checked already
	 * @param VisitedExpressions List of all expression keys that have been visited
	 */
	ENGINE_API bool ContainsInputLoopInternal(const FContainsInputLoopInternalExpressionStack& ExpressionStack, TSet<UMaterialExpression*>& VisitedExpressions, const bool bStopOnFunctionCall);

#endif // WITH_EDITOR
};

/** Specifies what reference point should be used */
UENUM()
enum class EPositionOrigin : uint8
{
	/** Absolute world position, i.e. relative to (0,0,0) */
	Absolute UMETA(DisplayName="Absolute World Position"),

	/** Camera relative world position, i.e. translated world space */
	CameraRelative UMETA(DisplayName="Camera Relative World Position")
};


#if WITH_EDITOR

/**
 * @brief An iterator for traversing the inputs of a UMaterialExpression.
 *
 * This struct provides a way to iterate over the inputs of a given
 * UMaterialExpression. It starts from the first input and advances
 * through subsequent inputs until all are iterated through.
 *
 * Usage:
 *
 * @code
 * UMaterialExpression* MyExpression = ...;
 * for (FExpressionInputIterator It{ MyExpression }; It; ++It)
 * {
 *     // Process the input, e.g. It->Input->IsConnected();
 * }
 * @endcode
 */
struct FExpressionInputIterator
{
	/** The expression whose inputs to iterate through. */
    UMaterialExpression* Expression;

	/** Current input. */
    FExpressionInput* Input;

	/** Current input index. */
	int Index;

    FExpressionInputIterator(UMaterialExpression* InExpression)
    : Expression{ InExpression }
    , Input{ InExpression->GetInput(0) }
    , Index{ 0 }
    {
    }

    operator bool() const
    {
        return Input != nullptr;
    }

    FExpressionInputIterator& operator++()
    {
		Index += 1;
		Input = Expression->GetInput(Index);
        return *this;
    }

    FExpressionInput* operator->()
    {
        return Input;
    }
};

/**
* @brief An iterator for traversing the outputs of a UMaterialExpression.
*
* This struct provides a way to iterate over the outputs of a given
* UMaterialExpression. It starts from the first output and advances
* through subsequent outputs until all are iterated through.
*
* Usage:
*
* @code
* UMaterialExpression* MyExpression = ...;
* for (FExpressionOutputIterator It{ MyExpression }; It; ++It)
* {
*     // Process the output, e.g. It->Output->IsConnected();
* }
* @endcode
*/
struct FExpressionOutputIterator
{
	/** The expression whose outputs to iterate through. */
    UMaterialExpression* Expression;

	/** Current output. */
    FExpressionOutput* Output;

	/** Current output index. */
	int Index;

    FExpressionOutputIterator(UMaterialExpression* InExpression)
    : Expression{ InExpression }
    , Output{ InExpression->GetOutput(0) }
    , Index{ 0 }
    {
    }

    operator bool() const
    {
        return Output != nullptr;
    }

    FExpressionOutputIterator& operator++()
    {
		Index += 1;
		Output = Expression->GetOutput(Index);
        return *this;
    }

    FExpressionOutput* operator->()
    {
        return Output;
    }
};

#endif // WITH_EDITOR
