// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Absolute value material expression for user-defined materials
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionAbs.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionAbs : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Link to the input expression to be evaluated */
	UPROPERTY()
	FExpressionInput Input;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface

};



