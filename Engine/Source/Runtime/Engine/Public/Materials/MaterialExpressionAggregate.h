// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialAggregate.h"
#include "MaterialExpressionAggregate.generated.h"

UENUM()
enum class EMaterialExpressionMakeAggregateKind
{
	MaterialAttributes,
	UserDefined
};

USTRUCT(MinimalAPI)
struct FMaterialExpressionAggregateEntry
{
	GENERATED_BODY()

	UPROPERTY()
	int32 AttributeIndex{};

	UPROPERTY()
	FExpressionInput Input;
};

UCLASS(MinimalAPI, meta=(NewMaterialTranslator))
class UMaterialExpressionAggregate : public UMaterialExpression
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=MaterialAggregate)
	EMaterialExpressionMakeAggregateKind Kind = EMaterialExpressionMakeAggregateKind::MaterialAttributes;

	UPROPERTY(EditAnywhere, Category=MaterialAggregate, Meta=(EditCondition="Kind == EMaterialExpressionMakeAggregateKind::UserDefined"))
	TObjectPtr<UMaterialAggregate> UserAggregate;

	UPROPERTY(EditAnywhere, Category=MaterialAggregate, Meta=(DisplayName="Attributes", GetOptions="GetPossibleAttributeNames"))
	TArray<FName> AttributeNames;
	
	UPROPERTY()
	FExpressionInput PrototypeInput;

	UPROPERTY()
	TArray<FMaterialExpressionAggregateEntry> Entries;

	UMaterialExpressionAggregate(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	// Returns the aggregate currently bound to this expression.
	const UMaterialAggregate* GetAggregate() const;

	// Returns the attribute with given index of the aggregate currently bound to this expression.
	const FMaterialAggregateAttribute* GetAggregateAttribute(int32 Index) const;
	
	// Utility function to list the attribute names in the attribute selection combobox.
	UFUNCTION()
	TArray<FString> GetPossibleAttributeNames() const;

	//~ Begin UMaterialExpression Interface
	virtual void 				GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FExpressionInput* 	GetInput(int32 InputIndex) override;
	virtual FName 				GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType 	GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType 	GetOutputValueType(int32 OutputIndex) override;
	virtual bool				IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual void 				PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void 				PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void 				Build(MIR::FEmitter& Emitter) override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	// Used to store the state of the expression before a change was made.
	struct FPrevEditData
	{
		const UMaterialAggregate* Aggregate{};
		TArray<FName> AttributeNames{};
		TArray<FMaterialExpressionAggregateEntry> Entries{};
	};

	FPrevEditData PreEditData;
#endif // WITH_EDITOR
};
