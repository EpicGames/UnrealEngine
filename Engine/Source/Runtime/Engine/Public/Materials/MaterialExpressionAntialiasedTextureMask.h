// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "MaterialExpressionAntialiasedTextureMask.generated.h"

class UTexture;

UENUM()
enum ETextureColorChannel : int
{
	TCC_Red,
	TCC_Green,
	TCC_Blue,
	TCC_Alpha,
	TCC_MAX,
};

UCLASS(MinimalAPI)
class UMaterialExpressionAntialiasedTextureMask : public UMaterialExpressionTextureSampleParameter2D
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionAntialiasedTextureMask, meta=(UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Threshold = 0.5f;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionAntialiasedTextureMask)
	TEnumAsByte<enum ETextureColorChannel> Channel = TCC_Alpha;

#if WITH_EDITOR
	//~ Begin UMaterialExpressionTextureSampleParameter Interface
	virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage) override;
	virtual void SetDefaultTexture() override;
	//~ End UMaterialExpressionTextureSampleParameter Interface

	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};



