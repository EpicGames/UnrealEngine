// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionAtmosphericFogColor.generated.h"

UCLASS(collapsecategories, hidecategories=Object, DisplayName = "Atmospheric Fog Color (deprecated)")
class UMaterialExpressionAtmosphericFogColor : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionAtmosphericFogColor)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



