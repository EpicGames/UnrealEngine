// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionAtmosphericLightVector.generated.h"

UCLASS(collapsecategories, hidecategories = Object, DisplayName = "Atmosphere Sun Light Vector")
class UMaterialExpressionAtmosphericLightVector : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



