// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionBentNormalCustomOutput.generated.h"

UCLASS()
class UMaterialExpressionBentNormalCustomOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Float3; }
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
#endif
	virtual int32 GetNumOutputs() const override { return 1; }
	virtual FString GetFunctionName() const override { return TEXT("GetBentNormal"); }
	virtual FString GetDisplayName() const override { return TEXT("BentNormal"); }
};



