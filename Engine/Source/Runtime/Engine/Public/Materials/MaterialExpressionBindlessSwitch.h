// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"
#include "MaterialExpressionBindlessSwitch.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionBindlessSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when rendering without bindless support"))
	FExpressionInput Default;

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when rendering with bindless support"))
	FExpressionInput Bindless;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};
