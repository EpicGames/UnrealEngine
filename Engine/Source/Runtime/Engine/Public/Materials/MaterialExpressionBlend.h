// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionBlend.generated.h"

UENUM()
enum class EMaterialExpressionBlendMode : uint8
{
	Blend,
	UseA,
	UseB,
};

UCLASS(MinimalAPI, CollapseCategories, HideCategories = Object, meta=(NewMaterialTranslator))
class UMaterialExpressionBlend : public UMaterialExpression
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY()
	FExpressionInput Alpha;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionBlend)
	EMaterialExpressionBlendMode PixelAttributesBlendMode;
	
	UPROPERTY(EditAnywhere, Category=MaterialExpressionBlend)
	EMaterialExpressionBlendMode VertexAttributesBlendMode;

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void 				GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType 	GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType 	GetOutputValueType(int32 OutputIndex) override;
	virtual void 				Build(MIR::FEmitter& Emitter) override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};
