// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionBlendMaterialAttributes.generated.h"

UENUM()
namespace EMaterialAttributeBlend
{
	enum Type : int
	{
		Blend,
		UseA,
		UseB
	};
}

UENUM()
namespace EMaterialAttributeBlendFunction
{
	enum Type : int
	{
		Horizontal,
		Vertical
	};
}

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionBlendMaterialAttributes : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
 	FMaterialAttributesInput A;

	UPROPERTY()
 	FMaterialAttributesInput B;

	UPROPERTY()
	FExpressionInput Alpha;

	// Optionally skip blending attributes of this type.
	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	TEnumAsByte<EMaterialAttributeBlend::Type> PixelAttributeBlendType = EMaterialAttributeBlend::Blend;

	// Optionally skip blending attributes of this type.
	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	TEnumAsByte<EMaterialAttributeBlend::Type> VertexAttributeBlendType = EMaterialAttributeBlend::Blend;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return true;}
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override {return true;}
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return InputIndex == 2 ? MCT_Float1 : MCT_MaterialAttributes;}
#endif
	//~ End UMaterialExpression Interface
};


/** Deprecated blend expression used by substrate material layers prototype. */
UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta = (Private))
class UMaterialExpressionLegacyBlendMaterialAttributes : public UMaterialExpressionBlendMaterialAttributes
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	FExpressionInput VertexAttribute_UseA;

	UPROPERTY()
	FExpressionInput VertexAttribute_UseB;

	UPROPERTY()
	FExpressionInput PixelAttribute_UseA;

	UPROPERTY()
	FExpressionInput PixelAttribute_UseB;

	UPROPERTY(EditAnywhere, Category = MaterialAttributes)
	TEnumAsByte<EMaterialAttributeBlendFunction::Type> BlendFunctionType;
};
