// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionBounds.generated.h"

UENUM()
enum EMaterialExpressionBoundsType : uint8
{
	/** Instance local */
	MEILB_InstanceLocal UMETA(DisplayName = "Instance Local"),

	/** Object local */
	MEILB_ObjectLocal UMETA(DisplayName = "Object Local"),

	/** Skinned local */
	MEILB_PreSkinnedLocal UMETA(DisplayName = "Pre-Skinned Local"),
};

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionBounds : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	enum OutputIndex
	{
		BoundsHalfExtentOutputIndex,
		BoundsExtentOutputIndex,
		BoundsMinOutputIndex,
		BoundsMaxOutputIndex,
	};

	/** The type of bounds to output */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionBounds, meta = (DisplayName = "Type", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EMaterialExpressionBoundsType> Type = MEILB_InstanceLocal;

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

	//~ End UMaterialExpression Interface

#if WITH_EDITORONLY_DATA
	TArray<FString> OutputToolTips;
#endif
#endif
};