// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionBreakMaterialAttributes.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionBreakMaterialAttributes : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

 	UPROPERTY()
 	FMaterialAttributesInput MaterialAttributes;

	//~ Begin UObject Interface
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface
 
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override { return InputIndex == 0 ? &MaterialAttributes : nullptr; }
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_MaterialAttributes;}
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;

	static TMap<EMaterialProperty, int32> PropertyToIOIndexMap;
	static void BuildPropertyToIOIndexMap();

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



