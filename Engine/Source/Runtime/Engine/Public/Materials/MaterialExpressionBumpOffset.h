// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionBumpOffset.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionBumpOffset : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	// Outputs: Coordinate + Eye.xy * (Height - ReferencePlane) * HeightRatio
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinate;

	UPROPERTY()
	FExpressionInput Height;

	UPROPERTY(meta=(RequiredInput = "false"))
	FExpressionInput HeightRatioInput;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionBumpOffset, meta=(OverridingInputProperty = "HeightRatioInput"))
	float HeightRatio = 0.05f; // Perceived height as a fraction of width.

	UPROPERTY(EditAnywhere, Category=MaterialExpressionBumpOffset)
	float ReferencePlane = 0.5f;    // Height at which no offset is applied.

	/** only used if Coordinate is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionBumpOffset, meta = (OverridingInputProperty = "Coordinate"))
	uint32 ConstCoordinate = 0;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



