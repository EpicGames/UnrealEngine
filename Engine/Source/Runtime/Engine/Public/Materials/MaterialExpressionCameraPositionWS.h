// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionCameraPositionWS.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionCameraPositionWS : public UMaterialExpression
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



