// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionChannelMaskParameterColor.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "MaterialExpressionChannelMaskParameter.generated.h"

UCLASS(collapsecategories, hidecategories=(Object, MaterialExpressionVectorParameter), MinimalAPI)
class UMaterialExpressionChannelMaskParameter : public UMaterialExpressionVectorParameter
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionChannelMaskParameter)
	TEnumAsByte<EChannelMaskParameterColor::Type> MaskChannel = EChannelMaskParameterColor::Red;

	UPROPERTY()
	FExpressionInput Input;

#if WITH_EDITOR
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		if (Super::GetParameterValue(OutMeta))
		{
			OutMeta.bUsedAsChannelMask = true;
			return true;
		}
		return false;
	}

	virtual bool SetParameterValue(FName InParameterName, FLinearColor InValue, EMaterialExpressionSetParameterValueFlags Flags) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return true;}
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_Float4;}

#endif

	virtual bool IsUsedAsChannelMask() const override {return true;}
};
