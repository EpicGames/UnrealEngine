// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionClamp.generated.h"

UENUM()
enum EClampMode : int
{
	CMODE_Clamp,
	CMODE_ClampMin,
	CMODE_ClampMax,
};

UCLASS(MinimalAPI)
class UMaterialExpressionClamp : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MinDefault' if not specified"))
	FExpressionInput Min;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'MaxDefault' if not specified"))
	FExpressionInput Max;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionClamp, Meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EClampMode> ClampMode = CMODE_Clamp;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionClamp, meta=(OverridingInputProperty = "Min"))
	float MinDefault = 0.0f;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionClamp, meta=(OverridingInputProperty = "Max"))
	float MaxDefault = 1.0f;


	//~ Begin UObject Interface
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



