// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Shader/ShaderTypes.h"
#include "MaterialExpressionClearCoatNormalCustomOutput.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionClearCoatNormalCustomOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Float3; }
#endif


	virtual FString GetFunctionName() const override { return TEXT("ClearCoatBottomNormal"); }

};



