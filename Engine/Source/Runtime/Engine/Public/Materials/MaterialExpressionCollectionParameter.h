// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialExpressionCollectionParameter.h - a node that references a single parameter in a MaterialParameterCollection
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionCollectionParameter.generated.h"

struct FPropertyChangedEvent;

UCLASS(hidecategories=object, MinimalAPI)
class UMaterialExpressionCollectionParameter : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** The Parameter Collection to use. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionParameter)
	TObjectPtr<class UMaterialParameterCollection> Collection;

	/** Name of the parameter being referenced. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list. The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionParameter)
	int32 SortPriority = 32;

	/** Id that is set from the name, and used to handle renaming of collection parameters. */
	UPROPERTY()
	FGuid ParameterId;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	virtual bool HasAParameterName() const override { return true; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }

	ENGINE_API virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const;
	ENGINE_API virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);

	virtual void ValidateParameterName(const bool bAllowDuplicateName) override {};

	virtual bool MatchesSearchQuery( const TCHAR* SearchQuery ) override;

#endif
	//~ End UMaterialExpression Interface
};



