// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialExpressionCollectionTransform.h - a node that references a transform (5 consecutive vector elements) in a MaterialParameterCollection
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionCollectionTransform.generated.h"

struct FPropertyChangedEvent;

UENUM()
enum class EParameterCollectionTransformType : uint8
{
	Position UMETA(ToolTip = "Float 4x4 matrix (includes translation, 4 input elements, output float3 or float4 depending on input)"),
	Vector UMETA(ToolTip = "Float 3x3 matrix (rotation only, 3 input elements, output float3)"),
	Projection UMETA(ToolTip = "Float 4x4 projection matrix (optimized assuming only m11 m22 m33 m34 m43 m44 non-zero, 4 input elements, output float4)"),
	LocalToWorld UMETA(ToolTip = "LWC matrix (float 4x4, post tile offset, 5 vector elements, output LWC float3)"),
	WorldToLocal UMETA(ToolTip = "LWC inverse matrix (float 4x4, pre tile offset, 5 vector elements, output float3 or float4 depending on input)")
};

// Node that uses 3 to 5 consecutive vector elements of a Material Parameter Collection as a Transform matrix
UCLASS(hidecategories = object, MinimalAPI)
class UMaterialExpressionCollectionTransform : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FExpressionInput Input;

	/** The Parameter Collection to use. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionTransform)
	TObjectPtr<class UMaterialParameterCollection> Collection;

	/** Name of the parameter being referenced. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionTransform)
	FName ParameterName;

	/** Id that is set from the name, and used to handle renaming of collection parameters. */
	UPROPERTY()
	FGuid ParameterId;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCollectionTransform)
	EParameterCollectionTransformType TransformType;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	virtual bool HasAParameterName() const override { return true; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override {};

	virtual bool MatchesSearchQuery( const TCHAR* SearchQuery ) override;
#endif
	//~ End UMaterialExpression Interface
};
