// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Curves/CurveLinearColor.h"
#include "MaterialExpressionColorRamp.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionColorRamp : public UMaterialExpression
{
	GENERATED_BODY()

public:
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstInput' if not specified"))
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionColorRamp", meta = (OverridingInputProperty = "Input"))
	float ConstInput = 0.5f;

	UPROPERTY(EditAnywhere, Instanced, Category = "MaterialExpressionColorRamp")
	TObjectPtr<UCurveLinearColor> ColorCurve;

	UMaterialExpressionColorRamp(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void HandleCurvePropertyChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};