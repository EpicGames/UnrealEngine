// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionComment.generated.h"

struct FPropertyChangedEvent;

UCLASS(MinimalAPI)
class UMaterialExpressionComment : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	int32 SizeX;

	UPROPERTY()
	int32 SizeY;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment, meta=(MultiLine=true))
	FString Text;

	/** Color to style comment with */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment)
	FLinearColor CommentColor = FLinearColor::White;

	/** Size of the text in the comment box */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment, meta=(ClampMin=1, ClampMax=1000))
	int32 FontSize = 18;

	/** Whether to show a zoom-invariant comment bubble when zoomed out (making the comment readable at any distance). */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment, meta=(DisplayName="Show Bubble When Zoomed"))
	uint32 bCommentBubbleVisible_InDetailsPanel:1 = false;

	/** Whether to use Comment Color to color the background of the comment bubble shown when zoomed out. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment, meta=(DisplayName="Color Bubble", EditCondition=bCommentBubbleVisible_InDetailsPanel))
	uint32 bColorCommentBubble:1 = false;

	/** Whether the comment should move any fully enclosed nodes around when it is moved */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionComment)
	bool bGroupMode = true;
	
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool MatchesSearchQuery( const TCHAR* SearchQuery ) override;
#endif // WITH_EDITOR
	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const override { return false; }
	//~ End UMaterialExpression Interface
};



