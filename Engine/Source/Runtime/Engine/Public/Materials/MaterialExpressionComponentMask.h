// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionComponentMask.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionComponentMask : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionComponentMask, Meta = (ShowAsInputPin = "Advanced"))
	uint32 R:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionComponentMask, Meta = (ShowAsInputPin = "Advanced"))
	uint32 G:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionComponentMask, Meta = (ShowAsInputPin = "Advanced"))
	uint32 B:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionComponentMask, Meta = (ShowAsInputPin = "Advanced"))
	uint32 A:1;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("component mask")); }
#endif
	//~ End UMaterialExpression Interface
};



