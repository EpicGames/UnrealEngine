// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionReroute.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "MaterialExpressionComposite.generated.h"

/**
 * Composite (subgraph) expression. Exists purely for organzational purposes.
 * Under the hood uses reroute expressions for graph compilation. See below
 * to understand how a particular reroute's input / output correlates to
 * the inputs / outputs of composites and their pin bases.
 * 
 *      _________________          _________________
 *     |   INPUT BASE    |        |   OUTPUT BASE   |
 *     +--------+--------+        +--------+--------+
 *     |        |   (>)  |   ->   |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |        |        |        |        |
 *     +--------+--------+        +--------+--------+
 *     | NODE IN:  NONE  |        | NODE IN:  PINS  |
 *     | NODE OUT: PINS  |        | NODE OUT: NONE  |
 *     |_________________|        |_________________|
 *   
 *                   ^                |                       
 *                   |                v                                        
 *              ____________________________         
 *             |         COMPOSITE          |
 *             +---------+--------+---------+
 *         ->  |    (>)  |        |  (>)    |  ->
 *             |    (>)  |        |  (>)    |
 *             |    (>)  |        |  (>)    |
 *             |    (>)  |        |  (>)    |
 *             |         |        |         |
 *             +---------+--------+---------+
 *             | NODE IN:  TO INPUT BASE    |
 *             | NODE OUT: FROM OUTPUT BASE |
 *             |____________________________|
 *
 */

UCLASS(MinimalAPI)
class UMaterialExpressionComposite : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(EditAnywhere, Category= MaterialExpressionComposite)
	FString SubgraphName;

	UPROPERTY()
	TObjectPtr<UMaterialExpressionPinBase> InputExpressions;

	UPROPERTY()
	TObjectPtr<UMaterialExpressionPinBase> OutputExpressions;

#if WITH_EDITOR
	/** Get all reroute expressions used by this composite & its pin bases */
	virtual TArray<UMaterialExpressionReroute*> GetCurrentReroutes() const;

	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
	virtual bool CanRenameNode() const override { return true; }
	virtual FString GetEditableName() const;
	virtual void SetEditableName(const FString& NewName);

	virtual TArray<FExpressionOutput>& GetOutputs() override;
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsExpressionConnected(FExpressionInput* Input, int32 OutputIndex) override;
	virtual void ConnectExpression(FExpressionInput* Input, int32 OutputIndex) override;

	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface

	//~ Begin UObject Interface.
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	//~ End UObject Interface.

	bool ModifyCompositeOnly(bool bAlwaysMarkDirty = true);
#endif // WITH_EDITOR
};



