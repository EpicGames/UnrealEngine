// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionConstant.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionConstant : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionConstant, DisplayName="Value", Meta=(ShowAsInputPin = "Primary"))
	float R;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FString GetDescription() const override;
#endif // WITH_EDITOR

	//~ End UMaterialExpression Interface
};



