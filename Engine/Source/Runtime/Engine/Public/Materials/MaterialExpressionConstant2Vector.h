// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionConstant2Vector.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionConstant2Vector : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionConstant2Vector, DisplayName = "X", Meta = (ShowAsInputPin = "Primary"))
	float R;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionConstant2Vector, DisplayName = "Y", Meta = (ShowAsInputPin = "Primary"))
	float G;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FString GetDescription() const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return MCT_Float2;}
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



