// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionConstant3Vector.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionConstant3Vector
	: public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionConstant3Vector, meta=(HideAlphaChannel, ShowAsInputPin = "Primary"))
	FLinearColor Constant;

public:

	//~ UMaterialExpression interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FString GetDescription() const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return MCT_Float3;}
#endif // WITH_EDITOR
};
