// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "MaterialExpressionConstant4Vector.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionConstant4Vector : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionConstant4Vector, Meta = (ShowAsInputPin = "Primary"))
	FLinearColor Constant;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FString GetDescription() const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return MCT_Float4;}
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



