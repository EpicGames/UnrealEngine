// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"

#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"

#include "MaterialExpressionConvert.generated.h"

class UEdGraphPin;

#define LOCTEXT_NAMESPACE "MaterialExpressionConvert"

UENUM()
enum class EMaterialExpressionConvertType : uint8
{
	Scalar,
	Vector2,
	Vector3,
	Vector4,
};

namespace MaterialExpressionConvertType
{
	static inline int32 GetComponentCount(const EMaterialExpressionConvertType InConvertType)
	{
		switch (InConvertType)
		{
			case EMaterialExpressionConvertType::Scalar: return 1;
			case EMaterialExpressionConvertType::Vector2: return 2;
			case EMaterialExpressionConvertType::Vector3: return 3;
			case EMaterialExpressionConvertType::Vector4: return 4;
			default: checkNoEntry(); return INDEX_NONE;
		}
	}

	static inline EMaterialValueType ToMaterialValueType(const EMaterialExpressionConvertType InConvertType)
	{
		switch (InConvertType)
		{
			case EMaterialExpressionConvertType::Scalar: return MCT_Float1;
			case EMaterialExpressionConvertType::Vector2: return MCT_Float2;
			case EMaterialExpressionConvertType::Vector3: return MCT_Float3;
			case EMaterialExpressionConvertType::Vector4: return MCT_Float4;
			default: checkNoEntry(); return MCT_Float;
		}
	}

	static inline FText ToText(const EMaterialExpressionConvertType InConvertType)
	{
		switch (InConvertType)
		{
			case EMaterialExpressionConvertType::Scalar:  return LOCTEXT("Scalar", "Scalar");
			case EMaterialExpressionConvertType::Vector2: return LOCTEXT("Vector2", "Vector2");
			case EMaterialExpressionConvertType::Vector3: return LOCTEXT("Vector3", "Vector3");
			case EMaterialExpressionConvertType::Vector4: return LOCTEXT("Vector4", "Vector4");
			default: checkNoEntry(); return INVTEXT("");
		}
	}
}

USTRUCT()
struct FMaterialExpressionConvertInput
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert", meta = (RequiredInput = "false"))
	FExpressionInput ExpressionInput;

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	EMaterialExpressionConvertType Type = EMaterialExpressionConvertType::Scalar;

	/** Default Value used when this input has no incoming connection */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	FLinearColor DefaultValue = FLinearColor::Black;
};

USTRUCT()
struct FMaterialExpressionConvertOutput
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	EMaterialExpressionConvertType Type = EMaterialExpressionConvertType::Scalar;

	/** Default Value used when this output has no connection */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	FLinearColor DefaultValue = FLinearColor::Black;
};

USTRUCT()
struct FMaterialExpressionConvertMapping
{
	GENERATED_BODY()

	FMaterialExpressionConvertMapping() = default;
	FMaterialExpressionConvertMapping(int32 InputIndex, int32 InputComponentIndex, int32 OutputIndex, int32 OutputComponentIndex)
	: InputIndex(InputIndex)
	, InputComponentIndex(InputComponentIndex)
	, OutputIndex(OutputIndex)
	, OutputComponentIndex(OutputComponentIndex)
	{}

	/** Which Input to map from */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	int32 InputIndex = 0;

	/** Which Input Component to map from */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	int32 InputComponentIndex = 0;

	/** Which Output to map to */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	int32 OutputIndex = 0;

	/** Which Output Component to map to */
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	int32 OutputComponentIndex = 0;
};

UCLASS(MinimalAPI)
class UMaterialExpressionConvert : public UMaterialExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	TArray<FMaterialExpressionConvertInput> ConvertInputs;

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	TArray<FMaterialExpressionConvertOutput> ConvertOutputs;

	/** Describes how data flows from input components to output components */
	UPROPERTY()
	TArray<FMaterialExpressionConvertMapping> ConvertMappings;

	UPROPERTY(EditAnywhere, Category = "MaterialExpressionConvert")
	FString NodeName = "Convert";

	UMaterialExpressionConvert(const FObjectInitializer& ObjectInitializer);

	//~Begin UObject Interface
	#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	#endif // WITH_EDITOR
	//~End UObject Interface

	//~ Begin UMaterialExpression Interface
	#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;

	virtual int32 CountInputs() const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;

	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual TArray<FExpressionOutput>& GetOutputs() override;

	virtual bool CanDeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) const override;
	virtual void DeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) override;
	virtual TSharedPtr<class SGraphNodeMaterialBase> CreateCustomGraphNodeWidget() override;
	virtual void RegisterAdditionalMenuActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FText& CategoryName) override;
	#endif //WITH_EDITOR
	//~ End UMaterialExpression Interface
private:
	#if WITH_EDITOR
	void RecreateOutputs();
	bool IsValidComponentIndex(int32 InComponentIndex, EMaterialExpressionConvertType InType) const;
	#endif //WITH_EDITOR
};

#undef LOCTEXT_NAMESPACE