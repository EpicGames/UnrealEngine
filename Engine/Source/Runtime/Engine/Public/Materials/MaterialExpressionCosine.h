// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionCosine.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionCosine : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCosine, Meta = (ShowAsInputPin = "Advanced"))
	float Period = 1.0f;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("cos"));}
#endif
	//~ End UMaterialExpression Interface
};



