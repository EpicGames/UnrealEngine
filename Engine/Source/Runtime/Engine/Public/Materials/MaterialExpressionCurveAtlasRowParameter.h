// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "MaterialExpressionCurveAtlasRowParameter.generated.h"

UCLASS(collapsecategories, hidecategories=(Object, MaterialExpressionScalarParameter), MinimalAPI)
class UMaterialExpressionCurveAtlasRowParameter : public UMaterialExpressionScalarParameter
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCurveAtlasRowParameter)
	TObjectPtr<class UCurveLinearColor> Curve;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionCurveAtlasRowParameter)
	TObjectPtr<class UCurveLinearColorAtlas> Atlas;

	UPROPERTY()
	FExpressionInput InputTime;

	virtual bool IsUsedAsAtlasPosition() const override { return true; }
	virtual UObject* GetReferencedTexture() const override;
	virtual bool CanReferenceTexture() const { return true; }

#if WITH_EDITOR
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override;
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override;

	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override 
	{
		return MCT_Float;
	}

	virtual bool IsInputConnectionRequired(int32 InputIndex) const override { return true; }
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetTexturesForceMaterialRecompile(TArray<UTexture *> &Textures) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override 
	{
		OutCaptions.Empty();
		OutCaptions.Add(TEXT(""));
	}
//~ Begin UMaterialExpression Interface
	virtual FName GetInputName(int32 InputIndex) const override
	{
		return TEXT("CurveTime");
	}

#endif
};

