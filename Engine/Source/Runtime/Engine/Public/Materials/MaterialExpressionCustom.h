// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionCustom.generated.h"

struct FPropertyChangedEvent;

UENUM()
enum ECustomMaterialOutputType : int
{
	CMOT_Float1,
	CMOT_Float2,
	CMOT_Float3,
	CMOT_Float4,
	CMOT_MaterialAttributes,
	CMOT_MAX,
};

USTRUCT()
struct FCustomInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=CustomInput)
	FName InputName;

	UPROPERTY()
	FExpressionInput Input;
};

USTRUCT()
struct FCustomOutput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomOutput)
	FName OutputName;

	UPROPERTY(EditAnywhere, Category = CustomOutput)
	TEnumAsByte<enum ECustomMaterialOutputType> OutputType = ECustomMaterialOutputType::CMOT_Float1;
};

USTRUCT()
struct FCustomDefine
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomInput)
	FString DefineName;

	UPROPERTY(EditAnywhere, Category = CustomInput)
	FString DefineValue;
};

UENUM()
enum ECustomMaterialClipInstruction : int
{
	/** Search the Code string for clip() or discard at compile time to determine whether pixel discarding is possible. */
	CMCI_Search	UMETA(DisplayName="Search"),
	/** Assume this expression may discard pixels, regardless of what the Code contains. */
	CMCI_Yes	UMETA(DisplayName="Yes"),
	/** Assume this expression never discards pixels, regardless of what the Code contains. */
	CMCI_No		UMETA(DisplayName="No"),
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionCustom : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom, meta=(MultiLine=true))
	FString Code;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	TEnumAsByte<enum ECustomMaterialOutputType> OutputType = ECustomMaterialOutputType::CMOT_Float1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	FString Description;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	TArray<struct FCustomInput> Inputs;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionCustom)
	TArray<struct FCustomOutput> AdditionalOutputs;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	TArray<struct FCustomDefine> AdditionalDefines;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	TArray<FString> IncludeFilePaths;

	/** Controls whether this expression is considered to contain a clip() or discard instruction that may conditionally discard pixels. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionCustom)
	TEnumAsByte<enum ECustomMaterialClipInstruction> ContainsClipInstruction = ECustomMaterialClipInstruction::CMCI_Search;

	UPROPERTY(VisibleAnywhere, Category=MaterialExpressionCustom)
	bool ShowCode;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API void RebuildOutputs();
#endif // WITH_EDITOR
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface.
	
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("Custom")); }
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_Unknown;}
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual void GetIncludeFilePaths(TSet<FString>& OutIncludeFilePaths) const override;
	//~ End UMaterialExpression Interface

	/** 
	 * Returns true if this expression may conditionally discard pixels using clip() or discard.
	 * The string search is conservative so ContainsClipInstruction lets the material author override if necessary.
	 */
	ENGINE_API bool HasPixelDiscard() const;
#endif // WITH_EDITOR
};



