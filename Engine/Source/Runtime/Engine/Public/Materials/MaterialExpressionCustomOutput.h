// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "RHIDefinitions.h"
#include "Shader/ShaderTypes.h"
#include "MaterialExpressionCustomOutput.generated.h"

UCLASS(abstract,collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionCustomOutput : public UMaterialExpression
{
	GENERATED_BODY()

public:

	// Override to enable multiple outputs
	virtual int32 GetNumOutputs() const { return 1; };
	// Override to limit the maximum number of outputs
	virtual int32 GetMaxOutputs() const { return -1; };
	virtual FString GetFunctionName() const PURE_VIRTUAL(UMaterialExpressionCustomOutput::GetFunctionName, return TEXT("GetCustomOutput"););
	virtual FString GetDisplayName() const { return GetFunctionName(); }

#if WITH_EDITOR
	// Allow custom outputs to generate their own source code
	virtual bool HasCustomSourceOutput() { return false; }
	virtual bool AllowMultipleCustomOutputs() { return false; }
	virtual bool NeedsCustomOutputDefines() { return true; }
	virtual bool ShouldCompileBeforeAttributes() { return false; }
	virtual bool NeedsPreviousFrameEvaluation() { return false; }

	UE_DEPRECATED(5.6, "Use GetShaderFrequency(uint32 OutputIndex) instead")
	virtual EShaderFrequency GetShaderFrequency() { return SF_Pixel; }
	
	virtual EShaderFrequency GetShaderFrequency(uint32 OutputIndex)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetShaderFrequency();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
};



