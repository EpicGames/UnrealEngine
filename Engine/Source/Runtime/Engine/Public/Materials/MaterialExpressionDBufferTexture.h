// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDBufferTexture.generated.h"

UENUM()
enum EDBufferTextureId : int
{
	/** DBuffer A (Base Color). RGB is pre-multiplied by alpha. A is one minus alpha. */
	DBT_A UMETA(DisplayName = "DBufferA (BaseColor)"),
	/** DBuffer B (World Normal). RGB is pre-multiplied by alpha. A is one minus alpha. */
	DBT_B UMETA(DisplayName = "DBufferB (WorldNormal)"),
	/** DBuffer C (Roughness). R is Roughness, G is Metallic, B is Specular,  all pre-multiplied by alpha. A is one minus alpha. */
	DBT_C UMETA(DisplayName = "DBufferC (Roughness)"),
};

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionDBufferTexture : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** UV in 0..1 range */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Coordinates;

	/** Which DBuffer texture we want to make a lookup into. */
	UPROPERTY(EditAnywhere, Category = UMaterialExpressionDBufferTexture, meta = (DisplayName = "DBuffer Texture"))
	TEnumAsByte<EDBufferTextureId> DBufferTextureId;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
