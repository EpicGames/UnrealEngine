// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "RHIFeatureLevel.h"
#include "MaterialExpressionDataDrivenShaderPlatformInfoSwitch.generated.h"

enum EShaderPlatform : uint16;

UENUM()
enum EDataDrivenShaderPlatformInfoCondition : int
{
	COND_True UMETA(DisplayName = "Property must be true"),
	COND_False UMETA(DisplayName = "Property must be false"),
};

USTRUCT()
struct FDataDrivenShaderPlatformInfoInput
{
	GENERATED_USTRUCT_BODY()

		UPROPERTY(EditAnywhere, Category = FDataDrivenShaderPlatformInfoInput, meta = (GetOptions = "GetNameOptions"))
		FName InputName;

		UPROPERTY(EditAnywhere, Category = FDataDrivenShaderPlatformInfoInput)
		TEnumAsByte<EDataDrivenShaderPlatformInfoCondition> PropertyCondition = EDataDrivenShaderPlatformInfoCondition::COND_True;
};

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionDataDrivenShaderPlatformInfoSwitch : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
		UPROPERTY()
		FExpressionInput InputTrue;
		
		UPROPERTY()
		FExpressionInput InputFalse;

		UPROPERTY(EditAnywhere, Category = DataDrivenShaderPlatformInfoInput)
		TArray<FDataDrivenShaderPlatformInfoInput> DDSPIPropertyNames;

		UFUNCTION()
		TArray<FString> GetNameOptions() const;
		
		UPROPERTY()
		uint32 bContainsInvalidProperty : 1;

		//~ Begin UObject Interface
		virtual void PostLoad() override;
#if WITH_EDITOR
		virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
		//~ End UObject Interface
		// 
#if WITH_EDITOR
	FExpressionInput* GetEffectiveInput(EShaderPlatform ShaderPlatform, FString& OutError);

	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Unknown; }
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
