// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionDecalLifetimeOpacity.generated.h"

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionDecalLifetimeOpacity : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
