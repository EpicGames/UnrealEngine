// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionDecalMipmapLevel.generated.h"

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionDecalMipmapLevel : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	/** The texture's size */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to '(Const Width, Const Height)' if not specified"))
	FExpressionInput TextureSize;

	/** only used if TextureSize is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDecalMipmapLevel, meta=(OverridingInputProperty = "TextureSize"))
	float ConstWidth;
	
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDecalMipmapLevel, meta=(OverridingInputProperty = "TextureSize"))
	float ConstHeight;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
