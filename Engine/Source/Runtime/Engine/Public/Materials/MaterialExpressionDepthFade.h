// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Scales Opacity by a Linear fade based on SceneDepth, from 0 at PixelDepth to 1 at FadeDistance
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDepthFade.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionDepthFade : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Input opacity which will be scaled by the result of the fade. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'OpacityDefault' if not specified"))
	FExpressionInput InOpacity;

	/** World space distance over which the fade should take place. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'FadeDistanceDefault' if not specified"))
	FExpressionInput FadeDistance;

	/** Opacity which will be scaled by the result of the fade.  This is used when InOpacity is unconnected. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionDepthFade, meta=(OverridingInputProperty = "InOpacity"))
	float OpacityDefault = 1.0f;

	/** World space distance over which the fade should take place.  This is used when FadeDistance is unconnected. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionDepthFade, meta=(OverridingInputProperty = "FadeDistance"))
	float FadeDistanceDefault = 100.0f;

#if WITH_EDITOR

	//~ Begin UMaterialExpression Interface
	virtual FName GetInputName(int32 InputIndex) const override
	{
		if (InputIndex == 0)
		{
			return TEXT("Opacity");
		}
		else
		{
			return Super::GetInputName(InputIndex);
		}
	}
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override
	{
		OutCaptions.Add(TEXT("Depth Fade"));
	}

#endif
	//~ End UMaterialExpression Interface#endif
};



