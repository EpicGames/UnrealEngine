// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionDepthOfFieldFunction.generated.h"

// Note: The index is used to map the enum to different code in the shader
UENUM()
enum EDepthOfFieldFunctionValue : int
{
	/** 0:in Focus .. 1:Near or Far. */
	TDOF_NearAndFarMask,
	/** 0:in Focus or Far .. 1:Near. */
	TDOF_NearMask,
	/** 0:in Focus or Near .. 1:Far. */
	TDOF_FarMask,
	/** in pixels, only works for CircleDOF, use Abs for the actual radius as the sign of the value indicates near out of focus, positive indicates far out of focus */
	TDOF_CircleOfConfusionRadius,
	TDOF_MAX,
};

UCLASS()
class UMaterialExpressionDepthOfFieldFunction : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	/** Determines the mapping place to use on the terrain. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionDepthOfFieldFunction)
	TEnumAsByte<enum EDepthOfFieldFunctionValue> FunctionValue;

	/** usually nothing or PixelDepth */
	UPROPERTY()
	FExpressionInput Depth;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



