// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDesaturation.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionDesaturation : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	// Outputs: Lerp(Input,dot(Input,LuminanceFactors)),Fraction)
	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY()
	FExpressionInput Fraction;

	/* 
	 * Luminance factors for converting a color to greyscale.
	 * 
	 * The default luminance factors values are now derived from the working color space. For uses cases
	 * outside scene rendering, users are responsible for updating these factors accordingly. For example,
	 * factors derived from an AP1 working color space would not be applicable to UI domain materials that
	 * remain in sRGB/Rec.709 and thus should instead use approximately [0.2126, 0.7152, 0.0722].
	 */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionDesaturation, Meta = (ShowAsInputPin = "Advanced"))
	FLinearColor LuminanceFactors;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override
	{
		OutCaptions.Add(TEXT("Desaturation"));
	}
#endif
	//~ End UMaterialExpression Interface
};



