// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDistanceFieldApproxAO.generated.h"

UCLASS()
class UMaterialExpressionDistanceFieldApproxAO : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world position if not specified"))
	FExpressionInput Position;

	/** Defines the reference space for the Position input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldApproxAO)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world normal if not specified"))
	FExpressionInput Normal;

	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput BaseDistance;

	/** only used if BaseDistance is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldApproxAO, meta=(OverridingInputProperty = "BaseDistance"))
	float BaseDistanceDefault = 15;

	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Radius;

	/** only used if Radius is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldApproxAO, meta=(OverridingInputProperty = "Radius"))
	float RadiusDefault = 150;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldApproxAO, meta = (UIMin = "1", UIMax = "4", ShowAsInputPin = "Primary", ToolTip = "Number of samples used to calculate occlusion"))
	uint32 NumSteps = 1;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldApproxAO, DisplayName = "Step Scale", meta = (UIMin = "1.0", UIMax = "8.0", ShowAsInputPin = "Advanced", ToolTip = "Used to control step distance distribution"))
	float StepScaleDefault = 3.0f;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};
