// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDistanceFieldGradient.generated.h"

UCLASS()
class UMaterialExpressionDistanceFieldGradient : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world position if not specified"))
	FExpressionInput Position;

	/** Defines the reference space for the Position input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDistanceFieldGradient)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};
