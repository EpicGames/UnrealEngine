// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionParameter.h"
#include "MaterialExpressionDoubleVectorParameter.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionDoubleVectorParameter : public UMaterialExpressionParameter
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorParameter, Meta = (ShowAsInputPin = "Primary"))
	FVector4d DefaultValue;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = DefaultValue;
		return Super::GetParameterValue(OutMeta);
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::DoubleVector)
		{
			if (SetParameterValue(Name, Meta.Value.AsVector4d(), Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	bool SetParameterValue(FName InParameterName, FVector4d InValue, EMaterialExpressionSetParameterValueFlags Flags);

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
