// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *	A material expression that routes particle emitter parameters to the
 *	material.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionDynamicParameter.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionDynamicParameter : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** 
	 *	The names of the parameters.
	 *	These will show up in Cascade when editing a particle system
	 *	that uses the material it is in...
	 */
	UPROPERTY(EditAnywhere, editfixedsize, Category=MaterialExpressionDynamicParameter)
	TArray<FString> ParamNames;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionDynamicParameter, Meta = (ShowAsInputPin = "Primary"))
	FLinearColor DefaultValue = FLinearColor::White;

	/** The index of the dynamic parameter for use in tools that allow > 1 */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionDynamicParameter, meta = (UIMin = 0, ClampMin = 0, UIMax = 3, ClampMax = 3, ShowAsInputPin = "Primary"))
	uint32 ParameterIndex = 0;

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual TArray<FExpressionOutput>& GetOutputs() override;
	virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	virtual int32 GetWidth() const override;
	virtual int32 GetLabelPadding() override { return 8; }
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("dynamic")); }

#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITORONLY_DATA
	/**
	 * Iterate through all of the expression nodes until we find another 
	 * dynamic parameter we can copy the properties from
	 */
	ENGINE_API void UpdateDynamicParameterProperties();
#endif // WITH_EDITORONLY_DATA

	/**
	 * Copy the properties from the specified dynamic parameter
	 *
	 * @param	FromParam	The param to copy from
	 * @return	true if sucessful
	 */
	bool CopyDynamicParameterProperties(const UMaterialExpressionDynamicParameter* FromParam);
};



