// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"

#include "MaterialExpressionExternalCodeBase.generated.h"

UCLASS(abstract, hidecategories=Object, MinimalAPI)
class UMaterialExpressionExternalCodeBase : public UMaterialExpression
{
	GENERATED_BODY()

public:
	/**
	 * List of identifiers of the external HLSL code this expression inserts into the material shader.
	 * If the number of elements is 1, the single element will always be used. Otherwise, OutputIndex selects the respective identifier.
	 */
	UPROPERTY(Config)
	TArray<FName> ExternalCodeIdentifiers;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface

};
