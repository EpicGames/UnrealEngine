// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionEyeAdaptationInverse.generated.h"

/**
 * Provides access to the EyeAdaptation render target.
 */


UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionEyeAdaptationInverse : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Value to apply EyeAdaptation inverse"))
	FExpressionInput LightValueInput;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Weight of the conversion. 1.0 means full conversion, 0.0 means no change."))
	FExpressionInput AlphaInput;
};
