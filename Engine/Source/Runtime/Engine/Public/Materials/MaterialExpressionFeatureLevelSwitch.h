// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "RHIFeatureLevel.h"
#include "MaterialExpressionFeatureLevelSwitch.generated.h"

enum EShaderPlatform : uint16;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionFeatureLevelSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Default connection, used when a certain feature level doesn't have an override. */
	UPROPERTY()
	FExpressionInput Default;

	UPROPERTY()
	FExpressionInput Inputs[ERHIFeatureLevel::Num];

	//~ Begin UObject Interface.
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface.

#if WITH_EDITOR
	ERHIFeatureLevel::Type GetFeatureLevelToCompile(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel);

	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_Unknown;}
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return MCT_Unknown;}
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
