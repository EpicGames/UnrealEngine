// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionFirstPersonOutput.generated.h"

/** Material output expression for writing first person rendering properties. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionFirstPersonOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Interpolates between world space and first person space. Valid range is [0, 1], from world space to first person space. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Interpolates between world space and first person space. Valid range is [0, 1], from world space to first person space. Defaults to 'ConstFirstPersonInterpolationAlpha' if not specified."))
	FExpressionInput FirstPersonInterpolationAlpha;

	/** Only used if FirstPersonInterpolationAlpha is not hooked up. Interpolates between world space and first person space. Valid range is [0, 1], from world space to first person space. */
	UPROPERTY(EditAnywhere, Category = "First Person", meta = (OverridingInputProperty = "FirstPersonInterpolationAlpha", UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f))
	float ConstFirstPersonInterpolationAlpha = 1.0f;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
#if WITH_EDITOR
	virtual bool NeedsPreviousFrameEvaluation() override { return true; }
	virtual EShaderFrequency GetShaderFrequency(uint32 OutputIndex) override { return SF_Vertex; }
#endif
	//~ End UMaterialExpressionCustomOutput Interface
};
