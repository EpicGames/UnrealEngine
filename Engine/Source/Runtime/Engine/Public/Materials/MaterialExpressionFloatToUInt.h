// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionFloatToUInt.generated.h"

UENUM()
enum class EFloatToIntMode : uint8
{
	Truncate UMETA(DisplayName="Truncate"),
	Floor UMETA(DisplayName="Floor"),
	Round UMETA(DisplayName="Round"),
	Ceil UMETA(DisplayName="Ceil")
};

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionFloatToUInt : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category=FloatToInt)
	EFloatToIntMode  Mode = EFloatToIntMode::Truncate;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionUIntToFloat : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
