// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Materials/MaterialExpression.h"
#include "RHIShaderPlatform.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectMacros.h"

#include "MaterialExpressionFontSample.generated.h"

class UTexture;
class ITargetPlatform;

namespace UE
{
namespace Shader
{
struct FTextureValue;
}
}

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionFontSample : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** font resource that will be sampled */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFontSample)
	TObjectPtr<class UFont> Font;

	/** allow access to the various font pages */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFontSample)
	int32 FontTexturePage;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	virtual int32 GetWidth() const override;
	virtual int32 GetLabelPadding() override { return 8; }
#endif

	/** 
	 * Callback to get any texture reference this expression emits.
	 * This is used to link the compiled uniform expressions with their default texture values. 
	 * Any UMaterialExpression whose compilation creates a texture uniform expression (eg Compiler->Texture, Compiler->TextureParameter) must implement this.
	 */
	virtual UObject* GetReferencedTexture() const override;

	virtual bool CanReferenceTexture() const override { return true; }
	//~ End UMaterialExpression Interface

protected:
	TTuple<bool, UTexture*, EMaterialSamplerType, FString> ValidateAndGetTextureSampler(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform);
};



