// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionFontSample.h"
#include "MaterialExpressionFontSampleParameter.generated.h"

class UFont;
struct FMaterialParameterInfo;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionFontSampleParameter : public UMaterialExpressionFontSample
{
	GENERATED_UCLASS_BODY()

	/** name to be referenced when we want to find and set thsi parameter */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFontSampleParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFontSampleParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list. The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFontSampleParameter)
	int32 SortPriority = 32;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool MatchesSearchQuery( const TCHAR* SearchQuery ) override;
	virtual bool CanRenameNode() const override { return true; }
	virtual FString GetEditableName() const override;
	virtual void SetEditableName(const FString& NewName) override;

	virtual bool HasAParameterName() const override { return true; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = FMaterialParameterValue(Font, FontTexturePage);
		OutMeta.Description = Desc;
		OutMeta.ExpressionGuid = ExpressionGUID;
		OutMeta.Group = Group;
		OutMeta.SortPriority = SortPriority;
		return true;
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::Font)
		{
			if (SetParameterValue(Name, Meta.Value.Font.Value, Meta.Value.Font.Page, Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface
	
#if WITH_EDITOR
	bool SetParameterValue(FName InParameterName, UFont* InFontValue, int32 InFontPage, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif

	/**
	*	Sets the default Font if none is set
	*/
	virtual void SetDefaultFont();
	
	virtual FGuid& GetParameterExpressionId() override
	{
		return ExpressionGUID;
	}
};



