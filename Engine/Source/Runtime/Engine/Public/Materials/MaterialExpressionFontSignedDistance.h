// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionFontSignedDistance.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionFontSignedDistance : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
