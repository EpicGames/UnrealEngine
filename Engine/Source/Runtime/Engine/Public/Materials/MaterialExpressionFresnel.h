// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Allows the artists to quickly set up a Fresnel term. Returns:
 *
 *		pow(1 - max(Normal dot Camera,0),Exponent)
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionFresnel.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionFresnel : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'Exponent' if not specified"))
	FExpressionInput ExponentIn;

	/** The exponent to pass into the pow() function */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFresnel, meta=(OverridingInputProperty = "ExponentIn"))
	float Exponent = 5.0f;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'BaseReflectFraction' if not specified"))
	FExpressionInput BaseReflectFractionIn;

	/** 
	 * Specifies the fraction of specular reflection when the surfaces is viewed from straight on.
	 * A value of 1 effectively disables Fresnel.
	 */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFresnel, meta=(OverridingInputProperty = "BaseReflectFractionIn"))
	float BaseReflectFraction = 0.04f;

	/** The normal to dot with the camera FVector */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to Pixel World Normal if not specified"))
	FExpressionInput Normal;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override
	{
		OutCaptions.Add(FString("Fresnel"));
	}
#endif
	//~ End UMaterialExpression Interface
};



