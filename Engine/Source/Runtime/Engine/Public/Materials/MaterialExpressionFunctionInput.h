// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionFunctionInput.generated.h"

class FMaterialCompiler;
struct FPropertyChangedEvent;

/** Supported input types */
UENUM(BlueprintType)
enum EFunctionInputType : int
{
	FunctionInput_Scalar UMETA(DisplayName = "Scalar"),
	FunctionInput_Vector2 UMETA(DisplayName = "Vector2"),
	FunctionInput_Vector3 UMETA(DisplayName = "Vector3"),
	FunctionInput_Vector4 UMETA(DisplayName = "Vector4"),
	FunctionInput_Texture2D UMETA(DisplayName = "Texture2D"),
	FunctionInput_TextureCube UMETA(DisplayName = "TextureCube"),
	FunctionInput_Texture2DArray UMETA(DisplayName = "Texture2DArray"),
	FunctionInput_VolumeTexture UMETA(DisplayName = "VolumeTexture"),
	FunctionInput_StaticBool UMETA(DisplayName = "StaticBool"),
	FunctionInput_MaterialAttributes UMETA(DisplayName = "MaterialAttributes"),
	FunctionInput_TextureExternal UMETA(DisplayName = "TextureExternal"),
	FunctionInput_Bool UMETA(DisplayName = "Bool"),
	FunctionInput_Substrate UMETA(DisplayName = "Substrate"),
	FunctionInput_MAX UMETA(DisplayName = "NULL"),
};
#define EXPECTED_NUM_INPUT_TYPES 13

/** Supported input types */
UENUM()
enum EBlendInputRelevance : int
{
	General,
	Top,
	Bottom
};

UCLASS(hidecategories=object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionFunctionInput : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Used for previewing when editing the function, or when bUsePreviewValueAsDefault is enabled. */
	UPROPERTY(meta=(RequiredInput = "false"))
	FExpressionInput Preview;

	/** The input's name, which will be drawn on the connector in function call expressions that use this function. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionFunctionInput)
	FName InputName;

	/** The input's description, which will be used as a tooltip on the connector in function call expressions that use this function. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFunctionInput)
	FString Description;

	/** Id of this input, used to maintain references through name changes. */
	UPROPERTY()
	FGuid Id;

	/** 
	 * Type of this input.  
	 * Input code chunks will be cast to this type, and a compiler error will be emitted if the cast fails.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionFunctionInput)
	TEnumAsByte<enum EFunctionInputType> InputType;

	/** Value used to preview this input when editing the material function. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionFunctionInput, meta=(OverridingInputProperty = "Preview"))
	FVector4f PreviewValue;

	/** Whether to use the preview value or texture as the default value for this input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionFunctionInput)
	uint32 bUsePreviewValueAsDefault:1;

	/** Controls where the input is displayed relative to the other inputs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpression)
	int32 SortPriority;

	/** Marks that this input is necessary for blend function use **/
	UPROPERTY(EditAnywhere, Category = MaterialExpressionFunctionInput)
	TEnumAsByte<enum EBlendInputRelevance> BlendInputRelevance = EBlendInputRelevance::General;

#if WITH_EDITOR
	/** 
	* These create and delete new FExpression objects on the heap, but are public to allow creation and removal from external systems. 
	* Use the corresponding pop and push to manage internal behaviour of EffectivePreviews within this class.
	*/
	FExpressionInput* AddNewEffectivePreviewDuringCompile(FExpressionInput& InEffectivePreview);
	void RemoveLastEffectivePreviewDuringCompile();
#endif

	//~ Begin UObject Interface.
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	virtual void PostEditImport() override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;


	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif // WITH_EDITOR
	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const override;
	//~ End UMaterialExpression Interface

	/** Generate the Id for this input. */
	ENGINE_API void ConditionallyGenerateId(bool bForce);

	ENGINE_API static FString GetInputTypeDisplayName(EFunctionInputType InputType);
	ENGINE_API static EMaterialValueType GetMaterialTypeFromInputType(EFunctionInputType InputType);
	ENGINE_API static EFunctionInputType GetInputTypeFromMaterialType(EMaterialValueType MaterialType);
#if WITH_EDITOR
	/** Validate InputName.  Must be called after InputName is changed to prevent duplicate inputs. */
	ENGINE_API void ValidateName();
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	/** Helper function which compiles this expression for previewing. */
	int32 CompilePreviewValue(FMaterialCompiler* Compiler);

	/** Stashed data between a Pre/PostEditChange event */
	FName InputNameBackup;

	/** Cascading preview inputs to use when compiling from another material and/or function */
	TArray<FExpressionInput*> EffectivePreviewDuringCompile;

	/** Push and pop management of the effective previews should be restricted to internal functions only to avoid memory leaks. */
	void PushEffectivePreviewDuringCompile(FExpressionInput* InEffectivePreview);
	FExpressionInput* PopEffectivePreviewDuringCompile();
#endif // WITH_EDITOR
};



