// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionFunctionOutput.generated.h"

struct FPropertyChangedEvent;

UCLASS(hidecategories=object, MinimalAPI)
class UMaterialExpressionFunctionOutput : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** The output's name, which will be drawn on the connector in function call expressions that use this function. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionFunctionOutput)
	FName OutputName;

	/** The output's description, which will be used as a tooltip on the connector in function call expressions that use this function. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionFunctionOutput, meta=(MultiLine=true))
	FString Description;

	/** Controls where the output is displayed relative to the other outputs. */
	UPROPERTY(EditAnywhere, Category=MaterialExpression)
	int32 SortPriority;

	/** Stores the expression in the material function connected to this output. */
	UPROPERTY()
	FExpressionInput A;

	/** Whether this output was previewed the last time this function was edited. */
	UPROPERTY()
	uint32 bLastPreviewed:1;

	/** Id of this input, used to maintain references through name changes. */
	UPROPERTY()
	FGuid Id;


	//~ Begin UObject Interface.
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	virtual void PostEditImport() override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override
	{
		return NAME_None;
	}
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif // WITH_EDITOR
	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const override;
	//~ End UMaterialExpression Interface


	/** Generate the Id for this input. */
	ENGINE_API void ConditionallyGenerateId(bool bForce);

#if WITH_EDITOR
	/** Validate OutputName.  Must be called after OutputName is changed to prevent duplicate outputs. */
	ENGINE_API void ValidateName();
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	/** Stashed data between a Pre/PostEditChange event */
	FName OutputNameBackup;
#endif // WITH_EDITOR
};



