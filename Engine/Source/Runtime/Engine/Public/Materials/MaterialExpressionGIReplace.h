// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionGIReplace.generated.h"

UCLASS()
class UMaterialExpressionGIReplace : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Used for direct lighting computations e.g. real-time shaders */
	UPROPERTY()
	FExpressionInput Default;

	/** Used for baked indirect lighting e.g. Lightmass */
	UPROPERTY()
	FExpressionInput StaticIndirect;

	/** Used for dynamic indirect lighting e.g. Light Propagation Volumes */
	UPROPERTY()
	FExpressionInput DynamicIndirect;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



