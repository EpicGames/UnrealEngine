// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Shader/ShaderTypes.h"
#include "MaterialExpressionGenericConstant.generated.h"

UCLASS(abstract, MinimalAPI)
class UMaterialExpressionGenericConstant : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	virtual UE::Shader::FValue GetConstantValue() const PURE_VIRTUAL(UMaterialExpressionGenericConstant::GetConstantValue, return UE::Shader::FValue(););

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FString GetDescription() const override;
#endif // WITH_EDITOR

	//~ End UMaterialExpression Interface
};

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionConstantDouble : public UMaterialExpressionGenericConstant
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionConstant, meta = (ShowAsInputPin = "Primary"))
	double Value;

	virtual UE::Shader::FValue GetConstantValue() const override { return Value; }
};
