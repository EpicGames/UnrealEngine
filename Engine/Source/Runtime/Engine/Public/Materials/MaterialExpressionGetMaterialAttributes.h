// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionGetMaterialAttributes.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionGetMaterialAttributes : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

 	UPROPERTY()
 	FMaterialAttributesInput MaterialAttributes;

	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	TArray<FGuid> AttributeGetTypes;
#if WITH_EDITOR
	TArray<FGuid> PreEditAttributeGetTypes;
#endif

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif
	//~ End UObject Interface
 
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API int32 CreateOrGetOutputAttribute(EMaterialProperty Attribute);
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return true;}
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_MaterialAttributes; }
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override { return OutputIndex == 0; }
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
