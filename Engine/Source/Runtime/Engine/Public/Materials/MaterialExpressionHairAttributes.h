// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionHairAttributes.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionHairAttributes : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** If enabled, this nodes outputs a tangent space tangent, otherwise it outputs a world space tangent. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionHairAttributes)
	uint32 bUseTangentSpace : 1 = true;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip);
#endif
	//~ End UMaterialExpression Interface
};
