// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionHairColor.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionHairColor : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Control the concentration of melanin in hair. Value is normalize over the plausible range of human hair.*/
	UPROPERTY(meta = (RequiredInput = "false", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FExpressionInput Melanin;

	/** Control the redness tint of the hair. Value is normalize over the plausible range of human hair.*/
	UPROPERTY(meta = (RequiredInput = "false", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FExpressionInput Redness;

	/** Control the dyeing color of the hair. By default there is no dyeing color.*/
	UPROPERTY(meta = (RequiredInput = "false", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FExpressionInput DyeColor;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
