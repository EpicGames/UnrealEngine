// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionIf.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionIf : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	UPROPERTY()
	FExpressionInput AGreaterThanB;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'A > B' if not specified"))
	FExpressionInput AEqualsB;

	UPROPERTY()
	FExpressionInput ALessThanB;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionIf, meta = (ShowAsInputPin = "Advanced"))
	float EqualsThreshold = 0.00001f;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionIf, meta = (OverridingInputProperty = "B"))
	float ConstB = 0.0f;

	UPROPERTY()
	float ConstAEqualsB_DEPRECATED;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 InputIndex) override {return MCT_Unknown;}
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};



