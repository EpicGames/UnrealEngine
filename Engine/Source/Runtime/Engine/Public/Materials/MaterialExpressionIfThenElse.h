// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionIfThenElse.generated.h"


UCLASS(collapsecategories, hidecategories = Object, MinimalAPI, meta=(NewMaterialTranslator, MIR_OnDemandInputRequest))
class UMaterialExpressionIfThenElse : public UMaterialExpression
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FExpressionInput Condition;

	UPROPERTY()
	FExpressionInput True;

	UPROPERTY()
	FExpressionInput False;
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
