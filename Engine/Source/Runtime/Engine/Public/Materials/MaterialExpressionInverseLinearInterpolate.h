// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"

#include "MaterialExpressionInverseLinearInterpolate.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionInverseLinearInterpolate : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstA' if not specified"))
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstValue' if not specified"))
	FExpressionInput Value;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionInverseLinearInterpolate, meta = (OverridingInputProperty = "A"))
	float ConstA = 0.0f;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionInverseLinearInterpolate, meta = (OverridingInputProperty = "B"))
	float ConstB = 1.0f;

	/** only used if Value is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionInverseLinearInterpolate, meta = (OverridingInputProperty = "Value"))
	float ConstValue = 0.0f;

	/** Clamp the result to 0 to 1 */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionInverseLinearInterpolate)
	bool bClampResult = false;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("InvLerp")); }
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("InverseLinearInterpolate")); }

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
