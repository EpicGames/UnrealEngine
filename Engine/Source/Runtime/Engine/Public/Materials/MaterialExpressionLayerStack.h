// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialLayersFunctions.h"
#include "MaterialExpressionLayerStack.generated.h"

class FMaterialCompiler;
class UMaterialFunctionInterface;

USTRUCT(Experimental)
struct FMaterialLayerInput : public FExpressionInput
{
	GENERATED_BODY()

	UPROPERTY()
	TEnumAsByte<EFunctionInputType> InputType;

	FMaterialLayerInput(FName NewInputName = NAME_None, EFunctionInputType NewInputType = FunctionInput_MAX);
	FString GetInputName() const;
};

UCLASS(hidecategories = Object, MinimalAPI, Experimental)
class UMaterialExpressionLayerStack : public UMaterialExpressionMaterialAttributeLayers
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FMaterialLayerInput> LayerInputs;

	UPROPERTY(EditAnywhere, Category = AvailableFunctions, meta = (DisallowedClasses = "/Script/Engine.MaterialFunctionMaterialLayerBlend, /Script/Engine.MaterialFunctionMaterialLayerBlendInstance"))
	TSet<TObjectPtr<UMaterialFunctionInterface>> AvailableLayers;

	UPROPERTY(EditAnywhere, Category = AvailableFunctions, meta = (AllowedClasses = "/Script/Engine.MaterialFunctionMaterialLayerBlend, /Script/Engine.MaterialFunctionMaterialLayerBlendInstance"))
	TSet<TObjectPtr<UMaterialFunctionInterface>> AvailableBlends;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	//~ End UObject Interface

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	//~ End UMaterialExpression Interface

	//~ Begin UMaterialExpressionMaterialAttributeLayers Interface
	virtual bool ValidateLayerConfiguration(FMaterialCompiler* Compiler, bool bReportErrors) override;
	ENGINE_API virtual void RebuildLayerGraph(bool bReportErrors);
	//~ End UMaterialExpressionMaterialAttributeLayers Interface

	//This struct is to circumvent having to extend the below lambda in future if we want to validate against additional node types.
	struct FValidLayerUsageTracker
	{
		int32 MAInputCount = 0;
		int32 MAOutputCount = 0;
		bool bContainsStatics = false;
	};

	static UMaterialFunctionInterface* ExtractParentFunctionFromInstance(class FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction);
	static bool PollFunctionExpressionsForLayerUsage(class FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, FValidLayerUsageTracker& Tracker, bool bCheckStatics);
	static bool ValidateFunctionForLayerUsage(class FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, bool bCheckStatics = false);
	static bool ValidateFunctionForBlendUsage(class FMaterialCompiler* Compiler, UMaterialFunctionInterface* CurrentFunction, bool bCheckStatics = false);

	TSharedPtr<FMaterialLayerStackFunctionsCache> GetSharedAvailableFunctionsCache();
	void ResolveLayerInputs();
	void CacheLayerInputs();
#endif
	inline TArray<FSoftObjectPath> GetAvailableLayerPaths() { return GetPathsFromAvailableFunctions(AvailableLayers); }
	inline TArray<FSoftObjectPath> GetAvailableBlendPaths() { return GetPathsFromAvailableFunctions(AvailableBlends); }

private:
	TSharedPtr<FMaterialLayerStackFunctionsCache> SharedCache;
	bool bAreAvailableLayersValid = false;

	static TArray<FSoftObjectPath> GetPathsFromAvailableFunctions(TSet<TObjectPtr<UMaterialFunctionInterface>>& InFunctions);
};
