// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *	A material expression that routes LightmapUVs to the material.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionLightmapUVs.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionLightmapUVs : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



