// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLinearInterpolate.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionLinearInterpolate : public UMaterialExpression
{
	GENERATED_BODY()

public:
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstA' if not specified"))
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstAlpha' if not specified"))
	FExpressionInput Alpha;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLinearInterpolate, meta=(OverridingInputProperty = "A"))
	float ConstA = 0.0f;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLinearInterpolate, meta=(OverridingInputProperty = "B"))
	float ConstB = 1.0f;

	/** only used if Alpha is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLinearInterpolate, meta=(OverridingInputProperty = "Alpha"))
	float ConstAlpha = 0.5f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("lerp"));}
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



