// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLocalPosition.generated.h"

/** Specifies which shader generated offsets should included in the position (displacement/WPO etc.) */
UENUM()
enum class EPositionIncludedOffsets
{
	/** Position with all material shader offsets applied */
	IncludeOffsets UMETA(DisplayName="Include Material Shader Offsets"),

	/** Position with no material shader offsets applied */
	ExcludeOffsets UMETA(DisplayName="Exclude Material Shader Offsets")
};

/** Specifies the reference point of the local position. This can be different in some cases, e.g. for instanced meshes. */
UENUM()
enum class ELocalPositionOrigin
{
	/** Position relative to instance */
	Instance UMETA(DisplayName="Instance"),

	/** Returns pre-skinned local position for skeletal meshes, usable in vertex shader only.
	Returns the instance position for non-skeletal meshes. Incompatible with GPU skin cache feature.*/
	InstancePreSkinning UMETA(DisplayName="Pre-Skinned Instance"),

	/** Position relative to primitive actor component */
	Primitive UMETA(DisplayName="Component")
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLocalPosition : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLocalPosition, meta=(DisplayName = "Shader Offsets", ShowAsInputPin = "Advanced"))
	EPositionIncludedOffsets IncludedOffsets = EPositionIncludedOffsets::IncludeOffsets;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLocalPosition, meta=(DisplayName = "Local Origin", ShowAsInputPin = "Advanced"))
	ELocalPositionOrigin LocalOrigin = ELocalPositionOrigin::Instance;
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("position preskinned local instance primitive"));}
#endif
	//~ End UMaterialExpression Interface
};
