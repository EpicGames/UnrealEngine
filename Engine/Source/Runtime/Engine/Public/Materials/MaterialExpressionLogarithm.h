// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLogarithm.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionLogarithm : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("log2"));}
#endif
	//~ End UMaterialExpression Interface

};
