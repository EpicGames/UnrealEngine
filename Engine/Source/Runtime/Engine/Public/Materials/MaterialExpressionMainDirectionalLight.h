// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionMainDirectionalLight.generated.h"

UCLASS()
class UMaterialExpressionMainDirectionalLight : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip)override;
#endif
	//~ End UMaterialExpression Interface
};


