// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionMakeMaterialAttributes.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionMakeMaterialAttributes : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput BaseColor;

	UPROPERTY()
	FExpressionInput Metallic;

	UPROPERTY()
	FExpressionInput Specular;

	UPROPERTY()
	FExpressionInput Roughness;

	UPROPERTY()
	FExpressionInput Anisotropy;

	UPROPERTY()
	FExpressionInput EmissiveColor;

	UPROPERTY()
	FExpressionInput Opacity;

	UPROPERTY()
	FExpressionInput OpacityMask;

	UPROPERTY()
	FExpressionInput Normal;

	UPROPERTY()
	FExpressionInput Tangent;

	UPROPERTY()
	FExpressionInput WorldPositionOffset;

	UPROPERTY()
	FExpressionInput SubsurfaceColor;

	UPROPERTY()
	FExpressionInput ClearCoat;

	UPROPERTY()
	FExpressionInput ClearCoatRoughness;

	UPROPERTY()
	FExpressionInput AmbientOcclusion;

	UPROPERTY()
	FExpressionInput Refraction;

	UPROPERTY()
	FExpressionInput CustomizedUVs[8];

	UPROPERTY()
	FExpressionInput PixelDepthOffset;

	UPROPERTY()
	FExpressionInput ShadingModel;

	UPROPERTY()
	FExpressionInput Displacement;

	/** Get the input for a material property. Returns nullptr if the property isn't supported. */
	FExpressionInput* GetExpressionInput(EMaterialProperty InProperty);

	//~ Begin UObject Interface
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override {return true;}
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;

	uint64 GetConnectedInputs() const;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



