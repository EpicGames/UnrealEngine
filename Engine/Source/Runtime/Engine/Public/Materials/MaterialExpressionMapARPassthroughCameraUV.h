// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *	A material expression that maps viewport UVs to AR passthrough camera UVs taking into account aspect ratio and device rotation.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionMapARPassthroughCameraUV.generated.h"

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionMapARPassthroughCameraUV : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input UV to map to AR Camera UV"))
	FExpressionInput Coordinates;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



