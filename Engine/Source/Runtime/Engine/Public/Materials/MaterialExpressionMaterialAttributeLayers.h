// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialLayersFunctions.h"
#include "MaterialExpressionMaterialAttributeLayers.generated.h"

class FMaterialCompiler;
class UMaterialFunctionInterface;
class UMaterialExpressionMaterialFunctionCall;
struct FMaterialParameterInfo;

/**
* These defines are for tracking the required number of inputs / outputs for blends rather than the values being
* hard coded throughout the layer functions and expressions code
*/
constexpr int8 AcceptableNumLayerMAInputs = 1;
constexpr int8 AcceptableNumBlendMAInputs = 2;
constexpr int8 AcceptableNumLayerBlendMAOutputs = 1;

UCLASS(hidecategories=Object, MinimalAPI)
class UMaterialExpressionMaterialAttributeLayers : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FMaterialAttributesInput Input;
	
	UPROPERTY(EditAnywhere, Category=Layers)
	FMaterialLayersFunctions DefaultLayers;

	const TArray<UMaterialFunctionInterface*>& GetLayers() const
	{
		return ParamLayers ? ParamLayers->Layers : DefaultLayers.Layers;
	}

	const TArray<UMaterialFunctionInterface*>& GetBlends() const
	{
		return ParamLayers ? ParamLayers->Blends : DefaultLayers.Blends;
	}

#if WITH_EDITOR
	const TArray<FText>& GetLayerNames() const
	{
		return ParamLayers ? ParamLayers->EditorOnly.LayerNames : DefaultLayers.EditorOnly.LayerNames;
	}

	const TArray<bool>& GetShouldFilterLayers() const
	{
		return ParamLayers ? ParamLayers->EditorOnly.RestrictToLayerRelatives : DefaultLayers.EditorOnly.RestrictToLayerRelatives;
	}

	const TArray<bool>& GetShouldFilterBlends() const
	{
		return ParamLayers ? ParamLayers->EditorOnly.RestrictToBlendRelatives : DefaultLayers.EditorOnly.RestrictToBlendRelatives;
	}

	const TArray<FGuid>& GetLayerGuids() const
	{
		return ParamLayers ? ParamLayers->EditorOnly.LayerGuids : DefaultLayers.EditorOnly.LayerGuids;
	}

	const TArray<bool>& GetLayerStates() const
	{
		return ParamLayers ? ParamLayers->EditorOnly.LayerStates : DefaultLayers.EditorOnly.LayerStates;
	}
#endif // WITH_EDITOR

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialExpressionMaterialFunctionCall>> LayerCallers;

	UPROPERTY(Transient)
	int32 NumActiveLayerCallers = 0;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialExpressionMaterialFunctionCall>> BlendCallers;

	UPROPERTY(Transient)
	int32 NumActiveBlendCallers = 0;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialExpressionMaterialFunctionCall> OutputCaller;

	UPROPERTY(Transient)
	bool bIsLayerGraphBuilt = false;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ Begin UObject Interface

#if WITH_EDITOR
	ENGINE_API virtual void RebuildLayerGraph(bool bReportErrors);
	ENGINE_API void OverrideLayerGraph(const FMaterialLayersFunctions* OverrideLayers);
#endif // WITH_EDITOR

#if WITH_EDITOR
	static void LogError(FMaterialCompiler* Compiler, const TCHAR* Format, ...);
	virtual bool ValidateLayerConfiguration(FMaterialCompiler* Compiler, bool bReportErrors);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const;
	void GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const;
#endif

	UMaterialFunctionInterface* GetParameterAssociatedFunction(const FHashedMaterialParameterInfo& ParameterInfo) const;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return false;}
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override {return true;}
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override {return false;}
#endif
	//~ End UMaterialExpression Interface

protected:
	/** Internal pointer to parameter-driven layer graph */
	const FMaterialLayersFunctions* ParamLayers = nullptr;
};
