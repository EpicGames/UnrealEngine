// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "MaterialExpressionCustomOutput.h"
#include "MaterialCache/MaterialCacheVirtualTextureTag.h"
#include "MaterialExpressionMaterialCache.generated.h"

USTRUCT()
struct FMaterialExpressionMaterialCacheAttribute
{
	GENERATED_BODY()

	/** Decorated name for printing */
	UPROPERTY()
	FString Decoration;

	/** EMaterialValueType, u64 for serialization */
	UPROPERTY()
	uint64 ValueType = 0;

	/** Bindable input */
	UPROPERTY()
	FExpressionInput Input;
};

UCLASS(collapsecategories, hidecategories = Object, meta = (DisplayName = "MaterialCache"))
class UMaterialExpressionMaterialCache : public UMaterialExpressionCustomOutput
{
	GENERATED_BODY()

public:
	/**
	 * Is this a material cache sample?
	 * Sample expressions do not output to the cache, and may sample the cache of other primitives
	 **/
	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	bool bIsSample = false;

	/** Optional, the tag of the cache to read/write */
	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	TObjectPtr<UMaterialCacheVirtualTextureTag> Tag;

	/** All attributes of this expression */
	UPROPERTY()
	TArray<FMaterialExpressionMaterialCacheAttribute> Attributes;

	/** Optional, the primitive whose material cache it to be sampled from */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Primitive;

	/** Optional, UV coordinate to sample on */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput UV;

	virtual FString GetFunctionName() const override { return TEXT("GetMaterialCache"); }
	virtual FString GetDisplayName() const override { return TEXT("MaterialCache"); }

	UMaterialExpressionMaterialCache(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	void ConstructFromTag();
	void ConstructLayout();
	void ConstructOutputs();
	
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual EShaderFrequency GetShaderFrequency(uint32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual int32 GetNumOutputs() const override;
	virtual int32 GetMaxOutputs() const override;
	virtual TArray<FExpressionOutput>& GetOutputs() override;
	virtual bool AllowMultipleCustomOutputs() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	uint32 GetPackedAttributeIndex(EMaterialCacheAttribute Attribute);
	
	/** Attributes in declaration order */
	TArray<EMaterialCacheAttribute> TagAttributes;
	
	/** Packed attribute layout */
	FMaterialCacheTagLayout TagLayout;
#endif
};
