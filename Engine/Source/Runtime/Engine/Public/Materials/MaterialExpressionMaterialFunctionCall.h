// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialExpressionMaterialFunctionCall - an expression which allows a material to use a material function
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionMaterialFunctionCall.generated.h"

class UMaterialFunction;
class FMaterialFunctionCompileState;
struct FPropertyChangedEvent;

/** Struct that stores information about a function input which is needed to maintain connections and implement the function call. */
USTRUCT()
struct FFunctionExpressionInput
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Reference to the FunctionInput in the material function.  
	 * This is a reference to a private object so it can't be saved, and must be generated by UpdateFromFunctionResource or SetMaterialFunction. 
	 */
	UPROPERTY(transient)
	TObjectPtr<class UMaterialExpressionFunctionInput> ExpressionInput;

	/** Id of the FunctionInput, used to link ExpressionInput. */
	UPROPERTY()
	FGuid ExpressionInputId;

	/** Actual input struct which stores information about how this input is connected in the material. */
	UPROPERTY()
	FExpressionInput Input;


	FFunctionExpressionInput()
		: ExpressionInput(NULL)
	{
	}

};

/** Struct that stores information about a function output which is needed to maintain connections and implement the function call. */
USTRUCT()
struct FFunctionExpressionOutput
{
	GENERATED_USTRUCT_BODY()

	/** 
	 * Reference to the FunctionOutput in the material function.  
	 * This is a reference to a private object so it can't be saved, and must be generated by UpdateFromFunctionResource or SetMaterialFunction. 
	 */
	UPROPERTY(transient)
	TObjectPtr<class UMaterialExpressionFunctionOutput> ExpressionOutput;

	/** Id of the FunctionOutput, used to link ExpressionOutput. */
	UPROPERTY()
	FGuid ExpressionOutputId;

	/** Actual output struct which stores information about how this output is connected in the material. */
	UPROPERTY()
	FExpressionOutput Output;


	FFunctionExpressionOutput()
		: ExpressionOutput(NULL)
	{
	}

};

UCLASS(hidecategories=object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionMaterialFunctionCall : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** The function to call. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionMaterialFunctionCall, meta=(DisallowedClasses="/Script/Engine.MaterialFunctionInstance"))
	TObjectPtr<class UMaterialFunctionInterface> MaterialFunction;

	/** Array of all the function inputs that this function exposes. */
	UPROPERTY()
	TArray<struct FFunctionExpressionInput> FunctionInputs;

	/** Array of all the function outputs that this function exposes. */
	UPROPERTY()
	TArray<struct FFunctionExpressionOutput> FunctionOutputs;

	/** Used by material parameters to split references to separate instances. */
	UPROPERTY(Transient)
	struct FMaterialParameterInfo FunctionParameterInfo;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	ENGINE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	ENGINE_API virtual void PostLoad() override;
	//~ End UObject Interface.

#if WITH_EDITORONLY_DATA
	ENGINE_API bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const;
	ENGINE_API void GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const;
#endif

#if WITH_EDITOR
	ENGINE_API void UnlinkFunctionFromCaller(FMaterialCompiler* Compiler);
	ENGINE_API void LinkFunctionIntoCaller(FMaterialCompiler* Compiler);

	ENGINE_API void PushParameterOwner(FMaterialCompiler* Compiler);
	ENGINE_API void PopParameterOwner(FMaterialCompiler* Compiler);
#endif

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	ENGINE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	ENGINE_API virtual FName GetInputName(int32 InputIndex) const override;
	ENGINE_API virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	ENGINE_API virtual FString GetDescription() const override;
	ENGINE_API virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	ENGINE_API virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	ENGINE_API virtual bool DeepSearchQuery(const TCHAR* SearchQuery, TSharedPtr<FFindInMaterialResult> SearchResult) override;
	ENGINE_API virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	ENGINE_API virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	ENGINE_API virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	ENGINE_API virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	ENGINE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	ENGINE_API virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	//~ End UMaterialExpression Interface

	/** Util to get name of a particular type, optionally with type appended in parenthesis */
	ENGINE_API FName GetInputNameWithType(int32 InputIndex, bool bWithType) const;

	/** 
	 * Set a new material function, given an old function so that links can be passed over if the name matches. 
	 *
	 *	@param OldFunctionResource				The function it was set to.
	 *	@param NewResource						The function to be set to.
	 *
	 *	@return									true if setting the function was a success, false if it failed.
	 */
	ENGINE_API bool SetMaterialFunctionEx(UMaterialFunctionInterface* OldFunctionResource, UMaterialFunctionInterface* NewResource);

	/** */
	UFUNCTION(BlueprintCallable, Category = "MaterialEditing")
	ENGINE_API virtual bool SetMaterialFunction(UMaterialFunctionInterface* NewMaterialFunction);

	/** 
	 * Update FunctionInputs and FunctionOutputs from the MaterialFunction.  
	 * This must be called to keep the inputs and outputs up to date with the function being used. 
	 */
	ENGINE_API void UpdateFromFunctionResource(bool bRecreateAndLinkNode = true);

	/** Temporary compilation state shared between function calls */
	void SetSharedCompileState(FMaterialFunctionCompileState* SharedState)
	{
		SharedCompileState = SharedState;
	}

private:	
	/** Helper that fixes up expression links where possible. */
	void FixupReferencingExpressions(
		const TArray<FFunctionExpressionOutput>& NewOutputs,
		const TArray<FFunctionExpressionOutput>& OriginalOutputs,
		TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions, 
		TArray<FExpressionInput*>& MaterialInputs,
		bool bMatchByName);

	/** Temporary compilation state shared between function calls */
	FMaterialFunctionCompileState* SharedCompileState;

	/** Stashed data between a Pre/PostEditChange event */
	UMaterialFunctionInterface* SavedMaterialFunction = nullptr;
#endif // WITH_EDITOR
};



