// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "MaterialExpressionMaterialLayerOutput.generated.h"

struct FPropertyChangedEvent;

UCLASS(hidecategories=object, MinimalAPI)
class UMaterialExpressionMaterialLayerOutput : public UMaterialExpressionFunctionOutput
{
	GENERATED_UCLASS_BODY()
#if WITH_EDITOR
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override
	{
		return MCT_MaterialAttributes;
	}

	virtual bool CanUserDeleteExpression() const override
	{
		return false;
	}
#endif
};



