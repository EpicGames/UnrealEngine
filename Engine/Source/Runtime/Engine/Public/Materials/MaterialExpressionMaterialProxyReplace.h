// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionMaterialProxyReplace.generated.h"

UCLASS()
class UMaterialExpressionMaterialProxyReplace : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Realtime;

	UPROPERTY()
	FExpressionInput MaterialProxy;


	// Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Unknown; }
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	// End UMaterialExpression Interface
};
