// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionMax.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionMax : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstA' if not specified"))
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionMax, meta=(OverridingInputProperty = "A"))
	float ConstA = 0.0f;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionMax, meta=(OverridingInputProperty = "B"))
	float ConstB = 1.0f;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



