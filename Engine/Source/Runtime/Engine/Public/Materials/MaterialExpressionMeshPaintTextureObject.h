// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpressionExternalCodeBase.h"

#include "MaterialExpressionMeshPaintTextureObject.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionMeshPaintTextureObject : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
