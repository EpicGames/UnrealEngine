// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"

#include "MaterialExpressionMeshPaintTextureReplace.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionMeshPaintTextureReplace : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when no valid MeshPaintTexture is available"))
	FExpressionInput Default;

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when a valid MeshPaintTexture is available"))
	FExpressionInput MeshPaintTexture;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
