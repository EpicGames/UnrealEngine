// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionMotionVectorWorldOffsetOutput.generated.h"

/** Material output expression for generating a per-pixel motion vector offset that will be added to the existing motion vector. */
UCLASS(collapsecategories, hidecategories=Object, Experimental, meta = (DisplayName = "Motion Vector World Offset (Per-Pixel)"), MinimalAPI)
class UMaterialExpressionMotionVectorWorldOffsetOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/**Motion vector offset in world space in the previous frame.
	* 
	*The input is added as a per-pixel offset to the motion vector generated by the velocity pass.
	*
	*Non-Nanite (Non-basepass velocity supported).
	*/
	UPROPERTY()
	FExpressionInput Input;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
