// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpressionRerouteBase.h"
#include "MaterialExpressionNamedReroute.generated.h"

class UMaterialExpressionNamedRerouteDeclaration;

UCLASS(abstract, MinimalAPI)
class UMaterialExpressionNamedRerouteBase : public UMaterialExpressionRerouteBase
{
	GENERATED_BODY()

protected:
	/**
	 * Find a variable declaration in an array of expressions
	 * @param	VariableGuid	The GUID of the variable to find
	 * @param	Expressions		The expressions to search in
	 * @return	null if not found
	 */
	template<typename ExpressionsArrayType>
	UMaterialExpressionNamedRerouteDeclaration* FindDeclarationInArray(const FGuid& VariableGuid, const ExpressionsArrayType& Expressions) const;

	/**
	 * Find a variable declaration in the entire graph
	 * @param	VariableGuid	The GUID of the variable to find
	 * @return	null if not found
	 */
	ENGINE_API UMaterialExpressionNamedRerouteDeclaration* FindDeclarationInMaterial(const FGuid& VariableGuid) const;
};

UCLASS(collapsecategories, hidecategories=Object, DisplayName = "Named Reroute Declaration", MinimalAPI)
class UMaterialExpressionNamedRerouteDeclaration : public UMaterialExpressionNamedRerouteBase
{
	GENERATED_UCLASS_BODY()

public:
	// The input pin
	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionNamedRerouteDeclaration)
	FName Name;

	/** The color of the graph node. The same color will apply to all linked usages of this Declaration node */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionNamedRerouteDeclaration)
	FLinearColor NodeColor;

	// The variable GUID, to support copy across graphs
	UPROPERTY()
	FGuid VariableGuid;

	//~ Begin UObject Interface
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual int32 CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	ENGINE_API virtual FText GetCreationDescription() const override;
	ENGINE_API virtual FText GetCreationName() const override;
	
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	ENGINE_API virtual bool CanRenameNode() const override;
	ENGINE_API virtual FString GetEditableName() const override;
	ENGINE_API virtual void SetEditableName(const FString& NewName) override;

	ENGINE_API virtual void PostCopyNode(const TArray<UMaterialExpression*>& CopiedExpressions) override;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface

protected:
	//~ Begin UMaterialExpressionRerouteBase Interface
	ENGINE_API virtual bool GetRerouteInput(FExpressionInput& OutInput) const override;
	//~ End UMaterialExpressionRerouteBase Interface

private:
	/** 
	* Generates a GUID for the variable if one doesn't already exist
	* @param bForceGeneration	Whether we should generate a GUID even if it is already valid.
	*/
	void UpdateVariableGuid(bool bForceGeneration, bool bAllowMarkingPackageDirty);
	void MakeNameUnique();
};

UCLASS(collapsecategories, hidecategories=Object, DisplayName = "Named Reroute Usage", MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionNamedRerouteUsage : public UMaterialExpressionNamedRerouteBase
{
	GENERATED_BODY()

public:
	// The declaration this node is linked to
	UPROPERTY()
	TObjectPtr<UMaterialExpressionNamedRerouteDeclaration> Declaration;
	
	// The variable GUID, to support copy across graphs
	UPROPERTY()
	FGuid DeclarationGuid;

	// Check that the declaration isn't deleted
	bool IsDeclarationValid() const;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual int32 CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex) override;

	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	
	ENGINE_API virtual void PostCopyNode(const TArray<UMaterialExpression*>& CopiedExpressions) override;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface

protected:
	//~ Begin UMaterialExpressionRerouteBase Interface
	ENGINE_API virtual bool GetRerouteInput(FExpressionInput& OutInput) const override;
	//~ End UMaterialExpressionRerouteBase Interface
};

template<typename ExpressionsArrayType>
inline UMaterialExpressionNamedRerouteDeclaration* UMaterialExpressionNamedRerouteBase::FindDeclarationInArray(const FGuid& VariableGuid, const ExpressionsArrayType& Expressions) const
{
	for (UMaterialExpression* Expression : Expressions)
	{
		auto* Declaration = Cast<UMaterialExpressionNamedRerouteDeclaration>(Expression);
		if (Declaration && this != Declaration && Declaration->VariableGuid == VariableGuid)
		{
			return Declaration;
		}
	}
	return nullptr;
}
