// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Node acts as a base class for TextureSamples and TextureObjects 
 * to cover their shared functionality. 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionNeuralPostProcessNode.generated.h"

struct FPropertyChangedEvent;
enum EMaterialSamplerType : int;

UENUM()
enum ENeuralIndexType : int
{
	/** Texture index UV in 0...1 range*/
	NIT_TextureIndex UMETA(DisplayName = "Texture Index"),

	/** Buffer index UV in LayerId (batch) x Channels x Height x Wdith*/
	NIT_BufferIndex UMETA(DisplayName = "Buffer Index"),

};

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionNeuralNetworkInput : public UMaterialExpressionCustomOutput 
{
	GENERATED_UCLASS_BODY()
	
	/**
	*	Coordinate to read values:
			Texture index (float4): Batch[ignored], StartChannel[int,0], ViewportUV in 0..1 range
			Buffer  index (float4): Batch[int,0], StartChannel[int,0], WidthHeight[ViewportUV] in 0..1 range
	*/
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Coordinates;

	/** First input node for neural post processing in post process material*/
	UPROPERTY()
	FExpressionInput Input0;

	/** Input is ignored if Mask is zero. 
		TODO: Used to optimize performance
	*/
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Mask;

	/**Indexing type: write to the texture or buffer */
	UPROPERTY(EditAnywhere, CateGory = "Default", meta = (DisplayName = "Index Type"))
	TEnumAsByte<ENeuralIndexType> NeuralIndexType;
	
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;

#endif // WITH_EDITOR

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};


// The input node that holds the output from the neural network to the post process material 
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionNeuralNetworkOutput : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** 
	*	Coordinate to store the values:
			Texture index (float4): Batch[ignored], StartChannel[int,0], ViewportUV in 0..1 range 
			Buffer  index (float4): Batch[int,0], StartChannel[int,0], WidthHeight[ViewportUV] in 0..1 range
	*/
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Coordinates;

	/**Indexing type: read from the texture or buffer. 
		Texture index is valid only when the first 2 dimension of the input and output dimension matches.
	*/
	UPROPERTY(EditAnywhere, Category = "Default", meta = (DisplayName = "Index Type"))
	TEnumAsByte<ENeuralIndexType> NeuralIndexType;

#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;

#endif // WITH_EDITOR

};


