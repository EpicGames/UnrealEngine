// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionNoise.generated.h"

UENUM()
enum ENoiseFunction : int
{
	/** High quality for direct use and bumps 
	 * ~77 instructions per level, 4 texture lookups
	 * Cannot tile
	 */
	NOISEFUNCTION_SimplexTex UMETA(DisplayName="Simplex - Texture Based"),

	/** High quality for direct use and bumps
	 * Non-tiled: ~61 instructions per level, 8 texture lookups
	 * Tiling: ~74 instructions per level, 8 texture lookups
	 * Even "non-tiled" mode has a repeat of 128. Useful Repeat Size range <= 128
	 * Formerly labeled as Perlin noise
	 */
	NOISEFUNCTION_GradientTex UMETA(DisplayName="Gradient - Texture Based"),

	/** High quality for direct use, BAD for bumps; doesn't work on Mobile
	 * ~16 instructions per level, 1 texture lookup
	 * Always tiles with a repeat of 16, "Tiling" mode is not an option for Fast Gradient noise
	 */
	NOISEFUNCTION_GradientTex3D UMETA(DisplayName = "Fast Gradient - 3D Texture"),

	/** High quality for direct use and bumps
	 * Non-tiled: ~80 instructions per level, no textures
	 * Tiling: ~143 instructions per level, no textures
	 */
	NOISEFUNCTION_GradientALU UMETA(DisplayName = "Gradient - Computational"),

	/** Low quality, but pure computation
	 * Non-tiled: ~53 instructions per level, no textures
	 * Tiling: ~118 instructions per level, no textures
	 * Formerly mis-labeled as Gradient noise
	 */
	NOISEFUNCTION_ValueALU UMETA(DisplayName="Value - Computational"),

	/** Also known as Worley or Cellular noise
	  * Quality=1 searches 8 cells, Quality=2 searches 16 cells
	  * Quality=3 searches 27 cells, Quality=4 searches 32 cells
	  * All are about 20 instructions per cell searched
	  */
	NOISEFUNCTION_VoronoiALU UMETA(DisplayName = "Voronoi"),

	NOISEFUNCTION_MAX,
};

UCLASS(MinimalAPI)
class UMaterialExpressionNoise : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** 3 dimensional vector */
	UPROPERTY()
	FExpressionInput Position;

	/** Defines the reference space for the Position input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionNoise)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** scalar, to clamp the Levels at pixel level, can be computed like this: max(length(ddx(Position)), length(ddy(Position)) */
	UPROPERTY()
	FExpressionInput FilterWidth;

	/** can also be done with a multiply on the Position */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise)
	float Scale;

	/** Lower numbers are faster and lower quality, higher numbers are slower and higher quality */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise, meta=(UIMin = "1", UIMax = "4"))
	int32 Quality;

	/** Noise function, affects performance and look */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise, meta=(DisplayName = "Function"))
	TEnumAsByte<enum ENoiseFunction> NoiseFunction;
	
	/** How multiple frequencies are getting combined */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise)
	uint32 bTurbulence:1;

	/** 1 = fast but little detail, .. larger numbers cost more performance */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise, meta=(UIMin = "1", UIMax = "10"))
	int32 Levels;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise)
	float OutputMin;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise)
	float OutputMax;

	/** usually 2 but higher values allow efficient use of few levels */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionNoise, meta=(UIMin = "2", UIMax = "8"))
	float LevelScale;

	/** Whether to use tiling noise pattern, useful for baking to seam-free repeating textures */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionNoise)
	uint32 bTiling:1;

	/** How many units in each tile (if Tiling is on) */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionNoise, meta=(UIMin = "4"))
	uint32 RepeatSize;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI)
class UMaterialExpressionScalarBlueNoise : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	// Later, we could add a 2D Position input if needed to overide pixel position

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};
