// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionNormalize.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionNormalize : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput VectorInput;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override { OutCaptions.Add(TEXT("Normalize")); }
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("normal")); }
#endif
	//~ End UMaterialExpression Interface
};



