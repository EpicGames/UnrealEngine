// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionObjectLocalBounds.generated.h"

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionObjectLocalBounds : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

	//~ End UMaterialExpression Interface

#if WITH_EDITORONLY_DATA
	TArray<FString> OutputToolTips;
#endif
#endif
};