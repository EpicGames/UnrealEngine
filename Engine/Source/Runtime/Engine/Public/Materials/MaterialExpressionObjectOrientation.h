// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionObjectOrientation.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionObjectOrientation : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



