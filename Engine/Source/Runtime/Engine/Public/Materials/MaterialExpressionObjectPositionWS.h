// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionObjectPositionWS.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionObjectPositionWS : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=UMaterialExpressionObjectPositionWS, meta=(DisplayName = "Origin", ShowAsInputPin = "Advanced"))
	EPositionOrigin OriginType;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
    virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};



