// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionOperator.generated.h"

UENUM()
enum class EMaterialExpressionOperatorKind
{
	// Unary
	BitwiseNot,
	Negate,
	Not,
	Abs,
	ACos,
	ACosFast,
	ACosh,
	ASin,
	ASinFast,
	ASinh,
	ATan,
	ATanFast,
	ATanh,
	Ceil,
	Cos,
	Cosh,
	Exponential,
	Exponential2,
	Floor,
	Frac,
	IsFinite,
	IsInf,
	IsNan,
	Length,
	Logarithm,
	Logarithm10,
	Logarithm2,
	LWCTile,
	Reciprocal,
	Round,
	Rsqrt,
	Saturate,
	Sign,
	Sin,
	Sinh,
	Sqrt,
	Tan,
	Tanh,
	Transpose,
	Truncate,

	// Binary
	Equals,
	GreaterThan,
	GreaterThanOrEquals,
	LessThan,
	LessThanOrEquals,
	NotEquals,
	And,
	Or,
	Add,
	Subtract,
	Multiply,
	MatrixMultiply,
	Divide,
	Modulo,
	BitwiseAnd,
	BitwiseOr,
	BitwiseXor,
	BitShiftLeft,
	BitShiftRight,
	ATan2,
	ATan2Fast,
	Cross,
	Distance,
	Dot,
	Fmod,
	Max,
	Min,
	Pow,
	Step,

	// Ternary
	Clamp,
	Lerp,
	Select,
	Smoothstep,
};

USTRUCT()
struct FMaterialExpressionOperatorInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FExpressionInput ExpressionInput;

	UPROPERTY(EditAnywhere, Category = "AddInput")
	float ConstValue;

	FMaterialExpressionOperatorInput()
		: ConstValue(1.0f)
	{
	}
};
 
UCLASS(MinimalAPI, meta=(NewMaterialTranslator))
class UMaterialExpressionOperator : public UMaterialExpression
{
	GENERATED_BODY()

public:
	/** Array of Inputs */
	UPROPERTY(EditAnywhere, Category = "Dynamic Inputs")
	TArray<FMaterialExpressionOperatorInput> DynamicInputs;	
	
	/** Default Operator */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionOperator)
	EMaterialExpressionOperatorKind Operator = EMaterialExpressionOperatorKind::Add;

	/** Default Number of Inputs */
	uint32 Arity = 2;

	/** If the current operator allows Add a Pin feature */
	bool bAllowAddPin = true;

	UMaterialExpressionOperator(const FObjectInitializer& ObjectInitializer);

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void AddInputPin();	
	virtual bool CanDeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) const override;
	virtual void DeletePin(EEdGraphPinDirection PinDirection, int32 PinIndex) override;	

	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual int32 CountInputs() const override;	

	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override;
	virtual FText GetCreationName() const override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;

	virtual void Build(MIR::FEmitter& Emitter) override;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



