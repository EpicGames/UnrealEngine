// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPanner.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionPanner : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinate;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to Game Time if not specified"))
	FExpressionInput Time;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Vector2 speed scale, if specified"))
	FExpressionInput Speed;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionPanner, meta = (OverridingInputProperty = "Speed"))
	float SpeedX;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionPanner, meta = (OverridingInputProperty = "Speed"))
	float SpeedY;

	/** only used if Coordinate is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionPanner, meta = (OverridingInputProperty = "Coordinate"))
	uint32 ConstCoordinate = 0;

	// Output only the fractional part of the pan calculation for greater precision.
	// Output is greater than or equal to 0 and less than 1.
	UPROPERTY(EditAnywhere, Category=MaterialExpressionPanner)
	bool bFractionalPart;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool NeedsRealtimePreview() override { return Time.Expression==NULL && (SpeedX != 0.f || SpeedY != 0.f); }
#endif
	//~ End UMaterialExpression Interface

};



