// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionParameter.generated.h"

struct FMaterialParameterInfo;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionParameter : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** The name of the parameter */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list.  The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionParameter)
	int32 SortPriority = 32;

	static FName ParameterDefaultName;


#if WITH_EDITOR

	//~ Begin UMaterialExpression Interface
	virtual bool MatchesSearchQuery( const TCHAR* SearchQuery ) override;
	virtual bool CanRenameNode() const override { return true; }
	virtual FString GetEditableName() const override;
	virtual void SetEditableName(const FString& NewName) override;

	virtual bool HasAParameterName() const override { return true; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;

	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Description = Desc;
		OutMeta.ExpressionGuid = ExpressionGUID;
		OutMeta.Group = Group;
		OutMeta.SortPriority = SortPriority;
		return true;
	}

#endif
	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const
	{
		// UMaterialExpressionParameter is not allowed, but subclasses of UMaterialExpressionParameter are allowed.
		return GetClass() != UMaterialExpressionParameter::StaticClass();
	}
	//~ End UMaterialExpression Interface

	virtual FGuid& GetParameterExpressionId() override
	{
		return ExpressionGUID;
	}
};
