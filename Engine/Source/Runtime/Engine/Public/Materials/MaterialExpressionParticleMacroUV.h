// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This UV node generates texture coordinates in view space centered on the particle system's MacroUVPosition, with tiling controlled by the particle system's MacroUVRadius.
 * It is useful for mapping a 'macro' noise texture in a continuous manner onto all particles of a particle system.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionParticleMacroUV.generated.h"

UCLASS()
class UMaterialExpressionParticleMacroUV : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



