// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionParticlePositionWS.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionParticlePositionWS : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=UMaterialExpressionParticlePositionWS, meta=(DisplayName = "Origin", ShowAsInputPin = "Advanced"))
	EPositionOrigin OriginType;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



