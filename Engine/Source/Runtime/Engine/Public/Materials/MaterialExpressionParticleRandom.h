// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	UMaterialExpressionParticleMotionBlurFade: Exposes a value used to fade
		particle sprites under the effects of motion blur.
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionParticleRandom.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionParticleRandom : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



