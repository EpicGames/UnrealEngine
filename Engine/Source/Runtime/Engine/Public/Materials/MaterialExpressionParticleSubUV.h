// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "MaterialExpressionParticleSubUV.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionParticleSubUV : public UMaterialExpressionTextureSample
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=MaterialExpressionParticleSubUV)
	uint32 bBlend:1 = true;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual int32 GetWidth() const override;	
	virtual int32 GetLabelPadding() override { return 8; }
#endif
	//~ End UMaterialExpression Interface
};



