// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPathTracingBufferTexture.generated.h"

UENUM()
enum EPathTracingBufferTextureId : int
{
	/** Radiance (Path Tracing). The raw radiance. */
	PTBT_Radiance UMETA(DisplayName = "Radiance (Path Tracing)"),
	/** Normal (Path Tracing). Stores the denoised radiance if denoising is turned on and complete for the current frame, otherwise, black. */
	PTBT_DenoisedRadiance UMETA(DisplayName = "Denoised radiance (Path Tracing)"),
	/** Albedo (Path Tracing). Average albedo at the current sample count. */
	PTBT_Albedo UMETA(DisplayName = "Albedo (Path Tracing)"),
	/** Normal (Path Tracing). Average normal at the current sample count. */
	PTBT_Normal UMETA(DisplayName = "Normal (Path Tracing)"),
	/** Variance (Path Tracing). Path tracing variance stored as standard derivation. Variance can be per channel variance or 
		variance of luminance, albedo, and normal based on the path tracing configuration. Hooking up this buffer can increase additional
		cost.
	*/
	PTBT_Variance UMETA(DisplayName = "Variance (Path Tracing)"),
};

/** Path tracing buffer is only accessable in postprocess material. */
UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionPathTracingBufferTexture : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** UV in 0..1 range */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Coordinates;

	/** Which path tracing buffer texture we want to make a lookup into.*/
	UPROPERTY(EditAnywhere, Category = UMaterialExpressionPathTracingBufferTexture, meta = (DisplayName = "Buffer Texture (Path Tracing)"))
	TEnumAsByte<EPathTracingBufferTextureId> PathTracingBufferTextureId;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
