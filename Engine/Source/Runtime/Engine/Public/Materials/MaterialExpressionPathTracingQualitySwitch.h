// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPathTracingQualitySwitch.generated.h"

UCLASS()
class UMaterialExpressionPathTracingQualitySwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Used for non-path traced modes */
	UPROPERTY()
	FExpressionInput Normal;

	/** Used by the path tracer */
	UPROPERTY()
	FExpressionInput PathTraced;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;

#endif
	//~ End UMaterialExpression Interface
};
