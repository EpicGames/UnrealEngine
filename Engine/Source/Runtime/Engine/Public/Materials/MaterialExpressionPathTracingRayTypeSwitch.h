// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPathTracingRayTypeSwitch.generated.h"

UCLASS()
class UMaterialExpressionPathTracingRayTypeSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Used for camera rays (or for non-path traced shading) */
	UPROPERTY()
	FExpressionInput Main;

	/** Used by the path tracer on shadow rays (this only applies for non-opaque blend modes) */
	UPROPERTY()
	FExpressionInput Shadow;

	/** Used by the path tracer on indirect diffuse rays */
	UPROPERTY()
	FExpressionInput IndirectDiffuse;

	/** Used by the path tracer on indirect specular rays */
	UPROPERTY()
	FExpressionInput IndirectSpecular;

	/** Used by the path tracer on indirect volume rays */
	UPROPERTY()
	FExpressionInput IndirectVolume;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Unknown; }
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
