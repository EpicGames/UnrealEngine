// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionPerInstanceCustomData.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionPerInstanceCustomData : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstDefaultValue' if not specified; Default value is used when no instances provided."))
	FExpressionInput DefaultValue;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionPerInstanceCustomData, meta = (OverridingInputProperty = "DefaultValue"))
	float ConstDefaultValue;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionPerInstanceCustomData, meta = (ShowAsInputPin = "Advanced"))
	uint32 DataIndex;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionPerInstanceCustomData3Vector : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstDefaultValue' if not specified; Default value is used when no instances provided."))
	FExpressionInput DefaultValue;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionPerInstanceCustomData, meta = (OverridingInputProperty = "DefaultValue"))
	FLinearColor ConstDefaultValue;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionPerInstanceCustomData, meta = (ShowAsInputPin = "Advanced"))
	uint32 DataIndex;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};