// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionReroute.h"
#include "EdGraph/EdGraphNode.h"

#include "MaterialExpressionPinBase.generated.h"

/**
 * Collection of pins used for tunneling between graphs.
 * Utilizes reroute expressions to ensure zero overhead in the compiled material.
 * 
 *      _________________          _________________
 *     |   INPUT BASE    |        |   OUTPUT BASE   |
 *     +--------+--------+        +--------+--------+
 *     |        |   (>)  |   ->   |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |   (>)  |        |  (>)   |        |
 *     |        |        |        |        |        |
 *     +--------+--------+        +--------+--------+
 *     | NODE IN:  NONE  |        | NODE IN:  PINS  |
 *     | NODE OUT: PINS  |        | NODE OUT: NONE  |
 *     |_________________|        |_________________|
 *
 */

USTRUCT()
struct FCompositeReroute
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=CustomInput, meta = (NoResetToDefault))
	FName Name;

	UPROPERTY(meta = (NoResetToDefault))
	TObjectPtr<UMaterialExpressionReroute> Expression;

	FCompositeReroute()
		: Name(NAME_None)
		, Expression(nullptr)
	{
	}

	FCompositeReroute(FName InName, UMaterialExpressionReroute* InExpression)
		: Name(InName)
		, Expression(InExpression)
	{
	}
};

UCLASS(MinimalAPI)
class UMaterialExpressionPinBase : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
	
	/** Underlying reroute pins used to compile material. Must call Modify after editing to update output expressions. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionPinBase, meta = (NoResetToDefault, TitleProperty = Name))
	TArray<FCompositeReroute> ReroutePins;

	/** Direction of the pins for this base. */
	UPROPERTY()
	TEnumAsByte<EEdGraphPinDirection> PinDirection;

#if WITH_EDITOR
	/** Helper function to clear all reroutes. */
	virtual void DeleteReroutePins();

	//~ Begin UObject Interface.
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
	virtual TArray<FExpressionOutput>& GetOutputs() override;
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool IsExpressionConnected(FExpressionInput* Input, int32 OutputIndex) override;
	virtual void ConnectExpression(FExpressionInput* Input, int32 OutputIndex) override;

	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif // WITH_EDITOR

	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const override { return false; }
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	//~ Begin UObject Interface.
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	//~ End UObject Interface.

	bool ModifyAsSubObject(bool bAlwaysMarkDirty = true);
private:
	/** Used to remove reroute expressions from the material after we remove them. */
	TArray<UMaterialExpressionReroute*> PreEditRereouteExpresions;

#endif // WITH_EDITOR
};



