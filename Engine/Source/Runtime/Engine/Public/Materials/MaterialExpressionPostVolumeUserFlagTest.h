// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPostVolumeUserFlagTest.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionPostVolumeUserFlagTest : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstBitIndex' if not specified"))
	FExpressionInput BitIndex;

	/** only used if Input is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionPostVolumeUserFlagTest, meta=(OverridingInputProperty = "BitIndex"))
	int32 ConstBitIndex;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("TestPostVolumeUserFlag")); }
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override { OutCaptions.Add(FString("TestPostVolumeUserFlag")); }
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
