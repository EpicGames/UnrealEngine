// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *	A material expression that allows access to the precomputed AO mask generated when bGenerateAmbientOcclusionMaterialMask is enabled in Lightmass World settings.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionPrecomputedAOMask.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionPrecomputedAOMask : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



