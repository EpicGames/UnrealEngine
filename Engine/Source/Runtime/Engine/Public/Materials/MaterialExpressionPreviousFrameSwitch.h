// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionPreviousFrameSwitch.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionPreviousFrameSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput CurrentFrame;

	UPROPERTY()
	FExpressionInput PreviousFrame;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return MCT_Unknown;}

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



