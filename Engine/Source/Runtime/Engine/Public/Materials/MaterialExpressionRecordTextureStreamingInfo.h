// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialExpression.h"
#include "MaterialExpressionRecordTextureStreamingInfo.generated.h"

/** Adds functionality to record the material UV scales for use by the automatic texture streaming system. */
UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionRecordTextureStreamingInfo : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** A texture object that will be sampled. */
	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput TextureObject;

	/** The texture coordinates that we expect to use when sampling the texture object. */
	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Coordinates;

#if WITH_EDITOR
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
#endif // WITH_EDITOR
};
