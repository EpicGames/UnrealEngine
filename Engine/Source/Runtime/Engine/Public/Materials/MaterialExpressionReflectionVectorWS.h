// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionReflectionVectorWS.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionReflectionVectorWS : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Optional world normal to reflect the camera view vector about. If unconnected, pixel normal is used */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to Reflection Vector if not specified"))
	FExpressionInput CustomWorldNormal;

	/** (true): The specified world normal will be normalized. (false): WorldNormal will just be used as is, faster but possible artifacts if normal length isn't 1 */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionReflectionVectorWS, meta=(DisplayName = "Normalize custom world normal", ShowAsInputPin = "Advanced"))
	uint32 bNormalizeCustomWorldNormal : 1;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



