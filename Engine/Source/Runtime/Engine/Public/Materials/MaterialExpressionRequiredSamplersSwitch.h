// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "RHIFeatureLevel.h"
#include "MaterialExpressionRequiredSamplersSwitch.generated.h"

UCLASS(MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionRequiredSamplersSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput InputTrue;
		
	UPROPERTY()
	FExpressionInput InputFalse;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionRequiredSamplersSwitch, meta=(UIMin = "16", UIMax = "255"))
	uint32 RequiredSamplers = 16;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Unknown; }
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
