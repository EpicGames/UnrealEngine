// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Absolute value material expression for user-defined materials
 *
 */

#pragma once
#include "Materials/MaterialExpressionRerouteBase.h"
#include "MaterialExpressionReroute.generated.h"

UCLASS(collapsecategories, hidecategories=Object, DisplayName = "Reroute", MinimalAPI)
class UMaterialExpressionReroute : public UMaterialExpressionRerouteBase
{
	GENERATED_BODY()

public:

	/** Link to the input expression to be evaluated */
	UPROPERTY()
	FExpressionInput Input;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual int32 CompilePreview(FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	ENGINE_API virtual FText GetCreationDescription() const override;
	ENGINE_API virtual FText GetCreationName() const override;
#endif
	//~ End UMaterialExpression Interface

protected:
	//~ Begin UMaterialExpressionRerouteBase Interface
	ENGINE_API virtual bool GetRerouteInput(FExpressionInput& OutInput) const override;
	//~ End UMaterialExpressionRerouteBase Interface
};
