// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"
#include "MaterialExpressionRerouteBase.generated.h"

class FMaterialExpressionKey;

UCLASS(abstract, MinimalAPI)
class UMaterialExpressionRerouteBase : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/**
	 * Trace through the graph to find the first non Reroute node connected to this input. If there is a loop for some reason, we will bail out and return nullptr.
	 *
	 * @param OutputIndex The output index of the connection that was traced back.
	 * @return The final traced material expression.
	*/
	ENGINE_API UMaterialExpression* TraceInputsToRealExpression(int32& OutputIndex) const;

	ENGINE_API FExpressionInput TraceInputsToRealInput() const;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	ENGINE_API virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	ENGINE_API virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	ENGINE_API virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	ENGINE_API void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	ENGINE_API FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface

protected:
	/**
	 * Get the reroute node input pin. We could use GetInput, but we'd loose const correctness
	 * @param	OutInput	The input. Can be left unitialized
	 * @return If the input pin is valid
	 */
	 virtual bool GetRerouteInput(FExpressionInput& OutInput) const { return false; }

private:
#if WITH_EDITOR
	/** Resolve the reroute target expression with loop-detection guards. Returns nullptr if unconnected or a cycle is detected. */
	UMaterialExpression* ResolveRerouteTarget(int32 OutputIndex, int32& OutRealOutputIndex);
#endif
	ENGINE_API FExpressionInput TraceInputsToRealExpressionInternal(TSet<FMaterialExpressionKey>& VisitedExpressions) const;
};
