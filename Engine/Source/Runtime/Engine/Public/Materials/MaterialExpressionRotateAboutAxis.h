// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionRotateAboutAxis.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionRotateAboutAxis : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput NormalizedRotationAxis;

	UPROPERTY()
	FExpressionInput RotationAngle;

	UPROPERTY()
	FExpressionInput PivotPoint;

	UPROPERTY()
	FExpressionInput Position;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionRotateAboutAxis, meta = (ShowAsInputPin = "Advanced"))
	float Period = 1.0f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface

};



