// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionRotator.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionRotator : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinate;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to Game Time if not specified"))
	FExpressionInput Time;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionRotator, meta = (ShowAsInputPin = "Advanced"))
	float CenterX = 0.5f;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionRotator, meta = (ShowAsInputPin = "Advanced"))
	float CenterY = 0.5f;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionRotator, meta = (ShowAsInputPin = "Advanced"))
	float Speed = 0.25f;

	/** only used if Coordinate is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionRotator, meta = (OverridingInputProperty = "Coordinate"))
	uint32 ConstCoordinate = 0;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool NeedsRealtimePreview() override { return Time.Expression==NULL && Speed != 0.f; }
#endif
	//~ End UMaterialExpression Interface

};



