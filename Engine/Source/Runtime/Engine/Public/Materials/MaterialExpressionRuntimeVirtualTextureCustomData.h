// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionRuntimeVirtualTextureCustomData.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionRuntimeVirtualTextureCustomData : public UMaterialExpression
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
