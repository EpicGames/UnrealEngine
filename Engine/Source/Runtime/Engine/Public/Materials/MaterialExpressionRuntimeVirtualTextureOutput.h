// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionRuntimeVirtualTextureOutput.generated.h"

/** Material output expression for writing to a runtime virtual texture. */
UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionRuntimeVirtualTextureOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Input for Base Color to output to virtual texture. */
	UPROPERTY()
	FExpressionInput BaseColor;

	/** Input for Specular to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Specular;

	/** Input for Roughness to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Roughness;

	/** Input for Surface Normal to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Normal;

	/** Input for World Height to output to virtual texture. */
	UPROPERTY()
	FExpressionInput WorldHeight;

	/** Input for Opacity value used for blending to virtual texture. */
	UPROPERTY()
	FExpressionInput Opacity;

	/** Input for Mask to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Mask;

	/** Input for World Height to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Displacement;

	/** Input for Mask to output to virtual texture. */
	UPROPERTY()
	FExpressionInput Mask4;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void Build(MIR::FEmitter& Em) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
