// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionRuntimeVirtualTextureReplace.generated.h"

/** Material output expression to switch logic for a path that renders to runtime virtual texture pages. */
UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionRuntimeVirtualTextureReplace : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Used by the default rendering passes. */
	UPROPERTY()
	FExpressionInput Default;

	/** Used by the pass that renders to a runtime virtual texture page. */
	UPROPERTY()
	FExpressionInput VirtualTextureOutput;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Unknown; }
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override { return MCT_Unknown; }
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
