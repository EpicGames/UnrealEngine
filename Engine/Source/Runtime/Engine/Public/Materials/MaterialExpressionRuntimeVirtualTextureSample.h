// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"
#include "VT/RuntimeVirtualTexture.h"
#include "MaterialExpressionRuntimeVirtualTextureSample.generated.h"

/**
 * Set how Mip levels are calculated.
 * Internally we will convert to ETextureMipValueMode which is used by internal APIs.
 */
UENUM()
enum ERuntimeVirtualTextureMipValueMode : int
{
	/* 
	 * Use default computed mip level. Takes into account UV scaling from using the WorldPosition pin.
	 */
	RVTMVM_None UMETA(DisplayName = "Default"),

	/* 
	 * Use an absolute mip level from the MipLevel pin. 
	 * 0 is full resolution.
	 */
	RVTMVM_MipLevel UMETA(DisplayName = "Mip Level"),

	/* 
	 * Bias the default computed mip level using the MipBias pin. 
	 * Negative values increase resolution.
	 */
	RVTMVM_MipBias UMETA(DisplayName = "Mip Bias"),

	/* 
	* Compute mip level from world position derivatives.
	* This is intended for cases where the value passed to the WorldPosition pin doesn't give good derivatives.
	* (For example when using a constant value from primitive world position).
	* This is deprecated. Use Derivative (World Space) instead, and pass in DDX/DDY of world space.
	 */
	RVTMVM_RecalculateDerivatives UMETA(Hidden, DisplayName = "Ignore Input WorldPosition"),

	/* 
	 * Compute mip level from explicitly provided DDX and DDY derivatives of the virtual texture UV coordinates.
	 */
	RVTMVM_DerivativeUV UMETA(DisplayName = "Derivatives (UV Space)"),

	/*
	 * Compute mip level from explicitly provided DDX and DDY derivatives of the world position.
	 */
	 RVTMVM_DerivativeWorld UMETA(DisplayName = "Derivatives (World Space)"),

	RVTMVM_MAX,
};

/**
 * Defines texture addressing behavior.
 */
UENUM()
enum ERuntimeVirtualTextureTextureAddressMode : int
{
	/* Clamp mode. */
	RVTTA_Clamp UMETA(DisplayName = "Clamp"),
	/* Wrap mode. */
	RVTTA_Wrap UMETA(DisplayName = "Wrap"),

	RVTTA_MAX,
};

enum class EVirtualTextureUnpackType
{
	None,
	BaseColorYCoCg,
	NormalBC3,
	NormalBC5,
	NormalBC3BC3,
	NormalBC5BC1,
	HeightR16,
	NormalBGR565,
	BaseColorSRGB,
	DisplacementR16,
};

struct FRuntimeVirtualTextureUnpackProperties
{
	bool bIsBaseColorValid : 1 = false;
	bool bIsSpecularValid : 1 = false;
	bool bIsRoughnessValid : 1 = false;
	bool bIsNormalValid : 1 = false;
	bool bIsWorldHeightValid : 1 = false;
	bool bIsMaskValid : 1 = false;
	bool bIsMask4Valid : 1 = false;
	bool bIsDisplacementValid : 1 = false;

	uint32 UnpackTarget = 0;
	uint32 UnpackMask = 0;
	EVirtualTextureUnpackType UnpackType = EVirtualTextureUnpackType::None;
	TArray<float, TInlineAllocator<4>> ConstantVector;
};

/** Material expression for sampling from a runtime virtual texture. */
UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionRuntimeVirtualTextureSample : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Optional UV coordinates input if we want to override standard world position based coordinates. */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Coordinates;

	/** Optional world position input to override the default world position. */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput WorldPosition;

	/** Meaning depends on MipValueMode. A single unit is one mip level.  */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput MipValue;

	/** Derivative over the X axis. Enabled only if MipValueMode is one of the derivative modes. Meaning depends on the derivative mode. */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput DDX;
	
	/** Derivative over the Y axis. Enabled only if MipValueMode is one of the derivative modes. Meaning depends on the derivative mode. */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput DDY;

	/** The virtual texture object to sample. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture)
	TObjectPtr<class URuntimeVirtualTexture> VirtualTexture;

	/** How to interpret the virtual texture contents. Note that the bound Virtual Texture should have the same setting for sampling to work correctly. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Virtual texture content"))
	ERuntimeVirtualTextureMaterialType MaterialType = ERuntimeVirtualTextureMaterialType::BaseColor;

	/** Enable page table channel packing. Note that the bound Virtual Texture should have the same setting for sampling to work correctly. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Enable packed page table"))
	bool bSinglePhysicalSpace = true;

	/** Enable sparse adaptive page tables. Note that the bound Virtual Texture should have valid adaptive virtual texture settings for sampling to work correctly. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Enable adaptive page table"))
	bool bAdaptive = false;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = "UV Coordinates")
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** Defines the texture addressing mode. */
	UPROPERTY(EditAnywhere, Category = TextureSample)
	TEnumAsByte<enum ERuntimeVirtualTextureTextureAddressMode> TextureAddressMode = RVTTA_Clamp;

	/** Defines how the mip level is calculated for the virtual texture lookup. */
	UPROPERTY(EditAnywhere, Category = TextureSample)
	TEnumAsByte<enum ERuntimeVirtualTextureMipValueMode> MipValueMode = RVTMVM_None;

	/** 
	 * Enable virtual texture feedback. 
	 * Disabling this can result in the virtual texture not reaching the correct mip level. 
	 * It should only be used in cases where we don't care about the correct mip level being resident, or some other process is maintaining the correct level.
	 */
	UPROPERTY(EditAnywhere, Category = TextureSample)
	bool bEnableFeedback = true;

	/** Init settings that affect shader compilation and need to match the current VirtualTexture */
	ENGINE_API bool InitVirtualTextureDependentSettings();

	/** Returns the sampler source mode for the respective TextureAddressMode value. */
	ESamplerSourceMode GetSamplerSourceMode() const;

protected:
	/** Initialize the output pins. */
	ENGINE_API void InitOutputs();

	//~ Begin UMaterialExpression Interface
	ENGINE_API virtual UObject* GetReferencedTexture() const override;
	virtual bool CanReferenceTexture() const { return true; }

#if WITH_EDITOR
	ENGINE_API virtual void PostLoad() override;
	FExpressionInput* GetInput(int32 InputIndex) override;
	ENGINE_API virtual FName GetInputName(int32 InputIndex) const override;
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	/** Returns true if this material expression has a valid virtual texture object that matches the parameters of this expression. */
	bool ValidateVirtualTextureParameters(FString& OutError, bool& bOutIsVirtualTextureValid) const;

	/** Returns true if this is a valid UMaterialExpressionRuntimeVirtualTextureSampleParameter. */
	bool IsParameter() const;

	bool GetRVTUnpackProperties(int32 OutputIndex, bool bIsVirtualTextureValid, FRuntimeVirtualTextureUnpackProperties& Output) const;

public:
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif
	//~ End UMaterialExpression Interface
};
