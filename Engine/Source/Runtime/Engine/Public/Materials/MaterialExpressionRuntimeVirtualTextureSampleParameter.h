// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionRuntimeVirtualTextureSample.h"
#include "MaterialExpressionRuntimeVirtualTextureSampleParameter.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionRuntimeVirtualTextureSampleParameter : public UMaterialExpressionRuntimeVirtualTextureSample
{
	GENERATED_UCLASS_BODY()

	/** Name to be referenced when we want to find and set this parameter */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list. The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	int32 SortPriority = 32;

#if WITH_EDITOR
	/** If this is the named parameter from this material expression, then set its value. */
	bool SetParameterValue(FName InParameterName, URuntimeVirtualTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual bool CanRenameNode() const override { return true; }
	virtual void SetEditableName(const FString& NewName) override;
	virtual FString GetEditableName() const override;
	virtual bool HasAParameterName() const override { return true; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = VirtualTexture;
		OutMeta.Description = Desc;
		OutMeta.ExpressionGuid = ExpressionGUID;
		OutMeta.Group = Group;
		OutMeta.SortPriority = SortPriority;
		return true;
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::RuntimeVirtualTexture)
		{
			if (SetParameterValue(Name, Meta.Value.RuntimeVirtualTexture, Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	virtual FGuid& GetParameterExpressionId() override { return ExpressionGUID; }
	//~ End UMaterialExpression Interface
};
