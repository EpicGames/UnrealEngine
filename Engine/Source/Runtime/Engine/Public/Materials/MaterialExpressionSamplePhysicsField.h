// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Field/FieldSystemTypes.h"
#include "MaterialExpressionSamplePhysicsField.generated.h"

/**
 * Material expresions to sample the global physics field
 */

UCLASS()
class UMaterialExpressionSamplePhysicsVectorField : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world position if not specified"))
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSamplePhysicsVectorField)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** Target Type to be accessed */
	UPROPERTY(EditAnywhere, Category = "Field Setup", meta = (DisplayName = "Target Type", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EFieldVectorType> FieldTarget;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};

UCLASS()
class UMaterialExpressionEvalPhysicsVectorField : public UMaterialExpressionSamplePhysicsVectorField
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS()
class UMaterialExpressionSamplePhysicsScalarField : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world position if not specified"))
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSamplePhysicsScalarField)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** Target Type to be accessed */
	UPROPERTY(EditAnywhere, Category = "Field Setup", meta = (DisplayName = "Target Type", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EFieldScalarType> FieldTarget = EFieldScalarType::Scalar_DynamicConstraint;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};

UCLASS()
class UMaterialExpressionEvalPhysicsScalarField : public UMaterialExpressionSamplePhysicsScalarField
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS()
class UMaterialExpressionSamplePhysicsIntegerField : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to current world position if not specified"))
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSamplePhysicsIntegerField)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** Target Type to be accessed */
	UPROPERTY(EditAnywhere, Category = "Field Setup", meta = (DisplayName = "Target Type", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EFieldIntegerType> FieldTarget = EFieldIntegerType::Integer_DynamicState;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};


UCLASS()
class UMaterialExpressionEvalPhysicsIntegerField : public UMaterialExpressionSamplePhysicsIntegerField
{
	GENERATED_BODY()

public:
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
