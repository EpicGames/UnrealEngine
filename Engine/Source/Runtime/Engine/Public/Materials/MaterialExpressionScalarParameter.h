// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Materials/MaterialExpressionParameter.h"
#include "MaterialExpressionScalarParameter.generated.h"

class IMaterialEnumerationProvider;

UENUM()
enum class EMaterialScalarParameterControlType
{
	/** Use the default numeric entry. */
	Numeric,
	/** Use a fixed enumeration combo box. */
	Enumeration,
	/** Use an enumeration combo box where the enumeration type is set by the material instance. */
	EnumerationIndex,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionScalarParameter : public UMaterialExpressionParameter
{
	GENERATED_UCLASS_BODY()

	/** The default parameter value. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionScalarParameter, meta = (ShowAsInputPin = "Primary"))
	float DefaultValue;

	/** UI control type used for setting values in the material instance editor. */
	UPROPERTY(EditAnywhere, Category = UI, meta = (EditCondition = "!bUseCustomPrimitiveData"))
	EMaterialScalarParameterControlType ControlType = EMaterialScalarParameterControlType::Numeric;

	/** Sets the lower bound for this parameter in the material instance editor. */
	UPROPERTY(EditAnywhere, Category = UI, meta = (EditCondition = "!bUseCustomPrimitiveData && ControlType == EMaterialScalarParameterControlType::Numeric"))
	float SliderMin = 0.f;

	/**
	 * Sets the upper bound for this parameter in the material instance editor.
	 * The bounds will be disabled if SliderMax <= SliderMin.
	 */
	UPROPERTY(EditAnywhere, Category = UI, meta = (EditCondition = "ControlType == EMaterialScalarParameterControlType::Numeric"))
	float SliderMax = 0.f;

	/** Enumeration object to use in the material instance editor. */
	UPROPERTY(EditAnywhere, Category = UI, meta = (EditCondition = "ControlType == EMaterialScalarParameterControlType::Enumeration", DisplayThumbnail = "false", AllowedClasses = "/Script/CoreUObject.Enum, /Script/Engine.MaterialEnumerationProvider"))
	TSoftObjectPtr<UObject> Enumeration;

	/** Index of the enumeration to use from the material instance Enumeration Objects. */
	UPROPERTY(EditAnywhere, Category = UI, meta = (EditCondition = "ControlType == EMaterialScalarParameterControlType::EnumerationIndex", ClampMin = "0"))
	uint8 EnumerationIndex = 0;

	/** True if this parameter value provided by custom data on the primitive. */
	UPROPERTY(EditAnywhere, Category = CustomPrimitiveData)
	bool bUseCustomPrimitiveData = false;

	/** The slot index for custom primitive data. */
	UPROPERTY(EditAnywhere, Category = CustomPrimitiveData, meta = (ClampMin = "0", EditCondition = "bUseCustomPrimitiveData"))
	uint8 PrimitiveDataIndex = 0;

#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override;
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override;
	virtual bool HasClassAndNameCollision(UMaterialExpression* OtherExpression) const override;
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	//~ End UMaterialExpression Interface

	bool SetParameterValue(FName InParameterName, float InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif
	
	/** Used to override behavior in derived curve atlas parameter types. */
	virtual bool IsUsedAsAtlasPosition() const { return false; }
};
