// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "MaterialExpressionSceneDepth.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionSceneDepth : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** 
	* Coordinates - UV coordinates to apply to the scene depth lookup.
	* OffsetFraction - An offset to apply to the scene depth lookup in a 2d fraction of the screen.
	*/ 
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSceneDepth, meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EMaterialSceneAttributeInputMode::Type> InputMode;

	/**
	* Based on the input mode the input will be treated as either:
	* UV coordinates to apply to the scene depth lookup or 
	* an offset to apply to the scene depth lookup, in a 2d fraction of the screen.
	*/
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstInput' if not specified"))
	FExpressionInput Input;

	UPROPERTY()
	FExpressionInput Coordinates_DEPRECATED;

	/** only used if Input is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSceneDepth, meta = (OverridingInputProperty = "Input"))
	FVector2D ConstInput = FVector2D(0.f, 0.f);

	//~ Begin UObject Interface.
	virtual void PostLoad() override;
	//~ End UObject Interface.

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
#endif
	//~ End UMaterialExpression Interface
};



