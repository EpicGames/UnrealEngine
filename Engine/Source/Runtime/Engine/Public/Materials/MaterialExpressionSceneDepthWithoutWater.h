// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionSceneColor.h"
#include "MaterialExpressionSceneDepthWithoutWater.generated.h"

UCLASS(collapsecategories, hidecategories = Object)
class UMaterialExpressionSceneDepthWithoutWater : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UMaterialExpressionSceneDepthWithoutWater();

	/**
	* Coordinates - UV coordinates to apply to the scene depth lookup.
	* OffsetFraction - An offset to apply to the scene depth lookup in a 2d fraction of the screen.
	*/
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSceneDepthWithoutWater, meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EMaterialSceneAttributeInputMode::Type> InputMode;

	/**
	* Based on the input mode the input will be treated as either:
	* UV coordinates to apply to the scene depth lookup or
	* an offset to apply to the scene depth lookup, in a 2d fraction of the screen.
	*/
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstInput' if not specified"))
	FExpressionInput Input;

	/** only used if Input is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSceneDepthWithoutWater, meta = (OverridingInputProperty = "Input"))
	FVector2D ConstInput;

	/** Depth to fall back to in case the needed texture isn't available on a particular platform or configuration */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSceneDepthWithoutWater, meta = (ShowAsInputPin = "Advanced"))
	float FallbackDepth = 1000000.0f;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;

#endif
	//~ End UMaterialExpression Interface
};



