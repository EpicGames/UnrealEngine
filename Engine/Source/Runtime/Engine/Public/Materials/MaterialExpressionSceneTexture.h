// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialSceneTextureId.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSceneTexture.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionSceneTexture : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** UV in 0..1 range */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Coordinates;

	/** Which scene texture (screen aligned texture) we want to make a lookup into */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionSceneTexture, meta=(DisplayName = "Scene Texture Id", ShowAsInputPin = "Advanced"))
	TEnumAsByte<ESceneTextureId> SceneTextureId;
	
	/** Whether to use point sampled texture lookup (default) or using [bi-linear] filtered (can be slower, avoid faceted lock with distortions), some SceneTextures cannot be filtered */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionSceneTexture, meta=(DisplayName = "Filtered", ShowAsInputPin = "Advanced"))
	bool bFiltered = false;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};

