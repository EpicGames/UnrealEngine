// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSetMaterialAttributes.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionSetMaterialAttributes : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FExpressionInput> Inputs;

	UPROPERTY(EditAnywhere, Category=MaterialAttributes)
	TArray<FGuid> AttributeSetTypes;
#if WITH_EDITOR
	TArray<FGuid> PreEditAttributeSetTypes;
#endif

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual TArrayView<FExpressionInput*> GetInputsView()override;
	virtual FExpressionInput* GetInput(int32 InputIndex)override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return true;}
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override {return true;}
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;

	ENGINE_API int32 CreateOrGetInputAttribute(EMaterialProperty Attribute);
	ENGINE_API bool ConnectInputAttribute(EMaterialProperty Attribute, UMaterialExpression* Expression, int32 OutputIndex = 0);
	bool GetSubstrateMaterialInputIndex(int32 OutputIndex, int32& InputIndex);
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

	uint64 GetConnectedInputs() const;
#endif
	//~ End UMaterialExpression Interface
};
