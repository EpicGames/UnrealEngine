// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "RHIDefinitions.h"
#include "MaterialExpressionShaderStageSwitch.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionShaderStageSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	static inline bool ShouldUsePixelShaderInput(EShaderFrequency InShaderFrequency)
	{
		// Compute is considered pixel shader here, as the majority of material behavior that depends on pixel shader also allows compute shader
		return (InShaderFrequency == SF_Pixel || InShaderFrequency == SF_Compute);
	}

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used in pixel (or compute) shader stages"))
	FExpressionInput PixelShader;

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be in vertex (or tessellation) shader stages"))
	FExpressionInput VertexShader;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("vertex pixel shader")); }

#endif
	//~ End UMaterialExpression Interface
};
