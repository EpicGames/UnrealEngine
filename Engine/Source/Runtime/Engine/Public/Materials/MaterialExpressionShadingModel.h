// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionShadingModel.generated.h"

class FMaterialCompiler;

/**
 * Compile a select "blend" between ShadingModels
 *
 * @param Compiler				The compiler to add code to
 * @param A						Select A if Alpha is less than 0.5f
 * @param B						Select B if Alpha is greater or equal to 0.5f
 * @param Alpha					Bland factor [0..1]
 * @return						Index to a new code chunk
 */
ENGINE_API extern int32 CompileShadingModelBlendFunction(FMaterialCompiler* Compiler, int32 A, int32 B, int32 Alpha);

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionShadingModel : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
public:
	UPROPERTY(EditAnywhere, Category=ShadingModel,  meta=(ValidEnumValues="MSM_DefaultLit, MSM_Subsurface, MSM_PreintegratedSkin, MSM_ClearCoat, MSM_SubsurfaceProfile, MSM_TwoSidedFoliage, MSM_Hair, MSM_Cloth, MSM_Eye", ShowAsInputPin = "Primary"))
	TEnumAsByte<enum EMaterialShadingModel> ShadingModel = MSM_DefaultLit;
	//~ End UMaterialExpression Interface
};