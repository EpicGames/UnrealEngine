// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "RHIDefinitions.h"
#include "MaterialExpressionShadingPathSwitch.generated.h"

enum EShaderPlatform : uint16;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionShadingPathSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** Default connection, used when a specific shading type input is missing. */
	UPROPERTY()
	FExpressionInput Default;

	UPROPERTY()
	FExpressionInput Inputs[ERHIShadingPath::Num];

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_Unknown;}
	virtual EMaterialValueType GetOutputValueType(int32 InputIndex) override {return MCT_Unknown;}

	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
protected:
	ERHIShadingPath::Type GetShadingPathToCompile(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type MaterialQualityLevel);
	FExpressionInput* GetEffectiveInput(class FMaterialCompiler* Compiler);
#endif
};
