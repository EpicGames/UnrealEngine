// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionShadowReplace.generated.h"

UCLASS(meta = (DisplayName = "ShadowPassSwitch"))
class UMaterialExpressionShadowReplace : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when rendering into non-ShadowMap passes"))
	FExpressionInput Default;

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Input will be used when rendering into ShadowMap"))
	FExpressionInput Shadow;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

#endif
	//~ End UMaterialExpression Interface
};
