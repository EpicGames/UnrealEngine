// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionSingleLayerWaterMaterialOutput.generated.h"

/** Material output expression for writing single layer water volume material properties. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionSingleLayerWaterMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Input for scattering coefficient describing how light scatter around and is absorbed. Valid range is [0,+inf[. Unit is 1/cm. */
	UPROPERTY()
	FExpressionInput ScatteringCoefficients;

	/** Input for scattering coefficient describing how light bounce is absorbed. Valid range is [0,+inf[. Unit is 1/cm. */
	UPROPERTY()
	FExpressionInput AbsorptionCoefficients;
		
	/** Input for phase function 'g' parameter describing how much forward(g>0) or backward (g<0) light scatter around. Valid range is [-1,1]. */
	UPROPERTY()
	FExpressionInput PhaseG;

	/** Input for custom color multiplier for scene color behind water. Can be used for caustics textures etc. Defaults to 1.0. Valid range is [0,+inf[. */
	UPROPERTY()
	FExpressionInput ColorScaleBehindWater;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
