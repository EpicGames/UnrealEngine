// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionSkyAtmosphereLightDirection.generated.h"

UCLASS()
class UMaterialExpressionSkyAtmosphereLightDirection : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	/** Index of the atmosphere light to sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTextureCoordinate, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", ShowAsInputPin = "Primary"))
	int32 LightIndex;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};


