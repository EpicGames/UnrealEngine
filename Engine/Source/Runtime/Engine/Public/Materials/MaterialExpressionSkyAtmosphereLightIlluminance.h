// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionSkyAtmosphereLightIlluminance.generated.h"

UCLASS()
class UMaterialExpressionSkyAtmosphereLightIlluminance : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	/** Index of the atmosphere light to sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTextureCoordinate, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", ShowAsInputPin = "Primary"))
	int32 LightIndex;

	/** World position of the sample. If not specified, the pixel world position will be used. */
	UPROPERTY()
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSkyAtmosphereLightIlluminance)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};



UCLASS()
class UMaterialExpressionSkyAtmosphereLightIlluminanceOnGround : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	/** Index of the atmosphere light to sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTextureCoordinate, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", ShowAsInputPin = "Primary"))
	int32 LightIndex;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;

#endif
	//~ End UMaterialExpression Interface
};



UCLASS()
class UMaterialExpressionSkyAtmosphereLightDiskLuminance : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	/** Index of the atmosphere light to sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTextureCoordinate, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", ShowAsInputPin = "Primary"))
	int32 LightIndex;

	/** Override the angular diameter of the disk in degree. If not specified, the radius specified on the directional light will be used. This can be used to decouple the directional light visual disk size used for the specular disk reflection on surfaces. However, be aware that screen space reflections will still catch the visual disk. */
	UPROPERTY()
	FExpressionInput DiskAngularDiameterOverride;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



UCLASS()
class UMaterialExpressionSkyAtmosphereAerialPerspective : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	/** World position of the sample. If not specified, the pixel world position will be used. Larger distance will result in more fog. Please make sure .SkyAtmosphere.AerialPerspectiveLUT.Depth is set far enough to have fog data.
		If you are scaling the sky dome pixel world position, make sure it is centered around the origin.*/
	UPROPERTY()
	FExpressionInput WorldPosition;

	/** Defines the reference space for the WorldPosition input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSkyAtmosphereAerialPerspective)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



UCLASS()
class UMaterialExpressionSkyAtmosphereDistantLightScatteredLuminance : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};


