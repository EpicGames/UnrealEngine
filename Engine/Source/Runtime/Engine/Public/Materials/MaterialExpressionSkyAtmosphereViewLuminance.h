// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionSkyAtmosphereViewLuminance.generated.h"

UCLASS()
class UMaterialExpressionSkyAtmosphereViewLuminance : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

	/** Sample the sky atmosphere using WorldDirection instead of the default camera-to-pixel V direction. */
	UPROPERTY()
	FExpressionInput WorldDirection;

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};


