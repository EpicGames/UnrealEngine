// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionSkyLightEnvMapSample.generated.h"

UCLASS()
class UMaterialExpressionSkyLightEnvMapSample : public UMaterialExpressionExternalCodeBase
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "The direction to sample the cubemap from"))
	FExpressionInput Direction;
	
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "The roughness used to specifiy the sharpness of the sample."))
	FExpressionInput Roughness;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};


