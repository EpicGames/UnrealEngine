// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"

#include "MaterialExpressionSmoothStep.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionSmoothStep : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstMin' if not specified"))
	FExpressionInput Min;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstMax' if not specified"))
	FExpressionInput Max;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstValue' if not specified"))
	FExpressionInput Value;

	/** only used if Min is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSmoothStep, meta = (OverridingInputProperty = "Min"))
	float ConstMin = 0.0f;

	/** only used if Max is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSmoothStep, meta = (OverridingInputProperty = "Max"))
	float ConstMax = 1.0f;

	/** only used if Value is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSmoothStep, meta = (OverridingInputProperty = "Value"))
	float ConstValue = 0.0f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("SmoothStep")); }
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("SmoothStep")); }

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
