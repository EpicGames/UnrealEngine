// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSobol.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionSobol : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** 2D integer cell in 256x256 grid.
	  * Uses cell (0,0) if not connected */
	UPROPERTY()
	FExpressionInput Cell;

	UPROPERTY(meta = (
		RequiredInput = "false", 
		Tooltip="Sobol point number. Use Const Index if not connected."))
	FExpressionInput Index;

	UPROPERTY(meta = (
		RequiredInput = "false",
		Tooltip="2D Seed for sequence randomization (0,0)-(1,1). Use Const Seed if not connected."))
	FExpressionInput Seed;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionSobol, meta = (
		OverridingInputProperty = "Index", 
		Tooltip="Sobol point number. Only used if Index is not connected."))
	uint32 ConstIndex = 0;
	
	UPROPERTY(EditAnywhere, Category = MaterialExpressionSobol, meta = (
		OverridingInputProperty = "Seed",
		Tooltip="2D Seed for sequence randomization. Only used if Seed is not connected."))
	FVector2D ConstSeed = FVector2D(0.f, 0.f);

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



