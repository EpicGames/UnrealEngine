// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialParameters.h"

#include "MaterialExpressionSparseVolumeTextureBase.generated.h"

class USparseVolumeTexture;
struct FMaterialParameterMetadata;

UCLASS(abstract, hidecategories = Object, MinimalAPI)
class UMaterialExpressionSparseVolumeTextureBase : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** The Sparse Volume Texture to sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SparseVolumeTexture)
	TObjectPtr<USparseVolumeTexture> SparseVolumeTexture;

	//~ Begin UMaterialExpression Interface
	ENGINE_API virtual UObject* GetReferencedTexture() const override;
	virtual bool CanReferenceTexture() const { return true; }
	//~ End UMaterialExpression Interface
};
