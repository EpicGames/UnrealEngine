// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionSparseVolumeTextureBase.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"

#include "MaterialExpressionSparseVolumeTextureObject.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionSparseVolumeTextureObject : public UMaterialExpressionSparseVolumeTextureBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};

UCLASS(collapsecategories, hidecategories = (Object, MaterialExpressionTextureSample), MinimalAPI)
class UMaterialExpressionSparseVolumeTextureObjectParameter : public UMaterialExpressionSparseVolumeTextureSampleParameter
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};
