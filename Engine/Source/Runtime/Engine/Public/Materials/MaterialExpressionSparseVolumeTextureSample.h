// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionSparseVolumeTextureBase.h"

#include "MaterialExpressionSparseVolumeTextureSample.generated.h"

class USparseVolumeTexture;
struct FMaterialParameterMetadata;

/** Material expression for sampling from a runtime virtual texture. */
UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionSparseVolumeTextureSample : public UMaterialExpressionSparseVolumeTextureBase
{
	GENERATED_UCLASS_BODY()

	/** 3D texture coordinate used to sample the sparse volume texture. */
	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput Coordinates;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'SparseVolumeTexture' if not specified"))
	FExpressionInput TextureObject;

	/** Meaning depends on MipValueMode, a single unit is one mip level  */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 0 if not specified"))
	FExpressionInput MipValue;

	/** Enabled only if MipValueMode == TMVM_Derivative */
	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Coordinates derivative over the X axis"))
	FExpressionInput CoordinatesDX;

	/** Enabled only if MipValueMode == TMVM_Derivative */
	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Coordinates derivative over the Y axis"))
	FExpressionInput CoordinatesDY;

	/** Defines how the MipValue property is applied to the texture lookup */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample, meta = (DisplayName = "MipValueMode", ShowAsInputPin = "Advanced"))
	TEnumAsByte<ETextureMipValueMode> MipValueMode;

	/**
	 * Controls where the sampler for this texture lookup will come from.
	 * Choose 'from texture asset' to make use of the USparseVolumeTexture addressing settings,
	 * Otherwise use one of the global samplers, which will not consume a sampler slot.
	 * This allows materials to use more than 16 unique textures on SM5 platforms.
	 */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample, Meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<ESamplerSourceMode> SamplerSource;

	/** Only used if MipValue is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample)
	int32 ConstMipValue;

protected:

#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	ENGINE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	ENGINE_API virtual FName GetInputName(int32 InputIndex) const override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	ENGINE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
public:
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UMaterialExpression Interface
};


UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionSparseVolumeTextureSampleParameter : public UMaterialExpressionSparseVolumeTextureSample
{
	GENERATED_UCLASS_BODY()

	/** Name to be referenced when we want to find and set this parameter */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list. The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category = MaterialParameter)
	int32 SortPriority = 32;

#if WITH_EDITOR
	/** If this is the named parameter from this material expression, then set its value. */
	bool SetParameterValue(FName InParameterName, USparseVolumeTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual bool CanRenameNode() const override { return true; }
	virtual void SetEditableName(const FString& NewName) override { ParameterName = *NewName; }
	virtual FString GetEditableName() const override { return ParameterName.ToString(); }
	virtual bool HasAParameterName() const override { return true; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override;
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override;
#endif
	virtual FGuid& GetParameterExpressionId() override { return ExpressionGUID; }
	//~ End UMaterialExpression Interface
};
