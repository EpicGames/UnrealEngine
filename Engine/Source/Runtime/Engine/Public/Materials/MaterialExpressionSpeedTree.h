// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSpeedTree.generated.h"

UENUM()
enum ESpeedTreeGeometryType : int
{
	STG_Branch		UMETA(DisplayName="Branch"),
	STG_Frond		UMETA(DisplayName="Frond"),
	STG_Leaf		UMETA(DisplayName="Leaf"),
	STG_FacingLeaf	UMETA(DisplayName="Facing Leaf"),
	STG_Billboard	UMETA(DisplayName="Billboard")
};

UENUM()
enum ESpeedTreeWindType : int
{
	STW_None		UMETA(DisplayName="None"),
	STW_Fastest		UMETA(DisplayName="Fastest"),
	STW_Fast		UMETA(DisplayName="Fast"),
	STW_Better		UMETA(DisplayName="Better"),
	STW_Best		UMETA(DisplayName="Best"),
	STW_Palm		UMETA(DisplayName="Palm"),
	STW_BestPlus	UMETA(DisplayName="BestPlus"),
};

UENUM()
enum ESpeedTreeLODType : int
{
	STLOD_Pop		UMETA(DisplayName="Pop"),
	STLOD_Smooth	UMETA(DisplayName="Smooth")
};


UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionSpeedTree : public UMaterialExpression
{
	GENERATED_BODY()	

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'WindType' if not specified"))
	FExpressionInput GeometryInput;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'GeometryType' if not specified"))
	FExpressionInput WindInput;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'LODType' if not specified"))
	FExpressionInput LODInput;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "An extra bending of the tree for local effects"))
	FExpressionInput ExtraBendWS;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionSpeedTree, meta=(DisplayName = "Geometry Type", ToolTip="The type of SpeedTree geometry on which this material will be used"))
	TEnumAsByte<enum ESpeedTreeGeometryType> GeometryType = STG_Branch;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionSpeedTree, meta=(DisplayName = "Wind Type", ToolTip="The type of wind effect used on this tree. This can only go as high as it was in the SpeedTree Modeler, but you can set it to a lower option for lower quality wind and faster rendering."))
	TEnumAsByte<enum ESpeedTreeWindType> WindType = STW_None;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionSpeedTree, meta=(DisplayName = "LOD Type", ToolTip="The type of LOD to use"))
	TEnumAsByte<enum ESpeedTreeLODType> LODType = STLOD_Pop;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionSpeedTree, meta=(DisplayName = "Billboard Threshold", ToolTip="The threshold for triangles to be removed from the bilboard mesh when not facing the camera (0 = none pass, 1 = all pass).", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float BillboardThreshold = 0.9f;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionSpeedTree, AdvancedDisplay, meta=(DisplayName = "Accurate Wind Velocities", ToolTip="Support accurate velocities from wind. This will incur extra cost per vertex."))
	bool bAccurateWindVelocities = false;

	//~ Begin UObject Interface
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
