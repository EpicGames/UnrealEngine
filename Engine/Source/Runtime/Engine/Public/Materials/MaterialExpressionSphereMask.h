// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSphereMask.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionSphereMask : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** 1 to 4 dimensional vector, should be the same type as B */
	UPROPERTY()
	FExpressionInput A;

	/** 1 to 4 dimensional vector, should be the same type as A */
	UPROPERTY()
	FExpressionInput B;

	/** in the units that A and B are measured, if not hooked up the internal constant is used */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'AttenuationRadius' if not specified"))
	FExpressionInput Radius;

	/** 0..1 for the range of 0\% to 100\%, if not hooked up the internal constant is used */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'HardnessPercent' if not specified"))
	FExpressionInput Hardness;

	/** in the unit that A and B are measured */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSphereMask, meta=(OverridingInputProperty = "Radius"), DisplayName = "Radius")
	float AttenuationRadius = 256.0f;

	/** in percent 0%=soft .. 100%=hard */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSphereMask, meta=(UIMin = "0.0", UIMax = "100.0", ClampMin = "0.0", ClampMax = "100.0", OverridingInputProperty = "Hardness", DisplayName = "Hardness"))
	float HardnessPercent = 100.0f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("sphere")); }
#endif
	//~ End UMaterialExpression Interface

};



