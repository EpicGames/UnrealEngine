// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionSphericalParticleOpacity.generated.h"

UCLASS(collapsecategories, hidecategories=Object)
class UMaterialExpressionSphericalParticleOpacity : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()

	/** Density of the particle sphere. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstantDensity' if not specified"))
	FExpressionInput Density;

	/** Constant density of the particle sphere.  Will be overridden if Density is connected. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSphericalParticleOpacity, meta=(OverridingInputProperty = "Density"))
	float ConstantDensity = 1;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override
	{
		OutCaptions.Add(TEXT("Spherical Particle Opacity"));
	}
#endif
	//~ End UMaterialExpression Interface
};



