// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionParameter.h"
#include "MaterialExpressionStaticBoolParameter.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionStaticBoolParameter : public UMaterialExpressionParameter
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticBoolParameter, meta = (ShowAsInputPin = "Primary"))
	uint32 DefaultValue:1;

	/**Change Parameter from "static bool" to (dynamic) bool type which enables it to be used with dynamic branching*/
	UPROPERTY()
	uint32 DynamicBranch:1;

public:
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override {return DynamicBranch ? MCT_Bool : MCT_StaticBool;}
	virtual bool IsStaticExpression() const override { return DynamicBranch == 0; }
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = (bool)DefaultValue;
		OutMeta.bDynamicSwitchParameter = DynamicBranch;
		return Super::GetParameterValue(OutMeta);
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::StaticSwitch)
		{
			if (SetParameterValue(Name, Meta.Value.AsStaticSwitch(), Meta.ExpressionGuid, Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	bool SetParameterValue(FName InParameterName, bool OutValue, FGuid InExpressionGuid, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif
};



