// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpressionParameter.h"
#include "MaterialExpressionStaticComponentMaskParameter.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionStaticComponentMaskParameter : public UMaterialExpressionParameter
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticComponentMaskParameter, meta = (ShowAsInputPin = "Advanced"))
	uint32 DefaultR:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticComponentMaskParameter, meta = (ShowAsInputPin = "Advanced"))
	uint32 DefaultG:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticComponentMaskParameter, meta = (ShowAsInputPin = "Advanced"))
	uint32 DefaultB:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticComponentMaskParameter, meta = (ShowAsInputPin = "Advanced"))
	uint32 DefaultA:1;


public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual bool IsStaticExpression() const override { return true; }
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = FMaterialParameterValue(DefaultR, DefaultG, DefaultB, DefaultA);
		return Super::GetParameterValue(OutMeta);
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::StaticComponentMask)
		{
			if (SetParameterValue(Name,
				Meta.Value.Bool[0],
				Meta.Value.Bool[1],
				Meta.Value.Bool[2],
				Meta.Value.Bool[3],
				Meta.ExpressionGuid,
				Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	bool SetParameterValue(FName InParameterName, bool InR, bool InG, bool InB, bool InA, FGuid InExpressionGuid, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif
};



