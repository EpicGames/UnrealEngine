// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionStaticSwitch.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI, meta=(MIR_OnDemandInputRequest))
class UMaterialExpressionStaticSwitch : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=MaterialExpressionStaticSwitch, meta=(OverridingInputProperty = "Value"))
	uint32 DefaultValue:1;

	UPROPERTY()
	FExpressionInput A;

	UPROPERTY()
	FExpressionInput B;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Value;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsStaticExpression() const override { return true; }

	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
protected:
	FExpressionInput* GetEffectiveInput(class FMaterialCompiler* Compiler);
#endif
};



