// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"

#include "MaterialExpressionStep.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionStep : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstY' if not specified"))
	FExpressionInput Y;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstX' if not specified"))
	FExpressionInput X;

	/** only used if Y is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionStep, meta = (OverridingInputProperty = "Y"))
	float ConstY = 0.0f;

	/** only used if X is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionStep, meta = (OverridingInputProperty = "X"))
	float ConstX = 1.0f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("Step")); }
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("Step")); }

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};
