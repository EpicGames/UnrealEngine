// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Materials/MaterialExpression.h"
#include "SubstrateDefinitions.h"
#include "MaterialCompiler.h"
#include "MaterialExpressionSubstrate.generated.h"

class FMaterialCompiler;
struct FSubstrateMaterialComplexity;
struct FSubstrateOperator;

namespace MIR
{
	struct FValueRef;
};

UENUM()
enum EMaterialSubSurfaceType : int
{
	MSS_None				= SSS_TYPE_NONE 				UMETA(DisplayName="None"),
	MSS_Wrap				= SSS_TYPE_WRAP 				UMETA(DisplayName="Wrap", ToolTip="Approximation using wrap-lighting. "),
	MSS_TwoSidedWrap		= SSS_TYPE_TWO_SIDED_WRAP		UMETA(DisplayName="Two-Sided Wrap", ToolTip="Approximation using wrap-lighting and handling thin surface (e.g.: foliage)"),
	MSS_Diffusion			= SSS_TYPE_DIFFUSION 			UMETA(DisplayName="Diffusion", ToolTip="Diffusion based sub-surface scattering"),
	MSS_DiffusionProfile	= SSS_TYPE_DIFFUSION_PROFILE 	UMETA(Hidden),
	MSS_SimpleVolume		= SSS_TYPE_SIMPLEVOLUME 		UMETA(DisplayName="Simple Volume", ToolTip="Approximation of optically thin slab (e.g.: glass) where light is visible through the material"),
	MSS_TransmittanceShadow	= SSS_TYPE_TRANSMITTANCE_SHADOW	UMETA(Hidden),	// Only used from shaders as of today and might be refactor so not added here explicitely in UI.
	MSS_MAX
};

EMaterialSubSurfaceType SubstrateMergeSubSurfaceType(EMaterialSubSurfaceType A, EMaterialSubSurfaceType B);

///////////////////////////////////////////////////////////////////////////////
// Functions

#if WITH_EDITOR

void AssignOperatorIndexIfNotNull(int32& NextOperatorPin, FSubstrateOperator* Operator);

void CombineFlagForParameterBlending(FSubstrateOperator& DstOp, FSubstrateOperator* OpA, FSubstrateOperator* OpB = nullptr);

#endif

UE_DEPRECATED(5.8, "CompileSubstrateBlendFunction is deprecated and will be removed. Material attributes are no longer allowed to flow SubstrateData.")
extern int32 CompileSubstrateBlendFunction(FMaterialCompiler* Compiler, const int32 A, const int32 B, const int32 Alpha);



///////////////////////////////////////////////////////////////////////////////
// BSDF nodes

// UMaterialExpressionSubstrateBSDF can only be used for Substrate nodes ouputing SubstrateData that would need a preview,
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, Abstract, DisplayName = "Substrate Expression")
class UMaterialExpressionSubstrateBSDF : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
#endif

	float DielectricSpecularToF0(float SpecularIn)
	{
		return 0.08f * SpecularIn;
	};
};


UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Shading Models")
class UMaterialExpressionSubstrateShadingModels : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Defines the overall color of the Material. (type = float3, unit = unitless, defaults to 0.18)
	 */
	UPROPERTY()
	FExpressionInput BaseColor;

	/**
	 * Controls how \"metal-like\" your surface looks like. 0 means dielectric, 1 means conductor (type = float, unit = unitless, defaults to 0)
	 */
	UPROPERTY()
	FExpressionInput Metallic;

	/**
	 * Used to scale the current amount of specularity on non-metallic surfaces and is a value between 0 and 1 (type = float, unit = unitless, defaults to plastic 0.5)
	 */
	UPROPERTY()
	FExpressionInput Specular;
	
	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. When using anisotropy, it is the roughness used along the Tangent axis. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;
		
	/**
	 * Controls the anisotropy factor of the roughness. Positive value elongates the specular lobe along the Tangent vector, Negative value elongates the specular lobe along the perpendicular of the Tangent. (type = float, unit = unitless).
	 */
	UPROPERTY()
	FExpressionInput Anisotropy;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, default = 0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * Take the surface normal as input. The normal is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex normal)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	* Take a surface tangent as input. The tangent is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex tangent)
	*/
	UPROPERTY()
	FExpressionInput Tangent;

	/**
	 * Scale the mean free path radius of the SSS profile according to a value between 0 and 1. Always used, when a subsurface profile is provided or not. (type = float, unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput SubSurfaceColor;

	/**
	 * Coverage of the clear coat layer. (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput ClearCoat;

	/**
	 * Roughness of the top clear coat layer. (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput ClearCoatRoughness;

	/**
	 * Opacity of the material
	 */
	UPROPERTY()
	FExpressionInput Opacity;

	/**
	 * The amount of transmitted light from the back side of the surface to the front side of the surface (type = float3, unit = unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput TransmittanceColor;

		/**
	* The single scattering Albedo defining the overall color of the Material (type = float3, unit = unitless, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterScatteringCoefficients;

	/**
	 * The rate at which light is absorbed or out-scattered by the medium. Mean Free Path = 1 / Extinction. (type = float3, unit = 1/cm, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterAbsorptionCoefficients;

	/**
	 * Anisotropy of the volume with values lower than 0 representing back-scattering, equal 0 representing isotropic scattering and greater than 0 representing forward scattering. (type = float, unit = unitless, defaults to 0)
	 */
	UPROPERTY()
	FExpressionInput WaterPhaseG;

	/**
	 * A scale to apply on the scene color behind the water surface. It can be used to approximate caustics for instance. (type = float3, unit = unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput ColorScaleBehindWater;

	/**
	 * The iris or clear coat bottom normal. (type = float3, unit = unitless, defaults to vertex normal)
	 */
	UPROPERTY()
	FExpressionInput ClearCoatNormal;

	/**
	 * Take the tangent output node as input. The tangent is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex tangent)
	 */
	UPROPERTY()
	FExpressionInput CustomTangent;

	/**
	 * Shading models
	 */
	UPROPERTY()
	FShadingModelMaterialInput ShadingModel;

	/**
	 * The coverage of the surface using a thin translucent shading model. This will reduce the visibility of the thin translucent surface & plastic/metal BRDF overall. (type = float1, unit = unitless, dafaults to 1, range is [0,1])
	 */
	UPROPERTY()
	FExpressionInput ThinTranslucentSurfaceCoverage;

	// Always show at the bottom of the pin list
	UPROPERTY(EditAnywhere, Category = ShadingModel, meta = (ShowAsInputPin = "Primary", DisplayName = "Single Shading Model"))
	TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride = MSM_DefaultLit;
	
	/** SubsurfaceProfile, for Screen Space Subsurface Scattering. The profile needs to be set up on both the Substrate diffuse node, and the material node at the moment. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	bool HasSSS() const;
	bool HasAnisotropy() const;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const FExpressionInput& EmissiveColor,
		const FExpressionInput& Anisotropy,
		const FExpressionInput& ClearCoatNormal,
		const FExpressionInput& CustomTangent,
		const FExpressionInput& ShadingModel);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const FExpressionInput& EmissiveColor,
		const FExpressionInput& Anisotropy,
		const FExpressionInput& ClearCoatNormal,
		const FExpressionInput& CustomTangent,
		const FExpressionInput& ShadingModel);

	static int32 CompileCommon(class FMaterialCompiler* Compiler,
		FExpressionInput& BaseColor, FExpressionInput& Specular, FExpressionInput& Metallic, FExpressionInput& Roughness, FExpressionInput& EmissiveColor,
		FExpressionInput& Opacity, FExpressionInput& SubSurfaceColor, FExpressionInput& ClearCoat, FExpressionInput& ClearCoatRoughness,
		FExpressionInput& ShadingModel, TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride,
		FExpressionInput& TransmittanceColor, FExpressionInput& ThinTranslucentSurfaceCoverage,
		FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
		const bool bHasAnisotropy, FExpressionInput& Anisotropy,
		FExpressionInput& Normal, FExpressionInput& Tangent,
		FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent,
		const bool bHasSSS, USubsurfaceProfile* SSSProfile,
		const class UMaterialEditorOnlyData* EditorOnlyData = nullptr /*Used to provide default values from the root node pins*/);

	static MIR::FValueRef BuildCommon(MIR::FEmitter& Em, UMaterial* Material,
		FExpressionInput& BaseColor, FExpressionInput& Specular, FExpressionInput& Metallic, FExpressionInput& Roughness, FExpressionInput& EmissiveColor,
		FExpressionInput& Opacity, FExpressionInput& SubSurfaceColor, FExpressionInput& ClearCoat, FExpressionInput& ClearCoatRoughness,
		FExpressionInput& ShadingModel, TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride,
		FExpressionInput& TransmittanceColor, FExpressionInput& ThinTranslucentSurfaceCoverage,
		FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
		const bool bHasAnisotropy, FExpressionInput& Anisotropy,
		FExpressionInput& Normal, FExpressionInput& Tangent,
		FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent,
		const bool bHasSSS, USubsurfaceProfile* SSSProfile,
		const class UMaterialEditorOnlyData* EditorOnlyData = nullptr /*Used to provide default values from the root node pins*/);
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Slab")
class UMaterialExpressionSubstrateSlabBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Defines the diffused albedo, the percentage of light reflected as diffuse from the surface. (type = float3, unit = unitless, defaults to 0.18)
	 */
	UPROPERTY()
	FExpressionInput DiffuseAlbedo;

	/**
	 * Defines the color and brightness of the specular highlight where the surface is facing the camera. Each specular contribution will fade to black as F0 drops below 0.02. (type = float3, unit = unitless, defaults to plastic 0.04)
	 */
	UPROPERTY()
	FExpressionInput F0;

	/**
	 * Defines the color of the specular highlight where the surface normal is 90 degrees from the view direction. Only the hue and saturation are preserved, the brightness is fixed at 1.0. Fades to black as F0 drops below 0.02. (type = float3, unit = unitless, defaults to 1.0f)
	 */
	UPROPERTY()
	FExpressionInput F90;

	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. When using anisotropy, it is the roughness used along the Tangent axis. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;
		
	/**
	 * Controls the anisotropy factor of the roughness. Positive value elongates the specular lobe along the Tangent vector, Negative value elongates the specular lobe along the perpendicular of the Tangent. (type = float, unit = unitless).
	 */
	UPROPERTY()
	FExpressionInput Anisotropy;

	/**
	 * Take the surface normal as input. The normal is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex normal)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	* Take a surface tangent as input. The tangent is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex tangent)
	*/
	UPROPERTY()
	FExpressionInput Tangent;

	/**
	 * Chromatic mean free path. Only used when there is not any sub-surface profile provided. (type = float3, unit = centimeters, default = 0).
	 *  - For subsurface scattering, it is recommended to specify the MFP as world space centimeter directly. As it is an intuitive way to specify how far the light will scatter/bleed.
	 *  - For translucent coat layer, it is recommended to use the TransmittanceToMFP node as it might be easier to specify the transmittance color.
	 */
	UPROPERTY()
	FExpressionInput SSSMFP;

	/**
	 * Scale the mean free path length. Value between 0 and 1. Always used, with or without a subsurface profile. (type = float, unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput SSSMFPScale;

	/**
	 * Phase function anisotropy. Positive value elongates the phase function along the lignt direction, causing forward scattering. Negative value elongates the phase function backward to the light direction, causing backward scattering.  (type = float, unitless, defaults to 1, valid value -1..1)
	 */
	UPROPERTY()
	FExpressionInput SSSPhaseAnisotropy;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, default = 0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	* Controls the roughness of a secondary specular lobe. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. Does not influence diffuse roughness. (type = float, unit = unitless, defaults to 0.5) When using Blendable GBuffer, this input is ignored. 
	 */
	UPROPERTY()
	FExpressionInput SecondRoughness;

	/**
	* The weight of the second specular lobe using SecondRoughness. The first specular using Roughness will have a weight of (1 - SecondRoughnessWeight). (type = float, unitless, default = 0) When using Blendable GBuffer, this input is ignored. 
	 */
	UPROPERTY()
	FExpressionInput SecondRoughnessWeight;

	/**
	* Controls how rough the Fuzz layer is. Roughness of 0 is smooth and 1 is rough. If FuzzRoughness Is not connected, the Roughness input will be used instead. (type = float, unit = unitless, defaults to 0.5). When using Blendable GBuffer, this input is ignored. Only the fuzz roughness is forced to 0.5. 
	 */
	UPROPERTY()
	FExpressionInput FuzzRoughness;

	/**
	 * The amount of fuzz on top of the surface used to simulate cloth-like appearance.
	 */
	UPROPERTY()
	FExpressionInput FuzzAmount;

	/**
	 * The base color of the fuzz.
	 */
	UPROPERTY()
	FExpressionInput FuzzColor;

	/**
	 * This represent the micro facet density. Only used when `r.Substrate.Glints=1`. 0 = only very sparse glints, 1 = fully covered with glints (which is equivalent to a regular specular lobe). Defaults to 1 (=no glint). When using Blendable GBuffer, this input is ignored.
	 */
	UPROPERTY()
	FExpressionInput GlintValue;

	/**
	 * The parameterization of the surface required to position glints on a surface. Only used when `r.Substrate.Glints=1`. Defaults to (0,0). When using Blendable GBuffer, this input is ignored.
	 */
	UPROPERTY()
	FExpressionInput GlintUV;

	/** SubsurfaceProfile, for Screen Space Subsurface Scattering. The profile needs to be set up on both the Substrate diffuse node, and the material node at the moment. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	/** SpecularProfile, for modulating specular appearance and simulating more complex visuals such as iridescence. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Specular Profile"))
	TObjectPtr<class USpecularProfile> SpecularProfile;

	UE_DEPRECATED(5.6, "Please, use SubSurfaceType to enable/disable SSS-diffusion")
	UPROPERTY()
	uint32 bUseSSSDiffusion : 1;

	/** Define sub-subsurface used behavior of the slab. This option trades quality over performance and will result into visual differences. 
	 *   * For slab not sitting at the bottom of the topology (e.g. slabs stacked with a vertial operator), only the SimpleVolume SSS type is available.
	 *   * In non-opaque blend modes, SSS Diffusion and SSS Diffusion Profile are not available and will fallback onto Wrap mode.
	 */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DisplayName = "Sub-Surface Type"))
	TEnumAsByte<enum EMaterialSubSurfaceType> SubSurfaceType = MSS_None;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	bool HasEdgeColor() const;
	bool HasFuzz() const;
	bool HasFuzzRoughness() const;
	bool HasSecondRoughness() const;
	bool HasSSS() const;
	bool HasSSSProfile() const;
	bool HasMFPPluggedIn() const;
	bool HasAnisotropy() const;
	bool HasGlint() const;
	bool HasSpecularProfile() const;

	FSubstrateMaterialComplexity GetHighestComplexity() const;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Simple Clear Coat")
class UMaterialExpressionSubstrateSimpleClearCoatBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Defines the diffused albedo, the percentage of light reflected as diffuse from the surface. (type = float3, unit = unitless, defaults to 0.18)
	 */
	UPROPERTY()
	FExpressionInput DiffuseAlbedo;

	/**
	 * Defines the color and brightness of the specular highlight where the surface is facing the camera. Each specular contribution will fade to black as F0 drops below 0.02. (type = float3, unit = unitless, defaults to plastic 0.04)
	 */
	UPROPERTY()
	FExpressionInput F0;

	/**
	 * Controls how rough the bottom layer of the material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;

	/**
	 * Controls the coverage of the clear coat layer: 0 means no clear coat, 1 means coat is fully visible. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput ClearCoatCoverage;

	/**
	 * Controls how rough the top layer of the material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput ClearCoatRoughness;

	/**
	 * Take the surface normal as input. The normal is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex normal)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	 * Emissive color of the medium (type = float3, unit = luminance, default = 0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * The bottom layer normal. Only used when r.ClearCoatNormal is 1. The normal is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex normal)
	 */
	UPROPERTY()
	FExpressionInput BottomNormal;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Volumetric-Fog-Cloud BSDF")
class UMaterialExpressionSubstrateVolumetricFogCloudBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	* The single scattering Albedo defining the overall color of the Material (type = float3, unit = unitless, default = 0)
	*/
	UPROPERTY()
	FExpressionInput Albedo;

	/**
	 * The rate at which light is absorbed or scattered by the medium. Mean Free Path = 1 / Extinction. (type = float3, unit = 1/m, default = 0)
	 */
	UPROPERTY()
	FExpressionInput Extinction;

	/**
	 * Emissive color of the medium (type = float3, unit = luminance, default = 0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * Ambient occlusion: 1 means no occlusion while 0 means fully occluded. (type = float, unit = unitless, default = 1)
	 */
	UPROPERTY()
	FExpressionInput AmbientOcclusion;

	/** Enabling this corresponds to selecting the Unlit shading model from the legacy material system. In this case, only the Emissive input will be considered. */
	UPROPERTY(EditAnywhere, Category = Shading, meta = (DisplayName = "Unlit (EmissiveOnly)"))
	uint32 bEmissiveOnly : 1;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const FExpressionInput& EmissiveColor,
		const FExpressionInput& AmbientOcclusion);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const FExpressionInput& EmissiveColor,
		const FExpressionInput& AmbientOcclusion);

	static int32 CompileCommon(class FMaterialCompiler* Compiler,
		FExpressionInput& Albedo, FExpressionInput& Extinction, FExpressionInput& EmissiveColor, FExpressionInput& AmbientOcclusion,
		bool bEmissiveOnly,
		const class UMaterialEditorOnlyData* EditorOnlyData = nullptr /*Used to provide default values from the root node pins*/);
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Unlit BSDF")
class UMaterialExpressionSubstrateUnlitBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	* Emissive color on top of the surface (type = float3, unit = Luminance, default = 0)
	*/
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * The amount of transmitted light from the back side of the surface to the front side of the surface (type = float3, unit = unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput TransmittanceColor;

	/**
	 * The surface normal. Only used for refraction effects when `IOR` or `pixel normal offset` modes are selected.
	 */
	UPROPERTY()
	FExpressionInput Normal;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Hair BSDF")
class UMaterialExpressionSubstrateHairBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()
		
	/**
	 * Hair fiber base color resulting from single and multiple scattering combined. (type = float3, unit = unitless, defaults to black)
	 */
	UPROPERTY()
	FExpressionInput BaseColor;
	
	/**
	 * Amount of light scattering, only available for non-HairStrand rendering (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput Scatter;
		
	/**
	 * Specular (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Specular;
		
	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;

	/**
	 * How much light contributs when lighting hairs from the back side opposite from the view, only available for HairStrand rendering (type = float3, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput Backlit;

	/**
	 * Tangent (type = float3, unit = unitless, defaults to +X vector)
	 */
	UPROPERTY()
	FExpressionInput Tangent;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Eye BSDF")
class UMaterialExpressionSubstrateEyeBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()
		
	/**
	 * Hair fiber base color resulting from single and multiple scattering combined. (type = float3, unit = unitless, defaults to black)
	 */
	UPROPERTY()
	FExpressionInput DiffuseColor;
		
	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;

	/**
	 * Normal of the sclera and cornea (type = float3, unit = unitless, defaults to +X vector)
	 */
	UPROPERTY()
	FExpressionInput CorneaNormal;

	/**
	 * Normal of the iris (type = float3, unit = unitless, defaults to +X vector)
	 */
	UPROPERTY()
	FExpressionInput IrisNormal;

	/**
	 * Normal of the iris plane (type = float3, unit = unitless, defaults to +X vector)
	 */
	UPROPERTY()
	FExpressionInput IrisPlaneNormal;

	/**
	 * Mask defining the iris surface (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput IrisMask;

	/**
	 * Distance from the center of the iris (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput IrisDistance;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/** SubsurfaceProfile, for Subsurface Scattering diffusion. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Single Layer Water BSDF")
class UMaterialExpressionSubstrateSingleLayerWaterBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Surface base color. (type = float3, unit = unitless, defaults to black)
	 */
	UPROPERTY()
	FExpressionInput BaseColor;

	/**
	 * Whether the surface represents a dielectric (such as plastic) or a conductor (such as metal). (type = float, unit = unitless, defaults to 0 = dielectric)
	 */
	UPROPERTY()
	FExpressionInput Metallic;

	/**
	 * Specular amount (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Specular;

	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;

	/**
	 * The normal of the surface (type = float3, unit = unitless, defaults to +Z vector)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * Opacity of the material layered on top of the water (type = float3, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput TopMaterialOpacity;

	/**
	* The single scattering Albedo defining the overall color of the Material (type = float3, unit = unitless, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterAlbedo;

	/**
	 * The rate at which light is absorbed or out-scattered by the medium. Mean Free Path = 1 / Extinction. (type = float3, unit = 1/cm, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterExtinction;

	/**
	 * Anisotropy of the volume with values lower than 0 representing back-scattering, equal 0 representing isotropic scattering and greater than 0 representing forward scattering. (type = float, unit = unitless, defaults to 0)
	 */
	UPROPERTY()
	FExpressionInput WaterPhaseG;

	/**
	 * A scale to apply on the scene color behind the water surface. It can be used to approximate caustics for instance. (type = float3, unit = unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput ColorScaleBehindWater;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Light Function")
class UMaterialExpressionSubstrateLightFunction : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * The output color of the light function
	 */
	UPROPERTY()
	FExpressionInput Color;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static int32 CompileCommon(class FMaterialCompiler* Compiler,
		FExpressionInput& Color,
		const UMaterialEditorOnlyData* EditorOnlyData = nullptr);
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Toon BSDF", meta = (DevelopmentStatus = "Experimental"))
class UMaterialExpressionSubstrateToonBSDF : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Surface base color. (type = float3, unit = unitless, defaults to 0.18 grey)
	 */
	UPROPERTY()
	FExpressionInput BaseColor;

	/**
	 * Surface metallic value to transition from a dielectric to a conductor. (type = float, unit = unitless, defaults to 0.0)
	 */
	UPROPERTY()
	FExpressionInput Metallic;

	/**
	 * Surface specular value. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Specular;

	/**
	 * Controls how rough the Material is. Roughness of 0 (smooth) is a mirror reflection and 1 (rough) is completely matte or diffuse. When using anisotropy, it is the roughness used along the Tangent axis. (type = float, unit = unitless, defaults to 0.5)
	 */
	UPROPERTY()
	FExpressionInput Roughness;

	/**
	 * The normal of the surface (type = float3, unit = unitless, defaults to +Z vector)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	 * Emissive color on top of the surface (type = float3, unit = luminance, defaults to black)
	 */
	UPROPERTY()
	FExpressionInput EmissiveColor;

	/**
	 * UVs used to render the diffuse, specular or any other artistic pattern. It can only store data in [0,1].
	 */
	UPROPERTY()
	FExpressionInput PatternUVs;

	/**
	 * Controls the anisotropy factor of the roughness. Positive value elongates the specular lobe along the Tangent vector, Negative value elongates the specular lobe along the perpendicular of the Tangent. (type = float, unit = unitless).
	 */
	UPROPERTY()
	FExpressionInput Anisotropy;

	/**
	* Take a surface tangent as input. The tangent is considered tangent or world space according to the space properties on the main material node. (type = float3, unit = unitless, defaults to vertex tangent)
	*/
	UPROPERTY()
	FExpressionInput Tangent;

	/** Toon profile used to control the appearance of this BSDF. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Toon Profile"))
	TObjectPtr<class UToonProfile> ToonProfile;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Post Process")
class UMaterialExpressionSubstratePostProcess : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * The output color of the post process: it represents a color added over the back buffer, or a color multiplied if the Substrate blend mode is TransmittanceOnly.
	 */
	UPROPERTY()
	FExpressionInput Color;

	/**
	 * The coverage of the post process: the more the value is high, the less the back buffer will be visible. Only used if "Output Alpha" is enabled on the root node.
	 */
	UPROPERTY()
	FExpressionInput Opacity;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static int32 CompileCommon(class FMaterialCompiler* Compiler,
		FExpressionInput& Color, FExpressionInput& Opacity,
		const UMaterialEditorOnlyData* EditorOnlyData = nullptr);
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate UI")
class UMaterialExpressionSubstrateUI : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * The output color of the UI element.
	 */
	UPROPERTY()
	FExpressionInput Color;

	/**
	 * The coverage of the UI element: the more the value is high, the less the back buffer will be visible.
	 */
	UPROPERTY()
	FExpressionInput Opacity;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex);

	static int32 CompileCommon(class FMaterialCompiler* Compiler,
		FExpressionInput& Color, FExpressionInput& Opacity,
		const UMaterialEditorOnlyData* EditorOnlyData = nullptr);
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Convert To Decal")
class UMaterialExpressionSubstrateConvertToDecal : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * The Substrate material to convert to a decal.
	 */
	UPROPERTY()
	FExpressionInput DecalMaterial;

	/**
	 * The coverage of the decal (default 1)
	 */
	UPROPERTY()
	FExpressionInput Coverage;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionSubstrateConvertMaterialAttributes : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FMaterialAttributesInput MaterialAttributes;

		/**
	* The single scattering Albedo defining the overall color of the Material (type = float3, unit = unitless, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterScatteringCoefficients;

	/**
	 * The rate at which light is absorbed or out-scattered by the medium. Mean Free Path = 1 / Extinction. (type = float3, unit = 1/cm, default = 0)
	 */
	UPROPERTY()
	FExpressionInput WaterAbsorptionCoefficients;

	/**
	 * Anisotropy of the volume with values lower than 0 representing back-scattering, equal 0 representing isotropic scattering and greater than 0 representing forward scattering. (type = float, unit = unitless, defaults to 0)
	 */
	UPROPERTY()
	FExpressionInput WaterPhaseG;

	/**
	 * A scale to apply on the scene color behind the water surface. It can be used to approximate caustics for instance. (type = float3, unit = unitless, defaults to 1)
	 */
	UPROPERTY()
	FExpressionInput ColorScaleBehindWater;

	/** SubsurfaceProfile, for Screen Space Subsurface Scattering. The profile needs to be set up on both the Substrate diffuse node, and the material node at the moment. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	UPROPERTY(EditAnywhere, Category = ShadingModel, meta = (ShowAsInputPin = "Primary", DisplayName = "Single Shading Model"))
	TEnumAsByte<enum EMaterialShadingModel> ShadingModelOverride = MSM_DefaultLit;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API static int32 CompileDefaultSlab(class FMaterialCompiler* Compiler, int32 EmissiveOverride);
	ENGINE_API static int32 CompileDefaultSlab(class FMaterialCompiler* Compiler, FVector3f EmissiveOverride = FVector3f::Zero());
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual bool IsInputConnectionRequired(int32 InputIndex) const override {return true;}
	
	bool HasSSS() const;

	UE_DEPRECATED(5.8, "Please, SubstrateGenerateMaterialTopologyTreeCommon taking the FSubstrateTranslatorData parameter.")
	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		class FMaterialCompiler* Compiler, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const uint64 CachedConnectedMaterialAttributesInputs, const bool bShadingModelFromMaterialExpression, const bool bIsEmissiveConnected);

	static FSubstrateOperator* SubstrateGenerateMaterialTopologyTreeCommon(
		struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, struct FGuid ThisExpressionGuid, class UMaterialExpression* Parent, int32 OutputIndex,
		const uint64 CachedConnectedMaterialAttributesInputs, const bool bShadingModelFromMaterialExpression, const bool bIsEmissiveConnected);

	static int32 CompileCommon(class FMaterialCompiler* Compiler, int32 OutputIndex,
		const uint64 CachedConnectedMaterialAttributesInputs, FMaterialAttributesInput& MaterialAttributes, enum EMaterialShadingModel ShadingModelOverride,
		FExpressionInput& WaterScatteringCoefficients, FExpressionInput& WaterAbsorptionCoefficients, FExpressionInput& WaterPhaseG, FExpressionInput& ColorScaleBehindWater,
		const bool bHasSSS, USubsurfaceProfile* SSSProfile,
		FExpressionInput& ClearCoatNormal, FExpressionInput& CustomTangent);
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



///////////////////////////////////////////////////////////////////////////////
// Operator nodes

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Horizontal Blend")
class UMaterialExpressionSubstrateHorizontalMixing : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()
		
	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput Background;

	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput Foreground;

	/**
	 * Lerp factor between Background (Mix == 0) and Foreground (Mix == 1).
	 */
	UPROPERTY()
	FExpressionInput Mix;

	/**
	 * Merge Background and Foreground into a single material by mixing their inputs rather than their evaluation. This makes lighting evaluation cheaper (Default: off)
	 */
	UPROPERTY(EditAnywhere, Category = Mode)
	uint32 bUseParameterBlending : 1;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Vertical Layer")
class UMaterialExpressionSubstrateVerticalLayering : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Substrate material layer on top of the Base material layer
	 */
	UPROPERTY()
	FExpressionInput Top;
	
	/**
	 * Substrate material layer below the Top material layer
	 */
	UPROPERTY()
	FExpressionInput Base;

	/**
	 * Thickness of the Top material layer in centimeter. Default value: 0.01cm.
	 * It can be modulated to achieve simple scattering/transmittance variation of the same material.
	 */
	UPROPERTY()
	FExpressionInput Thickness;

	/**
	 * Merge Top and Base into a single material by mixing their inputs rather than their evaluation. This makes lighting evaluation cheaper (Default: off)
	 */
	UPROPERTY(EditAnywhere, Category = Mode)
	uint32 bUseParameterBlending : 1;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Add")
class UMaterialExpressionSubstrateAdd : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput A;
	
	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput B;

	/**
	 * Merge A and B into a single material by mixing their inputs rather than their evaluation. This makes lighting evaluation cheaper (Default: off)
	 */
	UPROPERTY(EditAnywhere, Category = Mode)
	uint32 bUseParameterBlending : 1;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Coverage Weight")
class UMaterialExpressionSubstrateWeight : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput A;
	
	/**
	 * Weight to apply to the Substrate material BSDFs
	 */
	UPROPERTY()
	FExpressionInput Weight;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Select")
class UMaterialExpressionSubstrateSelect : public UMaterialExpressionSubstrateBSDF
{
	GENERATED_UCLASS_BODY()

	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput A;

	/**
	 * Substrate material
	 */
	UPROPERTY()
	FExpressionInput B;

	/**
	 * If <= Threshold, A is selected, otherwise B is selected. (Default: 0))
	 */
	UPROPERTY()
	FExpressionInput SelectValue;

	/**
	 * The threshold to use to select between A or B (Default: 0.5)
	 */
	UPROPERTY(EditAnywhere, Category = Mode, meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0, ClampMax = 1.0))
	float Threshold;

	/**
	 * Select A and B using parameter blending for material evaluation. This makes lighting evaluation cheaper (Default: On)
	 * As of today, parameter blending must be used since a single Substrate Tree cannot be used with hair or slabs for instance.
	 */
	const bool bUseParameterBlending = true;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool IsResultSubstrateMaterial(int32 OutputIndex) override;
	virtual void GatherSubstrateMaterialInfo(FSubstrateMaterialInfo& SubstrateMaterialInfo, int32 OutputIndex) override;
	virtual FSubstrateOperator* SubstrateGenerateMaterialTopologyTree(struct FSubstrateTranslatorDataInterface& SubstrateTranslatorData, class UMaterialExpression* Parent, int32 OutputIndex) override;
#endif
	//~ End UMaterialExpression Interface
};



///////////////////////////////////////////////////////////////////////////////
// Utilities

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, Abstract, DisplayName = "Substrate Utility Base Class")
class UMaterialExpressionSubstrateUtilityBase : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Transmittance-To-MeanFreePath")
class UMaterialExpressionSubstrateTransmittanceToMFP : public UMaterialExpressionSubstrateUtilityBase
{
	GENERATED_UCLASS_BODY()

	/**
	* The colored transmittance for a view perpendicular to the surface. The transmittance for other view orientations will automatically be deduced according to surface thickness.
	*/
	UPROPERTY()
	FExpressionInput TransmittanceColor;

	/**
	* Thickness of the layer in centimeter. Default value: 0.01cm.
	* Example of use case: this node output called thickness can be modulated before it is plugged in a Vertical Layering node Thickness input. This can be used to achieve simple scattering/transmittance variation of the same material.
	*/
	UPROPERTY()
	FExpressionInput Thickness;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Metalness-To-DiffuseColorF0")
class UMaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0 : public UMaterialExpressionSubstrateUtilityBase
{
	GENERATED_UCLASS_BODY()

	/**
	 * Defines the overall color of the Material. (type = float3, unit = unitless, defaults to 0.18)
	 */
	UPROPERTY()
	FExpressionInput BaseColor;

	/**
	 * Controls how \"metal-like\" your surface looks like. 0 means dielectric, 1 means conductor (type = float, unit = unitless, defaults to 0)
	 */
	UPROPERTY()
	FExpressionInput Metallic;

	/**
	 * Used to scale the current amount of specularity on non-metallic surfaces and is a value between 0 and 1 (type = float, unit = unitless, defaults to plastic 0.5)
	 */
	UPROPERTY()
	FExpressionInput Specular;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Haziness-To-Secondary-Roughness")
class UMaterialExpressionSubstrateHazinessToSecondaryRoughness : public UMaterialExpressionSubstrateUtilityBase
{
	GENERATED_UCLASS_BODY()

	/**
	* The base roughness of the surface. It represented the smoothest part of the reflection.
	*/
	UPROPERTY()
	FExpressionInput BaseRoughness;

	/**
	* Haziness represent the amount of irregularity of the surface. A high value will lead to a second rough specular lobe causing the surface too look `milky`.
	*/
	UPROPERTY()
	FExpressionInput Haziness;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};

UCLASS(MinimalAPI, collapsecategories, hidecategories = Object, DisplayName = "Substrate Thin-Film")
class UMaterialExpressionSubstrateThinFilm : public UMaterialExpressionSubstrateUtilityBase
{
	GENERATED_UCLASS_BODY()

	/**
	 * The normal of the surface to consider. This input respects the normal space setup on the root node (tangent or world)
	 */
	UPROPERTY()
	FExpressionInput Normal;

	/**
	 * Defines the color and brightness of the specular highlight where the surface is facing the camera. Each specular contribution will fade to black as F0 drops below 0.02. (type = float3, unit = unitless, defaults to plastic 0.04)
	 */
	UPROPERTY()
	FExpressionInput F0;

	/**
	 * Defines the color of the specular highlight where the surface normal is 90 degrees from the view direction. Only the hue and saturation are preserved, the brightness is fixed at 1.0. Fades to black as F0 drops below 0.02. (type = float3, unit = unitless, defaults to 1.0f)
	 */
	UPROPERTY()
	FExpressionInput F90;
	
	/**
	 * Controls the thickness of the thin film layer coating the current slab. 0 means disabled and 1 means a coating layer of 10 micrometer. (type = float, unitless, default = 0)
	 */
	UPROPERTY()
	FExpressionInput Thickness;

	/**
	 * Thin film IOR
	 */
	UPROPERTY()
	FExpressionInput IOR;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetExpressionToolTip(TArray<FString>& OutToolTip) override;
#endif
	//~ End UMaterialExpression Interface
};