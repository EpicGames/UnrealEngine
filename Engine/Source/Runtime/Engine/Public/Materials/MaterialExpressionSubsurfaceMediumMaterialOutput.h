// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionSubsurfaceMediumMaterialOutput.generated.h"

/** Material output expression for setting the mean free path and scattering distribution properties of subsurface profile (for the Path Tracer Only). */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionSubsurfaceMediumMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Input for mean free path (cm). Fallback to the default behavior if not set (e.g., Subsurfaceprofile shading: Use the derived MFP) */
	UPROPERTY()
	FExpressionInput MeanFreePath;

	/** Input for scattering distribution. Valid range is (-1, 1). Fallback to zero if not set*/
	UPROPERTY()
	FExpressionInput ScatteringDistribution;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
