// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionSubtract.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionSubtract : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstA' if not specified"))
	FExpressionInput A;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstB' if not specified"))
	FExpressionInput B;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSubtract, meta=(OverridingInputProperty = "A"))
	float ConstA = 1.0f;

	/** only used if B is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionSubtract, meta=(OverridingInputProperty = "B"))
	float ConstB = 1.0f;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("-"));}
#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



