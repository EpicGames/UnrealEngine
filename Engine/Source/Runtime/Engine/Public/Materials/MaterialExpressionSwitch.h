// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "MaterialExpressionSwitch.generated.h"

struct FPropertyChangedEvent;

UENUM()
enum ESwitchMaterialOutputType : int
{
	TMMOT_Float1,
	TMMOT_Float2,
	TMMOT_Float3,
	TMMOT_Float4,
	TMMOT_MAX,
};

USTRUCT()
struct FSwitchCustomInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=CustomInput)
	FName InputName;

	UPROPERTY()
	FExpressionInput Input;
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionSwitch : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category= UMaterialExpressionSwitch)
	FString Description = "Switch";

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstSwitchValue' if not specified"))
	FExpressionInput SwitchValue;

	/** only used if Selector is not hooked up */
	UPROPERTY(EditAnywhere, Category= UMaterialExpressionSwitch, meta=(OverridingInputProperty = "SwitchValue"))
	float ConstSwitchValue;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstDefault' if not specified"))
	FExpressionInput Default;

	/** only used if Selector is not hooked up */
	UPROPERTY(EditAnywhere, Category= UMaterialExpressionSwitch, meta=(OverridingInputProperty = "Default"))
	float ConstDefault;

	UPROPERTY(EditAnywhere, Category= UMaterialExpressionSwitch)
	TArray<struct FSwitchCustomInput> Inputs;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	void RebuildOutputs();
#endif // WITH_EDITOR
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Interface.
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetCreationName() const override { return FText::FromString(TEXT("Switch")); }
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return MCT_Unknown;}

#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface
};



