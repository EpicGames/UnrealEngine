// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionTemporalResponsivenessOutput.generated.h"

UCLASS(collapsecategories, hidecategories=Object, Experimental, meta = (DisplayName = "Temporal Responsiveness"), MinimalAPI)
class UMaterialExpressionTemporalResponsivenessOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/**Indicate the temporal responsiveness requested.
	*
	*Default:0. Normal temporal accumulation.
	*
	*Medium:(0,0.5]. Assumes a medium-level mismatch of motion vectors for pixel animation, and reject temporal history more aggressively, especially for small features. 
	*
	*Full:(0.5,1]. Motion vector is fully unreliable. Reject all temporal history. 
	*
	*Non-Nanite (Full support), Nanite (Support base pass only)
	*/
	UPROPERTY()
	FExpressionInput Input;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
