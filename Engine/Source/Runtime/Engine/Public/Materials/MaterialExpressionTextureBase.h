// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Node acts as a base class for TextureSamples and TextureObjects 
 * to cover their shared functionality. 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTextureBase.generated.h"

class UTexture;
struct FPropertyChangedEvent;
enum EMaterialSamplerType : int;
enum EShaderPlatform : uint16;

UCLASS(abstract, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureBase : public UMaterialExpression 
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionTextureBase)
	TObjectPtr<UTexture> Texture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionTextureBase, meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<EMaterialSamplerType> SamplerType;
	
	/** Is default selected texture when using mesh paint mode texture painting */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureBase)
	uint8 IsDefaultMeshpaintTexture:1 = false;
	
	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual FString GetDescription() const override;
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	ENGINE_API virtual FText GetPreviewOverlayText() const override;
#endif
	//~ End UMaterialExpression Interface

	/** 
	 * Callback to get any texture reference this expression emits.
	 * This is used to link the compiled uniform expressions with their default texture values. 
	 * Any UMaterialExpression whose compilation creates a texture uniform expression (eg Compiler->Texture, Compiler->TextureParameter) must implement this.
	 */
	ENGINE_API virtual UObject* GetReferencedTexture() const override;
	virtual bool CanReferenceTexture() const override { return true; }

#if WITH_EDITOR
	/**
	 * Automatically determines and set the sampler type for the current texture.
	 */
	ENGINE_API void AutoSetSampleType();

	UE_DEPRECATED(5.8, "Use MaterialExpressionUtils::GetSamplerTypeForTexture instead.")
	static ENGINE_API EMaterialSamplerType GetSamplerTypeForTexture(const UTexture* Texture, bool ForceNoVT = false);

	UE_DEPRECATED(5.8, "Use MaterialExpressionUtils::VerifySamplerType instead.")
	static ENGINE_API bool VerifySamplerType(const FString& TexturePathName, EMaterialSamplerType CorrectSamplerType, bool bSRGB, EMaterialSamplerType SamplerType, FString& OutErrorMessage);

	UE_DEPRECATED(5.8, "Use MaterialExpressionUtils::VerifySamplerType instead.")
	static ENGINE_API bool VerifySamplerType(EShaderPlatform ShaderPlatform,const ITargetPlatform* TargetPlatform,const UTexture* Texture,EMaterialSamplerType SamplerType,FString& OutErrorMessage);
#endif // WITH_EDITOR
};
