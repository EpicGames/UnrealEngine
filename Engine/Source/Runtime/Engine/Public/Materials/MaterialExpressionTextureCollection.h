// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureCollection.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTextureCollection.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionTextureCollection : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollection)
	TObjectPtr<UTextureCollection> TextureCollection;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;

	virtual UTextureCollection* GetReferencedTextureCollection() const { return TextureCollection; }
#endif
	//~ End UMaterialExpression Interface
};
