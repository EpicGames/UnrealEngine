// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpressionTextureCollection.h"
#include "MaterialExpressionTextureCollectionParameter.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionTextureCollectionParameter : public UMaterialExpressionTextureCollection
{
	GENERATED_UCLASS_BODY()

	/** The name of the parameter */
	UPROPERTY(EditAnywhere, Category= MaterialExpressionTextureCollectionParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category= MaterialExpressionTextureCollectionParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list.  The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category= MaterialExpressionTextureCollectionParameter)
	int32 SortPriority = 32;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void  GetCaption(TArray<FString>& OutCaptions) const override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;

	virtual bool    CanRenameNode() const override;
	virtual FString GetEditableName() const override;
	virtual void    SetEditableName(const FString& NewName) override;

	virtual bool  HasAParameterName() const override;
	virtual FName GetParameterName() const override;
	virtual void  SetParameterName(const FName& Name) override;
	virtual void  ValidateParameterName(const bool bAllowDuplicateName) override;

	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override;
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override;
#endif

	virtual FGuid& GetParameterExpressionId() override;
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	bool TextureCollectionIsValid(UTextureCollection* InTextureCollection, FString& OutMessage);
	bool SetParameterValue(const FName& InParameterName, UTextureCollection* InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);
#endif
};
