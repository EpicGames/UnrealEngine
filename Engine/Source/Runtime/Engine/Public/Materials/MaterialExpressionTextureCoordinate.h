// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTextureCoordinate.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureCoordinate : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Texture coordinate index */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTextureCoordinate, meta = (ShowAsInputPin = "Primary"))
	int32 CoordinateIndex;

	/** Controls how much the texture tiles horizontally, by scaling the U component of the vertex UVs by the specified amount. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionTextureCoordinate, meta = (ShowAsInputPin = "Advanced"))
	float UTiling = 1.0f;

	/** Controls how much the texture tiles vertically, by scaling the V component of the vertex UVs by the specified amount. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialExpressionTextureCoordinate, meta = (ShowAsInputPin = "Advanced"))
	float VTiling = 1.0f;

	/** Would like to unmirror U or V 
	 *  - if the texture is mirrored and if you would like to undo mirroring for this texture sample, use this to unmirror */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureCoordinate, meta = (ShowAsInputPin = "Advanced"))
	uint32 UnMirrorU:1;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureCoordinate, meta = (ShowAsInputPin = "Advanced"))
	uint32 UnMirrorV:1;


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("texcoord")); }
	virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
#endif
	//~ End UMaterialExpression Interface
};



