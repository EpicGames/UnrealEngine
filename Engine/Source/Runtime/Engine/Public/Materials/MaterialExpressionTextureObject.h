// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Node which outputs a texture object itself, instead of sampling the texture first.
 * This is used with material functions to provide a preview value for a texture function input.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "MaterialExpressionTextureObject.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureObject : public UMaterialExpressionTextureBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};



