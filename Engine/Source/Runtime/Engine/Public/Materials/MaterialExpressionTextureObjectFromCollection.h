// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureCollection.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "MaterialExpressionTextureObjectFromCollection.generated.h"

/** Describes how textures are sampled for materials */
UENUM(BlueprintType)
enum class ETextureCollectionMemberType : uint8
{
	Texture2D        UMETA(DisplayName = "Texture 2D"),
	TextureCube      UMETA(DisplayName = "Texture Cube"),
	Texture2DArray   UMETA(DisplayName = "Texture 2D Array"),
	TextureCubeArray UMETA(DisplayName = "Texture Cube Array"),
	TextureVolume    UMETA(DisplayName = "Volume Texture"),

	Max
};

inline EMaterialValueType MaterialValueTypeFromTextureCollectionMemberType(ETextureCollectionMemberType InType)
{
	switch (InType)
	{
	case ETextureCollectionMemberType::Texture2D:        return MCT_Texture2D;
	case ETextureCollectionMemberType::TextureCube:      return MCT_TextureCube;
	case ETextureCollectionMemberType::Texture2DArray:   return MCT_Texture2DArray;
	case ETextureCollectionMemberType::TextureCubeArray: return MCT_TextureCubeArray;
	case ETextureCollectionMemberType::TextureVolume:    return MCT_VolumeTexture;
	}
	return MCT_Texture2D;
}

UCLASS(MinimalAPI)
class UMaterialExpressionTextureObjectFromCollection : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput TextureCollection;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollection)
	TObjectPtr<UTextureCollection> TextureCollectionObject;

	UPROPERTY(meta = (RequiredInput = "false"))
	FExpressionInput CollectionIndex;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollection, meta = (OverridingInputProperty = "CollectionIndex"))
	int32 ConstCollectionIndex = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollection)
	ETextureCollectionMemberType TextureType = ETextureCollectionMemberType::Texture2D;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

	virtual UTextureCollection* GetReferencedTextureCollection() const { return TextureCollectionObject; }
#endif
	//~ End UMaterialExpression Interface
};
