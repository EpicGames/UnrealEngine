// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Node which creates a texture parameter and outputs the texture object itself, instead of sampling the texture first.
 * This is used with material functions to implement texture parameters without actually putting the parameter in the function.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialExpressionTextureObjectParameter.generated.h"

UCLASS(collapsecategories, hidecategories=(Object, MaterialExpressionTextureSample), MinimalAPI)
class UMaterialExpressionTextureObjectParameter : public UMaterialExpressionTextureSampleParameter
{
	GENERATED_UCLASS_BODY()
	
#if WITH_EDITOR
	//~ Begin UMaterialExpressionTextureSampleParameter Interface
	virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage) override;
	//~ End UMaterialExpressionTextureSampleParameter Interface
#endif

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;	
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 InputIndex) override {return MCT_Texture;}
#endif
	//~ End UMaterialExpression Interface
};



