// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTextureProperty.generated.h"

class UTexture;

/** Selects the texture property to output */
UENUM()
enum EMaterialExposedTextureProperty : int
{
	/* The texture's size. */
	TMTM_TextureSize UMETA(DisplayName="Texture Size"),

	/* The texture's texel size in the UV space (1 / Texture Size) */
	TMTM_TexelSize UMETA(DisplayName="Texel Size"),
	
	TMTM_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureProperty : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Texture Object to access the property from."))
	FExpressionInput TextureObject;
	
	/** Texture property to be accessed */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionTextureProperty, meta=(DisplayName = "Texture Property", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EMaterialExposedTextureProperty> Property = TMTM_TextureSize;
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetTexturesForceMaterialRecompile(TArray<UTexture *> &Textures) const override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;

#endif //WITH_EDITOR
	//~ End UMaterialExpression Interface
};
