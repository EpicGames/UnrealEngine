// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "MaterialExpressionTextureSample.generated.h"

struct FPropertyChangedEvent;
enum ETextureMipValueMode : int;
enum ESamplerSourceMode : int;

UENUM()
enum ETextureGatherMode : int
{
	TGM_None UMETA(DisplayName = "None"),
	TGM_Red UMETA(DisplayName = "Red"),
	TGM_Green UMETA(DisplayName = "Green"),
	TGM_Blue UMETA(DisplayName = "Blue"),
	TGM_Alpha UMETA(DisplayName = "Alpha"),

	TGM_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureSample : public UMaterialExpressionTextureBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstCoordinate' if not specified"))
	FExpressionInput Coordinates;

	/** 
	 * Texture object input which overrides Texture if specified. 
	 * This only shows up in material functions and is used to implement texture parameters without actually putting the texture parameter in the function.
	 */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'Texture' if not specified"))
	FExpressionInput TextureObject;

	/** Meaning depends on MipValueMode, a single unit is one mip level  */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'AutomaticViewMipBias' if not specified"))
	FExpressionInput MipValue;
	
	/** Enabled only if MipValueMode == TMVM_Derivative */
	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Coordinates derivative over the X axis"))
	FExpressionInput CoordinatesDX;
	
	/** Enabled only if MipValueMode == TMVM_Derivative */
	UPROPERTY(meta = (RequiredInput = "true", ToolTip = "Coordinates derivative over the Y axis"))
	FExpressionInput CoordinatesDY;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput AutomaticViewMipBiasValue;

	/** Defines how the MipValue property is applied to the texture lookup */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureSample, meta=(DisplayName = "MipValueMode", ShowAsInputPin = "Advanced"))
	TEnumAsByte<ETextureMipValueMode> MipValueMode;

	/** 
	 * Controls where the sampler for this texture lookup will come from.  
	 * Choose 'from texture asset' to make use of the UTexture addressing settings,
	 * Otherwise use one of the global samplers, which will not consume a sampler slot.
	 * This allows materials to use more than 16 unique textures on SM5 platforms.
	 */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureSample, Meta = (ShowAsInputPin = "Advanced"))
	TEnumAsByte<ESamplerSourceMode> SamplerSource;

	/**
	 * Whether to do a Gather of the given component from 4 neighboring texels (xyzw in counter clockwise order starting with the sample to the lower left).
	 * Only works with Texture2D or TextureCube.  Doesn't work with virtual textures.
	 */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample)
	TEnumAsByte<ETextureGatherMode> GatherMode;

	/** Whether the texture should be sampled with per view mip biasing for sharper output with Temporal AA. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample)
	uint8 AutomaticViewMipBias : 1;

	UPROPERTY(EditAnywhere, Category = ParameterCustomization, meta=(EditorOnlyLocalization))
	FParameterChannelNames ChannelNames;

protected:
	// Inherited parameter expressions can hide unused input pin
	uint8 bShowTextureInputPin : 1;

#if WITH_EDITOR
#endif

public:

	/** only used if Coordinates is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample, meta = (OverridingInputProperty = "Coordinates"))
	uint8 ConstCoordinate;

	/** only used if MipValue is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSample)
	int32 ConstMipValue;

	//~ Begin UObject Interface
#if WITH_EDITOR
	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void PostLoad() override;

	ENGINE_API void ApplyChannelNames();
	FParameterChannelNames GetTextureChannelNames() const
	{
		return ChannelNames;
	}
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	ENGINE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	ENGINE_API virtual FName GetInputName(int32 InputIndex) const override;
	ENGINE_API virtual int32 GetWidth() const override;
	ENGINE_API virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual int32 GetLabelPadding() override { return 8; }
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual bool CanIgnoreOutputIndex() { return true; }
	#endif // WITH_EDITOR
	//~ End UMaterialExpression Interface

	ENGINE_API void UpdateTextureResource(class UTexture* InTexture);
	
#if WITH_EDITOR
	ENGINE_API int32 CompileMipValue0(class FMaterialCompiler* Compiler);
	ENGINE_API int32 CompileMipValue1(class FMaterialCompiler* Compiler);
#endif // WITH_EDITOR
};
