// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "MaterialExpressionTextureSampleParameter.generated.h"

class UTexture;
struct FMaterialParameterInfo;

UCLASS(collapsecategories, abstract, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureSampleParameter : public UMaterialExpressionTextureSample
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureSampleParameter)
	FName ParameterName;

	/** GUID that should be unique within the material, this is used for parameter renaming. */
	UPROPERTY()
	FGuid ExpressionGUID;

	/** The name of the parameter Group to display in MaterialInstance Editor. Default is None group */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureSampleParameter)
	FName Group;

	/** Controls where the this parameter is displayed in a material instance parameter list.  The lower the number the higher up in the parameter list. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTextureSampleParameter)
	int32 SortPriority = 32;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	ENGINE_API virtual void Build(MIR::FEmitter& Emitter) override;
	ENGINE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	ENGINE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
	virtual bool CanRenameNode() const override { return true; }
	ENGINE_API virtual FString GetEditableName() const override;
	ENGINE_API virtual void SetEditableName(const FString& NewName) override;

	virtual bool HasAParameterName() const override { return true; }
	virtual FName GetParameterName() const override { return ParameterName; }
	virtual void SetParameterName(const FName& Name) override { ParameterName = Name; }
	ENGINE_API virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = Texture;
		OutMeta.Description = Desc;
		OutMeta.ExpressionGuid = ExpressionGUID;
		OutMeta.Group = Group;
		OutMeta.SortPriority = SortPriority;
		OutMeta.ChannelNames = GetTextureChannelNames();
		return true;
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::Texture)
		{
			if (SetParameterValue(Name, Meta.Value.Texture, Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	ENGINE_API bool SetParameterValue(FName InParameterName, UTexture* InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);

	/**
	 * Return true if the texture is a movie texture
	 *
	 * @param	InTexture - texture to test
	 * @param	OutMessage - if texture isn't valid, gives a description of the problem
	 * @return	true/false
	 */	
	ENGINE_API virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage);

	/**
	 *	Sets the default texture if none is set
	 */
	ENGINE_API virtual void SetDefaultTexture();
#endif // WITH_EDITOR

	virtual FGuid& GetParameterExpressionId() override
	{
		return ExpressionGUID;
	}
};
