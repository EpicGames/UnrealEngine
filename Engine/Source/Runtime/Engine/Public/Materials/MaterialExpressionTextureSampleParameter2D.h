// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialExpressionTextureSampleParameter2D.generated.h"

class UTexture;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureSampleParameter2D : public UMaterialExpressionTextureSampleParameter
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	ENGINE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface
	
	//~ Begin UMaterialExpressionTextureSampleParameter Interface
	ENGINE_API virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage) override;
	ENGINE_API virtual void SetDefaultTexture() override;
	//~ End UMaterialExpressionTextureSampleParameter Interface
#endif // WITH_EDITOR
};
