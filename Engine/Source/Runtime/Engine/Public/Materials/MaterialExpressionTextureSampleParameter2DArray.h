// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialExpressionTextureSampleParameter2DArray.generated.h"

UCLASS(collapsecategories, hidecategories = Object, MinimalAPI)
class UMaterialExpressionTextureSampleParameter2DArray : public UMaterialExpressionTextureSampleParameter
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// Begin UMaterialExpression Interface
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	// End UMaterialExpression Interface
#endif

	// Begin UMaterialExpressionTextureSampleParameter Interface
#if WITH_EDITOR
	virtual bool TextureIsValid( UTexture* InTexture, FString& OutMessage) override;
#endif
	virtual bool IsAllowedIn(const UObject* MaterialOrFunction) const override;
	virtual const TCHAR* GetRequirements();
	// End UMaterialExpressionTextureSampleParameter Interface
};