// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "MaterialExpressionTextureSampleParameterCube.generated.h"

class UTexture;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureSampleParameterCube : public UMaterialExpressionTextureSampleParameter
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	//~ End UMaterialExpression Interface

	//~ Begin UMaterialExpressionTextureSampleParameter Interface
	virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage) override;
	virtual void SetDefaultTexture() override;
	//~ End UMaterialExpressionTextureSampleParameter Interface
#endif // WITH_EDITOR
};
