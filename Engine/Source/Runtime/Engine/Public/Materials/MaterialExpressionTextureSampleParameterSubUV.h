// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionTextureSampleParameter2D.h"
#include "MaterialExpressionTextureSampleParameterSubUV.generated.h"

class UTexture;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTextureSampleParameterSubUV : public UMaterialExpressionTextureSampleParameter2D
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category=MaterialExpressionTextureSampleParameterSubUV)
	uint32 bBlend:1 = true;

#if WITH_EDITOR
	//~ Begin UMaterialExpressionTextureSampleParameter Interface
	virtual bool TextureIsValid(UTexture* InTexture, FString& OutMessage) override;
	//~ End UMaterialExpressionTextureSampleParameter Interface

	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface
#endif // WITH_EDITOR
};
