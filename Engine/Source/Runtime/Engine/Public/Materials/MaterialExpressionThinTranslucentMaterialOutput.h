// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionThinTranslucentMaterialOutput.generated.h"

/** Material output expression for setting absorption properties of thin translucent materials. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionThinTranslucentMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Input for the transmittance color for a view perpendicular to the surface. Valid range is [0,1]. */
	UPROPERTY()
	FExpressionInput TransmittanceColor;

	/** Input for the surface coverage of both the thin surface part and the material on top (controled using Opacity input of the graph root node). Valid range is [0,1]. */
	UPROPERTY()
	FExpressionInput SurfaceCoverage;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
