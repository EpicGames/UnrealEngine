// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTime.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionTime : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** This time continues advancing regardless of whether the game is paused. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTime)
	uint32 bIgnorePause:1;

	/** Enables or disables the Period value. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionTime, meta=(InlineEditConditionToggle))
	uint32 bOverride_Period:1 = false;

	/** Time will loop around once it gets to Period. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTime, meta = (editcondition = "bOverride_Period", ClampMin = "0.0", ToolTip = "Period at which to wrap around time"))
	float Period = 0.0f;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool NeedsRealtimePreview() override { return true; }
#endif
	//~ End UMaterialExpression Interface

};
