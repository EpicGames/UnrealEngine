// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTransform.generated.h"

UENUM()
enum EMaterialVectorCoordTransformSource : int
{
	/** Tangent space (relative to the surface) */
	TRANSFORMSOURCE_Tangent UMETA(DisplayName="Tangent Space"),
	
	/** Local space (relative to the rendered object, = object space) */
	TRANSFORMSOURCE_Local UMETA(DisplayName="Local Space"),
	
	/** World space, a unit is 1cm */
	TRANSFORMSOURCE_World UMETA(DisplayName="World Space"),
	
	/** View space (relative to the camera/eye, = camera space, differs from camera space in the shadow passes) */
	TRANSFORMSOURCE_View UMETA(DisplayName="View Space"),

	/** Camera space */
	TRANSFORMSOURCE_Camera  UMETA(DisplayName="Camera Space"),

	/** Particle space, deprecated value will be removed in a future release use instance space. */
	TRANSFORMSOURCE_ParticleWorld  UMETA(Hidden, DisplayName = "Mesh particle space"),

	/** Instance space (used to provide per instance transform, i.e. for Instanced Static Mesh / Particles). */
	TRANSFORMSOURCE_Instance UMETA(DisplayName = "Instance & Particle Space"),

	TRANSFORMSOURCE_MAX,
};

UENUM()
enum EMaterialVectorCoordTransform : int
{
	/** Tangent space (relative to the surface) */
	TRANSFORM_Tangent UMETA(DisplayName="Tangent Space"),
	
	/** Local space (relative to the rendered object, = object space) */
	TRANSFORM_Local UMETA(DisplayName="Local Space"),
	
	/** World space, a unit is 1cm */
	TRANSFORM_World UMETA(DisplayName="World Space"),
	
	/** View space (relative to the camera/eye, = camera space, differs from camera space in the shadow passes) */
	TRANSFORM_View UMETA(DisplayName="View Space"),

	/** Camera space */
	TRANSFORM_Camera  UMETA(DisplayName="Camera Space"),

	/** Particle space, deprecated value will be removed in a future release use instance space. */
	TRANSFORM_ParticleWorld UMETA(Hidden, DisplayName = "Mesh particle space"),

	/** Instance space (used to provide per instance transform, i.e. for Instanced Static Mesh / Particles). */
	TRANSFORM_Instance UMETA(DisplayName = "Instance & Particle Space"),

	TRANSFORM_MAX,
};

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionTransform : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** input expression for this transform */
	UPROPERTY()
	FExpressionInput Input;

	/** Source coordinate space of the FVector */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransform, meta=(DisplayName = "Source", ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EMaterialVectorCoordTransformSource> TransformSourceType = TRANSFORMSOURCE_Tangent;

	/** Destination coordinate space of the FVector */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransform, meta=(DisplayName = "Destination", ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EMaterialVectorCoordTransform> TransformType = TRANSFORM_World;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};



