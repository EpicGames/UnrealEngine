// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTransformPosition.generated.h"

UENUM()
enum EMaterialPositionTransformSource : int
{
	/** Local space */
	TRANSFORMPOSSOURCE_Local UMETA(DisplayName="Local Space"),
	
	/** Absolute world space */
	TRANSFORMPOSSOURCE_World UMETA(DisplayName="Absolute World Space"),

	/**
	  Like absolute world space, but the world origin is moved to the center of the tile the camera is in.
	  Logically similar to `fmod(CameraAbsoluteWorldPosition, TileSize) + CameraRelativeWorldPosition`.
	  This offers better precision and scalability than absolute world position.
	  Suitable as a position input for functions that tile based on world position, e.g. frac(Position / TileSize).
	  Works best when the tile size is a power of two.
	*/
	TRANSFORMPOSSOURCE_PeriodicWorld UMETA(DisplayName="Periodic World Space"),
	
	/** Translated world space, i.e. world space rotation and scale but with a position relative to the camera */
	TRANSFORMPOSSOURCE_TranslatedWorld  UMETA(DisplayName="Camera Relative World Space"),

	/** First person "space", which can be thought of as a transform that is applied to a position in translated world space. */
	TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld  UMETA(DisplayName = "First Person Space (Camera Relative World Space)"),

	/** View space (differs from camera space in the shadow passes) */
	TRANSFORMPOSSOURCE_View  UMETA(DisplayName="View Space"),

	/** Camera space */
	TRANSFORMPOSSOURCE_Camera  UMETA(DisplayName="Camera Space"),

	/** Particle space, deprecated value will be removed in a future release use instance space. */
	TRANSFORMPOSSOURCE_Particle UMETA(Hidden, DisplayName = "Mesh Particle Space"),

	/** Instance space (used to provide per instance transform, i.e. for Instanced Static Mesh / Particles). */
	TRANSFORMPOSSOURCE_Instance UMETA(DisplayName = "Instance & Particle Space"),

	TRANSFORMPOSSOURCE_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionTransformPosition : public UMaterialExpression
{
	GENERATED_BODY()

public:

	/** input expression for this transform */
	UPROPERTY()
	FExpressionInput Input;

	/** source format of the position that will be transformed */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransformPosition, meta=(DisplayName = "Source", ShowAsInputPin = "Advanced", InvalidEnumValues="TRANSFORMPOSSOURCE_FirstPersonTranslatedWorld"))
	TEnumAsByte<enum EMaterialPositionTransformSource> TransformSourceType = TRANSFORMPOSSOURCE_Local;

	/** type of transform to apply to the input expression */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransformPosition, meta=(DisplayName = "Destination", ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EMaterialPositionTransformSource> TransformType = TRANSFORMPOSSOURCE_Local;

	/** scale of the tiles used in Periodic World Space */
	UPROPERTY(meta=(DisplayName = "Periodic World Tile Size", ToolTip = "Distance the camera can move before the world origin is moved", RequiredInput = "false"))
	FExpressionInput PeriodicWorldTileSize;

	/** Interpolates between translated world space and first person space. Valid range is [0, 1], from translated world space to first person space. */
	UPROPERTY(meta = (DisplayName = "First Person Interpolation Alpha", ToolTip = "Interpolates between world space and first person space. Valid range is [0, 1], from world space to first person space. Defaults to 'ConstFirstPersonInterpolationAlpha' if not specified.", RequiredInput = "false"))
	FExpressionInput FirstPersonInterpolationAlpha;

	/** only used if PeriodicWorldTileSize is not hooked up */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransformPosition, meta=(DisplayName = "Periodic World Tile Size", ClampMin=0.0001, UIMax=524288.0, OverridingInputProperty = "PeriodicWorldTileSize", EditCondition=bUsesPeriodicWorldPosition, EditConditionHides, HideEditConditionToggle))
	float ConstPeriodicWorldTileSize = 32.0;

	/** Only used if FirstPersonInterpolationAlpha is not hooked up. Interpolates between translated world space and first person space. Valid range is [0, 1], from translated world space to first person space. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionTransformPosition, meta=(DisplayName = "First Person Interpolation Alpha", UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f, OverridingInputProperty = "FirstPersonInterpolationAlpha", EditCondition = bUsesFirstPersonInterpolationAlpha, EditConditionHides, HideEditConditionToggle))
	float ConstFirstPersonInterpolationAlpha = 1.0f;

private:
	UPROPERTY()
	bool bUsesPeriodicWorldPosition = false;
	
	UPROPERTY()
	bool bUsesFirstPersonInterpolationAlpha = false;

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetConnectorToolTip(int32 InputIndex, int32 OutputIndex, TArray<FString>& OutToolTip) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual TArrayView<FExpressionInput*> GetInputsView() override;
	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UMaterialExpression Interface

private:
	int32 GetAbsoluteInputIndex(int32 InputIndex);
};

