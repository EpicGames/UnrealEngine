// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionTruncateLWC.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionTruncateLWC : public UMaterialExpression
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FExpressionInput Input;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Em) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override { OutCaptions.Add(TEXT("TruncateLWC")); }
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("truncate lwc")); }
#endif
	//~ End UMaterialExpression Interface
};



