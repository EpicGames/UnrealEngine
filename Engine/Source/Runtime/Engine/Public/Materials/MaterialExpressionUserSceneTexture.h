// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionUserSceneTexture.generated.h"

UCLASS(MinimalAPI, collapsecategories, hidecategories=Object)
class UMaterialExpressionUserSceneTexture : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** UV in 0..1 range */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Coordinates;

	/** User Scene Texture (screen space texture with a user specified name, written by a previous PostProcess shader) to make a lookup into */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionUserSceneTexture, meta=(DisplayName = "User Scene Texture", ShowAsInputPin = "Advanced"))
	FName UserSceneTexture;
	
	/** Whether to use point sampled texture lookup (default) or using [bi-linear] filtered (can be slower, avoid faceted lock with distortions) */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionUserSceneTexture, meta=(DisplayName = "Filtered", ShowAsInputPin = "Advanced"))
	bool bFiltered = false;

	/** Whether to clamp the texture lookup.  Necessary when sampling a UserSceneTexture at reduced resolution with filtering, to avoid blending out of bounds pixels. */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionUserSceneTexture, meta=(DisplayName = "Clamped", ShowAsInputPin = "Advanced"))
	bool bClamped;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};

