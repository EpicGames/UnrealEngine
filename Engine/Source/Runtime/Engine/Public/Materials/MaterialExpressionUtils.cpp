// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionUtils.h"

#include "Containers/EnumAsByte.h"
#include "MaterialDomain.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Engine/Texture.h"
#include "Interfaces/ITargetPlatform.h"

namespace MaterialExpressionUtils
{

EMaterialSamplerType GetSamplerTypeForTexture( const UTexture* Texture, bool ForceNoVT)
{
	if (!Texture)
	{
		return SAMPLERTYPE_Color;
	}

	if (Texture->GetMaterialType() == MCT_TextureExternal)
	{
		return SAMPLERTYPE_External;
	}
	else if (Texture->LODGroup == TEXTUREGROUP_8BitData || Texture->LODGroup == TEXTUREGROUP_16BitData)
	{
		return SAMPLERTYPE_Data;
	}
			
	const bool bVirtual = ForceNoVT ? false : Texture->GetMaterialType() == MCT_TextureVirtual;

	switch (Texture->CompressionSettings)
	{
		case TC_Normalmap:
			return bVirtual ? SAMPLERTYPE_VirtualNormal : SAMPLERTYPE_Normal;
		case TC_Grayscale:
			return Texture->SRGB	? (bVirtual ?  SAMPLERTYPE_VirtualGrayscale : SAMPLERTYPE_Grayscale)
									: (bVirtual ? SAMPLERTYPE_VirtualLinearGrayscale : SAMPLERTYPE_LinearGrayscale);
		case TC_Alpha:
			return bVirtual ?  SAMPLERTYPE_VirtualAlpha : SAMPLERTYPE_Alpha;
		case TC_Masks:
			return bVirtual ?  SAMPLERTYPE_VirtualMasks : SAMPLERTYPE_Masks;
		case TC_DistanceFieldFont:
			return SAMPLERTYPE_DistanceFieldFont;
		default:
			return Texture->SRGB	? (bVirtual ? SAMPLERTYPE_VirtualColor : SAMPLERTYPE_Color) 
									: (bVirtual ? SAMPLERTYPE_VirtualLinearColor : SAMPLERTYPE_LinearColor);
	}
}

bool VerifySamplerType(const FString& TexturePathName, EMaterialSamplerType CorrectSamplerType, bool bSRGB, EMaterialSamplerType SamplerType, FString& OutErrorMessage)
{
	if (SamplerType != CorrectSamplerType)
	{
		UEnum* SamplerTypeEnum = StaticEnum<EMaterialSamplerType>();
		check(SamplerTypeEnum);

		const FString SamplerTypeDisplayName = SamplerTypeEnum->GetDisplayNameTextByValue(SamplerType).ToString();
		const FString TextureTypeDisplayName = SamplerTypeEnum->GetDisplayNameTextByValue(CorrectSamplerType).ToString();

		OutErrorMessage = FString::Printf(TEXT("Sampler type is %s, should be %s for %s"), *SamplerTypeDisplayName, *TextureTypeDisplayName, *TexturePathName);

		return false;
	}

	if ((SamplerType == SAMPLERTYPE_Normal || SamplerType == SAMPLERTYPE_Masks) && bSRGB)
	{
		UEnum* SamplerTypeEnum = StaticEnum<EMaterialSamplerType>();
		check(SamplerTypeEnum);

		const FString SamplerTypeDisplayName = SamplerTypeEnum->GetDisplayNameTextByValue(SamplerType).ToString();

		OutErrorMessage = FString::Printf(TEXT("To use '%s' as sampler type, SRGB must be disabled for %s"), *SamplerTypeDisplayName, *TexturePathName);

		return false;
	}

	return true;
}

bool VerifySamplerType(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, const UTexture* Texture, EMaterialSamplerType SamplerType, FString& OutErrorMessage)
{
	if (!Texture)
	{
		return true;
	}

	EMaterialSamplerType CorrectSamplerType = GetSamplerTypeForTexture(Texture);
	bool bIsVirtualTextured = IsVirtualSamplerType(SamplerType);
	if (bIsVirtualTextured && !UseVirtualTexturing(ShaderPlatform))
	{
		SamplerType = GetSamplerTypeForTexture(Texture, !bIsVirtualTextured);
	}

	// Textures whose SRGB flag is updated at runtime (e.g. UMediaTexture driven by the
	// currently playing sample) should accept Color and LinearColor sampler types
	// interchangeably. Otherwise the load-time default can mismatch an authored sampler
	// before any playback has occurred and cause the material to fail compile.
	if (Texture->HasRuntimeVariableSRGB())
	{
		auto IsColorOrLinearColorFn = [](EMaterialSamplerType Type)
		{
			return Type == SAMPLERTYPE_Color
				|| Type == SAMPLERTYPE_LinearColor
				|| Type == SAMPLERTYPE_VirtualColor
				|| Type == SAMPLERTYPE_VirtualLinearColor;
		};

		if (IsColorOrLinearColorFn(CorrectSamplerType) && IsColorOrLinearColorFn(SamplerType))
		{
			return true;
		}
	}

	return VerifySamplerType(Texture->GetPathName(), CorrectSamplerType, Texture->SRGB, SamplerType, OutErrorMessage);
}

} // MaterialExpressionUtils
