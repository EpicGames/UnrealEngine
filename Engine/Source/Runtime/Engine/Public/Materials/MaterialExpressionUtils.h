// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialExpressionUtils - Utility functions for the material translators
 */

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "RHIShaderPlatform.h"

struct FMaterialExternalCodeDeclaration;
class UTexture;
class ITargetPlatform;

namespace MaterialExpressionUtils
{
	
/**
 * Returns the default sampler type for the specified texture.
 * @param Texture - The texture for which the default sampler type will be returned.
 * @returns the default sampler type for the specified texture.
 */
ENGINE_API EMaterialSamplerType GetSamplerTypeForTexture( const UTexture* Texture, bool ForceNoVT = false );

/**
 * Verify that the texture and sampler type. Generates a compiler warning if they do not.
 */
ENGINE_API bool VerifySamplerType(const FString& TexturePathName, EMaterialSamplerType CorrectSamplerType, bool bSRGB, EMaterialSamplerType SamplerType, FString& OutErrorMessage);

/**
 * Verify that the texture and sampler type. Generates a compiler warning if they do not.
 * @param Texture - The texture to verify. A nullptr texture is considered valid!
 * @param SamplerType - The sampler type to verify.
 * @param OutErrorMessage - If 'false' is returned, will contain a message describing the error
 */
ENGINE_API bool VerifySamplerType(EShaderPlatform ShaderPlatform, const ITargetPlatform* TargetPlatform, const UTexture* Texture, EMaterialSamplerType SamplerType, FString& OutErrorMessage);

} // MaterialExpressionUtils
