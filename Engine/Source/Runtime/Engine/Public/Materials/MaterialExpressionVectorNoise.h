// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionVectorNoise.generated.h"

UENUM()
enum EVectorNoiseFunction : int
{
	/** Random color for each unit cell in 3D space.
	  * RGB output range 0 to 1
	  * R only = 9 instructions, RGB = 11 instructions
	  */
	VNF_CellnoiseALU UMETA(DisplayName = "Cellnoise"),

	/** Perlin-style noise with 3D vector/color output.
	  * RGB output range -1 to 1
	  * R only = ~83 instructions, RGB = ~125 instructions
	  */
	VNF_VectorALU UMETA(DisplayName = "Perlin 3D Noise"),

	/** Gradient of Perlin noise, useful for bumps.
	  * RGB = Gradient of scalar noise (signed 3D vector)
	  * A = Base scalar noise with range -1 to 1
	  * A only = ~83 instructions, RGBA = ~106 instructions
	  */
	VNF_GradientALU UMETA(DisplayName = "Perlin Gradient"),

	/** Curl of Perlin noise, useful for 3D flow directions.
	  * RGB = signed curl vector
	  * ~162 instructions
	  */
	VNF_CurlALU UMETA(DisplayName = "Perlin Curl"),

	/** Also known as Worley or Cellular noise.
	  * RGB = *position* of closest point at center of Voronoi cell
	  * A = distance to closest point with range 0 to about 4
	  * Quality levels 1-4 search 8, 16, 27 & 32 cells
	  * All ~20 instructions per cell searched
	  */
	VNF_VoronoiALU UMETA(DisplayName = "Voronoi"),

	VNF_MAX
};

UCLASS(MinimalAPI)
class UMaterialExpressionVectorNoise : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** 3 dimensional vector */
	UPROPERTY()
	FExpressionInput Position;

	/** Defines the reference space for the Position input. */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorNoise)
	EPositionOrigin WorldPositionOriginType = EPositionOrigin::Absolute;

	/** Noise function, affects performance and look */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorNoise, meta = (DisplayName = "Function", ShowAsInputPin = "Advanced"))
	TEnumAsByte<enum EVectorNoiseFunction> NoiseFunction;

	/** For noise functions where applicable, lower numbers are faster and lower quality, higher numbers are slower and higher quality */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionVectorNoise, meta=(UIMin = "1", UIMax = "4", ShowAsInputPin = "Advanced"))
	int32 Quality;

	/** Whether tile the noise pattern, useful for baking to seam-free repeating textures */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorNoise, meta = (ShowAsInputPin = "Advanced"))
	uint32 bTiling:1;

	/** How many units in each tile (if Tiling is on) 
	  * For Perlin noise functions, Tile Size must be a multiple of three
	  */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorNoise, meta=(UIMin = "4", ShowAsInputPin = "Advanced"))
	uint32 TileSize;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual FName GetInputName(int32 InputIndex) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



