// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionParameter.h"
#include "MaterialExpressionVectorParameter.generated.h"

struct FPropertyChangedEvent;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionVectorParameter : public UMaterialExpressionParameter
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionVectorParameter, meta = (ShowAsInputPin = "Primary"))
	FLinearColor DefaultValue;

	UPROPERTY(EditAnywhere, Category=CustomPrimitiveData)
	bool bUseCustomPrimitiveData = false;

	UPROPERTY(EditAnywhere, Category=CustomPrimitiveData, meta=(ClampMin="0"))
	uint8 PrimitiveDataIndex = 0;

	UPROPERTY(EditAnywhere, Category = ParameterCustomization, meta=(EditorOnlyLocalization))
	FParameterChannelNames ChannelNames;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual bool GetParameterValue(FMaterialParameterMetadata& OutMeta) const override
	{
		OutMeta.Value = DefaultValue;

		if (bUseCustomPrimitiveData)
		{
			OutMeta.PrimitiveDataIndex = PrimitiveDataIndex;
		}

		OutMeta.ChannelNames = ChannelNames;
		return Super::GetParameterValue(OutMeta);
	}
	virtual bool SetParameterValue(const FName& Name, const FMaterialParameterMetadata& Meta, EMaterialExpressionSetParameterValueFlags Flags) override
	{
		if (Meta.Value.Type == EMaterialParameterType::Vector)
		{
			if (SetParameterValue(Name, Meta.Value.AsLinearColor(), Flags))
			{
				if (EnumHasAnyFlags(Flags, EMaterialExpressionSetParameterValueFlags::AssignGroupAndSortPriority))
				{
					Group = Meta.Group;
					SortPriority = Meta.SortPriority;
				}
				return true;
			}
		}
		return false;
	}
#endif
	//~ End UMaterialExpression Interface

#if WITH_EDITOR
	virtual bool SetParameterValue(FName InParameterName, FLinearColor InValue, EMaterialExpressionSetParameterValueFlags Flags = EMaterialExpressionSetParameterValueFlags::None);

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;	

	void ApplyChannelNames();

	virtual void ValidateParameterName(const bool bAllowDuplicateName) override;
	virtual bool HasClassAndNameCollision(UMaterialExpression* OtherExpression) const override;
#endif

	virtual bool IsUsedAsChannelMask() const {return false;}

#if WITH_EDITOR
	FParameterChannelNames GetVectorChannelNames() const
	{
		return ChannelNames;
	}
#endif
};


