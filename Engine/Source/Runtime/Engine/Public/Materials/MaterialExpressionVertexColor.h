// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpressionExternalCodeBase.h"
#include "MaterialExpressionVertexColor.generated.h"

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionVertexColor : public UMaterialExpressionExternalCodeBase
{
	GENERATED_UCLASS_BODY()


	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override { return FText::FromString(TEXT("vertex color")); }
#endif
	//~ End UMaterialExpression Interface
};



