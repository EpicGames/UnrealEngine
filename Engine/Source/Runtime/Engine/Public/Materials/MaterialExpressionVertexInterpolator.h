// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialValueType.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionVertexInterpolator.generated.h"

UCLASS(MinimalAPI)
class UMaterialExpressionVertexInterpolator : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	ENGINE_API int32 CompileInput(class FMaterialCompiler* Compiler, int32 AssignedInterpolatorIndex);
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool HasCustomSourceOutput() override { return true; }

	virtual FExpressionInput* GetInput(int32 InputIndex) override;
	virtual FName GetInputName(int32 InputIndex) const override { return TEXT("VS"); }

#endif

	virtual FString GetFunctionName() const override { return TEXT("VertexInterpolator"); }

	int32				InterpolatorIndex;
	EMaterialValueType	InterpolatedType;
	int32				InterpolatorOffset;

#if WITH_EDITOR
	// Errors encountered during pre-translation to be appended if the interpolator is actually in-use
	TArray<TObjectPtr<UMaterialExpression>> CompileErrorExpressions;
	TArray<FString> CompileErrors;	
#endif
};



