// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionViewProperty.generated.h"

UENUM()
enum EMaterialExposedViewProperty : int
{	
	/** Horizontal and vertical size of the view's buffer in pixels */
	MEVP_BufferSize UMETA(DisplayName="Render Target Size"),
	/** Horizontal and vertical field of view angles in radian */
	MEVP_FieldOfView UMETA(DisplayName="Field Of View"),
	/** Tan(FieldOfView * 0.5) */
	MEVP_TanHalfFieldOfView UMETA(DisplayName="Tan(0.5 * Field Of View)"),
	/** Horizontal and vertical size of the view in pixels */
	MEVP_ViewSize UMETA(DisplayName="View Size"),
	/** Absolute world space view position (differs from the camera position in the shadow passes) */
	MEVP_WorldSpaceViewPosition UMETA(DisplayName="View Position (Absolute World Space)"),
	/** Absolute world space camera position */
	MEVP_WorldSpaceCameraPosition UMETA(DisplayName = "Camera Position (Absolute World Space)"),
	/** Horizontal and vertical position of the viewport in pixels within the buffer. */
	MEVP_ViewportOffset UMETA(DisplayName = "Viewport Offset"),
	/** Number of temporal AA sample used across multiple to converge to anti aliased output. */
	MEVP_TemporalSampleCount UMETA(DisplayName = "Temporal Sample Count"),
	/** Index of the Temporal AA jitter for this frame. */
	MEVP_TemporalSampleIndex UMETA(DisplayName = "Temporal Sample Index"),
	/** Offset of the temporal sample for this frame in pixel size. */
	MEVP_TemporalSampleOffset UMETA(DisplayName = "Temporal Sample Offset"),
	/** Mip Level that Runtime Virtual Texture Output is rendering to. */
	MEVP_RuntimeVirtualTextureOutputLevel UMETA(DisplayName = "Runtime Virtual Texture Output Level"),
	/** World space derivatives for Runtime Virtual Texture Output. */
	MEVP_RuntimeVirtualTextureOutputDerivative UMETA(DisplayName = "Runtime Virtual Texture Output Derivative"),
	/** Pre Exposure */
	MEVP_PreExposure UMETA(DisplayName = "Pre-Exposure"),
	/** Maximum mip level of Runtime Virtual Texture that Runtime Virtual Texture Output is rendering to. */
	MEVP_RuntimeVirtualTextureMaxLevel UMETA(DisplayName = "Runtime Virtual Texture Max Level"),
	/** Screen percentage at which the rendering resolution happens, to allow tech-art to remain consistent with dynamic resolution. */
	MEVP_ResolutionFraction UMETA(DisplayName = "ScreenPercentage / 100"),
	/** Post process volume user flags, useful for varying the behavior of a material per view */
	MEVP_PostVolumeUserFlags UMETA(DisplayName = "Post Process Volume User Flags"),
	/** Horizontal and vertical first person field of view angles in radian */
	MEVP_FirstPersonFieldOfView UMETA(DisplayName = "First Person Field Of View"),
	/** Tan(FirstPersonFieldOfView * 0.5) */
	MEVP_FirstPersonTanHalfFieldOfView UMETA(DisplayName = "Tan(0.5 * First Person Field Of View)"),
	/** Scaling factor applied to first person primitives to keep them from intersecting with the scene. */
	MEVP_FirstPersonScale UMETA(DisplayName = "First Person Scale"),
	/** Near plane. */
	MEVP_NearPlane UMETA(DisplayName = "View Near Plane"),

	MEVP_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionViewProperty : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
	
	/** View input property to be accessed */
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionViewProperty, meta=(DisplayName = "View Property", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EMaterialExposedViewProperty> Property = MEVP_FieldOfView;
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif
	//~ End UMaterialExpression Interface
};
