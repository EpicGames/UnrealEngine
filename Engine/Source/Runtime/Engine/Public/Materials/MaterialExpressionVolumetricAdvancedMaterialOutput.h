// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "UObject/ObjectMacros.h"
#include "Shader/ShaderTypes.h"
#include "MaterialExpressionVolumetricAdvancedMaterialOutput.generated.h"

/** Material output expression for writing advanced volumetric material properties. */
UCLASS(MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionVolumetricAdvancedMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** Parameter 'g' input to the phase function  describing how much forward(g<0) or backward (g>0) light scatter around. Valid range is [-1,1]. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Parameter 'g' input to the phase function  describing how much forward(g<0) or backward (g>0) light scatter around. Valid range is [-1,1]. Defaults to ConstPhaseG from properties panel if not specified. Evaluated per sample if EvaluatePhaseOncePerSample is true."))
	FExpressionInput PhaseG;
	
	/** Parameter 'g' input to the second phase function  describing how much forward(g<0) or backward (g>0) light scatter around. Valid range is [-1,1]. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Parameter 'g' input to the second phase function  describing how much forward(g<0) or backward (g>0) light scatter around. Valid range is [-1,1]. Defaults to ConstPhaseG2 from properties panel if not specified. Evaluated per sample if EvaluatePhaseOncePerSample is true."))
	FExpressionInput PhaseG2;
	
	/** Lerp factor when blending the two phase functions parameterized by G and G2. Valid range is [0,1]. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Lerp factor when blending the two phase functions parameterized by G and G2. Valid range is [0,1] Defaults to ConstPhaseBlend from properties panel if not specified. Evaluated per sample if EvaluatePhaseOncePerSample is true."))
	FExpressionInput PhaseBlend;
	

	/** Multi-scattering approximation: represents how much contribution each successive octave will add. Valid range is [0,1], from low to high contribution. Defaults to ConstMultiScatteringContribution from properties panel if not specified. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Multi-scattering approximation: represents how much contribution each successive octave will add. Evaluated per pixel. Valid range is [0,1], from low to high contribution. Defaults to ConstMultiScatteringContribution from properties panel if not specified. Evaluated per pixel (globally)."))
	FExpressionInput MultiScatteringContribution;
	
	/** Multi-scattering approximation: represents how much occlusion will be reduced for each successive octave. Valid range is [0,1], from low to high occlusion. Defaults to ConstMultiScatteringOcclusion from properties panel if not specified. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Multi-scattering approximation: represents how much occlusion will be reduced for each successive octave. Evaluated per pixel. Valid range is [0,1], from low to high occlusion. Defaults to ConstMultiScatteringOcclusion from properties panel if not specified. Evaluated per pixel (globally)."))
	FExpressionInput MultiScatteringOcclusion;
	
	/** Multi-scattering approximation: represents how much the phase will become isotropic for each successive octave. Valid range is [0,1], from anisotropic to isotropic phase. Defaults to ConstMultiScatteringEccentricity from properties panel if not specified. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Multi-scattering approximation: represents how much the phase will become isotropic for each successive octave. Evaluated per pixel. Valid range is [0,1], from anisotropic to isotropic phase. Defaults to ConstMultiScatteringEccentricity from properties panel if not specified. Evaluated per pixel (globally)."))
	FExpressionInput MultiScatteringEccentricity;

	/** This is a 3-components float vector. The X component must represent the participating medium conservative density. This is used to accelerate the ray marching by early skipping expensive material evaluation. For example, a simple top down 2D density texture would be enough to help by not evaluating the material in empty regions. The Y and Z components can contain parameters that can be recovered during the material evaluation using the VolumetricAdvancedMaterialInput node. Evaluated per sample. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "This is a 4-components float vector. The X component must represent the participating medium conservative density. This is used to accelerate the ray marching by early skipping expensive material evaluation. For example, the bare minimum would be a top down 2D density texture skipping evaluating the full material in empty regions (better, would be a simple blobby volume representation of clouds). The Y, Z and W components can contain parameters that can be recovered during the material graph evaluation using the VolumetricAdvancedMaterialInput node. Evaluated per sample."))
	FExpressionInput ConservativeDensity;

	/** Only used if PhaseG is not hooked up. Parameter 'g' input to the phase function  describing how much forward(g<0) or backward (g>0) light scatter around. */
	UPROPERTY(EditAnywhere, Category = "Phase", meta = (OverridingInputProperty = "PhaseG", UIMin = -0.999f, UIMax = 0.999f, ClampMin = -0.999f, ClampMax = 0.999f))
	float ConstPhaseG = 0.0f;

	/** Only used if PhaseG2 is not hooked up. Parameter 'g' input to the second phase function  describing how much forward(g<0) or backward (g>0) light scatter around. Valid range is [-1,1]. */
	UPROPERTY(EditAnywhere, Category = "Phase", meta = (OverridingInputProperty = "PhaseG2", UIMin = -0.999f, UIMax = 0.999f, ClampMin = -0.999f, ClampMax = 0.999f))
	float ConstPhaseG2 = 0.0f;

	/** Only used if PhaseBlend is not hooked up. Lerp factor when blending the two phase functions parameterized by G and G2. Valid range is [0,1]. */
	UPROPERTY(EditAnywhere, Category = "Phase", meta = (OverridingInputProperty = "PhaseBlend", UIMin = 0.0f, UIMax = 1.0f, ClampMin = -0.999f, ClampMax = 0.999f))
	float ConstPhaseBlend = 0.0f;

	/** Set this to true to force the phase function to be evaluated per sample, instead once per pixel (globally). Per sample evaluation is slower. */
	UPROPERTY(EditAnywhere, Category = "Phase")
	bool PerSamplePhaseEvaluation = false;

	/** How many octave to use for the multiple-scattering approximation. This makes the shader more expensive so you should only use 0 or 1 for better performance, and tweak multiple scattering parameters accordingly. 0 means single scattering only. The maximum value is 2 (expenssive). */
	UPROPERTY(EditAnywhere, Category = "Multi-Scattering", meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 2))
	uint32 MultiScatteringApproximationOctaveCount = 0;

	/** Only used if MultiScatteringContribution is not hooked up. Multi-scattering approximation: represents how much contribution each successive octave will add. Valid range is [0,1], from low to high contribution */
	UPROPERTY(EditAnywhere, Category = "Multi-Scattering", meta = (OverridingInputProperty = "MultiScatteringContribution", UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f))
	float ConstMultiScatteringContribution = 0.5f;

	/** Only used if MultiScatteringOcclusion is not hooked up. Multi-scattering approximation: represents how much occlusion will be reduced for each successive octave. Valid range is [0,1], from low to high occlusion. */
	UPROPERTY(EditAnywhere, Category = "Multi-Scattering", meta = (OverridingInputProperty = "MultiScatteringOcclusion", UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f))
	float ConstMultiScatteringOcclusion = 0.5f;

	/** Only used if MultiScatteringEccentricity is not hooked up. Multi-scattering approximation: represents how much the phase will become isotropic for each successive octave. Valid range is [0,1], from anisotropic to isotropic phase. */
	UPROPERTY(EditAnywhere, Category = "Multi-Scattering", meta = (OverridingInputProperty = "MultiScatteringEccentricity", UIMin = 0.0f, UIMax = 1.0f, ClampMin = 0.0f, ClampMax = 1.0f))
	float ConstMultiScatteringEccentricity = 0.5f;

	/** Sample the shadowed lighting contribution from the ground onto the medium (single scattering). This adds some costs to the tracing when enabled.*/
	UPROPERTY(EditAnywhere, Category = "Options")
	bool bGroundContribution = false;

	/** Set this for the material to only be considered grey scale, only using the R chanel of the input parameters internally. The lighting will still be colored. This is an optimisation.*/
	UPROPERTY(EditAnywhere, Category = "Options")
	bool bGrayScaleMaterial = false;

	/** Disable this to use the cloud shadow map instead of secondary raymarching. This is usually enough for clouds viewed from the ground and it result in a performance boost. Shadow now have infinite length but also becomes less accurate and gray scale.*/
	UPROPERTY(EditAnywhere, Category = "Options")
	bool bRayMarchVolumeShadow = true;

	/** Set whether multiple scattering contribution entry is clamped in [0,1] or not. When disabled, the artist is in charge for ensuring the visual remain in a reasonable brighness range.*/
	UPROPERTY(EditAnywhere, Category = "Options")
	bool bClampMultiScatteringContribution = true;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	//~ End UMaterialExpression Interface

	bool GetEvaluatePhaseOncePerSample() const;
	uint32 GetMultiScatteringApproximationOctaveCount() const;

#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};

/** USed to help the cloud system to fast skip empty space areas when ray marching. */
UCLASS(Experimental, MinimalAPI, collapsecategories, hidecategories = Object)
class UMaterialExpressionVolumetricCloudEmptySpaceSkippingOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	/** ContainsMatter must be 1 when the volume is occupied by any matter. This is for the tracing to later not miss it. Otherwise it can be 0 to accelerate the tracing by skipping that area. */
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Specify 0 if no matter (cloud or participating media) can be found within the area, otherwise should be set > 0."))
	FExpressionInput ContainsMatter;

public:
#if WITH_EDITOR
	//~ Begin UMaterialExpression Interface
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override;
	virtual FString GetFunctionName() const override;
	virtual FString GetDisplayName() const override;
	//~ End UMaterialExpressionCustomOutput Interface
};
