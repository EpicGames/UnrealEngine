// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionWorldPosition.generated.h"

/** Specifies which shader generated offsets should included in the world position (displacement/WPO etc.) */
UENUM()
enum EWorldPositionIncludedOffsets : int
{
	/** Absolute world position with all material shader offsets applied */
	WPT_Default UMETA(DisplayName="Absolute World Position (Including Material Shader Offsets)"),

	/** Absolute world position with no material shader offsets applied */
	WPT_ExcludeAllShaderOffsets UMETA(DisplayName="Absolute World Position (Excluding Material Shader Offsets)"),

	/** Camera relative world position with all material shader offsets applied */
	WPT_CameraRelative UMETA(DisplayName="Camera Relative World Position (Including Material Shader Offsets)"),

	/** Camera relative world position with no material shader offsets applied */
	WPT_CameraRelativeNoOffsets UMETA(DisplayName="Camera Relative World Position (Excluding Material Shader Offsets)"),

	WPT_MAX
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionWorldPosition : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(EditAnywhere, Category=UMaterialExpressionWorldPosition, meta=(DisplayName = "Shader Offsets", ShowAsInputPin = "Advanced"))
	TEnumAsByte<EWorldPositionIncludedOffsets> WorldPositionShaderOffset = WPT_Default;
	
	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual void Build(MIR::FEmitter& Emitter) override;
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual FText GetKeywords() const override {return FText::FromString(TEXT("position"));}
#endif
	//~ End UMaterialExpression Interface
};



