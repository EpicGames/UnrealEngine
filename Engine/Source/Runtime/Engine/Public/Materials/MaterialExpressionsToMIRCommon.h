// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Materials/MaterialIREmitter.h"

namespace MaterialToMIR
{

// Emits and returns the Texture Coordinates value with given index (i.e. TexCoord0, TexCoord1, etc).
// Note: returned type is a float2
MIR::FValueRef EmitTexCoord(MIR::FEmitter& Em, int32 Index);

// Emits and returns the world space pixel normal value (float3).
MIR::FValueRef EmitPixelNormalWS(MIR::FEmitter& Em);

// Emits and returns the vertex normal value (float3).
MIR::FValueRef EmitVertexNormal(MIR::FEmitter& Em);

// Emits and returns the vertex tangent value (float3).
MIR::FValueRef EmitVertexTangent(MIR::FEmitter& Em);

// Emits and returns the custom primitive data value with given index (float).
MIR::FValueRef EmitCustomPrimitiveDataFloat1(MIR::FEmitter& Em, int32 Index);

// Emits and returns the custom primitive data value starting from given index (float4).
// This is equilent to `Em.Vector(EmitCustomPrimitiveDataFloat1(Em, 10), ..., EmitCustomPrimitiveDataFloat1(Em, 13))`.
MIR::FValueRef EmitCustomPrimitiveDataFloat4(MIR::FEmitter& Em, int32 Index);

// Verifies that Input is connected to a real expression, tracing through any reroute nodes iteratively.
// Emits a compile error via Em.Errorf if the input is unconnected (or only connected through a dangling reroute chain).
void CheckInputIsConnected(MIR::FEmitter& Em, FExpressionInput& Input, const TCHAR* InputName);

} // MaterialToMIR
#endif // WITH_EDITOR
