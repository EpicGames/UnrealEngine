// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "Materials/MaterialFunctionInterface.h"
#include "StaticParameterSet.h"
#include "MaterialDomain.h"
#include "MaterialExpression.h"
#include "MaterialFunction.generated.h"

class UMaterial;
class UTexture;
struct FPropertyChangedEvent;
class UMaterialExpression;

UCLASS(MinimalAPI, Optional)
class UMaterialFunctionEditorOnlyData : public UMaterialFunctionInterfaceEditorOnlyData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FMaterialExpressionCollection ExpressionCollection;

	//~ Begin UObject Interface.
	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	//~ End UObject Interface.
};

/**
 * A Material Function is a collection of material expressions that can be reused in different materials
 */
UCLASS(BlueprintType, hidecategories=object, MinimalAPI)
class UMaterialFunction : public UMaterialFunctionInterface
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	virtual const UClass* GetEditorOnlyDataClass() const override { return UMaterialFunctionEditorOnlyData::StaticClass(); }

	UMaterialFunctionEditorOnlyData* GetEditorOnlyData() { return CastChecked<UMaterialFunctionEditorOnlyData>(Super::GetEditorOnlyData()); }
	const UMaterialFunctionEditorOnlyData* GetEditorOnlyData() const { return CastChecked<UMaterialFunctionEditorOnlyData>(Super::GetEditorOnlyData()); }
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	/** Used in the material editor, points to the function asset being edited, which this function is just a preview for. */
	UPROPERTY(transient)
	TObjectPtr<class UMaterialFunction> ParentFunction;
#endif // WITH_EDITORONLY_DATA

	/** Description of the function which will be displayed as a tooltip wherever the function is used. */
	UPROPERTY(EditAnywhere, Category=MaterialFunction, AssetRegistrySearchable)
	FString Description;

	/** Name of the function to be displayed on the node within the material editor instead of the asset name. */
	UPROPERTY(EditAnywhere, Category=MaterialFunction, AssetRegistrySearchable)
	FString UserExposedCaption;

	/** Whether to list this function in the material function library, which is a window in the material editor that lists categorized functions. */
	UPROPERTY(EditAnywhere, Category=MaterialFunction, AssetRegistrySearchable)
	uint8 bExposeToLibrary:1;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(All, "bPrefixParameterNames has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bPrefixParameterNames has been deprecated."))
	uint8 bPrefixParameterNames_DEPRECATED:1;

	UE_DEPRECATED(5.8, "Experimental bEnableExecWire is no longer supported.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Experimental bEnableExecWire is no longer supported."))
	uint8 bEnableExecWire_DEPRECATED : 1;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = MaterialFunction, meta=(DisplayName="Enable new material translator"))
	uint8 bEnableNewHLSLGenerator : 1;

#if WITH_EDITORONLY_DATA
	/** 
	 * Categories that this function belongs to in the material function library.  
	 * Ideally categories should be chosen carefully so that there are not too many.
	 */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FString> LibraryCategories_DEPRECATED;

	/** 
	 * Categories that this function belongs to in the material function library.  
	 * Ideally categories should be chosen carefully so that there are not too many.
	 */
	UPROPERTY(EditAnywhere, Category=MaterialFunction, AssetRegistrySearchable)
	TArray<FText> LibraryCategoriesText;
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterial> PreviewMaterial;

	// The UMaterial which represents this function while the function itself is open in the material editor
	TObjectPtr<UMaterial> EditorMaterial;

	UPROPERTY()
	TArray<TObjectPtr<class UMaterialExpressionMaterialFunctionCall>> DependentFunctionExpressionCandidates;

	/** Determines the blend mode when previewing a material function. */
	UPROPERTY(EditAnywhere, Category = Preview, AssetRegistrySearchable)
	TEnumAsByte<enum EBlendMode> PreviewBlendMode = BLEND_Opaque;

	/** The domain to use when previewing a material function. */
	UPROPERTY(EditAnywhere, Category = Preview, AssetRegistrySearchable)
	TEnumAsByte<enum EMaterialDomain> PreviewMaterialDomain;

	class UMaterialGraph* MaterialGraph = nullptr;

	/* Whether all expressions in the function loaded correctly. */
	uint8 bAllExpressionsLoadedCorrectly : 1;

private:
	/** Transient flag used to track re-entrance in recursive functions like IsDependent. */
	UPROPERTY(transient)
	uint8 bReentrantFlag:1;
#endif // WITH_EDITORONLY_DATA

public:
	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	//~ End UObject Interface.

	void SetMaterialFunctionUsage(EMaterialFunctionUsage Usage) { MaterialFunctionUsage = Usage; }

	//~ Begin UMaterialFunctionInterface interface
	virtual EMaterialFunctionUsage GetMaterialFunctionUsage() override { return MaterialFunctionUsage; }

#if WITH_EDITOR
	/** Recursively update all function call expressions in this function, or in nested functions. */
	virtual void UpdateFromFunctionResource() override;

	/** Get the inputs and outputs that this function exposes, for a function call expression to use. */
	virtual void GetInputsAndOutputs(TArray<struct FFunctionExpressionInput>& OutInputs, TArray<struct FFunctionExpressionOutput>& OutOutputs) const override;

	virtual void ForceRecompileForRendering(FMaterialUpdateContext& UpdateContext, UMaterial* InPreviewMaterial) override;
#endif

	virtual bool ValidateFunctionUsage(class FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output) override;

#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, const struct FFunctionExpressionOutput& Output) override;

	/** Called during compilation before entering the function. */
	virtual void LinkIntoCaller(const TArray<FFunctionExpressionInput>& CallerInputs) override;

	virtual void UnlinkFromCaller() override;
#endif

#if WITH_EDITORONLY_DATA
	/** @return true if this function is dependent on the passed in function, directly or indirectly. */
	virtual bool IsDependent(UMaterialFunctionInterface* OtherFunction) override;

	/**
	 * Iterates all functions that this function is dependent on, directly or indrectly.
	 *
	 * @param Predicate a visitor predicate returning true to continue iteration, false to break
	 *
	 * @return true if all dependent functions were visited, false if the Predicate did break iteration
	 */
	ENGINE_API virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const override;

	/** Returns an array of the functions that this function is dependent on, directly or indirectly. */
	ENGINE_API virtual void GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const override;

	/** Returns If returns an empty string, use the default class name for the material function. Otherwise, the string will be the name shown when the function is exposed to users in the material graph as a node, or from the contextual menu when searching for nodes. */
	virtual FString GetUserExposedCaption() const override { return UserExposedCaption; }

	/** Retrieves an array of all custom output expressions in a Material Function */
	void GetAllCustomOutputExpressions(TArray<class UMaterialExpressionCustomOutput*>& OutCustomOutputs) const;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual UMaterialInterface* GetPreviewMaterial() override;

	virtual void UpdateInputOutputTypes() override;

	virtual void UpdateDependentFunctionCandidates();

	/**
	 * Checks whether a Material Function is arranged in the old style, with inputs flowing from right to left
	 */
	virtual bool HasFlippedCoordinates() const override;
#endif

	virtual UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) override { return this; }
	virtual const UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) const override { return this; }
#if WITH_EDITORONLY_DATA
	ENGINE_API TConstArrayView<TObjectPtr<UMaterialExpression>> GetExpressions() const;
	ENGINE_API TConstArrayView<TObjectPtr<UMaterialExpressionComment>> GetEditorComments() const;

	ENGINE_API const FMaterialExpressionCollection& GetExpressionCollection() const;
	ENGINE_API FMaterialExpressionCollection& GetExpressionCollection();
	ENGINE_API void AssignExpressionCollection(const FMaterialExpressionCollection& InCollection);
#endif // WITH_EDITORONLY_DATA
	//~ End UMaterialFunctionInterface interface


#if WITH_EDITOR
	ENGINE_API bool SetParameterValueEditorOnly(const FName& ParameterName, const FMaterialParameterMetadata& Meta);
	ENGINE_API bool SetVectorParameterValueEditorOnly(FName ParameterName, FLinearColor InValue);
	ENGINE_API bool SetScalarParameterValueEditorOnly(FName ParameterName, float InValue);
	ENGINE_API bool SetTextureParameterValueEditorOnly(FName ParameterName, class UTexture* InValue);
	ENGINE_API bool SetRuntimeVirtualTextureParameterValueEditorOnly(FName ParameterName, class URuntimeVirtualTexture* InValue);
	ENGINE_API bool SetSparseVolumeTextureParameterValueEditorOnly(FName ParameterName, class USparseVolumeTexture* InValue);
	ENGINE_API bool SetFontParameterValueEditorOnly(FName ParameterName, class UFont* InFontValue, int32 InFontPage);
	ENGINE_API bool SetStaticComponentMaskParameterValueEditorOnly(FName ParameterName, bool R, bool G, bool B, bool A, FGuid OutExpressionGuid);
	ENGINE_API bool SetStaticSwitchParameterValueEditorOnly(FName ParameterName, bool OutValue, FGuid OutExpressionGuid);

	inline bool GetReentrantFlag() const { return bReentrantFlag; }
	inline void SetReentrantFlag(bool bIsReentrant) { bReentrantFlag = bIsReentrant; }

	virtual bool IsUsingNewHLSLGenerator() const override;
#endif // WITH_EDITOR

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<UMaterialExpression>> FunctionExpressions_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<class UMaterialExpressionComment>> FunctionEditorComments_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};
