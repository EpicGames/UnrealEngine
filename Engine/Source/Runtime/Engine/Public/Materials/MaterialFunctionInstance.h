// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialInstance.h"
#include "StaticParameterSet.h"
#include "MaterialFunctionInstance.generated.h"

/**
 * A material function instance defines parameter overrides for a parent material function.
 */
UCLASS(hidecategories=object, MinimalAPI)
class UMaterialFunctionInstance : public UMaterialFunctionInterface
{
	GENERATED_UCLASS_BODY()

	ENGINE_API void SetParent(UMaterialFunctionInterface* NewParent);

	virtual EMaterialFunctionUsage GetMaterialFunctionUsage() override;

	/** Parent function. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialFunctionInstance, AssetRegistrySearchable)
	TObjectPtr<UMaterialFunctionInterface> Parent;

	/** Base function. */
	UPROPERTY(AssetRegistrySearchable)
	TObjectPtr<UMaterialFunctionInterface> Base;

	/** Scalar parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FScalarParameterValue> ScalarParameterValues;

	/** Vector parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FVectorParameterValue> VectorParameterValues;

	/** DoubleVector parameters. */
	UPROPERTY(EditAnywhere, Category = MaterialFunctionInstance)
	TArray<struct FDoubleVectorParameterValue> DoubleVectorParameterValues;

	/** Texture parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FTextureParameterValue> TextureParameterValues;

	/** Texture Collection parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FTextureCollectionParameterValue> TextureCollectionParameterValues;

	/** Parameter Collection parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FParameterCollectionParameterValue> ParameterCollectionParameterValues;

	/** Font parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FFontParameterValue> FontParameterValues;

	/** Static switch parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FStaticSwitchParameter> StaticSwitchParameterValues;

	/** Static component mask parameters. */
	UPROPERTY(EditAnywhere, Category=MaterialFunctionInstance)
	TArray<struct FStaticComponentMaskParameter> StaticComponentMaskParameterValues;

	/** Runtime virtual texture parameters. */
	UPROPERTY(EditAnywhere, Category = MaterialFunctionInstance)
	TArray<struct FRuntimeVirtualTextureParameterValue> RuntimeVirtualTextureParameterValues;

	/** Sparse volume texture parameters. */
	UPROPERTY(EditAnywhere, Category = MaterialFunctionInstance)
	TArray<struct FSparseVolumeTextureParameterValue> SparseVolumeTextureParameterValues;

#if WITH_EDITOR
	ENGINE_API void UpdateParameterSet();
	ENGINE_API void OverrideMaterialInstanceParameterValues(class UMaterialInstance* Instance);
#endif // WITH_EDITOR

	//~ Begin UMaterialFunctionInterface interface
#if WITH_EDITOR
	virtual void UpdateFromFunctionResource() override;
	virtual void GetInputsAndOutputs(TArray<struct FFunctionExpressionInput>& OutInputs, TArray<struct FFunctionExpressionOutput>& OutOutputs) const override;
#endif
	virtual bool ValidateFunctionUsage(class FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output) override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, const struct FFunctionExpressionOutput& Output) override;
	virtual void LinkIntoCaller(const TArray<FFunctionExpressionInput>& CallerInputs) override;
	virtual void UnlinkFromCaller() override;
#endif

#if WITH_EDITORONLY_DATA
	virtual void Serialize(FArchive& Ar) override;
	virtual bool IsDependent(UMaterialFunctionInterface* OtherFunction) override;
	ENGINE_API virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const override;
	ENGINE_API virtual void GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const override;
#endif

#if WITH_EDITOR
	virtual UMaterialInterface* GetPreviewMaterial() override;
	virtual void UpdateInputOutputTypes() override;
	virtual bool HasFlippedCoordinates() const override;
#endif

	virtual UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) override;
	virtual const UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) const override;

#if WITH_EDITOR
	virtual bool GetParameterOverrideValue(EMaterialParameterType Type, const FName& ParameterName, FMaterialParameterMetadata& OutValue, FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) const override;
#endif // WITH_EDITOR
	//~ End UMaterialFunctionInterface interface

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<class UMaterialInstanceConstant> PreviewMaterial;
#endif // WITH_EDITORONLY_DATA
};
