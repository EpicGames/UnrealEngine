// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpression.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "MaterialRecursionGuard.h"
#include "StaticParameterSet.h"

#if WITH_EDITOR
#include "Materials/MaterialExpression.h"
#endif

#include "MaterialFunctionInterface.generated.h"

class UMaterial;
class UMaterialFunction;
class UTexture;
struct FPropertyChangedEvent;
class FMaterialUpdateContext;
class UMaterialInterface;
class UMaterialExpression;
struct FFunctionExpressionOutput;
class UMaterialParameterCollection;

/** Usage set on a material function determines feature compatibility and validation. */
UENUM()
enum class EMaterialFunctionUsage : uint8
{
	Default,
	MaterialLayer,
	MaterialLayerBlend
};

#define DEFAULT_SUBSTRATE_MATERIALLAYERBLEND_PATH TEXT("/Engine/EngineMaterials/Blends/MLB_Horizontal")

using FMFRecursionGuard = TMaterialRecursionGuard<class UMaterialFunctionInterface>;

UCLASS(Optional)
class UMaterialFunctionInterfaceEditorOnlyData : public UObject
{
	GENERATED_BODY()
public:
	
	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.
};

/**
 * A Material Function is a collection of material expressions that can be reused in different materials
 */
UCLASS(BlueprintType, abstract, hidecategories=object, MinimalAPI)
class UMaterialFunctionInterface : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
protected:
	friend class UMaterialFunctionInterfaceEditorOnlyData;

	UPROPERTY()
	TObjectPtr<UMaterialFunctionInterfaceEditorOnlyData> EditorOnlyData;

	ENGINE_API virtual const UClass* GetEditorOnlyDataClass() const;

public:
	UMaterialFunctionInterfaceEditorOnlyData* GetEditorOnlyData() { return EditorOnlyData; }
	const UMaterialFunctionInterfaceEditorOnlyData* GetEditorOnlyData() const { return EditorOnlyData; }
#endif // WITH_EDITORONLY_DATA

public:

	//~ Begin UObject Interface.
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;
	//~ End UObject Interface.

	/** Used by materials using this function to know when to recompile. */
	UPROPERTY()
	FGuid StateId;

protected:
	/** The intended usage of this function, required for material layers. */
	UPROPERTY(AssetRegistrySearchable)
	EMaterialFunctionUsage MaterialFunctionUsage = EMaterialFunctionUsage::Default;

public:
	virtual EMaterialFunctionUsage GetMaterialFunctionUsage()
		PURE_VIRTUAL(UMaterialFunctionInterface::GetMaterialFunctionUsage,return EMaterialFunctionUsage::Default;);

#if WITH_EDITOR
	virtual void UpdateFromFunctionResource()
		PURE_VIRTUAL(UMaterialFunctionInterface::UpdateFromFunctionResource,);

	virtual void GetInputsAndOutputs(TArray<struct FFunctionExpressionInput>& OutInputs, TArray<struct FFunctionExpressionOutput>& OutOutputs) const
		PURE_VIRTUAL(UMaterialFunctionInterface::GetInputsAndOutputs,);

	virtual void ForceRecompileForRendering(FMaterialUpdateContext& UpdateContext, UMaterial* InPreviewMaterial);
#endif

	virtual bool ValidateFunctionUsage(class FMaterialCompiler* Compiler, const FFunctionExpressionOutput& Output)
		PURE_VIRTUAL(UMaterialFunctionInterface::ValidateFunctionUsage,return false;);

#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, const struct FFunctionExpressionOutput& Output)
		PURE_VIRTUAL(UMaterialFunctionInterface::Compile,return INDEX_NONE;);

	virtual void LinkIntoCaller(const TArray<FFunctionExpressionInput>& CallerInputs)
		PURE_VIRTUAL(UMaterialFunctionInterface::LinkIntoCaller,);

	virtual void UnlinkFromCaller()
		PURE_VIRTUAL(UMaterialFunctionInterface::UnlinkFromCaller,);
#endif

#if WITH_EDITORONLY_DATA
	/** @return true if this function is dependent on the passed in function, directly or indirectly. */
	virtual bool IsDependent(UMaterialFunctionInterface* OtherFunction)
		PURE_VIRTUAL(UMaterialFunctionInterface::IsDependent,return false;);

	/**
	 * Iterates all functions that this function is dependent on, directly or indrectly.
	 *
	 * @param Predicate a visitor predicate returning true to continue iteration, false to break
	 *
	 * @return true if all dependent functions were visited, false if the Predicate did break iteration
	 */
	virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
		PURE_VIRTUAL(UMaterialFunctionInterface::IterateDependentFunctions,return false;);

	/** Returns an array of the functions that this function is dependent on, directly or indirectly. */
	virtual void GetDependentFunctions(TArray<UMaterialFunctionInterface*>& DependentFunctions) const
		PURE_VIRTUAL(UMaterialFunctionInterface::GetDependentFunctions,);

	/** Returns If returns an empty string, use the default class name for the material function. Otherwise, the string will be the name shown when the function is exposed to users in the material graph as a node, or from the contextual menu when searching for nodes. */
	virtual FString GetUserExposedCaption() const { return TEXT(""); }
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual UMaterialInterface* GetPreviewMaterial()
		PURE_VIRTUAL(UMaterialFunctionInterface::GetPreviewMaterial,return nullptr;);

	virtual void UpdateInputOutputTypes()
		PURE_VIRTUAL(UMaterialFunctionInterface::UpdateInputOutputTypes,);

	/** Checks whether a Material Function is arranged in the old style, with inputs flowing from right to left */
	virtual bool HasFlippedCoordinates() const
		PURE_VIRTUAL(UMaterialFunctionInterface::HasFlippedCoordinates,return false;);
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY(AssetRegistrySearchable)
	uint64 CombinedInputTypes;

	UPROPERTY(AssetRegistrySearchable)
	uint64 CombinedOutputTypes;

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
#endif
	
	virtual UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard())
		PURE_VIRTUAL(UMaterialFunction::GetBaseFunction,return nullptr;);

	virtual const UMaterialFunction* GetBaseFunction(FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) const
		PURE_VIRTUAL(UMaterialFunction::GetBaseFunction,return nullptr;);

	/** Returns GetBaseFunction() as a UMaterialFunctionInterface, useful if MaterialFunction.h hasn't been included yet, and implicit conversion to UMaterialInterface isn't availiable */
	ENGINE_API UMaterialFunctionInterface* GetBaseFunctionInterface();
	ENGINE_API const UMaterialFunctionInterface* GetBaseFunctionInterface() const;

#if WITH_EDITORONLY_DATA
	ENGINE_API TConstArrayView<TObjectPtr<UMaterialExpression>> GetExpressions() const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ENGINE_API const FString& GetDescription() const;
	ENGINE_API bool GetReentrantFlag() const;
	ENGINE_API void SetReentrantFlag(bool bIsReentrant);
#endif // WITH_EDITOR

public:

#if WITH_EDITOR
	/** Finds the first matching parameter by name and type */
	template<typename ExpressionType>
	bool GetNamedParameterOfType(const FHashedMaterialParameterInfo& ParameterInfo, ExpressionType*& Parameter, UMaterialFunctionInterface** OwningFunction = nullptr)
	{
		Parameter = nullptr;

		if (UMaterialFunctionInterface* ParameterFunction = GetBaseFunctionInterface())
		{
			const UClass* TargetClass = ExpressionType::StaticClass();

			auto GetExpressionParameterByNamePredicate = 
				[&ParameterInfo, &Parameter, &OwningFunction, TargetClass](UMaterialFunctionInterface* Function) -> bool
			{
				for (UMaterialExpression* FunctionExpression : Function->GetExpressions())
				{
					if (ExpressionType* ExpressionParameter = (FunctionExpression && FunctionExpression->IsA(TargetClass)) ? (ExpressionType *)FunctionExpression : nullptr)
					{
						if (ExpressionParameter->ParameterName == ParameterInfo.Name)
						{
							Parameter = ExpressionParameter;

							if (OwningFunction)
							{
								(*OwningFunction) = Function;
							}

							return false; // found, stop iterating
						}
					}
				}

				return true; // not found, continue iterating
			};
			
			if (!ParameterFunction->IterateDependentFunctions(GetExpressionParameterByNamePredicate))
			{
				return true;
			}
			return !GetExpressionParameterByNamePredicate(ParameterFunction);
		}

		return false;
	}

	/** Returns if any of the matching parameters have changed */
	template <typename ParameterType, typename ExpressionType>
	bool UpdateParameterSet(ParameterType& Parameter)
	{
		if (UMaterialFunctionInterface* ParameterFunction = GetBaseFunctionInterface())
		{
			TArray<UMaterialFunctionInterface*> Functions;
			ParameterFunction->GetDependentFunctions(Functions);
			Functions.AddUnique(ParameterFunction);

			for (UMaterialFunctionInterface* Function : Functions)
			{
				for (const TObjectPtr<UMaterialExpression>& FunctionExpression : Function->GetExpressions())
				{
					if (ExpressionType* ParameterExpression = Cast<ExpressionType>(FunctionExpression))
					{
						if (ParameterExpression->ParameterName == Parameter.ParameterInfo.Name)
						{
							Parameter.ExpressionGUID = ParameterExpression->ExpressionGUID;
							return true;
						}
					}
				}
			}
		}
		return false;
	}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Get all expressions of the requested type, recursing through any function expressions in the function */
	template<typename ExpressionType>
	bool HasAnyExpressionsOfType()
	{
		if (UMaterialFunctionInterface* ParameterFunction = GetBaseFunctionInterface())
		{
			TArray<UMaterialFunctionInterface*> Functions;
			ParameterFunction->GetDependentFunctions(Functions);
			Functions.AddUnique(ParameterFunction);

			for (UMaterialFunctionInterface* Function : Functions)
			{
				for (UMaterialExpression* FunctionExpression : Function->GetExpressions())
				{
					if (ExpressionType* FunctionExpressionOfType = Cast<ExpressionType>(FunctionExpression))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	/** Get all expressions of the requested type, recursing through any function expressions in the function */
	template<typename ExpressionType>
	void GetAllExpressionsOfType(TArray<ExpressionType*>& OutExpressions, const bool bRecursive = true)
	{
		if (UMaterialFunctionInterface* ParameterFunction = GetBaseFunctionInterface())
		{
			TArray<UMaterialFunctionInterface*> Functions;
			if (bRecursive)
			{
				ParameterFunction->GetDependentFunctions(Functions);
			}
			Functions.AddUnique(ParameterFunction);

			for (UMaterialFunctionInterface* Function : Functions)
			{
				for (UMaterialExpression* FunctionExpression : Function->GetExpressions())
				{
					if (ExpressionType* FunctionExpressionOfType = Cast<ExpressionType>(FunctionExpression))
					{
						OutExpressions.Add(FunctionExpressionOfType);
					}
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	virtual bool GetParameterOverrideValue(EMaterialParameterType Type, const FName& ParameterName, FMaterialParameterMetadata& OutValue, FMFRecursionGuard RecursionGuard = FMFRecursionGuard()) const;

	ENGINE_API bool OverrideNamedScalarParameter(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue);
	ENGINE_API bool OverrideNamedVectorParameter(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue);
	ENGINE_API bool OverrideNamedTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue);
	ENGINE_API bool OverrideNamedTextureCollectionParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue);
	ENGINE_API bool OverrideNamedRuntimeVirtualTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture*& OutValue);
	ENGINE_API bool OverrideNamedSparseVolumeTextureParameter(const FHashedMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture*& OutValue);
	ENGINE_API bool OverrideNamedFontParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UFont*& OutFontValue, int32& OutFontPage);
	ENGINE_API bool OverrideNamedStaticSwitchParameter(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, FGuid& OutExpressionGuid);
	ENGINE_API bool OverrideNamedParameterCollectionParameter(const FHashedMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection*& OutValue);
	ENGINE_API bool OverrideNamedStaticComponentMaskParameter(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutR, bool& OutG, bool& OutB, bool& OutA, FGuid& OutExpressionGuid);

	virtual bool IsUsingNewHLSLGenerator() const { return false; }
#endif // WITH_EDITOR
};
