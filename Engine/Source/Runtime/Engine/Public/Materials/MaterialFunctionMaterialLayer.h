// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "MaterialFunctionMaterialLayer.generated.h"

/**
 * Specialized Material Function that acts as a layer
 */
UCLASS(hidecategories=object, MinimalAPI)
class UMaterialFunctionMaterialLayer : public UMaterialFunction
{
	GENERATED_BODY()
};

/**
* Specialized Material Function Instance that instances a layer
*/
UCLASS(hidecategories = object, MinimalAPI)
class UMaterialFunctionMaterialLayerInstance : public UMaterialFunctionInstance
{
	GENERATED_BODY()
};
