// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "MaterialFunctionMaterialLayerBlend.generated.h"

//Legacy nodes rely on these hard coded names, this is mainly for fixup code.
constexpr const TCHAR* TopMaterialBlendInputName = TEXT("Top Layer");
constexpr const TCHAR* BottomMaterialBlendInputName = TEXT("Bottom Layer");

/**
 * Specialized Material Function that acts as a blend
 */
UCLASS(hidecategories=object, MinimalAPI)
class UMaterialFunctionMaterialLayerBlend : public UMaterialFunction
{
	GENERATED_BODY()
	virtual void PostLoad() override;
};

/**
* Specialized Material Function Instance that instances a blend
*/
UCLASS(hidecategories = object, MinimalAPI)
class UMaterialFunctionMaterialLayerBlendInstance : public UMaterialFunctionInstance
{
	GENERATED_BODY()
};
