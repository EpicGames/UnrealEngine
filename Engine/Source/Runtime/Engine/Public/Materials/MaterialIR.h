// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIRCommon.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRExtern.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionDBufferTexture.h" // @massimo.tristano remove this
#include "Engine/TextureDefines.h"
#include "Shader/Preshader.h"
#include "Shader/Preshader2.h"

#if WITH_EDITOR

struct FMaterialExternalCodeDeclaration;

class URuntimeVirtualTexture;

namespace MIR
{

// A collection of bit flags for a specific Value instance.
enum class EValueFlags : uint8
{
	None = 0,

	// Value has been analyzed for the vertex stage.
	AnalyzedForStageVertex = 1 << 0,

	// Value has been analyzed for the pixel stage.
	AnalyzedForStagePixel = 1 << 1,

	// Value has been analyzed for the compute stage.
	AnalyzedForStageCompute = 1 << 2,

	// Mask that includes all the analyzed flag in all stages.
	AnalyzedInAllStagesMask = AnalyzedForStageVertex | AnalyzedForStagePixel | AnalyzedForStageCompute,
};

ENUM_CLASS_FLAGS(EValueFlags);

// Enumeration of all different structs deriving from FValue. Used for dynamic casting.
enum EValueKind
{
	// Values

	VK_Poison,
	VK_AnalysisError,
	VK_Constant,
	VK_Builtin,
	VK_MaterialParameterCollection,
	VK_ShadingModel,
	VK_PrimitiveUniform,
	VK_TextureUniform,
	VK_VirtualTextureUniform,
	VK_GetVertexInterpolator,

	// Instructions

	VK_InstructionBegin,

	VK_Nop,
	VK_SetMaterialOutput,
	VK_Composite,
	VK_Operator,
	VK_Branch,
	VK_Subscript,
	VK_Scalar,
	VK_TextureRead,
	VK_VTPageTableRead,
	VK_Extern,
	VK_SubstrateDefaultSlab,
	VK_SubstrateSlab,
	VK_SubstrateShadingModels,
	VK_SubstrateToon,
	VK_SubstrateHorizontalMixing,
	VK_SubstrateVerticalLayering,
	VK_SubstrateCoverageWeight,
	VK_SubstrateAdd,
	VK_SubstrateSelect,
	VK_SubstratePromoteToOperator,
	VK_StageSwitch,
	VK_PartialDerivative,
	VK_SetVertexInterpolator,
	VK_Call,
	VK_CallParameterOutput,
	VK_PreshaderParameter,

	VK_InstructionEnd,
};

// Returns the string representation of given value kind.
const TCHAR* LexToString(EValueKind Kind);

// Values

// Base entity of all IR graph nodes.
//
// An IR module is a graph of values, connected by their "uses" relations. The graph of IR
// values is built by the `MaterialIRModuleBuilder` as the result of crawling through and
// analyizing the MaterialExpression graph contained in the translated Material. During
// this processing, IR values are emitted, and linked together. After the graph is
// constructed, it is itself analyzed: the builder will call `FMaterialIRValueAnalyzer::Analyze()`
// in each active* (i.e. truly used) value in the graph, making sure a value is analyzed only
// after its dependencies have been analyzed.
// A few notes:
// - FValues are automatically zero-initialized.
// - FValues are intended to be simple and inert data records. They cannot have non-trivia
//   ctor, dtor or copy operators.
// - The ModuleBuilder relies on this property to efficiently hashing values so that it
//   will reuse the same value instance instead of creating multiple instances of the
//   same computation (for a more efficient output shader).
// - All values have a MIR type.
// - Pure FValue instances are values that do not have other dependencies (called "uses").
// - If a value has some other value as dependency, it means that it is the result of a
//   calculation on those values. Values that have dependencies are Instructions (they
//   derive from FInstruction).
struct FValue
{
	// The runtime type this value has.
	FType Type;

	// Used to discern the concrete C++ type of this value (e.g. Subscript)
	TEnumAsByte<EValueKind> Kind;

	// Set of fundamental flags true for this value.
	EValueFlags Flags;

	// The set of properties that are true for this value. Some flags might have been
	// set but some upstream dependency leading to this value and not this value directly.
	// See `EGraphProperties` for more information.
	EGraphProperties GraphProperties;

	// Offset where this value will be written in the Preshader buffer plus one, with zero
	// indicating the value is not a Preshader.  During value analyze, before PreshaderFixup has
	// been called, stores a unique "register" index.  PreshaderFixup translates these registers
	// to offsets once locations in the output buffer have all been computed, taking into
	// account lifetimes and aliasing of the registers.
	int32 Analysis_PreshaderOffset;

	// Returns whether this value has been analyzed for specified stage.
	bool IsAnalyzed(EStage State) const;

	// Returns whether specified flags are true for this value.
	bool HasFlags(EValueFlags InFlags) const;

	// Enables the specified flags without affecting others.
	void SetFlags(EValueFlags InFlags);

	// Disables the specified flags without affecting others.
	void ClearFlags(EValueFlags InFlags);

	// Returns whether specified properties are true for this value.
	bool HasSubgraphProperties(EGraphProperties Properties) const;

	// Enables specified properties for this value.
	void SetSubgraphProperties(EGraphProperties Properties);

	// Returns the size in bytes of this value instance.
	uint32 GetSizeInBytes() const;

	// Gets the immutable array of all this value uses.
	// An use is another value referenced by this one (e.g. the operands of a binary expression).
	// Returns the immutable array of uses.
	TConstArrayView<FValue*> GetUses() const;

	// Gets the immutable array of this value uses filtered for a specific stage stage.
	//
	// Returns the immutable array of uses.
	TConstArrayView<FValue*> GetUsesForStage(MIR::EStage Stage) const;

	// Returns whether this value is of specified kind.
	bool IsA(EValueKind InKind) const { return Kind == InKind; }

	// Returns whether this value is poison.
	bool IsPoison() const { return Kind == VK_Poison; }

	// Returns whether this value is a constant boolean with value true.
	bool IsTrue() const;

	// Returns whether this value is a constant boolean with value false.
	bool IsFalse() const;

	// Returns whether this value is boolean and all components are true.
	bool AreAllTrue() const;

	// Returns whether this value is boolean and all components are false.
	bool AreAllFalse() const;

	// Returns whether this value is arithmetic and all components are exactly zero.
	bool AreAllExactlyZero() const;

	// Returns whether this value is arithmetic and all components are approximately zero.
	bool AreAllNearlyZero() const;

	// Returns whether this value is arithmetic and all components are exactly one.
	bool AreAllExactlyOne() const;

	// Returns whether this value is arithmetic and all components are approximately one.
	bool AreAllNearlyOne() const;

	//
	bool IsConstant() const;
	
	// Returns whether this value exactly equals Other.
	bool Equals(const FValue* Other) const;

	// Returns whether this value equals the given constant scalar.
	bool EqualsConstant(float Value) const;

	// Returns whether this value equals the given constant vector.
	bool EqualsConstant(FVector4f Value) const;

	// Returns the UTexture or URuntimeVirtualTexture object if this value references such texture. Returns null otherwise.
	UObject* GetTextureObject() const;

	// Returns the uniform index if this value associated to this value if its a uniform (primitive, texture, etc). Returns INDEX_NONE otherwise.
	int32 GetUniformIndex() const;

	// Tries to cast this value to specified type T and returns the casted pointer, if possible (nullptr otherwise).
	template <typename T>
	T* As() { return IsA(T::TypeKind) ? static_cast<T*>(this) : nullptr; }

	// Tries to cast this value to specified type T and returns the casted pointer, if possible (nullptr otherwise).
	template <typename T>
	const T* As() const { return IsA(T::TypeKind) ? static_cast<const T*>(this) : nullptr; }
};

// Tries to cast a value to a derived type.
// If specified value is not null, it tries to cast this value T and returns it. Otherwise, it returns null.
template <typename T>
T* As(FValue* Value)
{
	return Value && Value->IsA(T::TypeKind) ? static_cast<T*>(Value) : nullptr;
}

// Tries to cast a value to a derived type.
// If specified value is not null, it tries to cast this value T and returns it. Otherwise, it returns null.
template <typename T>
const T* As(const FValue* Value)
{
	return Value && Value->IsA(T::TypeKind) ? static_cast<const T*>(Value) : nullptr;
}

// Casts a value to an instruction.
// If specified value is not null, it tries to cast this value to an instruction and returns it. Otherwise, it returns null.
FInstruction* AsInstruction(FValue* Value);

// Casts a value to an instruction.
// If specified value is not null, it tries to cast this value to an instruction and returns it. Otherwise, it returns null.
const FInstruction* AsInstruction(const FValue* Value);

template <EValueKind TTypeKind>
struct TValue : FValue
{
	static constexpr EValueKind TypeKind = TTypeKind;
};

// A placeholder for an invalid value.
//
// A poison value represents an invalid value. It is produced by the emitter when an
// invalid operation is performed. Poison values can be passed as arguments to other operations,
// but they are "contagious": any instruction emitted with a poison value as an argument
// will itself produce a poison value.
struct FPoison : TValue<VK_Poison>
{
	static FPoison* Get();
};

// A utility value whose sole purpose is to report a semantic error if and when it is analyzed.
// This value is used to mark an invalid path, for example, in combination with StageSwitch, to ensure
// that a value is only available in a subset of stages.
// Note: this error is reported only if this value is analyzed. If the builder determines that this value
// does not contribute to the final output, no error is reported (since the value will not be analyzed either).
struct FAnalysisError : TValue<VK_AnalysisError>
{
	// The expression originating the invalid operation.
	UMaterialExpression* Expression;

	// The error message.
	FSimpleStringView Message;
};

// The integer type used inside MIR.
using TInteger = int64_t;

// The floating point type used inside MIR.
using TFloat = float;

// The double precision / LWC floating point type used inside MIR.
using TDouble = double;

// A constant value.
//
// A constant represents a translation-time known scalar primitive value. Operations on
// constant values can be folded by the builder, that is they can be evaluated statically
// while the builder constructs the IR graph of an input material.
struct FConstant : TValue<VK_Constant>
{
	union
	{
		bool  		Boolean;
		TInteger	Integer;
		TFloat 		Float;
		TDouble		Double;
	};

	// Returns the constant value of given type T. The type must be bool, integral or floating point.
	template <typename T>
	T Get() const
	{
		if constexpr (std::is_same_v<T, bool>)
		{
			return Boolean;
		}
		else if constexpr (std::is_integral_v<T>)
		{
			return Integer;
		}
		else if constexpr (std::is_floating_point_v<T>)
		{
			return Float;
		}
		else
		{
			check(false && "unexpected type T.");
		}
	}
};

// Enumeration of builtin values.
enum class EBuiltin
{
	TextureMipBias,				// A globally defined bias to the mip index used in texture sampling with automatic view mip bias
	TextureDerivativeMultiply,	// Used in texture sampling with automatic bip bias to correctly calculate texcoord screen differentials
};

// Returns the string representation of given builtin.
const TCHAR* LexToString(EBuiltin ID);

// It builtin is symbolic value known to the Material IR but provided externally.
// The actual numerical value is left to the backend, which can resolve it to a constant or any other uniform expression.
struct FBuiltin : TValue<VK_Builtin>
{
	EBuiltin Id;
};

// It represents a Material Parameter Collection object value.
struct FMaterialParameterCollection : TValue<VK_MaterialParameterCollection>
{
	UMaterialParameterCollection* Collection;

	// Index of this collection in the list of collections referenced by the material.
	// Note: This field is automatically set by the builder.
	int32 Analysis_CollectionIndex;
};

// An externally provided primitive uniform value, referenced by a given name.
struct FPrimitiveUniform : TValue<VK_PrimitiveUniform>
{
	// A uniquely identifying name.
	FScriptName Name; // Is FMinimalName ok too?

	// A *constant* value this uniform is initialized to if left unassigned at runtime.
	FValue* DefaultConstant;

	// Index of this uniform in the uniform expression set generated during analysis.
	int16 Analysis_UniformIndex;

	// Mask of which components of this uniform parameter are actually used.
	int32 Analysis_ComponentMask;

	// Number of components of this uniform parameter that are actually used.  Asserts if Analysis_ComponentMask hasn't been initialized.
	int32 Analysis_NumComponents() const;
};

// It represents a texture uniform value.
struct FTextureUniform : TValue<VK_TextureUniform>
{
	// A uniquely identifying name.
	FScriptName Name; // Is FMinimalName ok too?

	// The texture object.
	UTexture* Texture;

	// The sampler type associated to this texture object.
	EMaterialSamplerType SamplerType : 16;

	// Index of this uniform in the uniform expression set generated during analysis.
	int16 Analysis_UniformIndex;
};

// It represents a runtime virtual texture (RVT) uniform value.
struct FVirtualTextureUniform : TValue<VK_VirtualTextureUniform>
{
	// A uniquely identifying name.
	FScriptName Name; // Is FMinimalName ok too?

	// The runtime virtual texture object.
	// Note: URuntimeVirtualTexture does not extend UTexture.
	URuntimeVirtualTexture* VirtualTexture;

	// Field to explicitly assign a VT stack layer index (INDEX_NONE if unset). Regular VTs are assigned these indices automatically.
	int16 VTLayerIndex;

	// Field to explicitly assign a VT page table index. Regular VTs are assigned these indices automatically.
	uint16 VTPageTableIndex;

	// Index of this parameter in the uniform expression set generated during analysis.
	int16 Analysis_UniformIndex;
};

// A value that identifies a specific shading model
struct FShadingModel : TValue<VK_ShadingModel>
{
	EMaterialShadingModel Id;
};

// Value used to retrieve (in the pixel stage) a value interpolated in the vertex stage.
struct FGetVertexInterpolator : TValue<VK_GetVertexInterpolator>
{
	// Associated Set- instruction create at the time of emission.
	struct FSetVertexInterpolator* SetInstr;
};

/*------------------------------------ Instructions ------------------------------------*/

// A block of instructions.
//
// A block is a sequence of instructions in order of execution. Blocks are organized in a
// tree-like structure, used to model blocks nested inside other blocks.
struct FBlock
{
	// This block's parent block, if any. If this is null, this is a *root block.
	FBlock* Parent;

	// The linked-list head of the instructions contained in this block.
	// Links are contained in the FInstruction::Next field.
	FInstruction* Instructions;

	// Depth of this block in the tree structure (root blocks have level zero).
	int32 Level;

	// Finds and returns the common block between this and Other, if any. It returns null
	// if no common block was found.
	// Note: This has O(n) complexity, where n is the maximum depth of the tree structure.
	FBlock* FindCommonParentWith(MIR::FBlock* Other);
};

//
struct FInstructionLinkage
{
	// The next instruction executing after this one.
	FInstruction* Next;

	// This instruction parent block.
	FBlock* Block;

	// How many users (i.e., dependencies) this instruction has.
	uint32 NumUsers;

	// The number of users processed during instruction graph linking in each entry point.
	// Note: This information combined with NumUsers is used by the builder to push instructions
	//      in the appropriate block automatically.
	uint32 NumProcessedUsers;
};

// Base struct of an instruction value.
//
// An instruction is a value in a well defined order of evaluation.
// Instructions have a parent block and are organized in a linked list: the Next field
// indicates the instruction that will execute immediately after this one.
// Since the material shader has multiple stages, the same instruction can belong to two
// different graphs of execution, which explains why all fields in this structure have
// a different possible value per stage.
//
// Note: All fields in this struct are not expected to be set by the user when emitting
//      an instruction, and are instead automatically populated by the builder.
struct FInstruction : FValue
{
	// Array of linkage information (how this instruction is connected inside the IR graph)
	// for each entry point registered in the module.
	TArrayView<FInstructionLinkage> Linkage;

	// Returns the block in which the dependency with specified index should execute.
	FBlock* GetTargetBlockForUse(int32 EntryPointIndex, int32 UseIndex);

	// Returns the next instruction in specified entry point. 
	FInstruction* GetNext(int32 EntryPointIndex) const { return Linkage[EntryPointIndex].Next; }

	// Returns the number of user sin specified entry point.
	uint32 GetNumUsers(int32 EntryPointIndex) const { return Linkage[EntryPointIndex].NumUsers; }

	// Returns the block this instruction belongs to in specified entry point.
	const FBlock* GetBlock(int32 EntryPointIndex) const { return Linkage[EntryPointIndex].Block; }

	// Retrieves a user-specified name string if this instruction provides one.
	FStringView GetName() const;

	// Retrieves the priority if this instruction is used as a entry point output.
	int32 GetOutputPriority() const;
};

template <EValueKind TTypeKind, uint32 TNumStaticUses = 0>
struct TInstruction : FInstruction
{
	// The kind of this instruction.
	static constexpr EValueKind TypeKind = TTypeKind;

	// The number of values this instruction uses statically. Some instructions have a
	// dynamic number of uses, in which case NumStaticUses is 0.
	static constexpr uint32 NumStaticUses = TNumStaticUses;
};

// A "no operation" instruction is a dummy instruction used to perform analysis on a value (and its dependencies)
// but not push it to the list of instructions in a block.
// A NOP instructions should effectively bahave as the default value of Arg type.
struct FNop : TInstruction<VK_Nop, 1>
{
	FValue* Arg;
};

// An aggregate of other values.
//
// A dimensional is a fixed array of other values. This value is used to model vectors and matrices.
struct FComposite : TInstruction<VK_Composite, 0>
{
	// Returns the constant array of component values.
	TConstArrayView<FValue*> GetComponents() const;

	// Returns the mutable array of component values.
	TArrayView<FValue*> GetMutableComponents();

	// Returns whether all components are constant (i.e., they're instances of FConstant).
	bool AreComponentsConstant() const;
};

template <int TSize>
struct TComposite : FComposite
{
	FValue* Components[TSize];
};

// Whether a material output should be computed per-vertex or per-pixel.
enum class EMaterialOutputFrequency
{
	PerVertex,
	PerPixel,
};

// Instruction that sets material attribute (i.e., "BaseColor") to its value.
struct FSetMaterialOutput : TInstruction<VK_SetMaterialOutput, 1>
{
	// The material attribute to set.
	EMaterialProperty Property : 16;
	
	// Whether this output is evaluated per-vertex or per-pixel.
	EMaterialOutputFrequency Frequency : 16;

	// The name of this material output (e.g. "BaseColor")
	FSimpleStringView Name;

	// The value this material attribute should be set to.
	FValue* Arg;

	// Values in a module EntryPoint are sorted by this priority index (lowest to highest).
	// This value is used to give an order of evaluation of SetMaterialOutput instructions in an entry point.
	int Priority;
};

// Provides an implementation of a user-specified Extern definition.  
struct FExternImpl
{
	// The byte size of the whole Extern value.
	uint32 ByteSize;

	// Pointers to the trampoline functions to the user definition methods.
	// Note: it implements the CExternDefinition plus all optional functions
	TConstArrayView<FValue*> (*GetArguments)(const FExtern* Extern);
	MIR::FExternInfo (*GetInfo)(const FExtern* Extern);
	void (*Analyze)(FExtern* Extern, FExternAnalysisContext& Context);
	void (*AnalyzeInStage)(FExtern* Extern, FExternAnalysisContext& Context, EStage Stage);
	void (*ToHLSL)(const FExtern* Extern, FExternPrinterHLSL& Printer);
	void (*EmitDebugInfo)(const FExtern* Extern, FString& Out);
};

// An Extern represents a user-defined, backend-facing operation in the Material IR.
// It is a generic instruction whose behavior is fully described by a user provided
// struct that implements the CExternDefinition concept, which provides methods for argument access,
// type/info description, analysis, stage-specific analysis, HLSL emission, and optional debug output.
// The Extern instruction provides the extensibility point for users to model material-specific functionality
// not directly natively provided by MIR.
// See `MIR::CExternDefinition` for more information.
struct FExtern : TInstruction<VK_Extern, 0>
{
	// Pointer to the implementation table.
	const FExternImpl* Impl;

	// Returns the number of argument values which correspond to this instruction's uses.
	TConstArrayView<FValue*> GetArguments() const { return Impl->GetArguments(this); }

	// See `MIR::CExternDefinition`
	FExternInfo GetInfo() const { return Impl->GetInfo(this); }

	// See `MIR::CExternDefinition`
	void Analyze(FExternAnalysisContext& Context) { Impl->Analyze(this, Context); }

	// See `MIR::CExternDefinition`
	void AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage) { Impl->AnalyzeInStage(this, Context, Stage); }
	
	// See `MIR::CExternDefinition`
	void ToHLSL(FExternPrinterHLSL& Printer) const { Impl->ToHLSL(this, Printer); }

	// See `MIR::CExternDefinition`
	void EmitDebugInfo(FString& Out) const { Impl->EmitDebugInfo(this, Out); }
};

// Operator enumeration.
//
// Note: If you modify this enum, update the implementations of the helper functions below.
enum EOperator
{
	O_Invalid,

	// Unary
	UO_FirstUnaryOperator,

	// Unary operators
	UO_BitwiseNot = UO_FirstUnaryOperator, // ~(x)
	UO_Negate, // Arithmetic negation: negate(5) -> -5
	UO_Not, // Logical negation: not(true) -> false

	// Unary intrinsics
	UO_Abs,
	UO_ACos,
	UO_ACosFast,
	UO_ACosh,
	UO_ASin,
	UO_ASinFast,
	UO_ASinh,
	UO_ATan,
	UO_ATanFast,
	UO_ATanh,
	UO_Ceil,
	UO_Cos,
	UO_Cosh,
	UO_Exponential,
	UO_Exponential2,
	UO_Floor,
	UO_Frac,
	UO_IsFinite,
	UO_IsInf,
	UO_IsNan,
	UO_Length,
	UO_Logarithm,
	UO_Logarithm10,
	UO_Logarithm2,
	UO_LWCTile,
	UO_Reciprocal,
	UO_Round,
	UO_Rsqrt,
	UO_Saturate,
	UO_Sign,
	UO_Sin,
	UO_Sinh,
	UO_Sqrt,
	UO_Tan,
	UO_Tanh,
	UO_Transpose,
	UO_Truncate,
	
	BO_FirstBinaryOperator,

	// Binary comparisons
	BO_Equals = BO_FirstBinaryOperator,
	BO_GreaterThan,
	BO_GreaterThanOrEquals,
	BO_LessThan,
	BO_LessThanOrEquals,
	BO_NotEquals,

	// Binary logical
	BO_And,
	BO_Or,

	// Binary arithmetic
	BO_Add,
	BO_Subtract,
	BO_Multiply,
	BO_MatrixMultiply,
	BO_Divide,
	BO_Modulo,
	BO_BitwiseAnd,
	BO_BitwiseOr,
	BO_BitwiseXor,
	BO_BitShiftLeft,
	BO_BitShiftRight,
	
	// Binary intrinsics
	BO_ATan2,
	BO_ATan2Fast,
	BO_Cross,
	BO_Distance,
	BO_Dot,
	BO_Fmod,
	BO_Max,
	BO_Min,
	BO_Pow,
	BO_Step,

	TO_FirstTernaryOperator,

	// Ternary intrinsics
	TO_Clamp = TO_FirstTernaryOperator,
	TO_Lerp,
	TO_Select,
	TO_Smoothstep,

	OperatorCount,
};

// Whether the given operator identifies a unary operator.
bool IsUnaryOperator(EOperator Op);

// Whether the given operator identifies a binary operator.
bool IsBinaryOperator(EOperator Op);

// Whether the given operator identifies a ternary operator.
bool IsTernaryOperator(EOperator Op);

// Returns the arity of the operator (the number of arguments it take, 1, 2 or 3).
int GetOperatorArity(EOperator Op);

// Returns the string representation of the given operator.
const TCHAR* LexToString(EOperator Op);

// Translates the given MIR operator to a Preshader2 opcode, and specifies whether it is commutative.
// Returns EPreshader2Opcode::Invalid if no corresponding opcode.
EPreshader2Opcode ToPreshader2Opcode(EOperator Op, bool& bOutIsCommutative);

// A mathematical operator instruction.
//
// This instruction identifies a built-in operation on one, two or three argument values.
struct FOperator : TInstruction<VK_Operator, 3>
{
	// The first argument of the operation. This value is never null.
	FValue* AArg;

	// The second argument of the operation. This value is null for unary operators.
	FValue* BArg;

	// The third argument of the operation. This value is null for unary and binary operators.
	FValue* CArg;

	// It identifies which supported operation to carry.
	EOperator Op;
};

// A branch instruction.
//
// This instruction evaluates to one or another argument based on whether a third boolean
// condition argument is true or false. The builder will automatically place as many
// instructions as possible in the true/false inner blocks whilst respecting dependency
// requirements. This is done in an effort to avoid the unnecessary computation of the
// input value that was not selected by the condition.
struct FBranch : TInstruction<VK_Branch, 3>
{
	// The boolean condition argument used to forward the "true" or "false" argument.
	FValue* ConditionArg;

	// Value this branch evaluates to when the condition is true.
	FValue* TrueArg;

	// Value this branch evaluates to when the condition is false.
	FValue* FalseArg;

	// The inner block (in each module entry point) the subgraph evaluating `TrueArg` should be placed in.
	TArrayView<FBlock> TrueBlock;

	// The inner block (in each module entry point) the subgraph evaluating `FalseArg` should be placed in.
	TArrayView<FBlock> FalseBlock;
};

// A subscript instruction.
//
// This instruction is used to pull the inner value making a compound one. For example,
// it is used to extract an individual component of a vector value.
struct FSubscript : TInstruction<VK_Subscript, 1>
{
	// The argument to subscript.
	FValue* Arg;

	// The subscript index, i.e. the index of the component to extract.
	int32 Index;
};

// A scalar construction instruction.
//
// This instruction constructs a scalar from another (of possibly a different scalar kind).
struct FScalar : TInstruction<VK_Scalar, 1>
{
	// The initializing argument value.
	FValue* Arg;
};

// What texture gather mode to use in a texture read instruction (none indicates a sample).
enum class ETextureReadMode
{
	// Gather the four red components in a 2x2 pixel block.
	GatherRed,

	// Gather the four green components in a 2x2 pixel block.
	GatherGreen,

	// Gather the four blue components in a 2x2 pixel block.
	GatherBlue,

	// Gather the four alpha components in a 2x2 pixel block.
	GatherAlpha,

	// Texture gather with automatically calculated mip level
	MipAuto,

	// Texture gather with user specified mip level
	MipLevel,

	// Texture gather with automatically calculated mip level plus user specified bias
	MipBias,

	// Texture gather using automatically caluclated mip level based on user provided partial derivatives
	Derivatives,
};

// Returns the string representation of given mode.
const TCHAR* LexToString(ETextureReadMode Mode);

// This instruction performs texture read operaation (sample or gather).
struct FTextureRead : TInstruction<VK_TextureRead, 6>
{
	// The texture to sample.
	FValue* TextureUniform;

	// The texture coordinate at which to sample.
	FValue* TexCoord;

	// Optional. The mip index to sample, if any provided.
	FValue* MipValue;

	// Optional. The analytical partial derivative of the coordinates along the X axis.
	FValue* TexCoordDdx;

	// Optional. The analytical partial derivative of the coordinates along the Y axis.
	FValue* TexCoordDdy;

	// Only required for virtual textures. Must point to FVTPageTableRead value.
	FValue* VTPageTable;

	// The mip value mode to use for sampling.
	ETextureReadMode Mode : 8;

	// The sampler source mode to use for sampling.
	ESamplerSourceMode SamplerSourceMode : 8;

	// The sampler type to use for sampling.
	EMaterialSamplerType SamplerType : 8;

	// Used for VTs whether anisotropic filtering is used or not.
	bool bUseAnisoSampler : 1;
};

// This instruction reads a virtual texture (VT) page table and is allocated alongside VT sampling instructions.
struct FVTPageTableRead : TInstruction<VK_VTPageTableRead, 5>
{
	// The texture object to load the page table for.
	FValue* VirtualTextureUniform;

	// The texture coordinate at which to sample.
	FValue* TexCoord;

	// Optional. The analytical partial derivative of the coordinates along the X axis.
	FValue* TexCoordDdx;

	// Optional. The analytical partial derivative of the coordinates along the Y axis.
	FValue* TexCoordDdy;

	// Used with TMVM_MipBias and TMVM_MipLevel.
	FValue* MipValue;

	// Index of the virtual texture slot. This is determined in the analysis stage.
	int32 VTStackIndex : 16;

	// Index of the VT page table. This is determined in the analysis stage.
	int32 VTPageTableIndex : 16;

	// Texture address mode for U axis.
	TextureAddress AddressU : 8;

	// Texture address mode for V axis.
	TextureAddress AddressV : 8;

	// Specifies the sampler function to read the page table, i.e. TextureLoadVirtualPageTable(), TextureLoadVirtualPageTableGrad(), or TextureLoadVirtualPageTableLevel().
	ETextureMipValueMode MipValueMode : 4;

	// Specifies whether to enable virtual texture sampling feedback. This is only used for pixel shaders. Enabled by default.
	bool bEnableFeedback : 1;

	// Determines whether adaptive or regular VT page table loads will be emitted, i.e. TextureLoadVirtualPageTable*() or TextureLoadVirtualPageTableAdaptive*().
	bool bIsAdaptive : 1;

	// Aspect ratio (width/height) of the source texture. Used to prevent merging VTs with different aspect ratios into the same stack.
	// Defaults to 1.0f for RVTs and texture parameters with no default texture assigned.
	float AspectRatio = 1.0f;
};

// Utility value for selecting a different value based on the execution stage.
struct FStageSwitch : TInstruction<VK_StageSwitch, 1>
{
	// The argument for to be bypassed in each stage.
	FValue* Args[NumStages];

	// Use the specified value argument in the pixel stage, and another specified
	// argument for other stages.
	void SetArgs(FValue* PixelStageArg, FValue* OtherStagesArg);
};

// This instruction represents the screen-space partial derivative of a given value along an axis.
struct FPartialDerivative : TInstruction<VK_PartialDerivative, 1>
{
	// The value argument being differentiated.
	// If bHardwareDerivative is true, this can be any (valid) value that supports calculating the partial derivatives
	// by means of 2x2 quad differentiation.
	// If bHardwareDerivative is false, this argument value kind supports generating the differential somehow. For example
	// Arg may be an Extern or GetVertexInterpolator instruction.
	FValue* Arg;

	// The direction of partial derivative.
	EDerivativeAxis Axis : 1;
	
	// Whether this should use hardware differential calculation.
	bool bHardwareDerivative : 1;
};

// Instruction executed in the vertex stage that sets the value of some interpolated data, available
// in the pixel stage via its matching Get- value.
struct FSetVertexInterpolator : TInstruction<VK_SetVertexInterpolator, 1>
{
	// The value being set to the interpolator.
	FValue* Arg;

	// Interpolator registers are abstracted as an array of float scalars. This field, determined at
	// analysis time, is assigned to the index of the first float register occupied by this interpolator.
	uint32 Analysis_BaseInterpolatorFloatRegister;
};

// Kind of user-specified function.
enum class FFunctionKind : uint8
{
	HLSL,
};

// Describes a function parameter.
struct FFunctionParameter
{
	// The parameter name (used as identifier).
	FName Name;
	
	// The parameter type.
	FType Type;

	bool operator==(const FFunctionParameter&) const = default;

	friend uint32 GetTypeHash(const FFunctionParameter& F);
};

// Maximum number of parameters a user function can have.
constexpr uint32 MaxNumFunctionParameters = 32;

// Defines a user-specified function.
struct FFunction
{
	// The kind of function this is.
	FFunctionKind Kind;

	// Mask of stages this function is called in.
	EStageMask Analysis_CallStageMask;

	// This function's name (could be used as identifier).
	FStringView Name;

	// The function return value type
	FType ReturnType;

	// Number of input-only parameters.
	uint16 NumInputOnlyParams;

	// Number of input-output parameters.
	uint16 NumInputAndOutputParams;

	// Total number of parameters (input-only + input-output + output-only)
	uint16 NumParameters;

	// A module-unique id assigned to this function to disambiguate it with other functions with identical name.
	uint16 UniqueId;

	// The parameters this function has (in order: input-only, then input-output, finally output-only).
	FFunctionParameter Parameters[MaxNumFunctionParameters];

	// Returns the number of output parameters (input-output + output-only).
	uint32 GetNumOutputParameters() const { return NumParameters - NumInputOnlyParams; }

	// Returns the input-output/output-only parameter with given index.
	const FFunctionParameter& GetOutputParameter(uint32 Index) const { return Parameters[Index + NumInputOnlyParams]; }

	// Returns whether parameter with specified index is input.
	bool IsInputParameter(uint32 Index) const { return Index < NumInputAndOutputParams; }
	
	// Returns whether parameter with specified index is output.
	bool IsOutputParameter(uint32 Index) const { return Index >= NumInputOnlyParams; }

	// Whether this function equals specified (they have the same data).
	bool Equals(const FFunction* Other) const;

	friend uint32 GetTypeHash(const FFunction& F);
};

// Describes an HLSL preprocessor define, .e.g "#define MyDefine 123"
struct FFunctionHLSLDefine
{
	// The define name.
	FStringView Name;
	
	// The define value.
	FStringView Value;

	bool operator==(const FFunctionHLSLDefine& Other) const = default;

	friend uint32 GetTypeHash(const FFunctionHLSLDefine& D)
	{
		return HashCombine(GetTypeHash(D.Name), GetTypeHash(D.Value));
	}
};

// A user-function defined with user-provided HLSL code.
struct FFunctionHLSL : FFunction
{
	// The HLSL code of the body of this function.
	FStringView Code;

	// Array of #defines to declare in the material shader.
	TConstArrayView<FFunctionHLSLDefine> Defines;

	// Array of include directives to declare in the material shader.
	TConstArrayView<FStringView> Includes;

	// Whether this function equals specified (they have the same data).
	bool Equals(const FFunctionHLSL* Other) const;

	friend uint32 GetTypeHash(const FFunctionHLSL& F);
};

// A call instruction to a user-function.
struct FCall : TInstruction<VK_Call, 0>
{
	// The user-function to call.
	FFunction* Function;

	// The arguments to pass to the function parameters.
	FValue* Arguments[MaxNumFunctionParameters];
	
	// The number of arguments.
	int NumArguments;
};

// Instruction that evaluates some user-function call output parameter result.
struct FCallParameterOutput : TInstruction<VK_CallParameterOutput, 1>
{
	// The function call.
	FValue* Call;
	
	// The index of the output parameter to fetch the value from.
	int32 	Index;
};

// @massimo.tristano review this value for better integration and support of preshaders within new translator
struct FPreshaderParameterPayload
{
	// Uniform index for RVTs. Only used for UE::Shader::EPreshaderOpcode::RuntimeVirtualTextureUniform.
	int32 UniformIndex;
};

struct FPreshaderParameter : TInstruction<VK_PreshaderParameter, 1>
{
	// Argument that points to the source parameter this dynamic parameter depends on.
	FValue* SourceParameter;

	// Specifies the opcode that needs to be encoded for this parameter. For now, only a small subset of opcodes are supported for material preshader parameters.
	EPreshader2Opcode Opcode;

	// Index into UMaterialInterface::GetReferencedTextures() pointing to the source parameter's texture.
	int32 TextureIndex;

	// Payload data that depends on the opcode.
	FPreshaderParameterPayload Payload;
};



//------------------------------------------------------------------------
//                        SUBSTRATE INSTRUCTIONS
//------------------------------------------------------------------------

// Material IR representation of the FSubstrateData shader parameter to transition legacy materials into Substrate.
struct FSubstrateDefaultSlab : TInstruction<VK_SubstrateDefaultSlab, 0>
{
	FValue* Dummy;
};

struct FSubstrateSlab : TInstruction<VK_SubstrateSlab, 27> 
{
	FValue* Normal;
	FValue* DiffuseAlbedo;
	FValue* F0;
	FValue* F90;
	FValue* Roughness;
	FValue* Anisotropy;
	FValue* SSSProfileId;
	FValue* SSSMFP;
	FValue* SSSMFPScale;
	FValue* SSSPhaseAniso;
	FValue* SSSType;
	FValue* EmissiveColor;
	FValue* SecondRoughness;
	FValue* SecondRoughnessWeight;
	FValue* SecondRoughnessAsSimpleClearCoat;
	FValue* ClearCoatUseSecondNormal;
	FValue* ClearCoatBottomNormal;
	FValue* FuzzAmount;
	FValue* FuzzColor;
	FValue* FuzzRoughness;
	FValue* GlintValue;
	FValue* GlintUV;
	FValue* SpecularProfileId;
	FValue* Thickness;
	FValue* IsThin;
	FValue* IsAtBottom;
	FValue* LocalBasisIndex;

	bool bFullySimplifiedInstruction;
};

struct FSubstrateShadingModels : TInstruction<VK_SubstrateShadingModels, 24>
{
	FValue* BaseColor;
	FValue* Specular;
	FValue* Metallic;
	FValue* Roughness;
	FValue* Anisotropy;
	FValue* SubSurfaceColor;
	FValue* SubSurfaceProfileId;
	FValue* ClearCoat;
	FValue* ClearCoatRoughness;
	FValue* EmissiveColor;
	FValue* Opacity;
	FValue* ThinTranslucentTransmittanceColor;
	FValue* ThinTranslucentSurfaceCoverage;
	FValue* WaterScatteringCoefficients;
	FValue* WaterAbsorptionCoefficients;
	FValue* WaterPhaseG;
	FValue* ColorScaleBehindWater;
	FValue* ShadingModel;
	FValue* Normal;
	FValue* Tangent;
	FValue* ClearCoatNormal;
	FValue* CustomTangent;
	FValue* BasisIndexMacro;
	FValue* ClearCoat_BasisIndexMacro;

	bool bHasDynamicShadingModels;
	bool bFullySimplifiedInstruction;
};

struct FSubstrateToon : TInstruction<VK_SubstrateToon, 10> 
{
	FValue* Normal;
	FValue* ToonProfileId;
	FValue* BaseColor;
	FValue* Metallic;
	FValue* Specular;
	FValue* Roughness;
	FValue* EmissiveColor;
	FValue* PatternUVs;
	FValue* IsAtBottom;
	FValue* LocalBasisIndex;

	bool bFullySimplifiedInstruction;
};

struct FSubstrateHorizontalMixing : TInstruction<VK_SubstrateHorizontalMixing, 9>
{
	FValue* Background;
	FValue* Foreground;
	FValue* Mix;

	// ParameterBlending = 0
	FValue* OperatorIndex;
	FValue* MaxDistanceFromLeaves;

	// ParameterBlending = 1
	FValue* NormalMix;
	FValue* SharedLocalBasisIndexMacro;
	FValue* BackgroundNoV;
	FValue* ForegroundNoV;

	// Settings
	bool bParameterBlendingEnabled;
	bool bFullySimplifiedInstruction;
};

struct FSubstrateVerticalLayering : TInstruction<VK_SubstrateVerticalLayering, 7>
{
	FValue* Top;
	FValue* Base;

	// ParameterBlending = 0
	FValue* OperatorIndex;
	FValue* MaxDistanceFromLeaves;

	// ParameterBlending = 1
	FValue* SharedLocalBasisIndexMacro;
	FValue* TopNoV;
	FValue* BaseNoV;

	// Settings
	bool bParameterBlendingEnabled;
	bool bFullySimplifiedInstruction;
};

struct FSubstrateCoverageWeight : TInstruction<VK_SubstrateCoverageWeight, 4>
{
	FValue* A;
	FValue* Weight;

	// ParameterBlending = 0
	FValue* OperatorIndex;
	FValue* MaxDistanceFromLeaves;

	// Settings
	bool bParameterBlendingEnabled;
	bool bFullySimplifiedInstruction;
};

struct FSubstrateAdd : TInstruction<VK_SubstrateAdd, 8>
{
	FValue* A;
	FValue* B;

	// ParameterBlending = 0
	FValue* OperatorIndex;
	FValue* MaxDistanceFromLeaves;

	// ParameterBlending = 1
	FValue* NormalMix;
	FValue* SharedLocalBasisIndexMacro;
	FValue* ANoV;
	FValue* BNoV;

	// Settings
	bool bParameterBlendingEnabled;
	bool bFullySimplifiedInstruction;
};

struct FSubstrateSelect : TInstruction<VK_SubstrateSelect, 6>
{
	FValue* A;
	FValue* B;
	FValue* SelectValue;

	// ParameterBlending = 0
	FValue* OperatorIndex;
	FValue* MaxDistanceFromLeaves;

	// ParameterBlending = 1
	FValue* SharedLocalBasisIndexMacro;

	// Settings
	bool bParameterBlendingEnabled;
	bool bFullySimplifiedInstruction;
};

struct FSubstratePromoteToOperator : TInstruction<VK_SubstratePromoteToOperator, 5>
{
	FValue* SubstrateDataInput;
	FValue* OperatorIndex;
	FValue* BSDFIndex;
	FValue* LayerDepth;
	FValue* bIsBottom;

	// Settings
	bool bFullySimplifiedInstruction;
};


} // namespace MIR
#endif // WITH_EDITOR
