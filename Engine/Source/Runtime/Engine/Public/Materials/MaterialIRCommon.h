// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SceneTypes.h"
#include "RHIShaderPlatform.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Misc/MemStack.h"
#include "Materials/MaterialIRUtils.h"

#if WITH_EDITOR

//------------------------------------------------------------------------
//                          Forward declarations
//------------------------------------------------------------------------

enum EMaterialProperty : int;
enum EMaterialDomain : int;
enum EMaterialUsage : int;

struct FExpressionInput;
struct FExpressionOutput;
struct FMaterialAggregateAttribute;
struct FMaterialInputDescription;
struct FMaterialInsights;
struct FMaterialIRModuleBuilder;
struct FMaterialIRModuleBuilderImpl;
struct FShaderCompilerEnvironment;
struct FStaticParameterSet;
struct FMaterialExternalCodeDeclaration;
class FMaterial;
class FMaterialIRModule;
class ITargetPlatform;
class UMaterial;
class UMaterialAggregate;
class UMaterialExpression;
class UMaterialExpressionMaterialFunctionCall;
class UTexture;
class UMaterialInterface;

namespace Substrate
{
	struct FSubstrateTranslatorData;
};

namespace UE::Shader
{
	struct FValue;
}

namespace ERHIFeatureLevel { enum Type : int; }

namespace MIR
{
	class FEmitter;
	struct FBlock;
	struct FExtern;
	struct FExternImpl;
	struct FExternPrinterHLSL;
	struct FInstruction;
	struct FPrimitive;
	struct FSetMaterialOutput;
	struct FType;
	struct FValue;
	struct FSubgraphOutputMapping;
}

namespace MIR
{

// Identifies the block of execution in which an instruction runs.
// Note: If you introduce a new stage, make sure you update EValueFlags accordingly.
enum EStage
{
	Stage_Vertex,
	Stage_Pixel,
	Stage_Compute,
	NumStages
};

// Bitmask of execution stages, aligned with EStage indices.
enum class EStageMask : uint8
{
	None    = 0,
	Vertex  = 1 << 0,
	Pixel   = 1 << 1,
	Compute = 1 << 2,
	PixelAndCompute = Pixel | Compute,
	Any     = Vertex | Pixel | Compute
};

ENUM_CLASS_FLAGS(EStageMask);

// Gets the mask from the specified stage.
static inline EStageMask StageToMask(EStage Stage)
{
	return (EStageMask)(1 << (uint32)Stage);
}

// Returns whether given Stage is in given Mask.
static inline bool IsStageInMask(EStage Stage, EStageMask Mask)
{
	return (1 << (uint32)Stage & (uint32)Mask) != 0;
}

// Returns the string representation of given stage.
static inline const TCHAR* LexToString(EStage Stage)
{
	switch (Stage)
	{
		case Stage_Vertex:	return TEXT("Vertex");
		case Stage_Pixel: 	return TEXT("Pixel");
		case Stage_Compute: return TEXT("Compute");
		case NumStages: 	check(false);
	}
	return nullptr;
}

// A collection of graph properties used by a value.
//
// If a graph property is true, it generally means that either the value itself makes has of that
// property, or one of its dependencies (direct or indirect) has it. This entails these flags are
// automatically propagated to all dependant values (values that depend on a given one).  As an
// example, if "ReadsPixelNormal" is true on a specific value it means that either that value
// itself or some other upstream value that value is dependent on reads the pixel normal.
// 
// The properties starting with "Uniform" are exceptions.  Instead of being a union of child subgraph
// properties, "Uniform" represents an intersection, indicating *all* dependencies have "Uniform" set,
// as opposed to any.  "PreshaderOut" is set for "Uniform" nodes with an immediate non-Uniform parent,
// and "PreshaderTemp" is set for nodes in a subtree of a "PreshaderOut" node, representing
// intermediate values that don't need to be available on the GPU or in the output uniform buffer.
// And "TrivialTailSwizzle" handles a special case optimization.
enum class EGraphProperties : uint8
{
	None = 0,

	// This value is only meaningful and available in the pixel stage.
	PixelStageOnly = 1 << 0,

	// Some value reads the pixel normal.
	ReadsPixelNormal = 1 << 1,

	// Some value in the subtree is a parameter.
	HasParameter = 1 << 2,

	// This value and all its dependencies have the Uniform property set.
	Uniform = 1 << 3,

	// This value is used as an output in a Preshader, meaning it's visible to the GPU and needs a uniform buffer location.
	// Set if Uniform and HasParameter are both set, and the value is used by a non-Uniform.
	PreshaderOut = 1 << 4,

	// This value is used as a temporary in a Preshader, meaning it is only used by other Preshaders, and HLSL doesn't
	// need to be generated for it.  An exception is if TrivialTailSwizzle below is also set.  Set for values in a
	// subtree of a PreshaderOut, exclusive with PreshaderOut, which takes precedence.
	PreshaderTemp = 1 << 5,

	// This value is a trivial tail swizzle, meaning a subscript or composite of elements from a single source vector
	// (and optionally any number of constant elements) at the root of a subexpression tree that can be expanded on the
	// GPU for "free".  Forces the code required to expand the value to be emitted to the GPU, even if also a PreshaderTemp.
	TrivialTailSwizzle = 1 << 6,
};

ENUM_CLASS_FLAGS(EGraphProperties);

// Specifies the axis for computing screen-space derivatives.
//
// This enum is used to indicate whether a partial derivative should be taken
// along the X or Y screen-space direction, corresponding to HLSL's ddx and ddy
// functions.
enum class EDerivativeAxis : unsigned char
{
	X, Y,
};

} // namespace MIR

#endif // #if WITH_EDITOR
