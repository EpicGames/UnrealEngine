// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIRCommon.h"

#if WITH_EDITOR

namespace MIR
{

// Dumps a Graphviz Dot file to "{ProjectLogDir}/MaterialIRDumpUseGraph.dot" containing the
// full graph of IR nodes (with a string representation of value parameters inside each node),
// connected by the "Use" relationship. A value node will look connected to another value node
// iff it uses that other value as argument (e.g. an "Add" instruction will link to its operand values).
void DebugDumpIRUseGraph(const FMaterialIRModule& Module);

// Returns the IR generated by the module as a string.
FString DebugDumpIR(FStringView MaterialName, const FMaterialIRModule& Module);

} // namespace MIR

#endif // #if WITH_EDITOR
