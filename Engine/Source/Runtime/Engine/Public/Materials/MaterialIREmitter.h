// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIRCommon.h"
#include "Materials/MaterialIR.h"
#include "Materials/MaterialIRTypes.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialIRExtern.h"

#if WITH_EDITOR

#define UE_MIR_CHECKPOINT(Emitter) if (Emitter.IsInvalid()) return

namespace MIR {

enum class EVectorComponent : uint8_t
{
	X, Y, Z, W, 
};

// Return the lower case string representation of given component (e.g. "x")
const TCHAR* VectorComponentToString(EVectorComponent);

// This utility data structure is used to define how a swizzle operation should be performed.
struct FSwizzleMask
{
	// Which component should be extracted from the argument, in order. E.g. [Z, Z, X] to model "MyVec.zzx".
	EVectorComponent Components[4];

	// How many components have been defined for swizzle should be made of (maximum four).
	int NumComponents{};
	
	// Convenience ".xy" swizzle mask.
	static FSwizzleMask XY() { return { EVectorComponent::X, EVectorComponent::Y }; }
	
	// Convenience ".zw" swizzle mask.
	static FSwizzleMask ZW() { return { EVectorComponent::Z, EVectorComponent::W }; }

	// Convenience ".xyz" swizzle mask.
	static FSwizzleMask XYZ() { return { EVectorComponent::X, EVectorComponent::Y, EVectorComponent::Z }; }

	FSwizzleMask() {}
	FSwizzleMask(EVectorComponent X);
	FSwizzleMask(EVectorComponent X, EVectorComponent Y);
	FSwizzleMask(EVectorComponent X, EVectorComponent Y, EVectorComponent Z);
	FSwizzleMask(EVectorComponent X, EVectorComponent Y, EVectorComponent Z, EVectorComponent W);
	FSwizzleMask(bool bMaskX, bool bMaskY, bool bMaskZ, bool bMaskW);

	// Pushes a component to this swizzle mask.
	void Append(EVectorComponent Component);

	// Returns true if this is an identity swizzle mask XYZW, which implies i.e. it's a redundant swizzle mask.
	bool IsXYZW() const;

	const EVectorComponent* begin() const { return Components; }
	const EVectorComponent* end() const { return Components + NumComponents; }
};

// A lightweight wrapper around FValue* that also optionally tracks the source expression input.
// Used to carry both value and its origin in the expression graph.
// A null reference evaluates has the FValue* set to nullptr and evaluates false. It is
// returned by the FEmitter::TryInput() value to indicate a missing value. An invalid reference
// is either null or has a poison value. It is usually generated as the result of an invalid
// operation.
// Important: The Emitter APIs can safely take both null and poison values. Operations that
// take a single argument simply return the argument if it is invalid (null or poison),
// whereas operations that take multiple arguments return poison if any is invalid.
//
// When your intention is to test for whether a value is not-null (available), leverage the
// operator bool()
//
//		if (Value) { /* handle case when value is available */ }
//
// Otherwise, if you would like the value to be available AND valid (not-poison) you should
// test for IsValid()
//
//		if (Value.IsValid()) {
//			/* handle case when the operations that produced the value were all succesful */
//		}
//
// Note: In user Expression::Build() implementations, you should avoid testing values validity
// directly and instead leverage the UE_MIR_CHECKPOINT() macro to verify that all emitter
// performed so far were all valid, and thus the values you have computed so far are what
// you expect.
struct FValueRef
{
	// Default constructor. Creates an invalid reference.
	FValueRef() {}

	// Constructs from a value pointer.
	FValueRef(FValue* Value) : Value{ Value } {}

	// Constructs from a value pointer and its corresponding input source.
	FValueRef(FValue* Value, const FExpressionInput* Input) : Value{ Value }, Input{ Input } {}

	// Returns whether this references a non-null value.
	operator bool() const { return Value != nullptr; }

	// Returns the referenced value, which must be non-null.
	operator FValue*() const { return Value; }

	// Returns the referenced value, which must be non-null.
	FValue* operator->() const { return Value; }
	
	// Checks if the reference holds a valid non-null value.
	bool IsValid() const;

	// Checks if the reference holds a poison value.
	bool IsPoison() const;

	// Creates a new reference with the same input but a different value.
	FValueRef To(FValue* Value) const;

	// Creates a poison version of the current reference (same input, but poison value).
	FValueRef ToPoison() const;

	// Equality/inequality operators
	bool operator==(const FValueRef Other) const { return Value == Other.Value; }
	bool operator!=(const FValueRef Other) const { return Value != Other.Value; }
	
	// The referenced value.
	FValue* Value{};

	// The expression input being read.
	const FExpressionInput* Input{};
};

// An assignment to a material aggregate attribute, used when constructing an aggregate instance.
struct FAttributeAssignment
{
	// Then name of the assigned aggregate attribute.
	FName Name;

	// The value it is assigned to.
	FValueRef Value;
};

// Describes a user-specified function.
struct FFunctionDesc
{
	// The function name, used for debugging purposes and referenced in the generated HLSL. 
	FStringView Name{};

	// The function return type.
	FType ReturnType{};

	// The number of parameters that are input-only.
	uint32 NumInputOnlyParams{};

	// The number of input-output parameters after the input-only parameters.
	uint32 NumInputOutputParams{};

	// The number of output-only parameters after the input-output parameters.
	uint32 NumOutputOnlyParams{};

	// Array of descriptions of the function parameters.
	FFunctionParameter Parameters[MIR::MaxNumFunctionParameters];

	// Array of user-specified defines to declare before this function.
	TConstArrayView<FFunctionHLSLDefine> Defines;

	// Array of user-specified #include directives to declare before this function.
	TConstArrayView<FStringView> Includes;

	// Pushes an input-only parameter and returns whether there was an available free parameter to push this one into.
	bool PushInputOnlyParameter(FName Name, FType Type);
	
	// Pushes an input-output parameter and returns whether there was an available free parameter to push this one into.
	bool PushInputOutputParameter(FName Name, FType Type);
	
	// Pushes an output-only parameter and returns whether there was an available free parameter to push this one into.
	bool PushOutputOnlyParameter(FName Name, FType Type);

	// Returns the total number of parameters.
	uint32 GetNumParameters() const { return NumInputOnlyParams + NumInputOutputParams + NumOutputOnlyParams; }
};

// Describes a user-specified HLSL function.
struct FFunctionHLSLDesc : FFunctionDesc
{
	// The user-specified HLSL code inside the function body.
	FStringView Code{};
};

// Helper structure to simplify passing attributes to texture sampling emitter functions that share the same attributes.
struct FTextureSampleAttributes
{
	// Specifies which sampler source mode to use. By default SSM_FromTextureAsset.
	ESamplerSourceMode SamplerSourceMode { SSM_FromTextureAsset };
	
	// Specifies which sampler type to use. By default SAMPLERTYPE_MAX which implies the sampler type is fetched from the texture object.
	// See SamplerType field in FTextureUniform or FVirtualTextureUniform.
	EMaterialSamplerType SamplerType { SAMPLERTYPE_MAX };

	// Specifies whether to enable virtual texture sampling feedback. This is only used for pixel shaders. Enabled by default.
	bool bEnableFeedback : 1 { true };

	// True if adaptive VT sampling is used. Only used for virtual textures.
	bool bIsAdaptive : 1 { false };
};

//
struct FSubgraphOutputMapping
{
	//
	UMaterialExpression* Expression;
	
	//
	int32 OutputIndex;
};

// @massimo.tristano all Substrate should be moved out to a separate module
struct FSubstrateShadingModelsDesc
{
	FValueRef BaseColor;
	FValueRef Specular;
	FValueRef Metallic;
	FValueRef Roughness;
	FValueRef Anisotropy;
	FValueRef SubSurfaceColor;
	FValueRef SubSurfaceProfileId;
	FValueRef ClearCoat;
	FValueRef ClearCoatRoughness;
	FValueRef EmissiveColor;
	FValueRef Opacity;
	FValueRef ThinTranslucentTransmittanceColor;
	FValueRef ThinTranslucentSurfaceCoverage;
	FValueRef WaterScatteringCoefficients;
	FValueRef WaterAbsorptionCoefficients;
	FValueRef WaterPhaseG;
	FValueRef ColorScaleBehindWater;
	FValueRef ShadingModel;
	FValueRef Normal;
	FValueRef Tangent;
	FValueRef ClearCoatNormal;
	FValueRef CustomTangent;
	FValueRef BasisIndexMacro;
	FValueRef ClearCoat_BasisIndexMacro;
	bool bHasDynamicShadingModels;
};

struct FSubstrateSlabDesc
{
	FValueRef Normal;
	FValueRef DiffuseAlbedo;
	FValueRef F0;
	FValueRef F90;
	FValueRef Roughness;
	FValueRef Anisotropy;
	FValueRef SSSProfileId;
	FValueRef SSSMFP;
	FValueRef SSSMFPScale;
	FValueRef SSSPhaseAniso;
	FValueRef SSSType;
	FValueRef EmissiveColor;
	FValueRef SecondRoughness;
	FValueRef SecondRoughnessWeight;
	FValueRef SecondRoughnessAsSimpleClearCoat;
	FValueRef ClearCoatUseSecondNormal;
	FValueRef ClearCoatBottomNormal;
	FValueRef FuzzAmount;
	FValueRef FuzzColor;
	FValueRef FuzzRoughness;
	FValueRef GlintValue;
	FValueRef GlintUV;
	FValueRef SpecularProfileId;
	FValueRef Thickness;
	FValueRef IsThin;
	FValueRef IsAtBottom;
	FValueRef LocalBasisIndex;
};

struct FSubstrateToonDesc
{
	FValueRef Normal;
	FValueRef ToonProfileId;
	FValueRef BaseColor;
	FValueRef Metallic;
	FValueRef Specular;
	FValueRef Roughness;
	FValueRef EmissiveColor;
	FValueRef PatternUVs;
	FValueRef IsAtBottom;
	FValueRef LocalBasisIndex;
};

// The FEmitter is a helper object responsible for emitting MIR values and instructions.
// It is primarily used within UMaterialExpression::Build() implementations to lower the
// high-level semantics of a material expression into corresponding lower-level MIR nodes.
//
// During the emission process, the emitter automatically performs various
// simplifications and constant folding optimizations where possible.
//
// An instance of the emitter is created and managed by the FMaterialIRModuleBuilder.
// This instance is then passed to the UMaterialExpression::Build() functions
// so that they can emit their corresponding IR values.
//
// IMPORTANT: Users should *not* assume the exact MIR::FType of the FValue* returned
// by EmitXXX() functions. The emitter is free to optimize operations, which may
// result in a returned value having a different type than naively expected (e.g.,
// folding an operation into a constant).
//
// The emitter handles invalid inputs gracefully. If an operation is attempted
// with invalid arguments (e.g., acting on arguments of unexpected types), the emitter
// automatically reports an error and returns the MIR::FPoison value. Furthermore, all
// EmitXXX() functions are robust to receiving MIR::FPoison as input; they will simply
// propagate the poison value without attempting the operation.
class FEmitter
{
public:
	/*--------------------------------- Error handling ---------------------------------*/

	// Returns whether the emitter is in a non-clean state (pending dependency or error).
	// You can also use UE_MIR_CHECKPOINT() in your code to check this and automatically
	// return if a state is invalid and expression Build() execution should abort.
	bool IsInvalid() const { return State != EState::None; }

	// Reports an error using printf-like format and arguments to format the error message.
	template <typename... TArgs>
	void Errorf(FValueRef Source, UE::Core::TCheckedFormatString<FString::FmtCharType, TArgs...> Format, TArgs... Args)
	{
		Error(Source, FString::Printf(Format, Args...));
	}

	// Reports an error using printf-like format and arguments to format the error message.
	template <typename... TArgs>
	void Errorf(UE::Core::TCheckedFormatString<FString::FmtCharType, TArgs...> Format, TArgs... Args)
	{
		Error(FString::Printf(Format, Args...));
	}

	// Reports an error with given error message.
	void Error(FValueRef Source, FStringView Message);

	// Reports an error with given error message.
	void Error(FStringView Message);

	/*--------------------------------- Type handling ----------------------------------*/

	// Tries to find a common type between A and B.
	// The trivial case is when A and B are the same type, in which case that type is
	// the common type. When A and B are primitive, the common type is defined as the type
	// with the highest dimensions between the two and with a scalar kind that follows the
	// progression: bool -> int -> float
	// As an example, the common type between a float3 and a bool4 will be a float4.
	// Note; that this may cause a loss of data when casting from int to float.
	// It returns the common type if it exists, nullptr otherwise.
	FType TryGetCommonType(FType A, FType B);
	
	// Gets the common type between A and B and reports an error if no such type exists.
	// See TryGetCommonType()
	FType GetCommonType(FType A, FType B);
		
	// Gets the common type between the specified values types. Note that values in this array
	// are allowed to be null/poison.
	// See GetCommonType(FType A, FType B)
	FType GetCommonType(TConstArrayView<FValueRef> Values);

	// Returns the type of the i-th attribute in the given material aggregate.
	FType GetMaterialAggregateAttributeType(const UMaterialAggregate* Aggregate, int32 AttributeIndex);

	/*-------------------------------- Input management --------------------------------*/

	// Retrieves the value from the given input, if present. A null reference otherwise.
	// Note: it does not report an error if the value is missing.
	// If the expression is marked MIR_OnDemandInputRequest and the dependency hasn't been
	// built yet, sets Pending and returns poison, use UE_MIR_CHECKPOINT to defer.
	// It is safe to pass returned value to other Emitter functions as argument even if null.
	// For instance, the following is safe and idiomatic
	//
	//    FValueRef Value = Emitter.CheckIsArithmetic(Emitter.TryInput(&A));
	//    UE_MIR_CHECKPOINT(Emitter);
	//    if (Value) { /* handle case when value is provided */ }
	//
	// Keep in mind that a reference can be null (its Value is null) or invalid (which means
	// that it's null OR poison).
	FValueRef TryInput(const FExpressionInput* Input);

	// Retrieves the value flowing into given input.
	// If the value is missing, it reports an error and returns poison.
	// If the expression is marked MIR_OnDemandInputRequest and the dependency hasn't been
	// built yet, sets Pending and returns poison, use UE_MIR_CHECKPOINT to defer.
	FValueRef Input(const FExpressionInput* Input);

	// If the input has no value flowing in (e.g., because the input is disconnected),
	// these DefaultXXX() functions will emit a constant value, bind it to the input, and
	// return it.
	FValueRef InputDefaultBool(const FExpressionInput* Input, bool Default);
	FValueRef InputDefaultInt(const FExpressionInput* Input, TInteger Default);
	FValueRef InputDefaultInt2(const FExpressionInput* Input, UE::Math::TIntVector2<TInteger> Default);
	FValueRef InputDefaultInt3(const FExpressionInput* Input, UE::Math::TIntVector3<TInteger> Default);
	FValueRef InputDefaultInt4(const FExpressionInput* Input, UE::Math::TIntVector4<TInteger> Default);
	FValueRef InputDefaultFloat(const FExpressionInput* Input, TFloat Default);
	FValueRef InputDefaultFloat2(const FExpressionInput* Input, UE::Math::TVector2<TFloat> Default);
	FValueRef InputDefaultFloat3(const FExpressionInput* Input, UE::Math::TVector<TFloat> Default);
	FValueRef InputDefaultFloat4(const FExpressionInput* Input, UE::Math::TVector4<TFloat> Default);

	// Try read the value flowing into given Input, and if not present use Default instead.
	FValueRef InputDefault(const FExpressionInput* Input, FValueRef Default);

	// Retrieves the value from the given input of given type kind.
	// If the value is missing or its type does not have the given kind, it reports an error and returns poison.
	FValueRef CheckTypeIsKind(FValueRef Value, ETypeKind Kind);
	
	// Validates that the value is of a primitive type. If not, reports an error and returns poison.
	FValueRef CheckIsPrimitive(FValueRef Value);
	
	// Validates that the value is of an arithmetic type (int or float). Reports error and returns poison otherwise.
	FValueRef CheckIsArithmetic(FValueRef Value);

	// Validates that the value is of a boolean type. If not, reports an error and returns poison.
	FValueRef CheckIsBoolean(FValueRef Value);

	// Validates that the value is of a integral type. If not, reports an error and returns poison.
	FValueRef CheckIsInteger(FValueRef Value);

	// Validates that the value is a scalar. If not, reports error and returns poison.
	FValueRef CheckIsScalar(FValueRef Value);

	// Validates that the value is a vector. If not, reports error and returns poison.
	FValueRef CheckIsVector(FValueRef Value);

	// Validates that the value is a vector of number of components between specified min/max.
	// If not, reports error and returns poison.
	FValueRef CheckIsVector(FValueRef Value, int32 MinNumComponents, int32 MaxNumComponents);

	// Validates that the value is either a scalar or a vector. Reports error and returns poison otherwise.
	FValueRef CheckIsScalarOrVector(FValueRef Value);
	
	// Validates that the value is a matrix. If not, reports error and returns poison.
	FValueRef CheckIsMatrix(FValueRef Value);

	// Retrieves the texture value flowing into given input.
	// If the value is missing or cannot be cast to a texture, it reports an error and returns poison.
	FValueRef CheckIsTexture(FValueRef Value);

	// Validates that the value is of an aggregate type. If not, reports error and returns poison.
	FValueRef CheckIsAggregate(FValueRef Value, const UMaterialAggregate* Aggregate = nullptr);

	// Checks that the material domain is any of the ones specified.
	void CheckMaterialDomainIsAnyOf(TConstArrayView<EMaterialDomain> Domains);

	// Checks that the material has the specified usage flag.
	void CheckMaterialHasUsageFlag(EMaterialUsage UsageFlag);

	// Checks that the material feature level is equal or greater than the one specified.
	void CheckFeatureLevelIsAtLeast(ERHIFeatureLevel::Type FeatureLevel);

	// Casts the value flowing into this input to a constant boolean and returns it.
	// If the value is missing, or is not a constant boolean it reports an error and returns false.
	bool ToConstantBool(FValueRef Value);

	/*-------------------------------- Output management -------------------------------*/
	
	// Flows given `Value` out of the expression output with given `OutputIndex`.
	FEmitter& Output(int32 OutputIndex, FValueRef Value);

	// Flows given `Value` out of given expression `ExpressionOutput`.
	FEmitter& Output(const FExpressionOutput* ExpressionOutput, FValueRef Value);

	// Flows given `Value` out of all outputs in the current expression.
	FEmitter& OutputsWithComponentMask(FValueRef Value);

	/*------------------------------- Subgraph management ------------------------------*/
	
	// Registers the current expression as a subgraph.
	// - NumInputs is the number of input pins on this expression, used for connection insight deduplication.
	// - Outputs[i] is the inner expression that produces output i. When something later connects to
	//   this expression's output, resolution redirects to the mapped inner expression in the subgraph scope.
	void Subgraph(int32 NumInputs, TArrayView<FSubgraphOutputMapping> Outputs);

	// Retrieves the value from the subgraph expression's input pin at the given index, if present.
	// A null reference otherwise. Does not report an error if the value is missing.
	// If the expression is marked MIR_OnDemandInputRequest and the dependency hasn't been
	// built yet, sets Pending and returns poison, use UE_MIR_CHECKPOINT to defer.
	FValueRef TrySubgraphInput(int32 InputIndex);

	// Retrieves the value from the subgraph expression's input pin at the given index.
	// If the value is missing, it reports an error and returns poison.
	// If the expression is marked MIR_OnDemandInputRequest and the dependency hasn't been
	// built yet, sets Pending and returns poison, use UE_MIR_CHECKPOINT to defer.
	FValueRef SubgraphInput(int32 InputIndex);

	// Returns the expression that declared the current subgraph scope, or null if at root level.
	UMaterialExpression* GetSubgraphExpression();
	
	/*------------------------------- Constants emission -------------------------------*/

	// Converts the given UE::Shader::FValue to a MIR constant value and returns it.
	FValueRef ConstantFromShaderValue(const UE::Shader::FValue& InValue);

	// Emits a default value of given type (zero initialized).
	FValueRef ConstantDefault(FType Type);

	// Emits a scalar constant value of given kind with value 0 and returns it.
	FValueRef ConstantZero(EScalarKind Kind);

	// Emits a scalar constant value of given kind with value 1 and returns it.
	FValueRef ConstantOne(EScalarKind Kind);
	
	// Casts the given float value to the given scalar kind and returns its value.
	FValueRef ConstantScalar(EScalarKind Kind, TDouble Value);

	// Returns the constant boolean scalar `true`.
	FValueRef ConstantTrue();

	// Returns the constant boolean scalar `false`.
	FValueRef ConstantFalse();

	// Returns the given constant boolean scalar.
	FValueRef ConstantBool(bool InX);

	// Emits a constant bool 2D column vector and returns it.
	FValueRef ConstantBool2(bool InX, bool InY);

	// Emits a constant bool 3D column vector and returns it.
	FValueRef ConstantBool3(bool InX, bool InY, bool InZ);

	// Emits a constant bool 4D column vector and returns it.
	FValueRef ConstantBool4(bool InX, bool InY, bool InZ, bool InW);

	// Returns the given constant integer scalar.
	FValueRef ConstantInt(TInteger InX);
	
	// Emits a constant integer 2D column vector and returns it.
	FValueRef ConstantInt2(UE::Math::TIntVector2<TInteger> InValue);
	
	// Emits a constant integer 3D column vector and returns it.
	FValueRef ConstantInt3(UE::Math::TIntVector3<TInteger>  InValue);

	// Emits a constant integer 4D column vector and returns it.
	FValueRef ConstantInt4(UE::Math::TIntVector4<TInteger>  InValue);

	// Returns the given constant float scalar.
	FValueRef ConstantFloat(TFloat InX);

	// Emits a constant float 2D column vector and returns it.
	FValueRef ConstantFloat2(UE::Math::TVector2<TFloat> InValue);

	// Emits a constant float 3D column vector and returns it.
	FValueRef ConstantFloat3(UE::Math::TVector<TFloat> InValue);

	// Emits a constant float 4D column vector and returns it.
	FValueRef ConstantFloat4(UE::Math::TVector4<TFloat> InValue);

	// Returns the given constant LWC scalar.
	FValueRef ConstantDouble(TDouble InX);

	// Emits a constant double 2D column vector and returns it.
	FValueRef ConstantDouble2(UE::Math::TVector2<TDouble> InValue);

	// Emits a constant double 3D column vector and returns it.
	FValueRef ConstantDouble3(UE::Math::TVector<TDouble> InValue);

	// Emits a constant double 4D column vector and returns it.
	FValueRef ConstantDouble4(UE::Math::TVector4<TDouble> InValue);

	/*--------------------- Other non-instruction values emission ---------------------*/

	// Emits the poison value.
	FValueRef Poison();

	// Emits an analysis error value.
	// The purpose of this value is to report an error during analysis if the analyzer
	// proves that the value is effectively used. This value is useful when combined with
	// others, especially the StageSwitch instruction in order to gate off certain operations
	// on specific stage.
	FValueRef AnalysisError(FType Type, FStringView Message);

	// Emits an builtin value of given id and returns it.
	FValueRef Builtin(EBuiltin Id);

	// Emits a material parameter collection.
	FValueRef MaterialParameterCollection(UMaterialParameterCollection* Collection);

	// Emits a named uniform value of primitive type.
	// The uniform will be declared onto the MaterialCompilationOutput UniformExpressionSet as a numeric parameter.
	FValueRef NamedPrimitiveUniform(FName Name, FValueRef DefaultConstant);

	// Emits a texture uniform value of any type except for runtime virtual textures.
	// - SamplerType specifies the default sampler associated to this texture.
	FValueRef TextureUniform(FName Name, UTexture* Texture, EMaterialSamplerType SamplerType);

	// Emits a runtime virtual texture uniform value.
	// - VTLayerIndex specifies VT stack layer index. If set to INDEX_NONE, it is assigned automatically from the uniform expression set.
	// - VTPageTableIndex specifies the VT page table index.
	FValueRef VirtualTextureUniform(FName Name, URuntimeVirtualTexture* VirtualTexture, int32 VTLayerIndex, int32 VTPageTableIndex);

	// Emits a preshader parameter value with given opcode.
	FValueRef PreshaderParameter(FType Type, EPreshader2Opcode Opcode, FValueRef SourceParameter, FPreshaderParameterPayload Payload = {});

	// Emits a custom primitive data value with given index.
	FValueRef CustomPrimitiveData(uint32 PrimitiveDataIndex);

	/*----------------------------- Instructions emission -----------------------------*/
	
	// Emits an instruction that sets the material attribute with given property to given argument.
	// Note: This API is used by the builder and you should not use it manually.
	void Private_SetMaterialOutput(EMaterialProperty Property, FValueRef Arg);
	
	// Emits a group of custom outputs.
	// This function emits a new custom output group made of a number of custom outputs. The function emits a number
	// of SetMaterialOutput instruction for each argument in Args, giving each the specified group name appended by
	// the output index (e.g. "TangentOutput0" for a group named "TangentOutput").
	// The type of each custom output is inferred by the type of the argument value it is set to, therefore make sure
	// you cast each output argument to the desired type before calling this function.
	void SetCustomOutputs(FStringView Name, TConstArrayView<FValueRef> Args, EMaterialOutputFrequency Frequency);
	void SetCustomOutputs(FStringView Name, TConstArrayView<FValueRef> Args, EMaterialOutputFrequency Frequency, TFunctionRef<void(FMaterialIRModule&)> Func);
	
	// Emits a 2D column vector from X and Y scalars.
	FValueRef Vector2(FValueRef X, FValueRef Y);

	// Emits a 3D column vector concatenating A, B and C components.
	// Note: A, B and C can be scalars or vectors, but the sum of all components from A, B and possibly C must be 3.
	FValueRef Vector3(FValueRef A, FValueRef B, FValueRef C = {});

	// Emits a 4D column vector concatenating A, B, C and D components.
	// Note: A, B, C and D can be scalars or vectors, but the sum of all components from A, B and possibly C and D must be 4.
	FValueRef Vector4(FValueRef A, FValueRef B, FValueRef C = {}, FValueRef D = {});

	// Emits a vector putting together all the components of given arguments.
	// Arguments can be individual scalars or vectors themselves, but the concatenation of all their components cannot exceed 4 dimensions.
	// Example: calling this function with a (float, float2) will result in a float3.
	FValueRef Vector(FValueRef A, FValueRef B, FValueRef C = {}, FValueRef D = {});

	// Constructs an aggregate value using the provided aggregate definition.
	// The resulting value has default-initialized attributes.
	FValueRef Aggregate(const UMaterialAggregate* InAggregate);

	// Constructs an aggregate value using the given type and prototype, and initializes it with
	// the provided list of attribute values. Each value corresponds to an attribute in declaration order.
	// AttributeValues can be smaller than the number attributes in the aggregate. For excess attributes,
	// the value in the prototype (if provided) or default initialized otherwise.
	FValueRef Aggregate(const UMaterialAggregate* InAggregate, FValueRef InPrototype, TConstArrayView<FValueRef> AttributeValues);

	// Constructs an aggregate value using the given type and prototype, initializing its attributes using the provided assignments.
	// Each assignment specifies the target attribute and its initial value.
	// Unassigned attributes use the value in the prototype (if provided) or default initialized otherwise.
	FValueRef Aggregate(const UMaterialAggregate* InAggregate, FValueRef InPrototype, TConstArrayView<FAttributeAssignment> AttributeAssignments);

	// Emits a mathematical operation instruction with given operator and arguments.
	// Note: This function will try to simplify the operation at translation time if possible.
	//       The returned value is therefore not guaranteed to be an FOperator instruction instance.
	FValueRef Operator(EOperator Operator, FValueRef A, FValueRef B = {}, FValueRef C = {});
	
	// Emits a branch instruction.
	// When the result of the Condition argument is true, the instruction will evaluate
	// the True argument, otherwise the False argument.
	// This instruction will place the as much of the other instructions whose results
	// serve the computation of True or False within separate inner scopes, in order to
	// avoid unnecessarily computing the inactive argument.
	FValueRef Branch(FValueRef Condition, FValueRef True, FValueRef False);
		
	// Casts the given value to the specified target type. Returns poison if the cast is invalid.
	FValueRef Cast(FValueRef Value, FType TargetType);
	
	// Casts the given primitive value to a scalar preserving the same ScalarKind.
	// Note: If this is a vector, it will convert to its first component.
	FValueRef CastToScalar(FValueRef Value);

	// Casts the given primitive value to a row vector of specified columns preserving the same ScalarKind.
	FValueRef CastToVector(FValueRef Value, int32 NumColumns);

	// Casts the given primitive value to a given scalar kind, maintaining the same
	// number of rows and columns.
	FValueRef CastToScalarKind(FValueRef Value, EScalarKind ToScalarKind);
	FValueRef CastToBoolKind(FValueRef Value);
	FValueRef CastToIntKind(FValueRef Value);
	FValueRef CastToFloatKind(FValueRef Value);

	// These functions cast the given value to the a primitive scalar or column vector type of given rows.
	// If the value is missing ore could not be cast to the relative primitive type, they repors an error and return poison.
	FValueRef CastToBool(FValueRef Value, int NumColumns);
	FValueRef CastToInt(FValueRef Value, int NumColumns);
	FValueRef CastToFloat(FValueRef Value, int NumColumns);
	
	// Extracts the component with given index from the given argument value.
	// The argument value must be a scalar or a vector primitive type, otherwise it reports an error and returns poison.
	// Note that if no error occurred, the returned value is guaranteed to be a primitive scalar.
	FValueRef Subscript(FValueRef Value, int32 Index);

	// Swizzles the given argument by given mask.
	// The argument value must be a scalar or vector type. If the operation is invalid on
	// given argument and mask it reports an error and returns poison.
	FValueRef Swizzle(FValueRef Value, FSwizzleMask Mask);

	// Transposes the given MxN primitive value into the NxM result.
	// Note if Value is a row-vector, returns a column-vector and viceversa.
	// If Value is not a primitive type, reports an error and returns poison.
	FValueRef Transpose(FValueRef Value);

	// Emits a stage switch instruction. Use this instruction to select a different value
	// in each execution stage.
	// Note: ValuePerStage must have no more than MIR::NumStages elements.
	FValueRef StageSwitch(FType Type, TConstArrayView<FValueRef> ValuePerStage);

	// Emits a NOP instruction that behaves as a default value of Arg type, but will still
	// cause Arg to be analyzed. Use it when you want Arg to be analyzed (to record side effects
	// on the module), but not evaluated. 
	FValueRef Nop(FValueRef Arg);

	// Emits a texture gather instruction.
	// 
	// @param Texture The texture to sample. Must be a texture or virtual texture uniform value.
	// @param TexCoord The UV coordinates at which to sample the texture.
	// @param GatherMode What channel to gather (any ::GatherXXX value).
	// @param BaseAttributes Base sampling attributes for sampler source and type as well as VT attributes.
	FValueRef TextureGather(FValueRef Texture, FValueRef TexCoord, ETextureReadMode GatherMode, const FTextureSampleAttributes& BaseAttributes);

	// Emits a texture sample instruction.
	// 
	// @param Texture The texture to sample. Must be a texture or virtual texture uniform value.
	// @param TexCoord The UV coordinates at which to sample the texture.
	// @param bAddViewMaterialMipBias Whether to add the view material texture mip bias to the mip level.
	// @param BaseAttributes Base sampling attributes for sampler source and type as well as VT attributes.
	FValueRef TextureSample(FValueRef Texture, FValueRef TexCoord, bool bAddViewMaterialMipBias, const FTextureSampleAttributes& BaseAttributes);

	// Emits a texture sample instruction with a manually given mip level.
	// 
	// @param Texture The texture to sample. Must be a texture or virtual texture uniform value.
	// @param TexCoord The UV coordinates at which to sample the texture.
	// @param MipLevel Which mip level to use for this sample.
	// @param bAddViewMaterialMipBias Whether to add the view material texture mip bias to the mip level.
	// @param BaseAttributes Base sampling attributes for sampler source and type as well as VT attributes.
	FValueRef TextureSampleLevel(FValueRef Texture, FValueRef TexCoord, FValueRef MipLevel, bool bAddViewMaterialMipBias, const FTextureSampleAttributes& BaseAttributes);

	// Emits a texture sample instruction with bias added to the automatically selected mip level.
	// 
	// @param Texture The texture to sample. Must be a texture or virtual texture uniform value.
	// @param TexCoord The UV coordinates at which to sample the texture.
	// @param MipBias The bias to add to the automatically selected mip level.
	// @param bAddViewMaterialMipBias Whether to add the view material texture mip bias to the mip level.
	// @param BaseAttributes Base sampling attributes for sampler source and type as well as VT attributes.
	FValueRef TextureSampleBias(FValueRef Texture, FValueRef TexCoord, FValueRef MipBias, bool bAddViewMaterialMipBias, const FTextureSampleAttributes& BaseAttributes);
	
	// Emits a texture sample instruction using user provided partial derivatives.
	// 
	// @param Texture The texture to sample. Must be a texture or virtual texture uniform value.
	// @param TexCoord The UV coordinates at which to sample the texture.
	// @param TexCoordDdx The partial derivative of the texture coordinates along the screen space x axis.
	// @param TexCoordDdy The partial derivative of the texture coordinates along the screen space y axis.
	// @param bAddViewMaterialMipBias Whether to add the view material texture mip bias to the mip level.
	// @param BaseAttributes Base sampling attributes for sampler source and type as well as VT attributes.
	FValueRef TextureSampleGrad(FValueRef Texture, FValueRef TexCoord, FValueRef TexCoordDdx, FValueRef TexCoordDdy, bool bAddViewMaterialMipBias, const FTextureSampleAttributes& BaseAttributes);

	// TODO
	FValueRef VTPageTableLoad(
		FValueRef Texture, TextureAddress AddressU, TextureAddress AddressV, FValueRef TexCoord,
		FValueRef TexCoordDdx = {}, FValueRef TexCoordDdy = {}, bool bEnableFeedback = true, bool bIsAdaptive = false, ETextureMipValueMode MipValueMode = TMVM_None, FValueRef MipValue = {});
	
	// Emits the partial derivative of given value.
	// This function differentiates the given value with respect to the given axis.
	// If differentiation is not possible, it returns a poison value to indicate an error.
	// This API can be used regardless of the stages value will execute in. Hardware partial
	// derivatives will be used by default where available. If not available, the analytical
	// derivative will be computed instead.
	// @param Value The value to differentiate.
	// @param Axis The axis along which to compute the partial derivative (X for ddx, Y for ddy).
	// @return A new FValueRef representing the partial derivative, or a poison value if invalid.
	FValueRef PartialDerivative(FValueRef Value, EDerivativeAxis Axis);

	// Computes and emits the analytical partial derivative of a give value.
	// @param Value The value to differentiate.
	// @param Axis The axis along which to compute the partial derivative (X for ddx, Y for ddy).
	// @return A new FValueRef representing the analytical derivative, or a poison value if invalid.
	FValueRef AnalyticalPartialDerivative(FValueRef Value, EDerivativeAxis Axis);

	// Creates a vertex to pixel interpolator for the given value.
	// The value is computed in the vertex stage, linearly interpolated across the primitive,
	// and exposed in the pixel stage. The returned value is thus valid only in the pixel stage.
	FValueRef VertexInterpolator(FValueRef VertexValue);

	// Emits an extern instruction based on a user-specified definition struct.
	// An extern instruction is how you can extend MIR capabilities from the outside. An extern instruction implements a well-defined
	// interface (specified by the MIR::CExternDefinition concept) and provides a way to build functionality not natively offered by the
	// MIR instruction set. For example, material-specific rendering features are implemented through externs rather than by extending MIR
	// itself. The MIR module builder treats extern instructions as black boxes, but can still work with them through their interface.
	// To emit an extern instruction, you need to provide a "definition", meaning a concrete POD struct that implements the extern concept.
	// Parameters:
	//    - `DefinitionPrototype` is an instance of the TDefinition struct you provide. This is moved into the emitted extern instruction's
	//      own buffer.
	//    - With `Arguments`, you can specify zero or more FValueRef arguments to provide to the extern instruction.
	// For more usage and general information, see `MIR::CExternDefinition`.
	template <CExternDefinition TDefinition, typename... TArguments>
	requires (std::same_as<TArguments, FValueRef> && ...)
	FValueRef Extern(TDefinition DefinitionPrototype, TArguments... Arguments);

	// Defines a user-specified HLSL function to embed in the generated HLSL when generated IR Module is translated to HLSL.
	// This will create a local function with the specified description. You can issue calls to the returned functions using the Call() API.
	FFunction* FunctionHLSL(const FFunctionHLSLDesc& Desc);

	// Emits a call to a specified user function using specified input arguments.
	// Note that the InputArguments must provide an argument for *all* function input parameters (that is including the input-output ones).
	FValueRef Call(FFunction* Function, TConstArrayView<FValueRef> InputArguments);

	// Emits an instruction that retrieves the output parameter value from a function call.
	FValueRef CallParameterOutput(FValueRef Call, uint32 ParameterIndex);
	
	/*--------------------------- Unary operators shortcuts ----------------------------*/

	FValueRef BitwiseNot(FValueRef A) { return Operator(UO_BitwiseNot, A); }
	FValueRef Negate(FValueRef A) { return Operator(UO_Negate, A); }
	FValueRef Not(FValueRef A) { return Operator(UO_Not, A); }
	FValueRef Abs(FValueRef A) { return Operator(UO_Abs, A); }
	FValueRef ACos(FValueRef A) { return Operator(UO_ACos, A); }
	FValueRef ACosh(FValueRef A) { return Operator(UO_ACosh, A); }
	FValueRef ASin(FValueRef A) { return Operator(UO_ASin, A); }
	FValueRef ASinh(FValueRef A) { return Operator(UO_ASinh, A); }
	FValueRef ATan(FValueRef A) { return Operator(UO_ATan, A); }
	FValueRef ATanh(FValueRef A) { return Operator(UO_ATan, A); }
	FValueRef Ceil(FValueRef A) { return Operator(UO_Ceil, A); }
	FValueRef Cos(FValueRef A) { return Operator(UO_Cos, A); }
	FValueRef Cosh(FValueRef A) { return Operator(UO_Cosh, A); }
	FValueRef Exponential(FValueRef A) { return Operator(UO_Exponential, A); }
	FValueRef Exponential2(FValueRef A) { return Operator(UO_Exponential2, A); }
	FValueRef Floor(FValueRef A) { return Operator(UO_Floor, A); }
	FValueRef Frac(FValueRef A) { return Operator(UO_Frac, A); }
	FValueRef Length(FValueRef A) { return Operator(UO_Length, A); }
	FValueRef Logarithm(FValueRef A) { return Operator(UO_Logarithm, A); }
	FValueRef Logarithm2(FValueRef A) { return Operator(UO_Logarithm2, A); }
	FValueRef Logarithm10(FValueRef A) { return Operator(UO_Logarithm10, A); }
	FValueRef LWCTile(FValueRef A) { return Operator(UO_LWCTile, A); }
	FValueRef Reciprocal(FValueRef A) { return Operator(UO_Reciprocal, A); }
	FValueRef Round(FValueRef A) { return Operator(UO_Round, A); }
	FValueRef Rsqrt(FValueRef A) { return Operator(UO_Rsqrt, A); }
	FValueRef Saturate(FValueRef A) { return Operator(UO_Saturate, A); }
	FValueRef Sign(FValueRef A) { return Operator(UO_Sign, A); }
	FValueRef Sin(FValueRef A) { return Operator(UO_Sin, A); }
	FValueRef Sinh(FValueRef A) { return Operator(UO_Sinh, A); }
	FValueRef Sqrt(FValueRef A) { return Operator(UO_Sqrt, A); }
	FValueRef Tan(FValueRef A) { return Operator(UO_Tan, A); }
	FValueRef Truncate(FValueRef A) { return Operator(UO_Truncate, A); }

	/*-------------------------- Binary operators shortcuts ----------------------------*/
	
	FValueRef GreaterThan(FValueRef A, FValueRef B) { return Operator(BO_GreaterThan, A, B); }
	FValueRef GreaterThanOrEquals(FValueRef A, FValueRef B) { return Operator(BO_GreaterThanOrEquals, A, B); }
	FValueRef LessThan(FValueRef A, FValueRef B) { return Operator(BO_LessThan, A, B); }
	FValueRef LessThanOrEquals(FValueRef A, FValueRef B) { return Operator(BO_LessThanOrEquals, A, B); }
	FValueRef Equals(FValueRef A, FValueRef B) { return Operator(BO_Equals, A, B); }
	FValueRef NotEquals(FValueRef A, FValueRef B) { return Operator(BO_NotEquals, A, B); }
	FValueRef And(FValueRef A, FValueRef B) { return Operator(BO_And, A, B); }
	FValueRef Or(FValueRef A, FValueRef B) { return Operator(BO_Or, A, B); }
	FValueRef Add(FValueRef A, FValueRef B) { return Operator(BO_Add, A, B); }
	FValueRef Subtract(FValueRef A, FValueRef B) { return Operator(BO_Subtract, A, B); }
	FValueRef Multiply(FValueRef A, FValueRef B) { return Operator(BO_Multiply, A, B); }
	FValueRef MatrixMultiply(FValueRef A, FValueRef B) { return Operator(BO_MatrixMultiply, A, B); }
	FValueRef Divide(FValueRef A, FValueRef B) { return Operator(BO_Divide, A, B); }
	FValueRef BitwiseAnd(FValueRef A, FValueRef B) { return Operator(BO_BitwiseAnd, A, B); }
	FValueRef BitwiseOr(FValueRef A, FValueRef B) { return Operator(BO_BitwiseOr, A, B); }
	FValueRef BitwiseXor(FValueRef A, FValueRef B) { return Operator(BO_BitwiseXor, A, B); }
	FValueRef BitShiftLeft(FValueRef A, FValueRef B) { return Operator(BO_BitShiftLeft, A, B); }
	FValueRef BitShiftRight(FValueRef A, FValueRef B) { return Operator(BO_BitShiftRight, A, B); }
	FValueRef Fmod(FValueRef A, FValueRef B) { return Operator(BO_Fmod, A, B); }
	FValueRef Max(FValueRef A, FValueRef B) { return Operator(BO_Max, A, B); }
	FValueRef Modulo(FValueRef A, FValueRef B) { return Operator(BO_Modulo, A, B); }
	FValueRef Min(FValueRef A, FValueRef B) { return Operator(BO_Min, A, B); }
	FValueRef Step(FValueRef A, FValueRef B) { return Operator(BO_Step, A, B); }
	FValueRef Dot(FValueRef A, FValueRef B) { return Operator(BO_Dot, A, B); }
	FValueRef Cross(FValueRef A, FValueRef B) { return Operator(BO_Cross, A, B); }
	FValueRef Atan2(FValueRef A, FValueRef B) { return Operator(BO_ATan2, A, B); }
	FValueRef Pow(FValueRef A, FValueRef B) { return Operator(BO_Pow, A, B); }

	/*-------------------------- Ternary operators shortcuts ---------------------------*/

	FValueRef Clamp(FValueRef A, FValueRef B, FValueRef C) { return Operator(TO_Clamp, A, B, C); }
	FValueRef Lerp(FValueRef A, FValueRef B, FValueRef C) { return Operator(TO_Lerp, A, B, C); }
	FValueRef Select(FValueRef A, FValueRef B, FValueRef C) { return Operator(TO_Select, A, B, C); }
	FValueRef Smoothstep(FValueRef A, FValueRef B, FValueRef C) { return Operator(TO_Smoothstep, A, B, C); }

	/*----------------------------- Substrate instructions -----------------------------*/
	
	// Emits an instruction that promotes a SubstrateData representing a default slab, encoded as MCT_Substrate.
	FValueRef SubstrateDefaultSlab();

	// Promote a SubstrateData BSDF to an operator so that is becomes a node of the SubstrateTree and can be combined with other operators.
	FValueRef SubstratePromoteToOperator(FValueRef SubstrateData, FValueRef OperatorIndex, FValueRef BSDFIndex, FValueRef LayerDepth, FValueRef bIsBottom);

	// Create a SubstrateData representing a legacy BSDF.
	FValueRef SubstrateShadingModels(const FSubstrateShadingModelsDesc Desc);

	// Create a SubstrateData representing a Slab BSDF.
	FValueRef SubstrateSlab(const FSubstrateSlabDesc& Desc);

	// Create a SubstrateData representing a Toon BSDF.
	FValueRef SubstrateToon(const FSubstrateToonDesc& Desc);

	// Get the SubstrateData resulting from the horizontal mixing of the Background and Foreground SubstrateData.
	FValueRef SubstrateHorizontalMixing(FValueRef Background, FValueRef Foreground, FValueRef Mix, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves);

	// Parameter blended version of SubstrateHorizontalMixing.
	FValueRef SubstrateHorizontalMixingParameterBlending(FValueRef Background, FValueRef Foreground, FValueRef Mix, FValueRef NormalMix, FValueRef SharedLocalBasisIndexMacro, FValueRef BackgroundNoV, FValueRef ForegroundNoV);

	// Get the SubstrateData resulting from the vertical layering of the Top over the Base SubstrateData.
	FValueRef SubstrateVerticalLayering(FValueRef Top, FValueRef Base, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves);

	// Parameter blended version of SubstrateVerticalLayering.
	FValueRef SubstrateVerticalLayeringParameterBlending(FValueRef Top, FValueRef Base, FValueRef SharedLocalBasisIndexMacro, FValueRef TopNoV, FValueRef BaseNoV);

	// Get the SubstrateData resulting from the coverage weight of the A by Weight.
	FValueRef SubstrateCoverageWeight(FValueRef A, FValueRef Weight, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves);

	// Parameter blended version of SubstrateCoverageWeight.
	FValueRef SubstrateCoverageWeightParameterBlending(FValueRef A, FValueRef Weight);

	// Get the SubstrateData resulting from the addition of the A abd B slabs.
	FValueRef SubstrateAdd(FValueRef A, FValueRef B, FValueRef OperatorIndex, FValueRef MaxDistanceFromLeaves);

	// Parameter blended version of SubstrateAdd.
	FValueRef SubstrateAddParameterBlending(FValueRef A, FValueRef B, FValueRef NormalMix, FValueRef SharedLocalBasisIndexMacro, FValueRef ANoV, FValueRef BNoV);

	// Parameter blended version of SubstrateSelect, selecting and thus enforcing a single BSDF per pixel: A and B.
	FValueRef SubstrateSelectParameterBlending(FValueRef A, FValueRef B, FValueRef SelectValueArg, FValueRef SharedLocalBasisIndexMacro);

	/*---------------------------- Pass through accessors -----------------------------*/

	// Returns the shader platform the source material is translating for.
	EShaderPlatform GetShaderPlatform() const;
	
	// Returns the target platform the source material is translating for.
	const ITargetPlatform* GetTargetPlatform() const;
	
	// Returns the target feature level the source material is translating for. 
	ERHIFeatureLevel::Type GetFeatureLevel() const;

	// Returns the target quality level the source material is translating for. 
	EMaterialQualityLevel::Type GetQualityLevel() const;
	
	// Returns the source material interface. 
	const UMaterialInterface* GetMaterialInterface() const { return MaterialInterface; }
	const UMaterial* GetBaseMaterial() const { return BaseMaterial; }

	//
	const FStaticParameterSet* GetStaticParameterSet() const { return StaticParameterSet; }

	// Returns the Substrate translator data in use.
	Substrate::FSubstrateTranslatorData* GetSubstrateTranslatorData() const;

	// Internal
	struct FPrivate;

private:
	struct FValueKeyFuncs : DefaultKeyFuncs<FValue*>
	{
		static bool Matches(KeyInitType A, KeyInitType B);
		static uint32 GetKeyHash(KeyInitType Key);
	};

private:
	FEmitter() {}

	// Initializes the emitter, called by the friend builder.
	void Initialize();

	//
	FValueRef EmitPrototype_Internal(const FValue& Value);

private:
	// Pointer to the builder internal implementation.
	FMaterialIRModuleBuilderImpl* BuilderImpl{};

	// The material interface being translated.
	UMaterialInterface* MaterialInterface{};

	// The base material of the interface being translated (could be different from the interface if the interface is a material instance).
	UMaterial* BaseMaterial{};

	// The IR module being built.
	FMaterialIRModule* Module{};

	// The set of static parameter assignments to use during this translation.
	const FStaticParameterSet* StaticParameterSet{};

	// The current expression being built (set by the builder).
	UMaterialExpression* Expression{};

	// The substrate translator common data representing the material topology.
	struct Substrate::FSubstrateTranslatorData* SubstrateTranslatorData;

	// Global "true" constant.
	FValue* TrueConstant{};

	// Global "false" constant.
	FValue* FalseConstant{};

	// The set of all values previously emitted. It is used to avoid duplicating identical
	// values, explointing the strict SSA data-flow paradigm of MIR to efficiently reuse
	// calculations.
	TSet<FValue*, FValueKeyFuncs> ValueSet{};

	// Key funcs for deduplicating HLSL functions by content.
	struct FFunctionHLSLKeyFuncs : DefaultKeyFuncs<const FFunctionHLSL*>
	{
		static bool Matches(const FFunctionHLSL* A, const FFunctionHLSL* B) { return A->Equals(B); }
		static uint32 GetKeyHash(const FFunctionHLSL* Key);
	};

	// The set of unique HLSL functions emitted so far, used to deduplicate identical function definitions.
	TSet<FFunctionHLSL*, FFunctionHLSLKeyFuncs> FunctionHLSLSet{};

	// Tracks the current expression build state.
	enum class EState : uint8
	{
		None,
		Pending, // The expression has an unresolved input dependency and must be deferred.
		Error,   // An error was reported.
	};

	// Current emitter state for the expression being built.
	EState State = EState::None;

	// Whether the current expression opts into on-demand input resolution via MIR_OnDemandInputRequest metadata.
	bool bExpressionRequestsInputsOnDemand = false;

	friend FMaterialIRModuleBuilder;
	friend FMaterialIRModuleBuilderImpl;
};

//------------------------------------------------------------------------
//                        IMPLEMENTATION DETAILS
//------------------------------------------------------------------------

namespace Internal
{

template <int32 NumArguments>
struct TValueArgumentsArray
{
	FValue* Arguments[NumArguments];
};

template <>
struct TValueArgumentsArray<0>
{
};

template <typename TDefinition, int32 TNumArguments>
struct TConcreteExtern : FExtern, TValueArgumentsArray<TNumArguments>, TDefinition
{
	TConcreteExtern(TDefinition&& Definition) : TDefinition { MoveTemp(Definition) } {}

	static TDefinition& Get(FExtern *Self)
	{
		return *static_cast<TDefinition *>(static_cast<TConcreteExtern*>(Self));
	}

	static const TDefinition& Get(const FExtern *Self)
	{
		return *static_cast<const TDefinition *>(static_cast<const TConcreteExtern*>(Self));
	}
};

template <typename TDefinition, int32 TNumArguments>
struct TConcreteExternInterface
{
	using TMyConcreteExtern = TConcreteExtern<TDefinition, TNumArguments>;

	inline static const FExternImpl Instance =
	{
		.ByteSize = (uint32)sizeof(TMyConcreteExtern),

		.GetArguments = +[] (const FExtern* Extern) -> TConstArrayView<FValue*>
		{
			if constexpr (TNumArguments > 0)
			{
				return { static_cast<const TMyConcreteExtern*>(Extern)->Arguments, TNumArguments };
			}
			else
			{
				return {};
			}
		},

		.GetInfo = +[] (const FExtern* Extern) -> MIR::FExternInfo
		{
			return TMyConcreteExtern::Get(Extern).GetInfo();
		},

		.Analyze = +[] (FExtern* Extern, FExternAnalysisContext& Context) {
			if constexpr (CExternHasAnalyze<TDefinition>)
			{
				TMyConcreteExtern::Get(Extern).Analyze(Context);
			}
		},

		.AnalyzeInStage = +[] (FExtern* Extern, FExternAnalysisContext& Context, EStage Stage)
		{
			if constexpr (CExternHasAnalyzeInStage<TDefinition>)
			{
				TMyConcreteExtern::Get(Extern).AnalyzeInStage(Context, Stage);
			}
		},
		
		.ToHLSL = +[] (const FExtern* Extern, FExternPrinterHLSL& Printer)
		{
			TMyConcreteExtern::Get(Extern).ToHLSL(Printer);
		},

		.EmitDebugInfo = +[] (const FExtern* Extern, FString& Out)
		{
			if constexpr (CExternHasEmitDebugInfo<TDefinition>)
			{
				TMyConcreteExtern::Get(Extern).EmitDebugInfo(Out);
			}
		},
	};
};

} // namespace Internal

// Implementation of FEmitter::Extern
template <CExternDefinition TDefinition, typename... TArguments>
requires (std::same_as<TArguments, FValueRef> && ...)
FValueRef FEmitter::Extern(TDefinition DefinitionPrototype, TArguments... Arguments)
{
	if constexpr (sizeof...(Arguments) > 0)
	{
		if ((!FValueRef{ Arguments }.IsValid() || ...))
		{
			return Poison();
		}
	}

	using TConcreteExtern = typename Internal::TConcreteExtern<TDefinition, sizeof...(Arguments)>;

	// Read the value type from the definition prototype
	MIR::FType Type = DefinitionPrototype.GetInfo().Type;

	// Create the concrete extern instruction prototype.
	TTypeCompatibleBytes<TConcreteExtern> ProtoStorage;
	FMemory::Memzero(ProtoStorage);

	// Copy the definition into the zeroed storage. The approach depends on whether the definition
	// type has padding bytes: types without padding can be assigned directly, while types with
	// padding must use CopyTo to write only the real members, keeping padding bytes zeroed for
	// deterministic hashing.
	TConcreteExtern& ConcreteProto = *ProtoStorage.GetTypedPtr();
	if constexpr (std::is_empty_v<TDefinition>)
	{
		// Nothing to copy.
	}
	else if constexpr (std::has_unique_object_representations_v<TDefinition>)
	{
		*static_cast<TDefinition*>(&ConcreteProto) = DefinitionPrototype;
	}
	else
	{
		static_assert(CExternHasCopyTo<TDefinition>, "This extern definition type has padding bytes and must implement 'void CopyTo(T& Other) const' to ensure deterministic hashing. CopyTo must copy each member individually into Other.");
		DefinitionPrototype.CopyTo(static_cast<TDefinition&>(ConcreteProto));
	}

	// Copy over all the provided arguments
	int32 Index = 0;
	((ConcreteProto.Arguments[Index++] = Arguments), ...);

	// Setup the base extern instruction members
	MIR::FExtern& Proto = ConcreteProto;
	Proto.Kind = VK_Extern;
	Proto.Type = Type;
	Proto.Impl = &Internal::TConcreteExternInterface<TDefinition, sizeof...(Arguments)>::Instance;

	return EmitPrototype_Internal(Proto);
}

} // namespace MIR

#endif // #if WITH_EDITOR
