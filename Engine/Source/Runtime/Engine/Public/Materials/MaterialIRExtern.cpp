// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Materials/MaterialIRExtern.h"
#include "Materials/MaterialIRModule.h"
#include "Materials/MaterialExternalCodeRegistry.h"
#include "Materials/MaterialIR.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionUtils.h"
#include "UObject/Package.h"

namespace MIR
{

static EMaterialShaderFrequency MapToMaterialShaderFrequencyOrAny(MIR::EStage Stage)
{
	switch (Stage)
	{
		case MIR::Stage_Vertex: 	return EMaterialShaderFrequency::Vertex;
		case MIR::Stage_Pixel: 		return EMaterialShaderFrequency::Pixel;
		case MIR::Stage_Compute: 	return EMaterialShaderFrequency::Compute;
	}
	return EMaterialShaderFrequency::Any;
}

//------------------------------------------------------------------------
//                        FExternAnalysisContext
//------------------------------------------------------------------------

void FExternAnalysisContext::AddGraphProperties(MIR::EGraphProperties GraphProperties)
{
	Extern_Internal->GraphProperties |= GraphProperties;
}

bool FExternAnalysisContext::CheckStage(EStage Stage, EStageMask AllowedStages, const TCHAR* DebugName)
{
	if ((1 << (uint32)Stage) & (uint32)AllowedStages)
	{
		return true;
	}
	Module->AddError(nullptr, FString::Printf(TEXT("%s is not valid in the %s stage."), DebugName, LexToString(Stage)));
	return false;
}

//------------------------------------------------------------------------
//                        FExternFromMaterialDecl
//------------------------------------------------------------------------

FExternFromMaterialDecl::FExternFromMaterialDecl(const FMaterialExternalCodeDeclaration* Declaration)
	: Name{ Declaration->Name }
	, Declaration{ Declaration }
{
}

FExternFromMaterialDecl::FExternFromMaterialDecl(FName InName) 
	: Name { InName }
{
	Declaration = MaterialExternalCodeRegistry::Get().FindExternalCode(Name);
}

MIR::FExternInfo FExternFromMaterialDecl::GetInfo() const
{
	MIR::FExternInfo Info = { .Name = TEXTVIEW("ExternFromMaterialDecl") };
	if (Declaration)
	{
		Info.Type 			= FType::FromMaterialValueType(Declaration->GetReturnTypeValue());
		Info.DifferentialType	= Info.Type; // FExternFromMaterialDecl does not support specifying a different derivative type
		Info.Flags 			= Declaration->bIsInlined ? MIR::EExternFlags::Inline : MIR::EExternFlags::None;
	};
	return Info;
}

void FExternFromMaterialDecl::Analyze(FExternAnalysisContext& Context)
{
	if (!Declaration)
	{
		Context.Module->AddError(nullptr, FString::Printf(TEXT("Internal error: Could not find external code declaration '%s'."), *Name.ToString()));
		return;
	}

	// Validate this external code can be used for the current material domain. Empty list implies no  restriction on material domains.
	if (!Declaration->Domains.IsEmpty() && Declaration->Domains.Find(Context.MaterialInterface->GetMaterial()->MaterialDomain) == INDEX_NONE)
	{
		FString ErrorMessage;
		ErrorMessage.Appendf(TEXT("External code '%s' (asset: %s) is only available in the following material domains: "), *Declaration->Name.ToString(), *Context.MaterialInterface->GetOutermost()->GetFName().ToString());
		for (int32 i = 0; i < Declaration->Domains.Num(); ++i)
		{
			if (i > 0)
			{
				ErrorMessage.Append(TEXT(", "));
			}
			ErrorMessage.Append(MaterialDomainString(Declaration->Domains[i]));
		}
		Context.Module->AddError(nullptr, MoveTemp(ErrorMessage));
	}

	// Cast from material feature level enum to RHI feature level enum
	ERHIFeatureLevel::Type MinimumFeatureLevel = (ERHIFeatureLevel::Type)Declaration->MinimumFeatureLevel;
	if (Context.Module->GetFeatureLevel() < MinimumFeatureLevel)
	{
		FString FeatureLevelName;
		FString MinimumFeatureLevelName;
		GetFeatureLevelName(Context.Module->GetFeatureLevel(), FeatureLevelName);
		GetFeatureLevelName(MinimumFeatureLevel, MinimumFeatureLevelName);

		Context.Module->AddError(nullptr, FString::Printf(TEXT("Node %s requires feature level %s.  Current feature level is %s."), *Name.ToString(), *MinimumFeatureLevelName, *FeatureLevelName));
	}
}

void FExternFromMaterialDecl::AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage)
{
	if (!Declaration)
	{
		return;
	}

	const EMaterialShaderFrequency ShaderFrequency = MapToMaterialShaderFrequencyOrAny(Stage);
	if ((Declaration->ShaderFrequency & ShaderFrequency) == EMaterialShaderFrequency::None)
	{
		Context.Module->AddError(nullptr, FString::Printf(TEXT("External code declaration '%s' is not valid in the %s stage."), *Name.ToString(), LexToString(Stage)));
		return;
	}

	for (const FMaterialExternalCodeEnvironmentDefine& EnvironmentDefine : Declaration->EnvironmentDefines)
	{
		if ((int32)(EnvironmentDefine.ShaderFrequency & ShaderFrequency) != 0)
		{
			Context.Module->AddEnvironmentDefine(EnvironmentDefine.Name);
		}
	}
}

void FExternFromMaterialDecl::ToHLSL(FExternPrinterHLSL& Printer) const
{
	check(Declaration); // This should have been catched by Analyze()
	switch (Printer.Differential)
	{
		case EExternDifferential::None:        Printer << *Declaration->Definition; break;
		case EExternDifferential::ScreenSpaceX: Printer << *Declaration->DefinitionDDX; break;
		case EExternDifferential::ScreenSpaceY: Printer << *Declaration->DefinitionDDY; break;
	}
}

void FExternFromMaterialDecl::EmitDebugInfo(FString& Out) const
{
	Out.Appendf(TEXT("Name=\"%s\""), *Name.ToString());
}

void FExternFromMaterialDecl::CopyTo(FExternFromMaterialDecl& Other) const
{
	Other.Name = Name;
	Other.Declaration = Declaration;
}

//------------------------------------------------------------------------
//                          FExternFromInlineDecl
//------------------------------------------------------------------------

MIR::FExternInfo FExternFromInlineDecl::GetInfo() const
{
	check((Decl->CodeHLSLDDX && Decl->CodeHLSLDDY) || (!Decl->CodeHLSLDDX && !Decl->CodeHLSLDDY)); // Either both derivative snippets should be provided or neither
	
	// If no derivative was provided and this extern does not explicitly disallow derivatives, add the ZeroDerivatives flag
	EExternFlags Flags = Decl->Flags;
	if (!Decl->CodeHLSLDDX && (Flags & MIR::EExternFlags::NoDifferentials) == MIR::EExternFlags::None)
	{
		Flags |= MIR::EExternFlags::ZeroDifferentials;
	}

	return
		{
			.Name 			  = TEXT("ExternFromInlineDecl"),
			.Type 			  = Decl->Type,
			.DifferentialType = Decl->DifferentialType,
			.Flags 			  = Flags,
		};
}

void FExternFromInlineDecl::Analyze(FExternAnalysisContext& Context)
{
	// Validate this external code can be used for the current material domain. Empty list implies no  restriction on material domains.
	const EMaterialDomain Domain = Context.MaterialInterface->GetMaterial()->MaterialDomain;
	if ((Decl->MaterialDomainMask & (1 << (uint32)Domain)) == 0)
	{
		Context.Module->AddError(nullptr, FString::Printf(TEXT("Inline HLSL instruction does not support material domain '%s'."), *MaterialDomainString(Domain)));
	}

	// Cast from material feature level enum to RHI feature level enum
	if (Context.Module->GetFeatureLevel() < (ERHIFeatureLevel::Type)Decl->MinimumFeatureLevel)
	{
		FString FeatureLevelName;
		FString MinimumFeatureLevelName;
		GetFeatureLevelName(Context.Module->GetFeatureLevel(), FeatureLevelName);
		GetFeatureLevelName((ERHIFeatureLevel::Type)Decl->MinimumFeatureLevel, MinimumFeatureLevelName);

		Context.Module->AddError(nullptr, FString::Printf(TEXT("Inline HLSL instruction requires feature level %s. Current feature level is %s."), *MinimumFeatureLevelName, *FeatureLevelName));
	}

	// Merge our user specified graph properties
	Context.AddGraphProperties(Decl->GraphProperties);
}

void FExternFromInlineDecl::AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage)
{
	if (!IsStageInMask(Stage, Decl->AllowedStages))
	{
		Context.Module->AddError(nullptr, FString::Printf(TEXT("Inline HLSL instruction not valid in the %s stage."), LexToString(Stage)));
	}
}

void FExternFromInlineDecl::ToHLSL(FExternPrinterHLSL& Printer) const
{
	switch (Printer.Differential)
	{
		case EExternDifferential::None:        Printer << Decl->CodeHLSL; break;
		case EExternDifferential::ScreenSpaceX: check(Decl->CodeHLSLDDX); Printer << Decl->CodeHLSLDDX; break;
		case EExternDifferential::ScreenSpaceY: check(Decl->CodeHLSLDDY); Printer << Decl->CodeHLSLDDY; break;
	}
}

//------------------------------------------------------------------------
//                        FExternSimpleHLSL
//------------------------------------------------------------------------

static MIR::FType ToType(EExternSimpleType Id)
{
	switch (Id)
	{
		case EExternSimpleType::Bool1:			return MIR::FType::MakePrimitive(EScalarKind::Boolean, 1, 1);
		case EExternSimpleType::Bool2:			return MIR::FType::MakePrimitive(EScalarKind::Boolean, 1, 2);
		case EExternSimpleType::Bool3:			return MIR::FType::MakePrimitive(EScalarKind::Boolean, 1, 3);
		case EExternSimpleType::Bool4:			return MIR::FType::MakePrimitive(EScalarKind::Boolean, 1, 4);
		case EExternSimpleType::Int1:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 1, 1);
		case EExternSimpleType::Int2:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 1, 2);
		case EExternSimpleType::Int3:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 1, 3);
		case EExternSimpleType::Int4:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 1, 4);
		case EExternSimpleType::Int3x3:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 3, 3);
		case EExternSimpleType::Int4x4:			return MIR::FType::MakePrimitive(EScalarKind::Integer, 4, 4);
		case EExternSimpleType::Float1:			return MIR::FType::MakePrimitive(EScalarKind::Float, 1, 1);
		case EExternSimpleType::Float2:			return MIR::FType::MakePrimitive(EScalarKind::Float, 1, 2);
		case EExternSimpleType::Float3:			return MIR::FType::MakePrimitive(EScalarKind::Float, 1, 3);
		case EExternSimpleType::Float4:			return MIR::FType::MakePrimitive(EScalarKind::Float, 1, 4);
		case EExternSimpleType::Float3x3:		return MIR::FType::MakePrimitive(EScalarKind::Float, 3, 3);
		case EExternSimpleType::Float4x4:		return MIR::FType::MakePrimitive(EScalarKind::Float, 4, 4);
		case EExternSimpleType::Double1:		return MIR::FType::MakePrimitive(EScalarKind::Double, 1, 1);
		case EExternSimpleType::Double2:		return MIR::FType::MakePrimitive(EScalarKind::Double, 1, 2);
		case EExternSimpleType::Double3:		return MIR::FType::MakePrimitive(EScalarKind::Double, 1, 3);
		case EExternSimpleType::Double4:		return MIR::FType::MakePrimitive(EScalarKind::Double, 1, 4);
		case EExternSimpleType::Double3x3:		return MIR::FType::MakePrimitive(EScalarKind::Double, 3, 3);
		case EExternSimpleType::Double4x4:		return MIR::FType::MakePrimitive(EScalarKind::Double, 4, 4);
		case EExternSimpleType::SubstrateData:	return MIR::FType::MakeSubstrateData();
		default: UE_MIR_UNREACHABLE();
	}
}

MIR::FExternInfo FExternSimpleHLSL::GetInfo() const
{
	return
	{
		.Name 			= TEXT("ExternSimpleHLSL"),
		.Type 			= ToType(Type),
		.Flags 			= MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials,
	};
}

void FExternSimpleHLSL::ToHLSL(MIR::FExternPrinterHLSL& Printer) const
{
	Printer << CodeHLSL;
}

void FExternSimpleHLSL::EmitDebugInfo(FString& Out) const
{
	Out.Appendf(TEXT("Code=\"%s\""), CodeHLSL);
}

void FExternSimpleHLSL::CopyTo(FExternSimpleHLSL& Other) const
{
	Other.CodeHLSL = CodeHLSL;
	Other.Type = Type;
}

//------------------------------------------------------------------------
//                        FExternPrinterHLSL
//------------------------------------------------------------------------

FExternPrinterHLSL& FExternPrinterHLSL::operator<<(const TCHAR* String)
{
	const TCHAR* Ch = String;
	while (*Ch)
	{
		if (*Ch == TEXT('$'))
		{
			++Ch;
			int32 ArgIndex = 0;
			bool bHasDigits = false;
			while (FChar::IsDigit(*Ch))
			{
				bHasDigits = true;
				const int32 Digit = *Ch - TEXT('0');
				ArgIndex = ArgIndex * 10 + Digit;
				++Ch;
			}
			if (!bHasDigits)
			{
				UE_LOGF(LogMaterial, Error, "Dangling $ argument found in HLSL template string of MIR extern instruction during translation. Parsed string: %ls", String);
				return *this;
			}
			if (!Arguments.IsValidIndex(ArgIndex))
			{
				UE_LOGF(LogMaterial, Error, "Argument index %d out of bounds argument found in HLSL template string of MIR extern instruction during translation. Parsed string: %ls", ArgIndex, String);
				return *this;
			}
			*this << Arguments[ArgIndex];
			continue;
		}
		else if (*Ch == TEXT('<'))
		{
			if (FCString::Strncmp(Ch, TEXT("<PREV>"), 6) == 0)
			{
				if (bIsPreviousFrame)
				{
					Buffer.Append(TEXTVIEW("Prev"));
				}
				Ch += 6;
				continue;
			}
			else if (FCString::Strncmp(Ch, TEXT("<PREVFRAME>"), 11) == 0)
			{
				if (bIsPreviousFrame)
				{
					Buffer.Append(TEXTVIEW("PrevFrame"));
				}
				Ch += 11;
				continue;
			}
			else if (FCString::Strncmp(Ch, TEXT("<PREVIOUS>"), 10) == 0)
			{
				if (bIsPreviousFrame)
				{
					Buffer.Append(TEXTVIEW("Previous"));
				}
				Ch += 10;
				continue;
			}
		}
		Buffer.AppendChar(*Ch);
		++Ch;
	}
	return *this;
}

} // namespace MIR

#endif // WITH_EDITOR
