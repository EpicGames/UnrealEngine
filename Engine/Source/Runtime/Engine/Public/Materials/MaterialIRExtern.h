// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "Materials/MaterialIRCommon.h"
#include "Materials/MaterialIRTypes.h"
#include "MaterialDomain.h"

namespace MIR
{

// Selects which value to generate from an extern, either the undifferentiated original value or any screen-space partial differential.
enum class EExternDifferential
{
	None,			// Undifferentiated original value
	ScreenSpaceX,	// Screen-space partial differential along the X axis
	ScreenSpaceY,	// Screen-space partial differential along the Y axis
	MAX,
};

// Collection of state passed to an extern during IR graph analysis.
struct FExternAnalysisContext
{
	// The source material interface being translated.
	UMaterialInterface* MaterialInterface;

	// The output MIR module the material is being translated into.
	// Note: You typically use the module to gain access to the CompilationOutput instance
	//       and to set shader environment defines.
	FMaterialIRModule* Module;

	// For internal use only. Pointer to the extern instruction being analyzed.
	FExtern* Extern_Internal;

	// Adds the given graph properties to the analyzed extern instruction.
	void AddGraphProperties(MIR::EGraphProperties GraphProperties);

	// Utility function that checks whether the given Stage is within an allowed stages mask.
	// If not, it adds an error to the module, using DebugName for reference.
	bool CheckStage(EStage Stage, EStageMask AllowedStages, const TCHAR* DebugName);
};

// Set of user-provided flags used by extern instructions.
enum class EExternFlags : uint8
{
	None = 0,

	// Hint the IR-to-shader-code translator that any generated code does not need to get
	// its own temporary and can be inlined as a subexpression of another expression.
	// For instance, by turning this flag on, the generator can nest "MyFunc()" inside the + operator call as in
	//
	//     float Result = SomeOtherValue + MyFunc();
	//
	// instead of evaluating MyFunc() into its own temporary as in
	//
	//     float _14 = MyFunc();
	//     float Result = SomeOtherValue + _14;
	//
	// Note: This is only a hint for readability. Also, the translator will only inline the expression if it has
	//       only one user, meaning it is not shared computation used by more than one other instruction.
	Inline = 1 << 0,

	// This extern does not support analytical screen-space partial differential. Any attempt to evaluate its
	// differential will result in a translation error generated by the Emitter.
	NoDifferentials = 1 << 1,

	//
	ZeroDifferentials = 1 << 2,
};

ENUM_CLASS_FLAGS(EExternFlags);

// For convenience, this const array view of const MIR values is intended to represent the
// IR value arguments passed to an Extern instruction.
using FExternArgs = TConstArrayView<const FValue*>;

// Collection of data specific to an Extern instruction.
struct FExternInfo
{
	// This extern's user-friendly name, shown in debug representations of a MIR graph.
	FStringView Name = TEXTVIEW("Unknown");

	// The type of the value this extern evaluates.
	MIR::FType Type;

	// The type of the screen-space partial differential of the value this extern evaluates.
	// Note: If this is not provided (meaning it is the default Poison type), the emitter will reuse `Type`
	//       instead, unless `Flags` specifies that this extern supports no differential.
	MIR::FType DifferentialType;

	// A collection of flags that describe some of this extern's behavior.
	EExternFlags Flags{};
};

// This concept represents the static interface of a user-specified extern instruction definition.
// In traditional C++, this concept maps to the following interface:
//
// struct FMyExtern
// {
//     // Required. Returns a description of this extern. See `FExternInfo` for more info.
//     MIR::FExternInfo GetInfo() const;
//
//     // Optional. Invoked by the MIR module builder when this instruction is found to be
//     // necessary and needs to perform semantic analysis. The builder guarantees the
//     // following properties:
//     //  - This method is invoked at most once.
//     //  - This method is invoked only after all its dependencies (meaning this extern's arguments)
//     //    have been analyzed successfully.
//     //  - When this method is invoked, MIR graph properties have already been propagated
//     //    (see MIR::EGraphProperties for more info).
//     // If your extern needs to perform validation or generate side-effect data, for example
//     // by setting state on the MaterialCompilationOutput instance provided by the Module (inside `State`),
//     // or by adding new shader environment defines, this is the method to put your logic in.
//     // If you detect any invalid usage, you can push errors to the Module using the `::AddError()` method.
//     void Analyze(FExternAnalysisContext& Context);
//
//     // Optional. This method is invoked immediately after `Analyze`. The difference with `Analyze` is that
//     // this method is invoked once for each shader stage (vertex, pixel, compute) this extern instruction
//     // is proven to be evaluated in.
//     // If this method is not invoked for a particular stage, it means that the IR module builder could not
//     // find a case where this instruction is actually necessary to compute a needed value in that stage.
//     void AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage);
//
//     // Required. This method is invoked by the IR-to-HLSL translator to translate this extern instruction
//     // into HLSL.
//     // Example: `Printer << TEXTVIEW("Example(") << Printer.Arg(0) << TEXTVIEW(")")`
//     // For more information see `FExternPrinterHLSL`.
//     // Note: `Printer.Differential` specifies whether the original value or any analytical screen-space partial
//     //       differential is being generated.
//     // Important: You should not insert any validation or analysis code into this method, as it should
//     //            have all been performed in the `AnalyzeXXX` methods above. This method should contain
//     //            simple and straightforward string chunk emission logic through the provided Printer.
//     void ToHLSL(FExternPrinterHLSL& Printer) const;
//
//     // Optional. Invoked by MIR debug printers to display the parameters in your specific extern definition.
//     // In doing so, please use the "Property1=value, Property2=value" syntax for consistency with non-extern
//     // MIR instruction formatting.
//     //
//     // For instance, if your extern definition struct has `const TCHAR* ID` and a `uint32 Num` fields, you
//     // should display them as
//     //
//     //    ID="Foo", Num=42
//     //
//     void EmitDebugInfo(FString& Out) const;
// };
//
// Your extern definition must be trivially destructible and copyable. In other words, once constructed,
// it must behave as inert Plain Old Data. This requirement comes from MIR itself: all MIR nodes follow this design,
// as their bit pattern fully identifies them and their bit pattern is automatically made unique by the emitter.
// If you need non-trivial data to be created, referenced, or destroyed, use an external manager to handle
// its creation and lifetime.
//
// Note: Optional methods are not required to be defined. Not defining them is equivalent to defining them
//       and leaving their implementation empty.
//
// For reference, here is a minimal extern definition example:
//
//     enum class EColorConversionExtern
//     {
//         HSV,
//         ... /* other modes */
//     };
//
//     struct FColorConversionExtern
//     {
//         EColorConversionExtern Mode;
//
//         MIR::FExternInfo GetInfo() const
//         {
//             return
//                 {
//                     .Name = TEXT("ColorConversionExtern"),
//                     .Type = MIR::FType::MakeFloatVector(4),
//                     .Flags = MIR::EExternFlags::Inline,
//                 };
//         }
//
//         void ToHLSL(MIR::FExternPrinterHLSL& Printer) const
//         {
//             switch (Mode)
//             {
//                 case EColorConversionExtern::HSV: Printer << TEXTVIEW("RgbToHsv"); break;
//                 ... /* other modes */
//             }
//             Printer << MIR::ExternTag_DD << TEXT('(') << Printer.Arg(0) << TEXT(')'); // ex: "RgbToHsvDDX(<arg0>)"
//         }
//
//         void EmitDebugInfo(FString& Out) const
//         {
//             Out.Append(TEXTVIEW("Mode="));
//             switch (Mode)
//             {
//                 case EColorConversionExtern::HSV: Out.Append(TEXTVIEW("ToHSV")); break;
//                 ... /* other modes */
//             }
//         }
//
//         // Optional. If your definition struct does not satisfy std::has_unique_object_representations_v,
//         // you must implement CopyTo to copy each member individually into Other. The emitter writes
//         // your definition into a zero-initialized buffer via CopyTo so that padding bytes stay zero,
//         // ensuring deterministic hashing. Note: the trait is also false for structs containing floats
//         // (due to +0/-0 and NaN aliasing), even if they have no padding — C++ does not offer a trait
//         // to detect padding alone, so this is the best we can do. In that case CopyTo is still
//         // required but is a straightforward member-by-member copy.
//         void CopyTo(FColorConversionExtern& Other) const
//         {
//             Other.Mode = Mode;
//         }
//     };
//
template <typename T>
concept CExternDefinition = requires(const T& Self, FExternPrinterHLSL& Printer)
{
	requires std::is_trivially_destructible_v<T>;
	requires std::is_trivially_copy_constructible_v<T>;
	requires std::is_trivially_copy_assignable_v<T>;
	{ Self.GetInfo() } -> std::same_as<MIR::FExternInfo>;
	{ Self.ToHLSL(Printer) } -> std::same_as<void>;
};

// Detected by FEmitter::Extern to select the padding-safe copy path.
template <typename T>
concept CExternHasCopyTo = requires(const T& Self, T& Other)
{
	{ Self.CopyTo(Other) } -> std::same_as<void>;
};

// See CExternDefinition
template<typename T>
concept CExternHasAnalyze = requires(T& Self, FExternAnalysisContext& Context)
{
	{ Self.Analyze(Context) } -> std::same_as<void>;
};

// See CExternDefinition
template<typename T>
concept CExternHasAnalyzeInStage = requires(T& Self, FExternAnalysisContext& Context, EStage Stage)
{
	{ Self.AnalyzeInStage(Context, Stage) } -> std::same_as<void>;
};

// See CExternDefinition
template<typename T>
concept CExternHasEmitDebugInfo = requires(const T& Self, FString& Out)
{
	{ Self.EmitDebugInfo(Out) } -> std::same_as<void>;
};

// Built-in extern definition that uses an FMaterialExternalCodeDeclaration declaration as defined in
// the file BaseMaterialExpressions.ini.
struct FExternFromMaterialDecl
{
	// Name of the referenced external code declaration.
	FName Name;

	// Pointer to the resolved declaration with the given name, if any.
	// Note: If the given name does not map to any existing declaration, this will be null.
	const FMaterialExternalCodeDeclaration* Declaration;

	// Implementation of the CExternDefinition concept.
	FExternFromMaterialDecl(const FMaterialExternalCodeDeclaration* Declaration);
	FExternFromMaterialDecl(FName Name);
	MIR::FExternInfo GetInfo() const;
	void Analyze(FExternAnalysisContext& Context);
	void AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage);
	void ToHLSL(FExternPrinterHLSL& Printer) const;
	void EmitDebugInfo(FString& Out) const;
	void CopyTo(FExternFromMaterialDecl& Other) const;
};

// Helper function to create a material domain mask from a static list.
// Use when initializing `FExternInline::MaterialDomainMask`, example:
//     MakeDomainMask(MD_Opaque, MD_DeferredDecal);
template <typename... TMaterialDomains>
requires (std::same_as<TMaterialDomains, EMaterialDomain> && ...)
static constexpr uint32 MakeDomainMask(TMaterialDomains... Domains)
{
    return (0u | ... | (1u << static_cast<uint32>(Domains)));
}

// It wraps all the data necessary to implement an extern definition through static constant data.
// You must make sure that referenced instances of this struct are alive throughout material translation.
// It is recommended to locally declare a static const instance of this struct and use it by emitting
// a FExternFromInlineDecl. Example:
//
//    static const FExternInlineDeclaration MySpecialExtern
//        {
//            .Type               = MIR::FType::MakeFloatVector(3),
//            .CodeHLSL           = TEXT("GetMySpecialValue($0)"),
//            .MaterialDomainMask = MIR::MakeDomainMask(MD_Opaque),
//        };
//     return Em.Extern<FExternFromInlineDecl>(&MySpecialExtern);
//
struct FExternInlineDeclaration
{
	// The type this extern value evaluates to.
	MIR::FType Type{};

	// The type this extern value's screen-space partial differential evaluate to, if defined.
	// Note: You can leave this undefined. If you did not specify in `Flags` that this extern
	//       does not admit analytical partial differential, the emitter will reuse the non-differential
	//       `Type` instead.
	MIR::FType DifferentialType{};

	// The set of behavioral flags this extern adheres to.
	MIR::EExternFlags Flags = MIR::EExternFlags::Inline | MIR::EExternFlags::ZeroDifferentials;

	// The HLSL code snippet this extern evaluates to. This field is used when the printer
	// `Differential` field is `Value` (the original, non-differential value).
	// This string snippet supports various special escape patterns for convenience:
	//    - Arguments: You can use tags like "$2" as placeholders for the extern instruction
	//      argument value with index 2. For example, if this extern instruction was emitted
	//      with a single argument that happens to be a constant of value 42, the snippet
	//      `Foo(Bar, $0)` will be interpolated to `Foo(Bar, 42)`.
	//    - `<PREV>`: This tag will automatically be replaced with `Prev` while emitting code
	//                for the previous frame. This is supported since many functions in Unreal's
	//                shaders use this convention.
	const TCHAR* CodeHLSL{};

	// This works identically to `CodeHLSL`, but is used instead of it when emitting the
	// screen-space partial differential along the X/Y axis.
	// Note: If this is not provided (meaning it is left null), and `Flags` does not state that
	//       this extern does not support partial differential, this instruction will automatically
	//       generate a "zero" expression of the appropriate type.
	const TCHAR* CodeHLSLDDX{};
	const TCHAR* CodeHLSLDDY{};

	// Specifies what shader frequencies this inline extern is supported in. If this instruction
	// happens to be used in an unsupported frequency, a semantic error will be reported (during analysis).
	EStageMask AllowedStages = EStageMask::Any;

	// Specifies what material domains this extern is supported in. If this instruction happens to be
	// used in an unsupported domain, a semantic error will be reported (during analysis).
	// Note: Use `MIR::MakeDomainMask()` to conveniently generate a mask from a list of domains.
	uint32 MaterialDomainMask = 0xffffffff;
	static_assert(MD_MAX <= 32);

	// Specifies the minimum EMaterialFeatureLevel required by this inline extern instruction.
	// An error is reported if the translating material has a lower feature level.
	uint32 MinimumFeatureLevel{};

	// Array of this instruction MIR value graph properties.
	MIR::EGraphProperties GraphProperties = MIR::EGraphProperties::None;
};

// Extern definition that relies on the specification given by a FExternFromInlineDecl instance.
// Important: The declaration is only referenced to keep the Extern instruction size compact. Make sure the
//            declaration lifetime is equal or longer than the translated MIR Module.
struct FExternFromInlineDecl
{
	const FExternInlineDeclaration* Decl;

	FExternFromInlineDecl(const FExternInlineDeclaration* Decl) : Decl{ Decl } {}

	// Implementation of the CExternDefinition concept.
	MIR::FExternInfo GetInfo() const;
	void Analyze(FExternAnalysisContext& Context);
	void AnalyzeInStage(FExternAnalysisContext& Context, EStage Stage);
	void ToHLSL(FExternPrinterHLSL& Printer) const;
};

// Enumeration of types of a ExternSimpleHLSL instance. This enumeration is only 1 byte long, keeping the
// instances of FExternSimpleHLSL intentionally compact.
enum class EExternSimpleType : uint8
{
	Bool1,
	Bool2,
	Bool3,
	Bool4,

	Int1,
	Int2,
	Int3,
	Int4,
	Int3x3,
	Int4x4,

	Float1,
	Float2,
	Float3,
	Float4,
	Float3x3,
	Float4x4,

	Double1,
	Double2,
	Double3,
	Double4,
	Double3x3,
	Double4x4,

	SubstrateData,
};

// This simple extern definition covers the vast majority of use cases when you would like to implement
// our extern as a static HLSL snippet, without any domain, shader frequency, etc restriction and with
// analytical partial differential evaluating to zero. Example usage:
//
//     Em.Extern<MIR::FExternSimpleHLSL>({ MIR::EExternSimpleType::Float3, TEXT("RGBToHSV($0)") }, RGBValue);
//
struct FExternSimpleHLSL
{
	// Snippet of inlined HLSL for the non differential case.
	const TCHAR* CodeHLSL;

	// Type of the expression generated by CodeHLSL.
	EExternSimpleType Type;

	FExternSimpleHLSL(EExternSimpleType Type, const TCHAR* CodeHLSL)
	: CodeHLSL{ CodeHLSL }
	, Type{ Type }
	{}

	// Implementation of the CExternDefinition concept.
	MIR::FExternInfo GetInfo() const;
	void ToHLSL(MIR::FExternPrinterHLSL& Printer) const;
	void EmitDebugInfo(FString& Out) const;
	void CopyTo(FExternSimpleHLSL& Other) const;
};

// You can << the ExternTag_DD tag to automatically print "DDX" or "DDY" if the printer `Differential`
// is `ScreenSpaceX` or `ScreenSpaceY` respectively. If the original value is being printed, the tag prints nothing.
// This tag is useful when you want to refer to some HLSL identifier that has overloads with a "DDX" or "DDY" suffix.
enum EExternTag_DD { ExternTag_DD };

// Convenience tag to wrap a static string chunk that is only emitted when the translator is currently
// generating the extern partial differential. Otherwise it is omitted.
struct FExternTag_IfDifferential
{
	FStringView Chunk;
};

// This tag resolves to the string "Prev" when the translator is generating code for the previous frame.
enum EExternTag_Prev { ExternTag_Prev };

// A utility struct used to emit HLSL chunks.
// An instance of this struct is prepared by the IR-to-HLSL translator and passed to the individual externs that need
// lowering to their `ToHLSL()` methods.
// When lowering HLSL, make sure you check what needs to be emitted through the `Differential` field, either the original
// value or some partial differential (if supported).
struct FExternPrinterHLSL
{
	// The string buffer we are printing HLSL to.
	FString& Buffer;

	// Whether the translator requested the original value or any screen-space partial differential.
	EExternDifferential Differential;

	// What shader stage we are currently emitting for.
	MIR::EStage Stage;

	// Whether we are emitting for the vertex stage and calculating the previous-frame value.
	bool bIsPreviousFrame;

	// Array of MIR::FValue arguments that this instruction uses. Use Arg() or the "$X" tag to refer to them.
	FExternArgs Arguments;

	// Internal. Callback set by the IR-to-HLSL translator and called when a value argument needs to be lowered to HLSL,
	// and its user data. Do not alter these.
	void (*LowerValueFunction)(const FValue* Value, void* UserData) = {};
	void* LowerValueUserData = {};

	// Whether or not we are printing the regular value or its differential.
	bool IsNotDifferential() const { return Differential == EExternDifferential::None; }
	bool IsDifferential() const { return Differential != EExternDifferential::None; }

	// Returns the value argument with the specified `Index`. Index must be valid.
	const MIR::FValue* Arg(int32 Index) const
	{
		return Arguments[Index];
	}

	FExternPrinterHLSL& operator<<(TCHAR Char)
	{
		Buffer.AppendChar(Char);
		return *this;
	}

	FExternPrinterHLSL& operator<<(int32 Value)
	{
		Buffer.AppendInt(Value);
		return *this;
	}

	FExternPrinterHLSL& operator<<(uint32 Value)
	{
		Buffer.Appendf(TEXT("%u"), Value);
		return *this;
	}

	FExternPrinterHLSL& operator<<(float Value)
	{
		Buffer.Appendf(TEXT("%f"), Value);
		return *this;
	}

	FExternPrinterHLSL& operator<<(EExternTag_Prev)
	{
		if (bIsPreviousFrame)
		{
			Buffer.Append(TEXTVIEW("Prev"));
		}
		return *this;
	}

	FExternPrinterHLSL& operator<<(EExternTag_DD)
	{
		if (Differential == EExternDifferential::ScreenSpaceX)
		{
			Buffer.Append(TEXTVIEW("DDX"));
		}
		else if (Differential == EExternDifferential::ScreenSpaceY)
		{
			Buffer.Append(TEXTVIEW("DDY"));
		}
		return *this;
	}

	FExternPrinterHLSL& operator<<(FExternTag_IfDifferential Tag)
	{
		if (Differential != EExternDifferential::None)
		{
			Buffer.Append(Tag.Chunk);
		}
		return *this;
	}

	FExternPrinterHLSL& operator<<(const MIR::FValue* Value)
	{
		checkf(LowerValueFunction, TEXT("No HLSL generator was provided to this printer instance."));
		LowerValueFunction(Value, LowerValueUserData);
		return *this;
	}
	
	FExternPrinterHLSL& operator<<(FStringView String)
	{
		return operator<<(String.GetData());
	}

	FExternPrinterHLSL& operator<<(const TCHAR* String);

	template <typename... Types>
	FExternPrinterHLSL& Appendf(UE::Core::TCheckedFormatString<TCHAR, Types...> Fmt, Types... Args)
	{
		Buffer.Appendf(Fmt, Args...);
		return *this;
	}
};

} // namespace MIR

#endif // WITH_EDITOR
