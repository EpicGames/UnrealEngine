// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIR.h"
#include "MaterialShared.h"
#include "Misc/MemStack.h"

#if WITH_EDITOR

// This class represents the intermediate representation (IR) of a material build.
// The IRModule includes an IR value graph, produced through expression analysis,
// as well as metadata on resource usage and reflection. The IR graph serves as an
// abstract representation of the material and must be translated into a target backend
// such as HLSL or specific Preshader opcodes for execution.
//
// This class is designed to be backend-agnostic, meaning it does not contain any
// HLSL code nor does it configure a MaterialCompilationOutput instance. The data
// stored within this class should be sufficient to enable translation to any supported
// backend without requiring additional processing or validation.
class FMaterialIRModule
{
public:
	// Represents an error encountered during material processing.
	struct FError
	{
		// The material expression from which this error originates.
		UMaterialExpression* Expression;

		// A textual description of the error.
		FString Message;
	};

	// Stores information about the resources used by the translated material.
	struct FStatistics
	{
		// Mask to identify which TexCoord indices are actually used in the pixel/compute stages.
		int32 InterpolatedTexCoordsMask{};

		// Number of texture coordinates used by the material in the pixel/compute stages.
		int32 NumInterpolatedTexCoords{};
		
		// Total number of user vertex-interpolator float registers used by the material.
		int32 NumUsedVertexInterpolatorFloatRegisters{};

		// Dynamic particle parameter indices used by this material.
		uint32 DynamicParticleParameterMask{};
	};

	// An entry point is a sequence of instructions that evaluates a set of "output" instructions.
	// A way to think of an entry point is as the "main scope" or the "body" of a C function.
	// Each shader stage (Vertex, Pixel and Compute) gets its own entry point with one or more outputs.
	// Other entry points are crated by user specified "local functions", such as custom outputs.
	struct FEntryPoint
	{
		// This entry point's user name.
		FStringView Name;
		
		// The shader stage this entry point is intended to run in.
		MIR::EStage Stage;
		
		// The root block containing this entry point's instructions.
		MIR::FBlock RootBlock;

		// An array of terminal values this entry point evaluates.
		TArray<MIR::FInstruction*> Outputs;
	};

	// Models a group of custom outputs contained in the module.
	// Custom outputs are user-created and supplement the base outputs in the material (Normal, BaseColor, etc)
	struct FCustomOutputGroup
	{
		// Name of the custom output group, e.g. "TangentOutput".
		FStringView Name;
		
		// The number of outputs in this group.
		int32 NumOutputs;

		// The index in FMaterialIRModule::GetCustomOutputs() of the first output instruction in this group.
		// E.g. the index of "TangentOutput0"
		int32 CustomOutputInstructionsBegin;
	};

	// Specific Substrate data that needs to be handled in a specific way today when emitting back shader code from IR.
	struct FSubstrateData
	{
		FString PixelMembersDeclarationHLSL;
		FString PixelInputInitializerValuesHLSL;
		FString TreeHLSL;
	};

public:
	FMaterialIRModule();
	~FMaterialIRModule();

	// Clears the module, releasing all stored data.
	void Empty();

	// Returns the shader platform associated with this module.
	EShaderPlatform GetShaderPlatform() const { return ShaderPlatform; }

	// Returns the target platform interface associated with this module.
	const ITargetPlatform* GetTargetPlatform() const { return TargetPlatform; };

	// Returns the feature level associated with this module.
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	// Returns the quality level associated with this module.
	EMaterialQualityLevel::Type GetQualityLevel() const { return QualityLevel; }

	// Returns the blend mode associated with this module.
	EBlendMode GetBlendMode() const { return BlendMode; }

	// Retrieves the material compilation output.
	const FMaterialCompilationOutput& GetCompilationOutput() const { return CompilationOutput; }
	FMaterialCompilationOutput& GetCompilationOutput() { return CompilationOutput; }

	// Adds a new entry point with specified name, stage and optionally the expected number of evaluated output values hint.
	int32 AddEntryPoint(FStringView Name, MIR::EStage Stage, int32 NumOutputsHint = 0);

	// Returns the number of procedures inthe module.
	int32 GetNumEntryPoints() const { return EntryPoints.Num(); }

	// Gets the entry point of given Index previously added with AddEntryPoint().
	FEntryPoint& GetEntryPoint(int32 Index) { return EntryPoints[Index]; }
	const FEntryPoint& GetEntryPoint(int32 Index) const { return EntryPoints[Index]; }

	// ADds a value to the module's list of all values.
	void AddValue(MIR::FValue* Value) { Values.Add(Value); }

	// Sets the value of a material property for future reference.
	void SetPropertyValue(EMaterialProperty Property, MIR::FValue* Value) { check(Property < MP_MaterialAttributes); PropertyValues[Property] = Value; }
	
	// Records a custom output instruction.
	void AddCustomOutput(const MIR::FSetMaterialOutput* Instr) { CustomOutputs.Push(Instr); }

	// Returns the array custom output instructions.
	TConstArrayView<const MIR::FSetMaterialOutput*> GetCustomOutputs() const { return CustomOutputs; }

	//
	void BeginCustomOutputGroup(FStringView Name, int32 NumOutputs);

	// Returns the array custom output groups.
	TConstArrayView<FCustomOutputGroup> GetCustomOutputGroups() const { return CustomOutputGroups; }

	// Retrieves the value a meterial property is assigned to.
	const MIR::FValue* GetPropertyValue(EMaterialProperty Property) const { return PropertyValues[Property]; }

	// Returns a list of all environment define names this module requires to be enabled for shader compilation. @massimo.tristano refactor this
	const TSet<FName>& GetEnvironmentDefines() const { return EnvironmentDefines; }

	// Adds an environment define.
	void AddEnvironmentDefine(FName Name) { EnvironmentDefines.Add(Name); }

	// Temporary. @massimo.tristano refactor this
	TConstArrayView<TPair<FName, int32>> GetIntegerEnvironmentDefines() const { return IntegerEnvironmentDefines; }

	// Temporary. @massimo.tristano refactor this
	void AddIntegerEnvironmentDefine(FName Name, int32 Value) { IntegerEnvironmentDefines.Emplace(Name, Value); }

	// Provides access to the translated material statistics.
	const FStatistics& GetStatistics() const { return Statistics; }

	// Provides mutable access to the translated material statistics.
	FStatistics& GetStatistics() { return Statistics; }

	// Retrieves parameter info for a given parameter ID.
	const FMaterialParameterInfo& GetParameterInfo(uint32 ParameterId) const { return ParameterIdToData[ParameterId].Key; }

	// Retrieves parameter metadata for a given parameter ID.
	const FMaterialParameterMetadata& GetParameterMetadata(uint32 ParameterId) const { return ParameterIdToData[ParameterId].Value; }

	// Find existing or add a material parameter collection -- may return INDEX_NONE if too many have already been added.
	int32 FindOrAddParameterCollection(UMaterialParameterCollection* ParameterCollection);

	// Returns the list of material parameter collections used by this material.
	TConstArrayView<UMaterialParameterCollection*> GetParameterCollections() const { return ParameterCollections; }

	// Adds an HLSL function in the module.
	void AddFunctionHLSL(const MIR::FFunctionHLSL* Function);

	// Returns the array of HLSL-defined user-functions in the module.
	TConstArrayView<const MIR::FFunctionHLSL*> GetFunctionHLSLs() const { return FunctionHLSLs; };

	// Flags a shading model as used by this material.
	void AddShadingModel(EMaterialShadingModel InShadingModel);

	// Returns the shading models used by this material.
	FMaterialShadingModelField GetCompiledShadingModels() const { return this->ShadingModelsFromCompilation; }

	// Returns true if the given property has a non default value.  Similar to FHLSLMaterialTranslator::IsMaterialPropertyUsed.
	bool IsMaterialPropertyUsed(EMaterialProperty InProperty) const;

	// Stores a user-defined string and returns a view of it.
	FStringView InternString(FStringView InString);

	// Checks if the module is valid (i.e., contains no errors).
	bool IsValid() const { return Errors.IsEmpty(); }

	// Returns a list of errors encountered during processing.
	TArrayView<const FError> GetErrors() const { return Errors; }

	// Reports a translation error.
	void AddError(UMaterialExpression* Source, FString Message);
	
	// Allocates a chunk of memory using the module's internal memory block, to be used
	// by the user as they see fit.
	// Note: The returned array has the same lifetime as this module.
	void* Allocate(size_t Size, size_t Alignment) { return Allocator.Alloc(Size, Alignment); }

	// Allocates an array of elements using the module's internal memory block, to be used
	// by the user as they see fit.
	// Note: The returned array has the same lifetime as this module.
	template <typename T>
	TArrayView<T> AllocateArray(int32 Count)
	{
		static_assert(TIsTriviallyCopyAssignable<T>::Value);
		return { (T*)Allocate(sizeof(T) * Count, alignof(T)), Count };
	}

	const FSubstrateData& GetSubstrateData() const
	{
		return SubstrateData;
	}

private:
	// The chunk allocator used to contain child allocations (e.g., values, interned strings, etc)
	FMemStackBase Allocator;

	// Target shader platform.
	EShaderPlatform ShaderPlatform;

	// Target platform interface.
	const ITargetPlatform* TargetPlatform;

	// Target feature level.
	ERHIFeatureLevel::Type FeatureLevel;

	// Target quality level.
	EMaterialQualityLevel::Type QualityLevel;

	// Target blend mode -- separate from UMaterial, as it may potentially overridden in a UMaterialInstance.
	EBlendMode BlendMode;

	// Compilation output data.
	FMaterialCompilationOutput CompilationOutput;

	// Array of all the IR values contained in this module.
	TArray<MIR::FValue*> Values;

	// Array of entry points in this module.
	TArray<FEntryPoint> EntryPoints;

	// The final value assigned to each concrete material property.
	MIR::FValue* PropertyValues[MP_MaterialAttributes];

	// Array of user-generated custom outputs in this module. 
	TArray<const MIR::FSetMaterialOutput*> CustomOutputs;

	// Array of custom output groups in this module.
	TArray<FCustomOutputGroup> CustomOutputGroups;

	// Compilation statistics.
	FStatistics Statistics;

	// Maps parameter info to IDs.
	TMap<FMaterialParameterInfo, uint32> ParameterInfoToId;

	// Parameter metadata.
	TArray<TPair<FMaterialParameterInfo, FMaterialParameterMetadata>> ParameterIdToData;

	// Stores the set of user generated interned strings in this module.
	TSet<FStringView> InternedStrings;

	// Environment define names for shader compilation in this module.
	// @massimo.tristano refactor this into a better way to carry output flags from the builder to the backend.
	TSet<FName> EnvironmentDefines;
	TArray<TPair<FName, int32>> IntegerEnvironmentDefines;

	// Parameter collections referenced by this module.
	TArray<UMaterialParameterCollection*> ParameterCollections;

	// Array of HLSL-defined user-functions in the module.
	TArray<const MIR::FFunctionHLSL*> FunctionHLSLs;

	// Shading models used by this material.
	FMaterialShadingModelField ShadingModelsFromCompilation;

	// Substrate compiled data.
	FSubstrateData SubstrateData;

	// List of compilation errors.
	TArray<FError> Errors;

	// @massimo.tristano remove these friends and instead add the necessary public facing
	// non-const accessors
	friend MIR::FEmitter;
	friend FMaterialIRModuleBuilder;
	friend FMaterialIRModuleBuilderImpl;
};

#endif // #if WITH_EDITOR
