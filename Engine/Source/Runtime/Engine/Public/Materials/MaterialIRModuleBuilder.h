// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"
#include "Materials/MaterialIRCommon.h"

#if WITH_EDITOR

struct FMaterialIRModuleBuilder
{
	UMaterialInterface* MaterialInterface;
	EShaderPlatform ShaderPlatform;
	const ITargetPlatform* TargetPlatform;
	ERHIFeatureLevel::Type FeatureLevel;
	EMaterialQualityLevel::Type QualityLevel;
	EBlendMode BlendMode;
	const FStaticParameterSet* StaticParameters{};
	FMaterialInsights* TargetInsights{};
	UMaterialExpression* PreviewExpression{};

	bool Build(FMaterialIRModule* TargetModule);
};

#endif
