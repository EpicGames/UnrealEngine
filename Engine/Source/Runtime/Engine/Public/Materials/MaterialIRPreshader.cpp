// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialIRPreshader.h"
#include "Materials/MaterialIRModule.h"
#include "ShaderCompiler.h"

#if WITH_EDITOR

// Returns whether all components of the given composite are identical, and it could be aliased as a scalar.
static bool CompositeIsScalar(const MIR::FComposite* Composite)
{
	TConstArrayView<MIR::FValue*> Components = Composite->GetComponents();
	for (int32 ComponentIndex = 1; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		if (Components[ComponentIndex] != Components[0])
		{
			return false;
		}
	}
	return true;
}

// Checks if the composite is a subset of another vector, and could be aliased.  Returns index of component
// where the alias starts in the alias source vector, or INDEX_NONE if not an alias.  For example, if a
// Composite pointed to subscripts 1 and 2 of another vector (yz components), it would return "1".
// Sets OutAlias to the underlying vector.
static int32 CompositeIsAlias(MIR::FComposite* Composite, MIR::FValue*& OutAlias)
{
	TConstArrayView<MIR::FValue*> Components = Composite->GetComponents();
	int32 FirstSubscript = INDEX_NONE;
	MIR::FValue* FirstArg = nullptr;
	OutAlias = nullptr;

	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		if (MIR::FSubscript* Subscript = Components[ComponentIndex]->As<MIR::FSubscript>())
		{
			if (ComponentIndex == 0)
			{
				FirstSubscript = Subscript->Index;
				FirstArg = Subscript->Arg;
			}
			else if (Subscript->Index != FirstSubscript + ComponentIndex || Subscript->Arg != FirstArg)
			{
				return INDEX_NONE;
			}
		}
		else
		{
			return INDEX_NONE;
		}
	}

	OutAlias = FirstArg;
	return FirstSubscript;
}

// Copy a scalar constant to output location.
static void CopyConstant(const MIR::FConstant* Constant, uint32* OutData)
{
	if (Constant->Type.AsPrimitive()->ScalarKind == MIR::EScalarKind::Double)
	{
		FMemory::Memcpy(OutData, &Constant->Integer, 8);
	}
	else
	{
		FMemory::Memcpy(OutData, &Constant->Integer, 4);
	}
}

// Copy all the components of a composite or scalar value to the output array.
// Returns true if the constant has all components the same, and has been returned as a scalar.
static bool UnpackConstantValue(const MIR::FValue* Value, TArray<uint32, TFixedAllocator<8>>& OutData)
{
	int32 WordsPerComponent = Value->Type.AsPrimitive()->ScalarKind == MIR::EScalarKind::Double ? 2 : 1;

	if (const MIR::FConstant* Constant = Value->As<MIR::FConstant>())
	{
		OutData.SetNumUninitialized(WordsPerComponent);
		CopyConstant(Constant, OutData.GetData());
		return true;
	}
	else if (const MIR::FComposite* Composite = Value->As<MIR::FComposite>())
	{
		TConstArrayView<MIR::FValue*> Components = Composite->GetComponents();

		if (CompositeIsScalar(Composite))
		{
			// By design, we leave zero for child components that aren't constants (rather than say an error).
			// This is so constants can be generated for an output with some elements constant, and some requiring
			// evaluation, with the evaluation to happen in place after the constant members have been filled in.
			OutData.SetNumZeroed(WordsPerComponent);
			if ((Constant = Components[0]->As<MIR::FConstant>()))
			{
				CopyConstant(Constant, OutData.GetData());
			}
			return true;
		}
		else
		{
			OutData.SetNumZeroed(Components.Num() * WordsPerComponent);
			for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
			{
				if ((Constant = Components[ComponentIndex]->As<MIR::FConstant>()))
				{
					CopyConstant(Constant, &OutData[ComponentIndex * WordsPerComponent]);
				}
			}
			return false;
		}
	}
	else
	{
		// Return a scalar zero...
		OutData.SetNumZeroed(WordsPerComponent);
		return true;
	}
}

// Expands a scalar constant to a vector constant with NumComponents.
static void ExpandScalarConstant(int32 NumComponents, TArray<uint32, TFixedAllocator<8>>& InoutData)
{
	if (InoutData.Num() == 1)
	{
		InoutData.SetNum(NumComponents);
		for (int32 ComponentIndex = 1; ComponentIndex < NumComponents; ComponentIndex++)
		{
			InoutData[ComponentIndex] = InoutData[0];
		}
	}
	else
	{
		// Should be a double type, need to copy pairs of data elements
		check(InoutData.Num() == 2);

		InoutData.SetNum(NumComponents * 2);
		for (int32 ComponentIndex = 1; ComponentIndex < NumComponents; ComponentIndex++)
		{
			InoutData[ComponentIndex * 2 + 0] = InoutData[0];
			InoutData[ComponentIndex * 2 + 1] = InoutData[1];
		}
	}
}

void FMaterialIRPreshader::AddValueUse(const MIR::FValue* Source, const MIR::FValue* Dest)
{
	// If source isn't a preshader (meaning its a constant), we don't care about uses.  Also ignore nullptr
	// for Source, to handle passing a null BArg for a unary operator, without needing the caller to check.
	if (!Source || !Source->Analysis_PreshaderOffset)
	{
		return;
	}

	// Dest must be a preshader.
	check(Dest->Analysis_PreshaderOffset);

	// If source is an alias, we want the use to be on the underlying value.  There can be a chain of aliases
	// for the case of a composite that's a scalar alias of a subscript that's also an alias.
	while (ValueInfos[Source->Analysis_PreshaderOffset].Alias)
	{
		Source = ValueInfos[Source->Analysis_PreshaderOffset].Alias;
		check(Source->Analysis_PreshaderOffset);
	}

	FValueInfo& SourceInfo = ValueInfos[Source->Analysis_PreshaderOffset];
	FValueInfo& DestInfo = ValueInfos[Dest->Analysis_PreshaderOffset];

	// AddUnique, so multiple uses of the same source value by a single destination still only count as one use.
	DestInfo.Uses.AddUnique(Source->Analysis_PreshaderOffset);

	if (!SourceInfo.FirstUsedBy)
	{
		SourceInfo.FirstUsedBy = Dest->Analysis_PreshaderOffset;
	}
	SourceInfo.LastUsedBy = Dest->Analysis_PreshaderOffset;
	SourceInfo.UsedByCount++;
}

// Explicitly emit a primitive composite that's any combination of preshader values and constants as a full preshader value.
void FMaterialIRPreshader::EmitComposite(MIR::FComposite* Composite)
{
	int32 PreshaderIndex = AddValueInfo(Composite);

	MIR::FPrimitive PrimitiveType = *Composite->Type.AsPrimitive();
	EPreshader2Type Preshader2Type = ToPreshader2Type(PrimitiveType.ScalarKind);

	TConstArrayView<MIR::FValue*> Components = Composite->GetComponents();
	int32 FirstConstantIndex = INDEX_NONE;
	int32 LastConstantIndex = INDEX_NONE;

	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		if (Components[ComponentIndex]->IsA(MIR::VK_Constant))
		{
			if (FirstConstantIndex == INDEX_NONE)
			{
				FirstConstantIndex = ComponentIndex;
			}
			LastConstantIndex = ComponentIndex;
		}
	}

	if (FirstConstantIndex != INDEX_NONE)
	{
		TArray<uint32, TFixedAllocator<8>> ConstantData;
		bool bIsScalar = UnpackConstantValue(Composite, ConstantData);

		// The standalone constant op doesn't support scalars, unlike the binary constant ops.
		if (bIsScalar)
		{
			ExpandScalarConstant(Components.Num(), ConstantData);
		}

		if (FirstConstantIndex == 0)
		{
			UE::Preshader2::EmitConstantOp(Data, Preshader2Type, LastConstantIndex - FirstConstantIndex + 1, PreshaderIndex, &ConstantData[0]);
		}
		else
		{
			// We need to create an aliased location to write this constant to, as we are starting at an offset.  This alias
			// doesn't correspond to any value -- Value is left NULL in FValueInfo.
			int32 AliasIndex = ValueInfos.AddDefaulted();
			FMaterialIRPreshader::FValueInfo& AliasInfo = ValueInfos[AliasIndex];
			AliasInfo.Alias = Composite;
			AliasInfo.AliasOffset = FirstConstantIndex;

			UE::Preshader2::EmitConstantOp(Data, Preshader2Type, LastConstantIndex - FirstConstantIndex + 1, AliasIndex,
				&ConstantData[FirstConstantIndex * (Preshader2Type == EPreshader2Type::Double ? 2 : 1)]);
		}
	}

	// Now we can handle the moves.  Moves only support contiguous output, so potentially we need to emit multiple moves,
	// but in practice, where there is a mix of constants and moves, the moves will usually be contiguous (for example,
	// appending a one constant to XYZ to make a homogenous vector).
	TArray<uint16, TFixedAllocator<4>> SourceRegisters;
	TArray<uint8, TFixedAllocator<4>> SourceComponents;
	int32 ComponentStart = 0;

	for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
	{
		if (Components[ComponentIndex]->Analysis_PreshaderOffset)
		{
			AddValueUse(Components[ComponentIndex], Composite);

			// Track the start of our run of values we need to move
			if (SourceRegisters.IsEmpty())
			{
				ComponentStart = ComponentIndex;
			}

			FMaterialIRPreshader::FValueInfo& ComponentInfo = ValueInfos[Components[ComponentIndex]->Analysis_PreshaderOffset];
			if (ComponentInfo.Alias)
			{
				SourceRegisters.Add(ComponentInfo.Alias->Analysis_PreshaderOffset);
				SourceComponents.Add(ComponentInfo.AliasOffset);
			}
			else
			{
				SourceRegisters.Add(ComponentInfo.Value->Analysis_PreshaderOffset);
				SourceComponents.Add(0);
			}
		}

		// If this is the last element, or the current element is not a preshader (implying its a constant), emit what we have so far.
		if (ComponentIndex == Components.Num() - 1 || !Components[ComponentIndex]->Analysis_PreshaderOffset)
		{
			if (!SourceRegisters.IsEmpty())
			{
				UE::Preshader2::EmitMoveOpForFixup(Data, Preshader2Type, Composite->Analysis_PreshaderOffset, ComponentStart, SourceRegisters, SourceComponents);
				SourceRegisters.Empty();
				SourceComponents.Empty();
			}
		}
	}
}

// Emit SourceConstant to Destination.
MIR::FValue* FMaterialIRPreshader::EmitConstant(const MIR::FValue* SourceConstant, EPreshader2Type Preshader2Type, MIR::FValue* Destination)
{
	check(Destination->Analysis_PreshaderOffset && !SourceConstant->Analysis_PreshaderOffset);

	int32 NumComponents = Destination->Type.AsPrimitive()->NumComponents();

	TArray<uint32, TFixedAllocator<8>> SourceData;
	bool bIsScalar = UnpackConstantValue(SourceConstant, SourceData);

	// Constant op doesn't support scalars, so expand it to the number of output value components
	if (bIsScalar)
	{
		ExpandScalarConstant(NumComponents, SourceData);
	}

	// Emit the resulting constant data at Destination
	UE::Preshader2::EmitConstantOp(Data, Preshader2Type, NumComponents, Destination->Analysis_PreshaderOffset, &SourceData[0]);

	return Destination;
}

// Generates a new dummy constant for SourceConstant, which we can give a buffer location to, as constants
// in the main MIR tree by design don't support preshader locations.  The only current example is a constant
// to be passed to the Dot operation, where storing the constant in place in the output isn't possible, since
// the output of Dot is a smaller dimension than the input.  Since these are allocated via the MemStack
// context used by translation, they will get automatically freed when that goes out of scope.
MIR::FValue* FMaterialIRPreshader::EmitDummyConstant(const MIR::FValue* SourceConstant, EPreshader2Type Preshader2Type, int32 NumComponents)
{
	MIR::FValue* DummyConstant = (MIR::FValue*)FMemStack::Get().Alloc(sizeof(MIR::FValue), alignof(MIR::FValue));
	FMemory::Memzero(DummyConstant, sizeof(MIR::FValue));

	DummyConstant->Type = MIR::FType::MakeVector(SourceConstant->Type.AsPrimitive()->ScalarKind, NumComponents);

	int32 PreshaderIndex = AddValueInfo(DummyConstant);

	TArray<uint32, TFixedAllocator<8>> SourceData;
	bool bIsScalar = UnpackConstantValue(SourceConstant, SourceData);

	// Make sure the source constant is either a scalar (to be expanded below), or that the number of its components matches.
	// If the number of components doesn't match, it implies a bug, and we don't want to read past the end of SourceData.
	check(bIsScalar || SourceConstant->Type.AsPrimitive()->NumComponents() == NumComponents);

	// Constant op doesn't support scalars, so expand it to the number of output value components
	if (bIsScalar)
	{
		ExpandScalarConstant(NumComponents, SourceData);
	}

	// Emit the resulting constant data
	UE::Preshader2::EmitConstantOp(Data, Preshader2Type, NumComponents, PreshaderIndex, &SourceData[0]);

	return DummyConstant;
}

void FMaterialIRPreshader::EmitValue(MIR::FValue* Value)
{
	// FUniformParameter and FPreshaderParameter will already have their preshader code emitted in FMaterialIRValueAnalyzer::Analyze,
	// and will already have Analysis_PreshaderOffset set, so we can ignore those here.
	if (Value->Analysis_PreshaderOffset)
	{
		return;
	}

	if (MIR::FSubscript* Subscript = Value->As<MIR::FSubscript>())
	{
		// Subscripts are always aliases of components of the underlying value, and don't generate opcodes.
		FMaterialIRPreshader::FValueInfo& ValueInfo = AddValueInfo_GetRef(Value);
		ValueInfo.Alias = Subscript->Arg;
		ValueInfo.AliasOffset = Subscript->Index;
	}
	else if (MIR::FComposite* Composite = Value->As<MIR::FComposite>())
	{
		// Trivial swizzles (consecutive subscripts of the same underlying value) can be aliases.
		MIR::FValue* Alias;
		int32 AliasOffset = CompositeIsAlias(Composite, Alias);
		if (AliasOffset != INDEX_NONE)
		{
			FMaterialIRPreshader::FValueInfo& ValueInfo = AddValueInfo_GetRef(Value);
			ValueInfo.Alias = Alias;
			ValueInfo.AliasOffset = AliasOffset;
		}
		else
		{
			// Composites that are a copy of the same value can be aliases.
			if (CompositeIsScalar(Composite))
			{
				// In this case, AliasOffset is left as INDEX_NONE, indicating this is a Scalar alias
				FMaterialIRPreshader::FValueInfo& ValueInfo = AddValueInfo_GetRef(Value);
				ValueInfo.Alias = Composite->GetComponents()[0];
			}
			else
			{
				// Non-trivial case, will be instantiated on demand later.  A lot of composites may be dead stripped
				// if they are trivial tail swizzles.
			}
		}
	}
	else if (MIR::FOperator* Operator = Value->As<MIR::FOperator>())
	{
		check(Operator->AArg);

		// Check if any composite children of this operator need to be expanded.  We defer this so we don't generate opcodes
		// if it's not actually used (common for trivial tail swizzles that have been dead stripped).
		if (!Operator->AArg->Analysis_PreshaderOffset && Operator->AArg->HasSubgraphProperties(MIR::EGraphProperties::HasParameter) && Operator->AArg->IsA(MIR::VK_Composite))
		{
			EmitComposite(Operator->AArg->As<MIR::FComposite>());
		}
		if (Operator->BArg && !Operator->BArg->Analysis_PreshaderOffset && Operator->BArg->HasSubgraphProperties(MIR::EGraphProperties::HasParameter) && Operator->BArg->IsA(MIR::VK_Composite))
		{
			EmitComposite(Operator->BArg->As<MIR::FComposite>());
		}

		// For Dot operations with a constant BArg, pre-emit the dummy constant BEFORE registering the operator itself.  
		// EmitDummyConstant adds to ValueInfos, so it must come first to preserve topological ordering: every source must have a lower preshader index than its consumers.  
		MIR::FValue* PreEmittedDotDummyBArg = nullptr;
		if (Operator->Op == MIR::BO_Dot && Operator->BArg)
		{
			// Figure out which arg is the preshader and which is the constant, accounting for the commutative swap that will happen later.
			MIR::FValue* DotPreshaderArg = nullptr;
			MIR::FValue* DotConstantArg  = nullptr;
			if (Operator->AArg->Analysis_PreshaderOffset && !Operator->BArg->Analysis_PreshaderOffset)
			{
				DotPreshaderArg = Operator->AArg;
				DotConstantArg  = Operator->BArg;
			}
			else if (!Operator->AArg->Analysis_PreshaderOffset && Operator->BArg->Analysis_PreshaderOffset)
			{
				// AArg is constant and will be swapped to BArg position below.
				DotPreshaderArg = Operator->BArg;
				DotConstantArg  = Operator->AArg;
			}
			if (DotConstantArg)
			{
				EPreshader2Type DotPreshaderType = ToPreshader2Type(Operator->Type.AsPrimitive()->ScalarKind);
				PreEmittedDotDummyBArg = EmitDummyConstant(DotConstantArg, DotPreshaderType, DotPreshaderArg->Type.AsPrimitive()->NumComponents());
			}
		}

		// Generate preshader operator
		int32 PreshaderIndex = AddValueInfo(Value);
		FMaterialIRPreshader::FValueInfo& ValueInfo = ValueInfos[PreshaderIndex];

		MIR::FPrimitive PrimitiveType = *Operator->Type.AsPrimitive();
		EPreshader2Type Preshader2Type = ToPreshader2Type(PrimitiveType.ScalarKind);

		AddValueUse(Operator->AArg, Value);
		AddValueUse(Operator->BArg, Value);

		// These two ops have scalar outputs for vector inputs -- replace PrimitiveType with the first arg's type
		if (Operator->Op == MIR::UO_Length || Operator->Op == MIR::BO_Dot)
		{
			PrimitiveType = *Operator->AArg->Type.AsPrimitive();
		}

		bool bIsCommutative;
		EPreshader2Opcode Opcode = ToPreshader2Opcode(Operator->Op, bIsCommutative);
		check(Opcode != EPreshader2Opcode::Invalid);

		if (MIR::IsUnaryOperator(Operator->Op))
		{
			// MIR shouldn't generate unary ops on constants, as those should be optimized out.
			check(Operator->AArg && Operator->AArg->Analysis_PreshaderOffset);

			if (ValueIsScalarAlias(Operator->AArg))
			{
				// Preshader2 unary ops don't support promoting scalars to vectors, so if this is a scalar alias, we need to expand it
				// to the destination before applying the op to it in place.  For a 3 element vector, this adds 5 bytes (2 offsets
				// plus an extra opcode), as the in-place second op only takes 1 byte.  We could eventually have MIR optimize these
				// out at emit time if we wanted to (emit the unary op on the underlying scalar, then expand it as a composite).
				TArray<uint16, TFixedAllocator<4>> SourceRegisters;
				TArray<uint8, TFixedAllocator<4>> SourceComponents;

				uint16 SourceRegister = (uint16)Operator->AArg->Analysis_PreshaderOffset;
				for (int32 ComponentIndex = 0; ComponentIndex < PrimitiveType.NumComponents(); ComponentIndex++)
				{
					SourceRegisters.Add(SourceRegister);
					SourceComponents.Add(0);
				}

				UE::Preshader2::EmitMoveOpForFixup(Data, Preshader2Type, (uint16)PreshaderIndex, 0, SourceRegisters, SourceComponents);
				UE::Preshader2::EmitUnaryOp(Data, Opcode, Preshader2Type,
					(uint8)PrimitiveType.NumComponents(),
					(uint16)PreshaderIndex,
					(uint16)PreshaderIndex);
			}
			else
			{
				UE::Preshader2::EmitUnaryOp(Data, Opcode, Preshader2Type,
					(uint8)PrimitiveType.NumComponents(),
					(uint16)PreshaderIndex,
					(uint16)Operator->AArg->Analysis_PreshaderOffset);
			}
		}
		else if (MIR::IsBinaryOperator(Operator->Op))
		{
			MIR::FValue* AArg = Operator->AArg;
			MIR::FValue* BArg = Operator->BArg;

			// MIR shouldn't generate binary ops between two constants, as those should be optimized out.
			check(AArg && BArg && (AArg->Analysis_PreshaderOffset || BArg->Analysis_PreshaderOffset));

			// If first argument is a constant (not emitted to the preshader), attempt to swap it to the second argument,
			// so we can use an inline constant version of the opcode.
			if (!AArg->Analysis_PreshaderOffset && bIsCommutative)
			{
				Swap(AArg, BArg);
			}

			// If that failed (first arg still doesn't have a preshader offset), we need to emit two ops -- one to generate
			// the constant at the destination location, and a second for the op.  AArg is updated to the destination where
			// the constant will be stored.
			if (!AArg->Analysis_PreshaderOffset)
			{
				AArg = EmitConstant(AArg, Preshader2Type, Value);
			}

			// Cross and Dot are limited binary operators, which don't support immediate constants.  We need to emit a
			// constant for BArg, similar to AArg above, if that's a constant.  Note that we don't have to worry
			// about both being constant, as MIR would have optimized that out.
			if ((Operator->Op == MIR::BO_Cross || Operator->Op == MIR::BO_Dot))
			{
				if (!BArg->Analysis_PreshaderOffset)
				{
					if (Operator->Op == MIR::BO_Dot)
					{
						// The dummy constant should have been pre-emitted before this operator to preserve topological ordering in ValueInfos.  
						// Use the already-allocated value rather than emitting a new one.
						check(PreEmittedDotDummyBArg != nullptr);
						BArg = PreEmittedDotDummyBArg;

						// Allow the temporary memory for the dummy constant to be reclaimed by tracking its usage.
						AddValueUse(BArg, Value);
					}
					else
					{
						// Write constant to destination, where it will be overwritten in place (no need to allocate separate space)
						BArg = EmitConstant(BArg, Preshader2Type, Value);
					}
				}
			}

			if (BArg->Analysis_PreshaderOffset)
			{
				UE::Preshader2::EmitBinaryOp(Data, Opcode, Preshader2Type,
					(uint8)PrimitiveType.NumComponents(),
					(uint16)PreshaderIndex,
					(uint16)AArg->Analysis_PreshaderOffset,
					(uint16)BArg->Analysis_PreshaderOffset,
					ValueIsScalarAlias(AArg),
					ValueIsScalarAlias(BArg));
			}
			else
			{
				TArray<uint32, TFixedAllocator<8>> BData;
				bool bBIsScalar = UnpackConstantValue(BArg, BData);

				UE::Preshader2::EmitBinaryConstOp(Data, Opcode, Preshader2Type,
					(uint8)PrimitiveType.NumComponents(),
					(uint16)PreshaderIndex,
					(uint16)AArg->Analysis_PreshaderOffset,
					BData.GetData(),
					ValueIsScalarAlias(AArg),
					bBIsScalar);
			}
		}
		else
		{
			// TODO:  handle clamp, as min/max?
			UE_MIR_UNREACHABLE();
		}
	}
	// FUniformParameter and FPreshaderParameter will already have an analysis offset defined.  Everything else shouldn't
	// be a preshader, and shouldn't reach this code path.
	else if (!Value->Analysis_PreshaderOffset)
	{
		UE_MIR_UNREACHABLE();
	}
}

// Class to handle allocation of preshader vector components.  Supports efficient search for the best fit available
// location of allocations of a given component size, using a table of bit arrays to track free space by available
// component mask.
struct FComponentAllocator
{
	// Allocate float4 buffer space
	int32 AllocateFromBuffer(int32 NumOfFloat4s)
	{
		int32 Result = AllocatedFloat4s;
		AllocatedFloat4s += NumOfFloat4s;
		check(AllocatedFloat4s <= UINT16_MAX);
		return Result;
	}

	// Accessors for mask bit arrays.  Bit arrays aren't guaranteed to be sized to include the current maximum
	// number of elements -- it would be wasteful to grow all 16 mask bit arrays every time an element is added.
	// So we need to treat out of bounds bit accesses as false on read, and add storage on write as needed.
	// Sizing should be very cheap since it's a bit array with some inline storage.
	void SetIsAvailable(uint8 Mask, int32 Index)
	{
		// Ignore zero masks, these have nothing available, and we don't need overhead writing to AvailableByMask.
		if (Mask)
		{
			if (Index >= AvailableByMask[Mask].Num())
			{
				AvailableByMask[Mask].SetNum(Index + 1, false);
			}
			AvailableByMask[Mask][Index] = true;
		}

		// We still need to track which components are available at the given vector Index.
		if (Index >= AvailableComponents.Num())
		{
			AvailableComponents.SetNumZeroed(Index + 1);
		}
		AvailableComponents[Index] = Mask;
	}

	// Gets the item at the given index if available, setting the fetched item as no longer available.
	bool GetIfAvailable(uint8 Mask, int32 Index)
	{
		if (Index < AvailableByMask[Mask].Num() && AvailableByMask[Mask][Index])
		{
			AvailableByMask[Mask][Index] = false;
			return true;
		}
		else
		{
			return false;
		}
	}

	int32 FirstAvailable(uint8 Mask)
	{
		return AvailableByMask[Mask].Find(true);
	}

	int32 NextAvailable(uint8 Mask, int32 Index)
	{
		if (Index + 1 < AvailableByMask[Mask].Num())
		{
			return AvailableByMask[Mask].FindFrom(true, Index + 1);
		}
		return INDEX_NONE;
	}

	// Given a table of mask/offset pairs that can fit the desired element size, searches for the best available
	// fit in AvailableByMask, or allocates one if that fails.
	int32 FindOrAllocateBestFit(const uint8* MaskFitTable, int32 NumWords)
	{
		int32 VectorIndex = INDEX_NONE;
		for (; MaskFitTable[0]; MaskFitTable += 2)
		{
			uint8 Mask = MaskFitTable[0];

			VectorIndex = FirstAvailable(Mask);
			if (VectorIndex != INDEX_NONE)
			{
				// Remove the item we are using from its bitvector
				AvailableByMask[Mask][VectorIndex] = false;
				break;
			}
		}

		if (VectorIndex == INDEX_NONE)
		{
			// Back up to last element in table, which is the full vector mask element.
			MaskFitTable -= 2;

			VectorIndex = AllocateFromBuffer(1);
		}

		// Remove from mask elements we are occupying.
		uint8 Mask = MaskFitTable[0];
		uint8 Offset = MaskFitTable[1];
		uint8 MaskToRemove = ((1 << NumWords) - 1) << Offset;

		// Make sure there's not a bug in our lookup table
		check((Mask & MaskToRemove) == MaskToRemove);
		Mask &= ~MaskToRemove;

		// Add the remainder of the vector to an available list.
		SetIsAvailable(Mask, VectorIndex);

		// Add the component offset to the vector's offset, to get the overall global component offset.
		return VectorIndex * 4 + Offset;
	}

	// Returns the best available offset into the preshader buffer for a float vector with the specified number of components (1-4).
	uint32 BestComponentOffset(const MIR::FPrimitive& PrimitiveType)
	{
		const uint32 NumComponents = PrimitiveType.NumComponents();
		check(NumComponents >= 1 && NumComponents <= 4);

		// Following code calculates GlobalComponentOffset.
		// GlobalComponentOffset is the i-th component in the array of float4s that make the uniform buffer.
		// For example a GlobalComponentOffset of 13 references PreshaderBuffer[3].y.
		// Attempts to find a best fit of available components from previous allocations or freed elements,
		// in order to reduce the number of allocations and thus preshader buffer memory footprint.

		if (PrimitiveType.IsDouble())
		{
			// Double values must be 8-byte aligned, so only search for offsets at the correct alignment in the word.
			static const uint8 MaskDouble1[] = {
				0b0011,0,  0b1100,2,							// Single element free, filling exact space
				0b1011,0,  0b0111,0,  0b1101,2,  0b1110,2,		// Single element free, some remaining space
				0b1111,0,  0									// Full vector4 free, plus terminator
			};
			static const uint8 MaskDouble2[] = {
				0b1111,0,  0				// Full vector4 free, plus terminator
			};

			if (NumComponents == 1)
			{
				return FindOrAllocateBestFit(MaskDouble1, NumComponents*2);
			}
			else if (NumComponents == 2)
			{
				// For 2 component double, we take up a whole word.
				return FindOrAllocateBestFit(MaskDouble2, NumComponents*2);
			}
			else
			{
				// Beyond that, we need to take up more than a float4, which need to be consecutive.
				// So this needs custom allocation logic.  Scan each available full vector, then check
				// the vector following it to see if the rest will fit.
				int32 FullVector;
				uint8 Mask = 0;

				for (FullVector = FirstAvailable(0b1111); FullVector != INDEX_NONE; FullVector = NextAvailable(0b1111, FullVector))
				{
					int32 NextVector = FullVector + 1;
					if (NextVector == AllocatedFloat4s)
					{
						// The current full vector is the last allocated element.  Allocate a new one.
						AllocateFromBuffer(1);
						Mask = 0b1111;
						break;
					}

					// For 3 elements, we can check if the next vector has its first half available
					if (NumComponents == 3)
					{
						if (GetIfAvailable(0b0011, NextVector))
						{
							Mask = 0b0011;
							break;
						}
						if (GetIfAvailable(0b0111, NextVector))
						{
							Mask = 0b0111;
							break;
						}
						if (GetIfAvailable(0b1011, NextVector))
						{
							Mask = 0b1011;
							break;
						}
					}

					// Check if the full next vector is available
					if (GetIfAvailable(0b1111, NextVector))
					{
						Mask = 0b1111;
						break;
					}
				}

				if (FullVector != INDEX_NONE)
				{
					// Mark the full vector we are using as unavailable.  Note that the next vector will already
					// have been marked as unavailable in the calls to "GetIfAvailable" above.
					AvailableByMask[0b1111][FullVector] = false;
				}
				else
				{
					// If we didn't find anything, allocate two fresh vectors
					FullVector = AllocateFromBuffer(2);
					Mask = 0b1111;
				}

				// Mark availability mask of FullVector and the following vector.
				SetIsAvailable(0, FullVector);

				if (NumComponents == 3)
				{
					SetIsAvailable(Mask & ~0b0011, FullVector + 1);
				}
				else
				{
					SetIsAvailable(0, FullVector + 1);
				}

				// And finally return the allocated offset!
				return FullVector * 4;
			}
		}
		else
		{
			// Tables storing pairs of a best fit mask, and where to put the relevant element if that mask is chosen.
			// Lookup tables are simpler than writing complex logic to iterate over possibilities in the ideal order.
			static const uint8 MaskFloat1[] = {
				0b0001,0,  0b0010,1,  0b0100,2,  0b1000,3,					// Single element free
				0b0101,0,  0b1001,0,  0b1010,1,  0b1011,3,  0b1101,0,		// Single element holes
				0b0011,0,  0b0110,1,  0b0111,0,  0b1100,2,  0b1110,1,		// Miscellaneous
				0b1111,3,  0												// Full vector4 free, plus terminator
			};
			static const uint8 MaskFloat2[] = {
				0b0011,0,  0b0110,1,  0b1100,2,								// Pair of elements free
				0b1011,0,  0b1101,2,										// Pair of elements hole
				0b0111,0,  0b1110,1,										// Miscellaneous
				0b1111,0,  0												// Full vector4 free, plus terminator
			};
			static const uint8 MaskFloat3[] = {
				0b0111,0,  0b1110,1,										// Three elements free
				0b1111,0,  0,												// Full vector4 free, plus terminator
			};
			static const uint8 MaskFloat4[] = {
				0b1111,0,  0,												// Full vector4 free, plus terminator
			};

			static const uint8* MasksFloat[4] = { MaskFloat1, MaskFloat2, MaskFloat3, MaskFloat4 };

			return FindOrAllocateBestFit(MasksFloat[NumComponents - 1], NumComponents);
		}
	}

	void FreeComponentOffset(int32 Offset, int32 NumWords)
	{
		int32 VectorIndex = Offset / 4;
		if (NumWords >= 4)
		{
			// Freeing at least a whole word -- it should be vector aligned, and we know it's not in an available list.
			check((Offset & 3) == 0);
			SetIsAvailable(0b1111, VectorIndex);

			if (NumWords > 4)
			{
				// Double type that spans words -- NumWords should be even, and thus 6 or 8
				check(NumWords == 6 || NumWords == 8);
				if (NumWords == 6)
				{
					// Remove from its existing available list if necessary.
					uint8 ExistingMask = AvailableComponents[VectorIndex + 1];
					if (ExistingMask)
					{
						AvailableByMask[ExistingMask][VectorIndex + 1] = false;
					}

					// Mark the new availability
					check(!(ExistingMask & 0b0011));
					SetIsAvailable(ExistingMask | 0b0011, VectorIndex + 1);
				}
				else
				{
					// Another whole word -- we know it's not in an available list.
					SetIsAvailable(0b1111, VectorIndex + 1);
				}
			}
		}
		else
		{
			// Remove from its existing available list if necessary
			uint8 ExistingMask = AvailableComponents[VectorIndex];
			if (ExistingMask)
			{
				AvailableByMask[ExistingMask][VectorIndex] = false;
			}

			// Mark the new availability
			uint8 FreeingMask = ((1 << NumWords) - 1) << (Offset % 4);
			check(!(ExistingMask & FreeingMask));
			SetIsAvailable(ExistingMask | FreeingMask, VectorIndex);
		}
	}

	void Reset()
	{
		for (auto& AvailableItem : AvailableByMask)
		{
			AvailableItem.Empty();
		}
		AvailableComponents.Empty();
		AllocatedFloat4s = 0;
	}

	// Encodes available space in vectors by mask of which components are currently used.  The goal is to
	// allow efficient search for vectors with sufficient space to hold a new allocation, with minimal
	// space and dynamic allocation requirements.
	TStaticArray<TBitArray<TInlineAllocator<1>>, 16> AvailableByMask;

	// By vector offset, mask of which components are available -- used when freeing space to find which
	// AvailableByMask element to remove the item from.
	TArray<uint8> AvailableComponents;

	// Number of float4s allocated so far
	int32 AllocatedFloat4s = 0;
};

// Gets the primitive type for a preshader value, taking into account the uniform component counts override table.
static MIR::FPrimitive GetPrimitive(const FMaterialIRPreshader* Preshader, const TArray<uint8>& UniformComponentCounts, int32 PreshaderIndex)
{
	MIR::FPrimitive Primitive = Preshader->ValueInfos[PreshaderIndex].Value->Type.GetPrimitive();

	// Component count may be overridden for uniform parameters based on what's actually used -- generate a
	if (UniformComponentCounts[PreshaderIndex] && UniformComponentCounts[PreshaderIndex] != Primitive.NumComponents())
	{
		return MIR::FType::MakeVector(Primitive.ScalarKind, UniformComponentCounts[PreshaderIndex]).GetPrimitive();
	}
	else
	{
		return Primitive;
	}
}

void FMaterialIRPreshader::PreshaderFixup(UE::Shader::FPreshaderData& PreshaderOut, FMaterialIRModule* Module, FMaterialInsights* Insights)
{
	// One is added to Remap offsets during processing, so zero represents uninitialized, then subtracted after processing is finished.
	TArray<uint16> Remap;
	Remap.SetNumZeroed(ValueInfos.Num());

	// For values that are uniform parameters, tracks component count overrides based on how many components of the parameter are actually used.
	TArray<uint8> UniformComponentCounts;
	UniformComponentCounts.SetNumZeroed(ValueInfos.Num());

	// Generate UniformComponentCounts (based on used components) for all Parameters.  Needed for fixup of component
	// counts in FUniformExpressionSet::FixupNumericParameterEvaluations.  Zero indicates it's not a uniform component.
	// We always skip the first item, which is an unused dummy.
	for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
	{
		// Value can be null for constant aliases, which should be ignored here.
		MIR::FValue* Value = ValueInfos[PreshaderIndex].Value;
		if (!Value)
		{
			continue;
		}

		if (MIR::FPrimitiveUniform* Parameter = Value->As<MIR::FPrimitiveUniform>())
		{
			if (Parameter->Type.IsVector())
			{
				UniformComponentCounts[PreshaderIndex] = (uint8)Parameter->Analysis_NumComponents();
			}
		}
	}

	// Identify cases of output location propagation.  See comments in header for broader explanation of this optimization.
	// Scan the list of values in reverse, and identify cases where one of the inputs to a value can reuse the output
	// location of another value.
	for (int32 PreshaderIndex = ValueInfos.Num() - 1; PreshaderIndex >= 1; PreshaderIndex--)
	{
		// Value can be null for constant aliases, which should be ignored here.
		MIR::FValue* Value = ValueInfos[PreshaderIndex].Value;
		if (!Value)
		{
			continue;
		}

		MIR::FPrimitive ValuePrimitive = GetPrimitive(this, UniformComponentCounts, PreshaderIndex);

		for (uint16 UseIndex : ValueInfos[PreshaderIndex].Uses)
		{
			// We can reuse the output location if this is the last use, and types match.
			if (ValueInfos[UseIndex].LastUsedBy == PreshaderIndex)
			{
				if (ValuePrimitive == GetPrimitive(this, UniformComponentCounts, UseIndex))
				{
					// OutputLocation can be chained across multiple ops, so check if the destination already has an OutputLocation, and propagate that.
					uint16 DestOutputLocation = ValueInfos[PreshaderIndex].OutputLocation;
					ValueInfos[UseIndex].OutputLocation = DestOutputLocation ? DestOutputLocation : PreshaderIndex;
					break;
				}
			}
		}
	}

	FComponentAllocator OutAllocator;
	FComponentAllocator TempAllocator;

	// Helper to allocate a single PreshaderOut value in the OutAllocator.
	auto AllocatePreshaderOut = [&](int32 PreshaderIndex)
	{
		FMaterialIRPreshader::FValueInfo& ValueInfo = ValueInfos[PreshaderIndex];

		// Value can be null for constant aliases, which should be ignored here.
		MIR::FValue* Value = ValueInfo.Value;
		if (!Value)
		{
			return;
		}

		if (Value->HasSubgraphProperties(MIR::EGraphProperties::PreshaderOut) && !ValueInfo.Alias)
		{
			MIR::FPrimitive PrimitiveType = GetPrimitive(this, UniformComponentCounts, PreshaderIndex);

			uint32 GlobalComponentOffset = OutAllocator.BestComponentOffset(PrimitiveType);
			check(GlobalComponentOffset < UINT16_MAX);

			Remap[PreshaderIndex] = (uint16)GlobalComponentOffset + 1;		// Add one!
		}
	};

	// Allocate all PreshaderOut values first, so we know the starting offset for PreshaderTemp.
	// We don't have to worry about lifetime tracking for these, since they are permanent.
	// When UseMaterialVSUniformBuffer() is enabled, VS-frequency outputs must be allocated first
	// so they occupy a contiguous prefix of the PreshaderBuffer that the compact MaterialVS UB can reference.
	if (UseMaterialVSUniformBuffer())
	{
		// First pass: allocate VS-frequency PreshaderOut values at the beginning of the buffer.
		for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
		{
			MIR::FValue* Value = ValueInfos[PreshaderIndex].Value;
			if (Value && Value->IsAnalyzed(MIR::Stage_Vertex))
			{
				AllocatePreshaderOut(PreshaderIndex);
			}
		}
		VSPreshaderFloat4s = OutAllocator.AllocatedFloat4s;

		// Second pass: allocate non-VS PreshaderOut values after the VS prefix.
		for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
		{
			MIR::FValue* Value = ValueInfos[PreshaderIndex].Value;
			if (Value && !Value->IsAnalyzed(MIR::Stage_Vertex))
			{
				AllocatePreshaderOut(PreshaderIndex);
			}
		}
	}
	else
	{
		for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
		{
			AllocatePreshaderOut(PreshaderIndex);
		}
	}

	Module->GetCompilationOutput().UniformExpressionSet.AllocateFromUniformBuffer(OutAllocator.AllocatedFloat4s);

	uint32 UniformBufferWords = OutAllocator.AllocatedFloat4s * 4;
	check(UniformBufferWords <= UINT16_MAX);

	// Then allocate space for all uniform parameters -- these are loaded in a batch before bytecode execution, so
	// all need to start out in the temporary buffer, or in an OutputLocation already allocated as a PreshaderOut.
	// The assumption is that most preshader expressions will fall under that category, but as a potential future
	// optimization, we could allow space in the output buffer to be reused as input or temporary storage before it
	// finally gets written to as an output.  It adds complexity to the allocator for only a savings in temporary
	// space, though.
	//
	// In a large project, some categories of preshader in the original system, and their efficiency in Preshader2:
	//		45.0% parameter, with no other opcodes   -- no bytecode,   param to PreshaderOut
	//		25.6% parameter, tail swizzle            -- no bytecode,   param to PreshaderOut
	//		 4.6% parameter, unary op                -- 1 op,          param to PreshaderOut via OutputLocation
	//		 4.3% parameter, binary op with const    -- 1 op,          param to PreshaderOut via OutputLocation
	//		 3.2% parameter, 2 unary ops             -- 2 ops,         param to PreshaderOut via OutputLocation
	//		 0.9% runtime virtual texture            -- 1 op,          op to PreshaderOut
	//
	// That's 83.6% of preshaders that trivially don't require temporary space to evaluate, and many more won't.
	// Probably around 10% do, so you're looking at around 10% savings in temporary buffer space, perhaps around
	// 100 bytes for a complex shader.  The original preshader system stored temporaries on the stack, as
	// 4-element doubles, so an old preshader with just a binary op would already use 64 bytes of temporary storage.
	//
	for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
	{
		FMaterialIRPreshader::FValueInfo& ValueInfo = ValueInfos[PreshaderIndex];

		// If this already has a Remap location, it's a PreshaderOut, and can be skipped, as well as null Values.
		if (Remap[PreshaderIndex] || !ValueInfo.Value)
		{
			continue;
		}

		if (MIR::FPrimitiveUniform* Parameter = ValueInfo.Value->As<MIR::FPrimitiveUniform>())
		{
			if (ValueInfo.OutputLocation && Remap[ValueInfo.OutputLocation])
			{
				// Already allocated as a PreshaderOut OutputLocation!
				Remap[PreshaderIndex] = Remap[ValueInfo.OutputLocation];
			}
			else
			{
				// Allocate a temporary location.
				MIR::FPrimitive PrimitiveType = GetPrimitive(this, UniformComponentCounts, PreshaderIndex);
				int32 GlobalComponentOffset = TempAllocator.BestComponentOffset(PrimitiveType) + UniformBufferWords;
				check(GlobalComponentOffset < UINT16_MAX);

				Remap[PreshaderIndex] = (uint16)GlobalComponentOffset + 1;		// Add one!

				// Propagate the temporary location to OutputLocation, if present, so other intermediates
				// using the OutputLocation later can see it.
				if (ValueInfo.OutputLocation)
				{
					Remap[ValueInfo.OutputLocation] = Remap[PreshaderIndex];
				}
			}
		}
	}

	// Then allocate all remaining intermediate values and assign locations to aliases
	for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
	{
		FMaterialIRPreshader::FValueInfo& ValueInfo = ValueInfos[PreshaderIndex];

		// Mark temporary items as free that have encountered their last use, if allocated from the temporary buffer.
		// We do this before emitting the current item, so the current item can potentially use that space immediately.
		// Aliases are never marked used, so we can ignore those.
		if (!ValueInfo.Alias)
		{
			for (uint16 UseIndex : ValueInfos[PreshaderIndex].Uses)
			{
				// Ignore uses with an OutputLocation, as they won't become free until the last value in a chain,
				// which won't have OutputLocation set.
				if (ValueInfos[UseIndex].OutputLocation)
				{
					continue;
				}

				// All ValueInfos must appear topologically after values they use in the array, and so the used values should already have had their Remap set
				// in a previous loop iteration. Hitting this check implies ValueInfos were added out of dependency order in prior code.
				check(Remap[UseIndex]);

				// Temporary buffer starts at offset "UniformBufferWords".
				if ((ValueInfos[UseIndex].LastUsedBy == PreshaderIndex) && ((uint16)(Remap[UseIndex] - 1) >= UniformBufferWords))		// Subtract one!
				{
					MIR::FPrimitive UsePrimitiveType = GetPrimitive(this, UniformComponentCounts, UseIndex);
					int32 WordsPerComponent = UsePrimitiveType.IsDouble() ? 2 : 1;
					int32 NumComponents = UsePrimitiveType.NumComponents();

					TempAllocator.FreeComponentOffset(Remap[UseIndex] - 1 - UniformBufferWords, NumComponents * WordsPerComponent);		// Subtract one!
				}
			}
		}

		if (!Remap[PreshaderIndex])
		{
			// Check if there's an output location with a remap already set -- if so, pull remap from that.  The remap
			// might already exist if it's a PreshaderOut that's already been allocated above, or it's used in a chain,
			// and the first item using the OutputLocation in the chain already allocated it.
			if (ValueInfo.OutputLocation && Remap[ValueInfo.OutputLocation])
			{
				Remap[PreshaderIndex] = Remap[ValueInfo.OutputLocation];
			}
			else
			{
				int32 GlobalComponentOffset = 0;
				if (ValueInfo.Alias)
				{
					// Aliases should always appear after their dependency, and derive their offset from that.
					check(ValueInfo.Alias->Analysis_PreshaderOffset < PreshaderIndex);
					GlobalComponentOffset = Remap[ValueInfo.Alias->Analysis_PreshaderOffset] - 1 + FMath::Max(ValueInfo.AliasOffset, 0);		// Subtract one!
				}
				else
				{
					// Allocate a new offset.
					MIR::FPrimitive PrimitiveType = GetPrimitive(this, UniformComponentCounts, PreshaderIndex);
					GlobalComponentOffset = TempAllocator.BestComponentOffset(PrimitiveType) + UniformBufferWords;
				}
				check(GlobalComponentOffset < UINT16_MAX);

				Remap[PreshaderIndex] = (uint16)GlobalComponentOffset + 1;		// Add one!

				// If there's an OutputLocation corresponding to the value we just allocated, mirror the remapping
				// to the OutputLocation, so it can be picked up by all values sharing that OutputLocation.
				if (ValueInfo.OutputLocation)
				{
					Remap[ValueInfo.OutputLocation] = Remap[PreshaderIndex];
				}
			}
		}

		// Now that we know the buffer location, we can write parameter Insights.
		if (ValueInfo.Value)
		{
			MIR::FPrimitiveUniform* PrimitiveUniform = ValueInfo.Value->As<MIR::FPrimitiveUniform>();
			if (PrimitiveUniform && Insights)
			{
				// Push information about this parameter allocation to the insights.
				int32 GlobalComponentOffset = Remap[PreshaderIndex] - 1;		// Subtract one!
				MIR::FPrimitive PrimitiveType = GetPrimitive(this, UniformComponentCounts, PreshaderIndex);

				FMaterialInsights::FUniformParameterAllocationInsight ParamInsight;
				ParamInsight.BufferSlotIndex = GlobalComponentOffset / 4;
				ParamInsight.BufferSlotOffset = GlobalComponentOffset % 4;
				ParamInsight.ComponentsCount = PrimitiveType.NumComponents();
				ParamInsight.ParameterName = FName(PrimitiveUniform->Name);

				switch (PrimitiveType.ScalarKind)
				{
					case MIR::EScalarKind::Integer:    ParamInsight.ComponentType = FMaterialInsights::FUniformBufferSlotComponentType::CT_Int; break;
					case MIR::EScalarKind::Float:  ParamInsight.ComponentType = FMaterialInsights::FUniformBufferSlotComponentType::CT_Float; break;
					case MIR::EScalarKind::Double: ParamInsight.ComponentType = FMaterialInsights::FUniformBufferSlotComponentType::CT_LWC; break;
					default: UE_MIR_UNREACHABLE();
				}

				Insights->UniformParameterAllocationInsights.Push(ParamInsight);
			}
		}
	}

	Data.Preshader2TemporarySize = (uint16)TempAllocator.AllocatedFloat4s;

	// Subtract one from all offsets
	for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
	{
		Remap[PreshaderIndex]--;		// Subtract one!
	}

	// Fix up uniform parameter evaluations.
	Module->GetCompilationOutput().UniformExpressionSet.FixupNumericParameterEvaluations(Remap, UniformComponentCounts);

	// Fix up preshader bytecode.
	UE::Preshader2::FixupAndCompress(PreshaderOut, Data, Remap);

	// Fix up Analysis_PreshaderOffset -- needs to happen in separate loop, as the original Analysis_PreshaderOffset
	// values are used when generating the remapping.
	for (int32 PreshaderIndex = 1; PreshaderIndex < ValueInfos.Num(); PreshaderIndex++)
	{
		MIR::FValue* Value = ValueInfos[PreshaderIndex].Value;

		// One is added to this to guarantee the value is non-zero, and subtracted later when used.
		if (Value)
		{
			Value->Analysis_PreshaderOffset = Remap[Value->Analysis_PreshaderOffset] + 1;
		}
	}
}

#endif  // WITH_EDITOR