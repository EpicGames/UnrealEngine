// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIR.h"
#include "Materials/MaterialInsights.h"

#if WITH_EDITOR

// Manages data related to preshaders (expressions evaluated on the CPU, outside the GPU).  Expressions can be evaluated on the CPU if they
// are only dependent on uniform values, meaning values that don't vary per vertex or pixel.  The types of leaf values that are uniform are
// parameters and constants.  Constants by themselves don't need to be evaluated as preshaders, since they can just be inlined in the shader
// code.  So any preshader must include some sort of parameter in its expression tree, either FUniformParameter or FPreshaderParameter.
// 
// A leaf value by itself isn't an expression -- for something to become an expression, it must include operators (FOperator) or perhaps a
// swizzle or merge of some other values (combinations of FComposite and FSubscript nodes).  Not all MIR operators are supported by the
// runtime Preshader2 bytecode, so the FOperatorSignature struct in MIR includes markup for which are supported, and only those are treated
// as uniform.  To sum up, there are 5 MIR::FValue kinds supported by preshaders:
//
//		FUniformParameter, FPreshaderParameter, FOperator, FComposite, FSubscript
//
// In the future FScalar (used by MIR for type casting) could also be supported, but this is considered low priority, as currently shaders
// are unable to produce integer based uniform expressions at all, and uniform math on double values is exceedingly rare.  We do need to
// treat operations on double parameters as non-uniform in the meantime.  See MarkUniformIfSupportedPreshaderOperator in MaterialIREmitter.cpp.
//
// The MIR value analyzer step tracks subtrees where any child is a parameter (EGraphProperties::HasParameter), and subtrees where all
// children are uniform (EGraphProperties::Uniform).  Any value with both flags set is a Preshader.  The value analyzer further must
// differentiate between values that are visible to the GPU (EGraphProperties::PreshaderOut), and temporaries which are only used by other
// uniform calculations (EGraphProperties::PreshaderTemp).  We want to avoid wasting GPU buffer space for values not actually used on the GPU,
// while on the CPU, we can allocate a larger buffer with space at the end for temporaries.  It's possible for PreshaderOut values to also be
// an intermediate value to other calculations, but only the PreshaderOut flag will be set on such values.  To figure out whether a value is
// visible to the GPU requires checking its parent to see if it's non-uniform, handled by having non-uniform nodes flag their Uniform children
// with PreshaderOut.  Children of PreshaderOut are then recursively marked as PreshaderTemp, if not already marked as PreshaderOut.
//
// There's a special case optimization in the value analyzer where trivial tail swizzles (swizzles of components of a single source value,
// plus any number of constants) are NOT executed as Preshaders.  Such swizzles and constant loads are "free" on all modern scalar op GPUs,
// and there's no reason to waste CPU time cobbling them together into a contiguous vector.  For example, expressions like these:
//
//			MaterialOutput = Composite( Parameter.y, Parameter.x, Parameter.z );
//			MaterialOutput = Composite( Value.x, Value.y, 1.0 );
//
// Optimizing out trivial tail swizzles involves checking if a composite used by a non-uniform meets this criteria, and if so, marking the
// underlying Parameter or Value as the starting point of the Preshader, rather than the Composite.  This pattern is actually quite common,
// with trivial tail swizzles on a Parameter representing 25% of all Preshaders in the old material translator in an instrumented replay
// run on a large project.  This optimization also handles cases where the old material translator wouldn't generate swizzles, but the new
// material translator would try to, representing another 45% of Preshaders.  And it will frequently save additional ops on many other
// Preshaders.  That's why it's worthwhile to add this complexity.
// 
// It's necessary to separately mark up these composites as "TrivialTailSwizzle", to inform the GPU backend that it's OK to emit them, even
// if they are also marked as PreshaderTemp, which otherwise prevents emitting instructions to the GPU.  Put in a different way,
// "TrivialTailSwizzle" is similar to "PreshaderOut", in that both imply a value accessible on the GPU, just with the former involving the
// final Composite step being handled by the GPU backend.
//
// Now that we've identified Preshaders, we need to translate them to the Preshader2 bytecode format.  We don't know which values are
// outputs versus temporaries, which affects where the values should go in the buffer (temporaries go at the end), until the entire tree
// has been parsed.  We also need to handle "register" allocation and lifetime tracking.  "Register" is in quotes, because the Preshader2
// runtime doesn't technically have registers -- all operations are on buffer offsets (in words).  The MIR HLSL translator creates a unique
// variable for each intermediate value, but Preshader2 bytecode benefits from reusing registers, both in terms of using less memory for
// temporaries, and allowing opcodes to be compressed if an output matches the input or the previous instruction's output.
//
// To handle buffer allocation and register assignment, initial non-executable Preshader2 bytecode is generated with unique register indices
// per value, instead of buffer offsets.  These can then be replaced with buffer offsets with a call to UE::Preshader2::FixupAndCompress.
// The "bPreFixup" flag on the output data indicates whether we are generating such bytecode.  A special case is needed for the Preshader2
// MoveOp, which handles swizzling or concatenating components of vectors.  The runtime version of the bytecode can directly access individual
// components of vectors by word offset, but when they are in the register format, we need separate component indices within the registers,
// so a different MoveOpForFixup opcode is used.
//
// Preshader2 supports word based indexing, and most binary ops support scalar arguments.  This means that we can avoid explicit MoveOps
// in most cases, and directly access a subset of another intermediate result (either an individual component or sequential set of
// components).  This is handled using an Alias feature, where a given intermediate value can declare itself as a subset of another value.
// Vector aliases include an AliasOffset specifying the component where the Alias starts, while an AliasOffset of -1 (INDEX_NONE) indicates
// one component should be used for all, with the number of components coming from the MIR::FValue.  Aliases are automatically set up for all
// FSubscript, since those have only one component and are thus always representable as an alias.  For Composite, an alias will be set up where
// possible (sequential subset of components or all components the same), but in non-trivial cases where MoveOps would be required, generation
// of bytecode is deferred until used by another op, as it will be fairly common that the non-trivial composite won't actually be used, if it
// represents a TrivialTailSwizzle that gets optimized out.  ExpandComposite handles this case, and also supports mixing in components
// initialized from constants.
//
// Preshader2 includes opcodes with inline constants, so constants are usually rolled into binary ops, although some edge cases can't use
// an inline constant, and a constant load op is generated.  For example, a subtract (or any non-commutative op) where the first argument
// is a constant, as the constant is always implicitly the second argument.  In the future, if it would help optimization, we could add
// support for reversed order opcodes to Preshader2, to allow those cases to use inline constants, although it would increase code size.
//
// Aliases are a form of space optimization (avoids generating a separate buffer location).  As another space optimization, temporary
// buffer space can be freed and reused (see FComponentAllocator class in the cpp file).  A third important space optimization involves
// output location propagation.  This involves scanning the list of values in reverse, and identifying cases where one of the inputs to
// a value can reuse the output location of another value.  To illustrate where this is valuable, consider the following real world example,
// without the optimization:
//
//		UniformBuffer[2].xyz = #1 Name = "Vector" Type = "Vector" Mask = .xyz
//		UniformBuffer[1].xyz = Add_Const(UniformBuffer[2].xyz "Vector", { 0.500000f, 0.100000f, 0.100000f })  float3
//
// Here we have a vector added to a constant.  UniformBuffer[1] is an output location visible to the GPU, while UniformBuffer[2] (a raw
// uniform parameter value) is in a temporary location.  We could make this take less space by initially writing the input value to the
// same location as the output, completely eliminating the temporary location:
//
//		UniformBuffer[1].xyz = #1 Name = "Vector" Type = "Vector" Mask = .xyz
//		UniformBuffer[1].xyz = Add_ConstInPlace(UniformBuffer[1].xyz "Vector", { 0.500000f, 0.100000f, 0.100000f })  float3
//
// Not only does this save space in the temporary buffer, it also results in the more compact "InPlace" opcode being generated, saving
// space in the opcode data.  For chains of multiple such operations, the even more compact "Reuse" opcode variants can potentially be
// generated.  Here you can see why it's important to scan the values in reverse, as output location propagation can happen across
// multiple chained opcodes, and a value can't get the correct final location unless ones after it have already been processed.
//
// FUTURE OPTIMIZATIONS:  Output location propagation is only currently supported for whole vectors.  It's possible in theory to implement
// output location propagation for components cobbled together into a single vector.  For example:
// 
//		UniformBuffer[2].x = #1 Name = "ValueA" Type = "Scalar" Mask = .x
//		UniformBuffer[2].y = #1 Name = "ValueB" Type = "Scalar" Mask = .x
//		UniformBuffer[1].xy = Move(UniformBuffer[2].x "ValueA", UniformBuffer[2].y "ValueB");
//
// Here, a move operation is generated to cobble two vector components together into a float2 accessible on the GPU.  Trivial tail swizzle
// optimizations don't apply here, because the input values aren't from the same source vector (here, they happen to be adjacent in
// the temporary buffer, but this is not guaranteed).  Because Preshader2 supports flat access to individual words, it would be possible
// to just write the words into the correct place in the output vector, cutting out the Move:
//
// 		UniformBuffer[1].x = #1 Name = "ValueA" Type = "Scalar" Mask = .x
//		UniformBuffer[1].y = #1 Name = "ValueB" Type = "Scalar" Mask = .x
//
// The complexity of implementing this is that the move op will have already been emitted to the bytecode, and needs to be stripped out,
// and LastUsedBy on each component needs to be propagated forward to anything that uses the compound value.  It's a little bit
// difficult to unit test the possible permutations of this, compared to the simpler case of whole vector output location propagation.
//
// An even more specific case of output location propagation that could be optimized would be parallel in-place operations.  Consider
// a case like this:
//
//		UniformBuffer[2].xyzw = #1 Name = "Vector" Type = "Vector" Mask = .xyzw
//		UniformBuffer[1].xyz = Multiply_Const(UniformBuffer[2].xyz "Vector", { 0.500000f, 0.100000f, 0.100000f })  float3
//		UniformBuffer[1].w = Add_Const(UniformBuffer[2].w "Vector", { 1.00000f })  float
//
// In this case, different preshader math is applied more or less in place to a vector and scalar portion of the same float4.  This
// can happen much more frequently than you might think, because MIR does folding of operations involving constants independently per
// vector component, even if the original vector was strictly used as a whole in the original graph.  But it's also common for end
// user math to treat "w" differently in an example like the above by design.  The temporary could be eliminated:
//
//		UniformBuffer[1].xyzw = #1 Name = "Vector" Type = "Vector" Mask = .xyzw
//		UniformBuffer[1].xyz = Multiply_ConstInPlace(UniformBuffer[1].xyz "Vector", { 0.500000f, 0.100000f, 0.100000f })  float3
//		UniformBuffer[1].w = Add_ConstInPlace(UniformBuffer[1].w "Vector", { 1.00000f })  float
//
// To make this work, you would do output propagation like normal, and when you reach a fetch from an alias, you would check if the
// components being fetched are only used by that subtree (perhaps handled by extending UsedBy tracking to aliases).  And then check
// if the "components only used in subtree" property was true for all the components of the underlying value.  In that case, you
// would merge the two values into a single contiguous allocation.  It could be considered the logical opposite of cobbling together
// OUTPUT components to avoid move as described above -- instead it's cobbling together INPUT components.
//
// A final similar optimization is merging of parallel scalar math.  As mentioned above, MIR does constant operation folding
// independently per component, which can generate preshader bytecode like this:
// 
//		UniformBuffer[3].x = Add_Const(UniformBuffer[2].x "Vector", { 0.500000f })  float
//		UniformBuffer[3].y = Add_Const(UniformBuffer[2].y "Vector", { 0.100000f })  float
//		UniformBuffer[2].xyz = Move(UniformBuffer[3].x, UniformBuffer[3].y, UniformBuffer[2].z)  float3
//		UniformBuffer[1].xyz = Multiply_Const(UniformBuffer[2].xyz, { 0.300000f })  float3
//
// The original graph included the math:  (Vector + float3(0.5, 0.1, 0.0)) * 0.3
//
// MIR decided the add by zero could be optimized out for the Z component, and so split the operation for each individual scalar
// component, producing a separate constant multiply for the x and y components, even though they are from the same source vector.
// The simplest way to handle this would probably be to disable per-component constant operation folding in the first place for
// uniform subtrees, to produce this code, saving several bytes and two ops:
//
//		UniformBuffer[1].xyz = Add_ConstInPlace(UniformBuffer[1].xyz "Vector", { 0.500000f, 0.100000f, 0.000000f })  float3
//		UniformBuffer[1].xyz = Multiply_ConstReuse(UniformBuffer[1].xyz, { 0.300000f })  float3
//
// The challenge is that uniform subtrees right now are not identified until the analyze step, so you would need to move that
// earlier (which may or may not be difficult).  Merging the ops after the fact is also possible, but messier.  Perhaps
// merely adding a flag to operations that were split would allow the Preshader logic to merge ops easily enough (so it doesn't
// have to do a deep, and therefore complicated to debug, analysis of where merging subtrees of ops is possible).
// 
// Another intermediate option would be to have MIR not split adjacent parallel ops into individual scalars, saving one op:
//
//		UniformBuffer[3].xy = Add_Const(UniformBuffer[2].xy "Vector", { 0.500000f, 0.100000f })  float2
//		UniformBuffer[2].xyz = Move(UniformBuffer[3].x, UniformBuffer[3].y, UniformBuffer[2].z)  float3
//		UniformBuffer[1].xyz = Multiply_Const(UniformBuffer[2].xyz, { 0.300000f })  float3
//
// And the second future optimization mentioned above (parallel in-place ops) could optimize that down to this:
//
//		UniformBuffer[1].xy = Add_ConstInPlace(UniformBuffer[1].xy "Vector", { 0.500000f, 0.100000f })  float2
//		UniformBuffer[1].xyz = Multiply_ConstInPlace(UniformBuffer[1].xyz, { 0.300000f })  float3
//
// Achieving the minimal number of ops for this particular case in a different way.  Although this approach breaks down in
// cases where the components modified are not adjacent.
//
struct FMaterialIRPreshader
{
	FMaterialIRPreshader()
	{
		// Add an invalid dummy element to the array.
		ValueInfos.AddDefaulted();

		// Mark the preshader data as pre-fixup.
		Data.bPreFixup = true;
	}

	// Stores data for values in the MIR tree that are used for preshader evaluation.
	struct FValueInfo
	{
		MIR::FValue* Value = nullptr;
		MIR::FValue* Alias = nullptr;				// Value is an alias of this other value.
		int16 AliasOffset = INDEX_NONE;				// Offset of Value in its alias.  If INDEX_NONE, it's an alias of a scalar.
		uint16 FirstUsedBy = 0;						// First value this is used by
		uint16 LastUsedBy = 0;						// Last value this is used by
		uint16 UsedByCount = 0;						// Number of items this value is used by
		TArray<uint16, TFixedAllocator<4>> Uses;	// List of FValueInfo this uses
		uint16 OutputLocation = 0;					// Value pulls its output location from this other value.
	};

	// Stores list of preshader values encountered.  "Analysis_PreshaderOffset" in the FValue indexes into this array. Note that the first item is an
	// unused null dummy value, so valid Analysis_PreshaderOffset indices are non-zero, and zero initialized FValue data represents no preshader.
	TArray<FValueInfo> ValueInfos;

	// Data here is pre-fixup, with offsets representing indices into the ValueInfo array above.  Later UE::Preshader2::FixupAndCompress remaps the
	// offsets when generating the final output preshader data.
	UE::Shader::FPreshaderData Data;

	int32 OutputCount = 0;

	// Number of float4s allocated for VS-frequency PreshaderOut values. Set by PreshaderFixup when UseMaterialVSUniformBuffer() is true.
	uint32 VSPreshaderFloat4s = 0;

	static EPreshader2Type ToPreshader2Type(MIR::EScalarKind Kind)
	{
		switch (Kind)
		{
		case MIR::EScalarKind::Float:	return EPreshader2Type::Float;
		case MIR::EScalarKind::Double:	return EPreshader2Type::Double;
		case MIR::EScalarKind::Integer:	return EPreshader2Type::Integer;
		}
		return EPreshader2Type::Float;
	}

	int32 AddValueInfo(MIR::FValue* Value)
	{
		check(!Value->Analysis_PreshaderOffset);

		int32 PreshaderIndex = ValueInfos.AddDefaulted();
		Value->Analysis_PreshaderOffset = PreshaderIndex;

		FMaterialIRPreshader::FValueInfo& ValueInfo = ValueInfos[PreshaderIndex];
		ValueInfo.Value = Value;

		return PreshaderIndex;
	}

	FMaterialIRPreshader::FValueInfo& AddValueInfo_GetRef(MIR::FValue* Value)
	{
		return ValueInfos[AddValueInfo(Value)];
	}

	bool ValueIsScalarAlias(MIR::FValue* Value) const
	{
		check(Value->Analysis_PreshaderOffset);
		const FValueInfo& ValueInfo = ValueInfos[Value->Analysis_PreshaderOffset];
		return ValueInfo.Alias && ValueInfo.AliasOffset == INDEX_NONE;
	}

	// Track that "Dest" uses "Source".
	void AddValueUse(const MIR::FValue* Source, const MIR::FValue* Dest);

	// Explicitly emit a primitive composite that's any combination of preshader values and constants as a full preshader value.
	void EmitComposite(MIR::FComposite* Composite);

	// Emits a constant to a different destination location, useful to follow up with an in-place op on the destination.  Returns Destination.
	MIR::FValue* EmitConstant(const MIR::FValue* SourceConstant, EPreshader2Type Preshader2Type, MIR::FValue* Destination);

	// Emits a constant to a dummy value (not part of the MIR tree).  Used for rare case of constants that can't be inline or in-place.
	MIR::FValue* EmitDummyConstant(const MIR::FValue* SourceConstant, EPreshader2Type Preshader2Type, int32 NumComponents);

	// Emit a value as a preshader.  Non-trivial composites are not emitted here (as they can often be dead stripped), and instead
	// are handled by EmitComposite as needed.  Also doesn't support emitting constants -- use EmitConstant instead.
	void EmitValue(MIR::FValue* Value);

	// Final fixup after entire value tree has been processed.
	void PreshaderFixup(UE::Shader::FPreshaderData& PreshaderOut, FMaterialIRModule* Module, FMaterialInsights* Insights);
};

#endif  // WITH_EDITOR