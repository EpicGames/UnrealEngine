// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialIRCommon.h"

#if WITH_EDITOR

// Used to translate a generated Material MIR module into the material HLSL shader.
struct FMaterialIRToHLSLTranslation
{
	// The Material IR module, which contains the intermediate representation of the material graph.
	const FMaterialIRModule* Module{};

	// The material being translated.
	const FMaterial* Material{};
	
	// The set of static parameters assignements.
	const FStaticParameterSet* StaticParameters{};
	
	// The translation target platform.
	const ITargetPlatform* TargetPlatform{};
	
	// Executes the translation process, filling the material shader template parameters in OutShaderParameters and configuring OutEnvironment for shader compilation.
	// After this call, interpolate the MaterialTemplate shader with the list of parameters to get the final HLSL shader.
	void Run(TMap<FString, FString>& OutShaderParameters, FShaderCompilerEnvironment& OutEnvironment);
};

#endif // #if WITH_EDITOR
