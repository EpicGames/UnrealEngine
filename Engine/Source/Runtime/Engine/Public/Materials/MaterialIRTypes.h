// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Materials/MaterialIRCommon.h"
#include "Shader/ShaderTypes.h"
#include "MaterialValueType.h"
#include "Materials/MaterialParameters.h"

#if WITH_EDITOR

namespace MIR
{

// Represents the top-level kind of a type.
enum class ETypeKind : uint8
{
	Poison, 	// An invalid or error type.
	Void,  		// The void type, representing no value.
	Primitive, 	// A numeric, vector, or matrix type (e.g. float, float4, float4x4).
	Aggregate,	// A material aggregate.

	// Opaque Objects

	Texture2D,				// A 2D texture object.
	TextureCube,			// A cube texture object.
	Texture2DArray,			// A 2D texture array object.
	TextureCubeArray,		// A cube texture array object.
	Texture3D,				// A 3D (volume) texture object.
	TextureExternal,		// An external texture
	VirtualTexture,			// A virtual texture object (UTexture - MCT_TextureVirtual).
	RuntimeVirtualTexture,	// A runtime virtual texture object (URuntimeVirtualTexture).
	ParameterCollection,	// Material parameter collection object.
	SubstrateData,			// Substrate-specific data structure.
	VTPageTableResult,		// A virtual texture page table lookup result.
};

// Returns the string representation of a ETypeKind.
const TCHAR* TypeKindToString(ETypeKind Kind);

// Primitive types of a single scalar.
// Note: These are listed in precision order. Converting one to the other is then simply performed taking the max EScalarKind.
enum class EScalarKind : uint8
{
	Boolean,	// Boolean type (true/false).
	Integer,	// 32-bit signed integer type.
	Float,		// 32-bit floating-point type.
	Double,		// Large World Coordinates type.
};

// Returns the larger scalar kind between A and B.
static inline constexpr EScalarKind GetCommonScalarKind(EScalarKind A, EScalarKind B)
{
	return FMath::Max(A, B);
}

// Returns whether the specified scalar kind supports arithmetic operators (plus, minus, etc).
bool ScalarKindIsArithmetic(EScalarKind Kind);

// Returns whether the specified scalar kind is a floating point type (float, LWC).
bool ScalarKindIsAnyFloat(EScalarKind Kind);

// Returns the string representation of specified scalar kind.
const TCHAR* ScalarKindToString(EScalarKind Kind);

// Describes a primitive type (scalar, vector, or matrix).
struct FPrimitive
{
	// The underlying scalar data type (e.g. Bool, Int, Float).
	EScalarKind ScalarKind;

	// Number of rows. For a vector, this is the number of elements. For a scalar, this is 1.
	int8 NumRows;
	
	// Number of columns. For a vector or scalar, this is 1. For a matrix, this is > 1.
	int8 NumColumns;

	// If true, this represents an LWC inverse matrix. Only supported for EScalarKind::LWC and 4x4 dimensions.
	bool bIsLWCInverseMatrix;

	// Returns whether this primitive type is arithmetic (it supports arithmetic operations like addition).
	bool IsArithmetic() const { return ScalarKindIsArithmetic(ScalarKind); }
	
	// Returns whether this type is a primitive scalar (1x1).
	bool IsScalar() const { return NumComponents() == 1; }
	
	// Returns whether this primitive type is a row vector (rows = 1, columns > 1).
	bool IsRowVector() const { return NumRows == 1 && NumColumns > 1; }

	// Returns whether this primitive type is a column vector  (rows > 1, columns = 1).
	bool IsColumnVector() const { return NumRows > 1 && NumColumns == 1; }
	
	// Returns whether this primitive type is a matrix (rows > 1, columns > 1).
	bool IsMatrix() const { return NumRows > 1 && NumColumns > 1; }
	
	// Returns whether this is a boolean primitive type (of any dimension).
	bool IsBoolean() const { return ScalarKind == EScalarKind::Boolean; }
	
	// Returns whether this is a single boolean scalar.
	bool IsBoolScalar() const { return IsBoolean() && IsScalar(); }

	// Returns whether this is an integer primitive type (of any dimension).
	bool IsInteger() const { return ScalarKind == EScalarKind::Integer; }
	
	// Returns whether this is a float primitive type (of any dimension).
	bool IsFloat() const { return ScalarKind == EScalarKind::Float; }

	// Returns whether this is an LWC primitive type (of any dimension).
	bool IsDouble() const { return ScalarKind == EScalarKind::Double; }
	
	// Returns whether this is any floating point primitive type (e.g. float, LWC).
	bool IsAnyFloat() const { return ScalarKindIsAnyFloat(ScalarKind); }

	// Returns the total number of scalar components in this primitive type.
	int32 NumComponents() const { return NumRows * NumColumns; }

	// Returns a new FType with the same dimensions but a different scalar kind.
	FType ToScalarKind(EScalarKind InScalarKind) const;
	
	// Returns the scalar FType corresponding to this primitive's scalar kind.
	FType ToScalar() const;
	
	// Returns a vector FType with the given number of rows and this primitive's scalar kind.
	FType ToVector(int NumColumns) const;

	bool operator==(const FPrimitive& Other) const = default;
	bool operator!=(const FPrimitive& Other) const = default;
};

// Describes a complete type within the material IR. This can be a primitive, an aggregate
// (struct), an opaque object (e.g., a texture), or a special type like Poison or Void.
// Note: This struct is lightweight and intended to be passed by value.
struct FType
{
	// Returns the FType corresponding to a given UE::Shader::FType.
	static FType FromShaderType(const UE::Shader::FType& InShaderType);
	
	// Returns the FType corresponding to a given EMaterialValueType.
	static FType FromMaterialValueType(EMaterialValueType Type);

	// Returns the FType corresponding to a given EMaterialParameterType.
	static FType FromMaterialParameterType(EMaterialParameterType Type);
	
	// Returns the poison type, which represents an error or invalid type.
	static FType MakePoison();
	
	// Returns the void type, which represents the absence of a value.
	static FType MakeVoid();

	// Returns a scalar type of the given scalar kind.
	static FType MakeScalar(EScalarKind InScalarKind) { return MakePrimitive(InScalarKind, 1, 1); }

	// Returns a scalar type representing a boolean value.
	static FType MakeBoolScalar() { return MakeScalar(EScalarKind::Boolean); }

	// Returns a scalar type representing a integer value.
	static FType MakeIntScalar() { return MakeScalar(EScalarKind::Integer); }

	// Returns a scalar type representing a single precision floating point value.
	static FType MakeFloatScalar() { return MakeScalar(EScalarKind::Float); }

	// Returns a scalar type representing a double precision floating point value.
	static FType MakeDoubleScalar() { return MakeScalar(EScalarKind::Double); }

	// Returns a row-major vector type of the given scalar kind and number of columns.
	static FType MakeVector(EScalarKind InScalarKind, int NumColumns) { return MakePrimitive(InScalarKind, 1, NumColumns); }

	// Returns a row-major vector type of boolean values with the specified number of columns.
	static FType MakeBoolVector(int NumColumns) { return MakeVector(EScalarKind::Boolean, NumColumns); }

	// Returns a row-major vector type of integer values with the specified number of columns.
	static FType MakeIntVector(int NumColumns) { return MakeVector(EScalarKind::Integer, NumColumns); }

	// Returns a row-major vector type of single precision floating point values with the specified number of columns.
	static FType MakeFloatVector(int NumColumns) { return MakeVector(EScalarKind::Float, NumColumns); }

	// Returns a row-major vector type of double precision floating point values with the specified number of columns.
	static FType MakeDoubleVector(int NumColumns) { return MakeVector(EScalarKind::Double, NumColumns); }
	
	// Creates an integer primitive type with given dimensions.
	static FType MakeInt(int NumRows, int NumColumns) { return MakePrimitive(EScalarKind::Integer, NumRows, NumColumns); }

	// Creates a single precision float primitive type with given dimensions.
	static FType MakeFloat(int NumRows, int NumColumns) { return MakePrimitive(EScalarKind::Float, NumRows, NumColumns); }

	// Creates a double precision float primitive type with given dimensions.
	static FType MakeDouble(int NumRows, int NumColumns) { return MakePrimitive(EScalarKind::Double, NumRows, NumColumns); }

	// Returns a primitive type with the given kind and dimensions.
	// Double inverse matrix flag is only supported for EScalarKind::Double and 4x4 dimensions.
	static FType MakePrimitive(EScalarKind InScalarKind, int NumRows, int NumColumns, bool bIsDoubleInverseMatrix = false);

	// Returns the aggregate type associated with the given material aggregate definition.
	static FType MakeAggregate(const UMaterialAggregate* Aggregate);

	// Returns the Material Parameter Collection object type.
	static FType MakeParameterCollection();

	// Returns the Texture2D object type.
	static FType MakeTexture2D();

	// Returns the TextureCube object type.
	static FType MakeTextureCube();

	// Returns the Texture2DArray object type.
	static FType MakeTexture2DArray();

	// Returns the TextureCubeArray object type.
	static FType MakeTextureCubeArray();

	// Returns the Texture3D object type.
	static FType MakeTexture3D();

	// Returns the texture external object type.
	static FType MakeTextureExternal();

	// Returns the VirtualTexture object type (for MCT_TextureVirtual, backed by UTexture).
	static FType MakeVirtualTexture();

	// Returns the RuntimeVirtualTexture object type.
	static FType MakeRuntimeVirtualTexture();
	
	// Returns the SubstrateData object type.
	static FType MakeSubstrateData();
	
	// Returns the VTPageTableResult object type.
	static FType MakeVTPageTableResult();

	// Returns the this type name spelling (e.g. float4x4).
	FString GetSpelling() const;
	
	// Checks if this type is of the specified kind.
	bool Is(ETypeKind InKind) const { return Kind == InKind; }

	// Returns the high-level kind of this type.
	ETypeKind GetKind() const { return Kind; }

	// Returns whether this is the poison type, indicating an error.
	bool IsPoison() const { return Kind == ETypeKind::Poison; }
	
	// Returns whether this is the void type.
	bool IsVoid() const { return Kind == ETypeKind::Void; }

	// Returns true if the given type is a primitive that supports arithmetic operations.
	bool IsPrimitive() const { return Is(ETypeKind::Primitive); }

	// Returns true if the given type is a primitive that supports arithmetic operations.
	bool IsArithmetic() const { return Is(ETypeKind::Primitive) && Primitive.IsArithmetic(); }

	// Returns true if the given type is a primitive with a boolean scalar kind.
	bool IsBoolean() const { return Is(ETypeKind::Primitive) && Primitive.IsBoolean(); }

	// Returns true if the given type is a primitive with an integer scalar kind.
	bool IsInteger() const { return Is(ETypeKind::Primitive) && Primitive.IsInteger(); }

	// Returns true if the given type is a primitive with a float scalar kind.
	bool IsFloat() const { return Is(ETypeKind::Primitive) && Primitive.IsFloat(); }

	// Returns true if the given type is a primitive with an LWC scalar kind.
	bool IsDouble() const { return IsPrimitive() && Primitive.IsDouble(); }

	// Returns true if the given type is a primitive with any float-based scalar kind (e.g. float, LWC).
	bool IsAnyFloat() const { return Is(ETypeKind::Primitive) && Primitive.IsAnyFloat(); }

	// Returns true if the given type is a scalar primitive.
	bool IsScalar() const { return Is(ETypeKind::Primitive) && Primitive.IsScalar(); }

	// Returns true if the given type is a boolean scalar primitive.
	bool IsBoolScalar() const { return Is(ETypeKind::Primitive) && Primitive.IsBoolScalar(); }

	// Returns true if the given type is a vector primitive.
	bool IsVector() const { return Is(ETypeKind::Primitive) && Primitive.IsRowVector(); }

	// Returns true if the given type is a vector primitive of given components.
	bool IsVector(int32 NumComponents) const { return Is(ETypeKind::Primitive) && Primitive.IsRowVector() && Primitive.NumComponents() == NumComponents; }

	// Returns whether this type is a Material Parameter Collection object.
	bool IsParameterCollection() const { return Kind == ETypeKind::ParameterCollection; }

	// Returns true if the given type is a matrix primitive.
	bool IsMatrix() const { return Is(ETypeKind::Primitive) && Primitive.IsMatrix(); }

	// Returns whether this type is any texture object stored in FTextureUniform (includes virtual textures, excludes RuntimeVirtualTexture).
	bool IsTexture() const { return Kind == ETypeKind::Texture2D || Kind == ETypeKind::TextureCube || Kind == ETypeKind::Texture2DArray || Kind == ETypeKind::TextureCubeArray || Kind == ETypeKind::Texture3D || Kind == ETypeKind::VirtualTexture || Kind == ETypeKind::TextureExternal; }

	// Returns whether this type is a VirtualTexture object (MCT_TextureVirtual, backed by UTexture).
	bool IsVirtualTexture() const { return Kind == ETypeKind::VirtualTexture; }

	// Returns whether this type is a RuntimeVirtualTexture object.
	bool IsRuntimeVirtualTexture() const { return Kind == ETypeKind::RuntimeVirtualTexture; }

	// Returns whether this type is a SubstrateData object.
	bool IsSubstrateData() const { return Kind == ETypeKind::SubstrateData; }

	// Returns whether this type is a VTPageTableResult object.
	bool IsVTPageTableResult() const { return Kind == ETypeKind::VTPageTableResult; }

	// Returns the primitive type information. It asserts that this type is primitive.
	FPrimitive GetPrimitive() const { check(Is(ETypeKind::Primitive)); return Primitive; }
	
	// If this is a primitive type, returns its primitive info. Otherwise, returns None.
	TOptional<FPrimitive> AsPrimitive() const { return IsPrimitive() ? Primitive : TOptional<FPrimitive>{}; }

	// If this is a scalar type, returns its primitive info. Otherwise, returns None.
	TOptional<FPrimitive> AsScalar() const { return IsScalar() ? Primitive : TOptional<FPrimitive>{}; }

	// If this is a vector type, returns its primitive info. Otherwise, returns None.
	TOptional<FPrimitive> AsVector() const { return IsVector() ? Primitive : TOptional<FPrimitive>{}; }

	// If this is a matrix type, returns its primitive info. Otherwise, returns None.
	TOptional<FPrimitive> AsMatrix() const { return IsMatrix() ? Primitive : TOptional<FPrimitive>{}; }

	// If this is an aggregate type, returns a pointer to the UMaterialAggregate definition. Otherwise, returns nullptr.
	const UMaterialAggregate* AsAggregate() const { return Is(ETypeKind::Aggregate) ? Aggregate : nullptr; }

	// Converts this type to a corresponding UE::Shader::EValueType.
	UE::Shader::EValueType ToValueType() const;

	bool operator==(FType Other) const;
	bool operator!=(FType Other) const { return !operator==(Other); }

	friend uint32 GetTypeHash(const FType& T);

private:
	union
	{
		// The primitive type information.
		FPrimitive Primitive;

		// Pointer to the material aggregate.
		const UMaterialAggregate* Aggregate;
	};

	// Identifies the high-level kind of this type.
	ETypeKind Kind : 8;
};

// To avoid the compiler adding automatic padding, we must make sure that its size is a multiple of maximum alignment.
static_assert(sizeof(MIR::FType) % alignof(void*) == 0);
static_assert(sizeof(MIR::FType) == 16);

} // namespace MIR

#endif // #if WITH_EDITOR
