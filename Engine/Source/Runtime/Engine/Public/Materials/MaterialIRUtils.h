// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Misc/MemStack.h"

#if WITH_EDITOR

#define UE_MIR_UNREACHABLE() { check(!"Unreachable"); UE_ASSUME(false); }
#define UE_MIR_TODO() UE_MIR_UNREACHABLE()

namespace MIR
{

// A simple string view without constructors, to be used inside FValues.
struct FSimpleStringView
{
	const TCHAR* Ptr; // The string characters
	int32        Len; // The string length (excluding null byte)

	FStringView ToView() const { return { Ptr, Len }; }
	FString ToString() const { return FString{ ToView() }; }
	operator FStringView() const { return ToView(); }
	FSimpleStringView& operator=(FStringView View) { Ptr = View.GetData(); Len = View.Len(); return *this; }
	const TCHAR* operator*() const { return Ptr; }
};

// Use this to efficiently allocate a temporary array using MemStack, instead
// of using TArray and going through the global allocator. Only declare local variable
// of this struct (i.e. do not "new TTemporaryArray")
// Remark: The allocated memory lifespan is the same as the TTemporaryArray local variable
// (it is deallocated when TTemporaryArray goes out of scope).
template <typename T>
struct TTemporaryArray : public TArrayView<T>
{
	FMemMark MemMark;

	TTemporaryArray(int32 Num)
	: MemMark{ FMemStack::Get() }
	{
		*static_cast<TArrayView<T>*>(this) = { (T*)FMemStack::Get().Alloc(sizeof(T) * Num, alignof(T)), Num };
	}

	TTemporaryArray(TConstArrayView<T> Source)
	: TTemporaryArray { Source.Num() }
	{
		static_assert(std::is_trivially_copyable_v<T>);
		FMemory::Memcpy(TArrayView<T>::GetData(), Source.GetData(), Source.NumBytes());
	}

	operator TConstArrayView<T>() const
	{
		return { TArrayView<T>::GetData(), TArrayView<T>::Num() };
	}

	void Zero()
	{
		FMemory::Memzero(TArrayView<T>::GetData(), sizeof(T) * TArrayView<T>::Num());
	}
};

// Zeroes an array of elements.
template <typename T>
void ZeroArray(TArrayView<T> Array)
{
	static_assert(TIsTriviallyCopyAssignable<T>::Value);
	FMemory::Memzero(Array.GetData(), Array.Num() * sizeof(T));
}

// Tries to find a value with given Key into given TMap intance. If it's found, its value is assigned to OutValue.
// Returns whether the value was found and OutValue was set.
template <typename TKey, typename TValue>
bool Find(const TMap<TKey, TValue>& Map, const TKey& Key, TValue& OutValue)
{
	if (auto ValuePtr = Map.Find(Key)) {
		OutValue = *ValuePtr;
		return true;
	}
	return false;
}

// Returns whether two array views are equal by comparing their elements.
// Requires operator== to be defined for T.
template <typename T>
bool ArrayViewEquals(TArrayView<T> A, TArrayView<T> B)
{
	if (A.Num() != B.Num())
	{
		return false;
	}
	for (int32 i = 0; i < A.Num(); ++i)
	{
		if (!(A[i] == B[i]))
		{
			return false;
		}
	}
	return true;
}

// Computes a hash from a byte buffer using word-aligned reads and HashCombineFast.
// The pointer must be 4-byte aligned.
uint32 HashBytes(const void* InPtr, uint32 Size);

// Computes the hash of an array view by combining the hashes of its elements.
// Requires GetTypeHash to be defined for T.
template <typename T>
uint32 ArrayViewHash(TArrayView<T> Array)
{
	uint32 Hash = ::GetTypeHash(Array.Num());
	for (const T& Element : Array)
	{
		Hash = HashCombine(Hash, GetTypeHash(Element));
	}
	return Hash;
}

} // namespace MIR

#endif // #if WITH_EDITOR
