// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Shader/ShaderTypes.h"

struct FExpressionInput;
struct FExpressionOutput;
class UMaterialExpression;

/**
 * @brief A structure that holds reflection information about a material.
 * This structure is typically populated by the material translator as a side product of
 * the translation process itself.
 *
 * You can use these insights for things like providing semantic colouring
 * of the graph UI or accurately knowing what resources are referenced by the translated
 * materials.
 */
struct FMaterialInsights
{
    /**
     * @brief Nested structure that represents a single connection insight.
     */
    struct FConnectionInsight
    {
        const UObject* InputObject;                   ///< Pointer to the input object of the connection.
        const UMaterialExpression* OutputExpression;  ///< Pointer to the output expression of the connection.
        int InputIndex;                               ///< Index of the input in the connection.
        int OutputIndex;                              ///< Index of the output in the connection.
        UE::Shader::EValueType ValueType;             ///< Type of the value flowing through the connection.
    };

	enum FUniformBufferSlotComponentType {
		CT_Unused,
		CT_Int,
		CT_Float,
		CT_LWC,
	};

	/**
	* @brief Wraps information about a uniform parameter (ScalarParametr, VectorParameter) allocation in
	* the uniform expression data buffer.
	* An instance of this struct specifies wheere the components of a uniform parameter in the material were
	* allocated in the preshader uniform buffer, that is which vec4 slot and which target components of that vec4.
	*/
	struct FUniformParameterAllocationInsight
	{
		uint16 BufferSlotIndex;								///< Index of the preshader buffer constant float4 slot (e.g. PreshaderBuffer[0])
		uint16 BufferSlotOffset;							///< First component of the float4 above (e.g. 0 for .x, 1 for .y, etc)
		uint16 ComponentsCount;								///< Number of components stored, same for the Uniform and the Parameter
		FUniformBufferSlotComponentType ComponentType : 8;  ///< Parameter component type
		FName ParameterName;								///< The parameter name
	};

	/// Array of connection insights.
    TArray<FConnectionInsight> ConnectionInsights;
	
	/// Array of parameter allocation insights.
    TArray<FUniformParameterAllocationInsight> UniformParameterAllocationInsights;

	/// String of the IR after translation.
	FString IRString;

	/// HLSL template string parameters for the legacy translator
	TMap<FString, FString> Legacy_ShaderStringParameters;

	/// HLSL full legacy generated source
	FString LegacyHLSLCode;

	/// HLSL template string parameters for the new translator
	TMap<FString, FString> New_ShaderStringParameters;

	/// HLSL full new generate source
	FString NewHLSLCode;

    FMaterialInsights();

	/// Clears all insight data.
    void Empty();
};