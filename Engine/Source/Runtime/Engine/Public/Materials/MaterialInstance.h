// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialOverrideNanite.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"

#include "MaterialInstance.generated.h"

class ITargetPlatform;
class UPhysicalMaterial;
class USubsurfaceProfile;
class UTexture;
class UTextureCollection;

//
// Forward declarations.
//
class FMaterialShaderMap;
class FMaterialShaderMapId;
class FMaterialUpdateContext;
class FSHAHash;

/** Editable scalar parameter. */

USTRUCT()
struct FScalarParameterAtlasInstanceData
{
	GENERATED_BODY()
public:
	UPROPERTY()
	bool bIsUsedAsAtlasPosition;

	UPROPERTY()
	TSoftObjectPtr<class UCurveLinearColor> Curve;

	UPROPERTY()
	TSoftObjectPtr<class UCurveLinearColorAtlas> Atlas;

	bool operator==(const FScalarParameterAtlasInstanceData& Other) const
	{
		return
			bIsUsedAsAtlasPosition == Other.bIsUsedAsAtlasPosition &&
			Curve == Other.Curve &&
			Atlas == Other.Atlas;
	}
	bool operator!=(const FScalarParameterAtlasInstanceData& Other) const
	{
		return !((*this) == Other);
	}

	FScalarParameterAtlasInstanceData()
		: bIsUsedAsAtlasPosition(false)
	{
	}
};

USTRUCT(BlueprintType)
struct FScalarParameterValue
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName ParameterName_DEPRECATED;

	UPROPERTY()
	FScalarParameterAtlasInstanceData AtlasData;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ScalarParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ScalarParameterValue)
	float ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FScalarParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), float InValue = 0.0f, const FScalarParameterAtlasInstanceData& InAtlasData = FScalarParameterAtlasInstanceData())
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
#if WITH_EDITORONLY_DATA
		AtlasData = InAtlasData;
#endif
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return true; }

	bool operator==(const FScalarParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FScalarParameterValue& Other) const
	{
		return !((*this) == Other);
	}
	
	typedef float ValueType;
	static ValueType GetValue(const FScalarParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
		OutResult.ScalarCurve = AtlasData.Curve;
		OutResult.ScalarAtlas = AtlasData.Atlas;
		OutResult.bUsedAsAtlasPosition = AtlasData.bIsUsedAsAtlasPosition;
#endif // WITH_EDITORONLY_DATA
	}
};

/** Editable vector parameter. */
USTRUCT(BlueprintType)
struct FVectorParameterValue
{
	GENERATED_USTRUCT_BODY()
		
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName ParameterName_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VectorParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=VectorParameterValue)
	FLinearColor ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FVectorParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), const FLinearColor& InValue = FLinearColor(ForceInit))
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return true; }

	bool operator==(const FVectorParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FVectorParameterValue& Other) const
	{
		return !((*this) == Other);
	}
	
	typedef FLinearColor ValueType;
	static ValueType GetValue(const FVectorParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable vector parameter. */
USTRUCT(BlueprintType)
struct FDoubleVectorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VectorParameterValue)
	FMaterialParameterInfo ParameterInfo;

	// LWC_TODO: Blueprint?
	UPROPERTY(EditAnywhere, Category = VectorParameterValue)
	FVector4d ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FDoubleVectorParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), const FVector4d& InValue = FVector4d(0.f, 0.f, 0.f, 1.f))
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return true; }

	bool operator==(const FDoubleVectorParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FDoubleVectorParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef FVector4d ValueType;
	static ValueType GetValue(const FDoubleVectorParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable texture parameter. */
USTRUCT(BlueprintType)
struct FTextureParameterValue
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName ParameterName_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureParameterValue)
	TObjectPtr<class UTexture> ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FTextureParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), class UTexture* InValue = nullptr)
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FTextureParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FTextureParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef const UTexture* ValueType;
	static ValueType GetValue(const FTextureParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable texture collection parameter. */
USTRUCT(BlueprintType)
struct FTextureCollectionParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollectionParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TextureCollectionParameterValue)
	TObjectPtr<UTextureCollection> ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FTextureCollectionParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), UTextureCollection* InValue = nullptr)
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FTextureCollectionParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FTextureCollectionParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef const UTextureCollection* ValueType;
	static ValueType GetValue(const FTextureCollectionParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable parameter collection parameter. */
USTRUCT(BlueprintType)
struct FParameterCollectionParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParameterCollectionParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=ParameterCollectionParameterValue)
	TObjectPtr<UMaterialParameterCollection> ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FParameterCollectionParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), UMaterialParameterCollection* InValue = nullptr)
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FParameterCollectionParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FParameterCollectionParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef const UMaterialParameterCollection* ValueType;
	static ValueType GetValue(const FParameterCollectionParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable runtime virtual texture parameter. */
USTRUCT(BlueprintType)
struct FRuntimeVirtualTextureParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeVirtualTextureParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeVirtualTextureParameterValue)
	TObjectPtr<class URuntimeVirtualTexture> ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FRuntimeVirtualTextureParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), class URuntimeVirtualTexture* InValue = nullptr)
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FRuntimeVirtualTextureParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FRuntimeVirtualTextureParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef const URuntimeVirtualTexture* ValueType;
	static ValueType GetValue(const FRuntimeVirtualTextureParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable sparse volume texture parameter. */
USTRUCT(BlueprintType)
struct FSparseVolumeTextureParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeVirtualTextureParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeVirtualTextureParameterValue)
	TObjectPtr<class USparseVolumeTexture> ParameterValue;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FSparseVolumeTextureParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), class USparseVolumeTexture* InValue = nullptr)
		: ParameterInfo(InParameterInfo), ParameterValue(InValue)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FSparseVolumeTextureParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			ParameterValue == Other.ParameterValue &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FSparseVolumeTextureParameterValue& Other) const
	{
		return !((*this) == Other);
	}

	typedef const USparseVolumeTexture* ValueType;
	static ValueType GetValue(const FSparseVolumeTextureParameterValue& Parameter) { return Parameter.ParameterValue; }

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = ParameterValue;
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

/** Editable font parameter. */
USTRUCT(BlueprintType)
struct FFontParameterValue
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName ParameterName_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontParameterValue)
	TObjectPtr<class UFont> FontValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=FontParameterValue)
	int32 FontPage;

	UPROPERTY()
	FGuid ExpressionGUID;

	explicit FFontParameterValue(const FMaterialParameterInfo& InParameterInfo = FMaterialParameterInfo(), class UFont* InFont = nullptr, int32 InPage = 0)
		: ParameterInfo(InParameterInfo), FontValue(InFont), FontPage(InPage)
	{
	}

	bool IsOverride() const { return true; }

	bool IsValid() const { return GetValue(*this) != nullptr; }

	bool operator==(const FFontParameterValue& Other) const
	{
		return
			ParameterInfo == Other.ParameterInfo &&
			FontValue == Other.FontValue &&
			FontPage == Other.FontPage &&
			ExpressionGUID == Other.ExpressionGUID;
	}
	bool operator!=(const FFontParameterValue& Other) const
	{
		return !((*this) == Other);
	}
	
	typedef const UTexture* ValueType;
	static ValueType GetValue(const FFontParameterValue& Parameter);

	void GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Value = FMaterialParameterValue(FontValue, FontPage);
#if WITH_EDITORONLY_DATA
		OutResult.ExpressionGuid = ExpressionGUID;
#endif
	}
};

USTRUCT(BlueprintType)
struct FUserSceneTextureOverride
{
	GENERATED_USTRUCT_BODY()
		
	/** Key value of NONE represents override of UserSceneTexture output */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=UserSceneTextureOverrideValue)
	FName Key;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=UserSceneTextureOverrideValue)
	FName Value;
};

struct FPostProcessBlendableOverrides
{
	bool bOverrideBlendableLocation = false;
	bool bOverrideBlendablePriority = false;
	TEnumAsByte<EBlendableLocation> BlendableLocationOverride = BL_SceneColorAfterTonemapping;
	int32 BlendablePriorityOverride = 0;
};

template<class T>
bool CompareValueArraysByExpressionGUID(const TArray<T>& InA, const TArray<T>& InB)
{
	if (InA.Num() != InB.Num())
	{
		return false;
	}
	if (!InA.Num())
	{
		return true;
	}
	TArray<T> AA(InA);
	TArray<T> BB(InB);
	AA.Sort([](const T& A, const T& B) { return B.ExpressionGUID < A.ExpressionGUID; });
	BB.Sort([](const T& A, const T& B) { return B.ExpressionGUID < A.ExpressionGUID; });
	return AA == BB;
}

USTRUCT()
struct FMaterialInstanceCachedData
{
	GENERATED_USTRUCT_BODY()

	ENGINE_API static const FMaterialInstanceCachedData EmptyData;

#if WITH_EDITOR
	void InitializeForConstant(const FMaterialLayersFunctions* Layers, const FMaterialLayersFunctions* ParentLayers);
#endif // WITH_EDITOR
	void InitializeForDynamic(const FMaterialLayersFunctions* ParentLayers);

	UPROPERTY()
	TArray<int32> ParentLayerIndexRemap;
};

enum class EMaterialInstanceClearParameterFlag
{
	None = 0u,
	Numeric = (1u << 0),
	Texture = (1u << 1),
	Static = (1u << 2),

	AllNonStatic = Numeric | Texture,
	All = AllNonStatic | Static,
};
ENUM_CLASS_FLAGS(EMaterialInstanceClearParameterFlag);

#if WITH_EDITORONLY_DATA
class FMaterialInstanceParameterUpdateContext
{
public:
	ENGINE_API explicit FMaterialInstanceParameterUpdateContext(UMaterialInstance* InInstance, EMaterialInstanceClearParameterFlag ClearFlags = EMaterialInstanceClearParameterFlag::None);
	ENGINE_API ~FMaterialInstanceParameterUpdateContext();

	inline FStaticParameterSet& GetStaticParameters() { return StaticParameters; }

	ENGINE_API void SetParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags = EMaterialSetParameterValueFlags::None);
	ENGINE_API void SetForceStaticPermutationUpdate(bool bValue);
	ENGINE_API void SetBasePropertyOverrides(const FMaterialInstanceBasePropertyOverrides& InValue);
	ENGINE_API void SetMaterialLayers(const FMaterialLayersFunctions& InValue);

private:
	UMaterialInstance* Instance;
	FStaticParameterSet StaticParameters;
	FMaterialInstanceBasePropertyOverrides BasePropertyOverrides;
	bool bForceStaticPermutationUpdate;
};
#endif // WITH_EDITORONLY_DATA

UCLASS(MinimalAPI, Optional)
class UMaterialInstanceEditorOnlyData : public UMaterialInterfaceEditorOnlyData
{
	GENERATED_BODY()
public:
	UPROPERTY()
	FStaticParameterSetEditorOnlyData StaticParameters;
};

UCLASS(abstract, BlueprintType,MinimalAPI)
class UMaterialInstance : public UMaterialInterface
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	virtual const UClass* GetEditorOnlyDataClass() const override { return UMaterialInstanceEditorOnlyData::StaticClass(); }

	virtual UMaterialInstanceEditorOnlyData* GetEditorOnlyData() override { return CastChecked<UMaterialInstanceEditorOnlyData>(Super::GetEditorOnlyData(), ECastCheckedType::NullAllowed); }
	virtual const UMaterialInstanceEditorOnlyData* GetEditorOnlyData() const override { return CastChecked<UMaterialInstanceEditorOnlyData>(Super::GetEditorOnlyData(), ECastCheckedType::NullAllowed); }
#endif // WITH_EDITORONLY_DATA

	/** Physical material to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialInstance)
	TObjectPtr<class UPhysicalMaterial> PhysMaterial;

	/** Physical material map used with physical material mask, when it exists.*/
	UPROPERTY(EditAnywhere, Category = PhysicalMaterialMask)
	TObjectPtr<class UPhysicalMaterial> PhysicalMaterialMap[EPhysicalMaterialMaskColor::MAX];

	/** Parent material. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, AssetRegistrySearchable)
	TObjectPtr<class UMaterialInterface> Parent;

	/** An override material which will be used instead of this one when rendering with Nanite. */
	UPROPERTY(EditAnywhere, Category = MaterialInstance)
	FMaterialOverrideNanite NaniteOverrideMaterial;

#if WITH_EDITORONLY_DATA
	ENGINE_API const FStaticParameterSetEditorOnlyData& GetEditorOnlyStaticParameters() const;
#endif // WITH_EDITORONLY_DATA

	/**
	 * Gets static parameter set for this material.
	 *
	 * @returns Static parameter set.
	 */
	ENGINE_API bool HasStaticParameters() const;
	ENGINE_API FStaticParameterSet GetStaticParameters() const;

	const FMaterialInstanceCachedData& GetCachedInstanceData() const { return CachedData ? *CachedData : FMaterialInstanceCachedData::EmptyData; }

	/** Defines SpecularProfile override used by this instance. If not set, it uses the parent ones. */
	UPROPERTY()
	TObjectPtr<class USpecularProfile> SpecularProfileOverride;

	/** Defines ToonProfile override used by this instance. If not set, it uses the parent ones. */
	UPROPERTY()
	TObjectPtr<class UToonProfile> ToonProfileOverride;

	/**
	 * Indicates whether the instance has static permutation resources (which are required when static parameters are present) 
	 * Read directly from the rendering thread, can only be modified with the use of a FMaterialUpdateContext.
	 * When true, StaticPermutationMaterialResources will always be valid and non-null.
	 */
	UPROPERTY()
	uint8 bHasStaticPermutationResource:1;

	/** Defines if SubsurfaceProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance)
	uint8 bOverrideSubsurfaceProfile:1;

	/** Defines if SpecularProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance)
	uint8 bOverrideSpecularProfile:1;

	/** Defines if ToonProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance)
	uint8 bOverrideToonProfile:1;

	/** For post process materials, use BlendableLocationOverride. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialInstance)
	uint8 bOverrideBlendableLocation:1;

	/** For post process materials, use BlendablePriorityOverride. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialInstance)
	uint8 bOverrideBlendablePriority : 1;

	/** Defines if PhysMaterial from this instance is used or if the parent one is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialInstance)
	uint8 bOverridePhysMaterial : 1;

	uint8 TwoSided : 1;
	uint8 bIsThinSurface : 1;
	uint8 DitheredLODTransition : 1;
	uint8 bCastDynamicShadowAsMasked : 1;
	uint8 bOutputTranslucentVelocity : 1;
	uint8 bIsShadingModelFromMaterialExpression : 1;
	uint8 bHasPixelAnimation : 1;
	uint8 bEnableTessellation : 1;
	uint8 bEnableDisplacementFade : 1;
	uint8 bCompatibleWithLumenCardSharing : 1;
protected:
	uint8 bLoadedCachedData : 1;

public:
	TEnumAsByte<EBlendMode> BlendMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialInstance)
	TEnumAsByte<EBlendableLocation> BlendableLocationOverride;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialInstance)
	int32 BlendablePriorityOverride;

	FMaterialShadingModelField ShadingModels;

private:
	/** This instance resource has been created and is still alive (the render thread hasn't destroyed it yet). */
	mutable bool bResourceCreated = false;
	/** This instance is caching uniform expressions (the render thread hasn't canceled its creation). */
	mutable bool bCachingUniformExpressions = false;

public:
	//Cached copies of the base property overrides or the value from the parent to avoid traversing the parent chain for each access.
	float OpacityMaskClipValue;

	FDisplacementScaling DisplacementScaling;
	FDisplacementFadeRange DisplacementFadeRange;

	float MaxWorldPositionOffsetDisplacement;

protected:
	uint32 UsageFlags = 0;

public:
	inline bool GetReentrantFlag(bool bIsInGameThread = IsInGameThread()) const
	{
#if WITH_EDITOR
		if (!IsInParallelGameThread())
		{
			return ReentrantFlag[bIsInGameThread ? 0 : 1];
		}
#endif
		return false;
	}

	inline void SetReentrantFlag(const bool bValue, bool bIsInGameThread = IsInGameThread())
	{
#if WITH_EDITOR
		if (!IsInParallelGameThread())
		{
			ReentrantFlag[bIsInGameThread ? 0 : 1] = bValue;
		}
#endif
	}

	/** Scalar parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FScalarParameterValue> ScalarParameterValues;

	/** Vector parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FVectorParameterValue> VectorParameterValues;

	/** DoubleVector parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FDoubleVectorParameterValue> DoubleVectorParameterValues;

	/** Texture parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FTextureParameterValue> TextureParameterValues;

	/** Texture Collection parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FTextureCollectionParameterValue> TextureCollectionParameterValues;

	/** Texture Collection parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FParameterCollectionParameterValue> ParameterCollectionParameterValues;

	/** RuntimeVirtualTexture parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FRuntimeVirtualTextureParameterValue> RuntimeVirtualTextureParameterValues;

	/** Sparse Volume Texture parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FSparseVolumeTextureParameterValue> SparseVolumeTextureParameterValues;

	/** Font parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MaterialInstance, meta = (EditFixedOrder))
	TArray<struct FFontParameterValue> FontParameterValues;
	
	/** User scene texture overrides.  Applies to post process domain materials only. */
	UPROPERTY(EditAnywhere, Category = MaterialInstance)
	TArray<struct FUserSceneTextureOverride> UserSceneTextureOverrides;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bOverrideBaseProperties_DEPRECATED;

	/** Array of enumeration objects for use by scalar parameter enumeration indices. */ 
	UPROPERTY()
	TArray<TSoftObjectPtr<UObject>> EnumerationObjects;
#endif

	UPROPERTY(EditAnywhere, Category=MaterialInstance)
	struct FMaterialInstanceBasePropertyOverrides BasePropertyOverrides;

#if WITH_EDITOR
	/** Flag to detect cycles in the material instance graph, this is only used at content creation time where the hierarchy can be changed. */
	bool ReentrantFlag[2];

	/** Whether static parameter permutations should be disabled (e.g. due to having a restricted parent material) */
	bool bDisallowStaticParameterPermutations;
#endif

	UE_DEPRECATED(5.8, "Exposed public Instance Render Proxy derivative has been moved to protected. Please use either GetRenderProxy or GetInstanceRenderProxy.")
	class FMaterialInstanceRenderProxy* Resource;

#if WITH_EDITOR
	ENGINE_API virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif
	ENGINE_API virtual float GetTextureDensity(FName TextureName, const struct FMeshUVChannelInfo& UVChannelData) const override;
	ENGINE_API virtual bool CalculateTexturesSamplingInfo(UTexture* Texture, FTextureSamplingInfo& OutTextureSamplingInfo) const override;


	ENGINE_API bool Equivalent(const UMaterialInstance* CompareTo) const;

	// Returns true if using this material instance would be equivalent to using its parent. 
	// i.e. no parameters/etc of any kind have been set.
	ENGINE_API virtual bool IsRedundant() const;

private:

	/** Static parameter values that are overridden in this instance. */
	UPROPERTY()
	FStaticParameterSetRuntimeData StaticParametersRuntime;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FGuid> ReferencedTextureGuids;

	UPROPERTY()
	FStaticParameterSet StaticParameters_DEPRECATED;

	UPROPERTY()
	bool bSavedCachedData_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

protected:
	/**
	 * FMaterialRenderProxy derivative that represent this material instance to the renderer, when the renderer needs to fetch parameter values.
	 */
	class FMaterialInstanceRenderProxy* RenderProxy;

	TUniquePtr<FMaterialInstanceCachedData> CachedData;

private:
	/** Inline material resources serialized from disk. To be processed on game thread in PostLoad. */
	TArray<FMaterialResource> LoadedMaterialResources;

	/** 
	 * Material resources used for rendering this material instance, in the case of static parameters being present.
	 * These will always be valid and non-null when bHasStaticPermutationResource is true,
	 * But only the entries affected by CacheResourceShadersForRendering will be valid for rendering.
	 * There need to be as many entries in this array as can be used simultaneously for rendering.  
	 * For example the material instance needs to support being rendered at different quality levels and feature levels within the same process.
	 */
	TArray<FMaterialResource*> StaticPermutationMaterialResources;

#if WITH_EDITOR
	struct FTextureParameterOverride
	{
		FTextureParameterOverride(UTexture* InPreviousTexture, UTexture* InOverrideTexture)
			: PreviousTexture(InPreviousTexture)
			, OverrideTexture(InOverrideTexture)
		{}

		TObjectPtr<UTexture> PreviousTexture;
		TObjectPtr<UTexture> OverrideTexture;
	};

	TArray<FTextureParameterOverride> TransientTextureParameterOverrides;

	// Helper for setting an override texture
	void OverrideTextureParameterValue(const UTexture* InTextureToOverride, UTexture* OverrideTexture);
	TObjectPtr<UTexture> OverrideTextureParameterValueInternal(const UTexture* InTextureToOverride, UTexture* OverrideTexture);
	void ResetAllTextureParameterOverrides();
#endif // WITH_EDITOR
public:
	virtual ENGINE_API ~UMaterialInstance();

	// Begin UMaterialInterface interface.
	virtual ENGINE_API UMaterial* GetMaterial() override;
	virtual ENGINE_API const UMaterial* GetMaterial() const override;
	virtual ENGINE_API const UMaterial* GetMaterial_Concurrent(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	virtual ENGINE_API void GetMaterialInheritanceChain(FMaterialInheritanceChain& OutChain) const override;
	virtual ENGINE_API const FMaterialCachedExpressionData& GetCachedExpressionData(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	virtual ENGINE_API FMaterialResource* AllocatePermutationResource();
	virtual ENGINE_API FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) override;
	virtual ENGINE_API const FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) const override;

	ENGINE_API bool GetParameterOverrideValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue) const;
	virtual ENGINE_API bool GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue, EMaterialGetParameterValueFlags Flags = EMaterialGetParameterValueFlags::Default) const override;

	virtual ENGINE_API void GetUsedTextures(TArray<UTexture*>& OutTextures, TOptional<EMaterialQualityLevel::Type> InSpecificQualityLevel = TOptional<EMaterialQualityLevel::Type>(), TOptional<EShaderPlatform> InSpecificShaderPlatform = TOptional<EShaderPlatform>()) const override;
	virtual ENGINE_API void GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const;
	virtual ENGINE_API bool GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue, bool bOveriddenOnly = false) const override;
	virtual ENGINE_API bool GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue, bool bOveriddenOnly = false) const override;
	virtual ENGINE_API void OverrideTexture(const UTexture* InTextureToOverride, UTexture* OverrideTexture, ERHIFeatureLevel::Type InFeatureLevel) override;
	virtual ENGINE_API void SetNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value,EShaderPlatform InShaderPlatform) override;
	virtual ENGINE_API void ClearNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, EShaderPlatform InShaderPlatform) override;
	virtual ENGINE_API uint32 GetUsageFlags() const override;
	virtual ENGINE_API bool GetUsageByFlag(const EMaterialUsage Usage) const override;
	virtual ENGINE_API bool SetMaterialUsage(EMaterialUsage Usage) override;
	virtual ENGINE_API bool GetMaterialLayers(FMaterialLayersFunctions& OutLayers, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	virtual ENGINE_API bool IsDependent(UMaterialInterface* TestDependency) override;
	virtual ENGINE_API bool IsDependent_Concurrent(UMaterialInterface* TestDependency, TMicRecursionGuard RecursionGuard) override;
	virtual ENGINE_API void GetDependencies(TSet<UMaterialInterface*>& Dependencies) override;

	virtual ENGINE_API FMaterialRenderProxy* GetRenderProxy() const override;
	inline FMaterialInstanceRenderProxy* GetInstanceRenderProxy() const { return RenderProxy; }

	virtual ENGINE_API UPhysicalMaterial* GetPhysicalMaterial() const override;
	virtual ENGINE_API UPhysicalMaterialMask* GetPhysicalMaterialMask() const override;
	virtual ENGINE_API UPhysicalMaterial* GetPhysicalMaterialFromMap(int32 Index) const override;
	virtual ENGINE_API UMaterialInterface* GetNaniteOverride(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const override;
	virtual ENGINE_API bool UpdateLightmassTextureTracking() override;
	virtual ENGINE_API bool GetCastShadowAsMasked() const override;
	virtual ENGINE_API float GetEmissiveBoost() const override;
	virtual ENGINE_API float GetDiffuseBoost() const override;
	virtual ENGINE_API float GetExportResolutionScale() const override;
#if WITH_EDITOR
	virtual ENGINE_API bool GetGroupSortPriority(const FString& InGroupName, int32& OutSortPriority) const override;
	virtual ENGINE_API bool GetTexturesInPropertyChain(EMaterialProperty InProperty, TArray<UTexture*>& OutTextures,
		TArray<FName>* OutTextureParamNames, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type InQuality) override;
	ENGINE_API bool SetMaterialLayers(const FMaterialLayersFunctions& LayersValue);
#endif
	virtual ENGINE_API void RecacheUniformExpressions(bool bRecreateUniformBuffer) const override;
	virtual ENGINE_API bool GetRefractionSettings(float& OutBiasValue) const override;
	virtual ENGINE_API bool GetUserSceneTextureOverride(FName& InOutName) const override;
	virtual ENGINE_API EBlendableLocation GetBlendableLocation(const UMaterial* Base) const override;
	virtual ENGINE_API int32 GetBlendablePriority(const UMaterial* Base) const override;

	virtual ENGINE_API FGraphEventArray PrecachePSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs) override;


#if WITH_EDITOR
	ENGINE_API virtual void ForceRecompileForRendering(EMaterialShaderPrecompileMode CompileMode = EMaterialShaderPrecompileMode::Default) override;
#endif // WITH_EDITOR

	ENGINE_API virtual float GetOpacityMaskClipValue() const override;
	ENGINE_API virtual bool GetCastDynamicShadowAsMasked() const override;
	ENGINE_API virtual EBlendMode GetBlendMode() const override;
	ENGINE_API virtual FMaterialShadingModelField GetShadingModels() const override;
	ENGINE_API virtual bool IsShadingModelFromMaterialExpression() const override;
	ENGINE_API virtual bool IsTwoSided() const override;
	ENGINE_API virtual bool IsThinSurface() const override;
	ENGINE_API virtual bool IsTranslucencyWritingVelocity() const override;
	ENGINE_API virtual bool IsTranslucencyVelocityFromDepth() const override;
	ENGINE_API virtual bool IsDitheredLODTransition() const override;
	ENGINE_API virtual bool IsMasked() const override;
	ENGINE_API virtual bool WritesToRuntimeVirtualTexture() const override;
	ENGINE_API virtual bool HasMeshPaintTexture() const override;
	ENGINE_API virtual bool HasCustomPrimitiveData() const override;
	ENGINE_API virtual FDisplacementScaling GetDisplacementScaling() const override;
	ENGINE_API virtual bool IsDisplacementFadeEnabled() const override;
	ENGINE_API virtual FDisplacementFadeRange GetDisplacementFadeRange() const override;
	ENGINE_API virtual float GetMaxWorldPositionOffsetDisplacement() const override;
	ENGINE_API virtual bool ShouldAlwaysEvaluateWorldPositionOffset() const override;
	ENGINE_API virtual bool IsDeferredDecal() const override;
	ENGINE_API virtual bool IsUIMaterial() const override;
	ENGINE_API virtual bool IsPostProcessMaterial() const override;
	ENGINE_API virtual bool IsLightFunctionMaterial() const override;
	ENGINE_API virtual bool HasVertexInterpolator() const override;
	ENGINE_API virtual bool HasCustomizedUVs() const override;
	ENGINE_API virtual bool HasPixelAnimation() const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfile_Internal() const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal() const override;
	ENGINE_API uint32 NumSubsurfaceProfileRoot_Internal() const override;
	ENGINE_API USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal(uint32 Index) const override;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileOverride_Internal() const override;
	ENGINE_API virtual uint32 NumSpecularProfile_Internal() const override;
	ENGINE_API virtual USpecularProfile* GetSpecularProfile_Internal(uint32 Index) const override;
	ENGINE_API virtual USpecularProfile* GetSpecularProfileOverride_Internal() const override;
	ENGINE_API virtual uint32 NumToonProfile_Internal() const override;
	ENGINE_API virtual UToonProfile* GetToonProfile_Internal(uint32 Index) const override;
	ENGINE_API virtual UToonProfile* GetToonProfileOverride_Internal() const override;
	ENGINE_API virtual bool CastsRayTracedShadows() const override;
	ENGINE_API virtual bool IsTessellationEnabled() const override;
	ENGINE_API virtual bool HasSubstrateRoughnessTracking() const override;
	ENGINE_API virtual bool IsCompatibleWithLumenCardSharing() const override;

	/** Checks to see if an input property should be active, based on the state of the material */
	ENGINE_API virtual bool IsPropertyActive(EMaterialProperty InProperty) const override;
#if WITH_EDITOR
	/** Allows material properties to be compiled with the option of being overridden by the material attributes input. */
	ENGINE_API virtual int32 CompilePropertyEx(class FMaterialCompiler* Compiler, const FGuid& AttributeID) override;

	/* returns whether valid compilation output exists. Outputs all default textures used by the material, based on the compilation output. */
	ENGINE_API virtual bool TryGetUsedTexturesInCookCache(TArray<TObjectPtr<UObject>>& OutTextures) override;

#endif // WITH_EDITOR
	//~ End UMaterialInterface Interface.

	//~ Begin UObject Interface.
	virtual ENGINE_API void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual ENGINE_API void PostInitProperties() override;	
#if WITH_EDITOR
	virtual ENGINE_API void BeginCacheForCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
	virtual ENGINE_API bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
	virtual ENGINE_API void ClearCachedCookedPlatformData(const ITargetPlatform *TargetPlatform) override;
	virtual ENGINE_API void ClearAllCachedCookedPlatformData() override;

	virtual ENGINE_API void AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const override;
#endif
	virtual ENGINE_API void Serialize(FArchive& Ar) override;
	virtual ENGINE_API void PostDuplicate(bool bDuplicateForPIE) override;
	virtual ENGINE_API void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual ENGINE_API void BeginDestroy() override;
	virtual ENGINE_API bool IsReadyForFinishDestroy() override;
	virtual ENGINE_API void FinishDestroy() override;
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#if WITH_EDITOR
	virtual ENGINE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual ENGINE_API void PostEditUndo() override;

	/**
	 * Sets new static parameter overrides on the instance and recompiles the static permutation resources if needed (can be forced with bForceRecompile).
	 * Can be passed either a minimal parameter set (overridden parameters only) or the entire set generated by GetStaticParameterValues().
	 * Can also trigger recompile based on new set of FMaterialInstanceBasePropertyOverrides 
	 */
	ENGINE_API void UpdateStaticPermutation(const FStaticParameterSet& NewParameters, FMaterialInstanceBasePropertyOverrides& NewBasePropertyOverrides, const bool bForceStaticPermutationUpdate = false, FMaterialUpdateContext* MaterialUpdateContext = nullptr);
	/** Updates all default textures of this material instance, based on permutations and Parent Material. */
	ENGINE_API void UpdateReferencedDefaultTextures();

	/**
	* Set just the permutation parameters, without triggering any shader compilation.
	* This is useful for operations which require the Material Instance to have the right data
	* but it will be invalid for rendering until UpdateStaticPermutation() is called.
	*/
	ENGINE_API void SetPermutationParameters(const FStaticParameterSet& InParameters, FMaterialInstanceBasePropertyOverrides& InBasePropertyOverrides);

	/**
	* Sets new static parameter overrides on the instance and recompiles the static permutation resources if needed.
	* Can be passed either a minimal parameter set (overridden parameters only) or the entire set generated by GetStaticParameterValues().
	*/
	ENGINE_API void UpdateStaticPermutation(const FStaticParameterSet& NewParameters, FMaterialUpdateContext* MaterialUpdateContext = nullptr);
	/** Ensure's static permutations for current parameters and overrides are upto date. */
	ENGINE_API void UpdateStaticPermutation(FMaterialUpdateContext* MaterialUpdateContext = nullptr);

	ENGINE_API void SwapLayerParameterIndices(int32 OriginalIndex, int32 NewIndex);
	ENGINE_API void RemoveLayerParameterIndex(int32 Index);
	
	/**
	 * Returns this material instance is allowed to override static parameters and introduce a new shader permutation when it derives from `CandidateParent`.
	 */
	ENGINE_API bool IsStaticPermutationAllowedForCandidateParent(UMaterialInterface* CandidateParent) const;

	/**
	 * Ensures that current parent is a valid for this material instance and if not, it resets the parent to null.
	 */
	ENGINE_API void ValidateStaticPermutationAllowed();

#endif // WITH_EDITOR

	//~ End UObject Interface.

	/**
	 * Recompiles static permutations if necessary.
	 * Note: This modifies material variables used for rendering and is assumed to be called within a FMaterialUpdateContext!
	 */
	ENGINE_API void InitStaticPermutation(EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default);

	ENGINE_API void UpdateOverridableBaseProperties();

#if WITH_EDITOR
	/** 
	 * Cache resource shaders for rendering on the given shader platform. 
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 * The results will be applied to this FMaterial in the renderer when they are finished compiling.
	 * Note: This modifies material variables used for rendering and is assumed to be called within a FMaterialUpdateContext!
	 */
	void CacheResourceShadersForCooking(
		EShaderPlatform ShaderPlatform,
		TArray<FMaterialResourceForCooking>& OutCachedMaterialResources,
		EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default,
		const ITargetPlatform* TargetPlatform = nullptr,
		bool bBlocking = false
	);
#endif

	/** 
	 * Gathers actively used shader maps from all material resources used by this material instance
	 * Note - not refcounting the shader maps so the references must not be used after material resources are modified (compilation, loading, etc)
	 */
	void GetAllShaderMaps(TArray<FMaterialShaderMap*>& OutShaderMaps);

#if WITH_EDITORONLY_DATA

	ENGINE_API void SetStaticSwitchParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, bool Value);
	ENGINE_API void SetStaticComponentMaskParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FStaticComponentMaskValue Value);
#endif // WITH_EDITORONLY_DATA

	ENGINE_API virtual void GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const override;
#if WITH_EDITORONLY_DATA
	ENGINE_API virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const override;
	ENGINE_API virtual void GetDependentFunctions(TArray<class UMaterialFunctionInterface*>& DependentFunctions) const override;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** Get enumeration object by index. If nothing is found at the index we also look at parent chain until we find something. Returns a null pointer if nothing is found. */
	ENGINE_API TSoftObjectPtr<UObject> GetEnumerationObject(int32 Index) const;
#endif

#if WITH_EDITOR
	/** Add to the set any texture referenced by expressions, including nested functions, as well as any overrides from parameters. */
	ENGINE_API virtual void GetReferencedTexturesAndOverrides(TSet<const UTexture*>& InOutTextures) const;

	ENGINE_API virtual void UpdateCachedData();
#endif

	UE_DEPRECATED(5.8, "Use parameterless version returning a uint64; SHA1 hashes are no longer used")
	inline void GetBasePropertyOverridesHash(FSHAHash& OutHash) const
	{
		uint64 NewHash = GetBasePropertyOverridesHash();
		OutHash = FSHA1::HashBuffer(&NewHash, sizeof(NewHash));
	}
	ENGINE_API uint64 GetBasePropertyOverridesHash() const;
	ENGINE_API virtual bool HasOverridenBaseProperties() const;

#if WITH_EDITOR
	ENGINE_API FString GetBasePropertyOverrideString() const;
#endif

	// For all materials instances, UMaterialInstance::CacheResourceShadersForRendering
	ENGINE_API static void AllMaterialsCacheResourceShadersForRendering(bool bUpdateProgressDialog = false, bool bCacheAllRemainingShaders = true);

	/**
	 * Determine whether this Material Instance is a child of another Material
	 *
	 * @param	Material	Material to check against
	 * @return	true if this Material Instance is a child of the other Material.
	 */
	ENGINE_API bool IsChildOf(const UMaterialInterface* Material) const;


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/**
	 * Output to the log which materials and textures are used by this material.
	 * @param Indent	Number of tabs to put before the log.
	 */
	ENGINE_API virtual void LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const override;
#endif
	UE_DEPRECATED(5.7, "Please use GetMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API void ValidateTextureOverrides(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API void ValidateTextureOverrides(EShaderPlatform InShaderPlatform) const;

	/**
	 *	Returns all the Guids related to this material. For material instances, this includes the parent hierarchy.
	 *  Used for versioning as parent changes don't update the child instance Guids.
	 *
	 *	@param	bIncludeTextures	Whether to include the referenced texture Guids.
	 *	@param	OutGuids			The list of all resource guids affecting the precomputed lighting system and texture streamer.
	 */
	ENGINE_API virtual void GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const override;

	/**
	 * Looks through this Instance and all its parents for the relevant texture.
	 * At editor time, it looks through all default textures. 
	 * At runtime, it has access to default textures based on cook output, if ShouldStripUnusedDefaultTextures returns true.
	 * @return The default texture that is relevant to the given Idx or nullptr if not present.
	 */
	ENGINE_API virtual TObjectPtr<UObject> GetDefaultTextureAtIdx(const int32 InIdx, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) override;

	ENGINE_API virtual void DumpDebugInfo(FOutputDevice& OutputDevice) const override;
	void SaveShaderStableKeys(const class ITargetPlatform* TP);
	ENGINE_API virtual void SaveShaderStableKeysInner(const class ITargetPlatform* TP, const struct FStableShaderKeyAndValue& SaveKeyVal) override;

#if WITH_EDITOR
	ENGINE_API virtual void GetShaderTypes(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FDebugShaderTypeInfo>& OutShaderInfo) override;
#endif // WITH_EDITOR

#if WITH_EDITOR
	void BeginAllowCachingStaticParameterValues();
	void EndAllowCachingStaticParameterValues();
#endif // WITH_EDITOR

	ENGINE_API virtual void CacheShaders(EMaterialShaderPrecompileMode CompileMode) override;
#if WITH_EDITOR
	ENGINE_API virtual void CacheGivenTypesForCooking(EShaderPlatform Platform, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*> PipelineTypes, const TArray<const FShaderType*>& ShaderTypes) override;
#endif
	ENGINE_API virtual bool IsComplete() const override;

#if WITH_EDITOR
	ENGINE_API virtual bool IsCompiling() const override;
#endif

	/** Tracking of in-flight uniform expression cache update operations for the material instance, for thread safety destroying the resource. */
	void StartCacheUniformExpressions() const;
	void FinishCacheUniformExpressions() const;

protected:

	/**
	* Copies the uniform parameters (scalar, vector and texture) from a material or instance hierarchy.
	* This will typically be faster than parsing all expressions but still slow as it must walk the full
	* material hierarchy as each parameter may be overridden at any level in the chain.
	* Note: This will not copy static or font parameters
	*/
	void CopyMaterialUniformParametersInternal(UMaterialInterface* Source);

	/**
	 * Updates parameter names on the material instance, returns true if parameters have changed.
	 */
	bool UpdateParameters();

	/** Directly set the usage flags. Should only be called from within SetMaterialUsage(). */
	void SetUsageByFlag(EMaterialUsage Usage, bool NewValue);

	ENGINE_API bool SetParentInternal(class UMaterialInterface* NewParent, bool RecacheShaders);

	/** Returns the textures which are used by the Uniform Expressions of the given Material Resource, along with any existing transient overrides. */
	ENGINE_API virtual void GetTextureExpressionValues(const FMaterialResource* MaterialResource, TArray<UTexture*>& OutTextures, TArray<TArray<int32>>* OutIndices = nullptr, bool bIncludeTextureCollections = false, bool bAllowOverrides = false) const override;
	void GetTextureCollectionExpressionValues(const FMaterialResource* MaterialResource, TArray<UTextureCollection*>& OutTextureCollections) const;

#if WITH_EDITOR
	/**
	* Refresh parameter names using the stored reference to the expression object for the parameter.
	*/
	ENGINE_API void UpdateParameterNames();

#endif // WITH_EDITOR

	/**
	 * Internal interface for setting / updating values for material instances.
	 */
	void ReserveParameterValuesInternal(EMaterialParameterType Type, int32 Capacity);
	void AddParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags = EMaterialSetParameterValueFlags::None);
	void SetParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, const FMaterialParameterMetadata& Meta, EMaterialSetParameterValueFlags Flags = EMaterialSetParameterValueFlags::None);
	void SetVectorParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value);
	void SetDoubleVectorParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, FVector4d Value);
	bool SetVectorParameterByIndexInternal(int32 ParameterIndex, FLinearColor Value);
	bool SetScalarParameterByIndexInternal(int32 ParameterIndex, float Value);
	void SetScalarParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, float Value, bool bUseAtlas = false, FScalarParameterAtlasInstanceData AtlasData = FScalarParameterAtlasInstanceData());
#if WITH_EDITOR
	void SetScalarParameterAtlasInternal(const FMaterialParameterInfo& ParameterInfo, FScalarParameterAtlasInstanceData AtlasData);
#endif
	void SetTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class UTexture* Value);
	void SetTextureCollectionParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class UTextureCollection* Value);
	void SetParameterCollectionParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection* Value);
	void SetRuntimeVirtualTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture* Value);
	void SetSparseVolumeTextureParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture* Value);
	void SetFontParameterValueInternal(const FMaterialParameterInfo& ParameterInfo, class UFont* FontValue, int32 FontPage);
	void ClearParameterValuesInternal(EMaterialInstanceClearParameterFlag Flags = EMaterialInstanceClearParameterFlag::AllNonStatic);

	/** Initialize the material instance's resources. */
	ENGINE_API void InitResources();

	/** 
	 * Cache resource shaders for rendering on the given shader platform. 
	 * If a matching shader map is not found in memory or the DDC, a new one will be compiled.
	 * The results will be applied to this FMaterial in the renderer when they are finished compiling.
	 * Note: This modifies material variables used for rendering and is assumed to be called within a FMaterialUpdateContext!
	 */
	void CacheResourceShadersForRendering(EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default);
	void CacheResourceShadersForRendering(EMaterialShaderPrecompileMode PrecompileMode, FMaterialResourceDeferredDeletionArray& OutResourcesToFree);

	/** Caches shader maps for an array of material resources. */
	void CacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);

#if WITH_EDITOR
	/** Initiates caching for this shader resource that will be finished when each material resource IsCompilationFinished returns true. */
	void BeginCacheShadersForResources(EShaderPlatform ShaderPlatform, const TArray<FMaterialResource*>& ResourcesToCache, EMaterialShaderPrecompileMode PrecompileMode = EMaterialShaderPrecompileMode::Default, const ITargetPlatform* TargetPlatform = nullptr);
#endif

	/** 
	 * Copies over material instance parameters from the base material given a material interface.
	 * This is a slow operation that is needed for the editor.
	 * @param Source silently ignores the case if 0
	 */
	ENGINE_API void CopyMaterialInstanceParameters(UMaterialInterface* Source);

	// to share code between PostLoad() and PostEditChangeProperty()
	void PropagateDataToMaterialProxy();

	/** Allow resource to access private members. */
	friend class FMaterialInstanceRenderProxy;
	/** Editor-only access to private members. */
	friend class UMaterialEditingLibrary;
	/** Class that knows how to update MI's */
	friend class FMaterialUpdateContext;
	friend class FMaterialInstanceParameterUpdateContext;
};
