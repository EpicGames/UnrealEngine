// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 *	Utility class designed to allow you to connect a MaterialInterface to a Matinee action.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "MaterialInstanceActor.generated.h"

class UBillboardComponent;

UCLASS(hidecategories=Movement, hidecategories=Advanced, hidecategories=Collision, hidecategories=Display, hidecategories=Actor, hidecategories=Attachment, MinimalAPI)
class AMaterialInstanceActor : public AActor
{
	GENERATED_UCLASS_BODY()

	/** Pointer to actors that we want to control paramters of using Matinee. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MaterialInstanceActor)
	TArray<TObjectPtr<class AActor>> TargetActors;

#if WITH_EDITORONLY_DATA
private:
	// Reference to actor sprite
	UBillboardComponent* SpriteComponent;
#endif

public:

	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	ENGINE_API UBillboardComponent* GetSpriteComponent() const;
#endif
};



