// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "MaterialInstanceBasePropertyOverrides.generated.h"

/** Properties from the base material that can be overridden in material instances. */
USTRUCT()
struct FMaterialInstanceBasePropertyOverrides
{
	GENERATED_USTRUCT_BODY()

	/** Enables override of the opacity mask clip value. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_OpacityMaskClipValue : 1;

	/** Enables override of the blend mode. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_BlendMode : 1;

	/** Enables override of the shading model. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_ShadingModel : 1;

	/** Enables override of the dithered LOD transition property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_DitheredLODTransition : 1;

	/** Enables override of whether to shadow using masked opacity on translucent materials. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_CastDynamicShadowAsMasked : 1;

	/** Enables override of the two sided property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_TwoSided : 1;

	/** Enables override of the IsThinSurface property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_bIsThinSurface : 1;

	/** Enables override of the output velocity property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_OutputTranslucentVelocity : 1;
	
	/** Enables override of the has pixel animation property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_bHasPixelAnimation : 1;
	
	/** Enables override of the enable tessellation property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_bEnableTessellation : 1;

	/** Enables override of the displacement magnitude and center property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_DisplacementScaling : 1;

	/** Enables override of the eanble displacement fade property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_bEnableDisplacementFade : 1;

	/** Enables override of the displacement fading range. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_DisplacementFadeRange : 1;

	/** Enables override of the max world position offset property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_MaxWorldPositionOffsetDisplacement : 1;

	/** Enables override of the bCompatibleWithLumenCardSharing property. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bOverride_CompatibleWithLumenCardSharing : 1;

	/** Bitmask of usage flags override states. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint32 bOverride_UsageFlags;

	/** Indicates that the material should be rendered without backface culling and the normal should be flipped for backfaces. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_TwoSided"))
	uint8 TwoSided : 1;

	/** Indicates that the material should be rendered as. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_bIsThinSurface"))
	uint8 bIsThinSurface : 1;

	/** Whether the material should support a dithered LOD transition when used with the foliage system. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_DitheredLODTransition"))
	uint8 DitheredLODTransition : 1;

	/** Whether the material should cast shadows as masked even though it has a translucent blend mode. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_CastDynamicShadowAsMasked", NoSpinbox = true))
	uint8 bCastDynamicShadowAsMasked:1;

	/** Whether the material should output velocity even though it has a translucent blend mode. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_OutputTranslucentVelocity"))
	uint8 bOutputTranslucentVelocity : 1;
	
	/** Whether the opaque material has any pixel animations happening, that isn't included in the geometric velocities.
	 * This allows to disable renderer's heuristics that assumes animation is fully described with motion vector, such as TSR's anti-flickering heuristic.
	 */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_bHasPixelAnimation"))
	uint8 bHasPixelAnimation : 1;
	
	/** Whether or not tessellation is enabled. Required for displacement to work. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_bEnableTessellation"))
	uint8 bEnableTessellation : 1;

	/** Whether or not displacement fade is enabled. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_bEnableDisplacementFade"))
	uint8 bEnableDisplacementFade : 1;

	/** When true, allows to share Lumen Cards between different instances even when material uses world position or per instance data, which may change material look per instance. All materials on a component needs this flag set for sharing to work. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_CompatibleWithLumenCardSharing"))
	uint8 bCompatibleWithLumenCardSharing : 1;
	
	/** The blend mode */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_BlendMode"))
	TEnumAsByte<EBlendMode> BlendMode;

	/** The shading model */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_ShadingModel"))
	TEnumAsByte<EMaterialShadingModel> ShadingModel;

	/** If BlendMode is BLEND_Masked, the surface is not rendered where OpacityMask < OpacityMaskClipValue. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_OpacityMaskClipValue", NoSpinbox = true))
	float OpacityMaskClipValue;

	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_DisplacementScaling"))
	FDisplacementScaling DisplacementScaling;

	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_DisplacementFadeRange"))
	FDisplacementFadeRange DisplacementFadeRange;

	/** The maximum World Position Offset distance. Zero means no maximum. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (editcondition = "bOverride_MaxWorldPositionOffsetDisplacement", ClampMin=0.0f, NoSpinbox = true))
	float MaxWorldPositionOffsetDisplacement;

	/** Bitmask of usage flags. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint32 UsageFlags;

	ENGINE_API FMaterialInstanceBasePropertyOverrides();

	ENGINE_API bool operator==(const FMaterialInstanceBasePropertyOverrides& Other)const;
	ENGINE_API bool operator!=(const FMaterialInstanceBasePropertyOverrides& Other)const;
};
