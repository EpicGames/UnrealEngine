// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialInstance.h"

#include "MaterialInstanceConstant.generated.h"

class UPhysicalMaterialMask;

/**
 * Material Instances may be used to change the appearance of a material without incurring an expensive recompilation of the material.
 * General modification of the material cannot be supported without recompilation, so the instances are limited to changing the values of
 * predefined material parameters. The parameters are statically defined in the compiled material by a unique name, type and default value.
 */
UCLASS(hidecategories=Object, collapsecategories, BlueprintType,MinimalAPI)
class UMaterialInstanceConstant : public UMaterialInstance
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** Unique ID for this material instance's parameter set 
	 *  Updated on changes in the editor to allow those changes to be detected */
	UPROPERTY()
	FGuid ParameterStateId;
#endif

	virtual ENGINE_API void PostLoad() override;
	virtual ENGINE_API void FinishDestroy() override;

#if WITH_EDITOR
	/** For constructing new MICs. */
	friend class UMaterialInstanceConstantFactoryNew;
	/** For editing MICs. */
	friend class UMaterialEditorInstanceConstant;

	virtual ENGINE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual ENGINE_API void UpdateCachedData() override final;
	virtual ENGINE_API void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif

	/** Physical material mask to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TObjectPtr<class UPhysicalMaterialMask> PhysMaterialMask;

	// Begin UMaterialInterface interface.
	ENGINE_API virtual UPhysicalMaterialMask* GetPhysicalMaterialMask() const override;
	// End UMaterialInterface interface.

	/** Get the scalar (float) parameter value from an MIC */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Scalar Parameter Value", ScriptName = "GetScalarParameterValue", Keywords = "GetFloatParameterValue"), Category="Rendering|Material")
	float K2_GetScalarParameterValue(FName ParameterName);

	/** Get the MIC texture parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Texture Parameter Value", ScriptName = "GetTextureParameterValue"), Category="Rendering|Material")
	class UTexture* K2_GetTextureParameterValue(FName ParameterName);

	/** Get the MIC texture collection parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Texture Collection Parameter Value", ScriptName = "GetTextureCollectionParameterValue"), Category="Rendering|Material")
	class UTextureCollection* K2_GetTextureCollectionParameterValue(FName ParameterName);

	/** Get the MIC vector parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Vector Parameter Value", ScriptName = "GetVectorParameterValue", Keywords = "GetColorParameterValue"), Category="Rendering|Material")
	FLinearColor K2_GetVectorParameterValue(FName ParameterName);

#if WITH_EDITOR
	/** Set an override material which will be used when rendering with nanite. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	void SetNaniteOverrideMaterial(bool bInEnableOverride, UMaterialInterface* InOverrideMaterial);

	/**
	 * Set the parent of this material instance. This function may only be called in the Editor!
	 *   WARNING: You MUST call PostEditChange afterwards to propagate changes to other materials in the chain!
	 * @param NewParent - The new parent for this material instance.
 	 * @param RecacheShader - Will recache required shaders.
	 */
	ENGINE_API void SetParentEditorOnly(class UMaterialInterface* NewParent, bool RecacheShader = true);

	/**
	* Copies the uniform parameters (scalar, vector and texture) from a material or instance hierarchy.
	* This will typically be faster than parsing all expressions but still slow as it must walk the full
	* material hierarchy as each parameter may be overridden at any level in the chain.
	* Note: This will not copy font parameters
	*/
	ENGINE_API void CopyMaterialUniformParametersEditorOnly(UMaterialInterface* Source, bool bIncludeStaticParams = true);

	/**
	 * Set the value parameters. These functions may be called only in the Editor!
	 *   WARNING: You MUST call PostEditChange afterwards to propagate changes to other materials in the chain!
	 * @param ParameterName - The parameter's name.
	 * @param Value - The value to set.
	 */
	ENGINE_API void SetVectorParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value);
	ENGINE_API void SetDoubleVectorParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, FVector4d Value);
	ENGINE_API void SetScalarParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, float Value);
	ENGINE_API void SetScalarParameterAtlasEditorOnly(const FMaterialParameterInfo& ParameterInfo, FScalarParameterAtlasInstanceData AtlasData);
	ENGINE_API void SetTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class UTexture* Value);
	ENGINE_API void SetTextureCollectionParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class UTextureCollection* Value);
	ENGINE_API void SetParameterCollectionParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection* Value);
	ENGINE_API void SetRuntimeVirtualTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture* Value);
	ENGINE_API void SetSparseVolumeTextureParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture* Value);
	ENGINE_API void SetFontParameterValueEditorOnly(const FMaterialParameterInfo& ParameterInfo, class UFont* FontValue, int32 FontPage);

	/**
	 * Clear all parameter overrides on this material instance. This function
	 * may be called only in the Editor!
	 */
	ENGINE_API void ClearParameterValuesEditorOnly();

	ENGINE_API virtual uint32 ComputeAllStateCRC() const override;
#endif // #if WITH_EDITOR
};

