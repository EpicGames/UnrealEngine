// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialInstance.h"
#include "MaterialInstanceDynamic.generated.h"

class UTexture;
class UTextureCollection;

UCLASS(hidecategories=Object, collapsecategories, BlueprintType, MinimalAPI)
class UMaterialInstanceDynamic : public UMaterialInstance
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	ENGINE_API virtual void UpdateCachedData() override;
#endif

	/** Set a MID scalar (float) parameter value */
	UFUNCTION(BlueprintCallable, meta=(Keywords = "SetFloatParameterValue"), Category="Rendering|Material")
	ENGINE_API void SetScalarParameterValue(FName ParameterName, float Value);

	/** Set a MID scalar (float) parameter value using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetFloatParameterValue"), Category = "Rendering|Material")
	ENGINE_API void SetScalarParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, float Value);

	//~ NOTE: These Index-related functions should be used VERY carefully, and only in cases where optimization is
	//~ critical.  Generally that's only if you're using an unusually high number of parameters in a material AND
	//~ setting a huge number of parameters in the same frame.

	// Use this function to set an initial value and fetch the index for use in SetScalarParameterByIndex.  This
	// function should only be called once for a particular name, and then use SetScalarParameterByIndex for subsequent
	// calls.  However, beware using this except in cases where optimization is critical, which is generally only if
	// you're using an unusually high number of parameters in a material and setting a large number of parameters in the
	// same frame.  Also, if the material is changed in any way that can change the parameter list, the index can be
	// invalidated.
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetFloatParameterValue"), Category = "Rendering|Material")
	ENGINE_API bool InitializeScalarParameterAndGetIndex(const FName& ParameterName, float Value, int32& OutParameterIndex);

	// Use the cached value of OutParameterIndex from InitializeScalarParameterAndGetIndex to set the scalar parameter
	// ONLY on the exact same MID.  Do NOT presume the index can be used from one instance on another.  Only use this
	// pair of functions when optimization is critical; otherwise use either SetScalarParameterValueByInfo or
	// SetScalarParameterValue.
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetFloatParameterValue"), Category = "Rendering|Material")
	ENGINE_API bool SetScalarParameterByIndex(int32 ParameterIndex, float Value);

	// Use this function to set an initial value and fetch the index for use in the following function.
	ENGINE_API bool InitializeVectorParameterAndGetIndex(const FName& ParameterName, const FLinearColor& Value, int32& OutParameterIndex);
	// Use the cached value of OutParameterIndex above to set the vector parameter ONLY on the exact same MID
	ENGINE_API bool SetVectorParameterByIndex(int32 ParameterIndex, const FLinearColor& Value);

	/** Get the current scalar (float) parameter value from an MID */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Scalar Parameter Value", ScriptName = "GetScalarParameterValue", Keywords = "GetFloatParameterValue"), Category="Rendering|Material")
	ENGINE_API float K2_GetScalarParameterValue(FName ParameterName);

	/** Get the current scalar (float) parameter value from an MID, using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Scalar Parameter Value By Info", ScriptName = "GetScalarParameterValueByInfo", Keywords = "GetFloatParameterValue"), Category = "Rendering|Material")
	ENGINE_API float K2_GetScalarParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo);

	/** Set an MID texture parameter value */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetTextureParameterValue(FName ParameterName, class UTexture* Value);

	/** Set an MID texture parameter value using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, class UTexture* Value);

	/** Set an MID texture parameter value */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetRuntimeVirtualTextureParameterValue(FName ParameterName, class URuntimeVirtualTexture* Value);

	/** Set an MID texture parameter value using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetRuntimeVirtualTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture* Value);

	/** Set an MID texture parameter value */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetSparseVolumeTextureParameterValue(FName ParameterName, class USparseVolumeTexture* Value);

	/** Get the current MID texture parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Texture Parameter Value", ScriptName = "GetTextureParameterValue"), Category="Rendering|Material")
	ENGINE_API class UTexture* K2_GetTextureParameterValue(FName ParameterName);

	/** Get the current MID texture parameter value, using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Texture Parameter Value By Info", ScriptName = "GetTextureParameterValueByInfo"), Category = "Rendering|Material")
	ENGINE_API class UTexture* K2_GetTextureParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo);

	/** Set an MID texture collection parameter value */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API void SetTextureCollectionParameterValue(FName ParameterName, UTextureCollection* Value);

	/** Set an MID texture collection parameter value using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API void SetTextureCollectionParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, UTextureCollection* Value);

	/** Get the current MID texture collection parameter value, using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Texture Collection Parameter Value By Info", ScriptName = "GetTextureCollectionParameterValueByInfo"), Category = "Rendering|Material")
	ENGINE_API UTextureCollection* K2_GetTextureCollectionParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo);

	/** Get the current MID texture collection parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Texture Collection Parameter Value", ScriptName = "GetTextureCollectionParameterValue"), Category="Rendering|Material")
	ENGINE_API UTextureCollection* K2_GetTextureCollectionParameterValue(FName ParameterName);

	/** Set an MID vector parameter value */
	UFUNCTION(BlueprintCallable, meta=(Keywords = "SetColorParameterValue"), Category="Rendering|Material")
	ENGINE_API void SetVectorParameterValue(FName ParameterName, FLinearColor Value);

	// Conveniences to keep general FLinearColor constructors explicit without adding extra burden to existing users of this function (where the conversion is straightforward).
	inline void SetVectorParameterValue(FName ParameterName, const FVector& Value) { SetVectorParameterValue(ParameterName, FLinearColor(Value)); }
	inline void SetVectorParameterValue(FName ParameterName, const FVector4& Value) { SetVectorParameterValue(ParameterName, FLinearColor(Value)); }

	/** Set an MID vector parameter value */
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetVectorParameterValue"), Category = "Rendering|Material")
	ENGINE_API void SetDoubleVectorParameterValue(FName ParameterName, FVector4 Value);

	/** Set an MID vector parameter value, using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetColorParameterValue"), Category = "Rendering|Material")
	ENGINE_API void SetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, FLinearColor Value);

	inline void SetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, const FVector& Value) { SetVectorParameterValueByInfo(ParameterInfo, FLinearColor(Value)); }
	inline void SetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo, const FVector4& Value) { SetVectorParameterValueByInfo(ParameterInfo, FLinearColor(Value)); }

	/** Get the current MID vector parameter value */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Get Vector Parameter Value", ScriptName = "GetVectorParameterValue", Keywords = "GetColorParameterValue"), Category="Rendering|Material")
	ENGINE_API FLinearColor K2_GetVectorParameterValue(FName ParameterName);

	/** Get the current MID vector parameter value, using MPI (to allow access to layer parameters) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get VectorParameter Value By Info", ScriptName = "GetVectorParameterValueByInfo", Keywords = "GetColorParameterValue"), Category = "Rendering|Material")
	ENGINE_API FLinearColor K2_GetVectorParameterValueByInfo(const FMaterialParameterInfo& ParameterInfo);
	
	/**
	 * Interpolates the scalar and vector parameters of this material instance based on two other material instances, and an alpha blending factor
	 * The output is the object itself (this).
	 * Supports the case SourceA==this || SourceB==this
	 * Both material have to be from the same base material
	 * @param SourceA value that is used for Alpha=0, silently ignores the case if 0
	 * @param SourceB value that is used for Alpha=1, silently ignores the case if 0
	 * @param Alpha usually in the range 0..1, values outside the range extrapolate
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Interpolate Material Instance Parameters", ScriptName = "InterpolateMaterialInstanceParameters"), Category="Rendering|Material")
	ENGINE_API void K2_InterpolateMaterialInstanceParams(UMaterialInstance* SourceA, UMaterialInstance* SourceB, float Alpha);

	/**
	 * Copies over parameters given a material interface (copy each instance following the hierarchy)
	 * Very slow implementation, avoid using at runtime. Hopefully we can replace it later with something like CopyInterpParameters()
	 * The output is the object itself (this). Copying 'quick parameters only' will result in a much
	 * faster copy process but will only copy dynamic scalar, vector and texture parameters on clients.
	 * @param bQuickParametersOnly Copy scalar, vector and texture parameters only. Much faster but may not include required data
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Copy Material Instance Parameters", ScriptName = "CopyMaterialInstanceParameters"), Category="Rendering|Material")
	ENGINE_API void K2_CopyMaterialInstanceParameters(UMaterialInterface* Source, bool bQuickParametersOnly = false);

	/**
	* Copies the uniform parameters (scalar, vector and texture) from a material or instance hierarchy.
	* This will typically be faster than parsing all expressions but still slow as it must walk the full
	* material hierarchy as each parameter may be overridden at any level in the chain.
	* Note: This will not copy static or font parameters
	*/
	ENGINE_API void CopyMaterialUniformParameters(UMaterialInterface* Source);

	/**
	 * Copies over parameters given a material instance (only copy from the instance, not following the hierarchy)
	 * much faster than K2_CopyMaterialInstanceParameters(), 
	 * The output is the object itself (this).
	 * @param Source ignores the call if 0
	 */
	UFUNCTION(meta=(DisplayName = "Copy Interp Parameters"), Category="Rendering|Material")
	ENGINE_API void CopyInterpParameters(UMaterialInstance* Source);

	/**
	* Create a material instance dynamic parented to the specified material. [Optional] Specify name.
	*/
	static ENGINE_API UMaterialInstanceDynamic* Create( class UMaterialInterface* ParentMaterial, class UObject* InOuter, FName Name = NAME_None);

	/**
	 * Set the value of the given font parameter.  
	 * @param ParameterName - The name of the font parameter
	 * @param OutFontValue - New font value to set for this MIC
	 * @param OutFontPage - New font page value to set for this MIC
	 */
	ENGINE_API void SetFontParameterValue(const FMaterialParameterInfo& ParameterInfo, class UFont* FontValue, int32 FontPage);

	/** Remove all parameter values */
	ENGINE_API void ClearParameterValues();

	/**
	 * Copy parameter values from another material instance. This will copy only
	 * parameters explicitly overridden in that material instance!!
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName = "Copy Parameter Overrides"), Category="Rendering|Material")
	ENGINE_API void CopyParameterOverrides(UMaterialInstance* MaterialInstance);
		
	/**
	 * Copy all interpolatable (scalar/vector) parameters from *SourceMaterialToCopyFrom to *this, using the current QualityLevel and given FeatureLevel
	 * For runtime use. More specialized and efficient than CopyMaterialInstanceParameters().
	 */
	UE_DEPRECATED(5.7, "Please use CopyScalarAndVectorParameters with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API void CopyScalarAndVectorParameters(const UMaterialInterface& SourceMaterialToCopyFrom, ERHIFeatureLevel::Type InFeatureLevel);
	ENGINE_API void CopyScalarAndVectorParameters(const UMaterialInterface& SourceMaterialToCopyFrom, EShaderPlatform ShaderPlatform);

	ENGINE_API void SetNaniteOverride(UMaterialInterface* InMaterial);

	virtual bool HasOverridenBaseProperties()const override{ return false; }

	//Material base property overrides. MIDs cannot override these so they just grab from their parent.
	ENGINE_API virtual float GetOpacityMaskClipValue() const override;
	ENGINE_API virtual bool GetCastDynamicShadowAsMasked() const override;
	ENGINE_API virtual FMaterialShadingModelField GetShadingModels() const override;
	ENGINE_API virtual bool IsShadingModelFromMaterialExpression() const override;
	ENGINE_API virtual EBlendMode GetBlendMode() const override;
	ENGINE_API virtual bool IsTwoSided() const override;
	ENGINE_API virtual bool IsThinSurface() const override;
	ENGINE_API virtual bool IsTranslucencyWritingVelocity() const override;
	ENGINE_API virtual bool IsDitheredLODTransition() const override;
	ENGINE_API virtual bool IsMasked() const override;
	ENGINE_API virtual FDisplacementScaling GetDisplacementScaling() const override;
	ENGINE_API virtual bool IsDisplacementFadeEnabled() const override;
	ENGINE_API virtual FDisplacementFadeRange GetDisplacementFadeRange() const override;
	ENGINE_API virtual float GetMaxWorldPositionOffsetDisplacement() const override;
	ENGINE_API virtual bool HasPixelAnimation() const override;
	ENGINE_API virtual uint32 GetUsageFlags() const override;
	ENGINE_API virtual bool GetUsageByFlag(EMaterialUsage Usage) const override;
	ENGINE_API virtual bool SetMaterialUsage(EMaterialUsage Usage) override;

	/**
	 * In order to remap to the correct texture streaming data, we must keep track of each texture renamed.
	 * The following map converts from a texture from the dynamic material to the texture from the static material.
	 * The following map converts from a texture from the dynamic material to the texture from the static material.
	 */
	TMap<FName, TArray<FName> > RenamedTextures;
	
	// This overrides does the remapping before looking at the parent data.
	ENGINE_API virtual float GetTextureDensity(FName TextureName, const struct FMeshUVChannelInfo& UVChannelData) const override;

protected:

	ENGINE_API void InitializeMID(class UMaterialInterface* ParentMaterial);

private:

	ENGINE_API void UpdateCachedDataDynamic();

	/** Allocates CachedData if needed, populates ParentLayerIndexRemap from the parent's layers,
	 *  and pushes the result to the render proxy. Does NOT touch the texture-sampling cache. */
	void UpdateCachedLayerData();
};

