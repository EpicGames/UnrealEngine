// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Misc/App.h"

#if WITH_EDITOR
namespace MaterialInstance_Private
{
	/** Workaround - Similar to base call but evaluates all expressions found, not just the first */
	template<typename ExpressionType>
	void FindClosestExpressionByGUIDRecursive(const FName& InName, const FGuid& InGUID, TConstArrayView<TObjectPtr<UMaterialExpression>> InMaterialExpression, ExpressionType*& OutExpression)
	{
		for (int32 ExpressionIndex = 0; ExpressionIndex < InMaterialExpression.Num(); ExpressionIndex++)
		{
			UMaterialExpression* ExpressionPtr = InMaterialExpression[ExpressionIndex];
			UMaterialExpressionMaterialFunctionCall* MaterialFunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(ExpressionPtr);
			UMaterialExpressionMaterialAttributeLayers* MaterialLayers = Cast<UMaterialExpressionMaterialAttributeLayers>(ExpressionPtr);

			if (ExpressionPtr && ExpressionPtr->GetParameterExpressionId() == InGUID)
			{
				check(ExpressionPtr->bIsParameterExpression);
				if (ExpressionType* ParamExpression = Cast<ExpressionType>(ExpressionPtr))
				{
					// UE-57086, workaround - To deal with duplicated parameters with matching GUIDs we walk
					// through every parameter rather than taking the first. Either we return the first matching GUID
					// we encounter (as before), or if we find another with the same name that can take precedence.
					// Only taking the first parameter means we can incorrectly treat the parameter as a rename and
					// lose/move data when we encounter an illegal GUID duplicate.
					// Properly fixing duplicate GUIDs is beyond the scope of a hotfix, see UE-47863 for more info.
					// NOTE: The case where a parameter in a function is renamed but another function in the material
					// contains a duplicate GUID is still broken and may lose the data. This still leaves us in a
					// more consistent state than 4.18 and should minimize the impact to a rarer occurrence.
					if (!OutExpression || InName == ParamExpression->ParameterName)
					{
						OutExpression = ParamExpression;
					}
				}
			}
			else if (MaterialFunctionCall && MaterialFunctionCall->MaterialFunction)
			{
				FindClosestExpressionByGUIDRecursive<ExpressionType>(InName, InGUID, MaterialFunctionCall->MaterialFunction->GetExpressions(), OutExpression);
			}
			else if (MaterialLayers)
			{
				const TArray<UMaterialFunctionInterface*>& Layers = MaterialLayers->GetLayers();
				const TArray<UMaterialFunctionInterface*>& Blends = MaterialLayers->GetBlends();

				for (const auto* Layer : Layers)
				{
					if (Layer)
					{
						FindClosestExpressionByGUIDRecursive<ExpressionType>(InName, InGUID, Layer->GetExpressions(), OutExpression);
					}
				}

				for (const auto* Blend : Blends)
				{
					if (Blend)
					{
						FindClosestExpressionByGUIDRecursive<ExpressionType>(InName, InGUID, Blend->GetExpressions(), OutExpression);
					}
				}
			}
		}
	}

	template <typename ParameterType, typename ExpressionType>
	bool UpdateParameter_FullTraversal(ParameterType& Parameter, UMaterial* ParentMaterial)
	{
		for (UMaterialExpression* Expression : ParentMaterial->GetExpressions())
		{
			if (Expression->IsA<ExpressionType>())
			{
				ExpressionType* ParameterExpression = CastChecked<ExpressionType>(Expression);
				if (ParameterExpression->ParameterName == Parameter.ParameterInfo.Name)
				{
					Parameter.ExpressionGUID = ParameterExpression->ExpressionGUID;
					return true;
				}
			}
			else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
			{
				if (FunctionCall->MaterialFunction && FunctionCall->MaterialFunction->UpdateParameterSet<ParameterType, ExpressionType>(Parameter))
				{
					return true;
				}
			}
			else if (UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
			{
				const TArray<UMaterialFunctionInterface*> Layers = LayersExpression->GetLayers();
				const TArray<UMaterialFunctionInterface*> Blends = LayersExpression->GetBlends();

				for (auto* Layer : Layers)
				{
					if (Layer && Layer->UpdateParameterSet<ParameterType, ExpressionType>(Parameter))
					{
						return true;
					}
				}

				for (auto* Blend : Blends)
				{
					if (Blend && Blend->UpdateParameterSet<ParameterType, ExpressionType>(Parameter))
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	template <typename ParameterType, typename ExpressionType>
	bool UpdateParameterSet_FullTraversal(TArray<ParameterType>& Parameters, UMaterial* ParentMaterial)
	{
		bool bChanged = false;

		// Loop through all of the parameters and try to either establish a reference to the 
		// expression the parameter represents, or check to see if the parameter's name has changed.
		for (int32 ParameterIdx = 0; ParameterIdx < Parameters.Num(); ParameterIdx++)
		{
			bool bTryToFindByName = true;

			ParameterType& Parameter = Parameters[ParameterIdx];

			if (Parameter.ExpressionGUID.IsValid())
			{
				ExpressionType* Expression = nullptr;
				FindClosestExpressionByGUIDRecursive<ExpressionType>(Parameter.ParameterInfo.Name, Parameter.ExpressionGUID, ParentMaterial->GetExpressions(), Expression);

				// Check to see if the parameter name was changed.
				if (Expression)
				{
					bTryToFindByName = false;

					if (Parameter.ParameterInfo.Name != Expression->ParameterName)
					{
						Parameter.ParameterInfo.Name = Expression->ParameterName;
						bChanged = true;
					}
				}
			}

			// No reference to the material expression exists, so try to find one in the material expression's array if we are in the editor.
			if (bTryToFindByName && GIsEditor && !FApp::IsGame())
			{
				if (UpdateParameter_FullTraversal<ParameterType, ExpressionType>(Parameter, ParentMaterial))
				{
					bChanged = true;
				}
			}
		}

		return bChanged;
	}


	template <typename ParameterType, typename ExpressionType>
	bool UpdateParameterSet_WithCachedData(EMaterialParameterType ParamTypeEnum, TArray<ParameterType>& Parameters, UMaterial* ParentMaterial)
	{
		bool bChanged = false;

		TArray<FMaterialParameterInfo> CachedParamInfos;
		TArray<FGuid> CachedParamGuids;
		ParentMaterial->GetAllParameterInfoOfType(ParamTypeEnum, CachedParamInfos, CachedParamGuids);
		int32 NumCachedParams = CachedParamGuids.Num();
		check(NumCachedParams == CachedParamInfos.Num());

		// Loop through all of the parameters and try to either establish a reference to the 
		// expression the parameter represents, or check to see if the parameter's name has changed.
		for (int32 ParameterIdx = 0; ParameterIdx < Parameters.Num(); ParameterIdx++)
		{
			bool bTryToFindByName = true;

			ParameterType& Parameter = Parameters[ParameterIdx];

			if (Parameter.ExpressionGUID.IsValid())
			{
				int32 CachedParamCandidate = INDEX_NONE;
				for (int32 CachedParamIdx = 0; CachedParamIdx < NumCachedParams; ++CachedParamIdx)
				{
					if (CachedParamGuids[CachedParamIdx] == Parameter.ExpressionGUID)
					{
						// UE-57086, workaround - To deal with duplicated parameters with matching GUIDs we walk
						// through every parameter rather than taking the first. Either we return the first matching GUID
						// we encounter (as before), or if we find another with the same name that can take precedence.
						// Only taking the first parameter means we can incorrectly treat the parameter as a rename and
						// lose/move data when we encounter an illegal GUID duplicate.
						// Properly fixing duplicate GUIDs is beyond the scope of a hotfix, see UE-47863 for more info.
						// NOTE: The case where a parameter in a function is renamed but another function in the material
						// contains a duplicate GUID is still broken and may lose the data. This still leaves us in a
						// more consistent state than 4.18 and should minimize the impact to a rarer occurrence.
						if ((CachedParamCandidate == INDEX_NONE) || Parameter.ParameterInfo.Name == CachedParamInfos[CachedParamIdx].Name)
						{
							CachedParamCandidate = CachedParamIdx;
						}
					}
				}

				// Check to see if the parameter name was changed.
				if (CachedParamCandidate != INDEX_NONE)
				{
					const FMaterialParameterInfo& CandidateParamInfo = CachedParamInfos[CachedParamCandidate];
					bTryToFindByName = false;

					if (Parameter.ParameterInfo.Name != CandidateParamInfo.Name)
					{
						Parameter.ParameterInfo.Name = CandidateParamInfo.Name;
						bChanged = true;
					}
				}
			}

			// No reference to the material expression exists, so try to find one in the material expression's array if we are in the editor.
			if (bTryToFindByName && GIsEditor && !FApp::IsGame())
			{
				if (UpdateParameter_FullTraversal<ParameterType, ExpressionType>(Parameter, ParentMaterial))
				{
					bChanged = true;
				}
			}
		}

		return bChanged;
	}
}

/**
 * This function takes a array of parameter structs and attempts to establish a reference to the expression object each parameter represents.
 * If a reference exists, the function checks to see if the parameter has been renamed.
 *
 * @param Parameters		Array of parameters to operate on.
 * @param ParentMaterial	Parent material to search in for expressions.
 *
 * @return Returns whether or not any of the parameters was changed.
 */
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<ParameterType>& Parameters, UMaterial* ParentMaterial) { return MaterialInstance_Private::UpdateParameterSet_FullTraversal<ParameterType, ExpressionType>(Parameters, ParentMaterial); }

/**
 * Overloads for UpdateParameterSet to use cached data for types that can leverage it 
 */
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FScalarParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FScalarParameterValue, ExpressionType>(EMaterialParameterType::Scalar, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FVectorParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FVectorParameterValue, ExpressionType>(EMaterialParameterType::Vector, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FDoubleVectorParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FDoubleVectorParameterValue, ExpressionType>(EMaterialParameterType::DoubleVector, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FTextureParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FTextureParameterValue, ExpressionType>(EMaterialParameterType::Texture, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FTextureCollectionParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FTextureCollectionParameterValue, ExpressionType>(EMaterialParameterType::TextureCollection, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FFontParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FFontParameterValue, ExpressionType>(EMaterialParameterType::Font, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FRuntimeVirtualTextureParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FRuntimeVirtualTextureParameterValue, ExpressionType>(EMaterialParameterType::RuntimeVirtualTexture, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FSparseVolumeTextureParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FSparseVolumeTextureParameterValue, ExpressionType>(EMaterialParameterType::SparseVolumeTexture, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FStaticSwitchParameter>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FStaticSwitchParameter, ExpressionType>(EMaterialParameterType::StaticSwitch, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FStaticComponentMaskParameter>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FStaticComponentMaskParameter, ExpressionType>(EMaterialParameterType::StaticComponentMask, Parameters, ParentMaterial);
}
template <typename ParameterType, typename ExpressionType>
bool UpdateParameterSet(TArray<FParameterCollectionParameterValue>& Parameters, UMaterial* ParentMaterial)
{
	return MaterialInstance_Private::UpdateParameterSet_WithCachedData<FParameterCollectionParameterValue, ExpressionType>(EMaterialParameterType::ParameterCollection, Parameters, ParentMaterial);
}

#endif // WITH_EDITOR
