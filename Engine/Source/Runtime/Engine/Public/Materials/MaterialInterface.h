// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Async/RecursiveMutex.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Materials/MaterialParameters.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "Engine/EngineTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/ScriptMacros.h"
#include "RenderCommandFence.h"
#include "SceneTypes.h"
#include "Engine/BlendableInterface.h"
#include "Materials/MaterialLayersFunctions.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "MaterialSceneTextureId.h"
#include "Materials/MaterialRelevance.h"
#include "MaterialRecursionGuard.h"
#include "MaterialShaderPrecompileMode.h"
#include "MeshUVChannelInfo.h"
#include "RHIFeatureLevel.h"
#include "PSOPrecache.h"
#include "StaticParameterSet.h"
#include "Interfaces/Interface_AsyncCompilation.h"

#include "MaterialInterface.generated.h"

class FMaterialCompiler;
class FMaterialRenderProxy;
class FMaterialResource;
class FShaderPipelineType;
class FShaderType;
class FVertexFactoryType;
class UMaterial;
class UPhysicalMaterial;
class UPhysicalMaterialMask;
class USubsurfaceProfile;
class USpecularProfile;
class UNeuralProfile;
class UToonProfile;
class UTexture;
class UMaterialInstance;
struct FDebugShaderTypeInfo;
struct FMaterialParameterInfo;
struct FMaterialResourceLocOnDisk;
struct FMaterialCachedExpressionData;
struct FMaterialCachedExpressionEditorOnlyData;
#if WITH_EDITOR
struct FMaterialResourceForCooking;
#endif
#if WITH_EDITORONLY_DATA
struct FParameterChannelNames;
#endif
enum EShaderPlatform : uint16;
struct FSubstrateCompilationConfig;
class UMaterialExpressionCustomOutput;
struct FMaterialInsights;
struct FStreamingRenderAssetPrimitiveInfo;

typedef TArray<FMaterialResource*> FMaterialResourceDeferredDeletionArray;

/** Enumeration of material usage types. */
UENUM(BlueprintType)
enum EMaterialUsage : int
{
	/** Indicates that the material and its instances can be used with skeletal meshes. */
	MATUSAGE_SkeletalMesh,
	/** Indicates that the material and its instances can be used with particle sprites. */
	MATUSAGE_ParticleSprites,
	/** Indicates that the material and its instances can be used with beam trails. */
	MATUSAGE_BeamTrails,
	/** Indicates that the material and its instances can be used with mesh particles. */
	MATUSAGE_MeshParticles,
	/** Indicates that the material and its instances can be used with static lighting. */
	MATUSAGE_StaticLighting,
	/** Indicates that the material and its instances can be used with morph targets. */
	MATUSAGE_MorphTargets,
	/** Indicates that the material and its instances can be used with spline meshes. */
	MATUSAGE_SplineMesh,
	/** Indicates that the material and its instances can be used with instanced static meshes. */
	MATUSAGE_InstancedStaticMeshes,
	/** Indicates that the material and its instances can be use with geometry collections. */
	MATUSAGE_GeometryCollections,
	/** Indicates that the material and its instances can be used with clothing. */
	MATUSAGE_Clothing,
	/** Indicates that the material and its instances can be used with Niagara sprites. */
	MATUSAGE_NiagaraSprites,
	/** Indicates that the material and its instances can be used with Niagara ribbons. */
	MATUSAGE_NiagaraRibbons,
	/** Indicates that the material and its instances can be used with Niagara meshes. */
	MATUSAGE_NiagaraMeshParticles,
	/** Indicates that the material and its instances can be use with geometry caches. */
	MATUSAGE_GeometryCache,
	/** Indicates that the material and its instances can be use with water. */
	MATUSAGE_Water,
	/** Indicates that the material and its instances can be use with hair strands. */
	MATUSAGE_HairStrands,
	/** Indicates that the material and its instances can be use with LiDAR Point Clouds. */
	MATUSAGE_LidarPointCloud,
	/** Indicates that the material and its instances can be used with Virtual Heightfield Mesh. (Deprecated). */
	MATUSAGE_VirtualHeightfieldMesh,
	/** Indicates that the material and its instances can be used with Nanite meshes. */
	MATUSAGE_Nanite,
	/** Indicates that the material and its instances can be used with Nanite voxel meshes. */
	MATUSAGE_Voxels,
	/** Indicates that the material and its instances with volumetric cloud. Without that flag, it can only be used on volumetric fog. */
	MATUSAGE_VolumetricCloud,
	/** Indicates that the material and its instances with heterogeneous volumes. Without that flag, it can only be used on volumetric fog. */
	MATUSAGE_HeterogeneousVolumes,
	/** Indicates that the material and its instances can be used on static meshes. */
	MATUSAGE_StaticMesh,
	/** Indicates that the material and its instances can be used with editor compositing. */
	MATUSAGE_EditorCompositing,
	/** Indicates that the material and its instances can be used with neural network engine. */
	MATUSAGE_NeuralNetworks,
	/** Indicates that the material and its instances can be used with mesh deformers. */
	MATUSAGE_MeshDeformer,
	/** Indicates that the material and its instances can be used with instanced skinned meshes. */
	MATUSAGE_InstancedSkinnedMesh,
	/** Indicates that the material and its instances can be used with Nanite curve meshes. */
	MATUSAGE_Curves,

	MATUSAGE_MAX,
};

ENGINE_API EMaterialQualityLevel::Type GetCurrentMaterialQualityLevelChecked();

/** 
 *	UMaterial interface settings for Lightmass
 */
USTRUCT()
struct FLightmassMaterialInterfaceSettings
{
	GENERATED_USTRUCT_BODY()

	/** Scales the emissive contribution of this material to static lighting. */
	UPROPERTY()
	float EmissiveBoost;

	/** Scales the diffuse contribution of this material to static lighting. */
	UPROPERTY(EditAnywhere, Category=Material)
	float DiffuseBoost;

	/** 
	 * Scales the resolution that this material's attributes were exported at. 
	 * This is useful for increasing material resolution when details are needed.
	 */
	UPROPERTY(EditAnywhere, Category=Material)
	float ExportResolutionScale;

	/** If true, forces translucency to cast static shadows as if the material were masked. */
	UPROPERTY(EditAnywhere, Category = Material)
	uint8 bCastShadowAsMasked : 1;

	/** Boolean override flags - only used in MaterialInstance* cases. */
	/** If true, override the bCastShadowAsMasked setting of the parent material. */
	UPROPERTY()
	uint8 bOverrideCastShadowAsMasked:1;

	/** If true, override the emissive boost setting of the parent material. */
	UPROPERTY()
	uint8 bOverrideEmissiveBoost:1;

	/** If true, override the diffuse boost setting of the parent material. */
	UPROPERTY()
	uint8 bOverrideDiffuseBoost:1;

	/** If true, override the export resolution scale setting of the parent material. */
	UPROPERTY()
	uint8 bOverrideExportResolutionScale:1;

	FLightmassMaterialInterfaceSettings()
		: EmissiveBoost(1.0f)
		, DiffuseBoost(1.0f)
		, ExportResolutionScale(1.0f)
		, bCastShadowAsMasked(false)
		, bOverrideCastShadowAsMasked(false)
		, bOverrideEmissiveBoost(false)
		, bOverrideDiffuseBoost(false)
		, bOverrideExportResolutionScale(false)
	{}
};

/** 
 * This struct holds data about how a texture is sampled within a material.
 */
USTRUCT()
struct FMaterialTextureInfo
{
	GENERATED_USTRUCT_BODY()

	FMaterialTextureInfo() : SamplingScale(0), UVChannelIndex(INDEX_NONE)
	{
#if WITH_EDITORONLY_DATA
		TextureIndex = INDEX_NONE;
#endif
	}

	FMaterialTextureInfo(ENoInit) {}

	/** The scale used when sampling the texture */
	UPROPERTY()
	float SamplingScale;

	/** The coordinate index used when sampling the texture */
	UPROPERTY()
	int32 UVChannelIndex;

	/** The texture name. Used for debugging and also to for quick matching of the entries. */
	UPROPERTY()
	FName TextureName;

#if WITH_EDITORONLY_DATA
	/** The reference to the texture, used to keep the TextureName valid even if it gets renamed. */
	UPROPERTY()
	FSoftObjectPath TextureReference;

	/** 
	  * The texture index in the material resource the data was built from.
	  * This must be transient as it depends on which shader map was used for the build.  
	  */
	UPROPERTY(transient)
	int32 TextureIndex;
#endif

	/** Return whether the data is valid to be used */
	ENGINE_API bool IsValid(bool bCheckTextureIndex = false) const; 
};

using TMicRecursionGuard = TMaterialRecursionGuard<class UMaterialInterface>;

/** Holds information about a hierarchy of materials */
struct FMaterialInheritanceChain
{
	/** Base material at the root of the hierarchy */
	const UMaterial* BaseMaterial = nullptr;
	/** Cached expression data to use */
	const FMaterialCachedExpressionData* CachedExpressionData = nullptr;
	/** All the instances in the chain, starting with the current instance, and ending with the instance closest to the root material */
	TArray<const class UMaterialInstance*, TInlineAllocator<16>> MaterialInstances;

	inline const UMaterial* GetBaseMaterial() const { checkSlow(BaseMaterial); return BaseMaterial; }
	inline const FMaterialCachedExpressionData& GetCachedExpressionData() const { checkSlow(CachedExpressionData); return *CachedExpressionData; }
};

/**
 * Holds data about what is used in the shader graph of a specific material property or custom output
 */
struct FMaterialAnalysisResult
{
	/** The texture coordinates used */
	TBitArray<> TextureCoordinates;

	/** The shading models used (only relevant when analyzing property MP_ShadingModel) */
	FMaterialShadingModelField ShadingModels;

	/** Whether any vertex data is used */
	bool bRequiresVertexData = false;

	/** Whether the world position (excluding shader offsets) is used */
	bool bRequiresWorldPosition = false;

	/** If material translation was success, valid in AnalyzeMaterialTranslationOutput */
	bool bTranslationSuccess = false;

	/** Estimated amount of VS samplers used */
	uint16 EstimatedNumTextureSamplesVS = 0;

	/** Estimated amount of PS samplers used */
	uint16 EstimatedNumTextureSamplesPS = 0;
};

struct FTextureSamplingInfo
{
	FTextureSamplingInfo()
	{
		for (uint32 Index = 0; Index < MAX_TEXCOORDS; ++Index)
		{
			ChannelMinSamplingScale[Index] = UE_MAX_FLT;
		}
	}

	explicit FTextureSamplingInfo(UTexture* InTexture)
		: FTextureSamplingInfo()
	{
		Texture = InTexture;
	}

	TObjectPtr<UTexture> Texture = nullptr;

	float ChannelMinSamplingScale[MAX_TEXCOORDS] = { };

	ENGINE_API float GetTextureDensity(const FMeshUVChannelInfo& UVChannelData) const;
	bool operator==(const FTextureSamplingInfo&) const = default;
};

struct FMaterialCachedTexturesSamplingInfo
{
	struct FShaderPlatformMapping
	{
		TStaticArray<TArray<uint16>, EMaterialQualityLevel::Num> QualityMapping{};
	};

	FShaderPlatformMapping ShaderPlatformMapping;

	TArray<FTextureSamplingInfo> TextureSamplingInfos;

	void AddReferencedObjects(FReferenceCollector& Collector);
	void GetStreamingRenderAssetPrimitiveInfo(
		const FBoxSphereBounds& ComponentBounds,
		const float ComponentScaling,
		const FMeshUVChannelInfo& InUVChannelData,
		const EMaterialQualityLevel::Type& MaterialQualityLevel,
		TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingTextures) const;
};

UCLASS(Optional)
class UMaterialInterfaceEditorOnlyData : public UObject
{
	GENERATED_BODY()
public:
	UMaterialInterfaceEditorOnlyData();
	UMaterialInterfaceEditorOnlyData(FVTableHelper& Helper);
	ENGINE_API virtual ~UMaterialInterfaceEditorOnlyData();

	//~ Begin UObject Interface.
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

	TSharedPtr<FMaterialCachedExpressionEditorOnlyData> CachedExpressionData;

	/** Set if CachedExpressionData was loaded from disk, should typically be true when running with cooked data, and false in the editor */
	bool bLoadedCachedExpressionData = false;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnBaseMaterialIsSet, UMaterialInterface*);

UCLASS(abstract, BlueprintType, MinimalAPI, HideCategories = (Thumbnail))
class UMaterialInterface : public UObject, public IBlendableInterface, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
protected:
	friend class UMaterialInterfaceEditorOnlyData;

	UPROPERTY(VisibleAnywhere, Instanced, Category = Material, meta = (AllowEditInlineCustomization, ShowInnerProperties))
	TObjectPtr<UMaterialInterfaceEditorOnlyData> EditorOnlyData;

	ENGINE_API virtual const UClass* GetEditorOnlyDataClass() const;

public:
	virtual UMaterialInterfaceEditorOnlyData* GetEditorOnlyData() { return EditorOnlyData; }
	virtual const UMaterialInterfaceEditorOnlyData* GetEditorOnlyData() const { return EditorOnlyData; }
	bool IsEditorOnlyDataValid() const { return EditorOnlyData != nullptr; }
#endif // WITH_EDITORONLY_DATA

public:
	ENGINE_API TSharedPtr<const FMaterialCachedTexturesSamplingInfo> GetCachedTexturesSamplingInfo() const;
	ENGINE_API virtual bool CalculateTexturesSamplingInfo(UTexture* Texture, FTextureSamplingInfo& OutTextureSamplingInfo) const;

	UE_DEPRECATED(5.8, "CalculateTexturesSamplingInfo is now const, returns bool, and takes an FTextureSamplingInfo out-parameter. Override the new signature instead.")
	virtual FTextureSamplingInfo CalculateTexturesSamplingInfo(UTexture* Texture) final { return FTextureSamplingInfo{}; }

	/** SubsurfaceProfile, for Screen Space Subsurface Scattering.. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	/** Subsurface Profiles. For internal usage, not editable/visible. 
	 * For Substrate, there can be many in a material similarly to SpecularProfile (even though only one can be specified per pixel due to the post processing) 
	 */
	UPROPERTY()
	TArray<TObjectPtr<class USubsurfaceProfile>> SubsurfaceProfiles;

	/** Specular Profile. For internal usage, not editable/visible */
	UPROPERTY()
	TArray<TObjectPtr<class USpecularProfile>> SpecularProfiles;

	/** Toon Profile. For internal usage, not editable/visible */
	UPROPERTY()
	TArray<TObjectPtr<class UToonProfile>> ToonProfiles;

	/** Neural network profile. For internal usage, not editable/visible */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = PostProcessMaterial, meta = (DisplayName = "Neural Profile"))
	TObjectPtr<class UNeuralProfile> NeuralProfile;

	/** Event triggered when the base material is set */
	FOnBaseMaterialIsSet OnBaseMaterialSetEvent;

	/* -------------------------- */

	/** A fence to track when the primitive is no longer used as a parent */
	FRenderCommandFence ParentRefFence;

protected:
	/** The Lightmass settings for this object. */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	struct FLightmassMaterialInterfaceSettings LightmassSettings;

#if WITH_EDITORONLY_DATA
	/** Because of redirector, the texture names need to be resorted at each load in case they changed. */
	UPROPERTY(transient)
	bool bTextureStreamingDataSorted;
	UPROPERTY()
	int32 TextureStreamingDataVersion;
#endif

	/** Data used by the texture streaming to know how each texture is sampled by the material. Sorted by names for quick access. */
	UPROPERTY()
	TArray<FMaterialTextureInfo> TextureStreamingData;

	/** Default Textures. At runtime, based on cook output, if ShouldStripUnusedDefaultTextures returns true. */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ReferencedDefaultTextures;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Material)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	ENGINE_API void CacheTexturesSamplingInfo();
	ENGINE_API virtual bool CanCacheTexturesSamplingInfo() const;

	/** Global gate for the texture-sampling cache, independent of any single material instance. Returns
	 *  false while running the cook commandlet, otherwise true iff the cache is enabled by CVar or by
	 *  FSimpleStreamableAssetManager. Both driving CVars are read-only, so this value is fixed for the
	 *  lifetime of the process once engine init has run. CanCacheTexturesSamplingInfo() defers to this. */
	ENGINE_API static bool IsTexturesSamplingCacheGloballyEnabled();

	/** Build a fresh cache from GetUsedTextures + CalculateTexturesSamplingInfo. Stateless; holds no
	 *  locks. Callable on GT/parallel-GT only; must not run on a generic worker. Shared between the
	 *  proactive CacheTexturesSamplingInfo path and the lazy path inside GetCachedTexturesSamplingInfo. */
	ENGINE_API TSharedPtr<FMaterialCachedTexturesSamplingInfo> ComputeCachedTexturesSamplingInfo() const;

	/** Pre-cached texture sampling information used for texture streaming. Mutated on first lazy
	 *  read from GetCachedTexturesSamplingInfo and on each proactive refresh, so the field is mutable
	 *  to let the const read path populate it. Writes are serialized through CachedTexturesSamplingInfoLock. */
	mutable TSharedPtr<FMaterialCachedTexturesSamplingInfo> CachedTexturesSamplingInfo = nullptr;

	/** RW lock whose underlying platform primitive is only created when the texture-sampling cache is globally enabled. */
	class FCacheRWLock
	{
	public:
		explicit FCacheRWLock(bool bEnabled)
		{
			if (bEnabled)
			{
				Lock.Emplace();
			}
		}

		void ReadLock()
		{
			if (Lock.IsSet())
			{
				Lock->ReadLock();
			}
		}

		void ReadUnlock()
		{
			if (Lock.IsSet())
			{
				Lock->ReadUnlock();
			}
		}

		void WriteLock()
		{
			if (Lock.IsSet())
			{
				Lock->WriteLock();
			}
		}

		void WriteUnlock()
		{
			if (Lock.IsSet())
			{
				Lock->WriteUnlock();
			}
		}

	private:
		TOptional<FTransactionallySafeRWLock> Lock;
	};

	/** Guards CachedTexturesSamplingInfo publication so game-thread writers cannot tear against
	 *  worker/render-thread readers that copy the shared pointer through GetCachedTexturesSamplingInfo.
	 *  Only relevant when the cache is globally enabled; see FCacheRWLock. */
	mutable FCacheRWLock CachedTexturesSamplingInfoLock{ IsTexturesSamplingCacheGloballyEnabled() };

#if !WITH_EDITOR
	/**
	 * Cached material relevance for async worker-thread access (cooked builds only).
	 * Populated on GameThread via WarmRelevanceCache(). Generic workers read from cache
	 * instead of going through GetRelevance_Internal -> _GameThread() methods
	 * which read GameThreadShaderMap which is not safe from generic workers.
	 *
	 * bUsesMotionVectorWorldOffset and bUsesTemporalResponsiveness are not part of
	 * FMaterialRelevance because it is exactly 64 bits with no room for additional fields.
	 */
	struct FCachedMaterialRelevance
	{
		FMaterialRelevance Relevance;
		uint8 bUsesMotionVectorWorldOffset : 1 = 0;
		uint8 bUsesTemporalResponsiveness : 1 = 0;
	};

	/**
	 * Thread-safe write-once wrapper. 
	 * GameThread writes via Set(), workers read via TryGet(). 
	 */
	struct FAtomicCachedMaterialRelevance
	{
		void Set(const FCachedMaterialRelevance& InValue)
		{
			Value = InValue;
			bReady.store(true, std::memory_order_release);
		}

		// Cached relevance is not keyed by shader platform. In cooked builds the shader platform
		// is fixed at startup and cannot change at runtime, so this is safe.
		const FCachedMaterialRelevance* TryGet() const
		{
			if (bReady.load(std::memory_order_acquire))
			{
				return &Value;
			}
			return nullptr;
		}

	private:
		FCachedMaterialRelevance Value;
		std::atomic<bool> bReady{false};
	};

	FAtomicCachedMaterialRelevance CachedMaterialRelevance;
#endif

private:
	/** Feature levels to force to compile. */
	uint32 FeatureLevelsToForceCompile;

#if WITH_EDITOR
	/** Used for multithread access of TextureStreamingData in SortTextureStreamingData in editor (called by GetTextureDensity). */
	mutable UE::FRecursiveMutex TextureStreamingDataMutex;
#endif

public:

	/** Whether this material interface is included in the base game (and not in a DLC) */
	UPROPERTY(meta=(DisplayAfter="NeuralProfile"))
	uint8 bIncludedInBaseGame : 1;

	ENGINE_API UMaterialInterface();
	ENGINE_API UMaterialInterface(FVTableHelper& Helper);
	ENGINE_API virtual ~UMaterialInterface();

	//~ Begin IInterface_AssetUserData Interface
	ENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	ENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	ENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITORONLY_DATA
	/** List of all used but missing texture indices in TextureStreamingData. Used for visualization / debugging only. */
	UPROPERTY(transient)
	TArray<FMaterialTextureInfo> TextureStreamingDataMissingEntries;

	/** The mesh used by the material editor to preview the material.*/
	UPROPERTY(EditAnywhere, Category=Previewing, meta=(AllowedClasses="/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh", ExactClass="true"))
	FSoftObjectPath PreviewMesh;

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	UPROPERTY()
	TMap<FString, bool> LayerParameterExpansion;

	UPROPERTY()
	TMap<FString, bool> ParameterOverviewExpansion;

	/** Importing data and options used for this material */
	UPROPERTY(EditAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;
	
protected:
#if WITH_EDITOR
		/** Material resources being cached for cooking. */
		TMap<const ITargetPlatform*, TArray<FMaterialResourceForCooking>> CachedMaterialResourcesForCooking;
#endif

private:
	/** Unique ID for this material, used for caching during distributed lighting */
	UPROPERTY()
	FGuid LightingGuid;

#endif // WITH_EDITORONLY_DATA

private:
	/** Feature level bitfield to compile for all materials */
	ENGINE_API static uint32 FeatureLevelsForAllMaterials;
public:
	/** Set which feature levels this material instance should compile. GMaxRHIFeatureLevel is always compiled! */
	ENGINE_API void SetFeatureLevelToCompile(ERHIFeatureLevel::Type FeatureLevel, bool bShouldCompile);

	/** Set which feature levels _all_ materials should compile to. GMaxRHIFeatureLevel is always compiled. */
	ENGINE_API static void SetGlobalRequiredFeatureLevel(ERHIFeatureLevel::Type FeatureLevel, bool bShouldCompile);

	//~ Begin UObject Interface.
	ENGINE_API virtual void BeginDestroy() override;
	ENGINE_API virtual void FinishDestroy() override;
	ENGINE_API virtual bool IsReadyForFinishDestroy() override;
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void Serialize(FArchive& Ar) override;
	ENGINE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	ENGINE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	ENGINE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	ENGINE_API virtual void PostCDOContruct() override;
	ENGINE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;
	ENGINE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

#if WITH_EDITOR
	ENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	ENGINE_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
	ENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif // WITH_EDITOR
#if WITH_EDITOR
	static ENGINE_API void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif
	//~ End UObject Interface.

	//~ Begin Begin Interface IBlendableInterface
	ENGINE_API virtual void OverrideBlendableSettings(class FSceneView& View, float Weight) const override;
	//~ Begin End Interface IBlendableInterface

	/** Walks up parent chain and finds the base Material that this is an instance of. Just calls the virtual GetMaterial() */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API UMaterial* GetBaseMaterial();

	/** Callback triggered when the material has been assigned as an override material */
	ENGINE_API virtual void OnAssignedAsOverride(const UObject* Owner);
	/** Callback triggered when the material has been removed as an override material */
	ENGINE_API virtual void OnRemovedAsOverride(const UObject* Owner);

	/**
	 * Get the material which we are instancing.
	 * Walks up parent chain and finds the base Material that this is an instance of. 
	 */
	virtual class UMaterial* GetMaterial() PURE_VIRTUAL(UMaterialInterface::GetMaterial,return NULL;);
	/**
	 * Get the material which we are instancing.
	 * Walks up parent chain and finds the base Material that this is an instance of. 
	 */
	virtual const class UMaterial* GetMaterial() const PURE_VIRTUAL(UMaterialInterface::GetMaterial,return NULL;);

	/**
	 * Same as above, but can be called concurrently
	 */
	virtual const class UMaterial* GetMaterial_Concurrent(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const PURE_VIRTUAL(UMaterialInterface::GetMaterial_Concurrent,return NULL;);

	virtual void GetMaterialInheritanceChain(FMaterialInheritanceChain& OutChain) const PURE_VIRTUAL(UMaterialInterface::GetMaterialInheritanceChain, return;);

	ENGINE_API virtual const FMaterialCachedExpressionData& GetCachedExpressionData(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const;

#if WITH_EDITORONLY_DATA
	/** @return The Index at which the given texture is present in the editor only Default Textures array. */
	ENGINE_API int32 GetDefaultTextureIdx(UObject* InTexture) const;

	/** @return Whether the given texture is present in the editor only Default Textures. */
	ENGINE_API bool ContainsDefaultTexture(UObject* InTexture) const;

	/** @return The array of default textures. */
	ENGINE_API const TArray<TObjectPtr<UObject>>& GetDefaultTextures() const;
#endif

	ENGINE_API bool IsUsingNewHLSLGenerator() const;
	
	UE_DEPRECATED(5.8, "Use IsUsingNewHLSLGenerator instead")
	ENGINE_API bool IsUsingNewTranslatorPrototype() const;

	ENGINE_API const FSubstrateCompilationConfig& GetSubstrateCompilationConfig() const;
	ENGINE_API void SetSubstrateCompilationConfig(FSubstrateCompilationConfig& SubstrateCompilationConfig);

	/**
	* Test this material for dependency on a given material.
	* @param	TestDependency - The material to test for dependency upon.
	* @return	True if the material is dependent on TestDependency.
	*/
	virtual bool IsDependent(UMaterialInterface* TestDependency) { return TestDependency == this; }

	/**
	 * Same as above, but can be called concurrently
	 */
	virtual bool IsDependent_Concurrent(UMaterialInterface* TestDependency, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) { return TestDependency == this; }

	/**
	* Get this material dependencies.
	* @param	Dependencies - List of materials this interface depends on.
	*/
	virtual void GetDependencies(TSet<UMaterialInterface*>& Dependencies) PURE_VIRTUAL(UMaterialInterface::GetDependencies, return;);

	/**
	* Return a pointer to the FMaterialRenderProxy used for rendering.
	* @param	Selected	specify true to return an alternate material used for rendering this material when part of a selection
	*						@note: only valid in the editor!
	* @return	The resource to use for rendering this material instance.
	*/
	virtual class FMaterialRenderProxy* GetRenderProxy() const PURE_VIRTUAL(UMaterialInterface::GetRenderProxy,return NULL;);

	/**
	* Return a pointer to the physical material used by this material instance.
	* @return The physical material.
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics|Material")
	virtual UPhysicalMaterial* GetPhysicalMaterial() const PURE_VIRTUAL(UMaterialInterface::GetPhysicalMaterial,return NULL;);

	/**
	 * Return a pointer to the physical material mask used by this material instance.
	 * @return The physical material.
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics|Material")
	virtual UPhysicalMaterialMask* GetPhysicalMaterialMask() const PURE_VIRTUAL(UMaterialInterface::GetPhysicalMaterialMask, return nullptr;);

	/**
	 * Return a pointer to the physical material from mask map at given index.
	 * @return The physical material.
	 */
	UFUNCTION(BlueprintCallable, Category = "Physics|Material")
	virtual UPhysicalMaterial* GetPhysicalMaterialFromMap(int32 Index) const PURE_VIRTUAL(UMaterialInterface::GetPhysicalMaterialFromMap, return nullptr;);

	/** Determines whether each quality level has different nodes by inspecting the material's expressions.
	* Or is required by the material quality setting overrides.
	* @param	QualityLevelsUsed	output array of used quality levels.
	* @param	ShaderPlatform	The shader platform to use for the quality settings.
	* @param	bCooking		During cooking, certain quality levels may be discarded
	*/
	void GetQualityLevelUsage(TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num> >& QualityLevelsUsed, EShaderPlatform ShaderPlatform, bool bCooking = false);

	inline void GetQualityLevelUsageForCooking(TArray<bool, TInlineAllocator<EMaterialQualityLevel::Num> >& QualityLevelsUsed, EShaderPlatform ShaderPlatform)
	{
		GetQualityLevelUsage(QualityLevelsUsed, ShaderPlatform, true);
	}

	UE_DEPRECATED(5.7, "Deprecated: Use the version with the optional parameters InSpecificQualityLevel and InSpecificShaderPlatform. If unset, we search used textures for this material for all Quality Levels and Shader Platforms.")
	virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, EMaterialQualityLevel::Type QualityLevel, bool bAllQualityLevels, ERHIFeatureLevel::Type FeatureLevel, bool bAllFeatureLevels) const final {}

	/** Return the textures used to render this material.
	* @param	OutTextures	Output of used textures for this material.
	* @param	InSpecificQualityLevel	If set, only search textures for the given Quality Level, otherwise search for all Quality levels.
	* @param	InSpecificShaderPlatform	If set, only search textures for the given Shader Platform, otherwise search for all Shader Platforms.
	*/
	virtual void GetUsedTextures(TArray<UTexture*>& OutTextures, TOptional<EMaterialQualityLevel::Type> InSpecificQualityLevel = TOptional<EMaterialQualityLevel::Type>(), TOptional<EShaderPlatform> InSpecificShaderPlatform = TOptional<EShaderPlatform>()) const
		PURE_VIRTUAL(UMaterialInterface::GetUsedTextures, );
	/** 
	* Return the textures used to render this material and the material indices bound to each. 
	* Because material indices can change for each shader, this is limited to a single platform and quality level.
	* An empty array in OutIndices means the index is undefined.
	*/
	UE_DEPRECATED(5.7, "Please use GetUsedTexturesAndIndices with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual void GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel) const final;
	ENGINE_API virtual void GetUsedTexturesAndIndices(TArray<UTexture*>& OutTextures, TArray< TArray<int32> >& OutIndices, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform) const;

	/** Returns the textures which are used by the Uniform Expressions of the given Material Resource. */
	ENGINE_API virtual void GetTextureExpressionValues(const FMaterialResource* MaterialResource, TArray<UTexture*>& OutTextures, TArray<TArray<int32>>* OutIndices = nullptr, bool bIncludeTextureCollections = false, bool bAllowOverrides = false) const;

	/**
	 * Override a specific texture (transient)
	 *
	 * @param InTextureToOverride The texture to override
	 * @param OverrideTexture The new texture to use
	 */
	virtual void OverrideTexture(const UTexture* InTextureToOverride, UTexture* OverrideTexture, ERHIFeatureLevel::Type InFeatureLevel) PURE_VIRTUAL(UMaterialInterface::OverrideTexture, return;);

	/** 
	 * Overrides the default value of the given parameter (transient).  
	 * This is used to implement realtime previewing of parameter defaults. 
	 * Handles updating dependent MI's and cached uniform expressions.
	 */
	UE_DEPRECATED(5.7, "Please use SetNumericParameterDefault/ClearNumericParameterDefault instead with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	virtual void OverrideNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value, bool bOverride, ERHIFeatureLevel::Type FeatureLevel) final {}
	virtual void SetNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, const UE::Shader::FValue& Value, EShaderPlatform InShaderPlatform) PURE_VIRTUAL(UMaterialInterface::SetNumericParameterDefault, return;);
	virtual void ClearNumericParameterDefault(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, EShaderPlatform InShaderPlatform) PURE_VIRTUAL(UMaterialInterface::ClearNumericParameterDefault, return;);

	/** @return the name of the given usage flag. */
	ENGINE_API static FString GetUsageName(EMaterialUsage Usage);

	/** Returns the bit mask of all usage flags for this material. */
	virtual uint32 GetUsageFlags() const PURE_VIRTUAL(UMaterialInterface::GetUsageFlags, return 0;);

	/** @return true if this material can be used with the given usage flag. */
	virtual bool GetUsageByFlag(EMaterialUsage Usage) const PURE_VIRTUAL(UMaterialInterface::GetUsageByFlag, return false;);

	/** 
	 * Set the usage flag on this material. 
	 * @return true if the material can be used for rendering with the given flag.
     */
	virtual bool SetMaterialUsage(EMaterialUsage Usage) PURE_VIRTUAL(UMaterialInterface::SetMaterialUsage, return false;);

	/**
	 * Checks if the material can be used with the given usage flag.  
	 * In the editor, if the flag isn't set, it will be set and the material will be recompiled with it.
	 * @return true if the material can be used for rendering with the given flag.
	 */
	ENGINE_API bool CheckMaterialUsage(EMaterialUsage Usage);

	/** 
	 * Same as CheckMaterialUsage() but is valid to call from any thread. In the editor, this might spin and stall for a shader compile. 
	 * @return true if the material can be used for rendering with the given flag.
	 */
	ENGINE_API bool CheckMaterialUsage_Concurrent(EMaterialUsage Usage) const;

	/**
	 * Tests to see if this material needs a usage flag update.
	 * @return true if we need to call SetMaterialUsage.
	 */
	ENGINE_API bool NeedsSetMaterialUsage_Concurrent(bool &bOutHasUsage, EMaterialUsage Usage) const;
	
	/**
	 * Get the static permutation resource if the instance has one
	 * @return - the appropriate FMaterialResource if one exists, otherwise NULL
	 */
	UE_DEPRECATED(5.7, "Please use GetMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	virtual FMaterialResource* GetMaterialResource(ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) final { return NULL; }
	virtual FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) { return NULL; }

	/**
	 * Get the static permutation resource if the instance has one
	 * @return - the appropriate FMaterialResource if one exists, otherwise NULL
	 */
	UE_DEPRECATED(5.7, "Please use GetMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	virtual const FMaterialResource* GetMaterialResource(ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) const final { return NULL; }
	virtual const FMaterialResource* GetMaterialResource(EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::Num) const { return NULL; }

	/**
	 * Get the Material Resources created by the given Material, restricted to the given Quality Range and Shader Platform Range
	 * @return - The appropriate FMaterialResources, if any exist
	 */
	static TArray<const FMaterialResource*, TInlineAllocator<4>> GetMaterialResources(const UMaterialInterface* InMaterial, const FInt32Range InQualityLevelRange, const FInt32Range InShaderPlatformRange);

	/**
	 * Get the material layers stack
	 * @return - material layers stack, or nullptr if material has no layers
	 */
	virtual bool GetMaterialLayers(FMaterialLayersFunctions& OutLayers, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const PURE_VIRTUAL(UMaterialInterface::GetMaterialLayers, return false;);

	/**
	 * Get the associated nanite override material
	 * @return - nanite override, if none was set returns null as a signal to use this material instead
	 */
	virtual UMaterialInterface* GetNaniteOverride(TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) const PURE_VIRTUAL(UMaterialInterface::GetNaniteOverride, return nullptr;);

	/** Get the associated nanite override material. */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	UMaterialInterface* GetNaniteOverideMaterial() const { return GetNaniteOverride(); }

	/**
	 * Precache PSOs which can be used for this material for the given vertex factory type and material paramaters
	 */
	FGraphEventArray PrecachePSOs(const FVertexFactoryType* VertexFactoryType, const struct FPSOPrecacheParams& PreCacheParams)
	{
		return PrecachePSOs(MakeArrayView(&VertexFactoryType, 1), PreCacheParams);
	}
	FGraphEventArray PrecachePSOs(const TConstArrayView<const FVertexFactoryType*>& VertexFactoryTypes, const struct FPSOPrecacheParams& PreCacheParams)
	{
		TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
		return PrecachePSOs(VertexFactoryTypes, PreCacheParams, MaterialPSOPrecacheRequestIDs);
	}

	FGraphEventArray PrecachePSOs(const TConstArrayView<const FVertexFactoryType*>& VertexFactoryTypes, const struct FPSOPrecacheParams& PreCacheParams, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
	{
		return PrecachePSOs(VertexFactoryTypes, PreCacheParams, EPSOPrecachePriority::Medium, OutMaterialPSORequestIDs);
	}

	FGraphEventArray PrecachePSOs(const TConstArrayView<const FVertexFactoryType*>& VertexFactoryTypes, const struct FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority PSOPrecachePriority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs)
	{ 
		FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
		VertexFactoryDataList.SetNum(VertexFactoryTypes.Num());
		for (int i = 0; i < VertexFactoryTypes.Num(); ++i)
		{
			VertexFactoryDataList[i].VertexFactoryType = VertexFactoryTypes[i];
		}
		return PrecachePSOs(VertexFactoryDataList, PreCacheParams, PSOPrecachePriority, OutMaterialPSORequestIDs);
	}
	virtual FGraphEventArray PrecachePSOs(const FPSOPrecacheVertexFactoryDataList& VertexFactoryDataList, const struct FPSOPrecacheParams& PreCacheParams, EPSOPrecachePriority Priority, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSORequestIDs) { return FGraphEventArray(); }

#if WITH_EDITORONLY_DATA
	/**
	* Builds a composited set of static parameters, including inherited and overridden values
	*/
	ENGINE_API void GetStaticParameterValues(FStaticParameterSet& OutStaticParameters);

	/**
	* Get the value of the given static switch parameter
	*
	* @param	ParameterName	The name of the static switch parameter
	* @param	OutValue		Will contain the value of the parameter if successful
	* @return					True if successful
	*/
	ENGINE_API bool GetStaticSwitchParameterValue(const FHashedMaterialParameterInfo& ParameterInfo,bool &OutValue,FGuid &OutExpressionGuid, bool bOveriddenOnly = false) const;

	/**
	* Get the value of the given static component mask parameter
	*
	* @param	ParameterName	The name of the parameter
	* @param	R, G, B, A		Will contain the values of the parameter if successful
	* @return					True if successful
	*/
	ENGINE_API bool GetStaticComponentMaskParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& R, bool& G, bool& B, bool& A, FGuid& OutExpressionGuid, bool bOveriddenOnly = false) const;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/**
	* Get the sort priority index of the given parameter
	*
	* @param	ParameterName	The name of the parameter
	* @param	OutSortPriority	Will contain the sort priority of the parameter if successful
	* @return					True if successful
	*/
	ENGINE_API bool GetParameterSortPriority(const FHashedMaterialParameterInfo& ParameterInfo, int32& OutSortPriority) const;

	/**
	* Get the sort priority index of the given parameter group
	*
	* @param	InGroupName	The name of the parameter group
	* @param	OutSortPriority	Will contain the sort priority of the parameter group if successful
	* @return					True if successful
	*/
	virtual bool GetGroupSortPriority(const FString& InGroupName, int32& OutSortPriority) const
		PURE_VIRTUAL(UMaterialInterface::GetGroupSortPriority, return false;);
#endif // WITH_EDITOR

	ENGINE_API virtual void GetAllParameterInfoOfType(EMaterialParameterType Type, TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API virtual void GetAllParametersOfType(EMaterialParameterType Type, TMap<FMaterialParameterInfo, FMaterialParameterMetadata>& OutParameters) const;
	ENGINE_API void GetAllScalarParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllVectorParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllDoubleVectorParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllTextureCollectionParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllParameterCollectionParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllRuntimeVirtualTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllSparseVolumeTextureParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllFontParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;

#if WITH_EDITORONLY_DATA
	ENGINE_API void GetAllStaticSwitchParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;
	ENGINE_API void GetAllStaticComponentMaskParameterInfo(TArray<FMaterialParameterInfo>& OutParameterInfo, TArray<FGuid>& OutParameterIds) const;

	virtual bool IterateDependentFunctions(TFunctionRef<bool(UMaterialFunctionInterface*)> Predicate) const
		PURE_VIRTUAL(UMaterialInterface::IterateDependentFunctions,return false;);
	virtual void GetDependentFunctions(TArray<class UMaterialFunctionInterface*>& DependentFunctions) const
		PURE_VIRTUAL(UMaterialInterface::GetDependentFunctions,return;);
#endif // WITH_EDITORONLY_DATA

	ENGINE_API bool GetParameterDefaultValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue) const;
	ENGINE_API bool GetScalarParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue) const;
	ENGINE_API bool GetVectorParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue) const;
	ENGINE_API bool GetDoubleVectorParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, FVector4d& OutValue) const;
	ENGINE_API bool GetTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue) const;
	ENGINE_API bool GetTextureCollectionParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue) const;
	ENGINE_API bool GetParameterCollectionParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection*& OutValue) const;
	ENGINE_API bool GetRuntimeVirtualTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture*& OutValue) const;
	ENGINE_API bool GetSparseVolumeTextureParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture*& OutValue) const;
	ENGINE_API bool GetFontParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, class UFont*& OutFontValue, int32& OutFontPage) const;
	
#if WITH_EDITOR
	ENGINE_API bool GetStaticSwitchParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, FGuid& OutExpressionGuid) const;
	ENGINE_API bool GetStaticComponentMaskParameterDefaultValue(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutR, bool& OutG, bool& OutB, bool& OutA, FGuid& OutExpressionGuid) const;

	/** Add to the set any texture referenced by expressions, including nested functions, as well as any overrides from parameters. */
	ENGINE_API virtual void GetReferencedTexturesAndOverrides(TSet<const UTexture*>& InOutTextures) const;
#endif // WITH_EDITOR

	ENGINE_API TArrayView<const TObjectPtr<UObject>> GetReferencedTextures() const;

	/** @return The default texture relevant to the given Idx. At runtime, if ShouldStripUnusedDefaultTextures returns true, these are present based the compilation output. */
	virtual TObjectPtr<UObject> GetDefaultTextureAtIdx(const int32 InIdx, TMicRecursionGuard RecursionGuard = TMicRecursionGuard()) PURE_VIRTUAL(UMaterialInterface::GetDefaultTextureAtIdx, return nullptr;);

	ENGINE_API TConstArrayView<TObjectPtr<UTextureCollection>> GetReferencedTextureCollections() const;

	virtual void SaveShaderStableKeysInner(const class ITargetPlatform* TP, const struct FStableShaderKeyAndValue& SaveKeyVal)
		PURE_VIRTUAL(UMaterialInterface::SaveShaderStableKeysInner, );

	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API FMaterialParameterInfo GetParameterInfo(EMaterialParameterAssociation Association, FName ParameterName, UMaterialFunctionInterface* LayerFunction) const;

	/** @return The material's relevance. */
	UE_DEPRECATED(5.7, "Please use GetRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API FMaterialRelevance GetRelevance(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API FMaterialRelevance GetRelevance(EShaderPlatform ShaderPlatform) const;

	/** @return The material's relevance, from concurrent render thread updates. */
	UE_DEPRECATED(5.7, "Please use GetRelevance_Concurrent with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API FMaterialRelevance GetRelevance_Concurrent(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API FMaterialRelevance GetRelevance_Concurrent(EShaderPlatform ShaderPlatform) const;

#if !WITH_EDITOR
	/**
	 * Pre-warm the cached relevance on GT. No-op if already cached.
	 * Generic workers (e.g., FastGeo async mode) use the cached value via GetRelevance_Concurrent
	 * to avoid reading GameThreadShaderMap which is not safe from generic workers.
	 */
	void WarmRelevanceCache(EShaderPlatform InShaderPlatform);
#endif

	/** @return The material's relevance bUsesWorldPositionOffset, from concurrent render thread updates. */
	UE_DEPRECATED(5.7, "Please use IsUsingWorldPositionOffset_Concurrent with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API bool IsUsingWorldPositionOffset_Concurrent(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API bool IsUsingWorldPositionOffset_Concurrent(EShaderPlatform InShaderPlatform) const;

	ENGINE_API bool IsUsingTemporalResponsiveness_Concurrent(EShaderPlatform InShaderPlatform) const;
	ENGINE_API bool IsUsingMotionVectorWorldOffset_Concurrent(EShaderPlatform InShaderPlatform) const;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/**
	 * Output to the log which materials and textures are used by this material.
	 * @param Indent	Number of tabs to put before the log.
	 */
	virtual void LogMaterialsAndTextures(FOutputDevice& Ar, int32 Indent) const {}
#endif

	virtual void DumpDebugInfo(FOutputDevice& OutputDevice) const {}

private:
	// might get called from game or render thread
	UE_DEPRECATED(5.7, "Please use GetRelevance_Internal with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	FMaterialRelevance GetRelevance_Internal(const UMaterial* Material, ERHIFeatureLevel::Type InFeatureLevel) const;
	FMaterialRelevance GetRelevance_Internal(const UMaterial* Material, EShaderPlatform ShaderPlatform) const;

public:

	int32 GetWidth() const;
	int32 GetHeight() const;

	const FGuid& GetLightingGuid() const
	{
#if WITH_EDITORONLY_DATA
		return LightingGuid;
#else
		static const FGuid NullGuid( 0, 0, 0, 0 );
		return NullGuid; 
#endif // WITH_EDITORONLY_DATA
	}

	void SetLightingGuid()
	{
#if WITH_EDITORONLY_DATA
		LightingGuid = FGuid::NewGuid();
#endif // WITH_EDITORONLY_DATA
	}

	/**
	 *	Returns all the Guids related to this material. For material instances, this includes the parent hierarchy.
	 *  Used for versioning as parent changes don't update the child instance Guids.
	 *
	 *	@param	bIncludeTextures	Whether to include the referenced texture Guids.
	 *	@param	OutGuids			The list of all resource guids affecting the precomputed lighting system and texture streamer.
	 */
	ENGINE_API virtual void GetLightingGuidChain(bool bIncludeTextures, TArray<FGuid>& OutGuids) const;

#if WITH_EDITOR
	ENGINE_API virtual uint32 ComputeAllStateCRC() const;
#endif // WITH_EDITOR

	/**
	 *	Check if the textures have changed since the last time the material was
	 *	serialized for Lightmass... Update the lists while in here.
	 *	NOTE: This will mark the package dirty if they have changed.
	 *
	 *	@return	bool	true if the textures have changed.
	 *					false if they have not.
	 */
	virtual bool UpdateLightmassTextureTracking() 
	{ 
		return false; 
	}
	
	/** @return The override bOverrideCastShadowAsMasked setting of the material. */
	inline bool GetOverrideCastShadowAsMasked() const
	{
		return LightmassSettings.bOverrideCastShadowAsMasked;
	}

	/** @return The override emissive boost setting of the material. */
	inline bool GetOverrideEmissiveBoost() const
	{
		return LightmassSettings.bOverrideEmissiveBoost;
	}

	/** @return The override diffuse boost setting of the material. */
	inline bool GetOverrideDiffuseBoost() const
	{
		return LightmassSettings.bOverrideDiffuseBoost;
	}

	/** @return The override export resolution scale setting of the material. */
	inline bool GetOverrideExportResolutionScale() const
	{
		return LightmassSettings.bOverrideExportResolutionScale;
	}

	/** @return	The bCastShadowAsMasked value for this material. */
	virtual bool GetCastShadowAsMasked() const
	{
		return LightmassSettings.bCastShadowAsMasked;
	}

	/** @return	The Emissive boost value for this material. */
	virtual float GetEmissiveBoost() const
	{
		return 
		LightmassSettings.EmissiveBoost;
	}

	/** @return	The Diffuse boost value for this material. */
	virtual float GetDiffuseBoost() const
	{
		return LightmassSettings.DiffuseBoost;
	}

	/** @return	The ExportResolutionScale value for this material. */
	virtual float GetExportResolutionScale() const
	{
		return FMath::Clamp(LightmassSettings.ExportResolutionScale, .1f, 10.0f);
	}

	/** @param	bInOverrideCastShadowAsMasked	The override CastShadowAsMasked setting to set. */
	inline void SetOverrideCastShadowAsMasked(bool bInOverrideCastShadowAsMasked)
	{
		LightmassSettings.bOverrideCastShadowAsMasked = bInOverrideCastShadowAsMasked;
	}

	/** @param	bInOverrideEmissiveBoost	The override emissive boost setting to set. */
	inline void SetOverrideEmissiveBoost(bool bInOverrideEmissiveBoost)
	{
		LightmassSettings.bOverrideEmissiveBoost = bInOverrideEmissiveBoost;
	}

	/** @param bInOverrideDiffuseBoost		The override diffuse boost setting of the parent material. */
	inline void SetOverrideDiffuseBoost(bool bInOverrideDiffuseBoost)
	{
		LightmassSettings.bOverrideDiffuseBoost = bInOverrideDiffuseBoost;
	}

	/** @param bInOverrideExportResolutionScale	The override export resolution scale setting of the parent material. */
	inline void SetOverrideExportResolutionScale(bool bInOverrideExportResolutionScale)
	{
		LightmassSettings.bOverrideExportResolutionScale = bInOverrideExportResolutionScale;
	}

	/** @param	InCastShadowAsMasked	The CastShadowAsMasked value for this material. */
	inline void SetCastShadowAsMasked(bool InCastShadowAsMasked)
	{
		LightmassSettings.bCastShadowAsMasked = InCastShadowAsMasked;
	}

	/** @param	InEmissiveBoost		The Emissive boost value for this material. */
	inline void SetEmissiveBoost(float InEmissiveBoost)
	{
		LightmassSettings.EmissiveBoost = InEmissiveBoost;
	}

	/** @param	InDiffuseBoost		The Diffuse boost value for this material. */
	inline void SetDiffuseBoost(float InDiffuseBoost)
	{
		LightmassSettings.DiffuseBoost = InDiffuseBoost;
	}

	/** @param	InExportResolutionScale		The ExportResolutionScale value for this material. */
	inline void SetExportResolutionScale(float InExportResolutionScale)
	{
		LightmassSettings.ExportResolutionScale = InExportResolutionScale;
	}

#if WITH_EDITOR
	/**
	 *	Get all of the textures in the expression chain for the given property (ie fill in the given array with all textures in the chain).
	 *
	 *	@param	InProperty				The material property chain to inspect, such as MP_BaseColor.
	 *	@param	OutTextures				The array to fill in all of the textures.
	 *	@param	OutTextureParamNames	Optional array to fill in with texture parameter names.
	 *	@param	InStaticParameterSet	Optional static parameter set - if specified only follow StaticSwitches according to its settings
	 *
	 *	@return	bool			true if successful, false if not.
	 */
	virtual bool GetTexturesInPropertyChain(EMaterialProperty InProperty, TArray<UTexture*>& OutTextures,  TArray<FName>* OutTextureParamNames, struct FStaticParameterSet* InStaticParameterSet,
		ERHIFeatureLevel::Type InFeatureLevel = ERHIFeatureLevel::Num, EMaterialQualityLevel::Type InQuality = EMaterialQualityLevel::Num)
		PURE_VIRTUAL(UMaterialInterface::GetTexturesInPropertyChain,return false;);

	ENGINE_API bool GetGroupName(const FHashedMaterialParameterInfo& ParameterInfo, FName& GroupName) const;
	ENGINE_API bool GetParameterDesc(const FHashedMaterialParameterInfo& ParameterInfo, FString& OutDesc) const;
	ENGINE_API bool GetScalarParameterSliderMinMax(const FHashedMaterialParameterInfo& ParameterInfo, float& OutSliderMin, float& OutSliderMax) const;
#endif // WITH_EDITOR

	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FMemoryImageMaterialParameterInfo& ParameterInfo, FMaterialParameterMetadata& OutValue, EMaterialGetParameterValueFlags Flags = EMaterialGetParameterValueFlags::Default) const;

	ENGINE_API bool GetScalarParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, float& OutValue, bool bOveriddenOnly = false) const;
#if WITH_EDITOR
	ENGINE_API bool IsScalarParameterUsedAsAtlasPosition(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue, TSoftObjectPtr<class UCurveLinearColor>& Curve, TSoftObjectPtr<class UCurveLinearColorAtlas>&  Atlas) const;
#endif // WITH_EDITOR
	ENGINE_API bool GetVectorParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor& OutValue, bool bOveriddenOnly = false) const;
#if WITH_EDITOR
	ENGINE_API bool IsVectorParameterUsedAsChannelMask(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue) const;
	ENGINE_API bool GetVectorParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const;
#endif
	ENGINE_API bool GetDoubleVectorParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, FVector4d& OutValue, bool bOveriddenOnly = false) const;
#if WITH_EDITOR
	ENGINE_API bool IsDoubleVectorParameterUsedAsChannelMask(const FHashedMaterialParameterInfo& ParameterInfo, bool& OutValue) const;
	ENGINE_API bool GetDoubleVectorParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const;
#endif // WITH_EDITOR
	ENGINE_API virtual bool GetTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTexture*& OutValue, bool bOveriddenOnly = false) const;
	ENGINE_API virtual bool GetTextureCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UTextureCollection*& OutValue, bool bOveriddenOnly = false) const;
	ENGINE_API virtual bool GetParameterCollectionParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class UMaterialParameterCollection*& OutValue, bool bOveriddenOnly = false) const;
	ENGINE_API bool GetRuntimeVirtualTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class URuntimeVirtualTexture*& OutValue, bool bOveriddenOnly = false) const;
	ENGINE_API bool GetSparseVolumeTextureParameterValue(const FHashedMaterialParameterInfo& ParameterInfo, class USparseVolumeTexture*& OutValue, bool bOveriddenOnly = false) const;
#if WITH_EDITOR
	ENGINE_API bool GetTextureParameterChannelNames(const FHashedMaterialParameterInfo& ParameterInfo, FParameterChannelNames& OutValue) const;
#endif
	ENGINE_API bool GetFontParameterValue(const FHashedMaterialParameterInfo& ParameterInfo,class UFont*& OutFontValue, int32& OutFontPage, bool bOveriddenOnly = false) const;

	ENGINE_API virtual bool GetRefractionSettings(float& OutBiasValue) const;
	ENGINE_API virtual bool GetUserSceneTextureOverride(FName& InOutValue) const;
	ENGINE_API FName GetUserSceneTextureOutput(const UMaterial* Base) const;
	ENGINE_API virtual EBlendableLocation GetBlendableLocation(const UMaterial* Base) const;
	ENGINE_API virtual int32 GetBlendablePriority(const UMaterial* Base) const;

	/**
		Access to overridable properties of the base material.
	*/
	ENGINE_API virtual float GetOpacityMaskClipValue() const;
	ENGINE_API virtual bool GetCastDynamicShadowAsMasked() const;
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual EBlendMode GetBlendMode() const;
	ENGINE_API virtual FMaterialShadingModelField GetShadingModels() const;
	ENGINE_API virtual bool IsShadingModelFromMaterialExpression() const;
	ENGINE_API virtual bool IsTwoSided() const;
	ENGINE_API virtual bool IsThinSurface() const;
	ENGINE_API virtual bool IsDitheredLODTransition() const;
	ENGINE_API virtual bool IsTranslucencyWritingCustomDepth() const;
	ENGINE_API virtual bool IsTranslucencyWritingVelocity() const;
	ENGINE_API virtual bool IsTranslucencyVelocityFromDepth() const;
	ENGINE_API virtual bool IsTranslucencyWritingFrontLayerTransparency() const;
	ENGINE_API virtual bool IsMasked() const;
	ENGINE_API virtual bool IsDeferredDecal() const;
	ENGINE_API virtual bool IsUIMaterial() const;
	ENGINE_API virtual bool IsPostProcessMaterial() const;
	ENGINE_API virtual bool IsLightFunctionMaterial() const;
	ENGINE_API virtual bool WritesToRuntimeVirtualTexture() const;
	ENGINE_API virtual bool HasMeshPaintTexture() const;
	ENGINE_API virtual bool HasCustomPrimitiveData() const;
	ENGINE_API virtual FDisplacementScaling GetDisplacementScaling() const;
	ENGINE_API virtual bool IsDisplacementFadeEnabled() const;
	ENGINE_API virtual FDisplacementFadeRange GetDisplacementFadeRange() const;
	ENGINE_API virtual float GetMaxWorldPositionOffsetDisplacement() const;
	ENGINE_API virtual bool ShouldAlwaysEvaluateWorldPositionOffset() const;
	ENGINE_API virtual bool HasVertexInterpolator() const;
	ENGINE_API virtual bool HasCustomizedUVs() const;
	ENGINE_API virtual bool HasPixelAnimation() const;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfile_Internal() const;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal() const;
	ENGINE_API virtual uint32 NumSubsurfaceProfileRoot_Internal() const;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileRoot_Internal(uint32 Index) const;
	ENGINE_API virtual USubsurfaceProfile* GetSubsurfaceProfileOverride_Internal() const;
	ENGINE_API virtual uint32 NumSpecularProfile_Internal() const;
	ENGINE_API virtual USpecularProfile* GetSpecularProfile_Internal(uint32 Index) const;
	ENGINE_API virtual USpecularProfile* GetSpecularProfileOverride_Internal() const;
	ENGINE_API virtual uint32 NumToonProfile_Internal() const;
	ENGINE_API virtual UToonProfile* GetToonProfile_Internal(uint32 Index) const;
	ENGINE_API virtual UToonProfile* GetToonProfileOverride_Internal() const;
	ENGINE_API virtual UNeuralProfile* GetNeuralProfile_Internal() const;
	ENGINE_API virtual bool CastsRayTracedShadows() const;
	ENGINE_API virtual bool IsTessellationEnabled() const;
	ENGINE_API virtual bool HasSubstrateRoughnessTracking() const;
	ENGINE_API virtual bool IsCompatibleWithLumenCardSharing() const;

	/**
	 * Force the streaming system to disregard the normal logic for the specified duration and
	 * instead always load all mip-levels for all textures used by this material.
	 *
	 * @param OverrideForceMiplevelsToBeResident	- Whether to use (true) or ignore (false) the bForceMiplevelsToBeResidentValue parameter.
	 * @param bForceMiplevelsToBeResidentValue		- true forces all mips to stream in. false lets other factors decide what to do with the mips.
	 * @param ForceDuration							- Number of seconds to keep all mip-levels in memory, disregarding the normal priority logic. Negative value turns it off.
	 * @param CinematicTextureGroups				- Bitfield indicating texture groups that should use extra high-resolution mips
	 * @param bFastResponse							- USE WITH EXTREME CAUTION! Fast response textures incur sizable GT overhead and disturb streaming metric calculation. Avoid whenever possible.
	 */
	UFUNCTION(BlueprintCallable, Category = "Rendering|Material")
	ENGINE_API virtual void SetForceMipLevelsToBeResident( bool OverrideForceMiplevelsToBeResident, bool bForceMiplevelsToBeResidentValue, float ForceDuration, int32 CinematicTextureGroups = 0, bool bFastResponse = false );

	/**
	 * Re-caches uniform expressions for all material interfaces
	 * Set bRecreateUniformBuffer to true if uniform buffer layout will change (e.g. FMaterial is being recompiled).
	 * In that case calling needs to use FMaterialUpdateContext to recreate the rendering state of primitives using this material.
	 * 
	 * @param bRecreateUniformBuffer - true forces uniform buffer recreation.
	 */
	ENGINE_API static void RecacheAllMaterialUniformExpressions(bool bRecreateUniformBuffer);

	/**
	 * @brief Submits shaders to be compiled for all the materials in the world.
	 *
	 * This function will submit any remaining shaders to be compiled for all the materials in the passed in world.  By default
	 * these shader compilation jobs will be compiled in the background so if you need the results immediately you can call
	 * FinishAllCompilation() to block on the results.
	 *
	 * If the world is nullptr this will submit remaining shaders to be compiled for all the loaded materials.
	 *
	 * @code
	 * GShaderCompilingManager->SubmitRemainingJobsForWorld(World);
	 * GShaderCompilingManager->FinishAllCompilation();
	 * @endcode
	 *
	 * @param World Only compile shaders for materials that are used on primitives in this world.
	 * @param CompileMode Controls whether or not we block on the shader compile results.
	 *
	 * @note This will only submit shader compile jobs for missing shaders on each material.  Calling this multiple times on the same world
	 * will result in a no-op.
	 */
	ENGINE_API static void SubmitRemainingJobsForWorld(UWorld* World, EMaterialShaderPrecompileMode CompileMode = EMaterialShaderPrecompileMode::Default);

	/**
	 * Re-caches uniform expressions for this material interface                   
	 * Set bRecreateUniformBuffer to true if uniform buffer layout will change (e.g. FMaterial is being recompiled).
	 * In that case calling needs to use FMaterialUpdateContext to recreate the rendering state of primitives using this material.
	 *
	 * @param bRecreateUniformBuffer - true forces uniform buffer recreation.
	 */
	virtual void RecacheUniformExpressions(bool bRecreateUniformBuffer) const {}

	/** @brief Submits remaining shaders for recompilation.
	*
	* This function will submit any remaining shaders to be compiled for the given material.  By default
	* these shader compilation jobs will be compiled in the background.
	*
	* @param CompileMode Controls whether or not we block on the shader compile results.
	*/
	virtual void CacheShaders(EMaterialShaderPrecompileMode CompileMode = EMaterialShaderPrecompileMode::Default) {}

#if WITH_EDITOR
	virtual void CacheGivenTypesForCooking(EShaderPlatform Platform, ERHIFeatureLevel::Type FeatureLevel, EMaterialQualityLevel::Type QualityLevel, const TArray<const FVertexFactoryType*>& VFTypes, const TArray<const FShaderPipelineType*> PipelineTypes, const TArray<const FShaderType*>& ShaderTypes) {}
	virtual void AppendCompileStateDebugInfo(FStringBuilderBase& OutDebugInfo) const {}
#endif

	/** @brief Checks to see if this material has all its shaders cached.
	*
	* Materials are not guaranteed to have all their shaders compiled after loading.  It can be useful to
	* check for completeness in order to cache remaining shaders.
	* 
	* @return Whether or not all shaders for this material exist.
	*
	* @see CacheShaders
	* @note This function will return true if the resources are not cache for this material yet.
	*/
	virtual bool IsComplete() const { return true; }

#if WITH_EDITOR
	virtual bool IsCompiling() const { return false; };
#else
	inline bool IsCompiling() const { return false; }
#endif

	/** @brief Checks to see if this material has all its shaders cached and if not, will perform a synchronous compilation of those.
	*
	* Materials are not guaranteed to have all their shaders compiled after loading and using this function before a draw will ensure that the material will not render until it's 
	* ready to (and use a fallback material instead). This needs to be avoided in the common render path but can be useful in critical tools (e.g. landscape) that rely on the 
	* material and cannot afford a fallback material. 
	* In the editor, it will display a toast when waiting for the shaders to compile.
	*
	* @see CacheShaders
	*/
	ENGINE_API void EnsureIsComplete();

#if WITH_EDITOR
	/** Clears the shader cache and recompiles the shader for rendering. */
	virtual void ForceRecompileForRendering(EMaterialShaderPrecompileMode CompileMode = EMaterialShaderPrecompileMode::Default) {}
#endif // WITH_EDITOR

	/**
	 * Asserts if any default material does not exist.
	 */
	ENGINE_API static void AssertDefaultMaterialsExist();

	/**
	 * Asserts if any default material has not been post-loaded.
	 */
	ENGINE_API static void AssertDefaultMaterialsPostLoaded();

	/**
	 * Initializes all default materials.
	 */
	ENGINE_API static void InitDefaultMaterials();

	/**
	 * Check if default materials as been initialized.
	 */
	ENGINE_API static bool IsDefaultMaterialInitialized();

	/**
	 * Precache PSOs for fallback material usage.
	 */
	ENGINE_API static void PrecacheFallbackMaterialPSOs(UMaterialInterface* Material);

	UE_DEPRECATED(5.8, "Please use PrecacheFallbackMaterialPSOs which provides a MaterialInterface")
	ENGINE_API static void PrecacheDefaultMaterialPSOs();

	/** Checks to see if an input property should be active, based on the state of the material */
	ENGINE_API virtual bool IsPropertyActive(EMaterialProperty InProperty) const;

#if WITH_EDITOR
	/** Compiles a material property. */
	ENGINE_API int32 CompileProperty(FMaterialCompiler* Compiler, EMaterialProperty Property, uint32 ForceCastFlags = 0);

	/** Allows material properties to be compiled with the option of being overridden by the material attributes input. */
	ENGINE_API virtual int32 CompilePropertyEx( class FMaterialCompiler* Compiler, const FGuid& AttributeID );

	/** True if this Material Interface should force a plane preview */
	virtual bool ShouldForcePlanePreview()
	{
		return bShouldForcePlanePreview;
	}
	
	/** Set whether or not this Material Interface should force a plane preview */
	void SetShouldForcePlanePreview(const bool bInShouldForcePlanePreview)
	{
		bShouldForcePlanePreview = bInShouldForcePlanePreview;
	};
#endif // WITH_EDITOR

	/** Get bitfield indicating which feature levels should be compiled by default */
	ENGINE_API static uint32 GetFeatureLevelsToCompileForAllMaterials();

	/** Return number of used texture coordinates and whether or not the Vertex data is used in the shader graph */
	ENGINE_API void AnalyzeMaterialProperty(EMaterialProperty InProperty, int32& OutNumTextureCoordinates, bool& bOutRequiresVertexData);

	/** Return insight on what (e.g. texture coordinates, vertex data, etc) is used in the shader graph of a material property */
	ENGINE_API void AnalyzeMaterialPropertyEx(EMaterialProperty InProperty, FMaterialAnalysisResult& OutResult);

	/** Return insight on what (e.g. texture coordinates, vertex data, etc) is used in the shader graph of a material custom output */
	ENGINE_API void AnalyzeMaterialCustomOutput(UMaterialExpressionCustomOutput* InCustomOutput, int32 InOutputIndex, FMaterialAnalysisResult& OutResult);

	/** Return insight on what (e.g. texture coordinates, vertex data, etc) is used in the shader graph compiled by a callback */
	ENGINE_API void AnalyzeMaterialCompilationInCallback(TFunctionRef<void (FMaterialCompiler*)> InCompilationCallback, FMaterialAnalysisResult& OutResult);

	/** Return insight on errors or warnings generated during translation */
	UE_DEPRECATED(5.7, "Please use AnalyzeMaterialTranslationOutput with ValidationMode argument")
	ENGINE_API void AnalyzeMaterialTranslationOutput(FMaterialResource* MaterialResource, EShaderPlatform ShaderPlatform, FMaterialAnalysisResult& OutResult);
	ENGINE_API void AnalyzeMaterialTranslationOutput(FMaterialResource* MaterialResource, EShaderPlatform ShaderPlatform, bool ValidationMode, FMaterialAnalysisResult& OutResult);

#if WITH_EDITOR
	/** Checks to see if the given property references the texture */
	ENGINE_API bool IsTextureReferencedByProperty(EMaterialProperty InProperty, const UTexture* InTexture);
#endif // WITH_EDITOR

	/** Iterate over all feature levels currently marked as active */
	template <typename FunctionType>
	static void IterateOverActiveFeatureLevels(FunctionType InHandler) 
	{  
		uint32 FeatureLevels = GetFeatureLevelsToCompileForAllMaterials();
		while (FeatureLevels != 0)
		{
			InHandler((ERHIFeatureLevel::Type)FBitSet::GetAndClearNextBit(FeatureLevels));
		}
	}

	/** Access the cached uenum type information for material sampler type */
	static UEnum* GetSamplerTypeEnum() 
	{ 
		check(SamplerTypeEnum); 
		return SamplerTypeEnum; 
	}

	/** Return whether this material refer to any streaming textures. */
	ENGINE_API bool UseAnyStreamingTexture() const;
	/** Returns whether there is any streaming data in the component. */
	inline bool HasTextureStreamingData() const { return TextureStreamingData.Num() != 0; }
	/** Accessor to the data. */
	inline const TArray<FMaterialTextureInfo>& GetTextureStreamingData() const { return TextureStreamingData; }
	inline TArray<FMaterialTextureInfo>& GetTextureStreamingData() { return TextureStreamingData; }
	/** Find entries within TextureStreamingData that match the given name. */
	ENGINE_API bool FindTextureStreamingDataIndexRange(FName TextureName, int32& LowerIndex, int32& HigherIndex) const;

	/** Set new texture streaming data. */
	ENGINE_API void SetTextureStreamingData(const TArray<FMaterialTextureInfo>& InTextureStreamingData);

	/**
	* Returns the density of a texture in (LocalSpace Unit / Texture). Used for texture streaming metrics.
	*
	* @param TextureName			The name of the texture to get the data for.
	* @param UVChannelData			The mesh UV density in (LocalSpace Unit / UV Unit).
	* @return						The density, or zero if no data is available for this texture.
	*/
	ENGINE_API virtual float GetTextureDensity(FName TextureName, const struct FMeshUVChannelInfo& UVChannelData) const;

#if !WITH_EDITOR
    UE_DEPRECATED(5.8, "Use FTextureSamplingInfo::GetTextureDensity(const FMeshUVChannelInfo&) on the FTextureSamplingInfo directly.")
    float GetTextureDensityWithCache(const FTextureSamplingInfo& TextureSamplingInfo, const FMeshUVChannelInfo& UVChannelData) const { return 0.0f; }
#endif

	ENGINE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	/**
	* Sort the texture streaming data by names to accelerate search. Only sorts if required.
	*
	* @param bForceSort			If true, force the operation even though the data might be already sorted.
	* @param bFinalSort			If true, the means there won't be any other sort after. This allows to remove null entries (platform dependent).
	*/
	ENGINE_API void SortTextureStreamingData(bool bForceSort, bool bFinalSort);

#if WITH_EDITOR
	/**
	*	Gathers a list of shader types sorted by vertex factory types that should be cached for this material.  Avoids doing expensive material
	*	and shader compilation to acquire this information.
	*
	*	@param	Platform		The shader platform to get info for.
	*   @param  TargetPlatform	The target platform to get info for (e.g. WindowsClient). Various target platforms can share the same ShaderPlatform.
	*	@param	OutShaderInfo	Array of results sorted by vertex factory type, and shader type.
	*
	*/
	virtual void GetShaderTypes(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform, TArray<FDebugShaderTypeInfo>& OutShaderInfo) {};
#endif // WITH_EDITOR

	/** Filter out ShadingModels field to a shader platform settings */
	ENGINE_API static void FilterOutPlatformShadingModels(EShaderPlatform Platform, FMaterialShadingModelField& ShadingModels);

	/** Helper function for building a Range from a given Quality Level */
	static FInt32Range BuildQualityLevelRange(TOptional<EMaterialQualityLevel::Type> InQualityLevel);

	/** Helper function for building a Range from a given Shader Platform */
	static FInt32Range BuildShaderPlatformRange(TOptional<EShaderPlatform> InShaderPlatform);
	
#if WITH_EDITOR
	TUniquePtr<FMaterialInsights> MaterialInsight;
#endif // WITH_EDITOR

protected:
	/** Returns a bitfield indicating which feature levels should be compiled for rendering. GMaxRHIFeatureLevel is always present */
	ENGINE_API uint32 GetFeatureLevelsToCompileForRendering() const;

	void UpdateMaterialRenderProxy(FMaterialRenderProxy& Proxy);

	/** Set if CachedExpressionData was loaded from disk, should typically be true when running with cooked data, and false in the editor */
	bool bLoadedCachedExpressionData = false;
	
	/**
	 * Cached data generated from the material's expressions, may be nullptr
	 * UMaterials should always have cached data
	 * UMaterialInstances will have cached data if they have overriden material layers (possibly for other reasons in the future)
	 */
	TUniquePtr<FMaterialCachedExpressionData> CachedExpressionData;

	/** Bitfield to track which usage flags have triggered warnings so that we only warn once per usage. */
	uint32 UsageFlagWarnings = 0;

#if WITH_EDITOR
	/* returns whether unused default textures should be removed during the cooking process. */
	static bool ShouldStripUnusedDefaultTextures();

	/* returns whether valid compilation output exists. Outputs all default textures used by the material, based on the compilation output. */
	virtual bool TryGetUsedTexturesInCookCache(TArray<TObjectPtr<UObject>>& OutTextures) PURE_VIRTUAL(UMaterialInterface::TryGetUsedTexturesInCookCache, return false;);
	
	/** @return All FMaterialResources created during the cooking process. */
	TArray<const FMaterialResource*, TInlineAllocator<4>> GetCookedCacheMaterialResources() const;

	/** Removes all unused default textures based on the Cook Shader Compilation output. */
	void StripUnusedDefaultTextures(const bool bInKeepProjectPrivateTextures);
#endif

private:
	/** Bitfield to track game thread dispatch of usage flag checks. */
	mutable std::atomic<uint32> UsageFlagCacheMiss { 0u };

	/**
	 * Post loads all default materials.
	 */
	void PostLoadDefaultMaterials();

	/**
	* Cached type information for the sampler type enumeration. 
	*/
	static UEnum* SamplerTypeEnum;

#if WITH_EDITOR
public:
	/**
	 *	Set the bMarkAsEditorStreamingPool on all textures used by this MaterialInterface.
	 *	@param bInMarkAsEditorStreamingPool		Whether textures should only be counted for the Editor pool stats.
	 */
	ENGINE_API void SetMarkTextureAsEditorStreamingPool(bool bInMarkAsEditorStreamingPool);

protected:
	mutable TOptional<FStaticParameterSet> CachedStaticParameterValues;
	mutable uint8 AllowCachingStaticParameterValuesCounter = 0;

private:
	/**
	* Whether or not this material interface should force the preview to be a plane mesh.
	*/
	bool bShouldForcePlanePreview;
#endif
};

namespace UE::MaterialInterface::Private
{

/** Helper function to serialize inline shader maps for the given material resources. */
extern void SerializeInlineShaderMaps(
	FArchive& Ar,
	TArray<FMaterialResource>& OutLoadedResources,
	const FName& SerializingAsset = NAME_None
#if WITH_EDITOR
	, const TMap<const ITargetPlatform*, TArray<FMaterialResourceForCooking>>* PlatformMaterialResourcesToSave = nullptr
#endif
);

}

/** Helper function to process (register) serialized inline shader maps for the given material resources. */
extern void ProcessSerializedInlineShaderMaps(UMaterialInterface* Owner, TArray<FMaterialResource>& LoadedResources, TArray<FMaterialResource*>& OutMaterialResourcesLoaded);

UE_DEPRECATED(5.7, "Please use FindMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
extern FMaterialResource* FindMaterialResource(const TArray<FMaterialResource*>& MaterialResources, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality);
UE_DEPRECATED(5.7, "Please use FindMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
extern FMaterialResource* FindMaterialResource(TArray<FMaterialResource*>& MaterialResources, ERHIFeatureLevel::Type InFeatureLevel, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality);

extern FMaterialResource* FindMaterialResource(const TArray<FMaterialResource*>& MaterialResources, EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality);
extern FMaterialResource* FindMaterialResource(TArray<FMaterialResource*>& MaterialResources, EShaderPlatform InShaderPlatform, EMaterialQualityLevel::Type QualityLevel, bool bAllowDefaultQuality);

UE_DEPRECATED(5.7, "Please use FindOrCreateMaterialResource with EShaderPlatform argument and not ERHIFeatureLevel::Type")
ENGINE_API FMaterialResource* FindOrCreateMaterialResource(TArray<FMaterialResource*>& MaterialResources,
	UMaterial* OwnerMaterial,
	UMaterialInstance* OwnerMaterialInstance,
	ERHIFeatureLevel::Type InFeatureLevel,
	EMaterialQualityLevel::Type QualityLevel);

ENGINE_API FMaterialResource* FindOrCreateMaterialResource(TArray<FMaterialResource*>& MaterialResources,
	UMaterial* OwnerMaterial,
	UMaterialInstance* OwnerMaterialInstance,
	EShaderPlatform InShaderPlatform,
	EMaterialQualityLevel::Type QualityLevel);

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Materials/MaterialIRModule.h"
#endif
