// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialParameters.h"
#include "MaterialExpression.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#include "MaterialLayersFunctions.generated.h"

#define LOCTEXT_NAMESPACE "MaterialLayersFunctions"

class FArchive;
class FShaderKeyGenerator;
class FXxHash64Builder;
struct FMaterialLayersFunctions;
class UMaterialExpressionMaterialFunctionCall;

UENUM()
enum class EMaterialLayerLinkState : uint8
{
	Uninitialized = 0u, // Saved with previous engine version
	LinkedToParent, // Layer should mirror changes from parent material
	UnlinkedFromParent, // Layer is based on parent material, but should not mirror changes
	NotFromParent, // Layer was created locally in this material, not in parent
};

/** Serializable ID structure for FMaterialLayersFunctions which allows us to deterministically recompile shaders*/
struct FMaterialLayersFunctionsID
{
	TArray<FGuid> LayerIDs;
	TArray<FGuid> BlendIDs;
	TArray<bool> LayerStates;

	#if WITH_EDITOR
	bool operator==(const FMaterialLayersFunctionsID& Reference) const;
	inline bool operator!=(const FMaterialLayersFunctionsID& Reference) const { return !operator==(Reference); }

	void SerializeForDDC(FArchive& Ar);

	friend FMaterialLayersFunctionsID& operator<<(FArchive& Ar, FMaterialLayersFunctionsID& Ref)
	{
		Ref.SerializeForDDC(Ar);
		return Ref;
	}

	void UpdateHash(FXxHash64Builder& HashState) const;

	//TODO: Investigate whether this is really required given it is only used by FMaterialShaderMapId AND that one also uses UpdateHash
	void AppendKeyString(FString& KeyString) const;
	void Append(FShaderKeyGenerator& KeyGen) const;
	#endif
};

USTRUCT()
struct FMaterialLayersFunctionsEditorOnlyData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<bool> LayerStates;

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<FText> LayerNames;

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<bool> RestrictToLayerRelatives;

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<bool> RestrictToBlendRelatives;

	/** Guid that identifies each layer in this stack */
	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<FGuid> LayerGuids;

	/**
	 * State of each layer's link to parent material
	 */
	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<EMaterialLayerLinkState> LayerLinkStates;

	/**
	 * List of Guids that exist in the parent material that have been explicitly deleted
	 * This is needed to distinguish these layers from newly added layers in the parent material
	 */
	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<FGuid> DeletedParentLayerGuids;

#if WITH_EDITORONLY_DATA
	inline bool operator==(const FMaterialLayersFunctionsEditorOnlyData& Other) const
	{
		if (LayerStates != Other.LayerStates ||
			LayerLinkStates != Other.LayerLinkStates ||
			DeletedParentLayerGuids != Other.DeletedParentLayerGuids)
		{
			return false;
		}
		return true;
	}

	inline bool operator!=(const FMaterialLayersFunctionsEditorOnlyData& Other) const
	{
		return !operator==(Other);
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	void Empty()
	{
		LayerStates.Empty();
		LayerNames.Empty();
		RestrictToLayerRelatives.Empty();
		RestrictToBlendRelatives.Empty();
		LayerGuids.Empty();
		LayerLinkStates.Empty();
	}

	void LinkAllLayersToParent();
#endif // WITH_EDITOR
};

struct FMaterialLayerStackFunctionsCache
{
	//Helper struct for managing available layers and blends across parent/instance materials utilising the LayerStack node.
	TArray<FSoftObjectPath> AvailableLayerPaths;
	TArray<FSoftObjectPath> AvailableBlendPaths;
};

USTRUCT()
struct FMaterialLayersFunctionsRuntimeData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<TObjectPtr<class UMaterialFunctionInterface>> Layers;

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	TArray<TObjectPtr<class UMaterialFunctionInterface>> Blends;

	// In multiple locations there are const references to this runtime data/it's children, 
	// so this mutable entry is required to ensure we can keep available layers/blend up to date across all instances.
	mutable TSharedPtr<FMaterialLayerStackFunctionsCache> LayerStackCache;

	FMaterialLayersFunctionsRuntimeData() = default;
	FMaterialLayersFunctionsRuntimeData(const FMaterialLayersFunctionsRuntimeData& Rhs)
		: Layers(Rhs.Layers)
		, Blends(Rhs.Blends)
		, LayerStackCache(Rhs.LayerStackCache)
	{}

	FMaterialLayersFunctionsRuntimeData(const FMaterialLayersFunctions& Rhs) = delete;

	FMaterialLayersFunctionsRuntimeData& operator=(const FMaterialLayersFunctionsRuntimeData& Rhs)
	{
		Layers = Rhs.Layers;
		Blends = Rhs.Blends;
		LayerStackCache = Rhs.LayerStackCache;
		return *this;
	}

	FMaterialLayersFunctionsRuntimeData& operator=(const FMaterialLayersFunctions& Rhs) = delete;

	ENGINE_API ~FMaterialLayersFunctionsRuntimeData();

	void Empty()
	{
		Layers.Empty();
		Blends.Empty();
		LayerStackCache.Reset();
	}

	inline bool operator==(const FMaterialLayersFunctionsRuntimeData& Other) const
	{
		if (Layers != Other.Layers || Blends != Other.Blends)
		{
			return false;
		}

		if (LayerStackCache || Other.LayerStackCache)
		{
			if (!LayerStackCache || !Other.LayerStackCache 
				|| LayerStackCache->AvailableLayerPaths != Other.LayerStackCache->AvailableLayerPaths
				|| LayerStackCache->AvailableBlendPaths != Other.LayerStackCache->AvailableBlendPaths)
			{
				return false;
			}
		}
		return true;
	}

	inline bool operator!=(const FMaterialLayersFunctionsRuntimeData& Other) const
	{
		return !operator==(Other);
	}

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	
#if WITH_EDITOR
	const FMaterialLayersFunctionsID GetID(const FMaterialLayersFunctionsEditorOnlyData& EditorOnly) const;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
private:
	/** FMaterialLayersFunctionsRuntimeData can be deserialized from an FMaterialLayersFunctions property, will store the editor-only portion here */
	TUniquePtr<FMaterialLayersFunctionsEditorOnlyData> LegacySerializedEditorOnlyData;



	friend struct FStaticParameterSet;
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FMaterialLayersFunctionsRuntimeData> : TStructOpsTypeTraitsBase2<FMaterialLayersFunctionsRuntimeData>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};

USTRUCT()
struct FMaterialLayersFunctions : public FMaterialLayersFunctionsRuntimeData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITOR
	using ID = FMaterialLayersFunctionsID;
#endif // WITH_EDITOR

	static ENGINE_API const FGuid BackgroundGuid;

	FMaterialLayersFunctions() = default;
	FMaterialLayersFunctions(const FMaterialLayersFunctionsRuntimeData&) = delete;

	FMaterialLayersFunctionsRuntimeData& GetRuntime() { return *this; }
	const FMaterialLayersFunctionsRuntimeData& GetRuntime() const { return *this; }

	UPROPERTY(EditAnywhere, Category = MaterialLayers)
	FMaterialLayersFunctionsEditorOnlyData EditorOnly;

	void Empty()
	{
		FMaterialLayersFunctionsRuntimeData::Empty();
#if WITH_EDITOR
		EditorOnly.Empty();
#endif
	}

	inline bool IsEmpty() const { return Layers.Num() == 0; }

#if WITH_EDITOR
	ENGINE_API void AddDefaultBackgroundLayer();

	ENGINE_API int32 AppendBlendedLayer();

	ENGINE_API int32 AddLayerCopy(const FMaterialLayersFunctionsRuntimeData& Source,
		const FMaterialLayersFunctionsEditorOnlyData& SourceEditorOnly,
		int32 SourceLayerIndex,
		bool bVisible,
		EMaterialLayerLinkState LinkState);

	int32 AddLayerCopy(const FMaterialLayersFunctions& Source,
		int32 SourceLayerIndex,
		bool bVisible,
		EMaterialLayerLinkState LinkState)
	{
		return AddLayerCopy(Source, Source.EditorOnly, SourceLayerIndex, bVisible, LinkState);
	}

	ENGINE_API void InsertLayerCopy(const FMaterialLayersFunctionsRuntimeData& Source,
		const FMaterialLayersFunctionsEditorOnlyData& SourceEditorOnly,
		int32 SourceLayerIndex,
		EMaterialLayerLinkState LinkState,
		int32 LayerIndex);

	void InsertLayerCopy(const FMaterialLayersFunctions& Source,
		int32 SourceLayerIndex,
		EMaterialLayerLinkState LinkState,
		int32 LayerIndex)
	{
		return InsertLayerCopy(Source, Source.EditorOnly, SourceLayerIndex, LinkState, LayerIndex);
	}

	ENGINE_API void RemoveBlendedLayerAt(int32 Index);

	ENGINE_API void MoveBlendedLayer(int32 SrcLayerIndex, int32 DstLayerIndex);

	const ID GetID() const { return FMaterialLayersFunctionsRuntimeData::GetID(EditorOnly); }

	/** Gets a string representation of the ID */
	ENGINE_API FString GetStaticPermutationString() const;
	ENGINE_API void AppendStaticPermutationKey(FShaderKeyGenerator& KeyGen) const;

	ENGINE_API void UnlinkLayerFromParent(int32 Index);
	ENGINE_API bool IsLayerLinkedToParent(int32 Index) const;
	ENGINE_API void RelinkLayersToParent();
	ENGINE_API bool HasAnyUnlinkedLayers() const;

	void ToggleBlendedLayerVisibility(int32 Index)
	{
		if (Index < 0)
			return;
		check(EditorOnly.LayerStates.IsValidIndex(Index));
		EditorOnly.LayerStates[Index] = !EditorOnly.LayerStates[Index];
	}

	void SetBlendedLayerVisibility(int32 Index, bool InNewVisibility)
	{
		if (Index < 0)
			return;
		check(EditorOnly.LayerStates.IsValidIndex(Index));
		EditorOnly.LayerStates[Index] = InNewVisibility;
	}

	bool GetLayerVisibility(int32 Index) const
	{
		if (Index < 0)
			return true;
		check(EditorOnly.LayerStates.IsValidIndex(Index));
		return EditorOnly.LayerStates[Index];
	}

	FText GetLayerName(int32 Counter) const
	{
		FText LayerName = FText::Format(LOCTEXT("LayerPrefix", "Layer {0}"), Counter);
		if (EditorOnly.LayerNames.IsValidIndex(Counter))
		{
			LayerName = EditorOnly.LayerNames[Counter];
		}
		return LayerName;
	}

	void LinkAllLayersToParent()
	{
		EditorOnly.LinkAllLayersToParent();
	}

	static ENGINE_API bool MatchesParent(const FMaterialLayersFunctionsRuntimeData& Runtime,
		const FMaterialLayersFunctionsEditorOnlyData& EditorOnly,
		const FMaterialLayersFunctionsRuntimeData& ParentRuntime,
		const FMaterialLayersFunctionsEditorOnlyData& ParentEditorOnly);

	bool MatchesParent(const FMaterialLayersFunctions& Parent) const
	{
		return MatchesParent(GetRuntime(), EditorOnly, Parent.GetRuntime(), Parent.EditorOnly);
	}

	static ENGINE_API bool ResolveParent(const FMaterialLayersFunctionsRuntimeData& ParentRuntime,
		const FMaterialLayersFunctionsEditorOnlyData& ParentEditorOnly,
		FMaterialLayersFunctionsRuntimeData& Runtime,
		FMaterialLayersFunctionsEditorOnlyData& EditorOnly,
		TArray<int32>& OutRemapLayerIndices);

	bool ResolveParent(const FMaterialLayersFunctions& Parent, TArray<int32>& OutRemapLayerIndices)
	{
		return FMaterialLayersFunctions::ResolveParent(Parent.GetRuntime(), Parent.EditorOnly, GetRuntime(), EditorOnly, OutRemapLayerIndices);
	}

	static ENGINE_API void Validate(const FMaterialLayersFunctionsRuntimeData& Runtime, const FMaterialLayersFunctionsEditorOnlyData& EditorOnly);

	void Validate()
	{
		Validate(GetRuntime(), EditorOnly);
	}

	ENGINE_API void SerializeLegacy(FArchive& Ar);

#endif // WITH_EDITOR

	ENGINE_API void PostSerialize(const FArchive& Ar);

	inline bool operator==(const FMaterialLayersFunctions& Other) const
	{
		if (!FMaterialLayersFunctionsRuntimeData::operator==(Other))
		{
			return false;
		}
#if WITH_EDITORONLY_DATA
		if (EditorOnly != Other.EditorOnly)
		{
			return false;
		}
#endif // WITH_EDITORONLY_DATA
		return true;
	}

	inline bool operator!=(const FMaterialLayersFunctions& Other) const
	{
		return !operator==(Other);
	}

private:
	UPROPERTY()
	TArray<bool> LayerStates_DEPRECATED;

	UPROPERTY()
	TArray<FText> LayerNames_DEPRECATED;

	UPROPERTY()
	TArray<bool> RestrictToLayerRelatives_DEPRECATED;

	UPROPERTY()
	TArray<bool> RestrictToBlendRelatives_DEPRECATED;

	UPROPERTY()
	TArray<FGuid> LayerGuids_DEPRECATED;

	UPROPERTY()
	TArray<EMaterialLayerLinkState> LayerLinkStates_DEPRECATED;

	UPROPERTY()
	TArray<FGuid> DeletedParentLayerGuids_DEPRECATED;


	// Don't allowing comparing a full FMaterialLayersFunctions against partial RuntimeData
	friend bool operator==(const FMaterialLayersFunctions&, const FMaterialLayersFunctionsRuntimeData&) = delete;
	friend bool operator==(const FMaterialLayersFunctionsRuntimeData&, const FMaterialLayersFunctions&) = delete;
	friend bool operator!=(const FMaterialLayersFunctions&, const FMaterialLayersFunctionsRuntimeData&) = delete;
	friend bool operator!=(const FMaterialLayersFunctionsRuntimeData&, const FMaterialLayersFunctions&) = delete;
};

template<>
struct TStructOpsTypeTraits<FMaterialLayersFunctions> : TStructOpsTypeTraitsBase2<FMaterialLayersFunctions>
{
	enum { WithPostSerialize = true };
};

#undef LOCTEXT_NAMESPACE
