// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialParameterCollection.h - defines an asset that has a list of parameters, which can be referenced by any material and updated efficiently at runtime
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/Guid.h"
#include "Templates/UniquePtr.h"
#include "RenderCommandFence.h"
#include "MaterialParameterCollection.generated.h"

class FShaderParametersMetadata;
struct FPropertyChangedEvent;

/** Base struct for collection parameters */
USTRUCT()
struct FCollectionParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	FCollectionParameterBase()
	{
		FPlatformMisc::CreateGuid(Id);
	}
	
	/** The name of the parameter.  Changing this name will break any blueprints that reference the parameter. */
	UPROPERTY(EditAnywhere, Category=Parameter)
	FName ParameterName;

	/** Uniquely identifies the parameter, used for fixing up materials that reference this parameter when renaming. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid Id;
};

/** A scalar parameter */
USTRUCT()
struct FCollectionScalarParameter : public FCollectionParameterBase
{
	GENERATED_USTRUCT_BODY()

	FCollectionScalarParameter()
		: DefaultValue(0.0f)
	{
		ParameterName = FName("Scalar");
	}

	UPROPERTY(EditAnywhere, Category=Parameter)
	float DefaultValue;
};

/** A vector parameter */
USTRUCT()
struct FCollectionVectorParameter : public FCollectionParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	FCollectionVectorParameter()
		: DefaultValue(ForceInitToZero)
	{
		ParameterName = FName("Vector");
	}
	
	UPROPERTY(EditAnywhere, Category=Parameter)
	FLinearColor DefaultValue;
};

/** 
 * Asset class that contains a list of parameter names and their default values. 
 * Any number of materials can reference these parameters and get new values when the parameter values are changed.
 */
UCLASS(hidecategories=object, MinimalAPI, BlueprintType)
class UMaterialParameterCollection : public UObject
{
	GENERATED_UCLASS_BODY()

	friend class FMaterialParameterCollectionDetails;

public:

	/** Used by materials using this collection to know when to recompile. */
	UPROPERTY(duplicatetransient)
	FGuid StateId;

	UPROPERTY(EditAnywhere, Category=Material, Meta = (TitleProperty = "ParameterName"))
	TArray<FCollectionScalarParameter> ScalarParameters;

	UPROPERTY(EditAnywhere, Category=Material, Meta = (TitleProperty = "ParameterName"))
	TArray<FCollectionVectorParameter> VectorParameters;
	
#if WITH_EDITOR
	/** Set the default value of a scalar parameter on the Material Parameter Collection asset itself by struct **/
	bool SetScalarParameterDefaultValueByInfo(FCollectionScalarParameter ScalarParameter);

	/** Set the default value of a scalar parameter on the Material Parameter Collection asset itself by name **/
	bool SetScalarParameterDefaultValue(FName ParameterName, const float Value);

	/** Set the default value of a vector parameter on the Material Parameter Collection asset itself by struct **/
	bool SetVectorParameterDefaultValueByInfo(FCollectionVectorParameter VectorParameter);
	
	/** Set the default value of a vector parameter on the Material Parameter Collection asset itself by name **/
	bool SetVectorParameterDefaultValue(FName ParameterName, const FLinearColor& Value);
#endif // WITH_EDITOR

	/** Get the index in the ScalarParameters array of the parameter matching the input parameter name, returns -1 if not found **/
	int32 GetScalarParameterIndexByName(FName ParameterName) const;

	/** Get the index in the VectorParameters array of the parameter matching the input parameter name, returns -1 if not found **/
	int32 GetVectorParameterIndexByName(FName ParameterName) const;

	/** Returns an array of the names of all the scalar parameters in this Material Parameter Collection **/
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetScalarParameterNames"))
	ENGINE_API TArray<FName> GetScalarParameterNames() const;

	/** Returns an array of the names of all the vector parameters in this Material Parameter Collection **/
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetVectorParameterNames"))
	ENGINE_API TArray<FName> GetVectorParameterNames() const;

	/** Gets the default value of a scalar parameter from a material collection.
	 * @param ParameterName - The name of the value to get the value of
	 * @param bParameterFound - if a parameter with the input name was found
	 * @returns the value of the parameter
	 **/
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetFloatParameterDefaultValue"))
	ENGINE_API float GetScalarParameterDefaultValue(FName ParameterName, bool& bParameterFound) const;

	/** Gets the default value of a scalar parameter from a material collection.
	 * @param ParameterName - The name of the value to get the value of
	 * @param bParameterFound - if a parameter with the input name was found
	 * @returns the value of the parameter
	 **/
	UFUNCTION(BlueprintCallable, Category="Rendering|Material", meta=(Keywords="GetFloatParameterDefaultValue"))
	ENGINE_API FLinearColor GetVectorParameterDefaultValue(FName ParameterName, bool& bParameterFound) const;
	
	/** Gets the base Material Parameter Collection for this one. Returns nullptr if not set. */
	UFUNCTION(BlueprintCallable, Category="Rendering|Material")
	ENGINE_API UMaterialParameterCollection* GetBaseParameterCollection() const;

	//~ Begin UObject Interface.
#if WITH_EDITOR
	using Super::PreEditChange;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	virtual bool CanBeInCluster() const override { return false; }
	//~ End UObject Interface.

	/** Finds a parameter name given an Id, returns NAME_None if the parameter was not found. */
	ENGINE_API FName GetParameterName(const FGuid& Id) const;

	/** Finds a parameter id given a name, returns the default guid if the parameter was not found. */
	ENGINE_API FGuid GetParameterId(FName ParameterName) const;

	/** Gets a vector and component index for the given parameter, used when compiling materials, to know where to access a certain parameter. */
	ENGINE_API void GetParameterIndex(const FGuid& Id, int32& OutIndex, int32& OutComponentIndex) const;

	/** Returns total vector storage of the parameter collection, used for math involving OutIndex returned by GetParameterIndex above, when compiling materials. */
	ENGINE_API int32 GetTotalVectorStorage() const;

	/** Populates an array with either scalar or vector parameter names. */
	ENGINE_API void GetParameterNames(TArray<FName>& OutParameterNames, bool bVectorParameters) const;

	/** Utility to find a scalar parameter struct given a parameter name.  Returns NULL if not found. */
	ENGINE_API const FCollectionScalarParameter* GetScalarParameterByName(FName ParameterName) const;

	/** Utility to find a vector parameter struct given a parameter name.  Returns NULL if not found. */
	ENGINE_API const FCollectionVectorParameter* GetVectorParameterByName(FName ParameterName) const;

	/** Accessor for the uniform buffer layout description. */
	const FShaderParametersMetadata& GetUniformBufferStruct() const
	{
		check(UniformBufferStruct);
		return *UniformBufferStruct;
	}

	/** Returns whether this instance loaded correctly and is in valid state. */
	bool HasValidUniformBufferStruct() const
	{
		return UniformBufferStruct.IsValid();
	}

	/** Create an instance for this collection in every world. */
	ENGINE_API void SetupWorldParameterCollectionInstances();

	ENGINE_API void GetParameterData(TArray<FVector4f>& ParameterData, const TMap<FName, float>* ScalarParameterInstanceOverrides, const TMap<FName, FLinearColor>* VectorParameterInstanceOverrides) const;

	ENGINE_API bool IsInstanceOf(const FGuid& CollectionStateId) const;

#if WITH_EDITOR
	static ENGINE_API FName GetBaseParameterCollectionMemberName();

	DECLARE_MULTICAST_DELEGATE(FOnCollectionChanged);
	FOnCollectionChanged& OnCollectionChanged() const
	{
		return CollectionChangedDelegate;
	}
#endif

private:
	virtual ENGINE_API void FinishDestroy() override;
	virtual ENGINE_API bool IsReadyForFinishDestroy() override;

	template<typename FCollectionParameterType>
	void SanitizeParameters(TArray<FCollectionParameterType> UMaterialParameterCollection::* CollectionParameters);

	bool UpdateOverrides(UMaterialParameterCollection* BaseCollection);

	UPROPERTY(EditAnywhere, Category=Material, AssetRegistrySearchable)
	TObjectPtr<UMaterialParameterCollection> Base;

	UPROPERTY()
	FGuid BaseStateId;

	/** Overrides for scalar parameter values. */
	UPROPERTY()
	TMap<FGuid, float> ScalarParameterBaseOverrides;

	/** Overrides for vector parameter values. */
	UPROPERTY()
	TMap<FGuid, FLinearColor> VectorParameterBaseOverrides;

	/** Flag used to guarantee that the RT is finished using various resources in this UMaterial before cleanup. */
	FThreadSafeBool ReleasedByRT;

	/** Default resource used when no instance is available. */
	class FMaterialParameterCollectionInstanceResource* DefaultResource;

	TUniquePtr<FShaderParametersMetadata> UniformBufferStruct;

#if WITH_EDITOR
	/** Delegate for when the collection changes and consummers may need to be updated */
	mutable FOnCollectionChanged CollectionChangedDelegate;
#endif

	void CreateBufferStruct();

	/** Gets default values into data to be set on the uniform buffer. */
	void GetDefaultParameterData(TArray<FVector4f>& ParameterData) const;

	void UpdateDefaultResource(bool bRecreateUniformBuffer);
};
