// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialParameterCollectionInstance.h 
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MaterialParameterCollectionInstance.generated.h"

struct FCollectionScalarParameter;
struct FCollectionVectorParameter;
class FMaterialParameterCollectionInstanceResource;
class UMaterialParameterCollection;

/** 
 * Class that stores per-world instance parameter data for a given UMaterialParameterCollection resource. 
 * Instances of this class are always transient.
 */
UCLASS(hidecategories=object, MinimalAPI)
class UMaterialParameterCollectionInstance : public UObject
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface.
	ENGINE_API virtual void PostInitProperties() override;
	ENGINE_API virtual void FinishDestroy() override;
	//~ End UObject Interface.

	/** Initializes the instance with the collection it is based off of and the world it is owned by. */
	ENGINE_API void SetCollection(UMaterialParameterCollection* InCollection, UWorld* InWorld);

	bool IsCollectionValid() const { return Collection.IsValid(); }

	/** Sets parameter value overrides, returns false if the parameter was not found. */
	ENGINE_API bool SetScalarParameterValue(FName ParameterName, float ParameterValue);
	ENGINE_API bool SetVectorParameterValue(FName ParameterName, const FLinearColor& ParameterValue);
	bool SetVectorParameterValue(FName ParameterName, const FVector& ParameterValue) { return SetVectorParameterValue(ParameterName, FLinearColor(ParameterValue)); }
	bool SetVectorParameterValue(FName ParameterName, const FVector4& ParameterValue) { return SetVectorParameterValue(ParameterName, FLinearColor(ParameterValue)); }

	/** Gets parameter values, returns false if the parameter was not found. */
	ENGINE_API bool GetScalarParameterValue(FName ParameterName, float& OutParameterValue) const;
	ENGINE_API bool GetVectorParameterValue(FName ParameterName, FLinearColor& OutParameterValue) const;

	/** Alternate Get method for parameter values where the Collection parameter is provided */
	ENGINE_API bool GetScalarParameterValue(const FCollectionScalarParameter& Parameter, float& OutParameterValue) const;
	ENGINE_API bool GetVectorParameterValue(const FCollectionVectorParameter& Parameter, FLinearColor& OutParameterValue) const;

	class FMaterialParameterCollectionInstanceResource* GetResource()
	{
		return Resource;
	}

	const UMaterialParameterCollection* GetCollection() const
	{
		return Collection.Get();
	}

	using ScalarParameterUpdate = TPair<FName, float>;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnScalarParameterUpdated, ScalarParameterUpdate);
	FOnScalarParameterUpdated& OnScalarParameterUpdated()
	{
		return ScalarParameterUpdatedDelegate;
	}

	using VectorParameterUpdate = TPair<FName, FLinearColor>;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVectorParameterUpdated, VectorParameterUpdate);
	FOnVectorParameterUpdated& OnVectorParameterUpdated()
	{
		return VectorParameterUpdatedDelegate;
	}

	ENGINE_API void UpdateRenderState(bool bRecreateUniformBuffer);

	ENGINE_API void DeferredUpdateRenderState(bool bRecreateUniformBuffer);

	ENGINE_API void ForceReturnToDefaultValues();

	/** Tracks whether this instance has ever issued a missing parameter warning, to reduce log spam. */
	bool bLoggedMissingParameterWarning;

protected:

	/** Collection resource this instance is based off of. */
	UPROPERTY()
	TWeakObjectPtr<UMaterialParameterCollection> Collection;

	/** World that owns this instance. */
	TWeakObjectPtr<UWorld> World;

	/** Overrides for scalar parameter values. */
	TMap<FName, float> ScalarParameterValues;

	/** Overrides for vector parameter values. */
	TMap<FName, FLinearColor> VectorParameterValues;

	/** Instance resource which stores the rendering thread representation of this instance. */
	FMaterialParameterCollectionInstanceResource* Resource;

	/** Delegate for when a scalar parameter value is updated */
	FOnScalarParameterUpdated ScalarParameterUpdatedDelegate;

	/** Delegate for when a vector parameter value is updated */
	FOnVectorParameterUpdated VectorParameterUpdatedDelegate;

	/** Boils down the instance overrides and default values into data to be set on the uniform buffer. */
	ENGINE_API void GetParameterData(TArray<FVector4f>& ParameterData) const;
	
	/** Tracks whether this instance needs to update the render state from the game thread */
	bool bNeedsRenderStateUpdate;
};



