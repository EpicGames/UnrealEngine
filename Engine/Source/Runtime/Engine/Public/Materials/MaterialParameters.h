// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/MemoryLayout.h"
#include "UObject/SoftObjectPtr.h"
#include "Shader/ShaderTypes.h"
#include "MaterialParameters.generated.h"

class FShaderKeyGenerator;
class FXxHash64Builder;
class UTexture;
class UTextureCollection;
class UCurveLinearColor;
class UCurveLinearColorAtlas;
class UFont;
class URuntimeVirtualTexture;
class USparseVolumeTexture;
class UMaterialParameterCollection;

UENUM()
enum EMaterialParameterAssociation : int
{
	LayerParameter,
	BlendParameter,
	GlobalParameter,
};

USTRUCT(BlueprintType)
struct FMaterialParameterInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ParameterInfo)
	FName Name;

	/** Whether this is a global parameter, or part of a layer or blend */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ParameterInfo)
	TEnumAsByte<EMaterialParameterAssociation> Association;

	/** Layer or blend index this parameter is part of. INDEX_NONE for global parameters. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ParameterInfo)
	int32 Index;

	FMaterialParameterInfo(const TCHAR* InName, EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter, int32 InIndex = INDEX_NONE)
		: Name(InName)
		, Association(InAssociation)
		, Index(InIndex)
	{
	}
	FMaterialParameterInfo(FName InName = FName(), EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter, int32 InIndex = INDEX_NONE)
		: Name(InName)
		, Association(InAssociation)
		, Index(InIndex)
	{
	}

	explicit FMaterialParameterInfo(const struct FMemoryImageMaterialParameterInfo& Rhs);

	ENGINE_API void AppendString(FString& Out) const;
	ENGINE_API void Append(FShaderKeyGenerator& KeyGen) const;
	FString ToString() const
	{
		FString Out;
		AppendString(Out);
		return Out;
	}

	friend FArchive& operator<<(FArchive& Ar, FMaterialParameterInfo& Ref)
	{
		Ar << Ref.Name << Ref.Association << Ref.Index;
		return Ar;
	}

	ENGINE_API bool RemapLayerIndex(TArrayView<const int32> IndexRemap, FMaterialParameterInfo& OutResult) const;

	friend inline bool operator==(const FMaterialParameterInfo& Lhs, const FMaterialParameterInfo& Rhs)
	{
		return Lhs.Name.IsEqual(Rhs.Name) && Lhs.Association == Rhs.Association && Lhs.Index == Rhs.Index;
	}

	friend inline bool operator!=(const FMaterialParameterInfo& Lhs, const FMaterialParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	friend inline uint32 GetTypeHash(const FMaterialParameterInfo& Value)
	{
		return HashCombine(HashCombine(GetTypeHash(NameToScriptName(Value.Name)), Value.Index), (uint32)Value.Association);
	}
};

struct FMemoryImageMaterialParameterInfo
{
	DECLARE_TYPE_LAYOUT(FMemoryImageMaterialParameterInfo, NonVirtual);
public:
	FMemoryImageMaterialParameterInfo(const TCHAR* InName, EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter, int32 InIndex = INDEX_NONE)
		: Name(NameToScriptName(FName(InName)))
		, Index(InIndex)
		, Association(InAssociation)
	{}

	FMemoryImageMaterialParameterInfo(const FName& InName, EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter, int32 InIndex = INDEX_NONE)
		: Name(NameToScriptName(InName))
		, Index(InIndex)
		, Association(InAssociation)
	{}

	FMemoryImageMaterialParameterInfo(const FScriptName& InName = FScriptName(), EMaterialParameterAssociation InAssociation = EMaterialParameterAssociation::GlobalParameter, int32 InIndex = INDEX_NONE)
		: Name(InName)
		, Index(InIndex)
		, Association(InAssociation)
	{}

	FMemoryImageMaterialParameterInfo(const FMaterialParameterInfo& Rhs)
		: Name(NameToScriptName(Rhs.Name))
		, Index(Rhs.Index)
		, Association(Rhs.Association)
	{}

	inline FName GetName() const { return ScriptNameToName(Name); }

	friend FArchive& operator<<(FArchive& Ar, FMemoryImageMaterialParameterInfo& Ref)
	{
		FName RefName = ScriptNameToName(Ref.Name);
		Ar << RefName << Ref.Association << Ref.Index;
		Ref.Name = NameToScriptName(RefName);
		return Ar;
	}

	bool RemapLayerIndex(TArrayView<const int32> IndexRemap, FMemoryImageMaterialParameterInfo& OutResult) const;

	LAYOUT_FIELD(FScriptName, Name);
	LAYOUT_FIELD(int32, Index);
	LAYOUT_FIELD(TEnumAsByte<EMaterialParameterAssociation>, Association);

	friend inline bool operator==(const FMemoryImageMaterialParameterInfo& Lhs, const FMemoryImageMaterialParameterInfo& Rhs)
	{
		return Lhs.Name == Rhs.Name && Lhs.Association == Rhs.Association && Lhs.Index == Rhs.Index;
	}

	friend inline bool operator!=(const FMemoryImageMaterialParameterInfo& Lhs, const FMemoryImageMaterialParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	friend inline bool operator==(const FMaterialParameterInfo& Lhs, const FMemoryImageMaterialParameterInfo& Rhs)
	{
		return Lhs.Name == Rhs.Name && Lhs.Index == Rhs.Index && Lhs.Association == Rhs.Association;
	}

	friend inline bool operator!=(const FMaterialParameterInfo& Lhs, const FMemoryImageMaterialParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	friend inline bool operator==(const FMemoryImageMaterialParameterInfo& Lhs, const FMaterialParameterInfo& Rhs)
	{
		return Lhs.Name == Rhs.Name && Lhs.Index == Rhs.Index && Lhs.Association == Rhs.Association;
	}

	friend inline bool operator!=(const FMemoryImageMaterialParameterInfo& Lhs, const FMaterialParameterInfo& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	friend inline uint32 GetTypeHash(const FMemoryImageMaterialParameterInfo& Value)
	{
		return HashCombine(HashCombine(GetTypeHash(Value.Name), Value.Index), (uint32)Value.Association);
	}
};

inline FMaterialParameterInfo::FMaterialParameterInfo(const struct FMemoryImageMaterialParameterInfo& Rhs)
	: Name(ScriptNameToName(Rhs.Name))
	, Association(Rhs.Association)
	, Index(Rhs.Index)
{
}

// Backwards compat
using FHashedMaterialParameterInfo = FMemoryImageMaterialParameterInfo;

UENUM()
enum class EMaterialParameterType : uint8
{
	Scalar = 0u,
	Vector,
	DoubleVector,
	Texture,
	TextureCollection,
	Font,
	RuntimeVirtualTexture,
	SparseVolumeTexture,
	StaticSwitch,
	ParameterCollection,

	NumRuntime, // Runtime parameter types must go above here, and editor-only ones below

	// TODO - Would be nice to make static parameter values editor-only, but will save that for a future-refactor
	StaticComponentMask = NumRuntime,

	Num,
	None = 0xff,
};
DECLARE_INTRINSIC_TYPE_LAYOUT(EMaterialParameterType);

inline constexpr int32 NumMaterialParameterTypes = (int32)EMaterialParameterType::Num;
inline constexpr int32 NumMaterialRuntimeParameterTypes = (int32)EMaterialParameterType::NumRuntime;
inline constexpr int32 NumMaterialEditorOnlyParameterTypes = NumMaterialParameterTypes - NumMaterialRuntimeParameterTypes;

inline bool IsNumericMaterialParameter(EMaterialParameterType InType)
{
	switch (InType)
	{
	case EMaterialParameterType::Scalar:
	case EMaterialParameterType::Vector:
	case EMaterialParameterType::DoubleVector:
		return true;
	default:
		return false;
	}
}

inline bool IsStaticMaterialParameter(EMaterialParameterType InType)
{
	switch (InType)
	{
	case EMaterialParameterType::StaticSwitch:
	case EMaterialParameterType::StaticComponentMask:
		return true;
	default:
		return false;
	}
}

extern ENGINE_API UE::Shader::FType GetShaderValueType(EMaterialParameterType Type);

inline const UEnum* GetStaticMaterialParameterTypeEnum()
{
	const UEnum* Enum = StaticEnum<EMaterialParameterType>();
	check(Enum);
	return Enum;
}

inline FString MaterialParameterTypeToString(EMaterialParameterType Type)
{
	const UEnum* Enum = GetStaticMaterialParameterTypeEnum();
	return Enum->GetNameStringByValue(int64(Type));
}

inline EMaterialParameterType GetMaterialParameterTypeFromString(FString TypeName)
{
	const UEnum* Enum = GetStaticMaterialParameterTypeEnum();
	return static_cast<EMaterialParameterType>(Enum->GetValueByName(FName(TypeName)));
}

enum class EMaterialGetParameterValueFlags : uint32
{
	None = 0u,
	CheckNonOverrides = (1u << 0),
	CheckInstanceOverrides = (1u << 1),

	CheckAll = CheckNonOverrides | CheckInstanceOverrides,
	Default = CheckAll,
};
ENUM_CLASS_FLAGS(EMaterialGetParameterValueFlags);

enum class EMaterialSetParameterValueFlags : uint32
{
	None = 0u,
	SetCurveAtlas = (1u << 0),
};
ENUM_CLASS_FLAGS(EMaterialSetParameterValueFlags);

USTRUCT(BlueprintType)
struct FParameterChannelNames
{
	GENERATED_USTRUCT_BODY()

	FParameterChannelNames() = default;
	FParameterChannelNames(const FText& InR, const FText& InG, const FText& InB, const FText& InA) : R(InR), G(InG), B(InB), A(InA) {}

	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorParameter)
	FText R;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorParameter)
	FText G;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorParameter)
	FText B;

	UPROPERTY(EditAnywhere, Category = MaterialExpressionVectorParameter)
	FText A;

	bool operator==(const FParameterChannelNames& Other) const
	{
		return R.EqualTo(Other.R) && G.EqualTo(Other.G) && B.EqualTo(Other.B) && A.EqualTo(Other.A);
	}
};

USTRUCT()
struct FStaticComponentMaskValue
{
	GENERATED_USTRUCT_BODY();

	FStaticComponentMaskValue() : R(false), G(false), B(false), A(false) {}
	FStaticComponentMaskValue(bool InR, bool InG, bool InB, bool InA) : R(InR), G(InG), B(InB), A(InA) {}

	UPROPERTY()
	bool R = false;

	UPROPERTY()
	bool G = false;

	UPROPERTY()
	bool B = false;

	UPROPERTY()
	bool A = false;

	bool operator==(const FStaticComponentMaskValue& Other) const
	{
		return R == Other.R && G == Other.G && B == Other.B && A == Other.A;
	}
};

struct FMaterialTextureValue
{
	static FName GetTypeName() { return TEXT("FMaterialTextureValue"); }

	UTexture* Texture = nullptr;
	URuntimeVirtualTexture* RuntimeVirtualTexture = nullptr;
	FGuid ExternalTextureGuid;
	EMaterialSamplerType SamplerType = SAMPLERTYPE_Color;
	FMaterialParameterInfo ParameterInfo;
};

struct FMaterialParameterValue
{
	FMaterialParameterValue() : Type(EMaterialParameterType::None) {}
	FMaterialParameterValue(float InValue) : Type(EMaterialParameterType::Scalar) { Float[0] = InValue; }
	FMaterialParameterValue(const FLinearColor& InValue) : Type(EMaterialParameterType::Vector) { Float[0] = InValue.R; Float[1] = InValue.G; Float[2] = InValue.B; Float[3] = InValue.A; }
	FMaterialParameterValue(const FVector3f& InValue) : Type(EMaterialParameterType::Vector) { Float[0] = InValue.X; Float[1] = InValue.Y; Float[2] = InValue.Z; Float[3] = 0.0f; }
	FMaterialParameterValue(const FVector4d& InValue) : Type(EMaterialParameterType::DoubleVector) { Double[0] = InValue.X; Double[1] = InValue.Y; Double[2] = InValue.Z; Double[3] = InValue.W; }
	FMaterialParameterValue(UTexture* InValue) : Type(EMaterialParameterType::Texture) { Texture = InValue; }
	FMaterialParameterValue(const TObjectPtr<UTexture>& InValue) : Type(EMaterialParameterType::Texture) { Texture = InValue; }
	FMaterialParameterValue(const TObjectPtr<UTextureCollection>& InValue) : Type(EMaterialParameterType::TextureCollection) { TextureCollection = InValue; }
	FMaterialParameterValue(URuntimeVirtualTexture* InValue) : Type(EMaterialParameterType::RuntimeVirtualTexture) { RuntimeVirtualTexture = InValue; }
	FMaterialParameterValue(const TObjectPtr<URuntimeVirtualTexture>& InValue) : Type(EMaterialParameterType::RuntimeVirtualTexture) { RuntimeVirtualTexture = InValue; }
	FMaterialParameterValue(USparseVolumeTexture* InValue) : Type(EMaterialParameterType::SparseVolumeTexture) { SparseVolumeTexture = InValue; }
	FMaterialParameterValue(const TObjectPtr<USparseVolumeTexture>& InValue) : Type(EMaterialParameterType::SparseVolumeTexture) { SparseVolumeTexture = InValue; }
	FMaterialParameterValue(const TObjectPtr<UMaterialParameterCollection>& InValue) : Type(EMaterialParameterType::ParameterCollection) { ParameterCollection = InValue; }

	// Gamethread parameters are typically non-const, but renderthread parameters are const
	// Would be possible to store an additional const-flag member, and provide runtime checks to ensure constness is not violated...maybe worth doing in the future
	FMaterialParameterValue(const UTexture* InValue) : Type(EMaterialParameterType::Texture) { Texture = const_cast<UTexture*>(InValue); }
	FMaterialParameterValue(const TObjectPtr<const UTexture>& InValue) : Type(EMaterialParameterType::Texture) { Texture = const_cast<UTexture*>(InValue.Get()); }
	FMaterialParameterValue(const UTextureCollection* InValue) : Type(EMaterialParameterType::TextureCollection) { TextureCollection = const_cast<UTextureCollection*>(InValue); }
	FMaterialParameterValue(const TObjectPtr<const UTextureCollection>& InValue) : Type(EMaterialParameterType::TextureCollection) { TextureCollection = const_cast<UTextureCollection*>(InValue.Get()); }
	FMaterialParameterValue(const URuntimeVirtualTexture* InValue) : Type(EMaterialParameterType::RuntimeVirtualTexture) { RuntimeVirtualTexture = const_cast<URuntimeVirtualTexture*>(InValue); }
	FMaterialParameterValue(const TObjectPtr<const URuntimeVirtualTexture>& InValue) : Type(EMaterialParameterType::RuntimeVirtualTexture) { RuntimeVirtualTexture = const_cast<URuntimeVirtualTexture*>(InValue.Get()); }
	FMaterialParameterValue(const USparseVolumeTexture* InValue) : Type(EMaterialParameterType::SparseVolumeTexture) { SparseVolumeTexture = const_cast<USparseVolumeTexture*>(InValue); }
	FMaterialParameterValue(const TObjectPtr<const USparseVolumeTexture>& InValue) : Type(EMaterialParameterType::SparseVolumeTexture) { SparseVolumeTexture = const_cast<USparseVolumeTexture*>(InValue.Get()); }
	FMaterialParameterValue(const UMaterialParameterCollection* InValue) : Type(EMaterialParameterType::ParameterCollection) { ParameterCollection = const_cast<UMaterialParameterCollection*>(InValue); }

	FMaterialParameterValue(UFont* InValue, int32 InPage) : Type(EMaterialParameterType::Font) { Font.Value = InValue; Font.Page = InPage; }
	FMaterialParameterValue(bool InValue) : Type(EMaterialParameterType::StaticSwitch) { Bool[0] = InValue; }
	FMaterialParameterValue(const FStaticComponentMaskValue& InValue) : Type(EMaterialParameterType::StaticComponentMask) { Bool[0] = InValue.R; Bool[1] = InValue.G; Bool[2] = InValue.B; Bool[3] = InValue.A; }
	FMaterialParameterValue(bool bMaskR, bool bMaskG, bool bMaskB, bool bMaskA) : Type(EMaterialParameterType::StaticComponentMask) { Bool[0] = bMaskR; Bool[1] = bMaskG; Bool[2] = bMaskB; Bool[3] = bMaskA; }
	ENGINE_API FMaterialParameterValue(EMaterialParameterType Type, const UE::Shader::FValue& InValue);

	inline float AsScalar() const { check(Type == EMaterialParameterType::Scalar); return Float[0]; }
	inline FLinearColor AsLinearColor() const { check(Type == EMaterialParameterType::Vector); return FLinearColor(Float[0], Float[1], Float[2], Float[3]); }
	inline FVector4d AsVector4d() const { check(Type == EMaterialParameterType::DoubleVector); return FVector4d(Double[0], Double[1], Double[2], Double[3]); }
	inline bool AsStaticSwitch() const { check(Type == EMaterialParameterType::StaticSwitch); return Bool[0]; }
	inline FStaticComponentMaskValue AsStaticComponentMask() const { check(Type == EMaterialParameterType::StaticComponentMask); return FStaticComponentMaskValue(Bool[0], Bool[1], Bool[2], Bool[3]); }
	ENGINE_API UE::Shader::FValue AsShaderValue() const;
	ENGINE_API UObject* AsTextureObject() const;

	union
	{
		double Double[4];
		float Float[4];
		bool Bool[4];
		UTexture* Texture;
		UTextureCollection* TextureCollection;
		URuntimeVirtualTexture* RuntimeVirtualTexture;
		USparseVolumeTexture* SparseVolumeTexture;
		struct
		{
			UFont* Value;
			int32 Page;
		} Font;
		UMaterialParameterCollection* ParameterCollection;
		FName EnumEntry;
	};
	EMaterialParameterType Type;

	friend inline bool operator==(const FMaterialParameterValue& Lhs, const FMaterialParameterValue& Rhs)
	{
		const EMaterialParameterType ParameterType = Lhs.Type;
		if (ParameterType != Rhs.Type)
		{
			return false;
		}
		switch (ParameterType)
		{
		case EMaterialParameterType::None: return true;
		case EMaterialParameterType::Scalar: return Lhs.Float[0] == Rhs.Float[0];
		case EMaterialParameterType::Vector: return
			Lhs.Float[0] == Rhs.Float[0] &&
			Lhs.Float[1] == Rhs.Float[1] &&
			Lhs.Float[2] == Rhs.Float[2] &&
			Lhs.Float[3] == Rhs.Float[3];
		case EMaterialParameterType::DoubleVector: return
			Lhs.Double[0] == Rhs.Double[0] &&
			Lhs.Double[1] == Rhs.Double[1] &&
			Lhs.Double[2] == Rhs.Double[2] &&
			Lhs.Double[3] == Rhs.Double[3];
		case EMaterialParameterType::Texture: return Lhs.Texture == Rhs.Texture;
		case EMaterialParameterType::TextureCollection: return Lhs.TextureCollection == Rhs.TextureCollection;
		case EMaterialParameterType::Font: return Lhs.Font.Value == Rhs.Font.Value && Lhs.Font.Page == Rhs.Font.Page;
		case EMaterialParameterType::RuntimeVirtualTexture: return Lhs.RuntimeVirtualTexture == Rhs.RuntimeVirtualTexture;
		case EMaterialParameterType::SparseVolumeTexture: return Lhs.SparseVolumeTexture == Rhs.SparseVolumeTexture;
		case EMaterialParameterType::StaticSwitch: return Lhs.Bool[0] == Rhs.Bool[0];
		case EMaterialParameterType::ParameterCollection: return Lhs.ParameterCollection == Rhs.ParameterCollection;
		case EMaterialParameterType::StaticComponentMask: return
			Lhs.Bool[0] == Rhs.Bool[0] &&
			Lhs.Bool[1] == Rhs.Bool[1] &&
			Lhs.Bool[2] == Rhs.Bool[2] &&
			Lhs.Bool[3] == Rhs.Bool[3];
		default: checkNoEntry(); return false;
		}
	}

	friend inline bool operator!=(const FMaterialParameterValue& Lhs, const FMaterialParameterValue& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
};

/** Holds a value, along with editor-only metadata that describes that value */
struct FMaterialParameterMetadata
{
	FMaterialParameterValue Value;
	int32 PrimitiveDataIndex = INDEX_NONE;

	FMaterialParameterMetadata() = default;
	FMaterialParameterMetadata(const FMaterialParameterValue& InValue) : Value(InValue) {}
	FMaterialParameterMetadata(EMaterialParameterType Type, const UE::Shader::FValue& InValue) : Value(Type, InValue) {}

#if WITH_EDITORONLY_DATA
	/** Name of channels, for Vectors/Textures */
	FParameterChannelNames ChannelNames;

	/** Curve/Atlas used to generate scalar value */
	TSoftObjectPtr<class UCurveLinearColor> ScalarCurve;
	TSoftObjectPtr<class UCurveLinearColorAtlas> ScalarAtlas;

	/** Description of the parameter, typically taken from the 'Desc' field of the parameter's UMaterialExpression */
	FString Description;

	/** Name of the parameter's group */
	FName Group;

	/** UI range for scalar values */
	float ScalarMin = 0.0f;
	float ScalarMax = 0.0f;

	/** Enumeration object for scalar values to use in the UI */
	TSoftObjectPtr<UObject> ScalarEnumeration;

	/** Index into material instance enumeration objects for scalar values to use in the UI */
	int32 ScalarEnumerationIndex = INDEX_NONE;

	/** Used for sorting parameter within the group, in the UI */
	int32 SortPriority = 0;

	/** GUID of the UMaterialExpression this parameter came from */
	FGuid ExpressionGuid;

	/** Should curves be used? */
	bool bUsedAsAtlasPosition = false;

	/** Valid for Vector parameters */
	bool bUsedAsChannelMask = false;

	/** Is the parameter overriden on the material it was queried from? */
	bool bOverride = false;
#endif // WITH_EDITORONLY_DATA

	/** Is a Static Switch parameter using a dynamic branch? */
	bool bDynamicSwitchParameter = false;
};

class FSHA1;
struct FSubstrateCompilationConfig
{
	bool bFullSimplify = false;
	int16 BytesPerPixelOverride = -1;
	int16 ClosuresPerPixelOverride = -1;

#if WITH_EDITOR
	FString GetShaderMapKeyString() const;
	void Append(FShaderKeyGenerator& KeyGen) const;

	void UpdateHash(FXxHash64Builder& Hasher) const;

	void Serialize(FArchive& Ar);
#endif

	friend inline bool operator==(const FSubstrateCompilationConfig& Lhs, const FSubstrateCompilationConfig& Rhs)
	{
		return Lhs.bFullSimplify == Rhs.bFullSimplify && Lhs.BytesPerPixelOverride == Rhs.BytesPerPixelOverride && Lhs.ClosuresPerPixelOverride == Rhs.ClosuresPerPixelOverride;
	}

	friend inline bool operator!=(const FSubstrateCompilationConfig& Lhs, const FSubstrateCompilationConfig& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
private:
#if WITH_EDITOR
	// Hidden friend for the FShaderKeyGenerator Append API
	friend inline void Append(FShaderKeyGenerator& KeyGen, const FSubstrateCompilationConfig& Value)
	{
		Value.Append(KeyGen);
	}
#endif
};
