// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

struct FPrimitiveViewRelevance;

// the class is only storing bits, initialized to 0 and has an |= operator
// to provide a combined set of multiple materials (component / mesh)
struct FMaterialRelevance
{
	// bits that express which EMaterialShadingModel are used
	union
	{
		struct
		{
			// 32 bits
			uint32 ShadingModelMask : 14;
			uint32 SubstrateTileTypeMask : 4;
			uint32 SubstrateUintPerPixel : 6;
			uint32 SubstrateClosureCountMask : 8;

			// 32 bits
			uint8 CustomDepthStencilUsageMask : 2;
			uint8 bOpaque : 1;
			uint8 bMasked : 1;
			uint8 bDistortion : 1;
			uint8 bHairStrands : 1;
			uint8 bTwoSided : 1;
			uint8 bSeparateTranslucency : 1; // Translucency After DOF
			uint8 bTranslucencyModulate : 1;
			uint8 bPostMotionBlurTranslucency : 1;
			uint8 bNormalTranslucency : 1;
			uint8 bUsesSceneColorCopy : 1;
			uint8 bOutputsTranslucentVelocity : 1;
			uint8 bUsesGlobalDistanceField : 1;
			uint8 bUsesWorldPositionOffset : 1;
			uint8 bUsesDisplacement : 1;
			uint8 bUsesPixelDepthOffset : 1;
			uint8 bUsesCustomizedUVs : 1;
			uint8 bUsesVertexInterpolator : 1;
			uint8 bDecal : 1;
			uint8 bTranslucentSurfaceLighting : 1;
			uint8 bUsesSceneDepth : 1;
			uint8 bUsesSkyMaterial : 1;
			uint8 bUsesSingleLayerWaterMaterial : 1;
			uint8 bHasVolumeMaterialDomain : 1;
			uint8 bUsesDistanceCullFade : 1;
			uint8 bDisableDepthTest : 1;
			uint8 bUsesAnisotropy : 1;
			uint8 bIsLightFunctionAtlasCompatible : 1;
			uint8 bSamplesMaterialCache : 1;
			uint8 bHasMaterialCacheOutput : 1;
			uint8 bUsesFirstPersonInterpolation : 1;
		};
		uint64 Raw;
	};

	/** Default constructor */
	FMaterialRelevance()
		: Raw(0)
	{
	}

	/** Bitwise OR operator.  Sets any relevance bits which are present in either. */
	FMaterialRelevance& operator|=(const FMaterialRelevance& B)
	{
		const uint32 LocalSubstrateUintPerPixel = SubstrateUintPerPixel;
		Raw |= B.Raw;
		SubstrateUintPerPixel = FMath::Max(LocalSubstrateUintPerPixel, B.SubstrateUintPerPixel);
		return *this;
	}

	/** Copies the material's relevance flags to a primitive's view relevance flags. */
	ENGINE_API void SetPrimitiveViewRelevance(FPrimitiveViewRelevance& OutViewRelevance) const;
};

static_assert(sizeof(FMaterialRelevance) == sizeof(FMaterialRelevance::Raw), "Union Raw type is too small");
