// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "RenderResource.h"
#include "Async/Mutex.h"
#include "Tasks/Task.h"
#include "Engine/BlendableInterface.h"

enum class EMaterialParameterType : uint8;

class FMaterial;
class FMaterialRenderProxy;
class FMaterialShaderMap;
class FMaterialVirtualTextureStack;
class FRHIComputeCommandList;
class FUniformExpressionCacheAsyncUpdater;
class FUniformExpressionSet;
class IAllocatedVirtualTexture;
class UMaterialInterface;
class URuntimeVirtualTexture;
class USparseVolumeTexture;
class USubsurfaceProfile;
class USpecularProfile;
class UToonProfile;
class UNeuralProfile;
class UTexture;
class UTextureCollection;
class UMaterialParameterCollection;

struct FMaterialParameterValue;
struct FMaterialRenderContext;

struct FMemoryImageMaterialParameterInfo;
using FHashedMaterialParameterInfo = FMemoryImageMaterialParameterInfo;

namespace UE::Shader
{
	struct FValue;
}

/**
 * Cached uniform expression values.
 */
struct FUniformExpressionCache
{
	/** Material uniform buffer. */
	FUniformBufferRHIRef UniformBuffer;
	/** Material uniform buffer used only by VS */
	FUniformBufferRHIRef UniformBufferVS;
	/** Allocated virtual textures, one for each entry in FUniformExpressionSet::VTStacks */
	TArray<IAllocatedVirtualTexture*> AllocatedVTs;
	/** Allocated virtual textures that will need destroying during a call to ResetAllocatedVTs() */
	TArray<IAllocatedVirtualTexture*> OwnedAllocatedVTs;
	/** Ids of parameter collections needed for rendering. */
	TArray<FGuid> ParameterCollections;
	/** Shader map that was used to cache uniform expressions on this material. This is used for debugging, verifying correct behavior and checking if the cache is up to date. */
	const FMaterialShaderMap* CachedUniformExpressionShaderMap = nullptr;

	/** Destructor. */
	ENGINE_API ~FUniformExpressionCache();

	void ResetAllocatedVTs();
};

struct FUniformExpressionCacheContainer
{
	inline FUniformExpressionCache& operator[](int32 Index)
	{
#if WITH_EDITOR
		return Elements[Index];
#else
		return Elements;
#endif
	}
private:
#if WITH_EDITOR
	FUniformExpressionCache Elements[ERHIFeatureLevel::Num];
#else
	FUniformExpressionCache Elements;
#endif
};

/** Defines a scope to update deferred uniform expression caches using an async task to fill uniform buffers. If expression
 *  caches are updated within the scope, an async task may be launched. Otherwise the update is synchronous.
 */
class FUniformExpressionCacheAsyncUpdateScope
{
public:
	ENGINE_API FUniformExpressionCacheAsyncUpdateScope();
	ENGINE_API ~FUniformExpressionCacheAsyncUpdateScope();

	/** Call if a wait is required within the scope. */
	static ENGINE_API void WaitForTask();
};

/**
 * A material render proxy used by the renderer.
 */
class FMaterialRenderProxy : public FRenderResource, public FNoncopyable
{
public:

	/** Cached uniform expressions. */
	mutable FUniformExpressionCacheContainer UniformExpressionCache;

	/** Default constructor. */
	ENGINE_API FMaterialRenderProxy(FString InMaterialName);

	/** Destructor. */
	ENGINE_API virtual ~FMaterialRenderProxy();

	/**
	 * Evaluates uniform expressions and stores them in OutUniformExpressionCache.
	 * @param OutUniformExpressionCache - The uniform expression cache to build.
	 * @param MaterialRenderContext - The context for which to cache expressions.
	 */
	ENGINE_API void EvaluateUniformExpressions(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater = nullptr) const;

	/**
	 * Caches uniform expressions for efficient runtime evaluation.
	 */
	ENGINE_API void CacheUniformExpressions(FRHICommandListBase& RHICmdList, bool bRecreateUniformBuffer);

	/** Cancels an in-flight cache operation. */
	ENGINE_API void CancelCacheUniformExpressions();

	/**
	 * Enqueues a rendering command to cache uniform expressions for efficient runtime evaluation.
	 * bRecreateUniformBuffer - whether to recreate the material uniform buffer.
	 *		This is required if the FMaterial is being recompiled (the uniform buffer layout will change).
	 *		This should only be done if the calling code is using FMaterialUpdateContext to recreate the rendering state of primitives using this material, since cached mesh commands also cache uniform buffer pointers.
	 */
	ENGINE_API void CacheUniformExpressions_GameThread(bool bRecreateUniformBuffer);

	/**
	 * Invalidates the uniform expression cache.
	 */
	ENGINE_API void InvalidateUniformExpressionCache(bool bRecreateUniformBuffer);

	ENGINE_API void UpdateUniformExpressionCacheIfNeeded(ERHIFeatureLevel::Type InFeatureLevel) const;
	ENGINE_API const FMaterial* UpdateUniformExpressionCacheIfNeeded(FRHICommandListBase& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel) const;

	/** Returns the FMaterial, without using a fallback if the FMaterial doesn't have a valid shader map. Can return NULL. */
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const = 0;
	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const = 0;

	// These functions should only be called by the rendering thread.

	/**
	 * Finds the FMaterial to use for rendering this FMaterialRenderProxy.  Will fall back to a default material if needed due to a content error, or async compilation.
	 * The returned FMaterial is guaranteed to have a complete shader map, so all relevant shaders should be available
	 * OutFallbackMaterialRenderProxy - The proxy that corresponds to the returned FMaterial, should be used for further rendering.  May be a fallback material, or 'this' if no fallback was needed
	 */
	ENGINE_API const FMaterial& GetMaterialWithFallback(ERHIFeatureLevel::Type InFeatureLevel, const FMaterialRenderProxy*& OutFallbackMaterialRenderProxy) const;

	/**
	 * Finds the FMaterial to use for rendering this FMaterialRenderProxy.  Will fall back to a default material if needed due to a content error, or async compilation.
	 * Will always return a valid FMaterial, but unlike GetMaterialWithFallback, FMaterial's shader map may be incomplete
	 */
	ENGINE_API const FMaterial& GetIncompleteMaterialWithFallback(ERHIFeatureLevel::Type InFeatureLevel) const;

	virtual UMaterialInterface* GetMaterialInterface() const { return NULL; }

	ENGINE_API bool GetVectorValue(const FHashedMaterialParameterInfo& ParameterInfo, FLinearColor* OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetScalarValue(const FHashedMaterialParameterInfo& ParameterInfo, float* OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const UTexture** OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const URuntimeVirtualTexture** OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetTextureValue(const FHashedMaterialParameterInfo& ParameterInfo, const USparseVolumeTexture** OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetTextureCollectionValue(const FHashedMaterialParameterInfo& ParameterInfo, const UTextureCollection** OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetParameterCollectionValue(const FHashedMaterialParameterInfo& ParameterInfo, const UMaterialParameterCollection** OutValue, const FMaterialRenderContext& Context) const;
	ENGINE_API bool GetParameterShaderValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, UE::Shader::FValue& OutValue, const FMaterialRenderContext& Context) const;
	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const = 0;
	virtual bool GetUserSceneTextureOverride(FName& InOutValue) const { return false; }
	ENGINE_API FName GetUserSceneTextureOutput(const FMaterial* Base) const;
	ENGINE_API virtual EBlendableLocation GetBlendableLocation(const FMaterial* Base) const;
	ENGINE_API virtual int32 GetBlendablePriority(const FMaterial* Base) const;


	bool IsDeleted() const
	{
		return DeletedFlag != 0;
	}

	void MarkForGarbageCollection()
	{
		MarkedForGarbageCollection = -1;
	}

	bool IsMarkedForGarbageCollection() const
	{
		return MarkedForGarbageCollection != 0;
	}

	void MarkTransient()
	{
		bMarkedTransient = 1;
	}

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	ENGINE_API virtual void ReleaseRHI() override;
	ENGINE_API virtual void ReleaseResource() override;

#if WITH_EDITOR
	static const TSet<FMaterialRenderProxy*>& GetMaterialRenderProxyMap()
	{
		check(!FPlatformProperties::RequiresCookedData());
		return MaterialRenderProxyMap;
	}

	static FCriticalSection& GetMaterialRenderProxyMapLock()
	{
		return MaterialRenderProxyMapLock;
	}
#endif

	// Subsurface profile
	// When Substrate is enabled, this is ONLY used as an override for Subsurface Profile on material instance (override all Subsurface Profiles at once for now)
	void SetSubsurfaceProfileRT(const USubsurfaceProfile* Ptr) { SubsurfaceProfileRT = Ptr; }
	const USubsurfaceProfile* GetSubsurfaceProfileRT() const { return SubsurfaceProfileRT; }

	// Subsurface profiles
	void ClearSubsurfaceProfileRT() { SubsurfaceProfilesRT.Empty(); }
	void AddSubsurfaceProfileRT(const USubsurfaceProfile* Ptr) { SubsurfaceProfilesRT.Add(Ptr); }
	const USubsurfaceProfile* GetSubsurfaceProfileRT(uint32 Index) const { check(Index<uint32(SubsurfaceProfilesRT.Num())); return SubsurfaceProfilesRT[Index]; }
	const uint32 NumSubsurfaceProfileRT() const { return SubsurfaceProfilesRT.Num(); }

	// Specular profiles
	void AddSpecularProfileRT(const USpecularProfile* Ptr) { SpecularProfilesRT.Add(Ptr); }
	const USpecularProfile* GetSpecularProfileRT(uint32 Index) const { check(Index<uint32(SpecularProfilesRT.Num())); return SpecularProfilesRT[Index]; }
	const uint32 NumSpecularProfileRT() const { return SpecularProfilesRT.Num(); }
	// Specular profiles override
	void SetSpecularProfileOverrideRT(const USpecularProfile* Ptr) { SpecularProfileOverrideRT = Ptr; }
	const USpecularProfile* GetSpecularProfileOverrideRT() const { return SpecularProfileOverrideRT; }

	// Toon profiles
	void AddToonProfileRT(const UToonProfile* Ptr) { ToonProfilesRT.Add(Ptr); }
	const UToonProfile* GetToonProfileRT(uint32 Index) const { check(Index<uint32(ToonProfilesRT.Num())); return ToonProfilesRT[Index]; }
	const uint32 NumToonProfileRT() const { return ToonProfilesRT.Num(); }
	// Toon profiles override
	void SetToonProfileOverrideRT(const UToonProfile* Ptr) { ToonProfileOverrideRT = Ptr; }
	const UToonProfile* GetToonProfileOverrideRT() const { return ToonProfileOverrideRT; }

	// Neural profiles
	void SetNeuralProfileRT(const UNeuralProfile* Ptr) { NeuralProfileRT = Ptr; }
	const UNeuralProfile* GetNeuralProfileRT() const { return NeuralProfileRT; }

	static ENGINE_API void UpdateDeferredCachedUniformExpressions();
	static ENGINE_API void UpdateDeferredCachedUniformExpressions(FRHICommandListBase& RHICmdList, UE::Tasks::FTask* TaskIfAsync = nullptr);

	static ENGINE_API bool HasDeferredUniformExpressionCacheRequests();

	int32 GetExpressionCacheSerialNumber() const { return UniformExpressionCacheSerialNumber; }

	const FString& GetMaterialName() const { return MaterialName; }

protected:
	ENGINE_API virtual void EvaluateParameterCollections(FRHICommandListBase& RHICmdList, FUniformExpressionCache& OutUniformExpressionCache, const FMaterialRenderContext& Context, FUniformExpressionCacheAsyncUpdater* Updater) const;

private:
	ENGINE_API IAllocatedVirtualTexture* GetPreallocatedVTStack(const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const FMaterialVirtualTextureStack& VTStack) const;
	ENGINE_API IAllocatedVirtualTexture* AllocateVTStack(FRHICommandListBase& RHICmdList, const FMaterialRenderContext& Context, const FUniformExpressionSet& UniformExpressionSet, const FMaterialVirtualTextureStack& VTStack) const;

	virtual void StartCacheUniformExpressions() const {}
	virtual void FinishCacheUniformExpressions() const {}

	/** 0 if not set, game thread pointer, do not dereference, only for comparison */
	const USubsurfaceProfile* SubsurfaceProfileRT;	// Overrides all SubsurfaceProfilesRT when set, used when set from material instance to respect the legacy workflow
	TArray<const USubsurfaceProfile*> SubsurfaceProfilesRT;
	TArray<const USpecularProfile*> SpecularProfilesRT;
	const USpecularProfile* SpecularProfileOverrideRT = nullptr;
	const UNeuralProfile* NeuralProfileRT;
	FString MaterialName;
	TArray<const UToonProfile*> ToonProfilesRT;
	const UToonProfile* ToonProfileOverrideRT = nullptr;

	/** Incremented each time UniformExpressionCache is modified */
	mutable int32 UniformExpressionCacheSerialNumber = 0;

	/** For tracking down a bug accessing a deleted proxy. */
	mutable uint8 MarkedForGarbageCollection : 1;
	mutable uint8 DeletedFlag : 1;
	mutable uint8 ReleaseResourceFlag : 1;
	/** Set true if this is a transient, one frame object. */
	uint8 bMarkedTransient : 1;
	/** If any VT producer destroyed callbacks have been registered */
	mutable uint8 bHasVirtualTextureCallbacks : 1;
	mutable uint8 bHasMaterialCacheCallbacks : 1;

	/** Mutex for locking uniform expression invalidation / evaluation for this material. */
	mutable UE::FMutex Mutex;

#if WITH_EDITOR
	/**
	 * Tracks all material render proxies in all scenes.
	 * This is used to propagate new shader maps to materials being used for rendering.
	 */
	static ENGINE_API TSet<FMaterialRenderProxy*> MaterialRenderProxyMap;

	/**
	 * Lock that guards the access to the render proxy map
	 */
	static ENGINE_API FCriticalSection MaterialRenderProxyMapLock;
#endif

	static ENGINE_API TSet<FMaterialRenderProxy*> DeferredUniformExpressionCacheRequests;
	static ENGINE_API UE::FMutex DeferredUniformExpressionCacheRequestsMutex;
};

/**
 * An material render proxy which overrides the material's Color vector parameter.
 */
class FColoredMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const FLinearColor Color;
	FName ColorParamName;

	/** Initialization constructor. */
	ENGINE_API FColoredMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, FName InColorParamName = NAME_Color);

	// FMaterialRenderProxy interface.
	ENGINE_API virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override;
	ENGINE_API virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override;
	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override;
};

/**
 * An material render proxy which overrides the material's Color vector and Texture parameter (mixed together).
 */
class FColoredTexturedMaterialRenderProxy : public FColoredMaterialRenderProxy
{
public:
	const UTexture* Texture;
	FName TextureParamName;
	float UVChannel = 0;
	FName UVChannelParamName = FName("None");

	/** Initialization constructor. */
	ENGINE_API FColoredTexturedMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, FName InColorParamName, const UTexture* InTexture, FName InTextureParamName);

	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override;
};

/**
 * A material render proxy which overrides the selection color
 */
class FOverrideSelectionColorMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const FLinearColor SelectionColor;

	/** Initialization constructor. */
	ENGINE_API FOverrideSelectionColorMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InSelectionColor);

	// FMaterialRenderProxy interface.
	ENGINE_API virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override;
	ENGINE_API virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override;
	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override;
};

/**
 * An material render proxy which overrides the material's Color and Lightmap resolution vector parameter.
 */
class FLightingDensityMaterialRenderProxy : public FColoredMaterialRenderProxy
{
public:
	const FVector2D LightmapResolution;

	/** Initialization constructor. */
	ENGINE_API FLightingDensityMaterialRenderProxy(const FMaterialRenderProxy* InParent, const FLinearColor& InColor, const FVector2D& InLightmapResolution);

	// FMaterialRenderProxy interface.
	ENGINE_API virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override;
};
