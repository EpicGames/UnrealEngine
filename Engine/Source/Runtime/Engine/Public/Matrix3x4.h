// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMacros.h"

/**
* 3x4 matrix of floating point values.
*/

MS_ALIGN(16) struct FMatrix3x4
{
	float M[3][4];

	inline void SetMatrix(const FMatrix& Mat)
	{
		const FMatrix::FReal* RESTRICT Src = &(Mat.M[0][0]);
		float* RESTRICT Dest = &(M[0][0]);

		Dest[0] = (float)Src[0];   // [0][0]
		Dest[1] = (float)Src[1];   // [0][1]
		Dest[2] = (float)Src[2];   // [0][2]
		Dest[3] = (float)Src[3];   // [0][3]

		Dest[4] = (float)Src[4];   // [1][0]
		Dest[5] = (float)Src[5];   // [1][1]
		Dest[6] = (float)Src[6];   // [1][2]
		Dest[7] = (float)Src[7];   // [1][3]

		Dest[8] = (float)Src[8];   // [2][0]
		Dest[9] = (float)Src[9];   // [2][1]
		Dest[10] = (float)Src[10]; // [2][2]
		Dest[11] = (float)Src[11]; // [2][3]
	}

	inline void SetMatrixTranspose(const FMatrix& Mat)
	{
		const FMatrix::FReal* RESTRICT Src = &(Mat.M[0][0]);
		float* RESTRICT Dest = &(M[0][0]);

		Dest[0] = (float)Src[0];   // [0][0]
		Dest[1] = (float)Src[4];   // [1][0]
		Dest[2] = (float)Src[8];   // [2][0]
		Dest[3] = (float)Src[12];  // [3][0]

		Dest[4] = (float)Src[1];   // [0][1]
		Dest[5] = (float)Src[5];   // [1][1]
		Dest[6] = (float)Src[9];   // [2][1]
		Dest[7] = (float)Src[13];  // [3][1]

		Dest[8] = (float)Src[2];   // [0][2]
		Dest[9] = (float)Src[6];   // [1][2]
		Dest[10] = (float)Src[10]; // [2][2]
		Dest[11] = (float)Src[14]; // [3][2]
	}

	inline void SetIdentity()
	{
		float* RESTRICT Dest = &(M[0][0]);

		Dest[ 0] = 1.0f;  // [0][0]
		Dest[ 1] = 0.0f;  // [0][1]
		Dest[ 2] = 0.0f;  // [0][2]
		Dest[ 3] = 0.0f;  // [0][3]

		Dest[ 4] = 0.0f;  // [1][0]
		Dest[ 5] = 1.0f;  // [1][1]
		Dest[ 6] = 0.0f;  // [1][2]
		Dest[ 7] = 0.0f;  // [1][3]

		Dest[ 8] = 0.0f;  // [2][0]
		Dest[ 9] = 0.0f;  // [2][1]
		Dest[10] = 1.0f;  // [2][2]
		Dest[11] = 0.0f;  // [2][3]
	}
} GCC_ALIGN(16);

/** Serializes the Matrix. */
inline FArchive& operator<<(FArchive& Ar, FMatrix3x4& M)
{
	Ar << M.M[0][0] << M.M[0][1] << M.M[0][2] << M.M[0][3];
	Ar << M.M[1][0] << M.M[1][1] << M.M[1][2] << M.M[1][3];
	Ar << M.M[2][0] << M.M[2][1] << M.M[2][2] << M.M[2][3];
	return Ar;
}

template<>
struct TShaderParameterTypeInfo<FMatrix3x4>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 3;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FMatrix3x4, Alignment>::Type;

	static const FShaderParametersMetadata* GetStructMetadata() { return NULL; }
};

FORCEINLINE void TransposeTransform(FMatrix3x4& DstTransform, const FMatrix44f& SrcTransform)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	VectorRegister4Float InRow0 = VectorLoadAligned(&(SrcTransform.M[0][0]));
	VectorRegister4Float InRow1 = VectorLoadAligned(&(SrcTransform.M[1][0]));
	VectorRegister4Float InRow2 = VectorLoadAligned(&(SrcTransform.M[2][0]));
	VectorRegister4Float InRow3 = VectorLoadAligned(&(SrcTransform.M[3][0]));

	VectorRegister4Float Temp0 = VectorShuffle(InRow0, InRow1, 0, 1, 0, 1);
	VectorRegister4Float Temp1 = VectorShuffle(InRow2, InRow3, 0, 1, 0, 1);
	VectorRegister4Float Temp2 = VectorShuffle(InRow0, InRow1, 2, 3, 2, 3);
	VectorRegister4Float Temp3 = VectorShuffle(InRow2, InRow3, 2, 3, 2, 3);

	VectorStoreAligned(VectorShuffle(Temp0, Temp1, 0, 2, 0, 2), &(DstTransform.M[0][0]));
	VectorStoreAligned(VectorShuffle(Temp0, Temp1, 1, 3, 1, 3), &(DstTransform.M[1][0]));
	VectorStoreAligned(VectorShuffle(Temp2, Temp3, 0, 2, 0, 2), &(DstTransform.M[2][0]));
#else
	SrcTransform.To3x4MatrixTranspose((float*)DstTransform.M);
#endif
}

ENGINE_API void TransposeTransforms(FMatrix3x4* DstTransforms, const FMatrix44f* SrcTransforms, int64 Count);
