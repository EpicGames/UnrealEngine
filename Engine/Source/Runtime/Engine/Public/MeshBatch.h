// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "EngineDefines.h"
#include "Engine/EngineTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
// previously included by GPUSceneWriter.h
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"
#endif
#include "HitProxies.h"
#include "RHIDefinitions.h"
#include "SceneDefinitions.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "RHIResources.h"


#define USE_MESH_BATCH_VALIDATION !UE_BUILD_SHIPPING

class FIndexBuffer;
class FLightCacheInterface;
class FMaterial;
class FMaterialRenderProxy;
class FPrimitiveSceneProxy;
class FPrimitiveUniformShaderParameters;
class FVertexFactory;
struct FInstanceDynamicData;
struct FInstanceSceneData;
struct FMaterialShaderParameters;
struct FRenderBounds;
template<typename TBufferStruct> class TUniformBuffer;
class FShaderParametersMetadata;
class FGPUSceneWriteDelegate;

/**
 * Abstract reference to a GPUSceneWriteDelegate, such that implementation details can be left to the renderer.
 * Only contains a raw pointer and performs no life-time management, instead it relies on being constructed by FGPUSceneWriteDelegate
 * which will use AllocateOneFrameResource to ensure a lifetime compatible with mesh batches.
 */
class FGPUSceneWriteDelegateRef
{
public:
	inline bool IsBound() const { return GPUSceneWriteDelegate != nullptr; }

	inline FGPUSceneWriteDelegate* operator->() const
	{
		check( IsBound() );
		return GPUSceneWriteDelegate;
	}

private:
	friend class FGPUSceneWriteDelegate;
	FGPUSceneWriteDelegate* GPUSceneWriteDelegate = nullptr;
};

/** Specifies the point during scene rendering that GPU Scene can be written to */
enum class EGPUSceneGPUWritePass : int8
{
	/** Invalid GPU write pass. Also used to signify writes that occur during upload. */
	None = -1,

	/** Writes to the GPU Scene will happen after rendering opaques (as well has after FXSystem post-opaque updates). */
	PostOpaqueRendering,

	Num
};

enum EPrimitiveIdMode
{
	/** 
	 * PrimitiveId will be taken from the FPrimitiveSceneInfo corresponding to the FMeshBatch. 
	 * Primitive data will then be fetched by supporting VF's from the GPUScene persistent PrimitiveBuffer.
	 */
	PrimID_FromPrimitiveSceneInfo		= 0,

	/** 
     * The renderer will upload Primitive data from the FMeshBatchElement's PrimitiveUniformBufferResource to the end of the GPUScene PrimitiveBuffer, and assign the offset to DynamicPrimitiveIndex.
	 * PrimitiveId for drawing will be computed as Scene->NumPrimitives + FMeshBatchElement's DynamicPrimitiveIndex. 
	 */
	PrimID_DynamicPrimitiveShaderData	= 1,

	/** 
	 * PrimitiveId will always be 0.  Instancing not supported.  
	 * View.PrimitiveSceneDataOverrideSRV must be set in this configuration to control what the shader fetches at PrimitiveId == 0.
	 */
	PrimID_ForceZero					= 2,

	PrimID_Num							= 4,
	PrimID_NumBits						= 2,
};

// Flag used to mark a primtive ID as dynamic, and thus needing translation (by adding the offset from the dynamic primitive collector).
inline constexpr int32 GPrimIDDynamicFlag = 1 << 31;

struct FMeshBatchElementDynamicIndexBuffer
{
	/** The vertex buffer to bind for draw calls. */
	FIndexBuffer* IndexBuffer = nullptr;
	/** The offset in to the index buffer (arbitrary limit to 16M indices so that FirstIndex|PrimitiveType fits into 32bits. */
	uint32 FirstIndex : 24;
	/** The offset in to the index buffer. */
	uint32 PrimitiveType : PT_NumBits;

	/** Returns true if the allocation is valid. */
	inline bool IsValid() const
	{
		return IndexBuffer != NULL;
	}
};

ENGINE_API bool AreCompressedTransformsSupported();

/**
 * Dynamic primitive/instance data for a mesh batch element.
 * 
 * NOTES:
 * - When applied to a FMeshBatchElement, data provided to the TConstArrayView members are expected to live until the end of the frame on the render thread
 * - If `DataWriterGPU` is bound and the TConstArrayView members are left empty, the delegate is expected to write any missing data, as it will not be uploaded
 */
struct FMeshBatchDynamicPrimitiveData
{
	TConstArrayView<FInstanceSceneData> InstanceSceneData;
	TConstArrayView<FInstanceDynamicData> InstanceDynamicData;
	TConstArrayView<FRenderBounds> InstanceLocalBounds;
	TConstArrayView<float> InstanceCustomData;
	FGPUSceneWriteDelegateRef DataWriterGPU;		
	EGPUSceneGPUWritePass DataWriterGPUPass = EGPUSceneGPUWritePass::None;
	uint16 PayloadDataFlags = 0;
	uint32 NumInstanceCustomDataFloats = 0;

	inline void SetPayloadDataFlags(uint16 Flags, bool bValue)
	{
		if (bValue)
		{
			PayloadDataFlags |= Flags;
		}
		else
		{
			PayloadDataFlags &= ~Flags;
		}
	}
	
	inline void EnableInstanceDynamicData(bool bEnable)  { SetPayloadDataFlags(INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA, bEnable); }
	inline void EnableInstanceSkinningData(bool bEnable) { SetPayloadDataFlags(INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA, bEnable); }
	inline void EnableInstanceLocalBounds(bool bEnable)  { SetPayloadDataFlags(INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS, bEnable); }
	inline void SetNumInstanceCustomDataFloats(uint32 NumFloats)
	{
		SetPayloadDataFlags(INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA, NumFloats > 0);
		NumInstanceCustomDataFloats = NumFloats;
	}

	/**
	 * Computes the full float4 stride of the instance's payload data.
	 * NOTE: Needs to align with GetInstancePayloadDataOffsets in SceneData.ush
	 **/
	inline uint32 GetPayloadFloat4Stride() const
	{
		uint32 Total = 0;
		
		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS)
		{
			Total += 2;
		}
		else if (PayloadDataFlags & (INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET | INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA))
		{
			Total += 1;
		}

		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA)
		{
			if (AreCompressedTransformsSupported())
			{
				Total += 2;
			}
			else
			{
				Total += 3;
			}
		}

		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA)
		{
			Total += 1;
		}

		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS)
		{
			Total += 1;
		}
		
		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA)
		{
			Total += FMath::DivideAndRoundUp(NumInstanceCustomDataFloats, 4u);
		}

		return Total;
	}

	inline void Validate(uint32 NumInstances) const
	{
#if DO_CHECK
		// Ensure array views are sized exactly for all instances, or are empty and there is a GPU writer
		const bool bGPUWrite = DataWriterGPU.IsBound();
		checkf(uint32(InstanceSceneData.Num()) == NumInstances || (bGPUWrite && InstanceSceneData.Num() == 0),
			TEXT("DynamicPrimitiveData provided should have %u instances in InstanceSceneData. Found %d"),
			NumInstances, InstanceSceneData.Num());
		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA)
		{
			checkf(uint32(InstanceDynamicData.Num()) == NumInstances || (bGPUWrite && InstanceDynamicData.Num() == 0),
				TEXT("DynamicPrimitiveData provided should have %u elements in InstanceDynamicData. Found %d"),
				NumInstances, InstanceDynamicData.Num());
		}
		if (PayloadDataFlags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA)
		{
			checkf(NumInstanceCustomDataFloats > 0,
				TEXT("DynamicPrimitiveData provided has the custom data flag set, but NumInstanceCustomDataFloats == 0"));
			checkf(uint32(InstanceCustomData.Num()) == NumInstances * NumInstanceCustomDataFloats || (bGPUWrite && InstanceCustomData.Num() == 0),
				TEXT("DynamicPrimitiveData provided should have %u elements in InstanceCustomData. Found %d"),
				NumInstances * NumInstanceCustomDataFloats, InstanceCustomData.Num());
		}
#endif
	}
};

/**
 * A batch mesh element definition.
 */
struct FMeshBatchElement
{
	/** 
	 * Primitive uniform buffer RHI
	 * Must be null for vertex factories that manually fetch primitive data from scene data, in which case FPrimitiveSceneProxy::UniformBuffer will be used.
	 */
	FRHIUniformBuffer* PrimitiveUniformBuffer;

	/** 
	 * Primitive uniform buffer to use for rendering, used when PrimitiveUniformBuffer is null. 
	 * This interface allows a FMeshBatchElement to be setup for a uniform buffer that has not been initialized yet, (TUniformBuffer* is known but not the FRHIUniformBuffer*)
	 */
	const TUniformBuffer<FPrimitiveUniformShaderParameters>* PrimitiveUniformBufferResource;

	/** Uniform buffer containing the "loose" parameters that aren't wrapped in other uniform buffers. Those parameters can be unique per mesh batch, e.g. view dependent. */
	FUniformBufferRHIRef LooseParametersUniformBuffer;

	/** The index buffer to draw the mesh batch with. */
	const FIndexBuffer* IndexBuffer;

	/**
	 * Store dynamic index buffer
	 * This is used for objects whose triangles are dynamically sorted for a particular view (i.e., per-object order-independent-transparency)
	*/
	FMeshBatchElementDynamicIndexBuffer DynamicIndexBuffer;

	/** Instance runs, where number of runs is specified by NumInstances.  Run structure is [StartInstanceIndex, EndInstanceIndex]. */
	uint32* InstanceRuns;

	const void* UserData;

	// Meaning depends on the vertex factory, e.g. FGPUSkinPassthroughVertexFactory: element index in FGPUSkinCache::CachedElements
	void* VertexFactoryUserData;

	FRHIBuffer* IndirectArgsBuffer;
	uint32 IndirectArgsOffset;

	/** Assigned by renderer */
	EPrimitiveIdMode PrimitiveIdMode : PrimID_NumBits + 1;

	uint32 FirstIndex;
	/** When 0, IndirectArgsBuffer will be used. */
	uint32 NumPrimitives;

	/** Number of instances to draw.  If InstanceRuns is valid, this is actually the number of runs in InstanceRuns. */
	uint32 NumInstances;
	uint32 BaseVertexIndex;
	uint32 MinVertexIndex;
	uint32 MaxVertexIndex;
	int32 UserIndex;
	float MinScreenSize;
	float MaxScreenSize;

	uint32 InstancedLODIndex : 4;
	uint32 InstancedLODRange : 4;
	uint32 bUserDataIsColorVertexBuffer : 1;
	uint32 bIsInstanceRuns : 1;
	uint32 bForceInstanceCulling : 1;
	uint32 bPreserveInstanceOrder : 1;
	uint32 bFetchInstanceCountFromScene : 1;

#if UE_ENABLE_DEBUG_DRAWING
	/** Conceptual element index used for debug viewmodes. */
	int32 VisualizeElementIndex : 8;
	/** Skin Cache debug visualization color. */
	FColor SkinCacheDebugColor = FColor::White;
#endif

	/**
	 * Source instance scene data and payload data for dynamic primitives. Must be provided for dynamic primitives that have more than a single instance.
	 * NOTE: The lifetime of the object pointed to is expected to match or exceed that of the mesh batch itself.
	 **/
	const FMeshBatchDynamicPrimitiveData* DynamicPrimitiveData;
	uint32 DynamicPrimitiveIndex;
	uint32 DynamicPrimitiveInstanceSceneDataOffset;

	inline int32 GetNumPrimitives() const
	{
		if (bIsInstanceRuns && InstanceRuns)
		{
			int32 Count = 0;
			for (uint32 Run = 0; Run < NumInstances; Run++)
			{
				Count += NumPrimitives * (InstanceRuns[Run * 2 + 1] - InstanceRuns[Run * 2] + 1);
			}
			return Count;
		}
		else
		{
			return NumPrimitives * NumInstances;
		}
	}

	FMeshBatchElement()
	:	PrimitiveUniformBuffer(nullptr)
	,	PrimitiveUniformBufferResource(nullptr)
	,	IndexBuffer(nullptr)
	,	InstanceRuns(nullptr)
	,	UserData(nullptr)
	,	VertexFactoryUserData(nullptr)
	,	IndirectArgsBuffer(nullptr)
	,	IndirectArgsOffset(0)
	,	PrimitiveIdMode(PrimID_FromPrimitiveSceneInfo)
	,	NumInstances(1)
	,	BaseVertexIndex(0)
	,	UserIndex(-1)
	,	MinScreenSize(0.0f)
	,	MaxScreenSize(1.0f)
	,	InstancedLODIndex(0)
	,	InstancedLODRange(0)
	,	bUserDataIsColorVertexBuffer(false)
	,	bIsInstanceRuns(false)
	,	bForceInstanceCulling(false)
	,	bPreserveInstanceOrder(false)
	,	bFetchInstanceCountFromScene(false)
#if UE_ENABLE_DEBUG_DRAWING
	,	VisualizeElementIndex(INDEX_NONE)
#endif
	,	DynamicPrimitiveData(nullptr)
	,	DynamicPrimitiveIndex(INDEX_NONE)
	,	DynamicPrimitiveInstanceSceneDataOffset(INDEX_NONE)
	{
	}
};

// Helper functions for hair strands shaders
ENGINE_API bool IsHairStrandsGeometrySupported(const EShaderPlatform Platform);
ENGINE_API bool IsCompatibleWithHairStrands(const FMaterial* Material, const ERHIFeatureLevel::Type FeatureLevel);
ENGINE_API bool IsCompatibleWithHairStrands(EShaderPlatform Platform, const FMaterialShaderParameters& Parameters);

/**
 * A batch of mesh elements, all with the same material and vertex buffer
 */
struct FMeshBatch
{
	TArray<FMeshBatchElement,TInlineAllocator<1> > Elements;

	/** Vertex factory for rendering, required. */
	const FVertexFactory* VertexFactory;

	/** Material proxy for rendering, required. */
	const FMaterialRenderProxy* MaterialRenderProxy;

	// can be NULL
	const FLightCacheInterface* LCI;

	/** The current hit proxy ID being rendered. */
	FHitProxyId BatchHitProxyId;

	/** If set, the ID to use during culling when GPU skin cache is available to provide dynamic tight-fit bounds. */
	int32 DynamicMeshBoundsIndex = INDEX_NONE;

	/** Segment index of the original mesh */
	int32 SegmentIndex;

	/* Mesh Id in a primitive. Used for stable sorting of draws belonging to the same primitive. **/
	uint16 MeshIdInPrimitive;

	/** LOD index of the mesh, used for fading LOD transitions. */
	int8 LODIndex;

	uint32 ReverseCulling : 1;
	uint32 bDisableBackfaceCulling : 1;

	/** 
	 * Pass feature relevance flags.  Allows a proxy to submit fast representations for passes which can take advantage of it, 
	 * for example separate index buffer for depth-only rendering since vertices can be merged based on position and ignore UV differences.
	 */
	uint32 CastShadow		: 1;	// Whether it can be used in shadow renderpasses.
	uint32 bUseForMaterial	: 1;	// Whether it can be used in renderpasses requiring material outputs.
	uint32 bUseForDepthPass : 1;	// Whether it can be used in depth pass.
	uint32 bUseAsOccluder	: 1;	// Hint whether this mesh is a good occluder.
	uint32 bWireframe		: 1;
	// e.g. PT_TriangleList(default), PT_LineList, ..
	uint32 Type : PT_NumBits;
	// e.g. SDPG_World (default), SDPG_Foreground
	uint32 DepthPriorityGroup : SDPG_NumBits;

	/** Whether view mode overrides can be applied to this mesh eg unlit, wireframe. */
	uint32 bCanApplyViewModeOverrides : 1;

	/** 
	 * Whether to treat the batch as selected in special viewmodes like wireframe. 
	 * This is needed instead of just Proxy->IsSelected() because some proxies do weird things with selection highlighting, like FStaticMeshSceneProxy.
	 */
	uint32 bUseWireframeSelectionColoring : 1;

	/** 
	 * Whether the batch should receive the selection outline.  
	 * This is useful for proxies which support selection on a per-mesh batch basis.
	 * They submit multiple mesh batches when selected, some of which have bUseSelectionOutline enabled.
	 */
	uint32 bUseSelectionOutline : 1;

	/** Whether the mesh batch can be selected through editor selection, aka hit proxies. */
	uint32 bSelectable : 1;
	
	/** Whether the mesh batch should apply dithered LOD. */
	uint32 bDitheredLODTransition : 1;

	/** Whether the mesh batch can be rendered to virtual textures. */
	uint32 bRenderToVirtualTexture : 1;
	/** What virtual texture material type this mesh batch should be rendered with. */
	uint32 RuntimeVirtualTextureMaterialType : RuntimeVirtualTexture::MaterialType_NumBits;
	
	/** Whether mesh is rendered with overlay material. */
	uint32 bOverlayMaterial	: 1;

#if RHI_RAYTRACING
	uint32 CastRayTracedShadow : 1;	// Whether it casts ray traced shadow.
	uint32 bForceTwoSidedInRayTracing : 1;	// Whether segment should be considered two sided in ray tracing regardless of material.
#endif

	/** 
	 * Whether mesh has a view dependent draw arguments. 
	 * Gives an opportunity to override mesh arguments for a View just before creating MDC for a mesh (makes MDC "non-cached")
	 */
	uint32 bViewDependentArguments : 1;

	/** Whether the mesh batch should be used in the depth-only passes of rendering the water info texture for the water plugin */
	uint32 bUseForWaterInfoTextureDepth : 1;
	
	/** Gives the opportunity to select a different VF for the landscape for the lumen surface cache capture */
	uint32 bUseForLumenSurfaceCacheCapture : 1;

#if UE_ENABLE_DEBUG_DRAWING
	/** Conceptual HLOD index used for the HLOD Coloration visualization. */
	int8 VisualizeHLODIndex;

	/** Conceptual LOD index used for the LOD Coloration visualization. */
	int8 VisualizeLODIndex;

	/** Dithering direction for LOD transitions in debug visualization */
	enum class EDitherState : uint8
	{
		None = 0,
		FadeOut = 1,
		FadeIn = 2
	};

	/** Cached dithering state value for LOD transitions. Computed in GetDynamicMeshElements. */
	EDitherState CachedLODDither = EDitherState::None;
#endif

	ENGINE_API bool IsTranslucent(ERHIFeatureLevel::Type InFeatureLevel) const;

	// todo: can be optimized with a single function that returns multiple states (Translucent, Decal, Masked) 
	ENGINE_API bool IsDecal(ERHIFeatureLevel::Type InFeatureLevel) const;

	ENGINE_API bool IsDualBlend(ERHIFeatureLevel::Type InFeatureLevel) const;

	ENGINE_API bool UseForHairStrands(ERHIFeatureLevel::Type InFeatureLevel) const;

	ENGINE_API bool IsMasked(ERHIFeatureLevel::Type InFeatureLevel) const;

	inline int32 GetNumPrimitives() const
	{
		int32 Count = 0;
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ++ElementIdx)
		{
			Count += Elements[ElementIdx].GetNumPrimitives();
		}
		return Count;
	}

	inline bool HasAnyDrawCalls() const
	{
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ++ElementIdx)
		{
			if (Elements[ElementIdx].GetNumPrimitives() > 0 || Elements[ElementIdx].IndirectArgsBuffer || Elements[ElementIdx].bFetchInstanceCountFromScene)
			{
				return true;
			}
		}
		return false;
	}

	ENGINE_API void PreparePrimitiveUniformBuffer(const FPrimitiveSceneProxy* PrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel);

#if USE_MESH_BATCH_VALIDATION
	ENGINE_API bool Validate(const FPrimitiveSceneProxy* SceneProxy, ERHIFeatureLevel::Type FeatureLevel) const;
#else
	inline bool Validate(const FPrimitiveSceneProxy* SceneProxy, ERHIFeatureLevel::Type FeatureLevel) const { return true; }
#endif

	/** Default constructor. */
	FMeshBatch()
	:	VertexFactory(nullptr)
	,	MaterialRenderProxy(nullptr)
	,	LCI(nullptr)
	,	SegmentIndex(INDEX_NONE)
	,	MeshIdInPrimitive(0)
	,	LODIndex(INDEX_NONE)
	,	ReverseCulling(false)
	,	bDisableBackfaceCulling(false)
	,	CastShadow(true)
	,   bUseForMaterial(true)
	,	bUseForDepthPass(true)
	,	bUseAsOccluder(true)
	,	bWireframe(false)
	,	Type(PT_TriangleList)
	,	DepthPriorityGroup(SDPG_World)
	,	bCanApplyViewModeOverrides(false)
	,	bUseWireframeSelectionColoring(false)
	,	bUseSelectionOutline(true)
	,	bSelectable(true)
	,	bDitheredLODTransition(false)
	,	bRenderToVirtualTexture(false)
	,	RuntimeVirtualTextureMaterialType(0)
	,	bOverlayMaterial(false)
#if RHI_RAYTRACING
	,	CastRayTracedShadow(true)
	,	bForceTwoSidedInRayTracing(false)
#endif
	,	bViewDependentArguments(false)
	,	bUseForWaterInfoTextureDepth(false)
	,	bUseForLumenSurfaceCacheCapture(false)
#if (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
	,	VisualizeHLODIndex(INDEX_NONE)
#endif
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	,	VisualizeLODIndex(INDEX_NONE)
#endif
	{
		// By default always add the first element.
		Elements.AddDefaulted();
	}
};

struct FUniformBufferValue
{
	const FShaderParametersMetadata* Type = nullptr;
	FRHIUniformBuffer* UniformBuffer;
};




