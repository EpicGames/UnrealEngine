// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/DeveloperSettings.h"
#include "Logging/LogMacros.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

#include "MeshBudgetProjectSettings.generated.h"

class UStaticMesh;

DECLARE_LOG_CATEGORY_EXTERN(LogMeshBudget, Log, All)

USTRUCT()
struct FStaticMeshBudgetInfo
{
	GENERATED_BODY()
	/** The name of the LOD group we will use for this budget.*/
	UPROPERTY(EditAnywhere, Category = "StaticMesh")
	FName LodGroupName = NAME_None;

	/** The minimum volume extent to assign this budget info. We will compare the mesh bounding box extent to this value. */
	UPROPERTY(EditAnywhere, Category = "StaticMesh")
	double MinimumExtent = 0.0;
};

UCLASS(config=Engine, meta=(DisplayName="Mesh Budget"), MinimalAPI)
class UMeshBudgetProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/**
	 * Enable/disable the static mesh budget.
	 * Static mesh budget will auto assign a lod group to any static mesh when loading or importing the asset in the editor.
	 * The auto budget will not override a static mesh lod group, user can control the lod group for a specific asset.
	 * @note: When the static mesh budget is enable and properly configure, there will be no static mesh without a lod group in the editor.
	 *        
	 */
	UPROPERTY(EditAnywhere, config, Category = "StaticMesh")
	bool bEnableStaticMeshBudget = false;

	/**
	 * The static mesh budgets array.
	 */
	UPROPERTY(EditAnywhere, config, Category = "StaticMesh")
	TArray<FStaticMeshBudgetInfo> StaticMeshBudgetInfos;
};

class FMeshBudgetProjectSettingsUtils
{
public:
#if WITH_EDITOR
	static ENGINE_API void SetLodGroupForStaticMesh(UStaticMesh* StaticMesh);
#endif
};
