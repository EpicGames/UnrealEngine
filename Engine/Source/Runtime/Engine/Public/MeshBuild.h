// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshBuild.h: Contains commonly used functions and classes for building
	mesh data into engine usable form.

=============================================================================*/

#pragma once

#include "CoreMinimal.h"

struct FOverlappingThresholds
{
public:
	FOverlappingThresholds()
		: ThresholdPosition(UE_THRESH_POINTS_ARE_SAME)
		, ThresholdTangentNormal(UE_THRESH_NORMALS_ARE_SAME)
		, ThresholdUV(UE_THRESH_UVS_ARE_SAME)
		, MorphThresholdPosition(UE_THRESH_POINTS_ARE_NEAR)
	{}

	/** Threshold use to decide if two vertex position are equal. */
	float ThresholdPosition;
	
	/** Threshold use to decide if two normal, tangents or bi-normals are equal. */
	float ThresholdTangentNormal;
	
	/** Threshold use to decide if two UVs are equal. */
	float ThresholdUV;

	/** Threshold use to decide if two vertex position are equal. */
	float MorphThresholdPosition;
};


/**
 * Returns true if the specified points are about equal
 */
inline bool PointsEqual(const FVector3f& V1,const FVector3f& V2, bool bUseEpsilonCompare = true )
{
	const float Epsilon = bUseEpsilonCompare ? UE_THRESH_POINTS_ARE_SAME : 0.0f;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon && FMath::Abs(V1.Z - V2.Z) <= Epsilon;
}

inline bool PointsEqual(const FVector3f& V1, const FVector3f& V2, const FOverlappingThresholds& OverlappingThreshold)
{
	const float Epsilon = OverlappingThreshold.ThresholdPosition;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon && FMath::Abs(V1.Z - V2.Z) <= Epsilon;
}

/**
 * Returns true if the specified normal vectors are about equal
 */
inline bool NormalsEqual(const FVector3f& V1,const FVector3f& V2)
{
	const float Epsilon = UE_THRESH_NORMALS_ARE_SAME;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon && FMath::Abs(V1.Z - V2.Z) <= Epsilon;
}

inline bool UVsEqual(const FVector2f& V1, const FVector2f& V2)
{
	const float Epsilon = 1.0f / 1024.0f;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon;
}

inline bool NormalsEqual(const FVector3f& V1,const FVector3f& V2, const FOverlappingThresholds& OverlappingThreshold)
{
	const float Epsilon = OverlappingThreshold.ThresholdTangentNormal;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon && FMath::Abs(V1.Z - V2.Z) <= Epsilon;
}

inline bool UVsEqual(const FVector2f& V1, const FVector2f& V2, const FOverlappingThresholds& OverlappingThreshold)
{
	const float Epsilon = OverlappingThreshold.ThresholdUV;
	return FMath::Abs(V1.X - V2.X) <= Epsilon && FMath::Abs(V1.Y - V2.Y) <= Epsilon;
}
