// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneManagement.h"

class FMeshBuilderOneFrameResources : public FOneFrameResource
{
public:
	class FPooledDynamicMeshVertexBuffer* VertexBuffer = nullptr;
	class FPooledDynamicMeshIndexBuffer* IndexBuffer = nullptr;
	class FPooledDynamicMeshVertexFactory* VertexFactory = nullptr;
	class FDynamicMeshPrimitiveUniformBuffer* PrimitiveUniformBuffer = nullptr;
	virtual ENGINE_API ~FMeshBuilderOneFrameResources();

	inline bool IsValidForRendering() 
	{
		return VertexBuffer && IndexBuffer && PrimitiveUniformBuffer && VertexFactory;
	}
};
