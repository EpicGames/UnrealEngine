// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "DerivedMeshDataTaskUtils.h"
#include "MeshCardRepresentation.h"
#include "RenderDeferredCleanup.h"
#include "IAssetCompilingManager.h"

#if WITH_EDITORONLY_DATA
#include "MeshUtilities.h"
#endif

#include <atomic>

class FAsyncCompilationNotification;
class FObjectCacheContextScope;
class UStaticMesh;
class FDistanceFieldVolumeData;
struct FAppendToClassSchemaContext;
struct FSignedDistanceFieldBuildSectionData;

class FLumenCardBuildData
{
public:
	FLumenCardOBBf OBB;
	uint8 AxisAlignedDirectionIndex;
	ELumenCardDilationMode DilationMode = ELumenCardDilationMode::Disabled;

	bool ContainsNaN() const
	{
		return OBB.ContainsNaN();
	}

	friend FArchive& operator<<(FArchive& Ar, FLumenCardBuildData& Data)
	{
		// Note: this is derived data, no need for versioning (bump the DDC guid)
		Ar << Data.OBB;
		Ar << Data.AxisAlignedDirectionIndex;
		return Ar;
	}
};

class FLumenCardBuildDebugData
{
public:
	enum class ESurfelType : uint8
	{
		Valid,
		Invalid,

		Idle,
		Cluster,
		Used
	};

	struct FRay
	{
		FVector3f RayStart;
		FVector3f RayEnd;
		bool bHit;
	};

	struct FSurfel
	{
		FVector3f Position;
		FVector3f Normal;
		float Coverage = 0.0f;
		float Visibility = 1.0f;
		int32 SourceSurfelIndex = -1;
		ESurfelType Type;
	};

	struct FSurfelCluster
	{
		TArray<FSurfel> Surfels;
		TArray<FRay> Rays;
	};

	TArray<FSurfel> Surfels;
	TArray<FRay> SurfelRays;
	TArray<FSurfelCluster> Clusters;
	int32 NumSurfels = 0;

	void Init()
	{
		Surfels.Reset();
		SurfelRays.Reset();
		Clusters.Reset();
		NumSurfels = 0;
	}
};

class FMeshCardsBuildData
{
public:
	FBox Bounds = FBox(ForceInit);
	bool bMostlyTwoSided = false;
	TArray<FLumenCardBuildData> CardBuildData;

	// Temporary debug visualization data, don't serialize
	FLumenCardBuildDebugData DebugData;

	bool ContainsNaN() const
	{
		return Bounds.ContainsNaN() || CardBuildData.ContainsByPredicate([](const FLumenCardBuildData& Data) { return Data.ContainsNaN(); });
	}

	void FitToNewBounds(const FBox& NewBounds);

	friend FArchive& operator<<(FArchive& Ar, FMeshCardsBuildData& Data)
	{
		// Note: this is derived data, no need for versioning (bump the DDC guid)
		Ar << Data.Bounds;
		Ar << Data.bMostlyTwoSided;
		Ar << Data.CardBuildData;
		return Ar;
	}
};

// Unique id per card representation data instance.
class FCardRepresentationDataId
{
public:
	uint32 Value = 0;

	bool IsValid() const
	{
		return Value != 0;
	}

	bool operator==(FCardRepresentationDataId B) const
	{
		return Value == B.Value;
	}

	friend uint32 GetTypeHash(FCardRepresentationDataId DataId)
	{
		return GetTypeHash(DataId.Value);
	}
};

/** Card representation payload and output of the mesh build process. */
class FCardRepresentationData : public FDeferredCleanupInterface
{
public:

	FMeshCardsBuildData MeshCardsBuildData;

	FCardRepresentationDataId CardRepresentationDataId;

	FCardRepresentationData()
	{
		// 0 means invalid id.
		static std::atomic<uint32> NextCardRepresentationId { 1 };
		CardRepresentationDataId.Value = NextCardRepresentationId++;
	}

	void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) const
	{
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));
	}

	SIZE_T GetResourceSizeBytes() const
	{
		FResourceSizeEx ResSize;
		GetResourceSizeEx(ResSize);
		return ResSize.GetTotalMemoryBytes();
	}

	bool ContainsNaN() const
	{
		return MeshCardsBuildData.ContainsNaN();
	}

#if WITH_EDITORONLY_DATA

	void CacheDerivedData(
		const FString& InDDCKey,
		UStaticMesh* Mesh,
		UStaticMesh* GenerateSource,
		int32 MaxLumenMeshCards,
		bool bGenerateDistanceFieldAsIfTwoSided,
		FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData = FSourceMeshDataForDerivedDataTask(),
		TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData = TArray<FSignedDistanceFieldBuildPartData>());

	UE_DEPRECATED(5.6, "Provide OptionalSourceMeshData by value (ideally using MoveTemp(...) to avoid copying the data)")
	void CacheDerivedData(
		const FString& InDDCKey,
		const ITargetPlatform* TargetPlatform,
		UStaticMesh* Mesh,
		UStaticMesh* GenerateSource,
		int32 MaxLumenMeshCards,
		bool bGenerateDistanceFieldAsIfTwoSided,
		FSourceMeshDataForDerivedDataTask* OptionalSourceMeshData)
	{
		CacheDerivedData(InDDCKey, Mesh, GenerateSource, MaxLumenMeshCards, bGenerateDistanceFieldAsIfTwoSided, OptionalSourceMeshData ? *OptionalSourceMeshData : FSourceMeshDataForDerivedDataTask());
	}

#endif

	friend FArchive& operator<<(FArchive& Ar, FCardRepresentationData& Data)
	{
		// Note: this is derived data, no need for versioning (bump the DDC guid)
		Ar << Data.MeshCardsBuildData;
		return Ar;
	}
};


class FAsyncCardRepresentationTask;
class FAsyncCardRepresentationTaskWorker : public FNonAbandonableTask
{
public:
	FAsyncCardRepresentationTaskWorker(FAsyncCardRepresentationTask& InTask)
		: Task(InTask)
	{
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncCardRepresentationTaskWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();

private:
	FAsyncCardRepresentationTask& Task;
};

class FAsyncCardRepresentationTask
{
public:
	bool bSuccess = false;

#if WITH_EDITOR
	TArray<FSignedDistanceFieldBuildSectionData> SectionData;
	TArray<FSignedDistanceFieldBuildPartData> PartData;
#endif

	FSourceMeshDataForDerivedDataTask SourceMeshData;
	bool bGenerateDistanceFieldAsIfTwoSided = false;
	int32 MaxLumenMeshCards = 0;
	UStaticMesh* StaticMesh = nullptr;
	UStaticMesh* GenerateSource = nullptr;
	FDistanceFieldVolumeData* DistanceFieldData = nullptr;
	FString DDCKey;
	FCardRepresentationData* GeneratedCardRepresentation;
	TUniquePtr<FAsyncTask<FAsyncCardRepresentationTaskWorker>> AsyncTask = nullptr;
};

/** Class that manages asynchronous building of mesh distance fields. */
class FCardRepresentationAsyncQueue : IAssetCompilingManager
{
public:

	ENGINE_API FCardRepresentationAsyncQueue();

	ENGINE_API virtual ~FCardRepresentationAsyncQueue();

	/** Adds a new build task. */
	ENGINE_API void AddTask(FAsyncCardRepresentationTask* Task);

	/** Cancel the build on this specific static mesh or block until it is completed if already started. */
	ENGINE_API void CancelBuild(UStaticMesh* StaticMesh);

	/** Cancel the build on these meshes or block until they are completed if already started. */
	ENGINE_API void CancelBuilds(const TSet<UStaticMesh*>& InStaticMeshes);

	/** Blocks the main thread until the async build are either canceled or completed. */
	ENGINE_API void CancelAllOutstandingBuilds();

	/** Blocks the main thread until the async build of the specified mesh is complete. */
	ENGINE_API void BlockUntilBuildComplete(UStaticMesh* InStaticMesh, bool bWarnIfBlocked);

	/** Blocks the main thread until all async builds complete. */
	ENGINE_API void BlockUntilAllBuildsComplete();

	/** Called once per frame, fetches completed tasks and applies them to the scene. */
	ENGINE_API void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	/** Fetches completed tasks for a specific static mesh and applies them to the scene */
	ENGINE_API void ProcessAsyncTasksForMesh(const UStaticMesh* InStaticMesh, bool bLimitExecutionTime = false);

	/** Blocks until it is safe to shut down (worker threads are idle). */
	ENGINE_API void Shutdown() override;

	int32 GetNumOutstandingTasks() const
	{
		FScopeLock Lock(&CriticalSection);
		return ReferencedTasks.Num();
	}

	/** Returns the number of outstanding referenced tasks for a specific static mesh. */
	ENGINE_API int32 GetNumOutstandingTasks(const UStaticMesh* InStaticMesh) const;

	/** Get the name of the asset type this compiler handles */
	ENGINE_API static FName GetStaticAssetTypeName();

private:
	friend FAsyncCardRepresentationTaskWorker;
	friend class FDistanceFieldAsyncQueue;

	ENGINE_API FName GetAssetTypeName() const override;
	ENGINE_API FTextFormat GetAssetNameFormat() const override;
	ENGINE_API TArrayView<FName> GetDependentTypeNames() const override;
	ENGINE_API int32 GetNumRemainingAssets() const override;
	ENGINE_API void FinishAllCompilation() override;

	void ProcessPendingTasks();
	void ProcessPendingTasksForMesh(const UStaticMesh* InStaticMesh);

	TUniquePtr<FQueuedThreadPool> ThreadPool;

	/** Builds a single task. Called from the worker thread. */
	void Build(FAsyncCardRepresentationTask* Task);

	/** Change the priority of the background task. */
	void RescheduleBackgroundTask(FAsyncCardRepresentationTask* InTask, EQueuedWorkPriority InPriority);

	/** Task will be sent to a background worker. */
	void StartBackgroundTask(FAsyncCardRepresentationTask* Task);

	/** Task is being run immediately from an existing background worker (typically FAsyncDistanceFieldTask). */
	void RunImmediateTask(FAsyncCardRepresentationTask* Task);

	/** Cancel or finish any work for any task matching the predicate. */
	void CancelAndDeleteTaskByPredicate(TFunctionRef<bool(FAsyncCardRepresentationTask*)> ShouldCancelPredicate);

	/** Cancel or finish any work for the given task. */
	void CancelAndDeleteTask(const TSet<FAsyncCardRepresentationTask*>& Tasks);

	/** Handle generic finish compilation */
	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	/** Mark compilation of the provided objects as canceled. */
	void MarkCompilationAsCanceled(TArrayView<UObject* const> InObjects) override;

	/** Return whether the task has become invalid and should be canceled (i.e. reference unreachable objects) */
	bool IsTaskInvalid(FAsyncCardRepresentationTask* Task) const;

	/** Used to cancel tasks that are not needed anymore when garbage collection occurs */
	void OnPostReachabilityAnalysis();

#if WITH_EDITOR
	/** Finishes a singular async task to completion and deletes its owner task */
	void FinalizeAndDeleteTask(FAsyncCardRepresentationTask* InTask, FObjectCacheContextScope& InObjectCacheScope);
#endif

	/** Game-thread managed list of tasks in the async system. */
	TSet<FAsyncCardRepresentationTask*> ReferencedTasks;

	/** Tasks that are waiting on static mesh compilation to proceed */
	TSet<FAsyncCardRepresentationTask*> PendingTasks;

	/** Tasks that have completed processing. */
	TSet<FAsyncCardRepresentationTask*> CompletedTasks;

	FDelegateHandle PostReachabilityAnalysisHandle;

	class IMeshUtilities* MeshUtilities;

	mutable FCriticalSection CriticalSection;

	TUniquePtr<FAsyncCompilationNotification> Notification;
};

/** Global build queue. */
extern ENGINE_API FCardRepresentationAsyncQueue* GCardRepresentationAsyncQueue;

#if WITH_EDITORONLY_DATA
/**
 * IncrementalCook: Any UClass that uses data created by BeginCacheMeshCardRepresentation must add the version
 * numbers for that calculation to their ClassSchema, to invalidate their cooked data when those constants change.
 * This function handles that appending.
 */
extern ENGINE_API void AppendMeshCardConstantsToClassSchema(FAppendToClassSchemaContext& Context);

extern ENGINE_API FString BuildCardRepresentationDerivedDataKey(const FString& InMeshKey);

extern ENGINE_API void BeginCacheMeshCardRepresentation(
	UStaticMesh* StaticMeshAsset,
	class FStaticMeshRenderData& RenderData,
	const FString& DistanceFieldKey,
	FSourceMeshDataForDerivedDataTask&& OptionalSourceMeshData = FSourceMeshDataForDerivedDataTask(),
	TArray<FSignedDistanceFieldBuildPartData>&& OptionalPartData = TArray<FSignedDistanceFieldBuildPartData>());

UE_DEPRECATED(5.6, "Provide OptionalSourceMeshData by value (ideally using MoveTemp(...) to avoid copying the data)")
inline void BeginCacheMeshCardRepresentation(
	const ITargetPlatform* TargetPlatform,
	UStaticMesh* StaticMeshAsset,
	class FStaticMeshRenderData& RenderData,
	const FString& DistanceFieldKey,
	FSourceMeshDataForDerivedDataTask* OptionalSourceMeshData)
{
	BeginCacheMeshCardRepresentation(StaticMeshAsset, RenderData, DistanceFieldKey, OptionalSourceMeshData ? *OptionalSourceMeshData : FSourceMeshDataForDerivedDataTask());
}
#endif
