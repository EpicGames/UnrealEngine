// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshCardRepresentation.h
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "Math/Box.h"
#include "Math/Matrix.h"
#include "Math/Vector.h"

class FMeshCardsBuildData;

enum class ELumenCardDilationMode : uint8
{
	Disabled,
	DilateOneTexel,
	Num
};

namespace MeshCardRepresentation
{
	// Generation config
	ENGINE_API float GetMinDensity();
	ENGINE_API float GetNormalTreshold();

	// Debugging
	ENGINE_API bool IsDebugMode();
	ENGINE_API int32 GetDebugSurfelDirection();

	// Util
	ENGINE_API FVector3f GetAxisAlignedDirection(uint32 AxisAlignedDirectionIndex);
	ENGINE_API int32 GetAxisAlignedDirectionIndex(const FVector3f& Direction);
	ENGINE_API void SetCardsFromBounds(FMeshCardsBuildData& CardData, ELumenCardDilationMode DilationMode = ELumenCardDilationMode::Disabled, bool bCardCoversHalfBounds = false);
};

template<typename T>
class TLumenCardOBB
{
public:
	UE::Math::TVector<T> Origin;
	UE::Math::TVector<float> AxisX;
	UE::Math::TVector<float> AxisY;
	UE::Math::TVector<float> AxisZ;
	UE::Math::TVector<float> Extent;

	/** Default constructor (no initialization). */
	TLumenCardOBB() = default;

	/**
	 * Creates and initializes a new OBB with zeros
	 *
	 * Use enum value EForceInit::ForceInit to force OBB initialization.
	 */
	explicit TLumenCardOBB(EForceInit)
	{
		Reset();
	}

	// Conversion from other type.
	template<typename FArg UE_REQUIRES(!std::is_same_v<T, FArg>)>
	explicit TLumenCardOBB(const TLumenCardOBB<FArg>& From)
	{
		Origin = UE::Math::TVector<T>(From.Origin);
		AxisX = From.AxisX;
		AxisY = From.AxisY;
		AxisZ = From.AxisZ;
		Extent = From.Extent;
	}

	void Reset()
	{
		Origin = UE::Math::TVector<T>::ZeroVector;
		AxisX = UE::Math::TVector<float>::ZeroVector;
		AxisY = UE::Math::TVector<float>::ZeroVector;
		AxisZ = UE::Math::TVector<float>::ZeroVector;
		Extent = UE::Math::TVector<float>::ZeroVector;
	}

	UE::Math::TVector<float> GetDirection() const
	{
		return AxisZ;
	}

	UE::Math::TMatrix<T> GetCardToLocal() const
	{
		return UE::Math::TMatrix<T>(
			UE::Math::TVector<T>(AxisX),
			UE::Math::TVector<T>(AxisY),
			UE::Math::TVector<T>(AxisZ),
			Origin);
	}

	inline UE::Math::TVector<float> RotateCardToLocal(UE::Math::TVector<float> Vector3) const
	{
		return Vector3.X * AxisX + Vector3.Y * AxisY + Vector3.Z * AxisZ;
	}

	inline UE::Math::TVector<float> RotateLocalToCard(UE::Math::TVector<float> Vector3) const
	{
		return UE::Math::TVector<float>(Vector3 | AxisX, Vector3 | AxisY, Vector3 | AxisZ);
	}

	inline UE::Math::TVector<float> TransformLocalToCard(UE::Math::TVector<T> LocalPosition) const
	{
		const UE::Math::TVector<float> Offset(LocalPosition - Origin);
		return UE::Math::TVector<float>(Offset | AxisX, Offset | AxisY, Offset | AxisZ);
	}

	inline UE::Math::TVector<T> TransformCardToLocal(UE::Math::TVector<float> CardPosition) const
	{
		return Origin + UE::Math::TVector<T>(CardPosition.X * AxisX + CardPosition.Y * AxisY + CardPosition.Z * AxisZ);
	}

	float ComputeSquaredDistanceToPoint(UE::Math::TVector<T> WorldPosition) const
	{
		const UE::Math::TVector<float> CardPositon = TransformLocalToCard(WorldPosition);
		return ::ComputeSquaredDistanceFromBoxToPoint(-Extent, Extent, CardPositon);
	}

	template<typename TT UE_REQUIRES(sizeof(TT) >= sizeof(T))>
	TLumenCardOBB<TT> Transform(UE::Math::TMatrix<TT> LocalToWorld, bool* bAxisXFlippedPtr = nullptr) const
	{
		TLumenCardOBB<TT> WorldOBB;
		WorldOBB.Origin = LocalToWorld.TransformPosition(UE::Math::TVector<TT>(Origin));

		const UE::Math::TVector<TT> ScaledXAxis(LocalToWorld.TransformVector(UE::Math::TVector<TT>(AxisX)));
		const UE::Math::TVector<TT> ScaledYAxis(LocalToWorld.TransformVector(UE::Math::TVector<TT>(AxisY)));
		const UE::Math::TVector<TT> ScaledZAxis(LocalToWorld.TransformVector(UE::Math::TVector<TT>(AxisZ)));
		const TT XAxisLength = ScaledXAxis.Size();
		const TT YAxisLength = ScaledYAxis.Size();
		const TT ZAxisLength = ScaledZAxis.Size();

		// #lumen_todo: fix axisX flip cascading into entire card code
		WorldOBB.AxisY = FVector3f(ScaledYAxis / FMath::Max(YAxisLength, TT(UE_DELTA)));
		WorldOBB.AxisZ = FVector3f(ScaledZAxis / FMath::Max(ZAxisLength, TT(UE_DELTA)));
		WorldOBB.AxisX = FVector3f::CrossProduct(WorldOBB.AxisZ, WorldOBB.AxisY);
		FVector3f::CreateOrthonormalBasis(WorldOBB.AxisX, WorldOBB.AxisY, WorldOBB.AxisZ);

		if (bAxisXFlippedPtr)
		{
			*bAxisXFlippedPtr = FVector3f::DotProduct(FVector3f(ScaledXAxis), WorldOBB.AxisX) < 0.0f;
		}

		WorldOBB.Extent = Extent * UE::Math::TVector<float>(XAxisLength, YAxisLength, ZAxisLength);
		WorldOBB.Extent.Z = FMath::Max(WorldOBB.Extent.Z, 1.0f);

		return WorldOBB;
	}

	UE::Math::TBox<T> GetBox() const
	{
		UE::Math::TVector<T> BoxMin(AxisX.GetAbs() * -Extent.X + AxisY.GetAbs() * -Extent.Y + AxisZ.GetAbs() * -Extent.Z);
		UE::Math::TVector<T> BoxMax(AxisX.GetAbs() * +Extent.X + AxisY.GetAbs() * +Extent.Y + AxisZ.GetAbs() * +Extent.Z);
		return UE::Math::TBox<T>(BoxMin + Origin, BoxMax + Origin);
	}

	bool ContainsNaN() const
	{
		return Origin.ContainsNaN() || AxisX.ContainsNaN() || AxisY.ContainsNaN() || AxisZ.ContainsNaN() || Extent.ContainsNaN();
	}

	friend FArchive& operator<<(FArchive& Ar, TLumenCardOBB<T>& Data)
	{
		Ar << Data.AxisX;
		Ar << Data.AxisY;
		Ar << Data.AxisZ;
		Ar << Data.Origin;
		Ar << Data.Extent;
		return Ar;
	}
};

using FLumenCardOBBf = TLumenCardOBB<float>;
using FLumenCardOBBd = TLumenCardOBB<double>;
