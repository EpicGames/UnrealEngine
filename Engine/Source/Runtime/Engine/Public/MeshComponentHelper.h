// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/Material.h"
#include "Materials/MaterialRelevance.h"
#include "Materials/MaterialInterface.h"
#include "MaterialDomain.h"
#include "RHIFeatureLevel.h"

/** Helper class used to share implementation for different MeshComponent types */
class FMeshComponentHelper
{
public:
	template<class T>
	static FMaterialRelevance GetMaterialRelevance(const T& Component, EShaderPlatform InShaderPlatform);

	template<class T>
	static void GetMaterialSlotsOverlayMaterial(const T& Component, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials);
};

template<class T>
FMaterialRelevance FMeshComponentHelper::GetMaterialRelevance(const T& Component, EShaderPlatform InShaderPlatform)
{
	// Combine the material relevance for all materials.
	FMaterialRelevance Result;
	for (int32 ElementIndex = 0; ElementIndex < Component.GetNumMaterials(); ElementIndex++)
	{
		UMaterialInterface const* MaterialInterface = Component.GetMaterial(ElementIndex);
		if (!MaterialInterface)
		{
			MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		Result |= MaterialInterface->GetRelevance_Concurrent(InShaderPlatform);
	}

	TArray<TObjectPtr<UMaterialInterface>> AssetAndComponentMaterialSlotsOverlayMaterial;
	Component.GetMaterialSlotsOverlayMaterial(AssetAndComponentMaterialSlotsOverlayMaterial);
	bool bAllMaterialSlotOverride = true;
	for (const TObjectPtr<UMaterialInterface>& MaterialInterface : AssetAndComponentMaterialSlotsOverlayMaterial)
	{
		if (MaterialInterface)
		{
			Result |= MaterialInterface->GetRelevance_Concurrent(InShaderPlatform);
		}
		else
		{
			bAllMaterialSlotOverride = false;
		}
	}

	//If all material slot set an overlay material we wont use the global one
	if (!bAllMaterialSlotOverride)
	{
		UMaterialInterface const* OverlayMaterialInterface = Component.GetOverlayMaterial();
		if (OverlayMaterialInterface != nullptr)
		{
			Result |= OverlayMaterialInterface->GetRelevance_Concurrent(InShaderPlatform);
		}
	}

	return Result;
}

template<class T>
void FMeshComponentHelper::GetMaterialSlotsOverlayMaterial(const T& Component, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials)
{
	//Add all the component override
	OutMaterialSlotOverlayMaterials = Component.GetComponentMaterialSlotsOverlayMaterial();

	//Get the asset material slot overlay material
	TArray<TObjectPtr<UMaterialInterface>> AssetMaterialSlotOverlayMaterials;
	Component.GetDefaultMaterialSlotsOverlayMaterial(AssetMaterialSlotOverlayMaterials);

	//For each slot not override by the component set the asset slot value
	for (int32 SlotIndex = 0; SlotIndex < AssetMaterialSlotOverlayMaterials.Num(); ++SlotIndex)
	{
		if (!OutMaterialSlotOverlayMaterials.IsValidIndex(SlotIndex))
		{
			OutMaterialSlotOverlayMaterials.AddZeroed();
		}
		//If the component did not already override this slot index put the asset value
		if (!OutMaterialSlotOverlayMaterials[SlotIndex])
		{
			OutMaterialSlotOverlayMaterials[SlotIndex] = AssetMaterialSlotOverlayMaterials[SlotIndex];
		}
	}
}
