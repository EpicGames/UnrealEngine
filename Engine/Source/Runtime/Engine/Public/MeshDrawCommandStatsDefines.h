// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Build.h"

#define MESH_DRAW_COMMAND_STATS (!UE_BUILD_SHIPPING)
