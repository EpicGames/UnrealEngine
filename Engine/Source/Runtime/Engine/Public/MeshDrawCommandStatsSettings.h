// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/NameTypes.h"

#include "MeshDrawCommandStatsSettings.generated.h"

/** Description of a stat category used in the MeshDrawCommandStats system. */
USTRUCT()
struct FMeshDrawCommandStatsBudget
{
	GENERATED_BODY()

	/** Category name. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	FName CategoryName;
	/** Stat names that will match this category name. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	TArray<FName> LinkedStatNames;
	/** The category primitive budget. This is the maximum triangles expected, post-culling, summed across all passes. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 PrimitiveBudget = 0;	
	/** The category vertex budget. This is the maximum number of vertex expected, post-culling, summed across all passes. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 VertexBudget = 0;
	/** The collection which contains this budget. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 Collection = 0;
	/** Which passes contribute to this budget. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	TArray<FName> Passes;
};

/** Budget totals for each MeshDrawCommandStats collection. */
USTRUCT()
struct FMeshDrawCommandStatsBudgetTotals
{
	GENERATED_BODY()

	/** The total amount of primitives budgeted for this collection. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 PrimitiveBudget = 0;	
	/** The total amount of vertices budgeted for this collection. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 VertexBudget = 0;
	/** The collection this applies to. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 Collection = 0;
};

/** User settings used by the MeshDrawCommandStats system. */
UCLASS(MinimalAPI, Config=Engine, defaultconfig, perplatformconfig, meta = (DisplayName = "Mesh Stats"))
class UMeshDrawCommandStatsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Budgets used by r.MeshDrawCommands.Stats */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	TArray<FMeshDrawCommandStatsBudget> Budgets;
	/** The total primitive budget for a collection. */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	TArray<FMeshDrawCommandStatsBudgetTotals> BudgetTotals;
	/** Which collection to export to CSV */
	UPROPERTY(config, EditAnywhere, Category = Engine)
	int32 CollectionForCsvProfiler = 1;
};
