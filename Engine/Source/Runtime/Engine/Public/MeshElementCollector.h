// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshBatch.h"
#include "RendererInterface.h"

class FGlobalDynamicIndexBuffer;
class FGlobalDynamicReadBuffer;
class FGlobalDynamicVertexBuffer;
class FGPUScenePrimitiveCollector;
class FMaterialRenderProxy;
class FPrimitiveDrawInterface;
class FPrimitiveSceneProxy;
class FSimpleElementCollector;
struct FMeshBatchAndRelevance;

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace RuntimeVirtualTexture { class FDynamicMeshCollector; }

/** 
 * Encapsulates the gathering of meshes from the various FPrimitiveSceneProxy classes. 
 */
class FMeshElementCollector
{
public:

	/** Accesses the PDI for drawing lines, sprites, etc. */
	ENGINE_API FPrimitiveDrawInterface* GetPDI(int32 ViewIndex);

#if UE_ENABLE_DEBUG_DRAWING
	ENGINE_API FPrimitiveDrawInterface* GetDebugPDI(int32 ViewIndex);
#endif

	/** 
	 * Allocates an FMeshBatch that can be safely referenced by the collector (lifetime will be long enough).
	 * Returns a reference that will not be invalidated due to further AllocateMesh() calls.
	 */
	inline FMeshBatch& AllocateMesh()
	{
		const int32 Index = MeshBatchStorage.Add(1);
		return MeshBatchStorage[Index];
	}

	/** Return dynamic index buffer for this collector. */
	FGlobalDynamicIndexBuffer& GetDynamicIndexBuffer()
	{
		check(DynamicIndexBuffer);
		return *DynamicIndexBuffer;
	}

	/** Return dynamic vertex buffer for this collector. */
	FGlobalDynamicVertexBuffer& GetDynamicVertexBuffer()
	{
		check(DynamicVertexBuffer);
		return *DynamicVertexBuffer;
	}

	/** Return dynamic read buffer for this collector. */
	FGlobalDynamicReadBuffer& GetDynamicReadBuffer()
	{
		check(DynamicReadBuffer);
		return *DynamicReadBuffer;
	}

	/** Return the current RHI command list used to initialize resources. */
	FRHICommandList& GetRHICommandList()
	{
		check(RHICmdList);
		return *RHICmdList;
	}

	// @return number of MeshBatches collected (so far) for a given view
	uint32 GetMeshBatchCount(uint32 ViewIndex) const
	{
		return MeshBatches[ViewIndex]->Num();
	}

	// @return Number of elemenets collected so far for a given view.
	uint32 GetMeshElementCount(uint32 ViewIndex) const
	{
		return NumMeshBatchElementsPerView[ViewIndex];
	}

	/** 
	 * Adds a mesh batch to the collector for the specified view so that it can be rendered.
	 */
	ENGINE_API void AddMesh(int32 ViewIndex, FMeshBatch& MeshBatch);

	/** Add a material render proxy that will be cleaned up automatically */
	ENGINE_API void RegisterOneFrameMaterialProxy(FMaterialRenderProxy* Proxy);

	/** Adds a request to force caching of uniform expressions for a material render proxy. */
	ENGINE_API void CacheUniformExpressions(FMaterialRenderProxy* Proxy, bool bRecreateUniformBuffer);

	/** Allocates a temporary resource that is safe to be referenced by an FMeshBatch added to the collector. */
	template<typename T, typename... ARGS>
	T& AllocateOneFrameResource(ARGS&&... Args)
	{
		return *OneFrameResources.Create<T>(Forward<ARGS>(Args)...);
	}
	
	UE_DEPRECATED(5.3, "ShouldUseTasks has been deprecated.")
	inline bool ShouldUseTasks() const
	{
		return false;
	}
	
	UE_DEPRECATED(5.3, "AddTask has been deprecated.")
	inline void AddTask(TFunction<void()>&& Task) {}

	UE_DEPRECATED(5.3, "AddTask has been deprecated.")
	inline void AddTask(const TFunction<void()>& Task) {}

	UE_DEPRECATED(5.3, "ProcessTasks has been deprecated.")
	void ProcessTasks() {}

	inline ERHIFeatureLevel::Type GetFeatureLevel() const
	{
		return FeatureLevel;
	}

protected:
	enum class ECommitFlags
	{
		None = 0,

		// Defers material uniform expression updates until Commit or Finish is called.
		DeferMaterials = 1 << 0,

		// Defers GPU scene updates until Commit or Finish is called.
		DeferGPUScene  = 1 << 1,

		DeferAll = DeferMaterials | DeferGPUScene
	};
	FRIEND_ENUM_CLASS_FLAGS(ECommitFlags);

	ENGINE_API FMeshElementCollector(ERHIFeatureLevel::Type InFeatureLevel, FSceneRenderingBulkObjectAllocator& InBulkAllocator, ECommitFlags CommitFlags = ECommitFlags::None);

	ENGINE_API ~FMeshElementCollector();

	ENGINE_API void SetPrimitive(const FPrimitiveSceneProxy* InPrimitiveSceneProxy, const FHitProxyId& DefaultHitProxyId);

	ENGINE_API void Start(
		FRHICommandList& RHICmdList,
		FGlobalDynamicVertexBuffer& DynamicVertexBuffer,
		FGlobalDynamicIndexBuffer& DynamicIndexBuffer,
		FGlobalDynamicReadBuffer& DynamicReadBuffer);

	ENGINE_API void AddViewMeshArrays(
		const FSceneView* InView,
		TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>* ViewMeshes,
		FSimpleElementCollector* ViewSimpleElementCollector,
		FGPUScenePrimitiveCollector* DynamicPrimitiveCollector
#if UE_ENABLE_DEBUG_DRAWING
		, FSimpleElementCollector* DebugSimpleElementCollector = nullptr
#endif
		);

	ENGINE_API void ClearViewMeshArrays();

	ENGINE_API void Commit();

	ENGINE_API void Finish();

	/** 
	 * Using TChunkedArray which will never realloc as new elements are added
	 * @todo - use mem stack
	 */
	TChunkedArray<FMeshBatch, 16384, FConcurrentLinearArrayAllocator> MeshBatchStorage;

	/** Meshes to render */
	TArray<TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>*, TInlineAllocator<2, SceneRenderingAllocator> > MeshBatches;

	/** Number of elements in gathered meshes per view. */
	TArray<int32, TInlineAllocator<2, SceneRenderingAllocator> > NumMeshBatchElementsPerView;

	/** PDIs */
	TArray<FSimpleElementCollector*, TInlineAllocator<2, SceneRenderingAllocator> > SimpleElementCollectors;

#if UE_ENABLE_DEBUG_DRAWING
	TArray<FSimpleElementCollector*, TInlineAllocator<2, SceneRenderingAllocator> > DebugSimpleElementCollectors;
#endif

	/** Views being collected for */
	TArray<const FSceneView*, TInlineAllocator<2, SceneRenderingAllocator>> Views;

	/** Current Mesh Id In Primitive per view */
	TArray<uint16, TInlineAllocator<2, SceneRenderingAllocator>> MeshIdInPrimitivePerView;

	/** Material proxies that will be deleted at the end of the frame. */
	TArray<FMaterialRenderProxy*, SceneRenderingAllocator> MaterialProxiesToDelete;

	/** Material proxies to force uniform expression evaluation. */
	TArray<TPair<FMaterialRenderProxy*, bool>, SceneRenderingAllocator> MaterialProxiesToInvalidate;

	/** Material proxies to force uniform expression evaluation. */
	TArray<const FMaterialRenderProxy*, SceneRenderingAllocator> MaterialProxiesToUpdate;

	/** List of mesh batches that require GPU scene updates. */
	TArray<TPair<FGPUScenePrimitiveCollector*, FMeshBatch*>, SceneRenderingAllocator> MeshBatchesForGPUScene;

	/** Resources that will be deleted at the end of the frame. */
	FSceneRenderingBulkObjectAllocator& OneFrameResources;

	/** Current primitive being gathered. */
	const FPrimitiveSceneProxy* PrimitiveSceneProxy;

	/** Dynamic buffer pools. */
	FGlobalDynamicIndexBuffer* DynamicIndexBuffer = nullptr;
	FGlobalDynamicVertexBuffer* DynamicVertexBuffer = nullptr;
	FGlobalDynamicReadBuffer* DynamicReadBuffer = nullptr;

	FRHICommandList* RHICmdList = nullptr;

	const ERHIFeatureLevel::Type FeatureLevel;
	const ECommitFlags CommitFlags;
	const bool bUseGPUScene;

	/** Tracks dynamic primitive data for upload to GPU Scene for every view, when enabled. */
	TArray<FGPUScenePrimitiveCollector*, TInlineAllocator<2, SceneRenderingAllocator>> DynamicPrimitiveCollectorPerView;

	friend class FVisibilityTaskData;
	friend class FSceneRenderer;
	friend class FDeferredShadingSceneRenderer;
	friend class FProjectedShadowInfo;
	friend class FCardPageRenderData;
	friend class FViewFamilyInfo;
	friend class FShadowMeshCollector;
	friend class FDynamicMeshElementContext;
	friend class RuntimeVirtualTexture::FDynamicMeshCollector;
	friend FSceneRenderingBulkObjectAllocator;
};

ENUM_CLASS_FLAGS(FMeshElementCollector::ECommitFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////
