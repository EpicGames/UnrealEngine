// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshMaterialShader.h: Mesh material shader definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Shader.h"
#include "MaterialShaderType.h"

class FMaterial;
class FMaterialShaderMapId;
class FShaderCommonCompileJob;
class FShaderCompileJob;
class FUniformExpressionSet;
class FVertexFactoryType;
class FMeshMaterialShader;
enum class EShaderCompileJobPriority : uint8;

/**
 * A shader meta type for material-linked shaders which use a vertex factory.
 */
class FMeshMaterialShaderType : public FShaderType
{
public:
	struct CompiledShaderInitializerType : FMaterialShaderType::CompiledShaderInitializerType
	{
		const FVertexFactoryType* VertexFactoryType;
		CompiledShaderInitializerType(
			const FShaderType* InType,
			int32 PermutationId,
			const FShaderCompilerOutput& CompilerOutput,
			const FUniformExpressionSet& InUniformExpressionSet,
			uint64 InMaterialShaderMapHash,
			const FString& InDebugDescription,
			const FShaderPipelineType* InShaderPipeline,
			const FVertexFactoryType* InVertexFactoryType
			):
			FMaterialShaderType::CompiledShaderInitializerType(InType,PermutationId,CompilerOutput,InUniformExpressionSet,InMaterialShaderMapHash,InShaderPipeline,InVertexFactoryType,InDebugDescription),
			VertexFactoryType(InVertexFactoryType)
		{}
	};

	IMPLEMENT_SHADER_TYPE_CONSTRUCTOR(FMeshMaterialShaderType, EShaderTypeForDynamicCast::MeshMaterial, TEXT("mesh material shader"));

#if WITH_EDITOR
	/**
	 * Enqueues a compilation for a new shader of this type.
	 * @param Platform - The platform to compile for.
	 * @param Material - The material to link the shader with.
	 * @param VertexFactoryType - The vertex factory to compile with.
	 */
	void BeginCompileShader(
		EShaderCompileJobPriority Priority,
		uint32 ShaderMapJobId,
		int32 PermutationId,
		EShaderPlatform Platform,
		EShaderPermutationFlags PermutationFlags,
		const FMaterial* Material,
		const FMaterialCompilationOutput& CompilationOutput,
		FSharedShaderCompilerEnvironment* MaterialEnvironment,
		const FVertexFactoryType* VertexFactoryType,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs,
		const FString& DebugGroupName,
		const TCHAR* DebugDescription,
		const TCHAR* DebugExtension
		) const;

	static void BeginCompileShaderPipeline(
		EShaderCompileJobPriority Priority,
		uint32 ShaderMapJobId,
		int32 PermutationId,
		EShaderPlatform Platform,
		EShaderPermutationFlags PermutationFlags,
		const FMaterial* Material,
		const FMaterialCompilationOutput& CompilationOutput,
		FSharedShaderCompilerEnvironment* MaterialEnvironment,
		const FVertexFactoryType* VertexFactoryType,
		const FShaderPipelineType* ShaderPipeline,
		TArray<TRefCountPtr<FShaderCommonCompileJob>>& NewJobs,
		const FString& DebugGroupName,
		const TCHAR* DebugDescription,
		const TCHAR* DebugExtension
		);

	/**
	 * Either creates a new instance of this type or returns an equivalent existing shader.
	 * @param Material - The material to link the shader with.
	 * @param CurrentJob - Compile job that was enqueued by BeginCompileShader.
	 */
	FShader* FinishCompileShader(
		const FUniformExpressionSet& UniformExpressionSet, 
		uint64 MaterialShaderMapHash,
		const FShaderCompileJob& CurrentJob,
		const FShaderPipelineType* ShaderPipeline,
		const FString& InDebugDescription
		) const;
#endif // WITH_EDITOR

	/**
	 * Checks if the shader type should be cached for a particular platform, material, and vertex factory type.
	 * @param Platform - The platform to check.
	 * @param Material - The material to check.
	 * @param VertexFactoryType - The vertex factory type to check.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCompilePermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, int32 PermutationId, EShaderPermutationFlags Flags) const;

	static bool ShouldCompileVertexFactoryPermutation(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, const FShaderType* ShaderType, EShaderPermutationFlags Flags);

	static bool ShouldCompileVertexFactoryPipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, EShaderPermutationFlags Flags);

	static bool ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, EShaderPermutationFlags Flags);

#if WITH_EDITOR
	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param Platform - Platform to compile for.
	 * @param Environment - The shader compile environment that the function modifies.
	 */
	void SetupCompileEnvironment(EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, const FVertexFactoryType* VertexFactoryType, int32 PermutationId, EShaderPermutationFlags PermutationFlags, FShaderCompilerEnvironment& Environment) const;
#endif // WITH_EDITOR
};

