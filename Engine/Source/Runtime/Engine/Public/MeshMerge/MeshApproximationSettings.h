// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/MaterialMerging.h"
#include "MeshApproximationSettings.generated.h"


UENUM()
enum class EMeshApproximationType : uint8
{
	MeshAndMaterials,
	MeshShapeOnly
};

UENUM()
enum class EMeshApproximationBaseCappingType : uint8
{
	NoBaseCapping = 0,
	ConvexPolygon = 1,
	ConvexSolid = 2
};


UENUM()
enum class EOccludedGeometryFilteringPolicy : uint8
{
	NoOcclusionFiltering = 0,
	VisibilityBasedFiltering = 1
};

UENUM()
enum class EMeshApproximationSimplificationPolicy : uint8
{
	FixedTriangleCount = 0,
	TrianglesPerArea = 1,
	GeometricTolerance = 2
};

UENUM()
enum class EMeshApproximationGroundPlaneClippingPolicy : uint8
{
	NoGroundClipping = 0,
	DiscardWithZPlane = 1,
	CutWithZPlane = 2,
	CutAndFillWithZPlane = 3
};


UENUM()
enum class EMeshApproximationUVGenerationPolicy : uint8
{
	PreferUVAtlas = 0,
	PreferXAtlas = 1,
	PreferPatchBuilder = 2
};


USTRUCT(Blueprintable)
struct FMeshApproximationSettings
{
	GENERATED_BODY()

	/** Type of output from mesh approximation process */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings)
	EMeshApproximationType OutputType = EMeshApproximationType::MeshAndMaterials;


	//
	// Mesh Generation Settings
	//

	/** Approximation Accuracy in Meters, will determine (eg) voxel resolution */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings, meta = (DisplayName = "Approximation Accuracy (meters)", ClampMin = "0.001"))
	float ApproximationAccuracy = 1.0f;

	/** Maximum allowable voxel count along main directions. This is a limit on ApproximationAccuracy. Max of 1290 (1290^3 is the last integer < 2^31, using a bigger number results in failures in TArray code & probably elsewhere) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = ShapeSettings, meta = (ClampMin = "64", ClampMax = "1290"))
	int32 ClampVoxelDimension = 1024;

	/** if enabled, we will attempt to auto-thicken thin parts or flat sheets */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings)
	bool bAttemptAutoThickening = true;

	/** Multiplier on Approximation Accuracy used for auto-thickening */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings, meta = (ClampMin = "0.001", EditCondition = "bAttemptAutoThickening"))
	float TargetMinThicknessMultiplier = 1.5f;

	/** If enabled, tiny parts will be excluded from the mesh merging, which can improve performance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings)
	bool bIgnoreTinyParts = true;

	/** Multiplier on Approximation Accuracy used to define tiny-part threshold, using maximum bounding-box dimension */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings, meta = (ClampMin = "0.001", EditCondition = "bIgnoreTinyParts"))
	float TinyPartSizeMultiplier = 0.05f;


	/** Optional methods to attempt to close off the bottom of open meshes */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings)
	EMeshApproximationBaseCappingType BaseCapping = EMeshApproximationBaseCappingType::NoBaseCapping;


	/** Winding Threshold controls hole filling at open mesh borders. Smaller value means "more/rounder" filling */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = ShapeSettings, meta = (ClampMin = "0.01", ClampMax = "0.99"))
	float WindingThreshold = 0.5f;

	/** If true, topological expand/contract is used to try to fill small gaps between objects. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings)
	bool bFillGaps = true;

	/** Distance in Meters to expand/contract to fill gaps */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ShapeSettings, meta = (DisplayName = "Gap Filling Distance (meters)", ClampMin = "0.001", EditCondition = "bFillGaps"))
	float GapDistance = 0.1f;


	//
	// Output Mesh Filtering and Simplification Settings
	//

	/** Type of hidden geometry removal to apply */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings)
	EOccludedGeometryFilteringPolicy OcclusionMethod = EOccludedGeometryFilteringPolicy::VisibilityBasedFiltering;

	/** If true, then the OcclusionMethod computation is configured to try to consider downward-facing "bottom" geometry as occluded */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings)
	bool bOccludeFromBottom = true;

	/** Mesh Simplification criteria */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings)
	EMeshApproximationSimplificationPolicy SimplifyMethod = EMeshApproximationSimplificationPolicy::GeometricTolerance;

	/** Target triangle count for Mesh Simplification, for SimplifyMethods that use a Count*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings, meta = (ClampMin = "16", EditCondition = "SimplifyMethod == EMeshApproximationSimplificationPolicy::FixedTriangleCount" ))
	int32 TargetTriCount = 2000;

	/** Approximate Number of triangles per Square Meter, for SimplifyMethods that use such a constraint */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings, meta = (ClampMin = "0.01", EditCondition = "SimplifyMethod == EMeshApproximationSimplificationPolicy::TrianglesPerArea" ))
	float TrianglesPerM = 2.0f;

	/** Allowable Geometric Deviation in Meters when SimplifyMethod incorporates a Geometric Tolerance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings, meta = (DisplayName = "Geometric Deviation (meters)", ClampMin = "0.0001", EditCondition = "SimplifyMethod == EMeshApproximationSimplificationPolicy::GeometricTolerance"))
	float GeometricDeviation = 0.1f;

	/** Configure how the final mesh should be clipped with a ground plane, if desired */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings)
	EMeshApproximationGroundPlaneClippingPolicy GroundClipping = EMeshApproximationGroundPlaneClippingPolicy::NoGroundClipping;

	/** Z-Height for the ground clipping plane, if enabled */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SimplifySettings, meta = (EditCondition = "GroundClipping != EMeshApproximationGroundPlaneClippingPolicy::NoGroundClipping"))
	float GroundClippingZHeight = 0.0f;


	//
	// Mesh Normals and Tangents Settings
	//

	/** If true, normal angle will be used to estimate hard normals */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NormalsSettings)
	bool bEstimateHardNormals = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NormalsSettings, meta = (ClampMin = "0.0", ClampMax = "90.0", EditCondition = "bEstimateHardNormals"))
	float HardNormalAngle = 60.0f;


	//
	// Mesh UV Generation Settings
	//

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UVSettings)
	EMeshApproximationUVGenerationPolicy UVGenerationMethod = EMeshApproximationUVGenerationPolicy::PreferXAtlas;


	/** Number of initial patches mesh will be split into before computing island merging */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UVSettings, AdvancedDisplay, meta = (UIMin = "1", UIMax = "1000", ClampMin = "1", ClampMax = "99999999", EditCondition = "UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferPatchBuilder"))
	int InitialPatchCount = 250;

	/** This parameter controls alignment of the initial patches to creases in the mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UVSettings, AdvancedDisplay, meta = (UIMin = "0.1", UIMax = "2.0", ClampMin = "0.01", ClampMax = "100.0", EditCondition = "UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferPatchBuilder"))
	float CurvatureAlignment = 1.0f;

	/** Distortion/Stretching Threshold for island merging - larger values increase the allowable UV stretching */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UVSettings, AdvancedDisplay, meta = (UIMin = "1.0", UIMax = "5.0", ClampMin = "1.0", EditCondition = "UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferPatchBuilder"))
	float MergingThreshold = 1.5f;

	/** UV islands will not be merged if their average face normals deviate by larger than this amount */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UVSettings, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "90.0", ClampMin = "0.0", ClampMax = "180.0", EditCondition = "UVGenerationMethod == EMeshApproximationUVGenerationPolicy::PreferPatchBuilder"))
	float MaxAngleDeviation = 45.0f;

	//
	// Output Static Mesh Settings
	//

	/** Whether to generate a nanite-enabled mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSettings)
	bool bGenerateNaniteEnabledMesh = false;

	/** Which heuristic to use when generating the Nanite fallback mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = MeshSettings, meta = (EditConditionHides, EditCondition = "bGenerateNaniteEnabledMesh"))
	ENaniteFallbackTarget NaniteFallbackTarget = ENaniteFallbackTarget::Auto;

	/** Percentage of triangles to keep from source Nanite mesh for fallback. 1.0 = no reduction, 0.0 = no triangles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = MeshSettings, meta = (EditConditionHides, EditCondition = "bGenerateNaniteEnabledMesh && NaniteFallbackTarget == ENaniteFallbackTarget::PercentTriangles", ClampMin = 0, ClampMax = 1))
	float NaniteFallbackPercentTriangles = 1.0f;

	/** Reduce Nanite fallback mesh until at least this amount of error is reached relative to size of the mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = MeshSettings, meta = (EditConditionHides, EditCondition = "bGenerateNaniteEnabledMesh && NaniteFallbackTarget == ENaniteFallbackTarget::RelativeError", ClampMin = 0))
	float NaniteFallbackRelativeError = 1.0f;

	/** Whether ray tracing will be supported on this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSettings)
	bool bSupportRayTracing = true;

	/** Whether to allow distance field to be computed for this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MeshSettings)
	bool bAllowDistanceField = true;


	//
	// Material Baking Settings
	//

	/** If Value is > 1, Multisample output baked textures by this amount in each direction (eg 4 == 16x supersampling) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MaterialSettings, meta = (ClampMin = "0", ClampMax = "8", UIMin = "0", UIMax = "4"))
	int32 MultiSamplingAA = 0;

	/** If Value is zero, use MaterialSettings resolution, otherwise override the render capture resolution */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MaterialSettings, meta = (ClampMin = "0"))
	int32 RenderCaptureResolution = 2048;

	/** Material generation settings */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MaterialSettings)
	FMaterialProxySettings MaterialSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = MaterialSettings, meta = (ClampMin = "5.0", ClampMax = "160.0"))
	float CaptureFieldOfView = 30.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = MaterialSettings, meta = (ClampMin = "0.001", ClampMax = "1000.0"))
	float NearPlaneDist = 1.0f;


	//
	// Performance Settings
	//


	/** If true, LOD0 Render Meshes (or Nanite Fallback meshes) are used instead of Source Mesh data. This can significantly reduce computation time and memory usage, but potentially at the cost of lower quality output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PerformanceSettings)
	bool bUseRenderLODMeshes = false;

	/** If true, a faster mesh simplfication strategy will be used. This can significantly reduce computation time and memory usage, but potentially at the cost of lower quality output. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PerformanceSettings)
	bool bEnableSimplifyPrePass = true;

	/** If false, texture capture and baking will be done serially after mesh generation, rather than in parallel when possible. This will reduce the maximum memory requirements of the process.  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PerformanceSettings)
	bool bEnableParallelBaking = true;

	//
	// Debug Output Settings
	//


	/** If true, print out debugging messages */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = DebugSettings)
	bool bPrintDebugMessages = false;

	/** If true, write the full mesh triangle set (ie flattened, non-instanced) used for mesh generation. Warning: this asset may be extremely large!! */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = DebugSettings)
	bool bEmitFullDebugMesh = false;
	

	/** Equality operator. */
	ENGINE_API bool operator==(const FMeshApproximationSettings& Other) const;

	/** Inequality. */
	ENGINE_API bool operator!=(const FMeshApproximationSettings& Other) const;

#if WITH_EDITORONLY_DATA
	/** Handles deprecated properties */
	void PostSerialize(const FArchive& Ar);
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float NaniteProxyTrianglePercent_DEPRECATED = 0;
#endif
};

template<>
struct TStructOpsTypeTraits<FMeshApproximationSettings> : public TStructOpsTypeTraitsBase2<FMeshApproximationSettings>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithPostSerialize = true,
	};
#endif
};
