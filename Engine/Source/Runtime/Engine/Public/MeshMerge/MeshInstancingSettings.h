// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "MeshInstancingSettings.generated.h"


class AActor;
class UInstancedStaticMeshComponent;


/** Mesh instance-replacement settings */
USTRUCT(Blueprintable)
struct FMeshInstancingSettings
{
	GENERATED_BODY()

	ENGINE_API FMeshInstancingSettings();

	/** The actor class to attach new instance static mesh components to */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, NoClear, Category="Instancing")
	TSubclassOf<AActor> ActorClassToUse;

	/** The number of static mesh instances needed before a mesh is replaced with an instanced version */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Instancing", meta=(ClampMin=1))
	int32 InstanceReplacementThreshold;

	/**
	 * Whether to skip the conversion to an instanced static mesh for meshes with vertex colors.
	 * Instanced static meshes do not support vertex colors per-instance, so conversion will lose
	 * this data.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Instancing")
	bool bSkipMeshesWithVertexColors;

	/**
	 * Whether split up instanced static mesh components based on their intersection with HLOD volumes
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Instancing", meta=(DisplayName="Use HLOD Volumes"))
	bool bUseHLODVolumes;

	/**
	 * Whether to use the Instanced Static Mesh Compoment or the Hierarchical Instanced Static Mesh Compoment
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Instancing", meta = (DisplayName = "Select the type of Instanced Component", DisallowedClasses = "/Script/Foliage.FoliageInstancedStaticMeshComponent"))
	TSubclassOf<UInstancedStaticMeshComponent> ISMComponentToUse;
};
