// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/MaterialMerging.h"
#include "MeshMergingSettings.generated.h"


UENUM()
enum class EMeshLODSelectionType : uint8
{
	// Whether or not to export all of the LODs found in the source meshes
	AllLODs = 0 UMETA(DisplayName = "Use all LOD levels", ScriptName="AllLods;AllLODs"),
	// Whether or not to export all of the LODs found in the source meshes
	SpecificLOD = 1 UMETA(DisplayName = "Use specific LOD level"),
	// Whether or not to calculate the appropriate LOD model for the given screen size
	CalculateLOD = 2 UMETA(DisplayName = "Calculate correct LOD level"),
	// Whether or not to use the lowest-detail LOD
	LowestDetailLOD = 3 UMETA(DisplayName = "Always use the lowest-detail LOD (i.e. the highest LOD index)")
};

UENUM()
enum class EMeshMergeType : uint8
{
	MeshMergeType_Default,
	MeshMergeType_MergeActor
};

/** As UHT doesnt allow arrays of bools, we need this binary enum :( */
UENUM()
enum class EUVOutput : uint8
{
	DoNotOutputChannel,
	OutputChannel
};

/**
 * Specifies how the pivot point of the merged mesh is determined.
 */
UENUM(BlueprintType)
enum class EMeshMergePivotType : uint8
{
	/** Place pivot at the location of the first merged component. This is the default behavior. */
    Automatic,

    /** Place pivot at world origin (0, 0, 0). Useful for world-aligned merged meshes. */
    WorldOrigin,

    /** Place pivot at a user-specified location defined by CustomPivotLocation. */
    Custom
};

/**
 * Mesh merging settings
 */
USTRUCT(Blueprintable)
struct FMeshMergingSettings
{
	GENERATED_USTRUCT_BODY()

	/** The lightmap resolution used both for generating lightmap UV coordinates, and also set on the generated static mesh */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = MeshSettings, meta=(ClampMax = 4096, EditCondition = "!bComputedLightMapResolution", DisplayAfter="bGenerateLightMapUV", DisplayName="Target Lightmap Resolution"))
	int32 TargetLightMapResolution;

	/** Whether to output the specified UV channels into the merged mesh (only if the source meshes contain valid UVs for the specified channel) */
	UPROPERTY(EditAnywhere, Category = MeshSettings, meta=(DisplayAfter="bBakeVertexDataToMesh"))
	EUVOutput OutputUVs[8];	// Should be MAX_MESH_TEXTURE_COORDS but as this is an engine module we cant include RawMesh

	/** Material simplification */
	UPROPERTY(EditAnywhere, Category = MaterialSettings, BlueprintReadWrite, meta = (EditCondition = "bMergeMaterials", DisplayAfter="bMergeMaterials"))
	FMaterialProxySettings MaterialSettings;

	/** The gutter (in texels) to add to each sub-chart for our baked-out material for the top mip level */
	UPROPERTY(EditAnywhere, Category = MaterialSettings, meta=(DisplayAfter="MaterialSettings"))
	int32 GutterSize;

	/** Which selection mode should be used when generating the merged static mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings, meta = (DisplayAfter="bBakeVertexDataToMesh", DisplayName = "LOD Selection Type"))
	EMeshLODSelectionType LODSelectionType;

	/** A given LOD level to export from the source meshes, used if LOD Selection Type is set to SpecificLOD */
	UPROPERTY(EditAnywhere, Category = MeshSettings, BlueprintReadWrite, meta = (DisplayAfter="LODSelectionType", EditCondition = "LODSelectionType == EMeshLODSelectionType::SpecificLOD", ClampMin = "0", ClampMax = "7", UIMin = "0", UIMax = "7", EnumCondition = 1))
	int32 SpecificLOD;

	/** Whether to generate lightmap UVs for a merged mesh*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = MeshSettings, meta=(DisplayName="Generate Lightmap UV"))
	uint8 bGenerateLightMapUV:1;

	/** Whether or not the lightmap resolution should be computed by summing the lightmap resolutions for the input Mesh Components */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = MeshSettings, meta=(DisplayName="Computed Lightmap Resolution"))
	uint8 bComputedLightMapResolution:1;

	/** Whether to merge physics data (collision primitives)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	uint8 bMergePhysicsData:1;

	/** Whether to merge sockets */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	uint8 bMergeMeshSockets : 1;

	/** Whether to merge source materials into one flat material, ONLY available when LOD Selection Type is set to LowestDetailLOD */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialSettings, meta=(EditCondition="LODSelectionType == EMeshLODSelectionType::LowestDetailLOD || LODSelectionType == EMeshLODSelectionType::SpecificLOD"))
	uint8 bMergeMaterials:1;

	/** Whether or not vertex data such as vertex colours should be baked into the resulting mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	uint8 bBakeVertexDataToMesh:1;

	/** Whether or not vertex data such as vertex colours should be used when baking out materials */
	UPROPERTY(EditAnywhere, Category = MaterialSettings, BlueprintReadWrite, meta = (EditCondition = "bMergeMaterials"))
	uint8 bUseVertexDataForBakingMaterial:1;

	/** Whether or not to calculate varying output texture sizes according to their importance in the final atlas texture */
	UPROPERTY(Category = MaterialSettings, EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "bMergeMaterials"))
	uint8 bUseTextureBinning:1;

	/** Whether to attempt to re-use the source mesh's lightmap UVs when baking the material or always generate a new set. */
	UPROPERTY(EditAnywhere, Category = MaterialSettings)
	uint8 bReuseMeshLightmapUVs:1;

	/** Whether to attempt to merge materials that are deemed equivalent. This can cause artifacts in the merged mesh if world position/actor position etc. is used to determine output color. */
	UPROPERTY(EditAnywhere, Category = MaterialSettings)
	uint8 bMergeEquivalentMaterials:1;

	/** Whether or not to use available landscape geometry to cull away invisible triangles */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LandscapeCulling)
	uint8 bUseLandscapeCulling:1;

	/** Whether or not to include any imposter LODs that are part of the source static meshes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	uint8 bIncludeImposters:1;

	/** Whether ray tracing will be supported on this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
	uint8 bSupportRayTracing : 1;

	/** Whether to allow distance field to be computed for this mesh. Disable this to save memory if the merged mesh will only be rendered in the distance. */
	UPROPERTY(EditAnywhere, Category = MeshSettings)
	uint8 bAllowDistanceField:1;

	/** Settings related to building Nanite data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NaniteSettings)
	FMeshNaniteSettings NaniteSettings;

	/**
     * Determines how the pivot point of the merged mesh is calculated.
     * - WorldOrigin: Pivot at (0,0,0)
     * - Automatic: Pivot at the first merged component's location (default)
     * - Custom: Pivot at CustomPivotLocation
     */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings)
    EMeshMergePivotType PivotType;

    /**
     * The world-space location to use as the merged mesh's pivot point.
     * Only used when PivotType is set to Custom.
     */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshSettings, meta = (EditCondition = "PivotType == EMeshMergePivotType::Custom"))
    FVector CustomPivotLocation;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 bImportVertexColors_DEPRECATED:1;

	UPROPERTY()
	uint8 bCalculateCorrectLODModel_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportNormalMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportMetallicMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportRoughnessMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportSpecularMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bCreateMergedMaterial_DEPRECATED : 1;

	UPROPERTY()
	int32 MergedMaterialAtlasResolution_DEPRECATED;

	UPROPERTY()
	int32 ExportSpecificLOD_DEPRECATED;

	UPROPERTY()
	uint8 bGenerateNaniteEnabledMesh_DEPRECATED : 1;

	UPROPERTY()
	float NaniteFallbackTrianglePercent_DEPRECATED;

	UPROPERTY()
	uint8 bPivotPointAtZero_DEPRECATED : 1;
#endif

	EMeshMergeType MergeType;

	/** Default settings. */
	ENGINE_API FMeshMergingSettings();

#if WITH_EDITORONLY_DATA
	/** Handles deprecated properties */
	void PostSerialize(const FArchive& Ar);
#endif
};

template<>
struct TStructOpsTypeTraits<FMeshMergingSettings> : public TStructOpsTypeTraitsBase2<FMeshMergingSettings>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithPostSerialize = true,
	};
#endif
};
