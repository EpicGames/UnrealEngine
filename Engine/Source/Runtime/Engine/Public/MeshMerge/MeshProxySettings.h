// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/MaterialMerging.h"
#include "MeshProxySettings.generated.h"


UENUM()
namespace ELandscapeCullingPrecision
{
	enum Type : int
	{
		High = 0 UMETA(DisplayName = "High memory intensity and computation time"),
		Medium = 1 UMETA(DisplayName = "Medium memory intensity and computation time"),
		Low = 2 UMETA(DisplayName = "Low memory intensity and computation time")
	};
}

UENUM()
namespace EProxyNormalComputationMethod
{
	enum Type : int
	{
		AngleWeighted = 0 UMETA(DisplayName = "Angle Weighted"),
		AreaWeighted = 1 UMETA(DisplayName = "Area  Weighted"),
		EqualWeighted = 2 UMETA(DisplayName = "Equal Weighted")
	};
}


USTRUCT(Blueprintable)
struct FMeshProxySettings
{
	GENERATED_USTRUCT_BODY()
	/** Screen size of the resulting proxy mesh in pixels*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (ClampMin = "1", ClampMax = "1200", UIMin = "1", UIMax = "1200"))
	int32 ScreenSize;

	/** Override when converting multiple meshes for proxy LOD merging. Warning, large geometry with small sampling has very high memory costs*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = ProxySettings, meta = (EditCondition = "bOverrideVoxelSize", ClampMin = "0.1", DisplayName = "Override Spatial Sampling Distance"))
	float VoxelSize;

	/** Material simplification */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	FMaterialProxySettings MaterialSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 TextureWidth_DEPRECATED;
	UPROPERTY()
	int32 TextureHeight_DEPRECATED;

	UPROPERTY()
	uint8 bExportNormalMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportMetallicMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportRoughnessMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bExportSpecularMap_DEPRECATED:1;

	UPROPERTY()
	uint8 bBakeVertexData_DEPRECATED:1;

	UPROPERTY()
	uint8 bGenerateNaniteEnabledMesh_DEPRECATED : 1;
	
	UPROPERTY()
	float NaniteProxyTrianglePercent_DEPRECATED;
#endif

	/** Distance at which meshes should be merged together, this can close gaps like doors and windows in distant geometry */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	float MergeDistance;

	/** Base color assigned to LOD geometry that can't be associated with the source geometry: e.g. doors and windows that have been closed by the Merge Distance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (DisplayName = "Unresolved Geometry Color"))
	FColor UnresolvedGeometryColor;

	/** Override search distance used when discovering texture values for simplified geometry. Useful when non-zero Merge Distance setting generates new geometry in concave corners.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (EditCondition = "bOverrideTransferDistance", DisplayName = "Transfer Distance Override", ClampMin = 0))
	float MaxRayCastDist;

	/** Angle at which a hard edge is introduced between faces */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (EditCondition = "bUseHardAngleThreshold", DisplayName = "Hard Edge Angle", ClampMin = 0, ClampMax = 180))
	float HardAngleThreshold;

	/** Lightmap resolution */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (ClampMin = 32, ClampMax = 4096, EditCondition = "!bComputeLightMapResolution", DisplayAfter="NormalCalculationMethod", DisplayName="Lightmap Resolution"))
	int32 LightMapResolution;

	/** Controls the method used to calculate the normal for the simplified geometry */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (DisplayName = "Normal Calculation Method"))
	TEnumAsByte<EProxyNormalComputationMethod::Type> NormalCalculationMethod;

	/** Level of detail of the landscape that should be used for the culling */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = LandscapeCulling, meta = (EditCondition="bUseLandscapeCulling", DisplayAfter="bUseLandscapeCulling"))
	TEnumAsByte<ELandscapeCullingPrecision::Type> LandscapeCullingPrecision;

	/** Determines whether or not the correct LOD models should be calculated given the source meshes and transition size */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta=(DisplayAfter="ScreenSize"))
	uint8 bCalculateCorrectLODModel:1;

	/** If true, Spatial Sampling Distance will not be automatically computed based on geometry and you must set it directly */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category = ProxySettings, meta = (InlineEditConditionToggle))
	uint8 bOverrideVoxelSize : 1;

	/** Enable an override for material transfer distance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = MaxRayCastDist, meta = (InlineEditConditionToggle))
	uint8 bOverrideTransferDistance:1;

	/** Enable the use of hard angle based vertex splitting */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = HardAngleThreshold, meta = (InlineEditConditionToggle))
	uint8 bUseHardAngleThreshold:1;

	/** If ticked will compute the lightmap resolution by summing the dimensions for each mesh included for merging */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings, meta = (DisplayName="Compute Lightmap Resolution"))
	uint8 bComputeLightMapResolution:1;

	/** Whether Simplygon should recalculate normals, otherwise the normals channel will be sampled from the original mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bRecalculateNormals:1;

	/** Whether or not to use available landscape geometry to cull away invisible triangles */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = LandscapeCulling)
	uint8 bUseLandscapeCulling:1;

	/** Whether ray tracing will be supported on this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bSupportRayTracing : 1;

	/** Whether to allow distance field to be computed for this mesh. Disable this to save memory if the merged mesh will only be rendered in the distance. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bAllowDistanceField:1;

	/** Whether to attempt to re-use the source mesh's lightmap UVs when baking the material or always generate a new set. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bReuseMeshLightmapUVs:1;

	/** Bake identical meshes (or mesh instances) only once. Can lead to discrepancies with the source mesh visual, especially for materials that are using world position or per instance data. However, this will result in better quality baked textures & greatly reduce baking time. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bGroupIdenticalMeshesForBaking:1;

	/** Whether to generate collision for the merged mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bCreateCollision:1;

	/** Whether to allow vertex colors saved in the merged mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bAllowVertexColors:1;

	/** Whether to generate lightmap uvs for the merged mesh */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ProxySettings)
	uint8 bGenerateLightmapUVs:1;

	/** Settings related to building Nanite data. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NaniteSettings)
	FMeshNaniteSettings NaniteSettings;

	/** Default settings. */
	ENGINE_API FMeshProxySettings();
	
	/** Equality operator. */
	ENGINE_API bool operator==(const FMeshProxySettings& Other) const;

	/** Inequality. */
	ENGINE_API bool operator!=(const FMeshProxySettings& Other) const;

#if WITH_EDITORONLY_DATA
	/** Handles deprecated properties */
	void PostSerialize(const FArchive& Ar);
#endif
};

template<>
struct TStructOpsTypeTraits<FMeshProxySettings> : public TStructOpsTypeTraitsBase2<FMeshProxySettings>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithPostSerialize = true,
	};
#endif
};
