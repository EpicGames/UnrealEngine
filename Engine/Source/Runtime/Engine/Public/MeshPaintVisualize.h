// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FMaterialRenderProxy;
class FRHITexture;
class UTexture;

/** Mesh painting visualization channels. (Used for all mesh painting modes not just vertex color.) */
namespace EVertexColorViewMode
{
	enum Type
	{
		/** Invalid or undefined */
		Invalid,
		/** Color only */
		Color,
		/** Alpha only */
		Alpha,
		/** Red only */
		Red,
		/** Green only */
		Green,
		/** Blue only */
		Blue,
	};
}

/** Visualization modes for different mesh painting tools. */
namespace EMeshPaintVisualizePaintMode
{
	enum Type
	{
		VertexColor,
		TextureColor,
		TextureAsset,
	};
}

/** Visualization modes for mesh painting tools to define where the visualisation is applied. */
namespace EMeshPaintVisualizeShowMode
{
	enum Type
	{
		/* Apply visualization to all items. */
		ShowAll,
		/* Only apply visualization to selected items. */
		ShowSelected,
	};
}

/** Interface to set and get the mesh paint visualization settings that are used whenever the SHOW_VertexColors show flag is set. */
namespace MeshPaintVisualize
{
	ENGINE_API void SetPaintMode(EMeshPaintVisualizePaintMode::Type PaintMode);
	
	ENGINE_API void SetShowMode(EMeshPaintVisualizeShowMode::Type ShowMode);
	ENGINE_API EMeshPaintVisualizeShowMode::Type GetShowMode();
	
	ENGINE_API void SetChannelMode(EVertexColorViewMode::Type ChannelMode);
	ENGINE_API EVertexColorViewMode::Type GetChannelMode();
	
	ENGINE_API void SetTextureAsset(TWeakObjectPtr<UTexture> Texture);
	ENGINE_API FRHITexture* GetTextureAsset_RenderThread();
	
	ENGINE_API void SetTextureCoordinateIndex(int32 Index);
	ENGINE_API int32 GetTextureCoordinateIndex();
	
	/** Get the mesh paint visualization material proxy based on the current global settings. */
	ENGINE_API FMaterialRenderProxy* GetMaterialRenderProxy(bool bIsSelected, bool bIsHovered);
}

/** Deprecated vertex color global state interface. */
UE_DEPRECATED(5.5, "Use MeshPaintVisualize::SetChannelMode() instead.")
extern ENGINE_API EVertexColorViewMode::Type GVertexColorViewMode;
UE_DEPRECATED(5.5, "Use MeshPaintVisualize::SetTextureAsset() instead.")
extern ENGINE_API TWeakObjectPtr<UTexture> GVertexViewModeOverrideTexture;
UE_DEPRECATED(5.5, "Use MeshPaintVisualize::SetTextureCoordinateIndex() instead.")
extern ENGINE_API float GVertexViewModeOverrideUVChannel;
UE_DEPRECATED(5.5, "We no longer use names to enable visualization.")
extern ENGINE_API FString GVertexViewModeOverrideOwnerName;
UE_DEPRECATED(5.5, "We no longer use names to enable visualization.")
extern bool ShouldProxyUseVertexColorVisualization(FName OwnerName);
