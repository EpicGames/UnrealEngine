// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshParticleVertexFactory.h: Mesh particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "Components.h"
#include "SceneManagement.h"
#include "VertexFactory.h"
#include "ParticleVertexFactory.h"
//@todo - parallelrendering - remove once FOneFrameResource no longer needs to be referenced in header

class FMaterial;
class FVertexBuffer;
struct FDynamicReadBuffer;
struct FShaderCompilerEnvironment;

/**
 * Uniform buffer for mesh particle vertex factories.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FMeshParticleUniformParameters, ENGINE_API)
	SHADER_PARAMETER( FVector4f, SubImageSize )
	SHADER_PARAMETER( uint32, TexCoordWeightA )
	SHADER_PARAMETER( uint32, TexCoordWeightB )
	SHADER_PARAMETER( uint32, PrevTransformAvailable )
	SHADER_PARAMETER( float, DeltaSeconds )
	SHADER_PARAMETER( uint32, bUseLocalSpace )
	SHADER_PARAMETER( FVector3f, LWCTile )
	SHADER_PARAMETER( float, UseVelocityForMotionBlur )
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FMeshParticleUniformParameters> FMeshParticleUniformBufferRef;

class FMeshParticleInstanceVertices;


/**
 * Vertex factory for rendering instanced mesh particles with out dynamic parameter support.
 */
class FMeshParticleVertexFactory : public FParticleVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE(FMeshParticleVertexFactory);
public:
	FMeshParticleVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FParticleVertexFactoryBase(InFeatureLevel)
	{
	}

	struct FDataType : public FStaticMeshDataType
	{
		/** The stream to read the vertex  color from. */
		FVertexStreamComponent ParticleColorComponent;

		/** The stream to read the mesh transform from. */
		FVertexStreamComponent TransformComponent[3];

		/** The stream to read the particle velocity from */
		FVertexStreamComponent VelocityComponent;

		/** The stream to read SubUV parameters from.. */
		FVertexStreamComponent SubUVs;

		/** The stream to read SubUV lerp and the particle relative time from */
		FVertexStreamComponent SubUVLerpAndRelTime;

		/** Flag to mark as initialized */
		bool bInitialized;

		FDataType()
		: bInitialized(false)
		{
		}
	};

	class FBatchParametersCPU : public FOneFrameResource
	{
	public:
		const struct FMeshParticleInstanceVertex* InstanceBuffer;
		const struct FMeshParticleInstanceVertexDynamicParameter* DynamicParameterBuffer;
		const struct FMeshParticleInstanceVertexPrevTransform* PrevTransformBuffer;
	};

	/** Default constructor. */
	FMeshParticleVertexFactory(EParticleVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride)
		: FParticleVertexFactoryBase(InType, InFeatureLevel)
		, LODIdx(0xff)
		, DynamicVertexStride(InDynamicVertexStride)
		, DynamicParameterVertexStride(InDynamicParameterVertexStride)
		, InstanceVerticesCPU(nullptr)
	{}

	FMeshParticleVertexFactory()
		: FParticleVertexFactoryBase(PVFT_MAX, ERHIFeatureLevel::Num)
		, LODIdx(0xff)
		, DynamicVertexStride(-1)
		, DynamicParameterVertexStride(-1)
		, InstanceVerticesCPU(nullptr)
	{}

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);


	/**
	 * Modify compile environment to enable instancing
	 * @param OutEnvironment - shader compile environment to modify
	 */
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Get vertex elements used when during PSO precaching materials using this vertex factory type
	 */
	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride, FDataType& Data, FVertexDeclarationElementList& Elements);

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	ENGINE_API void SetData(FRHICommandListBase& RHICmdList, const FDataType& InData);

	UE_DEPRECATED(5.4, "SetData requires a command list.")
	ENGINE_API void SetData(const FDataType& InData);

	/**
	 * Set the uniform buffer for this vertex factory.
	 */
	inline void SetUniformBuffer( const FMeshParticleUniformBufferRef& InMeshParticleUniformBuffer )
	{
		MeshParticleUniformBuffer = InMeshParticleUniformBuffer;
	}

	/**
	 * Retrieve the uniform buffer for this vertex factory.
	 */
	inline FRHIUniformBuffer* GetUniformBuffer()
	{
		return MeshParticleUniformBuffer;
	}
	
	/**
	 * Update the data strides (MUST HAPPEN BEFORE InitRHI is called)
	 */
	void SetStrides(int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride)
	{
		ensure(!IsInitialized());
		DynamicVertexStride = InDynamicVertexStride;
		DynamicParameterVertexStride = InDynamicParameterVertexStride;
	}
	
	 /**
	 * Set the source vertex buffer that contains particle instance data.
	 */
	ENGINE_API void SetInstanceBuffer(const FVertexBuffer* InstanceBuffer, uint32 StreamOffset, uint32 Stride);

	/**
	 * Set the source vertex buffer that contains particle dynamic parameter data.
	 */
	ENGINE_API void SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride);

	ENGINE_API uint8* LockPreviousTransformBuffer(FRHICommandListBase& RHICmdList, uint32 ParticleCount);
	ENGINE_API void UnlockPreviousTransformBuffer(FRHICommandListBase& RHICmdList);
	ENGINE_API FRHIShaderResourceView* GetPreviousTransformBufferSRV() const;

	/**
	* Copy the data from another vertex factory
	* @param Other - factory to copy from
	*/
	ENGINE_API void Copy(const FMeshParticleVertexFactory& Other);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	FMeshParticleInstanceVertices*& GetInstanceVerticesCPU()
	{
		return InstanceVerticesCPU;
	}

	void SetLODIdx(uint8 InLODIdx)
	{
		LODIdx = InLODIdx;
	}

	uint8 GetLODIdx() const
	{
		return LODIdx;
	}
protected:
	static ENGINE_API void GetVertexElements(ERHIFeatureLevel::Type FeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride, FDataType& Data, FVertexDeclarationElementList& Elements, FVertexStreamList& InOutStreams);

protected:
	FDataType Data;
	
	uint8 LODIdx;

	/** Stride information for instanced mesh particles */
	int32 DynamicVertexStride;
	int32 DynamicParameterVertexStride;
	
	/** Uniform buffer with mesh particle parameters. */
	FRHIUniformBuffer* MeshParticleUniformBuffer;

	FDynamicReadBuffer PrevTransformBuffer;

	/** Used to remember this in the case that we reuse the same vertex factory for multiple renders . */
	FMeshParticleInstanceVertices* InstanceVerticesCPU;
};


inline FMeshParticleVertexFactory* ConstructMeshParticleVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
{
	return new FMeshParticleVertexFactory(InFeatureLevel);
}

inline FMeshParticleVertexFactory* ConstructMeshParticleVertexFactory(EParticleVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel, int32 InDynamicVertexStride, int32 InDynamicParameterVertexStride)
{
	return new FMeshParticleVertexFactory(InType, InFeatureLevel, InDynamicVertexStride, InDynamicParameterVertexStride);
}
