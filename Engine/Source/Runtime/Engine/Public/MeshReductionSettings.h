// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MeshReductionSettings.generated.h"


/** The importance of a mesh feature when automatically generating mesh LODs. */
UENUM()
namespace EMeshFeatureImportance
{
	enum Type : int
	{
		Off,
		Lowest,
		Low,
		Normal,
		High,
		Highest
	};
}


/** Enum specifying the reduction type to use when simplifying static meshes with the engines internal tool */
UENUM()
enum class EStaticMeshReductionTerimationCriterion : uint8
{
	Triangles UMETA(DisplayName = "Triangles", ToolTip = "Triangle percent criterion will be used for simplification."),
	Vertices UMETA(DisplayName = "Vertice", ToolTip = "Vertice percent criterion will be used for simplification."),
	Any UMETA(DisplayName = "First Percent Satisfied", ToolTip = "Simplification will continue until either Triangle or Vertex count criteria is met."),
};

/** Settings used to reduce a mesh. */
USTRUCT(Blueprintable)
struct FMeshReductionSettings
{
	GENERATED_USTRUCT_BODY()

	/** Percentage of triangles to keep. 1.0 = no reduction, 0.0 = no triangles. (Triangles criterion properties) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float PercentTriangles;

	/** The maximum number of triangles to retain when using percentage termination criterion. (Triangles criterion properties) */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Triangle Count", ClampMin = 2, UIMin = "2"))
	uint32 MaxNumOfTriangles;

	/** Percentage of vertices to keep. 1.0 = no reduction, 0.0 = no vertices. (Vertices criterion properties) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float PercentVertices;

	/** The maximum number of vertices to retain when using percentage termination criterion. (Vertices criterion properties) */
	UPROPERTY(EditAnywhere, Category = ReductionMethod, meta = (DisplayName = "Max Vertex Count", ClampMin = 4, UIMin = "4"))
	uint32 MaxNumOfVerts;

	/** The maximum distance in object space by which the reduced mesh may deviate from the original mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float MaxDeviation;

	/** The amount of error in pixels allowed for this LOD. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float PixelError;

	/** Threshold in object space at which vertices are welded together. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float WeldingThreshold;

	/** Angle at which a hard edge is introduced between faces. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	float HardAngleThreshold;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	int32 BaseLODModel;

	/** Higher values minimize change to border edges. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	TEnumAsByte<EMeshFeatureImportance::Type> SilhouetteImportance;

	/** Higher values reduce texture stretching. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	TEnumAsByte<EMeshFeatureImportance::Type> TextureImportance;

	/** Higher values try to preserve normals better. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	TEnumAsByte<EMeshFeatureImportance::Type> ShadingImportance;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	uint8 bRecalculateNormals:1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	uint8 bGenerateUniqueLightmapUVs:1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	uint8 bKeepSymmetry:1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	uint8 bVisibilityAided:1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	uint8 bCullOccluded:1;

	/** The method to use when optimizing static mesh LODs */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	EStaticMeshReductionTerimationCriterion TerminationCriterion;

	/** Higher values generates fewer samples*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	TEnumAsByte<EMeshFeatureImportance::Type> VisibilityAggressiveness;

	/** Higher values minimize change to vertex color data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ReductionSettings)
	TEnumAsByte<EMeshFeatureImportance::Type> VertexColorImportance;

	/** Default settings. */
	ENGINE_API FMeshReductionSettings();

	/** Equality operator. */
	ENGINE_API bool operator==(const FMeshReductionSettings& Other) const;

	/** Inequality. */
	ENGINE_API bool operator!=(const FMeshReductionSettings& Other) const;
};
