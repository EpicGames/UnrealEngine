// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshUVChannelInfo.generated.h"

// Constants.
enum { MAX_TEXCOORDS = 4 };

/** The world size for each texcoord mapping. Used by the texture streaming. */
USTRUCT(BlueprintType)
struct FMeshUVChannelInfo
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor (no initialization). */
	inline FMeshUVChannelInfo() { FMemory::Memzero(*this); }

	/** Constructor which initializes all components to zero. */
	FMeshUVChannelInfo(ENoInit) { }

	FMeshUVChannelInfo(float DefaultDensity) : bInitialized(true), bOverrideDensities(false)
	{
		for (float& Density : LocalUVDensities)
		{
			Density = DefaultDensity;
		}
	}

	/** Returns whether the structure contains any valid LocalUVDensities. */
	bool IsInitialized() const
	{
		if (bInitialized)
		{
			for (float Density : LocalUVDensities)
			{
				if (Density != 0)
				{
					return true;
				}
			}
		}
		return false;
	}

	UPROPERTY()
	bool bInitialized;

	/** Whether this values was set manually or is auto generated. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Basic)
	bool bOverrideDensities;

	/**
	 * The UV density in the mesh, before any transform scaling, in world unit per UV.
	 * This value represents the length taken to cover a full UV unit.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Basic, meta = (EditCondition = "bOverrideDensities"))
	float LocalUVDensities[MAX_TEXCOORDS];

	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FMeshUVChannelInfo& Info);
};
