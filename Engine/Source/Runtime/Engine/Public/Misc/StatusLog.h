// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StatusLog.h: high level status logging helper:
		should be minimal and track important game state changing events, travel, game ending failures, etc
		verbosity should be minimal, goal is 20 lines per match / round
=============================================================================*/

#pragma once

#include "CoreGlobals.h"

ENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogGlobalStatus, Verbose, All);


#define UE_LOGSTATUS(Verbosity, STRING, ...) UE_LOG(LogGlobalStatus, Verbosity, TEXT("%s ") STRING, ANSI_TO_TCHAR(__FUNCTION__), ##__VA_ARGS__ )
/* UE_LOGFMTSTATUS 
 * can only be used as such will cause compilation errors if not specifying format arguments completely
 * UE_LOGFMTSTATUS(Log, "format string {FormatArgument1} {FormatArguement2}", ("FormatArgument1", FormatArgument1Value), ("FormatArgument2", FormatArgument2Value));
 */
#define UE_LOGFMTSTATUS(Verbosity, STRING, ...) UE_LOGFMT(LogGlobalStatus, Verbosity, "{FUNCTIONNAME} " STRING, ("FUNCTIONNAME", __FUNCTION__), ##__VA_ARGS__)


