// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Model.h: Unreal UModel definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/StructOpsTypeTraits.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "PackedNormal.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RawIndexBuffer.h"
#include "LocalVertexFactory.h"
#include "Templates/UniquePtr.h"
#include "StaticMeshResources.h"

#include "Model.generated.h"

class AActor;
class ABrush;
class FBlake3;
class FMeshMapBuildData;
class ULevel;
class ULightComponent;
class UMaterialInterface;
class UModel;
class UModelComponent;
class UPolys;
struct FStaticLightingVertex;

//
// One vertex associated with a Bsp node's polygon.  Contains a vertex index
// into the level's FPoints table, and a unique number which is common to all
// other sides in the level which are cospatial with this side.
//
class FVert
{
public:
	// Variables.
	int32 	pVertex;	// Index of vertex.
	int32		iSide;		// If shared, index of unique side. Otherwise INDEX_NONE.

	/** The vertex's shadow map coordinate. */
	FVector2f ShadowTexCoord;

	/** The vertex's shadow map coordinate for the backface of the node. */
	FVector2f BackfaceShadowTexCoord;

	// Functions.
	friend FArchive& operator<< (FArchive &Ar, FVert &Vert)
	{
		// @warning BulkSerialize: FVert is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << Vert.pVertex << Vert.iSide;
		Ar << Vert.ShadowTexCoord;
		Ar << Vert.BackfaceShadowTexCoord;
		return Ar;
	}
};

//
//	FBspNode
//

// Flags associated with a Bsp node.
enum EBspNodeFlags
{
	// Flags.
	NF_NotCsg			= 0x01, // Node is not a Csg splitter, i.e. is a transparent poly.
	NF_NotVisBlocking   = 0x04, // Node does not block visibility, i.e. is an invisible collision hull.
	NF_BrightCorners	= 0x10, // Temporary.
	NF_IsNew 		 	= 0x20, // Editor: Node was newly-added.
	NF_IsFront     		= 0x40, // Filter operation bounding-sphere precomputed and guaranteed to be front.
	NF_IsBack      		= 0x80, // Guaranteed back.
};

//
// FBspNode defines one node in the Bsp, including the front and back
// pointers and the polygon data itself.  A node may have 0 or 3 to (MAX_NODE_VERTICES-1)
// vertices. If the node has zero vertices, it's only used for splitting and
// doesn't contain a polygon (this happens in the editor).
//
// vNormal, vTextureU, vTextureV, and others are indices into the level's
// vector table.  iFront,iBack should be INDEX_NONE to indicate no children.
//
// If iPlane==INDEX_NONE, a node has no coplanars.  Otherwise iPlane
// is an index to a coplanar polygon in the Bsp.  All polygons that are iPlane
// children can only have iPlane children themselves, not fronts or backs.
//
struct FBspNode // 62 bytes
{
	enum {MAX_NODE_VERTICES=255};	// Max vertices in a Bsp node.
	enum {MAX_ZONES=64};			// Max zones per level.

	// Persistent information.
	FPlane4f		Plane;			// 16 Plane the node falls into (X, Y, Z, W).
	int32			iVertPool;		// 4  Index of first vertex in vertex pool, =iTerrain if NumVertices==0 and NF_TerrainFront.
	int32			iSurf;			// 4  Index to surface information.

	/** The index of the node's first vertex in the UModel's vertex buffer. */
	int32			iVertexIndex;

	/** The index in ULevel::ModelComponents of the UModelComponent containing this node. */
	uint16		ComponentIndex;

	/** The index of the node in the UModelComponent's Nodes array. */
	uint16		ComponentNodeIndex;

	/** The index of the element in the UModelComponent's Element array. */
	int32			ComponentElementIndex;

	// iBack:  4  Index to node in front (in direction of Normal).
	// iFront: 4  Index to node in back  (opposite direction as Normal).
	// iPlane: 4  Index to next coplanar poly in coplanar list.
	union { int32 iBack; int32 iChild[1]; };
	        int32 iFront;
			int32 iPlane;

	int32		iCollisionBound;// 4  Collision bound.

	uint8	iZone[2];		// 2  Visibility zone in 1=front, 0=back.
	uint8	NumVertices;	// 1  Number of vertices in node.
	uint8	NodeFlags;		// 1  Node flags.
	int32		iLeaf[2];		// 8  Leaf in back and front, INDEX_NONE=not a leaf.

	// Functions.
	bool IsCsg( uint32 ExtraFlags=0 ) const
	{
		return (NumVertices>0) && !(NodeFlags & (NF_IsNew | NF_NotCsg | ExtraFlags));
	}
	bool ChildOutside( int32 IniChild, bool Outside, uint32 ExtraFlags=0 ) const
	{
		return IniChild ? (Outside || IsCsg(ExtraFlags)) : (Outside && !IsCsg(ExtraFlags));
	}
	friend ENGINE_API FArchive& operator<<( FArchive& Ar, FBspNode& N );
};

//
//	FZoneProperties
//

struct FZoneSet
{
	FZoneSet(): MaskBits(0) {}
	FZoneSet(uint64 InMaskBits): MaskBits(InMaskBits) {}

	friend FArchive& operator<<(FArchive& Ar,FZoneSet& S)
	{
		return Ar << S.MaskBits;
	}

private:

	uint64	MaskBits;
};


struct FZoneProperties
{
public:
	// Variables.
	class AActor*	ZoneActor;		// Optional actor defining the zone's property. @todo UE4 obsolete/unused - remove.
	float				LastRenderTime;	// Most recent level TimeSeconds when rendered.
	FZoneSet			Connectivity;	// (Connect[i]&(1<<j))==1 if zone i is adjacent to zone j.
	FZoneSet			Visibility;		// (Connect[i]&(1<<j))==1 if zone i can see zone j.
};

//
//	FLeaf
//

struct FLeaf
{
	int32		iZone;          // The zone this convex volume is in.

	// Functions.
	FLeaf()
	{}
	FLeaf(int32 iInZone):
		iZone(iInZone)
	{}
	friend FArchive& operator<<( FArchive& Ar, FLeaf& L )
	{
		// @warning BulkSerialize: FLeaf is serialized as memory dump
		// See TArray::BulkSerialize for detailed description of implied limitations.
		Ar << L.iZone;
		return Ar;
	}
};

//
//	FBspSurf
//

//
// One Bsp polygon.  Lists all of the properties associated with the
// polygon's plane.  Does not include a point list; the actual points
// are stored along with Bsp nodes, since several nodes which lie in the
// same plane may reference the same poly.
//
USTRUCT()
struct FBspSurf
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMaterialInterface>	Material;		// 4 Material.
	uint32							PolyFlags;		// 4 Polygon flags.
	int32							pBase;			// 4 Polygon & texture base point index (where U,V==0,0).
	int32							vNormal;		// 4 Index to polygon normal.
	int32							vTextureU;		// 4 Texture U-vector index.
	int32							vTextureV;		// 4 Texture V-vector index.
	int32							iBrushPoly;		// 4 Editor brush polygon index.
	UPROPERTY()
	TObjectPtr<ABrush>				Actor;			// 4 Brush actor owning this Bsp surface.
	FPlane4f						Plane;			// 16 The plane this surface lies on.
	float							LightMapScale;	// 4 The number of units/lightmap texel on this surface.
	int32							iLightmassIndex;// 4 Index to the lightmass settings

	bool				bHiddenEdTemporary;	// 1 Marks whether this surface is temporarily hidden in the editor or not. Not serialized.
	bool				bHiddenEdLevel;		// 1 Marks whether this surface is hidden by the level browser or not. Not serialized.
	bool				bHiddenEdLayer;		// 1 Marks whether this surface is hidden by the layer browser or not. Not serialized.
	// Functions.


#if WITH_EDITOR
	/** @return true if this surface is hidden in the editor; false otherwise	 */
	ENGINE_API bool IsHiddenEd() const;

	/** Returns true if this surface is hidden at editor startup */
	ENGINE_API bool IsHiddenEdAtStartup() const;
#endif

	friend ENGINE_API FArchive& operator<<( FArchive& Ar, FBspSurf& Surf );

	void AddReferencedObjects( FReferenceCollector& Collector );
};

// Flags describing effects and properties of a Bsp polygon.
enum EPolyFlags
{
	// Regular in-game flags.
	PF_Invisible			= 0x00000001,	// Poly is invisible.
	PF_NotSolid				= 0x00000008,	// Poly is not solid, doesn't block.
	PF_Semisolid			= 0x00000020,	// Poly is semi-solid = collision solid, Csg nonsolid.
	PF_GeomMarked			= 0x00000040,	// Geometry mode sometimes needs to mark polys for processing later.
	PF_TwoSided				= 0x00000100,	// Poly is visible from both sides.
	PF_Portal				= 0x04000000,	// Portal between iZones.

	// Editor flags.
	PF_Memorized     		= 0x01000000,	// Editor: Poly is remembered.
	PF_Selected      		= 0x02000000,	// Editor: Poly is selected.
	PF_HiddenEd				= 0x08000000,	// Editor: Poly is hidden in the editor at startup.
	PF_Hovered				= 0x10000000,	// Editor: Poly is currently hovered over in editor.

	// Internal.
	PF_EdProcessed 			= 0x40000000,	// FPoly was already processed in editorBuildFPolys.
	PF_EdCut       			= 0x80000000,	// FPoly has been split by SplitPolyWithPlane.

	// Combinations of flags.
	PF_NoEdit				= PF_Memorized | PF_Selected | PF_Hovered | PF_EdProcessed | PF_EdCut,
	PF_NoImport				= PF_NoEdit | PF_Memorized | PF_Selected | PF_Hovered | PF_EdProcessed | PF_EdCut,
	PF_AddLast				= PF_Semisolid | PF_NotSolid,
	PF_NoAddToBSP			= PF_EdCut | PF_EdProcessed | PF_Selected | PF_Hovered | PF_Memorized,
	PF_ModelComponentMask	= 0,

	PF_DefaultFlags			= 0,
};

struct FModelVertex
{
	FVector3f Position;
	FVector3f TangentX;
	FVector4f TangentZ;
	FVector2f TexCoord;
	FVector2f ShadowTexCoord;

	FVector3f GetTangentY() const
	{
		FVector3f TanX = TangentX;
		FVector3f TanZ = TangentZ;

		return (TanZ ^ TanX) * TangentZ.W;
	};
	/**
	* Serializer
	*
	* @param Ar - archive to serialize with
	* @param V - vertex to serialize
	* @return archive that was used
	*/
	friend FArchive& operator<<(FArchive& Ar,FModelVertex& V);
};


struct FDepecatedModelVertex
{
	FVector3f Position;
	FDeprecatedSerializedPackedNormal TangentX;
	FDeprecatedSerializedPackedNormal TangentZ;
	FVector2f TexCoord;
	FVector2f ShadowTexCoord;

	operator FModelVertex() const
	{
		FModelVertex Ret;
		Ret.Position = Position;
		Ret.TangentX = TangentX;
		Ret.TangentZ = TangentZ;
		Ret.TexCoord = TexCoord;
		Ret.ShadowTexCoord = ShadowTexCoord;
		return Ret;
	}

	friend FArchive& operator<<(FArchive& Ar, FDepecatedModelVertex& V);
};

/**
 * A vertex buffer for a set of BSP nodes.
 */
class FModelVertexBuffer
{
public:
	// Number of FModelSceneProxy's using this
	// Only access in CreateRenderState_Concurrent and DestroyRenderState_Concurrent
	uint32 RefCount = 0;

	/** model vertex data */
	TArray<FModelVertex> Vertices;

	/** Resources used for Rendering. */
	FStaticMeshVertexBuffers Buffers;

	/** Minimal initialization constructor. */
	FModelVertexBuffer(UModel* InModel);
	
	/**
	* Serializer for this class
	* @param Ar - archive to serialize to
	* @param B - data to serialize
	*/
	friend FArchive& operator<<(FArchive& Ar,FModelVertexBuffer& B);
#if WITH_EDITOR
	friend void UpdateHash(FBlake3& Builder, const FModelVertexBuffer& B);
#endif

private:
	UModel* Model;
	uint32 NumVerticesRHI;
};

/** A struct that contains a set of conodes that will be used in one mapping */
struct FNodeGroup
{
	ENGINE_API ~FNodeGroup();

	/** List of nodes in the node group */
	TArray<int32> Nodes;

	/** List of relevant lights for this nodegroup */
	TArray<ULightComponent*> RelevantLights;

	FVector3f TangentX;
	FVector3f TangentY;
	FVector3f TangentZ;

	FMatrix MapToWorld;
	FMatrix WorldToMap;

	FBox BoundingBox;

	int32 SizeX;
	int32 SizeY;

	/** The surface's vertices. */
	TArray<FStaticLightingVertex> Vertices;

	/** The vertex indices of the surface's triangles. */
	TArray<int32> TriangleVertexIndices;

	/** For each triangle, record which LightmassSettings to use (material, boost, etc) */
	TArray<int32> TriangleSurfaceMap;
};

//
//	UModel
//

enum {MAX_NODES  = 65536};
enum {MAX_POINTS = 128000};
UCLASS(MinimalAPI)
class UModel : public UObject
{
	GENERATED_BODY()
public:
	UModel(FVTableHelper& Helper);

#if WITH_EDITORONLY_DATA
	// Arrays and subobjects.
	UPROPERTY(SkipSerialization)
	TObjectPtr<UPolys>						Polys;
#endif // WITH_EDITOR

	TArray<FBspNode>		Nodes;
	TArray<FVert>			Verts;
	TArray<FVector3f>			Vectors;
	TArray<FVector3f>			Points;

	UPROPERTY(SkipSerialization)
	TArray<FBspSurf>		Surfs;

#if WITH_EDITOR
	TArray<int32>				LeafHulls;
	TArray<FLeaf>				Leaves;
#endif // WITH_EDITOR

	TArray<FLightmassPrimitiveSettings>	LightmassSettings;

	/** An index buffer for each material used by the model, containing all the nodes with that material applied. */
	TMap<UMaterialInterface*,TUniquePtr<FRawIndexBuffer16or32> > MaterialIndexBuffers;

	/** A vertex buffer containing the vertices for all nodes in the UModel. */
	FModelVertexBuffer VertexBuffer;

	/** A fence which is used to keep track of the rendering thread releasing the model resources. */
	FRenderCommandFence ReleaseResourcesFence;

	/** True if surfaces in the model have been changed without calling ULevel::CommitModelSurfaces. */
	bool InvalidSurfaces;

	/** True if only the material index buffers should be rebuilt when committing model surfaces */
	bool bOnlyRebuildMaterialIndexBuffers;

	/** True if static lighting now can not be validly built for this model */
	bool bInvalidForStaticLighting;

	/** The number of unique vertices. */
	uint32 NumUniqueVertices;

	/** Unique ID for this model, used for caching during distributed lighting */
	FGuid LightingGuid;

#if WITH_EDITOR
	/** A map of NodeGroup ID to the NodeGroup object */
	TMap<int32, FNodeGroup*> NodeGroups;

	/** Cache the mappings that are going to be calculated during lighting (we delay applying to join mappings into bigger lightmaps) */
	TArray<class FBSPSurfaceStaticLighting*> CachedMappings;

	/** How many node groups still need to be completed before we start joining by brightness, etc */
	int32 NumIncompleteNodeGroups;

	/** The level used to generate NodeGroups */
	ULevel* LightingLevel;

	/** Cached transform of the owner brush when the geometry was last built */
	FVector3f OwnerLocationWhenLastBuilt;
	FRotator OwnerRotationWhenLastBuilt;
	FVector3f OwnerScaleWhenLastBuilt;

	/** Specifies whether the above cached transform is valid */
	bool bCachedOwnerTransformValid;
#endif // WITH_EDITOR

	// Other variables.
	bool						RootOutside;
	bool						Linked;
	int32						NumSharedSides;
	FBoxSphereBounds			Bounds;
private:
	ENGINE_API static float BSPTexelScale;
public:

	// Constructors.
	UModel(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	ENGINE_API void Initialize();
	ENGINE_API void Initialize(ABrush* Owner, bool InRootOutside = true);

	// UObject interface.
	virtual void Serialize( FArchive& Ar ) override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif // WITH_EDITOR
	virtual bool Rename( const TCHAR* InName=NULL, UObject* NewOuter=NULL, ERenameFlags Flags=REN_None ) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * @return The global BSP Texel scale 
	 */
	static float GetGlobalBSPTexelScale() { return BSPTexelScale; }

	/**
	 * Sets the global texel scale used for BSP UV's
	 *
	 * @param The new global texel scale
	 */
	static void SetGlobalBSPTexelScale( float InBSPTexelScale ) { BSPTexelScale = InBSPTexelScale; }

	/**
	 * Called after duplication & serialization and before PostLoad. Used to make sure UModel's FPolys
	 * get duplicated as well.
	 */
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void BeginDestroy() override;
	virtual bool IsReadyForFinishDestroy() override;
	
	virtual bool IsAsset() const override { return false; }

	/**
	* @return		Sum of the size of textures referenced by this material.
	*/
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	// UModel interface.
	ENGINE_API void EmptyModel( int32 EmptySurfInfo, int32 EmptyPolys );

	/** Begins releasing the model's resources. */
	ENGINE_API void BeginReleaseResources();

	/** Begins initializing the model's VertexBuffer. */
	ENGINE_API void UpdateVertices();

	/** Compute the "center" location of all the verts */
	FVector GetCenter();

	ENGINE_API void GetSurfacePlanes(
		const AActor*	Owner,
		TArray<FPlane>& OutPlanes);

#if WITH_EDITOR
	/** Build the model's bounds (min and max). */
	ENGINE_API void BuildBound();

	/** Transform this model by its coordinate system. */ 
	void Transform( ABrush* Owner );

	/** Shrink all stuff to its minimum size. */
	ENGINE_API void ShrinkModel();

	/**
	* Initialize vertex buffer data from UModel data
	* Returns the number of vertices in the vertex buffer.
	*/
	int32 BuildVertexBuffers();

	// UModel transactions.
	ENGINE_API void ModifySelectedSurfs( bool UpdateBrushes );
	void ModifyAllSurfs( bool UpdateBrushes );
	ENGINE_API void ModifySurf( int32 InIndex, bool UpdateBrushes );
	ENGINE_API bool HasSelectedSurfaces() const;
#endif

	ENGINE_API float FindNearestVertex
	(
		const FVector3f	&SourcePoint,
		FVector3f			&DestPoint,
		float			MinRadius,
		int32				&pVertex
	) const;
	void PrecomputeSphereFilter
	(
		const FPlane	&Sphere
	);

	/* Find the source brush actor associated with this point, or NULL if the point does not lie on a BSP surface. */
	ENGINE_API ABrush* FindBrush(const FVector3f &SourcePoint) const;

	/**
	 * Creates a bounding box for the passed in node
	 *
	 * @param	Node	Node to create a bounding box for
	 * @param	OutBox	The created box
	 */
	ENGINE_API void GetNodeBoundingBox( const FBspNode& Node, FBox& OutBox ) const;

	/**
	 * Groups all nodes in the model into NodeGroups (cached in the NodeGroups object)
	 *
	 * @param Level The level for this model
	 * @param Lights The possible lights that will be cached in the NodeGroups
	 */
	ENGINE_API void GroupAllNodes(ULevel* Level, const TArray<class ULightComponentBase*>& Lights);

	/**
	 * Applies all of the finished lighting cached in the NodeGroups 
	 */
	void ApplyStaticLighting(const FStaticLightingBuildContext* LightingContext);

	/**
	 * Apply world origin changes
	 */
	void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift);

	/**
	* Clears local (non RHI) data associated with MaterialIndexBuffers
	*/
	ENGINE_API void ClearLocalMaterialIndexBuffersData();

	void CalculateUniqueVertCount();

	friend class UWorld;
	friend class UBrushComponent;
	friend class UStaticMeshComponent;
	friend class AActor;
	friend class AVolume;

private:
	FGuid ConstructLightingGuid() const;
};

/**
 * A set of BSP nodes which have the same material and relevant lights.
 */
class FModelElement
{
public:

	/** The model component containing this element. */
	TObjectPtr<class UModelComponent> Component;

	/** The material used by the nodes in this element. */
	TObjectPtr<class UMaterialInterface> Material;

	/** The nodes in the element. */
	TArray<uint16> Nodes;

	FMeshMapBuildData* LegacyMapBuildData;

	/** Uniquely identifies this component's built map data. */
	FGuid MapBuildDataId;

	/** A pointer to the index buffer holding this element's indices. */
	FRawIndexBuffer16or32* IndexBuffer;

	/** The first index in the component index buffer used by this element. */
	uint32 FirstIndex;

	/** The number of triangles contained by the component index buffer for this element. */
	uint32 NumTriangles;

	/** The lowest vertex index used by this element. */
	uint32 MinVertexIndex;

	/** The highest vertex index used by this element. */
	uint32 MaxVertexIndex;

	/** The bounding box of the vertices in the element. */
	FBox BoundingBox;

	/**
	 * Minimal initialization constructor.
	 */
	ENGINE_API FModelElement(UModelComponent* InComponent,UMaterialInterface* InMaterial);
	ENGINE_API FModelElement();
	ENGINE_API virtual ~FModelElement();

	ENGINE_API const FMeshMapBuildData* GetMeshMapBuildData() const;

	/**
	 * Serializer.
	 */
	friend FArchive& operator<<(FArchive& Ar,FModelElement& Element);
};
