// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModelLight.h: Unreal model lighting.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "StaticLighting.h"
#include "Model.h"

class FShadowMapData2D;
class ULevel;
class ULightComponent;
class UModelComponent;
struct FQuantizedLightmapData;

/** Represents a BSP surface to the static lighting system. */
class FBSPSurfaceStaticLighting : public FStaticLightingTextureMapping, public FStaticLightingMesh
{
public:

	/** The surface's static lighting node group mapping info. */
	const FNodeGroup* NodeGroup;

	/** true if the surface has complete static lighting. */
	bool bComplete;

	TMap<ULightComponent*,FShadowMapData2D*> ShadowMapData;

	/** Quantized light map data */
	FQuantizedLightmapData* QuantizedData;

	/** Minimum rectangle that encompasses all mapped texels. */
	FIntRect MappedRect;

	/** Initialization constructor. */
	ENGINE_API FBSPSurfaceStaticLighting(
		const FNodeGroup* InNodeGroup,
		UModel* Model,
		UModelComponent* Component
		);

	// FStaticLightingMesh interface.
	virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const override;
	virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const override;
	virtual FLightRayIntersection IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const override;
	//FStaticLightingTextureMapping interface.

	virtual bool IsInstancedMesh() const override { return false; }

	virtual bool IsValidMapping() const override
	{
		return Model.IsValid() && !Model->bInvalidForStaticLighting;
	}

#if WITH_EDITOR
	virtual void Apply(FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext) override;
	virtual bool DebugThisMapping() const override;

	/** 
	* Export static lighting mapping instance data to an exporter 
	* @param Exporter - export interface to process static lighting data
	**/
	UNREALED_API virtual void ExportMapping(class FLightmassExporter* Exporter) override;
#endif	//WITH_EDITOR

	/**
	 * Returns the Guid used for static lighting.
	 * @return FGuid that identifies the mapping
	 **/
	virtual const FGuid& GetLightingGuid() const override
	{
		return Guid;
	}

	virtual FString GetDescription() const override
	{
		return FString("BSPMapping");
	}

	const UModel* GetModel() const
	{
		return Model.Get();
	}

#if WITH_EDITOR
	/**
	 *	@return	UOject*		The object that is mapped by this mapping
	 */
	UNREALED_API virtual UObject* GetMappedObject() const override;
#endif	//WITH_EDITOR

private:

	/** The model this lighting data is for */
	TWeakObjectPtr<UModel> Model;
};
