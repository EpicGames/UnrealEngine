// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "BandwidthTestActor.generated.h"

//-----------------------------------------------------------------------------
//
USTRUCT()
struct FBandwidthTestItem
{
	GENERATED_BODY()

	// Contains up to 1000 bytes
	UPROPERTY()
	TArray<uint8> Kilobyte;
};

//-----------------------------------------------------------------------------
//
USTRUCT()
struct FBandwidthTestGenerator
{
	GENERATED_BODY()

	bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms);

	UPROPERTY()
	TArray<FBandwidthTestItem> ReplicatedBuffers;

	void FillBufferForSize(int32 SizeInKilobytes);

	void OnSpikePeriod();

	double TimeForNextSpike = 0.0;
	double SpikePeriodInSec = 0.0f;
};

//-----------------------------------------------------------------------------
//
template<>
struct TStructOpsTypeTraits<FBandwidthTestGenerator> : public TStructOpsTypeTraitsBase2<FBandwidthTestGenerator>
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

/**
 * This ABandwidthTestActor class is used to generate an easily controllable amount of bandwidth.
 * It uses property replication to generate it's traffic via a NetDeltaSerializer struct
 * Note that the property data is never stored in memory on the simulated clients 
 */
UCLASS(transient, notplaceable)
class ABandwidthTestActor : public AActor
{
	GENERATED_BODY()

public:
	ABandwidthTestActor();

	UPROPERTY(Replicated)
	FBandwidthTestGenerator BandwidthGenerator;

	void StartGeneratingBandwidth(float BandwidthInKilobytesPerSec);
	void StopGeneratingBandwidth();

	void StartBandwidthSpike(float SpikeInKilobytes, int32 PeriodInMS);
};
