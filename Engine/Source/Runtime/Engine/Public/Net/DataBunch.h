// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataBunch.h: Unreal bunch class.
=============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "Misc/NetworkGuid.h"
#include "UObject/CoreNet.h"
#include "EngineLogs.h"
#include "Net/Core/Trace/NetTraceConfig.h"
#include "Net/Core/NetToken/NetToken.h"

class UChannel;
class UNetConnection;

extern const int32 MAX_BUNCH_SIZE;

//
// A bunch of data to send.
//
class FOutBunch : public FNetBitWriter
{
public:
	// Variables.
	FOutBunch *				Next;
	UChannel *				Channel;
	double					Time;
	int32					ChIndex;
	FName					ChName;
	int32					ChSequence;
	int32					PacketId;
	uint8					ReceivedAck:1;
	uint8					bOpen:1;
	uint8					bClose:1;
	UE_DEPRECATED(5.3, "Replication pausing is deprecated")
	uint8					bIsReplicationPaused:1;   // Replication on this channel is being paused by the server
	uint8					bReliable:1;
	uint8					bPartial:1;				// Not a complete bunch
	uint8					bPartialInitial:1;		// The first bunch of a partial bunch
	uint8					bPartialFinal:1;			// The final bunch of a partial bunch
	uint8					bHasPackageMapExports:1;	// This bunch has networkGUID name/id pairs
	uint8					bHasMustBeMappedGUIDs:1;	// This bunch has guids that must be mapped before we can process this bunch
	uint8					bPartialCustomExportsFinal:1;	// This bunch marks the end of the CustomExports data that needs to be processed immediately (not queued)
	uint8					bOutWantsFullInitState:1;	// Set this true to force all replicated properties to be serialized in the initial bunch even if they do not differ from Archetype baseline

	EChannelCloseReason		CloseReason;

	TArray< FNetworkGUID >	ExportNetGUIDs;			// List of GUIDs that went out on this bunch
	TArray< uint64 >		NetFieldExports;
	TArray<UE::Net::FNetToken, TInlineAllocator<4>> NetTokensPendingExport; // List of NetTokens that will be exported if needed with this bunch.

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString			DebugString;
	void	SetDebugString(FString DebugStr)
	{
			DebugString = DebugStr;
	}
	FString	GetDebugString() const
	{
		return DebugString;
	}
#else
	inline void SetDebugString(FString DebugStr)
	{

	}
	inline FString	GetDebugString() const
	{
		return FString();
	}
#endif

	// Functions.
	ENGINE_API FOutBunch();
	ENGINE_API explicit FOutBunch(int64 InMaxBits);
	ENGINE_API FOutBunch( class UChannel* InChannel, bool bClose );
	ENGINE_API FOutBunch( UPackageMap * PackageMap, int64 InMaxBits = 1024 );

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOutBunch(FOutBunch&&) = default;
	FOutBunch(const FOutBunch&) = default;
	FOutBunch& operator=(FOutBunch&&) = default;
	FOutBunch& operator=(const FOutBunch&) = default;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~FOutBunch();

	FString	ToString()
	{
		// String cating like this is super slow! Only enable in non shipping builds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FString Str("FOutBunch: ");
		Str += FString::Printf(TEXT("Channel[%d] "), ChIndex);
		Str += FString::Printf(TEXT("ChSequence: %d "), ChSequence);
		Str += FString::Printf(TEXT("NumBits: %lld "), GetNumBits());
		Str += FString::Printf(TEXT("PacketId: %d "), PacketId);
		Str += FString::Printf(TEXT("bOpen: %d "), bOpen);
		Str += FString::Printf(TEXT("bClose: %d "), bClose);
		if (bClose)
		{
			Str += FString::Printf(TEXT("CloseReason: %s "), LexToString(CloseReason));
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Str += FString::Printf(TEXT("bIsReplicationPaused: %d "), bIsReplicationPaused);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Str += FString::Printf(TEXT("bReliable: %d "), bReliable);
		Str += FString::Printf(TEXT("bPartial: %d//%d//%d "), bPartial, bPartialInitial, bPartialFinal);
		Str += FString::Printf(TEXT("bHasPackageMapExports: %d "), bHasPackageMapExports);
		Str += FString::Printf(TEXT("NetTokensPendingExport: %d "), NetTokensPendingExport.Num());
		Str += GetDebugString();
#else
		FString Str = FString::Printf(TEXT("Channel[%d]. Seq %d. PacketId: %d"), ChIndex, ChSequence, PacketId);
#endif
		return Str;
	}

	ENGINE_API virtual void CountMemory(FArchive& Ar) const override;
	ENGINE_API virtual void Reset() override;
};

//
// A bunch of data received from a channel.
//
class FInBunch : public FNetBitReader
{
public:
	// Variables.
	int32				PacketId;	// Note this must stay as first member variable in FInBunch for FInBunch(FInBunch, bool) to work
	FInBunch *			Next;
	UNetConnection *	Connection;
	int32				ChIndex;
	FName				ChName;
	int32				ChSequence;
	uint8				bOpen:1;
	uint8				bClose:1;
	UE_DEPRECATED(5.3, "Replication pausing is deprecated")
	uint8				bIsReplicationPaused:1;		// Replication on this channel is being paused by the server
	uint8				bReliable:1;
	uint8				bPartial:1;					// Not a complete bunch
	uint8				bPartialInitial:1;			// The first bunch of a partial bunch
	uint8				bPartialFinal:1;			// The final bunch of a partial bunch
	uint8				bHasPackageMapExports:1;	// This bunch has networkGUID name/id pairs
	uint8				bHasMustBeMappedGUIDs:1;	// This bunch has guids that must be mapped before we can process this bunch
	uint8				bPartialCustomExportsFinal:1;	// This bunch marks the end of the extensions data that needs to be processed immediately (not queued)
	uint8				bIgnoreRPCs:1;

	EChannelCloseReason		CloseReason;

	FString	ToString()
	{
		// String cating like this is super slow! Only enable in non shipping builds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FString Str("FInBunch: ");
		Str += FString::Printf(TEXT("Channel[%d] "), ChIndex);
		Str += FString::Printf(TEXT("ChSequence: %d "), ChSequence);
		Str += FString::Printf(TEXT("NumBits: %lld "), GetNumBits());
		Str += FString::Printf(TEXT("PacketId: %d "), PacketId);
		Str += FString::Printf(TEXT("bOpen: %d "), bOpen);
		Str += FString::Printf(TEXT("bClose: %d "), bClose);
		if (bClose)
		{
			Str += FString::Printf(TEXT("CloseReason: %s "), LexToString(CloseReason));
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Str += FString::Printf(TEXT("bIsReplicationPaused: %d "), bIsReplicationPaused);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Str += FString::Printf(TEXT("bReliable: %d "), bReliable);
		Str += FString::Printf(TEXT("bPartial: %d//%d//%d "), bPartial, bPartialInitial, bPartialFinal);
		Str += FString::Printf(TEXT("bHasPackageMapExports: %d "), bHasPackageMapExports );
		Str += FString::Printf(TEXT("bHasMustBeMappedGUIDs: %d "), bHasMustBeMappedGUIDs );
		Str += FString::Printf(TEXT("bPartialCustomExportsFinal: %d "), bPartialCustomExportsFinal);
		Str += FString::Printf(TEXT("bIgnoreRPCs: %d "), bIgnoreRPCs );
#else
		FString Str = FString::Printf(TEXT("Channel[%d]. Seq %d. PacketId: %d"), ChIndex, ChSequence, PacketId);
#endif
		return Str;
	}
 
	// Functions.
	ENGINE_API FInBunch( UNetConnection* InConnection, uint8* Src=NULL, int64 CountBits=0 );
	ENGINE_API FInBunch( FInBunch &InBunch, bool CopyBuffer );

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInBunch(FInBunch&&) = default;
	FInBunch(const FInBunch&) = default;
	FInBunch& operator=(FInBunch&&) = default;
	FInBunch& operator=(const FInBunch&) = default;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void CountMemory(FArchive& Ar) const override;
		
	virtual uint32 EngineNetVer() const override;
	virtual uint32 GameNetVer() const override;
	virtual void SetEngineNetVer(const uint32 InEngineNetVer) override;
	virtual void SetGameNetVer(const uint32 InGameNetVer) override;
};

/** out bunch for the control channel (special restrictions) */
struct FControlChannelOutBunch : public FOutBunch
{
	ENGINE_API FControlChannelOutBunch(class UChannel* InChannel, bool bClose);

	FArchive& operator<<(FName& Name)
	{
		UE_LOGF(LogNet, Fatal,"Cannot send Names on the control channel");
		SetError();
		return *this;
	}
	FArchive& operator<<(UObject*& Object)
	{
		UE_LOGF(LogNet, Fatal,"Cannot send Objects on the control channel");
		SetError();
		return *this;
	}
};


