// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataChannel.h: Unreal datachannel class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/NetworkGuid.h"
#include "Misc/NetworkVersion.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Net/Core/Connection/NetEnums.h"
#include "Net/NetIDVariant.h"
#include "Net/DataBunch.h"
#include "Engine/ChildConnection.h"

// Forward declarations
enum class ENetPingControlMessage : uint8;

namespace UE::Net
{
template<int32 Index, typename... ParamTypes>
class TNetControlMessageImpl;
}

/*-----------------------------------------------------------------------------
	UControlChannel base class.
-----------------------------------------------------------------------------*/

/** network control channel message types
 * 
 * to add a new message type, you need to:
 * - add a DEFINE_CONTROL_CHANNEL_MESSAGE for the message type with the appropriate parameters to this file
 * - add IMPLEMENT_CONTROL_CHANNEL_MESSAGE for the message type to DataChannel.cpp
 * - implement the fallback behavior (eat an unparsed message) to UControlChannel::ReceivedBunch()
 *
 * @warning: modifying control channel messages breaks network compatibility (update GEngineMinNetVersion)
 */
template<uint8 MessageType> class FNetControlMessage
{
};
/** contains info about a message type retrievable without static binding (e.g. whether it's a valid type, friendly name string, etc) */
class FNetControlMessageInfo
{
public:
	static inline const TCHAR* GetName(uint8 MessageIndex)
	{
		CheckInitialized();
		return Names[MessageIndex];
	}
	static inline bool IsRegistered(uint8 MessageIndex)
	{
		CheckInitialized();
		return Names[MessageIndex][0] != 0;
	}
	template<int32 Index, typename... ParamTypes> friend class UE::Net::TNetControlMessageImpl;
private:
	static void CheckInitialized()
	{
		if (!bInitialized)
		{
			for (int32 i = 0; i < UE_ARRAY_COUNT(Names); i++)
			{
				Names[i] = TEXT("");
			}
			bInitialized = true;
		}
	}
	static void SetName(uint8 MessageType, const TCHAR* InName)
	{
		CheckInitialized();
		Names[MessageType] = InName;
	}

	template<typename... ParamTypes>
	static void SendParams(FControlChannelOutBunch& Bunch, ParamTypes&... Params) {}

	template<typename FirstParamType, typename... ParamTypes>
	static void SendParams(FControlChannelOutBunch& Bunch, FirstParamType& FirstParam, ParamTypes&... Params)
	{
		Bunch << FirstParam;
		SendParams(Bunch, Params...);
	}

	template<typename... ParamTypes>
	static void ReceiveParams(FInBunch& Bunch, ParamTypes&... Params) {}
	
	template<typename FirstParamType, typename... ParamTypes>
	static void ReceiveParams(FInBunch& Bunch, FirstParamType& FirstParam, ParamTypes&... Params)
	{
		Bunch << FirstParam;
		ReceiveParams(Bunch, Params...);
	}
	
	static constexpr int MaxNames = TNumericLimits<uint8>::Max() + 1;
	static ENGINE_API const TCHAR* Names[MaxNames];
	static ENGINE_API bool bInitialized;
};

namespace UE::Net
{

template<int32 Index, typename... ParamTypes>
class TNetControlMessageImpl
{
public:
	static uint8 Initialize(const TCHAR* InName)
	{
		FNetControlMessageInfo::SetName(Index, InName);
		return 0;
	}

	/**
	 * sends a message of this type on the specified connection's control channel
	 * @note: const not used only because of the FArchive interface; the parameters are not modified
	 */
	static void Send(UNetConnection* Conn, ParamTypes&... Params)
	{
		static_assert(Index < FNetControlMessageInfo::MaxNames, "Control channel message must be a byte.");
		checkSlow(!Conn->IsA(UChildConnection::StaticClass())); /** control channel messages can only be sent on the parent connection */
		if (Conn->Channels[0] != nullptr && !Conn->Channels[0]->Closing)
		{
			FControlChannelOutBunch Bunch(Conn->Channels[0], false);
			uint8 MessageType = Index;
			Bunch << MessageType;
			FNetControlMessageInfo::SendParams(Bunch, Params...);
			Conn->Channels[0]->SendBunch(&Bunch, true);
		}
	}

	/** receives a message of this type from the passed in bunch */
	[[nodiscard]] static bool Receive(FInBunch& Bunch, ParamTypes&... Params)
	{
		FNetControlMessageInfo::ReceiveParams(Bunch, Params...);
		return !Bunch.IsError();
	}

	/** throws away a message of this type from the passed in bunch */
	static void Discard(FInBunch& Bunch)
	{
		TTuple<ParamTypes...> Params;
		VisitTupleElements([&Bunch](auto& Param)
			{
				Bunch << Param;
			},
			Params);
	}
};

}

#define DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, ...) \
enum { NMT_##Name = Index }; \
template<> class FNetControlMessage<Index> : public UE::Net::TNetControlMessageImpl<Index, ##__VA_ARGS__> \
{ \
};

#define DEFINE_CONTROL_CHANNEL_MESSAGE_ZEROPARAM(Name, Index) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_ONEPARAM(Name, Index, TypeA) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_TWOPARAM(Name, Index, TypeA, TypeB) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_THREEPARAM(Name, Index, TypeA, TypeB, TypeC) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_FOURPARAM(Name, Index, TypeA, TypeB, TypeC, TypeD) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC, TypeD)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_FIVEPARAM(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_SIXPARAM(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_SEVENPARAM(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF, TypeG) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF, TypeG)
#define DEFINE_CONTROL_CHANNEL_MESSAGE_EIGHTPARAM(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF, TypeG, TypeH) DEFINE_CONTROL_CHANNEL_MESSAGE(Name, Index, TypeA, TypeB, TypeC, TypeD, TypeE, TypeF, TypeG, TypeH)

#define IMPLEMENT_CONTROL_CHANNEL_MESSAGE(Name) static uint8 Dummy##_FNetControlMessage_##Name = FNetControlMessage<NMT_##Name>::Initialize(TEXT(#Name));

// message type definitions
DEFINE_CONTROL_CHANNEL_MESSAGE(Hello, 0, uint8, uint32, FString, EEngineNetworkRuntimeFeatures); // initial client connection message
DEFINE_CONTROL_CHANNEL_MESSAGE(Welcome, 1, FString, FString, FString); // server tells client they're ok'ed to load the server's level
DEFINE_CONTROL_CHANNEL_MESSAGE(Upgrade, 2, uint32, EEngineNetworkRuntimeFeatures); // server tells client their version is incompatible
DEFINE_CONTROL_CHANNEL_MESSAGE(Challenge, 3, FString); // server sends client challenge string to verify integrity
DEFINE_CONTROL_CHANNEL_MESSAGE(Netspeed, 4, int32); // client sends requested transfer rate
DEFINE_CONTROL_CHANNEL_MESSAGE(Login, 5, FString, FString, FUniqueNetIdRepl, FString); // client requests to be admitted to the game
DEFINE_CONTROL_CHANNEL_MESSAGE(Failure, 6, FString); // indicates connection failure
DEFINE_CONTROL_CHANNEL_MESSAGE(Join, 9); // final join request (spawns PlayerController)
DEFINE_CONTROL_CHANNEL_MESSAGE(JoinSplit, 10, FString, FUniqueNetIdRepl); // child player (splitscreen) join request
DEFINE_CONTROL_CHANNEL_MESSAGE(Skip, 12, FGuid); // client request to skip an optional package
DEFINE_CONTROL_CHANNEL_MESSAGE(Abort, 13, FGuid); // client informs server that it aborted a not-yet-verified package due to an UNLOAD request
DEFINE_CONTROL_CHANNEL_MESSAGE(PCSwap, 15, int32); // client tells server it has completed a swap of its Connection->Actor
DEFINE_CONTROL_CHANNEL_MESSAGE(ActorChannelFailure, 16, int32); // client tells server that it failed to open an Actor channel sent by the server (e.g. couldn't serialize Actor archetype)
DEFINE_CONTROL_CHANNEL_MESSAGE(DebugText, 17, FString); // debug text sent to all clients or to server
DEFINE_CONTROL_CHANNEL_MESSAGE(NetGUIDAssign, 18, FNetworkGUID, FString); // Explicit NetworkGUID assignment. This is rare and only happens if a netguid is only serialized client->server (this msg goes server->client to tell client what ID to use in that case)
DEFINE_CONTROL_CHANNEL_MESSAGE(SecurityViolation, 19, FString); // server tells client that it has violated security and has been disconnected
DEFINE_CONTROL_CHANNEL_MESSAGE(GameSpecific, 20, uint8, FString); // custom game-specific message routed to UGameInstance for processing
DEFINE_CONTROL_CHANNEL_MESSAGE(EncryptionAck, 21);
DEFINE_CONTROL_CHANNEL_MESSAGE(DestructionInfo, 22);
DEFINE_CONTROL_CHANNEL_MESSAGE(CloseReason, 23, FString); // Reason for client NetConnection Close, for analytics/logging
DEFINE_CONTROL_CHANNEL_MESSAGE(NetPing, 24, ENetPingControlMessage /* MessageType */, FString /* MessageStr */);

// 			Beacon control channel flow
// Client												Server
//	Send<Hello>
//														Receive<Hello> - compare version / game id
//															Send<Upgrade> if incompatible
//															Send<Failure> if wrong game
//															Send<BeaconWelcome> if good so far
//	Receive<BeaconWelcome>
//		Send<NetSpeed>
//		Send<BeaconJoin> with beacon type
//														Receive<Netspeed>
//														Receive<BeaconJoin> - create requested beacon type and create NetGUID
//															Send<Failure> if unable to create or bad type
//															Send<BeaconAssignGUID> with NetGUID for new beacon actor
//	Receive<BeaconAssignGUID> - assign NetGUID to client actor
//		Send<BeaconNetGUIDAck> acknowledging receipt of NetGUID
//														Receive<BeaconNetGUIDAck> - connection complete

DEFINE_CONTROL_CHANNEL_MESSAGE(BeaconWelcome, 25); // server tells client they're ok to attempt to join (client sends netspeed/beacontype)
DEFINE_CONTROL_CHANNEL_MESSAGE(BeaconJoin, 26, FString, FUniqueNetIdRepl);  // server tries to create beacon type requested by client, sends NetGUID for actor sync
DEFINE_CONTROL_CHANNEL_MESSAGE(BeaconAssignGUID, 27, UE::Net::FNetIDVariant); // client assigns NetGUID from server to beacon actor, sends NetGUIDAck
DEFINE_CONTROL_CHANNEL_MESSAGE(BeaconNetGUIDAck, 28, FString); // server received NetGUIDAck from client, connection established successfully

DEFINE_CONTROL_CHANNEL_MESSAGE(IrisProtocolMismatch, 29, uint64); // client has a different protocol hash from the server
DEFINE_CONTROL_CHANNEL_MESSAGE(IrisNetRefHandleError, 30, UE::Net::ENetRefHandleError, uint64, TArray<uint64>); // a specific handle caused an error and we want the remote connection to log all information it has on it

// Called in place of Join or JoinSplit during connection handshake, and will created a connection that has the following properties:
//	* Won't spawn an instance of APawn.
//	* Won't call AGameModeBase::Login().
//	* Spawns ANoPawnPlayerController instead of using the class in AGameMode::PlayerControllerClass.
//	* Network relevency will be based on the location of ANoPawnPlayerController.
//
// Relevancy: Since network relevancy will be based on the location of ANoPawnPlayerController, it's important to set the location of
//			  the actor using the RPC ANoPawnPlayerController::SetViewTargetPosition(), either from the server or client using this connection.
//
// These messages are only intended to be used when called from one game server to another game server. For this reason it is disabled
// by default and must be explicitly enabled on the receiving server using the cvar net.EnableSupportForNoPawnConnection or calling
// UNetDriver::SetSupportForNoPawnConnection() directly.
DEFINE_CONTROL_CHANNEL_MESSAGE(JoinNoPawn, 31);
DEFINE_CONTROL_CHANNEL_MESSAGE(JoinNoPawnSplit, 32, FString, FUniqueNetIdRepl);

// Close a child connection remotely without closing all other child and parent connections.
//
// The argument to this message is the value of NetPlayerIndex associated with the child connection that must be closed.
//
// It is expected that the local child connection will be cleaned up manually and does not depend on this message being 
// successfully processed by the remote server.
DEFINE_CONTROL_CHANNEL_MESSAGE(CloseChildConnection, 34, int32);

// a specific handle caused an error and we want the remote connection to log all information it has on it
DEFINE_CONTROL_CHANNEL_MESSAGE(IrisNetRefHandleErrorWithDiagnosticMessage, 35, UE::Net::ENetRefHandleError, uint64, TArray<uint64>, FString);

// Protocol mismatch with client CDO state data for server-side comparison
DEFINE_CONTROL_CHANNEL_MESSAGE(IrisProtocolMismatchWithCDOState, 36, uint64, TArray<uint8>);