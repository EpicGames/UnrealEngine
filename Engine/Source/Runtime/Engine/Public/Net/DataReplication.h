// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DataReplication.h:
	Holds classes for data replication (properties and RPCs).
=============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "Misc/NetworkGuid.h"
#include "UObject/CoreNet.h"
#include "Engine/EngineTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/GCObject.h"
#include "HAL/LowLevelMemTracker.h"

LLM_DECLARE_TAG_API(NetObjReplicator, ENGINE_API);

class FNetFieldExportGroup;
class FOutBunch;
class FRepChangelistState;
class FRepLayout;
class FRepState;
class FSendingRepState;
class UNetConnection;
class UNetDriver;
class AActor;

bool inline IsCustomDeltaProperty(const FStructProperty* StructProperty)
{
	return EnumHasAnyFlags(StructProperty->Struct->StructFlags, STRUCT_NetDeltaSerializeNative);
}

bool inline IsCustomDeltaProperty(const FProperty* Property)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	return StructProperty && IsCustomDeltaProperty(StructProperty);
}

/** struct containing property and offset for replicated actor properties */
struct FReplicatedActorProperty
{
	/** offset into the Actor where this reference is located - includes offsets from any outer structs */
	int32 Offset;

	/** Reference to property object */
	const class FObjectPropertyBase* Property;

	FReplicatedActorProperty(int32 InOffset, const FObjectPropertyBase* InProperty)
		: Offset(InOffset)
		, Property(InProperty)
	{}
};

/**
 * Represents an object that is currently being replicated or handling RPCs.
 *
 *
 *
 *		|----------------|
 *		| NetGUID ObjRef |
 * 		|----------------|
 *      |                |		
 *		| Properties...  |
 *		|                |	
 *		| RPCs...        |
 *      |                |
 *      |----------------|
 *		| </End Tag>     |
 *		|----------------|
 *	
 */
class FObjectReplicator
{
public:

	enum class ESkipRpcBehavior
	{
		// Always execute RPCs immediately
		NeverSkip,

		// Skip RPC execution only if it's not ready to execute
		SkipIfNotReady,

		// Always skip execution of RPC (do validation only, caller will handle execution)
		AlwaysSkip
	};

	ENGINE_API FObjectReplicator();
	ENGINE_API ~FObjectReplicator();

	struct FRPCCallInfo 
	{
		FName FuncName;
		int32 Calls;
		double LastCallTimestamp;
	};

	struct FRPCPendingLocalCall
	{
		/** Index to the RPC that was delayed */
		int32 RPCFieldIndex;

		/** Flags this was replicated with */
		FReplicationFlags RepFlags;

		/** Buffer to serialize RPC out of */
		TArray<uint8> Buffer;

		/** Number of bits in buffer */
		int64 NumBits;

		/** The replication frame number when the RPC was queued. Used to track the latency caused by the delay. */
		uint32 FrameQueuedAt;

		/** Guids being waited on */
		TSet<FNetworkGUID> UnmappedGuids;

		FRPCPendingLocalCall(
			const FFieldNetCache* InRPCField,
			const FReplicationFlags& InRepFlags,
			FNetBitReader& InReader,
			uint32 InReplicationFrame,
			const TSet<FNetworkGUID>& InUnmappedGuids)
			: RPCFieldIndex(InRPCField->FieldNetIndex)
			, RepFlags(InRepFlags)
			, Buffer(InReader.GetBuffer())
			, NumBits(InReader.GetNumBits())
			, FrameQueuedAt(InReplicationFrame)
			, UnmappedGuids(InUnmappedGuids)
		{}

		void CountBytes(FArchive& Ar) const;
	};

	ENGINE_API void InitWithObject(
		UObject* InObject,
		UNetConnection* InConnection,
		bool bUseDefaultState = true);

	ENGINE_API void CleanUp();

	ENGINE_API void StartReplicating(class UActorChannel* InActorChannel);
	ENGINE_API void StopReplicating(class UActorChannel* InActorChannel);

	/** Recent/dirty related functions */
	ENGINE_API void InitRecentProperties(uint8* Source);

	/** Takes Data, and compares against shadow state to log differences */
	ENGINE_API bool ValidateAgainstState(const UObject* ObjectState);

	//~ Both of these should be private, IMO, but we'll leave them public for now for back compat
	//~ in case anyone was using SerializeCustomDeltaProperty already.
	ENGINE_API bool SendCustomDeltaProperty(
		UObject* InObject,
		uint16 CustomDeltaProperty,
		FNetBitWriter& OutBunch,
		TSharedPtr<INetDeltaBaseState>& NewFullState,
		TSharedPtr<INetDeltaBaseState>& OldState);

	/** Packet was dropped */
	ENGINE_API void ReceivedNak(int32 NakPacketId);

	ENGINE_API void CountBytes(FArchive& Ar) const;

	ENGINE_API void ReplicateCustomDeltaProperties(FNetBitWriter& Bunch, FReplicationFlags RepFlags, bool& bSkippedPropertyCondition);
	ENGINE_API bool ReplicateProperties(FOutBunch& Bunch, FReplicationFlags RepFlags);
	ENGINE_API bool ReplicateProperties(FOutBunch& Bunch, FReplicationFlags RepFlags, FNetBitWriter& Writer);
	ENGINE_API bool ReplicateProperties_r(FOutBunch& Bunch, FReplicationFlags RepFlags, FNetBitWriter& Writer);

	ENGINE_API void PostSendBunch(FPacketIdRange& PacketRange, uint8 bReliable);

	ENGINE_API bool ReceivedBunch(
		FNetBitReader& Bunch,
		const FReplicationFlags& RepFlags,
		const bool bHasRepLayout,
		bool& bOutHasUnmapped);

	/**
	 * @param Reader the buffer that contains the RPC (this function reads RPC parameters from the current position)
	 * @param OutUnmappedGuids any GUIDs that are not yet resolvable because we do not have the data (yet)
	 * @param bOutSkippedRpcExec did we skip execution of this RPC? If so, the caller must handle execution (e.g. delay).
	 * @param RepFlags the replication flags (state) of the replicated object
	 * @param FieldCache the RPC FieldCache used to speed-up property lookups.  This cannot be null.
	 * @param SkipRpcBehavior under what circumstances should we skip the execution of the RPC?
	 *
	 * @return True if the Reader is still valid. False if the Reader should be abandoned.
	 * @see ReceivedBunch(), UpdateUnmappedObjects()
	 */
	ENGINE_API bool ReceivedRPC(
		FNetBitReader& Reader,
		TSet<FNetworkGUID>& OutUnmappedGuids,
		bool& bOutSkippedRpcExec,
		const FReplicationFlags& RepFlags,
		const FFieldNetCache* FieldCache,
		ESkipRpcBehavior SkipRpcBehavior);

	inline bool ReceivedRPC(
		FNetBitReader& Reader,
		const FReplicationFlags& RepFlags,
		const FFieldNetCache* FieldCache,
		const bool bCanDelayRPC,
		bool& bOutDelayRPC,
		TSet<FNetworkGUID>& OutUnmappedGuids)
	{
		return ReceivedRPC(Reader, OutUnmappedGuids, bOutDelayRPC, RepFlags, FieldCache, bCanDelayRPC ? ESkipRpcBehavior::SkipIfNotReady : ESkipRpcBehavior::NeverSkip);
	}

	ENGINE_API void UpdateGuidToReplicatorMap();
	ENGINE_API bool MoveMappedObjectToUnmapped(const FNetworkGUID& GUID);
	ENGINE_API void PostReceivedBunch();

	ENGINE_API void ForceRefreshUnreliableProperties();

	ENGINE_API void QueueRemoteFunctionBunch(UFunction* Func, FOutBunch &Bunch);

	ENGINE_API bool ReadyForDormancy(bool bDebug=false);

	ENGINE_API void StartBecomingDormant();

	ENGINE_API void CallRepNotifies(bool bSkipIfChannelHasQueuedBunches) const;

	ENGINE_API void UpdateUnmappedObjects(bool& bOutHasMoreUnmapped);

	/**
	 * Enqueue any pending RPCs meant to be done within an AutoRTFM transaction.
	 * @see UNetDriver::PostTickDispatch
	 */
	ENGINE_API void EnqueuePendingTransactionalRPCs();

	inline TWeakObjectPtr<UObject>	GetWeakObjectPtr() const
	{
		return WeakObjectPtr;
	}

	inline UObject* GetObject() const
	{
		// If this replicator is dormant we have released our strong ref but the object may still be alive.
		return ObjectPtr ? ObjectPtr : WeakObjectPtr.Get();
	}

	inline void SetObject(UObject* NewObj)
	{
		ObjectPtr = NewObj;
		WeakObjectPtr = NewObj;
	}

	void ReleaseStrongReference()
	{
		ObjectPtr = nullptr;
	}

	inline void PreNetReceive()
	{ 
		UObject* Object = GetObject();
		if ( Object != NULL )
		{
			Object->PreNetReceive();
		}
	}

	inline void PostNetReceive()
	{ 
		UObject* Object = GetObject();
		if ( Object != NULL )
		{
			Object->PostNetReceive();
		}
	}

	ENGINE_API void QueuePropertyRepNotify(
		UObject* Object,
		FProperty* Property,
		const int32 ElementIndex,
		TArray<uint8>& MetaData);
		
	ENGINE_API void WritePropertyHeaderAndPayload(
		UObject* Object,
		FProperty*				Property,
		FNetFieldExportGroup* NetFieldExportGroup,
		FNetBitWriter& Bunch,
		FNetBitWriter& Payload) const;	

	/**
	 * @return True if we've determined nothing needs to be updated / resent by the replicator, meaning
	 *			we can safely skip updating it this frame.
	 */
	ENGINE_API bool CanSkipUpdate(FReplicationFlags Flags);

	bool IsDirtyForReplay() const { return bDirtyForReplay; }
	void ResetReplayDirtyTracking() { bDirtyForReplay = true; }

public:

	/** Net GUID for the object we're replicating. */
	FNetworkGUID ObjectNetGUID;

	/** The amount of memory (in bytes) that we're using to track Unmapped GUIDs. */
	int32 TrackedGuidMemoryBytes;

	/** True if last update (ReplicateActor) produced no replicated properties */
	uint32 bLastUpdateEmpty : 1;

	/** Whether or not the Actor Channel on which we're replicating has been Opened / Acked by the receiver. */
	uint32 bOpenAckCalled : 1;

	/** True if we need to do an unmapped check next frame. */
	uint32 bForceUpdateUnmapped : 1;

	/** Whether or not we've already replicated properties this frame. */
	uint32 bHasReplicatedProperties : 1;
	
	/** Is this a subobject replicator and have we sent the creation data yet. */
    uint32 bSentSubObjectCreation : 1;

private:

	/** Whether or not we are going to use Fast Array Delta Struct Delta Serialization. See FFastArraySerializer::FastArrayDeltaSerialize_DeltaSerializeStructs. */
	uint32 bSupportsFastArrayDelta : 1;

	/**
	 * Whether or not this object replicator is eligible to skip replication calls based on
	 * simple flag checks.
	 */
	uint32 bCanUseNonDirtyOptimization : 1;

	/** Used to track if we've replicated this object into a checkpoint */
	uint32 bDirtyForReplay : 1;

public:
	
	TSharedPtr<class FReplicationChangelistMgr> ChangelistMgr;
	TSharedPtr<FRepLayout> RepLayout;
	TUniquePtr<FRepState>  RepState;
	TUniquePtr<FRepState> CheckpointRepState;

	TObjectPtr<UClass> ObjectClass;

	UObject* ObjectPtr;

	/** Connection this replicator was created on. */
	UNetConnection* Connection;

	/** The Actor Channel that we're replicating on. This expected to be owned by Connection. */
	class UActorChannel* OwningChannel;

	FOutBunch* RemoteFunctions;

	/** Meta information on pending net RPCs (to be sent) */
	TArray<FRPCCallInfo> RemoteFuncInfo;

	/** Information on RPCs that have been received but not yet executed */
	TArray<FRPCPendingLocalCall> PendingLocalRPCs;

	//~ RepNotify properties were moved to FReceivingRepState.
	//~ CustomDeltaState properties were moved to FSendingRepState.
	//~ Retirement properties were moved to FSendingRepState.

	TSet<FNetworkGUID> ReferencedGuids;

private:
	TWeakObjectPtr<UObject> WeakObjectPtr;

	/**
	 * These are RPCs that are saved so that they may be executed transactionally at a later time
	 * @see EnqueuePendingTransactionalRPCs
	 */
	TArray<FRPCPendingLocalCall> PendingTransactionalRPCs;

	static void CallProcessEventForReceivedRPC(UObject* Object, UFunction* Function, uint8* Params);
};

class FScopedActorRoleSwap
{
public:
	ENGINE_API FScopedActorRoleSwap(AActor* InActor);
	ENGINE_API ~FScopedActorRoleSwap();

	FScopedActorRoleSwap(const FScopedActorRoleSwap&) = delete;
	FScopedActorRoleSwap& operator=(const FScopedActorRoleSwap&) = delete;

	FScopedActorRoleSwap(FScopedActorRoleSwap&& Other)
	{
		Actor = Other.Actor;
		Other.Actor = nullptr;
	}
	FScopedActorRoleSwap& operator=(FScopedActorRoleSwap&& Other)
	{
		Actor = Other.Actor;
		Other.Actor = nullptr;
		return *this;
	}

private:
	AActor* Actor;
};
