// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Net/Core/Connection/NetResultManager.h"


namespace UE
{
namespace Net
{

// Forward Declarations
class FNetConnectionFaultRecoveryBase;
class FNetConnectionFaultRecovery;

/**
 * Default Fault Handler
 *
 * Implements default fault handling, for all Engine level fault types.
 *
 * To add fault handling for custom netcode (e.g. new NetConnection's or PacketHandler components),
 * create a new fault handler added to NetConnection.FaultManager, and use a custom enum with TNetResult (see DefaultFaultHandler/OodleFaultHandler).
 */
class FDefaultFaultHandler final : public FNetResultHandler
{
	friend FNetConnectionFaultRecovery;

private:
	void InitFaultRecovery(FNetConnectionFaultRecoveryBase* InFaultRecovery);

	virtual EHandleNetResult HandleNetResult(FNetResult&& InResult) override;


private:
	/** The NetConnection FaultRecovery instance, which we forward fault counting to */
	FNetConnectionFaultRecoveryBase* FaultRecovery = nullptr;
};

}
}

