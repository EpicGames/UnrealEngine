// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Engine/EngineTypes.h"
#include "UObject/ObjectPtr.h"
#include "EngineReplicationBridge.generated.h"

class UNetDriver;
class UIrisObjectReferencePackageMap;
struct FAnalyticsEventAttribute;

class UActorComponent;
class UWorld;

namespace UE::Net
{
	enum class ENetRefHandleError : uint32;

	class FDeferredReplicationSystemCalls;
}

namespace UE::Net
{

// If actor should be replicated using IRIS or old replication system
ENGINE_API bool ShouldUseIrisReplication(const UObject* Actor);

// Currently just a direct mapping of EEEndPlayReason::Type but we might want to add more specific reasons later on
enum class EStopReplicatingReason : uint32
{
	/** When the Actor or Component is explicitly destroyed. */
	Destroyed = 0,
	/** When the world is being unloaded for a level transition. */
	LevelTransition,
	/** When the world is being unloaded because PIE is ending. */
	EndPlayInEditor,
	/** When the level it is a member of is streamed out. */
	RemovedFromWorld,
	/** When the application is being exited. */
	Quit,
	/** An object is removed from a server but the client should keep it and maintain its internal data, expecting to receive object updates from another server later. Only supported by clients with replication systems configured to use UE::Net::EProxyType::Backend. */
	ProxyReuse,
};

ENGINE_API const TCHAR* LexToString(EStopReplicatingReason Reason);

}

struct FActorReplicationParams
{
	enum EFilterType
	{
		/** Let the config filter configs assign a filter based on the class type. */
		ConfigFilter = 0,
		/** When set don't assign any dynamic filter and default to being always relevant. */
		AlwaysRelevant,
		/** When set use the default spatial filter of the bridge. Generally that is the NetObjectGridFilter. */
		DefaultSpatial,
		/** When set use filter defined by ExplicitDynamicFilterName. */
		ExplicitFilter,
	};

	/**
	 * The default behavior for actors (e.g. ConfigFilter) is that they are automatically assigned a filter based on the class type via the engine config and UObjectReplicationBridgeConfig::FilterConfigs.
	 * Choosing a different option allows you to ignore the automatic assignment and select a specific filter for the replicated actor.
	 * @see FObjectReplicationBridgeFilterConfig
	 */
	EFilterType FilterType = ConfigFilter;

	/** Only used when ExplicitFilter is the type used. The dynamic filter to assign to this actor. */
	FName ExplicitDynamicFilterName;

	/** Optional  factory name if the actor is not using the default NetActorFactory */
	FName NetFactoryName;
};

struct FStopReplicatingActorParams
{
	explicit ENGINE_API FStopReplicatingActorParams(EEndPlayReason::Type);

	UE::Net::EStopReplicatingReason StopReplicatingReason = UE::Net::EStopReplicatingReason::Destroyed;

	//$TODO: Remove when we can remove EndReplication
	EEndPlayReason::Type EndPlayReason = EEndPlayReason::Type::Destroyed;
};

UCLASS(Transient, MinimalAPI, config=Engine)
class UEngineReplicationBridge final : public UObjectReplicationBridge
{
	GENERATED_BODY()

public:
	ENGINE_API UEngineReplicationBridge();
	virtual ENGINE_API ~UEngineReplicationBridge() override;

	ENGINE_API static UEngineReplicationBridge* Create(UNetDriver* NetDriver);

	/** Sets the net driver for the bridge. */
	ENGINE_API void SetNetDriver(UNetDriver* const InNetDriver);
	
	/** Get net driver used by the bridge .*/
	inline UNetDriver* GetNetDriver() const { return NetDriver; }

	/** Begin replication of an actor and its registered ActorComponents and SubObjects. */
	ENGINE_API FNetRefHandle StartReplicatingActor(AActor* Instance);

	/** Stop replicating an actor. This will destroy the handle of the actor and those of it's compponents and subobjects. */
	ENGINE_API void StopReplicatingActor(AActor* Actor, const FStopReplicatingActorParams& Params);

	/** Convert StopReplication reason to EndReplicationFlags */
	ENGINE_API EEndReplicationFlags ConvertToEndReplicationFlags(UE::Net::EStopReplicatingReason StopReplicatingReason) const;
		
	/**
	 * Begin replication of an ActorComponent and its registered SubObjects, 
	 * if the ActorComponent already is replicated any set NetObjectConditions will be updated.
	*/
	ENGINE_API FNetRefHandle StartReplicatingComponent(FNetRefHandle RootObjectHandle, UActorComponent* ActorComponent);

	/** Begin replication of a subobject. */
	ENGINE_API FNetRefHandle StartReplicatingSubObject(UObject* SubObject, const FSubObjectReplicationParams& Params);

	/** Stop replicating an ActorComponent and its associated SubObjects. */
	ENGINE_API void StopReplicatingComponent(UActorComponent* ActorComponent, EEndReplicationFlags EndReplicationFlags = EEndReplicationFlags::None);

	/** Get object reference packagemap. Used in special cases where serialization hasn't been converted to use NetSerializers.  */
	UIrisObjectReferencePackageMap* GetObjectReferencePackageMap() const { return ObjectReferencePackageMap; }

	/** Tell the remote connection that we detected a reading error with a specific replicated object */
	ENGINE_API virtual void SendErrorWithNetRefHandle(UE::Net::ENetRefHandleError ErrorType, FNetRefHandle RefHandle, uint32 ConnectionId, TConstArrayView<const FNetRefHandle> ExtraNetRefHandle = {}, const UE::Net::FNetErrorContext& ErrorContext = UE::Net::FNetErrorContext()) override;

	/** Add the rootobject to the level's filter group so it will only be relevant if the connection has that level streamed in. */
	UE_DEPRECATED(5.8, "Call AddRootObjectToContainerGroup instead.")
	ENGINE_API void AddRootObjectToLevelGroup(const UObject* RootObject, const ULevel* Level);

	/** Add the root object to the container's filter group so it will only be relevant if the connection has that container streamed in. */
	ENGINE_API void AddRootObjectToContainerGroup(const UObject* RootObject, const UObject* Container);
	
	/** Updates the level group for an actor that changed levels */
	void ActorChangedLevel(const AActor* Actor, const ULevel* PreviousLevel);

	/** Called when NetUpdateFrequency has changed on the Actor. */
	void OnNetUpdateFrequencyChanged(const AActor* Actor);

	void WakeUpObjectInstantiatedFromRemote(AActor* Actor) const;

	/**
	 * Add relevant network metrics gathered since the last call to ConsumeNetMetrics.
	 * Any periodic stat will be reset here too.
	 * @param OutAttrs A list of Name/Value pairings that will be sent to an AnalyticsProvider
	 */
	ENGINE_API void ConsumeNetMetrics(TArray<FAnalyticsEventAttribute>& OutAttrs);

	/** Access to the factory id that handles actors */
	UE::Net::FNetObjectFactoryId GetActorFactoryId() const { return ActorFactoryId; }

	/** Access to the factory id that handles subobjects */
	UE::Net::FNetObjectFactoryId GetSubObjectFactoryId() const { return SubObjectFactoryId; }
	
	/** Return true if AActor::PreReplication() will be called on replicated objects (default: true). */
	bool AllowExecutePreReplication() const;

protected:
	// UReplicationBridge
	virtual void OnPreSeamlessTravelGarbageCollect() override;

	// UObjectReplicationBridge
	virtual void Initialize(UReplicationSystem* ReplicationSystem) override;
	virtual void Deinitialize() override;
	virtual void GetInitialDependencies(FNetRefHandle Handle, FNetDependencyInfoArray& OutDependencies) const override;
	virtual bool RemapPathForPIE(uint32 ConnectionId, FString& Path, bool bReading) const override;
	virtual bool ObjectContainerHasFinishedLoading(UObject* Object) const override;
	virtual void OnProtocolMismatchDetected(FNetRefHandle ObjectHandle, const TArray<uint8>& CDOStateBytes) override;
	virtual void OnProtocolMismatchReported(FNetRefHandle RefHandle, uint32 ConnectionId, const TArray<uint8>& ClientCDOStateBytes) override;
	virtual bool CanCreateDestructionInfo() const override;

	/** Returns true if the class is derived from Actor and its CDO has set bReplicates. */
	virtual bool IsClassReplicatedByDefault(const UClass* Class) const;

	[[nodiscard]] virtual FString PrintConnectionInfo(uint32 ConnectionId) const override;

private:

	/** Only send one NMT_IrisProtocolMismatchWithCDOState message per client to avoid spamming the server. */
	bool bProtocolMismatchMessageSent = false;

	/** Tracks which replication protocol ids have already had an NMT_IrisNetRefHandleErrorWithDiagnosticMessage sent for them, so we send at most one diagnostic message per protocol id. */
	TSet<UE::Net::FReplicationProtocolIdentifier> SentDiagnosticProtocolIds;

	/** Number of NMT_IrisNetRefHandleErrorWithDiagnosticMessage messages sent. Capped by net.Iris.MaxNetRefHandleErrorDiagnosticMessages. */
	int32 NumSentDiagnosticMessages = 0;

	void OnMaxTickRateChanged(UNetDriver* InNetDriver, int32 NewMaxTickRate, int32 OldMaxTickRate);

	/** Return true if actors that have a local role other than ROLE_Authority can be replicated to clients (default: false). */
	bool AllowReplicationOfActorsWithAnyRole() const;

	/** Cancel a start or stop replicating request for an actor. */
	void CancelDeferredActor(AActor* Actor);

	/** Cancel a start or stop replicating request for a component. */
	void CancelDeferredComponent(UActorComponent* Component);

	/** 
	 * Deferred versions of the following functions:
	 *	StartReplicatingActor()
	 *	StopReplicatingActor()
	 *	StartReplicatingComponent()
	 *	StopReplicatingComponent()
	 *
	 * These functions will be deferred until FlushDeferred() is called.
	 * 
	 * For any given actor or component, start can only be called if the object isn't already registered
	 * in the replication system or stop has been previously deferred; and stop can only be called if the
	 * object is already registered in the replication system or start has been previously deferred.
	 *
	 * When FlushDeferred() is called, two calls to start and stop for the same actor or component will
	 * cancel out so neither action is performed.
	 * 
	 * If StopReplicatingActorDefer() gets called in FlushDeferred() then it will use the value of 
	 * FStopReplicatingActorParams from the last call to StopReplicatingActorDefer().
	 * 
	 * If StartReplicatingComponentDefer() gets called in FlushDeferred() then it will use the value of
	 * AActor* from the last call to StartReplicatingComponentDefer().
	 * 
	 * E.g. 
	 * 	Assume these functions are called in order:
	 *		StopReplicatingActorDefer(MyActor, StopParams1)
	 *		StartReplicatingActorDefer(MyActor)
	 *		StopReplicatingActorDefer(MyActor, StopParams2)
	 * 
	 * When calling FlushDeferred() then StopReplicatingActor() will be called for MyActor since it 
	 * has the greater call count, and it will use the argument StopParams2 because that was the argument
	 * from the last call to StopReplicatingActorDefer().
	 *
	 * The order of deferred calls is not currently guaranteed when processed in FlushDeferred().
	 */
	void StartReplicatingActorDefer(AActor* Instance);
	void StopReplicatingActorDefer(AActor* Instance, const FStopReplicatingActorParams& Params);
	void StartReplicatingComponentDefer(const UObject* RootObject, UActorComponent* Component);
	void StopReplicatingComponentDefer(UActorComponent* Component, EEndReplicationFlags EndReplicationFlags);
	
	/** Process all deferred function calls. */
	void FlushDeferred();

private:

	friend class UE::Net::FDeferredReplicationSystemCalls;

	UE::Net::FNetObjectFactoryId ActorFactoryId;
	UE::Net::FNetObjectFactoryId SubObjectFactoryId;

	UNetDriver* NetDriver = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UIrisObjectReferencePackageMap> ObjectReferencePackageMap = nullptr;

	UPROPERTY(Config)
	/** Name of the NetObjectFactory to use for default replicated actors */
	FName ActorFactoryName;

	UPROPERTY(Config)
	/** Name of the NetObjectFactory to use for default replicated subobjects */
	FName SubObjectFactoryName;

	bool bAllowExecutePreReplication = true;
	bool bAllowReplicationOfActorsWithAnyRole = false;

	struct FDeferredStartStop
	{
		/** 
		 * Used to determine if start, stop or nothing should be called for an actor or component.
		 * 
		 *  < 0		Stop is called
		 * == 0		Nothing happens
		 *  > 0		Start is called
		 */
		int32 Count = 0;

		/** The parameters passed into the last call to StopReplicatingActorDefer(). */
		FStopReplicatingActorParams StopReplicatingActorParams = FStopReplicatingActorParams(EEndPlayReason::Type::Destroyed);

		/** The end replication flags passed into the last call to StopReplicatingComponentDefer(). */
		EEndReplicationFlags StopReplicatingComponentFlags = EEndReplicationFlags::None;

		/** The root/parent UObject used by the component passed into the last call to StartReplicatingComponentDefer(). */
		TObjectKey<UObject> RootObject;
	};

	/** A flag used to prevent adding or removing new deferred start and stops while flushing. */
	bool bIsFlushDeferredRunning = false;

	TMap<TObjectKey<AActor>, FDeferredStartStop> DeferredStartStopCallsActors;
	TMap<TObjectKey<UActorComponent>, FDeferredStartStop> DeferredStartStopCallsComponents;
};
