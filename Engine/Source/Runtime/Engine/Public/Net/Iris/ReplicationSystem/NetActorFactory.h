// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetObjectFactory.h"

#include "Iris/Core/NetObjectReference.h"

#include "NetActorFactory.generated.h"

namespace UE::Net
{
	enum class EActorNetSpawnInfoFlags : uint32
	{
		None = 0U,
		QuantizeScale = 1U,
		QuantizeLocation = QuantizeScale << 1U,
		QuantizeVelocity = QuantizeLocation << 1U,
	};
	ENUM_CLASS_FLAGS(EActorNetSpawnInfoFlags)
}

class UNetActorFactory;


namespace UE::Net
{

/**
 * Header information to be able to tell if its a dynamic or static header
 */
class FBaseActorNetCreationHeader : public FNetObjectCreationHeader
{
public:
	/** Is the header representing a dynamic or stable actor */
	virtual bool IsDynamic() const = 0;
	/** Is the header representing a dynamic pre-registered actor */
	virtual bool IsPreregistered() const = 0;

	ENGINE_API EActorNetSpawnInfoFlags GetFactorySpawnFlags(const UNetActorFactory* ActorFactory) const;

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context);

public:

	TArray<uint8> CustomCreationData;
	uint16 CustomCreationDataBitCount = 0;
};

/**
 * Header information representing static actors
 */
class FStaticActorNetCreationHeader : public FBaseActorNetCreationHeader
{
private:

	typedef FBaseActorNetCreationHeader Super;

public:

	virtual bool IsDynamic() const
	{
		return false;
	}

	virtual bool IsPreregistered() const
	{
		return false;
	}

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const override;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context) override;

	ENGINE_API virtual FString ToString() const override;

public:

	FNetObjectReference ObjectReference;
};


/**
 * Header information representing dynamic actors
 */
class FDynamicActorNetCreationHeader : public FBaseActorNetCreationHeader
{
private:

	typedef FBaseActorNetCreationHeader Super;

public:

	struct FActorNetSpawnInfo
	{
		FActorNetSpawnInfo()
		: Location(EForceInit::ForceInitToZero)
		, Rotation(EForceInit::ForceInitToZero)
		, Scale(FVector::OneVector)
		, Velocity(EForceInit::ForceInitToZero)
		{}

		FVector Location;
		FRotator Rotation;
		FVector Scale;
		FVector Velocity;
	};

public:

	virtual bool IsDynamic() const override
	{ 
		return true;
	}

	virtual bool IsPreregistered() const
	{
		return false;
	}

	ENGINE_API virtual FString ToString() const override;

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const override;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context) override;

public:

	FActorNetSpawnInfo SpawnInfo;

	FNetObjectReference ArchetypeReference;
	FNetObjectReference LevelReference; // Only when bUsePersistentLevel is false
	
	bool bUsePersistentLevel = false;
	bool bIsPreRegistered = false;
};

/**
 * Header information representing dynamic pre-registered actors
 * @See UObjectReplicationBridge::PreRegisterNewObjectReferenceHandle
 */
class FPreRegisteredActorNetCreationHeader : public FBaseActorNetCreationHeader
{
private:

	typedef FBaseActorNetCreationHeader Super;

public:

	virtual bool IsDynamic() const override
	{ 
		return true;
	}

	virtual bool IsPreregistered() const
	{
		return true;
	}

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const override;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context) override;

	ENGINE_API virtual FString ToString() const override;
};

} // end namespace UE::Net

/**
 * Responsible for creating headers allowing remote factories to spawn replicated actors
 */
UCLASS(MinimalAPI)
class UNetActorFactory : public UNetObjectFactory
{
	GENERATED_BODY()

public:

	ENGINE_API static FName GetFactoryName();

public:

	ENGINE_API virtual void OnInit() override;

	ENGINE_API virtual FInstantiateResult InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	ENGINE_API virtual void PostInstantiation(const FPostInstantiationContext& Context) override;

	ENGINE_API virtual void PostInit(const FPostInitContext& Context) override;

	ENGINE_API virtual void SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated) override;

	ENGINE_API virtual void SubObjectDetachedFromReplication(const FDetachContext& Context, const FSubObjectDetachContext& SubObjectContext) override;
	ENGINE_API virtual void DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext) override;

	ENGINE_API virtual TOptional<FWorldInfoData> GetWorldInfo(const FWorldInfoContext& Context) const override;

protected:

	ENGINE_API virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) override;

	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndFillHeader(UE::Net::FNetRefHandle Handle) override;
	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context) override;

	ENGINE_API virtual bool SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header)  override;

protected:

	/** Fill the required header information */
	ENGINE_API bool FillHeader(UE::Net::FBaseActorNetCreationHeader* BaseHeader, UE::Net::FNetRefHandle Handle, AActor* Actor);

public:

	/** Cache of the default values of the spawn info to delta serialize against */
	const UE::Net::FDynamicActorNetCreationHeader::FActorNetSpawnInfo DefaultSpawnInfo;

protected:

	friend UE::Net::FBaseActorNetCreationHeader;

	/** Cache of the spawn info flags created from the net.QuantizeActor cvars */
	UE::Net::EActorNetSpawnInfoFlags SpawnInfoFlags;

};



