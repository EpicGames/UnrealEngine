// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"

#include "Misc/NotNull.h"

#include "Iris/ReplicationSystem/NetRefHandle.h"

class UWorld;
class ULevel;
namespace UE::Net
{
	struct FRootObjectReplicationParamsContext;
	struct FRootObjectReplicationParams;
}

namespace UE::Net
{

struct FRootObjectSettings
{
	/** Set this true so the object will be relevant to everyone. */
	bool bIsAlwaysRelevant = false;

	/** Set this true when you don't want the root object to be relevant to anyone by default. Useful when you want to make the object relevant only via another root object. */
	bool bIsNotRouted = false;

	/** When set to true, the object will be registered/unregistered in the ReplicationSystem when it has an active dependency to another object. Use this to reduce the amount of objects handled by the replication system if the object is not always active.  */
	bool bOnlyReplicateWhenLinked = false;

	//TODO: Add bRelevantForOwner support

	/** Optional name used when the object is handled by a factory different from the NetRootObjectFactory */
	FName FactoryName;
};


/**
 * Class that is added by composition to any UObject that wants to be a replicated root object
 * 
 * This class is meant to provide an easy to use solution for quickly transforming any non-actor into an autonomous replicated object.
 * But be aware it is not made to be the optimal solution if you intend to create alot of instances of the class using it.
 * The memory overheaded of the variables in this class could be optimized away entirely if your class directly interacted with the Iris API in a similar way to what the adapter does.
 * So for a more optimal solution, it is recommended to look at this class for inspiration and instead add the minimum necessary features you need to your own base class.
 * 
 * Important: The class that wants to be replicate autonomously must immplement a INetRootObjectFactoryExtension either in the class itself or as a standalone class able to retrieve this adapter and call FillRootObjectReplicationParams on it. 
  * 
 * The expectation is then for the class to call at runtime in order:
 * 
 * 1) InitAdapter() : Initialize the adapter so it knows who to replicate
 * 2) Configure() : Configure the replication settings prior to starting replication
 * 3) Provide the ULevel the UObject is a part of via SetAttachedLevel() or in StartReplication()
 * 4) StartReplication() or RelevantWith() to add the object to the replication system
 * 5) StopReplication() or RemoveRelevantWith when the object no longer should replicate
 * 6) DeinitAdapter() before the object goes out of existing (optional)
 * 
 */
class FNetRootObjectAdapter
{
public:

	FNetRootObjectAdapter() = default;
	ENGINE_API FNetRootObjectAdapter(const FRootObjectSettings& Settings);

	/** Initialize the adapter with the object it will be responsible to replicate */
	ENGINE_API void InitAdapter(UObject* ReplicatedObject);

	/** Deinitialize the adapter and clear references */
	ENGINE_API void DeinitAdapter();

	/** 
	 * Configure how the object will be replicated. @See FRootObjectSettings for available options. 
	 * The default behavior is the object will be Always Relevant.
	 */
	ENGINE_API void Configure(const FRootObjectSettings& Settings);

	enum class ELevelValidation
	{
		/** Default behavior is to make sure the level passed is valid */
		All,
		/** Don't validate that the level matches the Object's outer */
		IgnoreObjectOuter,
	};

	/** 
	 * Every object needs to be assigned to a ULevel to know which UWorld->ReplicationSystem it will be added to. 
	 * The level will also be used to set level filtering logic to replicate the object only to clients that loaded the level locally.
	 * This must be called early if your object will be replicated conditionally via RelevantWith
	 * Note: if you don't have a specific level you can use the World->PersistentLevel.
	 * 
	 * @param Level: The level under which the object resides
	 * @param LevelValidation: Use this to skip unecessary validations
	 */
	ENGINE_API void SetAttachedLevel(ULevel* Level, ELevelValidation LevelValidation=ELevelValidation::All);

	/** 
	 * Start replicating the object. This will cause the UNetRootObjectFactory to look for an INetRootObjectFactoryExtension for the class to retrieve the replication details.
	 * @param Level: Level the object is a part of. Can be set to null if SetAttachedLevel was previously set of if you want to use the ReplicatedObject's Level found via his outer chain.
	 */
	ENGINE_API void StartReplication(ULevel* Level=nullptr);

	/** Stop replicating the object. */
	ENGINE_API void StopReplication();

	/**
	 * Start replicating the object but make it relevant only via another replicated object.
	 * You can have relevancy links to multiple different objects. As long as of those is relevant to a client then this object will be too.
	 * @OtherReplicatedObject:  A replicated object we want to be tie our relevancy status to. If it's a subobject, we will use it's root object instead.
	 */
	ENGINE_API void RelevantWith(const UObject* OtherReplicatedObject);

	/**
	 * Remove the relevancy dependency between our object and another replicated object.
	 * @OtherReplicatedObject: The replicated object this object was set to be relevant with
	 */
	ENGINE_API void RemoveRelevantWith(const UObject* OtherReplicatedObject);

	/** The netobject factory to use with this rootobject. By default it will use the UNetRootObjectFactory. */
	ENGINE_API FName GetNetFactoryName() const;

	/** Set to use a netobject factory different from the default. Must be set before replication starts. */
	ENGINE_API void SetNetFactoryName(FName InFactoryName);

	/**
	 * Utility function that fills in the FRootObjectReplicationParams. It can be called from an INetRootObjectFactoryExtension interface for example when starting replication or a replay recording. 
	 * @see INetRootObjectFactoryExtension
	 */
	ENGINE_API void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) const;

	// Status / Config getters

	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	/** Server only: tells if the object was added to the replication system or not */
	bool IsReplicating() const
	{
		return bIsReplicating;
	}

	/** Server only: tells if the object is already attached to a level or not */
	bool HasAttachedLevel() const
	{
		return WeakLevel.IsValid();
	}

	bool IsAlwaysRelevant() const
	{
		return bIsAlwaysRelevant;
	}

	bool IsNotRouted() const
	{
		return bIsNotRouted;
	}

private:

	bool ValidateRootObject();

	ULevel* FindLevelForObject(UObject* InObject) const;

private:

	/** Map of root objects and how many times we are dependent to each. */
	TMap<FObjectKey /* RootObject */, uint32 /* DependentCount */> DependentRootObjects;

	FName ExplicitNetFactoryName;	

	FWeakObjectPtr WeakReplicatedObject;
	TWeakObjectPtr<ULevel> WeakLevel;

	// Status flags

	/** Is the adapter correctly setup */
	bool bIsInitialized:1 = false;
	/** Server only variable set when the object is part of the replication system */
	bool bIsReplicating:1 = false;

	// Configuration flags 

	/** By default objects will be always relevant. Use FRootObjectSettings to customize the object. */
	bool bIsAlwaysRelevant:1 = true;
	/** When true the object will not be relevant until an object it is replicating with is also relevant. */
	bool bIsNotRouted:1 = false;
	/** When true we automatically stop replication when the object does not have any objects to replicate with anymore. */
	bool bOnlyReplicateWhenLinked:1 = false;
};

} // end namespace UE::Net