// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetObjectFactory.h"
#include "Iris/Core/NetObjectReference.h"
#include "UObject/Interface.h"

#include "NetRootObjectFactory.generated.h"

namespace UE::Net
{

class FNetRootObjectCreationHeader : public FNetObjectCreationHeader
{
public:
	
	// Option 1: Root Object is stable
	// Option 2: Root Object is pre-registered
	// Option 3: Root Object is dynamic

	/** References a pathname we want to load on the client. Can be either the object itself when bIsStable or the template to clone from when dynamic. */
	FNetObjectReference Reference;

	//TODO: Add optional way to send the object's Outer

	/** When stable we expect the object to exist on the client */
	bool bIsStable = false;

	/** When preregistered we will find the object via it's NetHandle in the bridge */
	bool bIsPreregistered = false;

public:

	ENGINE_API virtual FString ToString() const override;

	/** Serialize/Deserialize the information held in the header */
	ENGINE_API virtual bool Serialize(const UE::Net::FCreationHeaderContext& Context) const;
	ENGINE_API virtual bool Deserialize(const UE::Net::FCreationHeaderContext& Context);
};

} // end namespace UE::Net


UINTERFACE(MinimalApi, meta=(CannotImplementInterfaceInBlueprint))
class UNetRootObjectFactoryExtension : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface allowing for instance or class specific overrides.
 * If implemented as a standalone interface and configured via FNetRootObjectFactoryExtensionConfig as part of the UNetRootObjectFactoryConfig one need to inherit both from UObject and INetRootObjectFactoryExtension.
 * The UNetRootObjectFactory will instantiate a new object from the configured class.
 */
class INetRootObjectFactoryExtension
{
	GENERATED_BODY()

public:
	virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) const = 0;
};

/**
 * Factory that can be used for any UObject based class that wants to be replicated autonomously.
 * Can be overridden if the class implementation requires more advanced control over discovery, instantiation, location, etc.
 */
UCLASS(MinimalAPI)
class UNetRootObjectFactory : public UNetObjectFactory
{
	GENERATED_BODY()

public:

	ENGINE_API static FName GetFactoryName();
	ENGINE_API static float GetDefaultNetUpdateFrequency();

protected:

	// UNetObjectFactory interface
	ENGINE_API virtual void OnInit() override;

	ENGINE_API virtual FInstantiateResult InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	ENGINE_API virtual void DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext) override;

	ENGINE_API virtual TOptional<FWorldInfoData> GetWorldInfo(const FWorldInfoContext& Context) const override;

	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndFillHeader(UE::Net::FNetRefHandle Handle) override;

	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context) override;

	ENGINE_API virtual bool SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	ENGINE_API virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) override;

protected:

	// Methods available to derived classes that want to reuse basic functionalities

	/** Fill the basic header info for a given object */
	ENGINE_API bool FillHeader(UE::Net::FNetRootObjectCreationHeader* RootObjectHeader, UE::Net::FNetRefHandle Handle, const UObject* RootObjectPtr);

protected:
	// Cached class of INetRootObjectFactoryExtension
	UClass* FactoryExtensionInterfaceClass = nullptr;
};
