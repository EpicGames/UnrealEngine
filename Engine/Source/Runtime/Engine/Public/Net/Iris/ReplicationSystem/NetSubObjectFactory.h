// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetObjectFactory.h"

#include "Iris/Core/NetObjectReference.h"

#include "NetSubObjectFactory.generated.h"


namespace UE::Net
{

/**
* Header information to to tell if it is a dynamic or static header
*/
class FNetBaseSubObjectCreationHeader : public FNetObjectCreationHeader
{
public:
	
	virtual bool IsDynamic() const = 0;

	virtual bool Serialize(const FCreationHeaderContext& Context) const { return false; }
	virtual bool Deserialize(const FCreationHeaderContext& Context) { return false; }
};


/**
* Header information representing subobjects that can be found via their pathname (aka: static or stable name)
*/
class FNetStaticSubObjectCreationHeader : public FNetBaseSubObjectCreationHeader
{
public:

	virtual bool IsDynamic() const override
	{
		return false;
	}

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const override;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context) override;

	ENGINE_API virtual FString ToString() const override;

	FNetObjectReference ObjectReference; // Only for static objects
};

/**
* Header information representing subobjects that must be instantiated
*/
class FNetDynamicSubObjectCreationHeader : public FNetBaseSubObjectCreationHeader
{
public:

	virtual bool IsDynamic() const override
	{ 
		return true;
	}

	ENGINE_API virtual bool Serialize(const FCreationHeaderContext& Context) const override;
	ENGINE_API virtual bool Deserialize(const FCreationHeaderContext& Context) override;

	ENGINE_API virtual FString ToString() const override;

	FNetObjectReference TemplateReference;
	FNetObjectReference OuterReference;
	uint8 bUsePersistentLevel : 1 = false;
	uint8 bOuterIsTransientLevel : 1 = false;	// When set the OuterReference was not sent because the Outer is the default transient level.
	uint8 bOuterIsRootObject : 1 = false;		// When set the OuterReference was not sent because the Outer is the known RootObject.

};

} // end namespace UE::Net

/**
 * Responsible for creating subobjects only
 */
UCLASS(MinimalAPI)
class UNetSubObjectFactory : public UNetObjectFactory
{
	GENERATED_BODY()

public:

	ENGINE_API static FName GetFactoryName();
	
	ENGINE_API virtual FInstantiateResult InstantiateReplicatedObjectFromHeader(const FInstantiateContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	ENGINE_API virtual void SubObjectCreatedFromReplication(UE::Net::FNetRefHandle RootObject, UE::Net::FNetRefHandle SubObjectCreated) override;

	ENGINE_API virtual void DetachedFromReplication(const FDetachContext& Context, const TOptional<FSubObjectDetachContext>& SubObjectContext) override;

	ENGINE_API virtual TOptional<FWorldInfoData> GetWorldInfo(const FWorldInfoContext& Context) const override;

protected:

	// FillRootObjectReplicationParams should not be called for subobjects
	ENGINE_API virtual void FillRootObjectReplicationParams(const UE::Net::FRootObjectReplicationParamsContext& Context, UE::Net::FRootObjectReplicationParams& OutParams) override final;

	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndFillHeader(UE::Net::FNetRefHandle Handle) override;
	ENGINE_API virtual TUniquePtr<UE::Net::FNetObjectCreationHeader> CreateAndDeserializeHeader(const UE::Net::FCreationHeaderContext& Context) override;

	ENGINE_API virtual bool SerializeHeader(const UE::Net::FCreationHeaderContext& Context, const UE::Net::FNetObjectCreationHeader* Header) override;

	/** Write the needed header information for a dynamic subobject */
	ENGINE_API bool FillDynamicHeader(UE::Net::FNetDynamicSubObjectCreationHeader* DynamicHeader, const UObject* SubObject, UE::Net::FNetRefHandle Handle);
};

