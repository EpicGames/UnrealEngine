// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Can't find includes if UE_WITH_IRIS is not set


#include "Iris/ReplicationSystem/ReplicationBridgeTypes.h"
#include "Iris/ReplicationSystem/Conditionals/ReplicationCondition.h"
#include "Net/Core/NetHandle/NetHandle.h"
#include "Engine/EngineTypes.h"

class AActor;
class UActorComponent;
class UEngine;
class UEngineReplicationBridge;
class UObject;
class UReplicationSystem;
class UWorld;
class UNetConnection;
class UNetDriver;
class FRepChangedPropertyTracker;

struct FActorReplicationParams;
struct FStopReplicatingActorParams;

namespace UE::Net
{
	enum class EDependentObjectSchedulingHint : uint8;
}

namespace UE::Net
{

// Helper methods to interact with ReplicationSystem from engine code
struct FReplicationSystemUtil
{
	/** Enum representing the type of system you want to iterate on */
	enum class EType : uint8 
	{
		Any,
		Client,
		Server,
	};

	/** Call a function for all ReplicationBridges of a specific world and type. */
	ENGINE_API static void ForEachBridge(const UWorld* World, TFunctionRef<void(UEngineReplicationBridge*)> Function, EType Type = EType::Any);

	/** Call a function for all ReplicationSystems of a specific world and type. */
	ENGINE_API static void ForEachSystem(const UWorld* World, TFunctionRef<void(UReplicationSystem*)> Function, EType Type = EType::Any);

	/** Call a function for all ReplicationSystems + ReplicationBridges of a specific world and type. */
	ENGINE_API static void ForEachSystemAndBridge(const UWorld* World, TFunctionRef<void(UReplicationSystem*, UEngineReplicationBridge*)> Function, EType Type = EType::Any);

	/** Call a function for all Server ReplicationBridges */
	static void ForEachServerBridge(const UWorld* World, TFunctionRef<void(UEngineReplicationBridge*)> Function)
	{
		ForEachBridge(World, Function, EType::Server);
	}

	/** Call a function for all Server ReplicationSystems */
	static void ForEachServerSystem(const UWorld* World, TFunctionRef<void(UReplicationSystem*)> Function)
	{
		ForEachSystem(World, Function, EType::Server);
	}

	/** Call a function for all Server ReplicationSystems + ReplicationBridges */
	static void ForEachServerSystemAndBridge(const UWorld* World, TFunctionRef<void(UReplicationSystem*, UEngineReplicationBridge* Bridge)> Function)
	{
		ForEachSystemAndBridge(World, Function, EType::Server);
	}

	/** Call a function on all ReplicationSystems of a specific world. */
	ENGINE_API static void ForEachReplicationSystem(const UEngine* Engine, const UWorld* World, TFunctionRef<void(UReplicationSystem*)> Function);

	/** Call a function on all existing ReplicationSystems. This could include systems from other clients (e.g. PIE worlds) */
	ENGINE_API static void ForEachReplicationSystem(TFunctionRef<void(UReplicationSystem*)> Function);

	/** Returns the UReplicationSystem for the main NetDriver assigned to the Actor. Note that an Actor may be replicated by multiple ReplicationSystems. May return null. */
	ENGINE_API static UReplicationSystem* GetReplicationSystem(const AActor* Actor);

	/** Returns the UReplicationSystem for a specific NetDriver. May return null. */
	ENGINE_API static UReplicationSystem* GetReplicationSystem(const UNetDriver* NetDriver);

	/** Returns the UReplicationSystem for the main NetDriver of this World. May return null. */
	ENGINE_API static UReplicationSystem* GetReplicationSystem(const UWorld* World);

	/** Returns the UEngineReplicationBridge of the UReplicationSystem belonging to the main NetDriver assigned to the Actor. May return null. */
	ENGINE_API static UEngineReplicationBridge* GetActorReplicationBridge(const AActor* Actor);

	/** Returns the UReplicationSystem of the UNetDriver the UNetConnection belongs to. */
	ENGINE_API static UEngineReplicationBridge* GetActorReplicationBridge(const UNetConnection* NetConnection);

	/** Returns the UEngineReplicationBridge of the UReplicationSystem belonging to the main NetDriver of this World. May return null. */
	ENGINE_API static UEngineReplicationBridge* GetEngineReplicationBridge(const UWorld* World);

	/** Returns the NetHandle for any replicated object. The returned handle may be invalid. */
	ENGINE_API static FNetHandle GetNetHandle(const UObject* ReplicatedObject);

	/** Start replicating actor and all of its registered components and subobjects. If any ReplicationSystem wants to replicate the actor a NetHandle will be created. */
	ENGINE_API static void StartReplicatingActor(AActor* Actor);
	
	/** Stop replicating an actor. Will destroy handle for actor and registered subobjects. */
	ENGINE_API static void StopReplicatingActor(AActor* Actor, const FStopReplicatingActorParams& StopReplicatingActorParams);

	/** Create NetHandle for actor component and add it as a subobject to the actor. */
	ENGINE_API static void StartReplicatingActorComponent(const AActor* Actor, UActorComponent* ActorComponent);

	/** Stop replicating an ActorComponent and its associated SubObjects. */
	ENGINE_API static void StopReplicatingActorComponent(UActorComponent* SubObject);

	/** Create NetHandle for SubObject and add it as a subobject to the actor. */
	ENGINE_API static void StartReplicatingActorSubObject(const AActor* Actor, UObject* SubObject, ELifetimeCondition NetCondition);

	/** Stop replicating a SubObject and its associated SubObjects. */
	ENGINE_API static void StopReplicatingActorSubObject(const AActor* Actor, UObject* SubObject);

	/** Create NetHandle for SubObject and add it as a subobject to the actor component, but only replicated if the ActorComponent replicates. */
	ENGINE_API static void StartReplicatingActorComponentSubObject(UActorComponent* ActorComponent, UObject* SubObject, ELifetimeCondition Condition);

	/** Stop replicating an SubObject and its associated SubObjects of an ActorComponent. */
	ENGINE_API static void StopReplicatingActorComponentSubObject(UActorComponent* ActorComponent, UObject* SubObject);

	/** 
	 * Set subobject NetCondition for a subobject, the conditions is used to determine if the SubObject should be replicated or not.
	 * @note: As the filtering is done at the serialization level it is typically more efficient to use a separate object for connection 
	 * specific data as filtering can be done at a higher level.
	 */
	ENGINE_API static void SetNetConditionForActorSubObject(const AActor* Actor, UObject* SubObject, ELifetimeCondition NetCondition);
	
	/** 
	 * Set subobject NetCondition for a subobject, the conditions is used to determine if the SubObject should be replicated or not.
	 * @note: As the filtering is done at the serialization level it is typically more efficient to use a separate object for connection 
	 * specific data as filtering can be done at a higher level.
	 */
	ENGINE_API static void SetNetConditionForActorComponent(const UActorComponent* SubObject, ELifetimeCondition Condition);

	/** Update group memberships used by COND_NetGroup sub object filtering for the specified SubObject */
	ENGINE_API static void UpdateSubObjectGroupMemberships(const UObject* SubObject, const UWorld* World);

	/** Update replication status for all NetGroups used by COND_NetGroup sub object filtering for the provided PlayerController */
	ENGINE_API static void UpdateSubObjectGroupMemberships(const APlayerController* PC);

	/** Update replication status for the PlayerController to not include the specified NetGroup */
	ENGINE_API static void RemoveSubObjectGroupMembership(const APlayerController* PC, const FName NetGroup);

	/**
	 * Adds a dependent actor. A dependent actor can replicate separately or if a parent replicates.
	 * Dependent actors cannot be filtered out by dynamic filtering unless the parent is also filtered out.
	 * @note There is no guarantee that the data will end up in the same packet so it is a very loose form of dependency.
	 */
	ENGINE_API static void AddDependentActor(const AActor* Parent, AActor* Child, EDependentObjectSchedulingHint SchedulingHint);

	ENGINE_API static void AddDependentActor(const AActor* Parent, AActor* Child);

	/** Remove dependent actor from parent. The dependent actor will function as a standard standalone replicated actor. */
	ENGINE_API static void RemoveDependentActor(const AActor* Parent, AActor* Child);

	/** Start replication for all networked actors that belong to a specific driver in the world. */
	ENGINE_API static void StartReplicatingActorsInWorldForNetDriver(UWorld* World, UNetDriver* NetDriver);

	/** Notify the ReplicationSystem of a dormancy change. */
	ENGINE_API static void NotifyActorDormancyChange(UReplicationSystem* ReplicationSystem, AActor* Actor, ENetDormancy OldDormancyState);

	/** Trigger replication of dirty state for actor wanting to be dormant. */
	ENGINE_API static void FlushNetDormancy(UReplicationSystem* ReplicationSystem, AActor* Actor, bool bWasDormInitial);

	/** Enable or disable a replication condition. This will affect the replication of properties with conditions. */
	ENGINE_API static void SetReplicationCondition(FNetHandle NetHandle, EReplicationCondition Condition, bool bEnableCondition);

	/**
	 * Sets a fixed priority for a replicated object 
	 * @see ReplicationSystem.h for details
	 */
	ENGINE_API static void SetStaticPriority(const AActor* Actor, float Priority);

	/** Set the cull distance for an actor. This will cause affected code to ignore the NetCullDistanceSquared property. */
	ENGINE_API static void SetCullDistanceOverride(const AActor* Actor, float CullDistSqr);

	/** Clears any previously set cull distance override for an actor. This will cause affected code to respect the NetCullDistanceSquared property. */
	ENGINE_API static void ClearCullDistanceOverride(const AActor* Actor);

	/** Set the poll frequency for an object and its subobjects. */
	ENGINE_API static void SetPollFrequency(const UObject* Object, float PollFrequency);
};

/** Expose internal deferred replication system calls to the rest of the engine. */
class FDeferredReplicationSystemCalls
{
private:

	/** 
	 * Deferred version of the following functions:
	 *	FReplicationSystemUtil::StartReplicatingActor(AActor*)
	 *	FReplicationSystemUtil::StopReplicatingActor(AActor*, FStopReplicatingActorParams*)
	 * 	FReplicationSystemUtil::StartReplicatingActorComponent(const AActor*, UActorComponent*)
	 *	FReplicationSystemUtil::StopReplicatingActorComponent(UActorComponent*)
	 *
	 * The functions below are wrappers for equivalent functions in UEngineReplicationBridge.
	 * 
	 * All deferred functions will be processed when FlushDeferred() is called.
	 */
	static void StartReplicatingActorDefer(AActor* Actor);
	static void StopReplicatingActorDefer(AActor* Actor, const FStopReplicatingActorParams& StopReplicatingActorParams);
	static void StartReplicatingActorComponentDefer(const AActor* Actor, UActorComponent* ActorComponent);
	static void StopReplicatingActorComponentDefer(UActorComponent* ActorComponent, EEndReplicationFlags EndReplicationFlags);
	static void FlushDeferred(UEngineReplicationBridge* Bridge);

	friend class ::AActor;
	friend class ::UActorComponent;
	friend class ::APlayerController;
	friend class ::UNetDriver;
};

}

