// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetTokenStore.h"
#include "Iris/ReplicationSystem/StructNetTokenDataStore.h"
#include "Engine/NetConnection.h"
#include "Engine/PackageMapClient.h"
#include "Net/RepLayout.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

namespace UE::Net
{
	
// Helper for Serializing the NetTokenStructs 
template <typename T>
class TStructNetTokenDataStoreHelper
{
public:

	using DataStoreType = TStructNetTokenDataStore<T>;

	static void SerializeHelper(T& InOutData, FArchive& Ar, UPackageMap* Map)
	{
		UPackageMapClient* MapClient = Cast<UPackageMapClient>(Map);
		check(::IsValid(MapClient));

		UNetConnection* NetConnection = MapClient->GetConnection();
		check(::IsValid(NetConnection));
		check(::IsValid(NetConnection->GetDriver()));

		UScriptStruct* Struct = StaticStruct<T>();
		const TSharedPtr<FRepLayout> RepLayout = NetConnection->GetDriver()->GetStructRepLayout(Struct);
		check(RepLayout.IsValid());

		bool bHasUnmapped = false;
		RepLayout->SerializePropertiesForStruct(Struct, static_cast<FBitArchive&>(Ar), Map, &InOutData, bHasUnmapped);
	}
public:
	static bool NetSerializeAndExportToken(FArchive& Ar, UPackageMap* Map, T& InOutStateData)
	{
		if (!DataStoreType::NetSerializeScriptDelegate.IsBound())
		{
			DataStoreType::NetSerializeScriptDelegate.BindStatic(&SerializeHelper);
		}

		if (!TStructNetTokenDataStore<T>::NetSerializerRegistryDelegates.HasPostFreezeBeenCalled())
		{
			UE_LOGF(LogNetToken, Warning, "NetSerializer registries have not been initialized for %ls", *DataStoreType::GetTokenStoreName().ToString());
		}
		
		if (Ar.IsSaving())
		{
			FNetTokenExportContext* ExportContext = FNetTokenExportContext::GetNetTokenExportContext(Ar);
			FNetTokenStore* NetTokenStore = ExportContext ? ExportContext->GetNetTokenStore() : nullptr;
			DataStoreType* StateTokenStore = NetTokenStore ? NetTokenStore->GetDataStore<DataStoreType>() : nullptr;
			if (!StateTokenStore)
			{
				return false;
			}

			UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess NetTokenStoreAccess(NetTokenStore));

			UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Map);
			if (PackageMapClient && PackageMapClient->GetConnection() && PackageMapClient->GetConnection()->IsReplay())
			{
				return NetSerializeStateDataAsReplayData(StateTokenStore, InOutStateData, Ar, *PackageMapClient);
			}
			
			FNetToken StateToken = StateTokenStore->GetOrCreateToken(InOutStateData);
			StateTokenStore->WriteNetToken(Ar, StateToken);
			ExportContext->AddNetTokenPendingExport(StateToken);
		}
		else
		{
			const FNetTokenResolveContext* NetTokenResolveContext = Map ? Map->GetNetTokenResolveContext() : nullptr;
			FNetTokenStore* NetTokenStore = NetTokenResolveContext ? NetTokenResolveContext->NetTokenStore : nullptr;
			DataStoreType* StateTokenStore = NetTokenStore ? NetTokenStore->GetDataStore<DataStoreType>() : nullptr;
			if (!StateTokenStore)
			{
				return false;
			}

			UE_IRIS_PARALLEL_EXPR(FScopedNetTokenStoreAccess NetTokenStoreAccess(NetTokenStore));

			UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Map);
			if (PackageMapClient && PackageMapClient->GetConnection() && PackageMapClient->GetConnection()->IsReplay())
			{
				return NetSerializeStateDataAsReplayData(StateTokenStore, InOutStateData, Ar, *PackageMapClient);
			}
			
			FNetToken StateToken = StateTokenStore->ReadNetToken(Ar);
			if (Ar.IsError())
			{
				return false;
			}
			InOutStateData = StateTokenStore->ResolveRemoteToken(StateToken, *NetTokenResolveContext->RemoteNetTokenStoreState);
		}
		return true;
	}

	// Helpers for Replay Support
	static bool NetSerializeStateDataAsReplayData(DataStoreType* StateTokenStore, T& InOutStateData, FArchive& Ar, UPackageMapClient& PackageMapClient)
	{
		TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient.GetNetFieldExportGroup(DataStoreType::GetTokenStoreName().ToString());
		if (Ar.IsSaving())
		{
			if (!NetFieldExportGroup.IsValid())
			{
				SetupReplayExportGroup(StateTokenStore, PackageMapClient);
			}
			FNetToken StateToken = StateTokenStore->GetOrCreateToken(InOutStateData);
			StateTokenStore->WriteNetToken(Ar, StateToken);

			if (!NetFieldExportGroup->NetFieldExports.IsValidIndex(StateToken.GetIndex()))
			{
				NetFieldExportGroup->NetFieldExports.SetNum(StateToken.GetIndex() + 1, EAllowShrinking::No);
				NetFieldExportGroup->bDirtyForReplay = true;
			}
			FNetFieldExport& NetFieldExport = NetFieldExportGroup->NetFieldExports[StateToken.GetIndex()];

			AddStateToReplayExportGroup(StateTokenStore, InOutStateData, NetFieldExportGroup, PackageMapClient);
			PackageMapClient.TrackNetFieldExport(NetFieldExportGroup.Get(), StateToken.GetIndex());

			return true;
		}
		else if (Ar.IsLoading())
		{
			FNetToken StateToken = StateTokenStore->ReadNetToken(Ar);
			if (Ar.IsError())
			{
				return false;
			}
			if (!NetFieldExportGroup.IsValid())
			{
				return false;
			}

			if (StateToken.IsValid())
			{
				if(GetStateFromReplayExportGroup(StateTokenStore, InOutStateData, StateToken, NetFieldExportGroup, PackageMapClient))
				{
					return true;
				}
				InOutStateData = DataStoreType::GetInvalidState();
				return false;
			}
		}
		return false;
	}

	static bool GetStateFromReplayExportGroup(DataStoreType* StateTokenStore, T& OutData, FNetToken& StateToken, TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup, UPackageMapClient& Map)
	{
		if (!NetFieldExportGroup.IsValid())
		{
			return false;
		}
		const uint32 TagTokenIndex = StateToken.GetIndex();
		if (static_cast<int32>(TagTokenIndex) >= NetFieldExportGroup->NetFieldExports.Num())
		{
			return false;
		}
		FNetFieldExport& NetFieldExport = NetFieldExportGroup->NetFieldExports[StateToken.GetIndex()];
		FNetBitReader TempAr;
		TArray<uint8>& Data = NetFieldExportGroup->NetFieldExports[TagTokenIndex].Blob;
		TempAr.SetData(Data.GetData(),Data.NumBytes()*8);
		StateTokenStore->ReadTokenData(TempAr, StateToken, &Map);
		OutData = StateTokenStore->ResolveToken(StateToken);
		return true;
	}
	
	static bool AddStateToReplayExportGroup(DataStoreType* StateTokenStore, T& InData, TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup, UPackageMapClient& Map)
	{
		if (!NetFieldExportGroup.IsValid())
		{
			return false;
		}
		FNetToken StateToken = StateTokenStore->GetOrCreateToken(InData);
		if (static_cast<int32>(StateToken.GetIndex()) >= NetFieldExportGroup->NetFieldExports.Num())
		{
			NetFieldExportGroup->NetFieldExports.SetNum(StateToken.GetIndex() + 1, EAllowShrinking::No);
		}
		FNetFieldExport& NetFieldExport = NetFieldExportGroup->NetFieldExports[StateToken.GetIndex()];
		if (NetFieldExport.bExported)
		{
			return false;
		}
		
		FNetBitWriter TempAr;
		StateTokenStore->WriteTokenData(TempAr, StateTokenStore->GetOrCreatePersistentState(InData), &Map);

		NetFieldExport = FNetFieldExport(StateToken.GetIndex(),0,NAME_None,*TempAr.GetBuffer());
		NetFieldExportGroup->bDirtyForReplay = true;
		UE_LOGF(LogNetToken, Verbose, "Replay> Exported for %ls as NetFieldIndex %u", *DataStoreType::GetTokenStoreName().ToString(), StateToken.GetIndex());
		return true;
	}

	static bool SetupReplayExportGroup(DataStoreType* StateTokenStore, UPackageMapClient& PackageMapClient)
	{
		const FString StoreName = DataStoreType::GetTokenStoreName().ToString();
		TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient.GetNetFieldExportGroup(StoreName);
		if (NetFieldExportGroup.IsValid())
		{
			return true;
		}
		NetFieldExportGroup = MakeShared<FNetFieldExportGroup>();
		NetFieldExportGroup->PathName = StoreName;
		NetFieldExportGroup->NetFieldExports.SetNum(StateTokenStore->StoredStates.Num());
		for (auto& Pair : StateTokenStore->StoredStates)
		{
			AddStateToReplayExportGroup(StateTokenStore, Pair.Value, NetFieldExportGroup, PackageMapClient);
		}
		PackageMapClient.AddNetFieldExportGroup(StoreName, NetFieldExportGroup);
		return true;
	}
};
	
}