// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Tracks an FName ID to a time value. Time will be context dependent, but usually
 * represents the total amount of time a specific action took (how long a package
 * took to load, how long an actor had queued bunches, etc.)
 *
 * Could have used a TPair, but this will make it more obvious what we're tracking.
 */
struct FDelinquencyNameTimePair
{
public:

	FDelinquencyNameTimePair(FName InName, float InTimeSeconds)
		: Name(InName)
		, TimeSeconds(InTimeSeconds)
	{
	}

	FName Name;
	float TimeSeconds;
};

// With this formatting, an array of these (or even just a single entry) will have
// the same form as a map "key:value,key:value,..."
inline FString LexToString(const FDelinquencyNameTimePair& Value)
{
	return FString::Printf(TEXT("%s:%f"), *Value.Name.ToString(), Value.TimeSeconds);
}

struct FDelinquencyKeyFuncs : public BaseKeyFuncs<FDelinquencyNameTimePair, FDelinquencyNameTimePair, false>
{
	static KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element;
	}

	static bool Matches(KeyInitType LHS, KeyInitType RHS)
	{
		return LHS.Name == RHS.Name;
	}

	static uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key.Name);
	}
};

/**
 * Convenience type that can be used to tracks information about things that can result in prolonged
 * periods of apparent network inactivity, despite actually receiving traffic.
 *
 * The overall number of entries is expected to be small, but ultimately is left up to callers.
 */
struct FDelinquencyAnalytics
{
public:

	ENGINE_API explicit FDelinquencyAnalytics(const uint32 InNumberOfTopOffendersToTrack);

	ENGINE_API FDelinquencyAnalytics(FDelinquencyAnalytics&& Other);

	FDelinquencyAnalytics(const FDelinquencyAnalytics&) = delete;
	const FDelinquencyAnalytics& operator=(const FDelinquencyAnalytics&) = delete;
	FDelinquencyAnalytics& operator=(FDelinquencyAnalytics&&) = default;

	void Emplace(FName Name, float TimeSeconds)
	{
		Add(FDelinquencyNameTimePair(Name, TimeSeconds));
	}

	/**
	 * Adds the event to the delinquency tracking, by accumulating its time into total time,
	 * and updating any existing events to choose the one with the highest time.
	 *
	 * When NumberOfTopOffendersToTrack == 0, we will just track the set of all events as well as the total time.
	 *
	 * When NumberOfTopOffendersToTrack > 0, we will track the set, total time, and also maintain sorted list
	 * (highest to lowest) of events that occurred.
	 *
	 * By setting NumberOfTopOffendersToTrack to 0, users can manage their own lists of "TopOffenders", or
	 * otherwise avoid the per add overhead of this tracking.
	 */
	ENGINE_API void Add(FDelinquencyNameTimePair&& ToTrack);

	const TArray<FDelinquencyNameTimePair>& GetTopOffenders() const
	{
		return TopOffenders;
	}

	const TSet<FDelinquencyNameTimePair, FDelinquencyKeyFuncs>& GetAllDelinquents() const
	{
		return AllDelinquents;
	}

	const float GetTotalTime() const
	{
		return TotalTime;
	}

	const uint32 GetNumberOfTopOffendersToTrack() const
	{
		return NumberOfTopOffendersToTrack;
	}

	ENGINE_API void Reset();

	ENGINE_API void CountBytes(class FArchive& Ar) const;

private:

	TArray<FDelinquencyNameTimePair> TopOffenders;
	TSet<FDelinquencyNameTimePair, FDelinquencyKeyFuncs> AllDelinquents;
	float TotalTime;

	// This is explicitly non const, as we will be copying / moving these structs around.
	uint32 NumberOfTopOffendersToTrack;
};

/**
 * Tracks data related specific to a NetDriver that can can result in prolonged periods of apparent
 * network inactivity, despite actually receiving traffic.
 *
 * This includes things like Pending Async Loads.
 *
 * Also @see FConnectionDelinquencyAnalytics and FDelinquencyAnalytics.
 */
struct FNetAsyncLoadDelinquencyAnalytics
{
	FNetAsyncLoadDelinquencyAnalytics()
		: DelinquentAsyncLoads(0)
		, MaxConcurrentAsyncLoads(0)
	{
	}

	FNetAsyncLoadDelinquencyAnalytics(const uint32 NumberOfTopOffendersToTrack) :
		DelinquentAsyncLoads(NumberOfTopOffendersToTrack),
		MaxConcurrentAsyncLoads(0)
	{
	}

	FNetAsyncLoadDelinquencyAnalytics(FNetAsyncLoadDelinquencyAnalytics&& Other) :
		DelinquentAsyncLoads(MoveTemp(Other.DelinquentAsyncLoads)),
		MaxConcurrentAsyncLoads(Other.MaxConcurrentAsyncLoads)
	{
	}

	FNetAsyncLoadDelinquencyAnalytics(const FNetAsyncLoadDelinquencyAnalytics&) = delete;
	const FNetAsyncLoadDelinquencyAnalytics& operator=(const FNetAsyncLoadDelinquencyAnalytics&) = delete;
	FNetAsyncLoadDelinquencyAnalytics& operator=(FNetAsyncLoadDelinquencyAnalytics&&) = default;

	void CountBytes(FArchive& Ar) const
	{
		DelinquentAsyncLoads.CountBytes(Ar);
	}

	void Reset()
	{
		DelinquentAsyncLoads.Reset();
		MaxConcurrentAsyncLoads = 0;
	}

	FDelinquencyAnalytics DelinquentAsyncLoads;
	uint32 MaxConcurrentAsyncLoads;
};

/**
 * Tracks data related specific to a NetConnection that can can result in prolonged periods of apparent
 * network inactivity, despite actually receiving traffic.
 *
 * Also @see FDriverDelinquencyAnalytics and FDelinquencyAnalytics.
 */
struct FNetQueuedActorDelinquencyAnalytics
{
	FNetQueuedActorDelinquencyAnalytics()
		: DelinquentQueuedActors(0)
		, MaxConcurrentQueuedActors(0)
	{
	}

	FNetQueuedActorDelinquencyAnalytics(const uint32 NumberOfTopOffendersToTrack) :
		DelinquentQueuedActors(NumberOfTopOffendersToTrack),
		MaxConcurrentQueuedActors(0)
	{
	}

	FNetQueuedActorDelinquencyAnalytics(FNetQueuedActorDelinquencyAnalytics&& Other) :
		DelinquentQueuedActors(MoveTemp(Other.DelinquentQueuedActors)),
		MaxConcurrentQueuedActors(Other.MaxConcurrentQueuedActors)
	{
	}

	FNetQueuedActorDelinquencyAnalytics(const FNetQueuedActorDelinquencyAnalytics&) = delete;
	const FNetQueuedActorDelinquencyAnalytics& operator=(const FNetQueuedActorDelinquencyAnalytics&) = delete;
	FNetQueuedActorDelinquencyAnalytics& operator=(FNetQueuedActorDelinquencyAnalytics&&) = default;


	void CountBytes(FArchive& Ar) const
	{
		DelinquentQueuedActors.CountBytes(Ar);
	}

	void Reset()
	{
		DelinquentQueuedActors.Reset();
		MaxConcurrentQueuedActors = 0;
	}

	FDelinquencyAnalytics DelinquentQueuedActors;
	uint32 MaxConcurrentQueuedActors;
};

/** Struct wrapping Per Net Connection saturation analytics. */
struct FNetConnectionSaturationAnalytics
{
public:

	FNetConnectionSaturationAnalytics()
		: NumberOfTrackedFrames(0)
		, NumberOfSaturatedFrames(0)
		, LongestRunOfSaturatedFrames(0)
		, NumberOfReplications(0)
		, NumberOfSaturatedReplications(0)
		, LongestRunOfSaturatedReplications(0)
		, CurrentRunOfSaturatedFrames(0)
		, CurrentRunOfSaturatedReplications(0)
	{
	}

	/** The total number of frames that we have currently tracked. */
	const uint32 GetNumberOfTrackedFrames() const
	{
		return NumberOfTrackedFrames;
	}

	/** The number of frames we have reported as saturated.*/
	const uint32 GetNumberOfSaturatedFrames() const
	{
		return NumberOfSaturatedFrames;
	}

	/** The longest number of consecutive frames that we have been saturated. */
	const uint32 GetLongestRunOfSaturatedFrames() const
	{
		return LongestRunOfSaturatedFrames;
	}

	/**
	 * The number of times we have tried to replicate data on this connection
	 * (UNetDriver::ServerReplicateActors / UReplicationGraph::ServerReplicateActors)
	 */
	const uint32 GetNumberOfReplications() const
	{
		return NumberOfReplications;
	}

	/** The number of times we have been pre-empted from replicating all data, due to saturation. */
	const uint32 GetNumberOfSaturatedReplications() const
	{
		return NumberOfSaturatedReplications;
	}

	/** The longest number of consecutive replication attempts where we were pre-empted due to saturation. */
	const uint32 GetLongestRunOfSaturatedReplications() const
	{
		return LongestRunOfSaturatedReplications;
	}

	/** Resets the state of tracking. */
	ENGINE_API void Reset();

private:

	friend class UNetConnection;

	ENGINE_API void TrackFrame(const bool bIsSaturated);

	ENGINE_API void TrackReplication(const bool bIsSaturated);

	uint32 NumberOfTrackedFrames;
	uint32 NumberOfSaturatedFrames;
	uint32 LongestRunOfSaturatedFrames;

	uint32 NumberOfReplications;
	uint32 NumberOfSaturatedReplications;
	uint32 LongestRunOfSaturatedReplications;

	uint32 CurrentRunOfSaturatedFrames;
	uint32 CurrentRunOfSaturatedReplications;
};

/** Struct wrapper Per Net Connection analytics for things like packet loss and jitter. */
struct FNetConnectionPacketAnalytics
{
public:

	FNetConnectionPacketAnalytics()
		: bSawPacketLossBurstThisFrame(false)
		, NumberOfAcksThisFrame(0)
		, NumberOfNaksThisFrame(0)
		, NumberOfMissingPacketsThisFrame(0)
		, NumberOfPacketsThisFrame(0)
		, CurrentRunOfDroppedOutPackets(0)
		, LongestRunOfDroppedOutPackets(0)
		, LongestRunOfDroppedInPackets(0)
		, NumberOfFramesWithBurstsOfPacketLoss(0)
		, NumberOfFramesWithNoPackets(0)
		, NumberOfTrackedFrames(0)
	{
	}

	/** Longest number of consecutive dropped incoming packets that was tracked. */
	const uint32 GetLongestRunOfDroppedInPackets() const
	{
		return LongestRunOfDroppedInPackets;
	}

	/** Longest number of consecutive outgoing packets that was tracked. */
	const uint32 GetLongestRunOfDroppedOutPackets() const
	{
		return LongestRunOfDroppedOutPackets;
	}

	/** Number of frames where we saw bursts of packet loss. */
	const uint32 GetNumberOfFramesWithBurstsOfPacketLoss() const
	{
		return NumberOfFramesWithBurstsOfPacketLoss;
	}

	/** The total number of frames where we were not notified of any packets (ACK, NAK, in or out). */
	const uint32 GetNumberOfFramesWithNoPackets() const
	{
		return NumberOfFramesWithNoPackets;
	}

	/** Total number of frames that have been tracked. */
	const uint64 GetNumberOfTrackedFrames() const
	{
		return NumberOfTrackedFrames;
	}

	const double GetBurstyPacketLossPerSecond(double DurationSec) const
	{
		return DurationSec > 0.0 ? GetNumberOfFramesWithBurstsOfPacketLoss() / DurationSec : 0.0;
	}

	/** Resets the state of tracking. */
	ENGINE_API void Reset();

private:

	friend class UNetConnection;

	ENGINE_API void TrackAck(int32 PacketId);
	ENGINE_API void TrackNak(int32 PacketId);
	ENGINE_API void TrackInPacket(uint32 InPacketId, uint32 NumberOfMissingPackets);

	ENGINE_API void Tick();

	bool bSawPacketLossBurstThisFrame : 1;

	uint32 NumberOfAcksThisFrame;
	uint32 NumberOfNaksThisFrame;

	uint32 NumberOfMissingPacketsThisFrame;
	uint32 NumberOfPacketsThisFrame;

	uint32 CurrentRunOfDroppedOutPackets;
	uint32 LongestRunOfDroppedOutPackets;
	uint32 LongestRunOfDroppedInPackets;

	uint32 NumberOfFramesWithBurstsOfPacketLoss;
	uint32 NumberOfFramesWithNoPackets;
	uint64 NumberOfTrackedFrames;
};


/**
* Calculates the current packet loss along with a rolling average of the past X updates
* Constructed with the number of samples you want to average over time.
*/
template<uint32 NbPeriodsForAverage>
struct TPacketLossData
{
public:

	/** The loss percentage over the last StatPeriod */
	float GetLossPercentage() const { return LossPercentage; }

	/** The average loss percentage over the previous X StatPeriods */
	float GetAvgLossPercentage() const { return AvgLossPercentage; }

	/** Update the packet loss based on total and lost packets */
	void UpdateLoss(int32 NbPacketsLost, int32 TotalPackets, int32 SampleCount)
	{
		// Update the current statistic
		const int32 PacketsDuringLastPeriod = TotalPackets - TotalPacketsAtPeriodStart;
		LossPercentage = PacketsDuringLastPeriod > 0 ? ((float)NbPacketsLost / (float)PacketsDuringLastPeriod) : 0.0f;

		// Update the rolling average
		const int32 SampleIndex = SampleCount % NbPeriodsForAverage;
		LossSamples[SampleIndex] = LossPercentage;

		float RollingAverage = 0.0f;
		for (float LossSample : LossSamples)
		{
			RollingAverage += LossSample;
		}
		AvgLossPercentage = RollingAverage / NbPeriodsForAverage;

		// Store data for the next update
		TotalPacketsAtPeriodStart = TotalPackets;
	}

private:
	/**
	* The amount of samples to average.
	* The length of the period will be NbPeriodsForAverage * StatPeriod.
	* With default values that is 3 samples * 1 second -> an average of the past 3 seconds
	*/
	float LossSamples[NbPeriodsForAverage] = { 0.0f };

	int32 TotalPacketsAtPeriodStart = 0;
	float LossPercentage = 0.0f;
	float AvgLossPercentage = 0.0f;
};
