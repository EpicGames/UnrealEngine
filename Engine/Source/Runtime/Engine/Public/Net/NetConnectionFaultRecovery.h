// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Includes
#include "Net/Core/Connection/NetConnectionFaultRecoveryBase.h"
#include "Net/DefaultFaultHandler.h"

#include "NetConnectionFaultRecovery.generated.h"

// Forward declarations
class UNetConnection;


/**
 * Configuration class for FNetFaultState state escalation configuration
 */
UCLASS(config=Engine, PerObjectConfig)
class UNetFaultConfig : public UEscalationManagerConfig
{
	GENERATED_BODY()

private:
	virtual void InitConfigDefaultsInternal() override;
};


namespace UE
{
namespace Net
{

/**
 * NetConnection Fault Recovery
 *
 * Implements generic escalation handling for net faults, for all Engine level fault types - with the ability to extend with custom fault handling.
 *
 * To add fault handling for custom netcode (e.g. new NetConnection's or PacketHandler components),
 * create a new fault handler (with its own custom TNetResult etc.) added to FaultManager.
 */
class FNetConnectionFaultRecovery final : public FNetConnectionFaultRecoveryBase
{
	friend UNetConnection;

private:
	/**
	 * Initializes NetConnection-specified default values
	 *
	 * @param InConfigContext	The context to use for altering this objects configuration (NetDriver name)
	 * @param InConnection		The connection which owns this fault handler
	 */
	ENGINE_API void InitDefaults(FString InConfigContext, UNetConnection* InConnection);

	/**
	 * Ticks fault recovery
	 * NOTE: Takes approximate current time, instead of DeltaTime
	 *
	 * @param TimeSeconds	The approximate current time
	 */
	ENGINE_API void TickRealtime(double TimeSeconds);

	/**
	 * Whether or not calls to Tick are presently required
	 *
	 * @return	Whether Tick is required
	 */
	ENGINE_API bool DoesRequireTick() const;

	ENGINE_API virtual void InitEscalationManager() override;

	ENGINE_API void NotifySeverityUpdate(const FEscalationState& OldState, const FEscalationState& NewState, ESeverityUpdate UpdateType);


private:
	/** Additional context for altering this objects configuration (typically the NetDriver name) */
	FString ConfigContext;

	/** Caches the owning NetConnection */
	UNetConnection* Connection = nullptr;

	/** Tracks the last read value of NetConnection.InTotalHandlerPackets, for calculating total packet counts between Tick's */
	int32 LastInTotalHandlerPackets = 0;

	/** Tracks NetConnection.InTotalHandlerPackets over the course of the last second (for priming dormant escalation manager) */
	int32 LastInTotalHandlerPacketsPerSec = 0;

	/** The current rate of NetConnection.InTotalHandlerPackets per second */
	int32 InTotalHandlerPacketsPerSec = 0;

	/** Last time LastInTotalHandlerPacketsPerSec was updated */
	double LastPerSecPacketCheck = 0.0;

	/** Default fault handler for FaultManager, for base NetConnection fault handling */
	FDefaultFaultHandler DefaultFaultHandler;
};

}
}

