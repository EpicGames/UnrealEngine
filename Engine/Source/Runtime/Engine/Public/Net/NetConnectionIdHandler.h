// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/BitArray.h"

class FNetConnectionIdHandler
{
public:
	ENGINE_API FNetConnectionIdHandler();
	ENGINE_API ~FNetConnectionIdHandler();

	ENGINE_API void Init(uint32 IdCount);

	ENGINE_API uint32 Allocate();
	ENGINE_API void Free(uint32 Id);

	ENGINE_API uint32 GetMaxConnectionIdCount() const;

private:
	TBitArray<> UsedIds;
	uint32 IdHint;
};
