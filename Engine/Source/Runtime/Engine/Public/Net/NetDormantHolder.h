// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "Templates/SharedPointer.h"
#include "Serialization/Archive.h"
#include "Misc/NetworkGuid.h"

class AActor;
class FObjectReplicator;
class FNetworkObjectList;

namespace UE::Net
{
	/** Function definition allowed in the ForEach functions */
	typedef TFunctionRef<void(FObjectKey OwnerActorKey, FObjectKey ObjectKey, const TSharedRef<FObjectReplicator>& ReplicatorRef)> FExecuteForEachDormantReplicator;

	typedef TMap<FNetworkGUID, TWeakObjectPtr<UObject>> FDormantObjectMap;
}

namespace UE::Net::Private
{

/** 
* Container holding the FObjectReplicator of a dormant replicated object
*/
struct FDormantObjectReplicator
{
	/** Default constructor that will always create a replicator */
	explicit FDormantObjectReplicator(FObjectKey InObjectKey);

	/** Constructor that copies an existing replicator */
	explicit FDormantObjectReplicator(FObjectKey InObjectKey, const TSharedRef<FObjectReplicator>& ExistingReplicator);

	bool operator==(const FDormantObjectReplicator& rhs) const { return ObjectKey == rhs.ObjectKey; }
	bool operator==(FObjectKey rhs) const { return ObjectKey == rhs; }

	/** The replicated object's key mapped to this replicator */
	FObjectKey ObjectKey;

	/** The replicator of the replicated object we are storing */
	TSharedRef<FObjectReplicator> Replicator;
};


/** 
* Container that stores all the replicators owned by a given dormant Actor. 
* Note that the actor's own replicator will be part of this list.
*/
struct FActorDormantReplicators
{
	explicit FActorDormantReplicators(FObjectKey InOwnerActorKey) : OwnerActorKey(InOwnerActorKey) {}

	bool operator==(const FActorDormantReplicators& rhs) const { return OwnerActorKey == rhs.OwnerActorKey; }
	bool operator==(FObjectKey rhs) const { return OwnerActorKey == rhs; }

	void CountBytes(FArchive& Ar) const { DormantReplicators.CountBytes(Ar); }

	/** KeyFuncs that make it so the TSet can match a struct with a simple FObjectKey. */
	struct FDormantObjectReplicatorKeyFuncs : BaseKeyFuncs<FDormantObjectReplicator, FObjectKey, false>
	{
		static KeyInitType GetSetKey(ElementInitType Element) { return Element.ObjectKey; }
		static bool Matches(KeyInitType lhs, KeyInitType rhs) { return lhs == rhs; }
		static uint32 GetKeyHash(KeyInitType Key) { return GetTypeHash(Key); }
	};

	/** The dormant actor who owns all the different object replicators */
	FObjectKey OwnerActorKey;

	typedef TSet<FDormantObjectReplicator, FDormantObjectReplicatorKeyFuncs> FObjectReplicatorSet;
	/** List of all object replicates stored for this dormant actor */
	FObjectReplicatorSet DormantReplicators;
};

/**
* Container that stores the object replicators of dormant actors and their subobjects.
* Note that the actor stores it's own replicator and the replicators of it's subobjects
*/
struct FDormantReplicatorHolder
{
	/** 
	* Are we currently storing an object replicator owned by the dormant actor. 
	* 
	* @param DormantActor The dormant actor that owns the replicated object
	* @param ReplicatedObject The object that was replicated and for whom we could be storing it's FObjectReplicator.
	* @return True if we are storing a replicator.
	*/
	bool DoesReplicatorExist(AActor* DormantActor, UObject* ReplicatedObject) const;

	/**
	* Return the objector replicator for the given replicated object .
	* 
	* @param DormantActor The dormant actor that owns the replicated object.
	* @param ReplicatedObject The object that was replicated
	* @return The object replicator of the replicated object if we were holding one. Otherwise an invalid pointer
	*/
	TSharedPtr<FObjectReplicator> FindReplicator(AActor* DormantActor, UObject* ReplicatedObject);

	/**
	* Return the object replicator for the given replicated object but also remove the reference to that object replicator.
	* 
	* @param DormantActor The dormant actor that owns the replicated object
	* @param ReplicatedObject The object that was replicated and for whom we want it's FObjectReplicator.
	* @return The object replicator of the replicated object if we were holding one. Otherwise an invalid pointer
	*/
	TSharedPtr<FObjectReplicator> FindAndRemoveReplicator(AActor* DormantActor, UObject* ReplicatedObject);

	/**
	* Create an object replicator for the given replicated object and store a reference to it.
	* Does not initialize the Replicator but simply allocates it.
	* 
	* @param DormantActor The dormant actor that owns the replicated object
	* @param ReplicatedObject The object that will be tied to the new ObjectReplicator.
	* @param bOverwroteExistingReplicator When it returns true it means we erased an existing replicator.
	* @return Return a reference to the object replicator we created and are now storing.
	*/
	const TSharedRef<FObjectReplicator>& CreateAndStoreReplicator(AActor* DormantActor, UObject* ReplicatedObject, bool& bOverwroteExistingReplicator);

	/**
	* Store an existing replicator tied to the given replicated object
	* 
	* @param DormantActor The dormant actor that owns the replicated object
	* @param ReplicatedObject The object that is tied to the existing replicator
	* @param ObjectReplicator The existing replicator we that will be stored
	*/
	void StoreReplicator(AActor* DormantActor, UObject* ReplicatedObject, const TSharedRef<FObjectReplicator>& ObjectReplicator);

	/**
	* Remove our reference to the object replicator tied to the given replicated object
	* 
	* @param DormantActor The dormant actor that owns the replicated object
	* @param ReplicatedObjectKey The object key to a replicated object that may have stored it's object replicator here
	* @return Return true if we did find the replicated object and removed it
	*/
	bool RemoveStoredReplicator(AActor* DormantActor, FObjectKey ReplicatedObjectKey);

	/**
	* Remove the references to all the object replicators tied to the given actor
	* 
	* @param DormantActor The dormant actor we want to clean any reference to
	*/
	void CleanupAllReplicatorsOfActor(AActor* DormantActor);

	/**
	* Iterate over all the stored object replicators and destroy any that are tied to a replicated object that is now considered invalid.
	* This version will decrement the subobject references stored in the networkobject list too.
	* 
	* @param FNetworkObjectList The netdriver's network object list
	* @param ReferenceOwner The connection owner that is storing those dormant objects
	*/
	void CleanupStaleObjects(FNetworkObjectList& NetworkObjectList, UObject* ReferenceOwner);

	/** 
	* Execute the passed function on each dormant replicator we are holding
	*/
	void ForEachDormantReplicator(UE::Net::FExecuteForEachDormantReplicator Function);

	/**
	* Execute the passed function on each dormant replicatof owned by the given dormant actor
	*/
	void ForEachDormantReplicatorOfActor(AActor* DormantActor, UE::Net::FExecuteForEachDormantReplicator Function);

	/**
	* Remove every reference that is still stored.
	*/
	void EmptySet();

	void CountBytes(FArchive& Ar) const;

	/** KeyFuncs that make it so the TSet can only needs a simple AActor pointer for the Key */
	struct FActorDormantReplicatorsKeyFuncs : BaseKeyFuncs<FActorDormantReplicators, FObjectKey, false>
	{
		static KeyInitType GetSetKey(ElementInitType Element) { return Element.OwnerActorKey; }
		static bool Matches(KeyInitType lhs, KeyInitType rhs) { return lhs == rhs; }
		static uint32 GetKeyHash(KeyInitType Key) { return GetTypeHash(Key); }
	};

	typedef TSet<FActorDormantReplicators, FActorDormantReplicatorsKeyFuncs> FActorReplicatorSet;
	/** The TSet indexed by Actor that stores all their dormant object replicators */
	FActorReplicatorSet ActorReplicatorSet;

	/**
	 * Retrieve stored set of replicated sub-objects of the given actor at the time of the last dormancy flush
	 * This data is cleared when the actor is processed by ReplicateActor
	 *
	 * @param Actor		The actor to retrieve the object map for
	 * @return A map of network guids to weak object pointers, or nullptr
	 */
	UE::Net::FDormantObjectMap* FindFlushedObjectsForActor(AActor* Actor);

	/**
	 * Retrieve stored set of replicated sub-objects of the given actor at the time of the last dormancy flush
	 * This data is cleared when the actor is processed by ReplicateActor
	 *
	 * @param Actor		The actor to retrieve the object map for
	 * @return A map of network guids to weak object pointers
	 */
	UE::Net::FDormantObjectMap& FindOrAddFlushedObjectsForActor(AActor* Actor);

	/**
	 * Clear stored flushed replicated sub-objects for a given actor, generally after replication or when the actor is destroyed
	 *
	 * @param Actor		The actor to clear the flushed object data for
	 */
	void ClearFlushedObjectsForActor(AActor* Actor);

	/** Map of actors to guid/weakptr pairs from the last dormancy flush */
	TMap<FObjectKey, UE::Net::FDormantObjectMap> FlushedObjectMap;
};

}//end namespace UE::Net::Private

