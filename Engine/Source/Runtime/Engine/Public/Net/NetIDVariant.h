// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetRefHandle.h"
#include "Misc/NetworkGuid.h"
#include "Misc/TVariant.h"
#include "Templates/TypeHash.h"

class FArchive;

namespace UE::Net
{

/**
 * Simple variant wrapper to hold a generic or Iris network object ID (FNetworkGUID or FNetRefHandle).
 * Intended for transitioning systems that were using FNetworkGUID directly to Iris, not intended for general use.
 */
class FNetIDVariant
{
public:
	/** Placeholder to represent empty/unknown type. Always considered invalid. */
	struct FEmptyID {};

	using FVariantType = TVariant<FEmptyID, FNetworkGUID, FNetRefHandle>;

	/** Constructs a variant holding an FEmptyID */
	FNetIDVariant() = default;

	explicit ENGINE_API FNetIDVariant(FNetworkGUID NetGUID);
	explicit ENGINE_API FNetIDVariant(FNetRefHandle NetRefHandle);

	/** Serializes or deserializes the stored ID or handle, suitable for networking */
	friend ENGINE_API FArchive& operator<<(FArchive& Ar, FNetIDVariant& NetID);

	/** Returns whether the stored ID or handle is valid */
	bool ENGINE_API IsValid() const;

	/** Returns the stored TVariant */
	FVariantType GetVariant() const
	{
		return Variant;
	}

	/** Equality comparison */
	bool ENGINE_API operator==(const FNetIDVariant& RHS) const;

	/** Output to string */
	ENGINE_API FString ToString() const;

private:
	FVariantType Variant;
};

inline uint32 GetTypeHash(const FNetIDVariant& NetID)
{
	// Assuming the internal index of TVariant is uint8 (which it is) even though the public API uses SIZE_T
	uint8 TypeIndex = static_cast<uint8>(NetID.GetVariant().GetIndex());

	uint32 IDHash = 0;
	if (NetID.GetVariant().IsType<FNetworkGUID>())
	{
		IDHash = GetTypeHash(NetID.GetVariant().Get<FNetworkGUID>());
	}
	else if (NetID.GetVariant().IsType<FNetRefHandle>())
	{
		IDHash = GetTypeHash(NetID.GetVariant().Get<FNetRefHandle>());
	}

	return HashCombineFast(TypeIndex, IDHash);
}

}
