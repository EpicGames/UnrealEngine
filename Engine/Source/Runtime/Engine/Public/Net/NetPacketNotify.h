// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineLogs.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Templates/IsSigned.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "Util/SequenceNumber.h"
#include "Util/SequenceHistory.h"

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#	define UE_NET_ENABLE_PACKET_NOTIFY_LOG 0
#else
#	define UE_NET_ENABLE_PACKET_NOTIFY_LOG 1
#endif 

#if UE_NET_ENABLE_PACKET_NOTIFY_LOG
#	define UE_LOG_PACKET_NOTIFY(Format, ...)  UE_LOG(LogNetTraffic, Verbose, Format, ##__VA_ARGS__)
#else
#	define UE_LOG_PACKET_NOTIFY(...)
#endif

#define UE_LOG_PACKET_NOTIFY_WARNING(Format, ...)  UE_LOG(LogNetTraffic, Warning, Format, ##__VA_ARGS__)

struct FBitWriter;
struct FBitReader;

namespace UE::Net::Private
{
	struct FNetPacketNotifyTestUtil;
}

/** 
	FNetPacketNotify - Drives delivery of sequence numbers, acknowledgments and notifications of delivery sequence numbers
*/
class FNetPacketNotify
{
public:
	enum { SequenceNumberBits = 14 };
	enum { MaxSequenceHistoryLength = 256 };

	typedef TSequenceNumber<SequenceNumberBits, uint16> SequenceNumberT;
	typedef TSequenceHistory<MaxSequenceHistoryLength> SequenceHistoryT;

	struct FNotificationHeader
	{
		// Bitfield with acknowledged packets.
		SequenceHistoryT History;
		// Size of history in words
		SIZE_T HistoryWordCount = 0;
		// Incoming sequence number
		SequenceNumberT Seq;
		// Latest sequence that the remote has received. Note: It does not mean that the packet was accepted.
		SequenceNumberT AckedSeq;
		// Sequence number of the first entry in the sequence history, normally the same as AckedSeq
		SequenceNumberT AckedHistorySeq;
		// True if remote is flushing history due to sequence history overflow.
		bool bRemoteIsWaitingForHistoryReset = false;
	};

	/** Constructor */
	FNetPacketNotify();

	/** Init notification with expected initial sequence numbers */
	void Init(SequenceNumberT InitialInSeq, SequenceNumberT InitialOutSeq);

	/** Mark Seq as received and update current InSeq, missing sequence numbers will be marked as lost */
	void AckSeq(SequenceNumberT Seq) { AckSeq(Seq, true); }

	/** Explicitly mark Seq as not received and update current InSeq, additional missing sequence numbers will be marked as lost */
	void NakSeq(SequenceNumberT Seq) { AckSeq(Seq, false); }

	/** Increment outgoing seq number and commit data*/
	SequenceNumberT CommitAndIncrementOutSeq();

	/** Write NotificationHeader, and update outgoing ack record 
		if bRefresh is true we will attempt to refresh a previously written header if the resulting size will be the same as the already written header.
		returns true if data was written, and false if no data was written which might be the case if we try to rewrite an existing header but the required size differs.
	*/
	bool WriteHeader(FBitWriter& Writer, bool bRefresh = false);
	
	/** Read header from stream */
	bool ReadHeader(FNotificationHeader& Data, FBitReader& Reader) const;

	/**
	 * Gets the delta between the present sequence, and the sequence inside the specified header - if the delta is positive
	 */
	SequenceNumberT::DifferenceT GetSequenceDelta(const FNotificationHeader& NotificationData)
	{
		if (NotificationData.Seq > InSeq && NotificationData.AckedSeq >= OutAckSeq && OutSeq > NotificationData.AckedSeq)
		{
			return SequenceNumberT::Diff(NotificationData.Seq, InSeq);
		}
		else
		{
			return 0;
		}
	}

	/**
	 * Update state of PacketNotification based on received header and invoke packet notifications for received acks.
	 *
	 * @param NotificationData			The header to update from
	 * @param InFunc					A function in the format (void)(FNetPacketNotify::SequenceNumberT AckedSequence, bool bDelivered) to handle packet notifications.
	 * @return							The > 0 delta of the incoming seq if within half the seq number space. 0 if the received seq is outside current window ,or the ack seq received is invalid.
	*/
	template<class Functor>
	SequenceNumberT::DifferenceT Update(const FNotificationHeader& NotificationData, Functor&& InFunc);

	/** Get the current SequenceHistory */
	const SequenceHistoryT& GetInSeqHistory() const { return InSeqHistory; }

	/** Get the last received in sequence number */
	SequenceNumberT GetInSeq() const { return InSeq; }

	/** Get the last received sequence number that we have accepted, InAckSeq cannot be larger than InSeq */
	SequenceNumberT GetInAckSeq() const { return InAckSeq; }

	/** Get the current outgoing sequence number */
	SequenceNumberT GetOutSeq() const { return OutSeq; }

	/** Get the last outgoing sequence number acknowledged by remote */
	SequenceNumberT GetOutAckSeq() const { return OutAckSeq; }

	/** If we do have more unacknowledged sequence numbers in-flight than our maximum sendwindow we should not send more as the receiving end will not be able to detect if the sequence number has wrapped around */
	bool CanSend() const { SequenceNumberT NextOutSeq = OutSeq; ++NextOutSeq; return NextOutSeq >= OutAckSeq; }

	/**
	 * Return whether we can send packets without exhausting the packet sequence history window, as it could cause packets to be NAKed even when they've been received by the remote peer. 
	 * @param SafetyMargin A small number representing how many packets you would like to keep as a safety margin for heart beats or other important packets.
	 */
	bool IsSequenceWindowFull(uint32 SafetyMargin=0U) const;

	/** Get the current sequenceHistory length in bits, clamped to the maximum history length */
	SequenceNumberT::DifferenceT GetCurrentSequenceHistoryLength() const;

	/** Returns true if we are currently waiting for a flush of the sequence window */
	bool IsWaitingForSequenceHistoryFlush() const { return WaitingForFlushSeqAck > OutAckSeq; }

private:
	struct FSentAckData
	{
		SequenceNumberT OutSeq;	// Not needed... just to verify that things work as expected
		SequenceNumberT InAckSeq;
	};
	typedef TResizableCircularQueue<FSentAckData, TInlineAllocator<128>> AckRecordT;

	AckRecordT AckRecord;				// Track acked seq for each sent packet to track size of ack history
	SIZE_T WrittenHistoryWordCount;		// Bookkeeping to track if we can update data
	SequenceNumberT WrittenInAckSeq;	// When we call CommitAndIncrementOutSequence this will be committed along with the current outgoing sequence number for bookkeeping

	// Track incoming sequence data
	SequenceHistoryT InSeqHistory;		// BitBuffer containing a bitfield describing the history of received packets
	SequenceNumberT InSeq;				// Last sequence number received from remote
	SequenceNumberT InAckSeq;			// Last sequence number received from remote that we have acknowledged, this is needed since we support accepting a packet but explicitly not acknowledge it as received.
	SequenceNumberT InAckSeqAck;		// Last sequence number received from remote that we have acknowledged and also knows that the remote has received the ack, used to calculate how big our history must be
	SequenceNumberT WaitingForFlushSeqAck;

	// Track outgoing sequence data
	SequenceNumberT OutSeq;				// Outgoing sequence number
	SequenceNumberT OutAckSeq;			// Last sequence number that we know that the remote side have received.

private:

	SequenceNumberT UpdateInAckSeqAck(SequenceNumberT::DifferenceT AckCount, SequenceNumberT AckedSeq);
	SequenceNumberT::DifferenceT InternalUpdate(const FNotificationHeader& NotificationData, SequenceNumberT::DifferenceT InSeqDelta);

	// Returns true if sequence history contains any packets marked as received for which we have not yet received an ack
	bool GetHasUnacknowledgedAcks() const;

	// Returns true if we can acknowledge the Seq without overshooting the sequence history
	bool WillSequenceFitInSequenceHistory(SequenceNumberT Seq) const;

	// Initiates a wait for a flush of the sequence history
	void SetWaitForSequenceHistoryFlush();

	template<class Functor>
	inline void ProcessReceivedAcks(const FNotificationHeader& NotificationData, Functor&& InFunc);
	void AckSeq(SequenceNumberT AckedSeq, bool IsAck);

#if WITH_AUTOMATION_WORKER
	friend UE::Net::Private::FNetPacketNotifyTestUtil;
#endif
};

inline bool FNetPacketNotify::IsSequenceWindowFull(uint32 SafetyMargin) const
{
	const SequenceNumberT SequenceLength = OutSeq - OutAckSeq;
	return SequenceLength > MaxSequenceHistoryLength || (SafetyMargin >= MaxSequenceHistoryLength) || (SequenceLength.Get() > (MaxSequenceHistoryLength - SafetyMargin));
}

template<class Functor>
FNetPacketNotify::SequenceNumberT::DifferenceT FNetPacketNotify::Update(const FNotificationHeader& NotificationData, Functor&& InFunc)
{
	const SequenceNumberT::DifferenceT InSeqDelta = GetSequenceDelta(NotificationData);

	if (InSeqDelta > 0)
	{
		UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::Update - Seq %u, InSeq %u"), NotificationData.Seq.Get(), InSeq.Get());
	
		ProcessReceivedAcks(NotificationData, InFunc);

		return InternalUpdate(NotificationData, InSeqDelta);
	}
	else
	{
		return 0;
	}
}

template<class Functor>
void FNetPacketNotify::ProcessReceivedAcks(const FNotificationHeader& NotificationData, Functor&& InFunc)
{
	if (NotificationData.AckedSeq > OutAckSeq)
	{
		// Total ack count including implicit naks
		SequenceNumberT::DifferenceT AckCount = SequenceNumberT::Diff(NotificationData.AckedSeq, OutAckSeq);
		UE_LOG_PACKET_NOTIFY(TEXT("Notification::ProcessReceivedAcks - AckedSeq: %u, OutAckSeq: %u AckCount: %u RemoteWaitingForFlush: %u"), NotificationData.AckedSeq.Get(), OutAckSeq.Get(), AckCount, NotificationData.bRemoteIsWaitingForHistoryReset ? 1U : 0U);

		// Update InAckSeqAck used to track the needed number of bits to transmit our ack history
		// Note: As we might reset sequence history we need to check if we already have advanced the InAckSeqAck
		{
			const SequenceNumberT NewInAckSeqAck = UpdateInAckSeqAck(AckCount, NotificationData.AckedSeq);
			if (NewInAckSeqAck > InAckSeqAck)
			{
				UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::ProcessReceivedAcks - Advance InAckSeqAck: %u"), NewInAckSeqAck.Get());
				InAckSeqAck = NewInAckSeqAck;
			}
		}

		// ExpectedAck = OutAckSeq + 1
		SequenceNumberT CurrentAck(OutAckSeq);
		++CurrentAck;

		// Make sure that we only look at the sequence history bit included in the notification data as the sequence history might have been reset, 
		// in which case we might not receive the max size history even though the ack-count is bigger than the history
		const SequenceNumberT::DifferenceT HistoryBits = static_cast<SequenceNumberT::DifferenceT>(NotificationData.HistoryWordCount * SequenceHistoryT::BitsPerWord);

		// Warn if the received sequence number is greater than our history buffer, since if that is the case we have to treat the data as lost
		// Note: This should not be a problem as we have a mechanism to re-sync history
		// If this occurs with no hitches on server or client, there might be reason to investigate if too much data is being sent in which case the the size sequence history might have to be increased.
		SequenceNumberT::DifferenceT MissingAckCount = 0U;
		if (AckCount > HistoryBits)
		{
			MissingAckCount = AckCount - HistoryBits;
			UE_LOG_PACKET_NOTIFY(TEXT("FNetPacketNotify::ProcessReceivedAcks - Missed Acks: AckedSeq: %u, OutAckSeq: %u, FirstMissingSeq: %u Count: %u"), NotificationData.AckedSeq.Get(), OutAckSeq.Get(), CurrentAck.Get(), MissingAckCount);
		}

		if (NotificationData.bRemoteIsWaitingForHistoryReset)
		{
			// Re-sync sequence history
			// Extra care has to be taken, we might have to synthesize naks both before and after the acks carried in the sequence history.
			if (NotificationData.AckedHistorySeq > OutAckSeq)
			{
				SequenceNumberT::DifferenceT AckCountInHistory = SequenceNumberT::Diff(NotificationData.AckedHistorySeq, OutAckSeq);

				// Synthesize missing naks before start of sequence history, this can occur if we get burst packet drops and then overflow history
				while (AckCountInHistory > HistoryBits)
				{
					--AckCountInHistory;
					--AckCount;
					InFunc(CurrentAck, false);
					++CurrentAck;
				}

				// Acknowledge packets based on information from sequence history
				while (AckCountInHistory > 0)
				{
					--AckCountInHistory;
					--AckCount;

					const uint32 AckIndex = AckCountInHistory;

					UE_LOG_PACKET_NOTIFY(TEXT("Notification::ProcessReceivedAcks Seq: %u - IsAck: %u HistoryIndex: %u"), CurrentAck.Get(), NotificationData.History.IsDelivered(AckIndex) ? 1u : 0u, AckIndex);
					InFunc(CurrentAck, NotificationData.History.IsDelivered(AckIndex));
					++CurrentAck;
				}
			}

			// Synthesize missing naks after history window as the packets are discarded during re-sync
			while (AckCount > 0)
			{
				--AckCount;
				InFunc(CurrentAck, false);
				++CurrentAck;
			}
		}
		else
		{
			// Normal path, missing naks are at the beginning

			// Synthesize missing naks before start of history window, this can occur if we get burst packet drops with no pending acks
			while (AckCount > HistoryBits)
			{
				--AckCount;
				InFunc(CurrentAck, false);
				++CurrentAck;
			}

			// For sequence numbers contained in the history we lookup the delivery status from the history
			while (AckCount > 0)
			{
				--AckCount;
				UE_LOG_PACKET_NOTIFY(TEXT("Notification::ProcessReceivedAcks Seq: %u - IsAck: %u HistoryIndex: %u"), CurrentAck.Get(), NotificationData.History.IsDelivered(AckCount) ? 1u : 0u, AckCount);
				InFunc(CurrentAck, NotificationData.History.IsDelivered(AckCount));
				++CurrentAck;
			}
		}
			
		// Reset history if we are done with reset of sequence history.
		if (IsWaitingForSequenceHistoryFlush() && NotificationData.AckedSeq >= WaitingForFlushSeqAck)
		{
			UE_LOG_PACKET_NOTIFY(TEXT("ClearIsWaitingForSequenceHistoryFlush %u"), NotificationData.AckedSeq.Get());
			InSeqHistory.Reset();
			WaitingForFlushSeqAck = NotificationData.AckedSeq;
		}

		OutAckSeq = NotificationData.AckedSeq;

		// Deal with sequence wraparound.
		if (OutAckSeq > WaitingForFlushSeqAck)
		{
			WaitingForFlushSeqAck = OutAckSeq;
		}
	}
}
