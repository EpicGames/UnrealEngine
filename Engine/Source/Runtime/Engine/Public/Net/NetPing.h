// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Net/Core/Analytics/NetStatsUtils.h"
#include "GameFramework/PlayerState.h"

#include "NetPing.generated.h"

// Forward declarations
enum class EPingType : uint32;
class FInternetAddr;

namespace UE::Net
{

namespace Private
{

/**
 * Converts an EPingType flag to an array index at compile time.
 *
 * @param PingType	The EPingType flag to convert
 * @return			The index that the flag represents
 */
constexpr int32 PingTypeToIdx(EPingType PingType)
{
	// FGenericPlatformMath::FloorLog2 as constexpr

	uint32 Value = static_cast<uint32>(PingType);
	uint32 Pos = 0;
	if (Value >= 1<<16) { Value >>= 16; Pos += 16; }
	if (Value >= 1<< 8) { Value >>=  8; Pos +=  8; }
	if (Value >= 1<< 4) { Value >>=  4; Pos +=  4; }
	if (Value >= 1<< 2) { Value >>=  2; Pos +=  2; }
	if (Value >= 1<< 1) {				Pos +=  1; }

	return Pos;
}

/**
 * Converts an EPingType flag to an array index at runtime.
 *
 * @param PingType	The EPingType flag to convert
 * @return			The index that the flag represents
 */
static int32 PingTypeToIdxRuntime(EPingType PingType)
{
	return FMath::FloorLog2(static_cast<uint32>(PingType));
}

}
}

/**
 * Flags specifying different types of ping.
 */
UENUM()
enum class EPingType : uint32
{
	None					= 0x00000000,
	RoundTrip				= 0x00000001,	// Round Trip ping which includes any server/client frame time delay
	RoundTripExclFrame		= 0x00000002,	// Round Trip ping which may attempt to exclude server frame time delay (inaccurate)
	ICMP					= 0x00000004,	// Standard ICMP ping (GameNetDriver client only, accuracy affected by network ICMP throttling)
	UDPQoS					= 0x00000008,	// UDP based ping used with QoS ping host on server (more accurate than ICMP)

	Max						= UDPQoS,
	Count					= UE::Net::Private::PingTypeToIdx(static_cast<EPingType>(EPingType::Max)) + 1
};

ENUM_CLASS_FLAGS(EPingType);


/**
 * The type of averaging to perform on ping values
 */
UENUM()
enum class EPingAverageType : uint8
{
	None,			// No averaging
	MovingAverage,	// Moving average of the last 4 seconds of ping data
	PlayerStateAvg	// Original PlayerState ping code, calculating a moving average of the last 4 seconds of ping data. To be deprecated.
};

/**
 * NMT_NetPing control channel sub-messages
 */
enum class ENetPingControlMessage : uint8
{
	SetPingAddress,				// Sent from the server to the client, to override the ICMP/UDP ping address
	SetAlternatePingAddress,	// Sent from the server to the client, to set an alternate ICMP/UDP ping address (not immediately activated)
	PingFailure,				// Sent from the client to the server, to report failure to ping an ICMP/UDP address

	Max = PingFailure
};


/**
 * Which ping endpoint are we setting here? 
 */
enum class ENetPingTargetEndpoint : uint8
{
	DefaultEndpoint,				// Used when setting the primary ping endpoint
	AlternateEndpoint,				// Used when setting an alternate ping endpoint, typically used to handle accelerated routing.

	Max = AlternateEndpoint
};


namespace UE::Net
{
/**
 * Whether or not an EPingType value is valid
 */
static bool IsValidPingType(EPingType PingType)
{
	const uint32 PingTypeVal = static_cast<uint32>(PingType);

	return FMath::IsPowerOfTwo(PingTypeVal) && PingTypeVal <= static_cast<uint32>(EPingType::Max);
}

namespace Private
{
	class FNetPingICMP;
	class FNetPingUDPQoS;
}

/**
 * Struct for returning ping values, in seconds.
 */
struct FPingValues
{
	/** The current/latest ping value, or -1.0 if not set */
	double Current = -1.0;

	/** The smallest measured ping value (no averaging), or -1.0 if not set */
	double Min = -1.0;

	/** The highest measured ping value (no averaging), or -1.0 if not set */
	double Max = -1.0;
};

/**
 * Central class for handling all Player/NetConnection ping collection and calculation.
 * Can work with any NetDriver type (except ICMP), but restricted to GameNetDriver for now.
 */
class FNetPing
{
	template <typename T> friend struct UE::Core::Private::PimplPtr::TPimplHeapObjectImpl;

private:
	class FAverageStorageBase
	{
	public:
		/** The total number of pings (including timeouts) */
		uint32 PingCount = 0;

		/** The number of ping timeouts, if applicable */
		uint32 PingTimeouts = 0;
	};

	class FNoAverageStorage : public FAverageStorageBase
	{
	public:
		/** Stores the final Min/Max/Current ping values */
		TSampleMinMaxAvg<EMinMaxValueMode::PerSample> PingValues;
	};

	class FMovingAverageStorage : public FAverageStorageBase
	{
	public:
		/** Stores the final Min/Max/Current ping values */
		TSampleMinMaxAvg<EMinMaxValueMode::PerMeasurement> PingValues;

		/** Accumulates ping readings and outputs the moving average to PingValues */
		TBinnedMovingAvg<decltype(PingValues), TBinParms::NumBins(4)> PingAccumulator{TBinParms::TimePerBin(1.0)};


	public:
		FMovingAverageStorage();
	};

	/**
	 * Implements storage and calculation duplicating original PlayerState ping code, for backwards-compatible/representative comparison.
	 */
	class FPlayerStateAvgStorage : public FAverageStorageBase
	{
	private:
		/** The current PingBucket index that is being filled */
		uint8 CurPingBucket = 0;

		/**
		 * Stores the last 4 seconds worth of ping data (one second per 'bucket').
		 * It is stored in this manner, to allow calculating a moving average,
		 * without using up a lot of space, while also being tolerant of changes in ping update frequency
		 */
		PingAvgData PingBucket[4];

		/** The timestamp for when the current PingBucket began filling */
		double CurPingBucketTimestamp = 0.0;

	public:
		/** Stores the final Min/Max/Current ping values */
		TSampleMinMaxAvg<EMinMaxValueMode::PerMeasurement> PingValues;


	public:
		/**
		 * Resets values to their base state
		 */
		void Reset();

		/**
		 * Receives ping updates for the client (both clientside and serverside), from the net driver
		 * NOTE: This updates much more frequently clientside, thus the clientside ping will often be different to what the server displays
		 */
		void UpdatePing(double TimeVal, float InPing);

	private:
		/** Recalculates the replicated Ping value once per second (both clientside and serverside), based upon collected ping data */
		void RecalculateAvgPing();
	};


private:
	FNetPing() = delete;

	FNetPing(UNetConnection* InOwner);

public:
	~FNetPing();

	/**
	 * Creates a new FNetPing instance, owned by the specified NetConnection.
	 * NOTE: Presently only enabled for GameNetDriver, but can support other NetDriver's (except ICMP ping).
	 *
	 * @param InOwner	The owner of the new FNetPing instance.
	 * @return			Returns the new FNetPing instance, or nullptr if disabled. Uses TPimplPtr for compatibility with forward-declare.
	 */
	static TPimplPtr<FNetPing> CreateNetPing(UNetConnection* InOwner);

	/**
	 * Updates the value of the specified ping type, with a new ping reading (in seconds).
	 *
	 * @param PingType		The type of ping being updated.
	 * @param TimeVal		The time that the ping reading was calculated.
	 * @param PingValue		The value of the new ping reading, in seconds.
	 */
	void UpdatePing(EPingType PingType, double TimeVal, double PingValue);

	/**
	 * Records a ping timeout, for the specified ping type.
	 *
	 * @param PingType		The type of ping which timed out.
	 */
	void UpdatePingTimeout(EPingType PingType);

	/**
	 * Retrieves the ping results for the specified ping type, in seconds.
	 *
	 * @param PingType		The type of ping to retrieve results for.
	 * @return				Returns the ping results for the specified type, in seconds.
	 */
	ENGINE_API FPingValues GetPingValues(EPingType PingType) const;

	/**
	 * Ticks the ping handler.
	 *
	 * @param CurTimeSeconds	The current time in seconds.
	 */
	void TickRealtime(double CurTimeSeconds);

	/**
	 * Handles NMT_NetPing control channel messages, serverside/clientside.
	 *
	 * @param Connection	The connection that received the message
	 * @param MessageType	The type of message received
	 * @param MessageStr	The message content.
	 */
	ENGINE_API static void HandleNetPingControlMessage(UNetConnection* Connection, ENetPingControlMessage MessageType, FString MessageStr);

	/**
	 * Returns the list of enabled ping types.
	 */
	EPingType GetPingTypes() const
	{
		return PingTypes;
	}

	/**
	 * Activates a previously received alternate ping address for the specified ping type,
	 * replacing the current active ping address override.
	 *
	 * @param PingType	The type of ping to activate the alternate address for.
	 */
	ENGINE_API void UseAlternatePingAddress(EPingType PingType);

	/**
	 * Returns whether an alternate ping address has been received for the specified ping type.
	 *
	 * @param PingType	The type of ping to check for an alternate address.
	 * @return			True if an alternate address is available.
	 */
	ENGINE_API bool HasAlternatePingAddress(EPingType PingType) const;

	/** Delegate broadcast when the client receives an alternate ping address from the server. */
	using FOnAlternatePingAddressReceivedSignature = TMulticastDelegate<void(UNetConnection* /*Connection*/, EPingType /*PingType*/, const FString& /*Address*/)>;
	static FOnAlternatePingAddressReceivedSignature::RegistrationType& GetOnAlternatePingAddressReceivedDelegate() { return OnAlternatePingAddressReceivedDelegate; }
	
private:
	/**
	 * Initialize the ping handler
	 */
	void Init();

#if STATS
	/**
	 * Update the 'stat ping' ping value for the specified ping type.
	 *
	 * @param PingType		The type of ping having its stats value set.
	 * @param CurrentValue	The value to assign to stats, for the specified ping value.
	 */
	void UpdatePingStats(EPingType PingType, double CurrentValue);

	/**
	 * Update the 'stat ping' timeout values for the specified ping type.
	 *
	 * @param PingType			The type of ping having its stats value set.
	 * @param NumTimeouts		The number of times this ping type has timed out.
	 * @param TimeoutPercent	The percentage of time this ping type has timed out.
	 */
	void UpdateTimeoutStats(EPingType PingType, uint32 NumTimeouts, double TimeoutPercent);
#endif

	/**
	 * Returns the read-only stored values, for the specified ping type.
	 * NOTE: Not safe to call without verifying PingType is enabled in PingTypes, first.
	 *
	 * @param PingType	The ping type to retrieve stored values for.
	 * @return			Read-only reference to the stored ping values.
	 */
	const FSampleMinMaxAvg& GetPingStorageValues(EPingType PingType) const;

	/**
	 * Returns the modifiable base average storage (containing only common/shared stats), for the specified ping type.
	 * NOTE: Not safe to call without verifying PingType is enabled in PingTypes, first.
	 *
	 * @param PingType	The ping type to retrieve the base storage for.
	 * @return			Modifiable base storage for specified ping type.
	 */
	FAverageStorageBase& GetPingStorageBase(EPingType PingType);

	/**
	 * Resets the ping values for the specified ping type.
	 *
	 * @param PingType	The ping type to reset the values of
	 */
	void ResetPingValues(EPingType PingType);


	/**
	 * Returns the cached IP address of the remote connection (server, as this is clientside only), initializing the cached value if necessary.
	 *
	 * @return	The address of the remote connection.
	 */
	const TCHAR* GetRemoteAddr();

	/**
	 * Returns the address that ICMP pings should use
	 */
	FString GetICMPAddr();

	/**
	 * Returns the address that UDP pings should use
	 */
	FString GetUDPQoSAddr();

	/**
	 * Updates the cached remote address, if necessary.
	 *
	 * @param CurTimeSeconds	The current time in seconds, used for rate limiting.
	 */
	void UpdateRemoteAddr(double CurTimeSeconds);

	/**
	 * Performs regular ICMP pings.
	 *
	 * @param CurTimeSeconds	The current time in seconds.
	 */
	void PingTimerICMP(double CurTimeSeconds);

	/**
	 * Performs regular UDP pings.
	 *
	 * @param CurTimeSeconds	The current time in seconds.
	 */
	void PingTimerUDP(double CurTimeSeconds);


	/**
	 * Clientside implementation of the 'ENetPingControlMessage::SetPingAddress' control channel message.
	 * Allows the server to specify the ICMP/UDPQoS ping address/port.
	 *
	 * @param PingType				The type of ping having its address assigned.
	 * @param PingAddress			The address being assigned to the specified ping type.
	 * @param TargetEndpoint		The target endpoint that the ping address is being set for
	 */
	void ServerSetPingAddress(EPingType PingType, FString PingAddress, ENetPingTargetEndpoint TargetEndpoint = ENetPingTargetEndpoint::DefaultEndpoint);

private:
	/** The NetConnection which owns this ping handler */
	TObjectPtr<UNetConnection> Owner;

	/** Weak reference to the remote address, for checking if it has changed */
	TWeakPtr<const FInternetAddr> WeakRemoteAddr;

	/** Caches string value for the remote address */
	FString CachedRemoteAddr;

	/** The next time the remote address should be updated */
	double NextRemoteAddrCheckTimestamp = 0.0;

	/** Flags specifying the ping types that are enabled */
	EPingType PingTypes = EPingType::None;

	/** Specifies the type of averaging to use for each EPingType index */
	EPingAverageType PingAverageTypes[static_cast<uint32>(EPingType::Count)] = { EPingAverageType::None };

	/** Specifies the index into a storage TArray, for each EPingType index */
	uint8 PingAverageStorageIdx[static_cast<uint32>(EPingType::Count)] = { 0 };

	/** Storage used for EPingType's that don't use averaging */
	TArray<FNoAverageStorage> PingNoAverageStorage;

	/** Storage used for EPingType's that use a moving average */
	TArray<FMovingAverageStorage> PingMovingAverageStorage;

	/** Storage used for EPingType's that use PlayerState ping code averaging */
	TArray<FPlayerStateAvgStorage> PingPlayerStateAvgStorage;

	/** The next time to perform an ICMP ping */
	double NextPingTimerICMPTimestamp = 0.0;

	/** The next time to perform a UDP ping */
	double NextPingTimerUDPTimestamp = 0.0;

	/** Pointer to the ICMP ping handler, if set (only one ICMP handler instance can exist, held by the GameNetDriver client) */
	TSharedPtr<Private::FNetPingICMP> NetPingICMP;

	/** Pointer to a UDP ping handler, if set (no restrictions on the number of UDP handler instances nor supported drivers) */
	TSharedPtr<Private::FNetPingUDPQoS> NetPingUDPQoS;

	/** Overrides the address to use for ICMP pings (set by server) */
	FString OverridePingICMPAddress;

	/** Overrides the address to use for UDPQoS pings (set by server) */
	FString OverridePingUDPQoSAddress;

	/** Overrides the port to use for UDPQoS pings (set by server) */
	int32 OverridePingUDPQosPort = INDEX_NONE;

	/** The number of times a ping address was overridden. Used to limit the number of changes. */
	int32 OverridePingAddressCount = 0;

	/** Alternate address for UDPQoS pings, stored but not active until UseAlternatePingAddress is called */
	FString AlternatePingUDPQoSAddress;

	/** Alternate port for UDPQoS pings, stored but not active until UseAlternatePingAddress is called */
	int32 AlternatePingUDPQoSPort = INDEX_NONE;

	/** Delegate fired when an alternate Ping Address is received*/
	ENGINE_API static FOnAlternatePingAddressReceivedSignature OnAlternatePingAddressReceivedDelegate;
};

}
