// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "NetPushModelHelpers.generated.h"

UCLASS(MinimalAPI)
class UNetPushModelHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	/** Mark replicated property as dirty with the Push Model networking system */
	UFUNCTION(BlueprintCallable, Category="Networking")
	static ENGINE_API void MarkPropertyDirty(UObject* Object, FName PropertyName);

	UFUNCTION(BlueprintCallable, Category = "Networking", Meta=(BlueprintInternalUseOnly = "true", HidePin = "Object|RepIndex|PropertyName"))
	static ENGINE_API void MarkPropertyDirtyFromRepIndex(UObject* Object, int32 RepIndex, FName PropertyName);
};
