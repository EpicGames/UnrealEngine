// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifndef UE_WITH_NETWORK_GRANULAR_MEM_TRACKING
#define UE_WITH_NETWORK_GRANULAR_MEM_TRACKING	!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif // UE_WITH_NETWORK_GRANULAR_MEM_TRACKING

#if UE_WITH_NETWORK_GRANULAR_MEM_TRACKING

namespace UE::Net::Private::GranularMemoryTracking
{
	struct FScopeMarker
	{
		ENGINE_API FScopeMarker(FArchive& InAr, FString&& InScopeName);
		ENGINE_API ~FScopeMarker();

		ENGINE_API void BeginWork();

		ENGINE_API void EndWork(const FString& WorkName);

		ENGINE_API void LogCustomWork(const FString& WorkName, const uint64 Bytes) const;

		const bool IsEnabled() const
		{
			return ScopeStack != nullptr;
		}

		const FString& GetScopeName()
		{
			return ScopeName;
		}

	private:

		friend struct FNetworkMemoryTrackingScopeStack;

		uint64 PreWorkPos = 0;

		const FArchive& Ar;
		const FString ScopeName;
		struct FNetworkMemoryTrackingScopeStack* ScopeStack;
	};
}

#define GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Archive, ScopeName) UE::Net::Private::GranularMemoryTracking::FScopeMarker GranularNetworkMemoryScope(Archive, ScopeName);
#define GRANULAR_NETWORK_MEMORY_TRACKING_TRACK(Id, Work) \
	{ \
		GranularNetworkMemoryScope.BeginWork(); \
		Work; \
		GranularNetworkMemoryScope.EndWork(Id); \
	}
#define GRANULAR_NETWORK_MEMORY_TRACKING_CUSTOM_WORK(Id, Value) GranularNetworkMemoryScope.LogCustomWork(Id, Value);

#else // UE_WITH_NETWORK_GRANULAR_MEM_TRACKING

#define GRANULAR_NETWORK_MEMORY_TRACKING_INIT(Archive, ScopeName) 
#define GRANULAR_NETWORK_MEMORY_TRACKING_TRACK(Id, Work) { Work; }
#define GRANULAR_NETWORK_MEMORY_TRACKING_CUSTOM_WORK(Id, Work) 

#endif // UE_WITH_NETWORK_GRANULAR_MEM_TRACKING
