// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "NetworkMetricsConfig.generated.h"

struct FNetworkMetricsMutator;

UENUM()
enum class ENetworkMetricEnableMode : uint8
{
	EnableForAllReplication,
	EnableForIrisOnly,
	EnableForNonIrisOnly,
	ServerOnly,
	ClientOnly,
};

USTRUCT()
struct FNetworkMetricConfig
{
	GENERATED_BODY()

public:
	/** The name of the metric to register the listener. Optional if a Mutator is specified. */
	UPROPERTY()
	FName MetricName;

	/** Mutator to add to the listener. Optional if a MetricName is specified. */
	UPROPERTY()
	TInstancedStruct<FNetworkMetricsMutator> Mutator;

	/** A sub-class of UNetworkMetricBaseListener. */
	UPROPERTY()
	TSoftClassPtr<class UNetworkMetricsBaseListener> Class;

	UPROPERTY()
	ENetworkMetricEnableMode EnableMode = ENetworkMetricEnableMode::EnableForAllReplication;
};

UCLASS(Config=Engine)
class UNetworkMetricsConfig : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(Config)
	TArray<FNetworkMetricConfig> Listeners;
};
