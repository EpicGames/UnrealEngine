// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Iris/IrisConfig.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "NetworkMetricsDatabase.generated.h"

#define UE_API ENGINE_API

struct FNetworkMetricsMutator;

template<typename BaseStructT>
struct [[nodiscard]] TInstancedStruct;

namespace UE::Net
{

template<class MetricType>
struct FNetworkMetric
{
	FName Name;
	MetricType Value;
};

struct FNetworkMetricSnapshot
{
	TArray<FNetworkMetric<int64>> MetricInts;
	TArray<FNetworkMetric<float>> MetricFloats;

	void Reset()
	{
		MetricInts.Reset();
		MetricFloats.Reset();
	}
};

} // namespace UE::Net

UCLASS(MinimalAPI)
class UNetworkMetricsDatabase : public UObject
{
	GENERATED_BODY()

public:
	/* Add a floating point metric. */
	UE_API void CreateFloat(const FName MetricName, float DefaultValue);
	/* Add an integer metric. */
	UE_API void CreateInt(const FName MetricName, int64 DefaultValue);
	/* Set the value of an existing floating point metric. */
	UE_API bool SetFloat(const FName MetricName, float Value);
	/* Set the value of a floating point metric if it's smaller than the existing value. */
	UE_API bool SetMinFloat(const FName MetricName, float Value);
	/* Set the value of a floating point metric if it's bigger than the existing value. */
	UE_API bool SetMaxFloat(const FName MetricName, float Value);
	/** Get the value of an existing floating point metric. */
	UE_API float GetFloat(const FName MetricName);
	/* Set the value of an existing integer metric. */
	UE_API bool SetInt(const FName MetricName, int64 Value);
	/* Set the value of an integer metric if it's smaller than the existing value. */
	UE_API bool SetMinInt(const FName MetricName, int64 Value);
	/* Set the value of an integer metric if it's bigger than the existing value. */
	UE_API bool SetMaxInt(const FName MetricName, int64 Value);
	/* Increment the value of an existing integer metric. */
	UE_API bool IncrementInt(const FName MetricName, int64 Value);
	/** Get the value of an existing integer metric. */
	UE_API int64 GetInt(const FName MetricName);
	/* Returns true if a metric has been created in the database. */
	UE_API bool Contains(const FName MetricName) const;

	/* Call all registered listeners.*/
	UE_API void ProcessListeners();
	/* Remove all registered metrics and listeners. */
	UE_API void Reset();
	/* Remove all registered listeners. */
	UE_API void ResetListeners();

	/* Register a listener to be called for a given metric. */
	UE_API void Register(const FName MetricName, TWeakObjectPtr<UNetworkMetricsBaseListener> Reporter);

	/* Register a mutator with a listener. */
	UE_API void RegisterMutator(TWeakObjectPtr<UNetworkMetricsBaseListener> Listener, const TInstancedStruct<FNetworkMetricsMutator>& Mutator);

private:
	/* Return true if the report interval time for a listener has elapsed, and update its last reported time if it has. */
	bool HasReportIntervalPassed(double CurrentTimeSeconds, UNetworkMetricsBaseListener* Listener);

	enum class EMetricType { Integer, Float };

	void AddMetricToSnapshot(UE::Net::FNetworkMetricSnapshot& Snapshot, const FName MetricName, const EMetricType MetricType);

	TMap<FName, EMetricType> MetricTypes;

	TMap<FName, UE::Net::FNetworkMetric<int64>> MetricInts;
	TMap<FName, UE::Net::FNetworkMetric<float>> MetricFloats;

	using FNameAndType = TPair<FName, EMetricType>;

	/* The time, in seconds, metrics were reported to a listener. */
	TMap<TWeakObjectPtr<UNetworkMetricsBaseListener>, double> LastReportListener;

	// Map of listeners to metrics registered with them. A listener may have an entry with an empty set if it has no metrics but some mutators registered. */
	TMap<TWeakObjectPtr<UNetworkMetricsBaseListener>, TSet<FNameAndType>> ListenersToMetrics;
#if UE_SUPPORT_PARALLEL_IRIS
	mutable FTransactionallySafeRWLock NetworkMetricLock;
#endif // UE_SUPPORT_PARALLEL_IRIS
};

/** 
 * An abstract class for metrics listeners that are registered with FNetworkMetricsDatabase.
 * 
 * Listeners are the recommended method for reading the current value of metrics from FNetworkMetricsDatabase.
 * 
 * Begin by creating a sub-class of UNetworkMetricsBaseListener that overrides the Report() function. This function will be called 
 * by FNetworkMetricsDatabase::ProcessListeners() once a frame and will be provided an array of metrics that are registered 
 * with FNetworkMetricsDatabase::Register():
 *
 * UCLASS()
 * class UNetworkMetricsMyListener : public UNetworkMetricsBaseListener
 * {
 *		GENERATED_BODY()
 * public:
 *		virtual ~UNetworkMetricsMyListener() = default;
 * 
 *		void Report(const TArray<UE::Net::FNetworkMetricSnapshot>& Stats)
 *		{
 *			for (const UE::Net::FNetworkMetric<int64>& Metric : Snapshot.MetricInts)
 *			{
 *				// Do something with integer metrics...
 *			}
 *
 *			for (const UE::Net::FNetworkMetric<float>& Metric : Snapshot.MetricFloats)
 *			{
 *				// Do something with floating point metrics...
 *			}
 *		}
 * };
 *
 * Listeners can either be registered explicitly using FNetworkMetricsDatabase::Register() or through the engine configuration files. A configuration 
 * file is the prefered way to register a listener because it allows metrics reporting to be configured without rebuilding the application.
 *
 * This is an example configuration from an ini file (e.g. DefaultEngine.ini) that registers metrics with the example listener above:
 * 
 * [/Script/Engine.NetworkMetricsConfig]
 * +Listeners=(MetricName=ExampleMetric1, ClassName=/Script/Engine.NetworkMetricsMyListener)
 * 
 * All sub-classes of UNetworkMetricsBaseListener can set a time interval between calls to Report(). This is a useful method for limiting the rate
 * at which metrics need to be recorded (e.g. you may only want to report metrics to an external analytics services every 60 seconds). This time interval
 * can be set by calling UNetworkMetricsBaseListener::SetInterval() or in a configuration file by setting the IntervalSeconds property on the
 * listener sub-class.
 * 
 * This is an example configuration from an ini file (e.g. DefaultEngine.ini) that sets the interval between calling UNetworkMetricsMyListener::Report()
 * to 1 second:
 * 
 * [/Script/Engine.NetworkMetricsMyListener]
 * IntervalSeconds=1
 */
UCLASS(MinimalAPI, abstract, Config=Engine)
class UNetworkMetricsBaseListener : public UObject
{
	GENERATED_BODY()

public:
	UE_API UNetworkMetricsBaseListener();
	virtual ~UNetworkMetricsBaseListener();

	/* Set the interval, in seconds, between calling Report(). */
	void SetInterval(double Seconds)
	{
		if (ensureMsgf(Seconds >= 0, TEXT("SetInterval() called with a negative time interval.")))
		{
			IntervalSeconds = Seconds;
		}
	}

	/* Get the interval, in seconds, between calling Report(). */
	double GetInterval() const
	{
		return IntervalSeconds;
	}

	const TArray<TInstancedStruct<FNetworkMetricsMutator>>& GetMutators() const
	{
		return Mutators;
	}

	TArray<TInstancedStruct<FNetworkMetricsMutator>>& GetMutators()
	{
		return Mutators;
	}

	virtual void Report(const UE::Net::FNetworkMetricSnapshot& Snapshot) {};

private:
	UPROPERTY(Config)
	double IntervalSeconds;

	/** List of optional mutators for this listener. Configure them via UNetworkMetricsConfig. */
	UPROPERTY()
	TArray<TInstancedStruct<FNetworkMetricsMutator>> Mutators;
};

/** 
 * A metrics listener that prints stats to a log statement at a fixed frequency to the LogNetworkMetrics log category.
 *
 * To use UNetworkMetricsLog in a configuration file, the class must be used when registering a listener:
 * 
 * [/Script/Engine.NetworkMetricsConfig]
 * +Listeners=(MetricName=ExampleMetric, ClassName=/Script/Engine.NetworkMetricsLog)
 *
 * Additionally, the following log category must be set:
 *
 * [Core.Log]
 * LogNetworkMetrics=Log
 */
UCLASS(MinimalAPI)
class UNetworkMetricsLog : public UNetworkMetricsBaseListener
{
	GENERATED_BODY()

public:
	virtual ~UNetworkMetricsLog() = default;

	UE_API virtual void Report(const UE::Net::FNetworkMetricSnapshot& Snapshot);
};

/**
 * A metrics listener that reports an array of metrics to CSV.
 * 
 * The function SetCategory() is expected to be called before the listener is registered with
 * FNetworkMetricsDatabase::Register(). This function will associate the instance of UNetworkMetricsCSV
 * with a category of values in CSV.
 * 
 * To use UNetworkMetricsCSV in a configuration file, a sub-class of UNetworkMetricsCSV must be 
 * created that calls SetCategory() from the constructor to provide the CSV category to use.
 * 
 * UCLASS()
 * class UNetworkMetricsCSV_ExampleCategory : public UNetworkMetricsCSV
 * {
 *		GENERATED_BODY()
 * public:
 *		virtual ~UNetworkMetricsCSV_ExampleCategory() = default;
 * 
 *		UNetworkMetricsCSV_ExampleCategory() : UNetworkMetricsCSV()
 *		{
 *			SetCategory("ExampleCategory");
 *		}
 * };
 * 
 * This sub-class can then be used in the configuration file when registering a listener:
 * 
 *	[/Script/Engine.NetworkMetricsConfig]
 *	+Listeners=(MetricName=ExampleMetric, ClassName=/Script/Engine.NetworkMetricsCSV_ExampleCategory)
 * 
 * If the base UNetworkMetricsCSV class is used in the configuration file, CSV stats will be recorded to the default 'Networking' category.
 */
UCLASS(MinimalAPI)
class UNetworkMetricsCSV : public UNetworkMetricsBaseListener
{
	GENERATED_BODY()

public:
	UE_API UNetworkMetricsCSV();
	virtual ~UNetworkMetricsCSV() = default;

	/* Set the CSV category. */
	UE_API void SetCategory(const FString& CategoryName);

	UE_API virtual void Report(const UE::Net::FNetworkMetricSnapshot& Snapshot);

private:
	int32 CategoryIndex;
};

/** A metrics listener that writes a metric to the 'Replication' CSV category. */
UCLASS(MinimalAPI)
class UNetworkMetricsCSV_Replication : public UNetworkMetricsCSV
{
	GENERATED_BODY()

public:
	UNetworkMetricsCSV_Replication()
	{
		SetCategory("Replication");
	}

	virtual ~UNetworkMetricsCSV_Replication() = default;
};

/**
 * A metrics listener that reports an array of metrics to PerfCounters.
 * 
 * To use UNetworkMetricsPerfCounters in a configuration file, the class must be used when registering a listener:
 * 
 *	[/Script/Engine.NetworkMetricsConfig]
 *	+Listeners=(MetricName=ExampleMetric, ClassName=/Script/Engine.NetworkMetricsPerfCounters)
 */
UCLASS(MinimalAPI)
class UNetworkMetricsPerfCounters : public UNetworkMetricsBaseListener
{
	GENERATED_BODY()

public:
	virtual ~UNetworkMetricsPerfCounters() = default;

	UE_API virtual void Report(const UE::Net::FNetworkMetricSnapshot& Snapshot);
};

/**
 * A metrics listener that reports a metric to a single Stat. 
 * 
 * The function SetStatName() is expected to be called before the listener is registered with
 * FNetworkMetricsDatabase::Register(). This function will associate the instance of UNetworkMetricStats
 * with a specific Stat.
 * 
 * Since each instance of this class is associated with a single Stat it can only be registered
 * as a listener to a single metric in FNetworkMetricsDatabase.
 * 
 * UNetworkMetricsStats is not intended to be used from configuration files!
 */
UCLASS(MinimalAPI)
class UNetworkMetricsStats : public UNetworkMetricsBaseListener
{
	GENERATED_BODY()

public:
	UE_API UNetworkMetricsStats();
	virtual ~UNetworkMetricsStats() = default;

	/** Set the name of the pre-defined Stat (normally defined with DEFINE_STAT()). */
	void SetStatName(const FName Name)
	{
		StatName = Name;
	}

	UE_API virtual void Report(const UE::Net::FNetworkMetricSnapshot& Snapshot);

private:
	FName StatName;
};

#undef UE_API
