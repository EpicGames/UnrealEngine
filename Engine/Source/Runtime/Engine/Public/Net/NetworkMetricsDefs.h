// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Net::Metric
{

extern ENGINE_API const FName AddedConnections;
extern ENGINE_API const FName ClosedConnectionsDueToReliableBufferOverflow;
extern ENGINE_API const FName InPackets;
extern ENGINE_API const FName OutPackets;
extern ENGINE_API const FName InPacketsPerFrame;
extern ENGINE_API const FName OutPacketsPerFrame;
extern ENGINE_API const FName InLostPacketsFoundPerFrame;
extern ENGINE_API const FName OutLostPacketsFoundPerFrame;
extern ENGINE_API const FName InPacketsClientPerSecondAvg;
extern ENGINE_API const FName InPacketsClientPerSecondMax;
extern ENGINE_API const FName InPacketsClientPerSecondMin;
extern ENGINE_API const FName OutPacketsClientPerSecondAvg;
extern ENGINE_API const FName OutPacketsClientPerSecondMax;
extern ENGINE_API const FName OutPacketsClientPerSecondMin;
extern ENGINE_API const FName InPacketsClientAvg;
extern ENGINE_API const FName InPacketsClientMax;
extern ENGINE_API const FName OutPacketsClientAvg;
extern ENGINE_API const FName OutPacketsClientMax;
extern ENGINE_API const FName InRate;
extern ENGINE_API const FName OutRate;
extern ENGINE_API const FName InRateClientAvg;
extern ENGINE_API const FName InRateClientMax;
extern ENGINE_API const FName InRateClientMin;
extern ENGINE_API const FName OutRateClientAvg;
extern ENGINE_API const FName OutRateClientMax;
extern ENGINE_API const FName OutRateClientMin;
extern ENGINE_API const FName InPacketsLost;
extern ENGINE_API const FName OutPacketsLost;
extern ENGINE_API const FName AverageJitterInMS;
extern ENGINE_API const FName InBunches;
extern ENGINE_API const FName OutBunches;
extern ENGINE_API const FName Ping;
extern ENGINE_API const FName AvgPing;
extern ENGINE_API const FName MinPing;
extern ENGINE_API const FName MaxPing;
extern ENGINE_API const FName OutKBytes;
extern ENGINE_API const FName OutNetGUIDKBytesSec;
extern ENGINE_API const FName ServerReplicateActorTimeMS;
extern ENGINE_API const FName GatherPrioritizeTimeMS;
extern ENGINE_API const FName ReplicateActorTimeMS;
extern ENGINE_API const FName NumReplicatedActors;
extern ENGINE_API const FName Connections;
extern ENGINE_API const FName NumReplicateActorCallsPerConAvg;
extern ENGINE_API const FName NumberOfActiveActors;
extern ENGINE_API const FName NumberOfFullyDormantActors;
extern ENGINE_API const FName NumSkippedObjectEmptyUpdates;
extern ENGINE_API const FName NumOpenChannels;
extern ENGINE_API const FName NumTickingChannels;
extern ENGINE_API const FName SatConnections;
extern ENGINE_API const FName SharedSerializationPropertyHit;
extern ENGINE_API const FName SharedSerializationPropertyMiss;
extern ENGINE_API const FName SharedSerializationRPCHit;
extern ENGINE_API const FName SharedSerializationRPCMiss;
extern ENGINE_API const FName NumClientUpdateLevelVisibility;
extern ENGINE_API const FName Channels;
extern ENGINE_API const FName NumActorChannels;
extern ENGINE_API const FName NumDormantActors;
extern ENGINE_API const FName NumActors;
extern ENGINE_API const FName NumNetActors;
extern ENGINE_API const FName NumNetGUIDsAckd;
extern ENGINE_API const FName NumNetGUIDsPending;
extern ENGINE_API const FName NumNetGUIDsUnAckd;
extern ENGINE_API const FName NetSaturated;
extern ENGINE_API const FName MaxPacketOverhead;
extern ENGINE_API const FName NetGUIDInRate;
extern ENGINE_API const FName NetGUIDOutRate;
extern ENGINE_API const FName NetNumClients;
extern ENGINE_API const FName NumClients;
extern ENGINE_API const FName NumConnections;
extern ENGINE_API const FName NumAllConnections;
extern ENGINE_API const FName NumConsideredActors;
extern ENGINE_API const FName NumInitiallyDormantActors;
extern ENGINE_API const FName PrioritizedActors;
extern ENGINE_API const FName NumRelevantDeletedActors;
extern ENGINE_API const FName NumReplicatedActorBytes;
extern ENGINE_API const FName ImportedNetGuids;
extern ENGINE_API const FName PendingOuterNetGuids;
extern ENGINE_API const FName NetInBunchTimeOvershootPercent;
extern ENGINE_API const FName UnmappedReplicators;
extern ENGINE_API const FName PercentInVoice;
extern ENGINE_API const FName PercentOutVoice;
extern ENGINE_API const FName VoiceBytesRecv;
extern ENGINE_API const FName VoiceBytesSent;
extern ENGINE_API const FName VoicePacketsRecv;
extern ENGINE_API const FName VoicePacketsSent;
extern ENGINE_API const FName PingBucketInt0;
extern ENGINE_API const FName PingBucketInt1;
extern ENGINE_API const FName PingBucketInt2;
extern ENGINE_API const FName PingBucketInt3;
extern ENGINE_API const FName PingBucketInt4;
extern ENGINE_API const FName PingBucketInt5;
extern ENGINE_API const FName PingBucketInt6;
extern ENGINE_API const FName PingBucketInt7;
extern ENGINE_API const FName OutgoingReliableMessageQueueMaxSize;
extern ENGINE_API const FName IncomingReliableMessageQueueMaxSize;
extern ENGINE_API const FName RawPing;
extern ENGINE_API const FName RawPingClientMin;
extern ENGINE_API const FName RawPingClientMax;
extern ENGINE_API const FName RawPingClientAvg;
extern ENGINE_API const FName NumPlayerControllersForReplication;
extern ENGINE_API const FName NumNoPawnPlayerControllersForReplication;
extern ENGINE_API const FName NumPawnsForReplication;
extern ENGINE_API const FName NumReplicationViewers;
}
