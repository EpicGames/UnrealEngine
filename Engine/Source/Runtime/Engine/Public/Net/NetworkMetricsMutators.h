// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/ArrayView.h"
#include "Misc/TVariant.h"
#include "StructUtils/InstancedStruct.h"

#include <limits>

#include "NetworkMetricsMutators.generated.h"

namespace UE::Net
{
	struct FNetworkMetricSnapshot;
}

/**
 * A metrics mutator is a component of a UNetworkMetricsBaseListener that receices a metrics snapshot
 * every frame. This enables a mutator to accumulate and/or transform (possibly multiple) input metrics
 * into a single output value over the reporting interval of its owning listener.
 * For example, FNetworkMetricsMutatorAvg allows a listener to report the average of a metric over
 * a whole reporting interval.
 */
USTRUCT()
struct FNetworkMetricsMutator
{
	GENERATED_BODY()

public:
	using FValueVariant = TVariant<float, int64>;

	virtual ~FNetworkMetricsMutator() = default;

	/**
	 * Returns the name of this mutator. Independent of metric names in the database.
	 * Either NameOverride if set, otherwise returns GetNameInternal().
	 */
	FName GetName() const;

	/** Called every frame with a snapshot containing this mutator's metrics. */
	virtual void ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot) PURE_VIRTUAL(FNetworkMetricsMutator::ProcessFrame, );

	/** Clears internal data, typically used after a listener reports its metrics. */
	virtual void Reset() PURE_VIRTUAL(FNetworkMetricsMutator::Reset, );

	/** Returns this mutator's current computed value */
	virtual FValueVariant GetValue() const PURE_VIRTUAL(FNetworkMetricsMutator::GetValue, return FValueVariant(TInPlaceType<float>(), 0.0f););

	/** Returns a list of all metrics used by this mutator. */
	virtual TArrayView<const FName> GetAllMetricNames() const PURE_VIRTUAL(FNetworkMetricsMutator::GetAllMetricNames, return TArrayView<const FName>(););

private:
	/** Returns the name of this mutator. Independent of metric names in the database. */
	virtual FName GetNameInternal() const PURE_VIRTUAL(FNetworkMetricsMutator::GetNameInternal, return FName(););

	/**
	 * User-configurable name for this mutator that if set, will override the internal name returned by GetNameInternal().
	 * Use it to disambiguate naming when the internal name would collide with another on the same listener,
	 * or to clarify an internal name that doesn't work very well (for example, RawPingClientMaxMax).
	 */
	UPROPERTY(Config)
	FName NameOverride;
};

template<>
struct TStructOpsTypeTraits<FNetworkMetricsMutator> : public TStructOpsTypeTraitsBase2<FNetworkMetricsMutator>
{
	enum
	{
		WithPureVirtual = true
	};
};

/** Metrics mutator that accumulates an average of a metric. */
USTRUCT()
struct FNetworkMetricsMutatorAvg : public FNetworkMetricsMutator
{
	GENERATED_BODY()

public:
	virtual void ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot) override;
	virtual void Reset() override;
	virtual FValueVariant GetValue() const override;

	virtual TArrayView<const FName> GetAllMetricNames() const override
	{
		return MakeConstArrayView(&MetricName, 1);
	}

private:
	virtual FName GetNameInternal() const override;

	void AddIntSample(int64 Sample);
	void AddFloatSample(float Sample);

	int64 IntTotal = 0;
	float FloatTotal = 0.0f;
	uint32 NumSamples = 0;

	/** The network metric to average. */
	UPROPERTY(Config)
	FName MetricName;

	/** The default name of this mutator, derived from MetricName */
	mutable FName MutatorName;
};

/** Metrics mutator that tracks the minimum of a metric. */
USTRUCT()
struct FNetworkMetricsMutatorMin : public FNetworkMetricsMutator
{
	GENERATED_BODY()

public:
	virtual void ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot) override;
	virtual void Reset() override;
	virtual FValueVariant GetValue() const override;

	virtual TArrayView<const FName> GetAllMetricNames() const override
	{
		return MakeConstArrayView(&MetricName, 1);
	}

private:
	virtual FName GetNameInternal() const override;

	FValueVariant MinValue = FValueVariant(TInPlaceType<int64>(), std::numeric_limits<int64>::max());

	/** The network metric to track the min of. */
	UPROPERTY(Config)
	FName MetricName;

	/** The default name of this mutator, derived from MetricName */
	mutable FName MutatorName;
};

/** Metrics mutator that tracks the maximum of a metric. */
USTRUCT()
struct FNetworkMetricsMutatorMax : public FNetworkMetricsMutator
{
	GENERATED_BODY()

public:
	virtual void ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot) override;
	virtual void Reset() override;
	virtual FValueVariant GetValue() const override;

	virtual TArrayView<const FName> GetAllMetricNames() const override
	{
		return MakeConstArrayView(&MetricName, 1);
	}

private:
	virtual FName GetNameInternal() const override;

	FValueVariant MaxValue = FValueVariant(TInPlaceType<int64>(), std::numeric_limits<int64>::lowest());

	/** The network metric to track the max of. */
	UPROPERTY(Config)
	FName MetricName;

	/** The default name of this mutator, derived from MetricName */
	mutable FName MutatorName;
};

/**
 * Metrics mutator that reports a 0-100 percentage using two source metrics.
 */
USTRUCT()
struct FNetworkMetricsMutatorPercent : public FNetworkMetricsMutator
{
	GENERATED_BODY()

public:
	virtual void ProcessFrame(const UE::Net::FNetworkMetricSnapshot& Snapshot) override;
	virtual void Reset() override;

	/** Returns the packet loss percentage as a float from 0-100. */
	virtual FValueVariant GetValue() const override;

	virtual TArrayView<const FName> GetAllMetricNames() const override;

protected:
	void SetNumeratorMetricName(FName MetricName)
	{
		NumeratorName = MetricName;
	}

	void SetDenominatorMetricName(FName MetricName)
	{
		DenominatorName = MetricName;
	}

private:
	virtual FName GetNameInternal() const override;

	// Private template so it can be defined in the .cpp file.
	template<class T>
	static void UpdateFromSnapshotValue(FNetworkMetricsMutator::FValueVariant& InOutValue, T MetricValue);

	// Source metric names stored in an array so we can return a view to it from GetAllMetricNames.
	mutable TArray<FName, TInlineAllocator<2>> SourceMetricNames;

	FValueVariant Numerator = FValueVariant(TInPlaceType<int64>(), 0);
	FValueVariant Denominator = FValueVariant(TInPlaceType<int64>(), 0);

	UPROPERTY(Config)
	FName NumeratorName;

	UPROPERTY(Config)
	FName DenominatorName;

	// Auto-generated mutator name, derived from source metrics.
	mutable FName MutatorName;
};

/** Metrics mutator that tracks out packet loss, based on the OutLostPacketsFoundPerFrame and OutPacketsPerFrame metrics. */
USTRUCT()
struct FNetworkMetricsMutatorOutPacketLoss : public FNetworkMetricsMutatorPercent
{
	GENERATED_BODY()

public:
	FNetworkMetricsMutatorOutPacketLoss();

private:
	virtual FName GetNameInternal() const override
	{
		return FName("OutPacketLoss");
	}
};

/** Metrics mutator that tracks out packet loss, based on the InLostPacketsFoundPerFrame and InPacketsPerFrame metrics. */
USTRUCT()
struct FNetworkMetricsMutatorInPacketLoss : public FNetworkMetricsMutatorPercent
{
	GENERATED_BODY()

public:
	FNetworkMetricsMutatorInPacketLoss();

private:
	virtual FName GetNameInternal() const override
	{
		return FName("InPacketLoss");
	}
};
