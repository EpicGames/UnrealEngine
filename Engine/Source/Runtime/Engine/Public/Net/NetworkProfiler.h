// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkProfiler.h: network profiling support.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "IPAddress.h"
#include "Containers/BitArray.h"
#include "Serialization/MemoryWriter.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"

class AActor;
class FOutBunch;
class UNetConnection;
struct FURL;

#if USE_NETWORK_PROFILER 

CSV_DECLARE_CATEGORY_EXTERN(NetworkProfiler);

#define NETWORK_PROFILER( x ) \
	if ( GNetworkProfiler.IsTrackingEnabled() ) \
	{ \
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(NetworkProfiler); \
		x; \
	}

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNetworkProfileStarted, const FString& /*Filename */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNetworkProfileFinished, const FString& /*Filename */);

enum class ENetworkProfilerVersionHistory : uint32;

/*=============================================================================
	Network profiler header.
=============================================================================*/

class FNetworkProfilerHeader
{
private:
	/** Magic to ensure we're opening the right file.	*/
	uint32	Magic;
	/** Version number to detect version mismatches.	*/
	ENetworkProfilerVersionHistory	Version;

	/** Tag, set via -networkprofiler=TAG				*/
	FString Tag;
	/** Game name, e.g. Example							*/
	FString GameName;
	/** URL used to open/ browse to the map.			*/
	FString URL;

public:
	/** Constructor.									*/
	FNetworkProfilerHeader();

	/** Resets the header info for a new session.		*/
	void Reset(const FURL& InURL);

	/** Returns the URL stored in the header.			*/
	FString GetURL() const { return URL; }

	/**
	 * Serialization operator.
	 *
	 * @param	Ar			Archive to serialize to
	 * @param	Header		Header to serialize
	 * @return	Passed in archive
	 */
	friend FArchive& operator << ( FArchive& Ar, FNetworkProfilerHeader& Header );
};

/*=============================================================================
	FNetworkProfiler
=============================================================================*/

/**
 * Network profiler, using serialized token emission like e.g. script and malloc profiler.
 */
class FNetworkProfiler
{
private:

	friend struct FNetworkProfilerScopedIgnoreReplicateProperties;
	friend struct FNetworkProfilerCVarHelper;

	/** Whether or not want to track granular information about comparisons. This can be very expensive. */
	static ENGINE_API bool bIsComparisonTrackingEnabled;

	/** File writer used to serialize data.															*/
	FArchive*								FileWriter;

	/** Critical section to sequence tracking.														*/
	FTransactionallySafeCriticalSection		CriticalSection;

	/** Mapping from name to index in name array.													*/
	TMap<FString,int32>						NameToNameTableIndexMap;

	/** Array of unique names.																		*/
	TArray<FString>							NameArray;

	/** Mapping from address to index in address array.												*/
	TMap<FString, int32>					AddressTableIndexMap;

	/** Whether noticeable network traffic has occured in this session. Used to discard it.			*/
	bool									bHasNoticeableNetworkTrafficOccured;
	/** Whether tracking is enabled. Set after a session change.									*/
	bool									bIsTrackingEnabled;	
	/** Whether tracking should be enabled.															*/
	bool									bShouldTrackingBeEnabled;

	/** Header for the current session.																*/
	FNetworkProfilerHeader					CurrentHeader;

	/** Last known address																			*/
	TSharedPtr<const FInternetAddr>			LastAddress;

	/** Delegate that's fired when tracking starts on the current network profile */
	FOnNetworkProfileStarted				OnNetworkProfileStartedDelegate;

	/** Delegate that's fired when tracking stops on the current network profile */
	FOnNetworkProfileFinished				OnNetworkProfileFinishedDelegate;

	FTimerHandle							AutoStopTimerHandle;

	/** All the data required for writing sent bunches to the profiler stream						*/
	struct FSendBunchInfo
	{
		uint16 ChannelIndex;
		uint32 ChannelTypeNameIndex;
		uint16 NumHeaderBits;
		uint16 NumPayloadBits;

		FSendBunchInfo()
			: ChannelIndex(0)
			, ChannelTypeNameIndex(0)
			, NumHeaderBits(0)
			, NumPayloadBits(0) {}

		FSendBunchInfo( uint16 InChannelIndex, uint32 InChannelTypeNameIndex, uint16 InNumHeaderBits, uint16 InNumPayloadBits )
			: ChannelIndex(InChannelIndex)
			, ChannelTypeNameIndex(InChannelTypeNameIndex)
			, NumHeaderBits(InNumHeaderBits)
			, NumPayloadBits(InNumPayloadBits) {}
	};

	/** Stack outgoing bunches per connection, the top bunch for a connection may be popped if it gets merged with a new bunch.		*/
	TMap<UNetConnection*, TArray<FSendBunchInfo>>	OutgoingBunches;

	/** Data required to write queued RPCs to the profiler stream */
	struct FQueuedRPCInfo
	{
		UNetConnection* Connection;
		UObject* TargetObject;
		uint32 ActorNameIndex;
		uint32 FunctionNameIndex;
		uint32 NumHeaderBits;
		uint32 NumParameterBits;
		uint32 NumFooterBits;

		FQueuedRPCInfo()
			: Connection(nullptr)
			, TargetObject(nullptr)
			, ActorNameIndex(0)
			, FunctionNameIndex(0)
			, NumHeaderBits(0)
			, NumParameterBits(0)
			, NumFooterBits(0) {}
	};
	
	TArray<FQueuedRPCInfo> QueuedRPCs;

	/**
	 * Returns index of passed in name into name array. If not found, adds it.
	 *
	 * @param	Name	Name to find index for
	 * @return	Index of passed in name
	 */
	ENGINE_API int32 GetNameTableIndex( const FString& Name );

	/**
	* Returns index of passed in address into address array. If not found, adds it.
	*
	* @param	Address	Address to find index for
	* @return	Index of passed in name
	*/
	ENGINE_API int32 GetAddressTableIndex( const FString& Address );

	ENGINE_API void TrackCompareProperties_Unsafe(const FString& ObjectName, uint32 Cycles, TBitArray<>& PropertiesCompared, TBitArray<>& PropertiesThatChanged, TArray<uint8>& PropertyNameExportData);

	// Used with FScopedIgnoreReplicateProperties.
	uint32 IgnorePropertyCount;

	// Set of names that correspond to Object's whose top level property names have been exported.
	TSet<FString> ExportedObjects;

	ENGINE_API void AutoStopTracking();

	/**
	 * Sets the current connection to Connection and serializes the property.
	 * The CriticalSection lock must be held while calling.
	 *
	 * @param	NameTableIndex	Name of the property being replicated
	 * @param	NumBits			Number of bits used to replicate this property
	 */
	void SerializeReplicateProperty( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection );
	
	/**
	 * Sets the current connection to Connection and serializes the property header.
	 * The CriticalSection lock must be held while calling.
	 *
	 * @param	NameTableIndex	Name of the property being replicated
	 * @param	NumBits			Number of bits used in the header
	 */
	void SerializeWritePropertyHeader( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Sets the current connection to Connection and serializes the property handle.
	 * The CriticalSection lock must be held while calling.
	 *
	 * @param NumBits Number of bits in the property handle
	 */
	void SerializeWritePropertyHandle( uint16 NumBits, UNetConnection* Connection );

	/**
	 * Sets the current connection to Connection and serializes the start of actor content block headers
	 *
	 * @param NameTableIndex the name of the object being replicated (might be a subobject of the actor)
	 * @param NumBits the number of bits in the content block header
	 */
	void SerializeBeginContentBlock( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Sets the current connection to Connection and serializes the end of actor content block headers
	 *
	 * @param NameTableIndex the name of the object being replicated (might be a subobject of the actor)
	 * @param NumBits the number of bits in the content block header
	 */
	void SerializeEndContentBlock( uint32 NameTableIndex, uint16 NumBits, UNetConnection* Connection );
 
	
	/** The filename to use next time a file writer is opened.  If empty then an autogenerated name will be used. */
	FString NextFileName;

public:
	/**
	 * Constructor, initializing members.
	 */
	ENGINE_API FNetworkProfiler();

	/**
	 * Enables/ disables tracking. Emits a session changes if disabled.
	 *
	 * @param	bShouldEnableTracking	Whether tracking should be enabled or not
	 */
	ENGINE_API void EnableTracking( bool bShouldEnableTracking );

	/** Force the next profiling file to use the given filename. If none are set an autogenerated name will be used instead */
	ENGINE_API void SetNextFileName(const FString& FileName);

	/**
	 * Marks the beginning of a frame.
	 */
	ENGINE_API void TrackFrameBegin();

	/**
	* Tracks when connection address changes
	*/
	ENGINE_API void SetCurrentConnection( UNetConnection* Connection );

	
	/**
	 * Tracks and RPC being sent.
	 * 
	 * @param	Actor				Actor RPC is being called on
	 * @param	Function			Function being called
	 * @param	NumHeaderBits		Number of bits serialized into the header for this RPC
	 * @param	NumParameterBits	Number of bits serialized into parameters of this RPC
	 * @param	NumFooterBits		Number of bits serialized into the footer of this RPC (EndContentBlock)
	 */
	ENGINE_API void TrackSendRPC(const AActor* Actor, const UFunction* Function, uint32 NumHeaderBits, uint32 NumParameterBits, uint32 NumFooterBits, UNetConnection* Connection);
	
	/**
	 * Tracks queued RPCs (unreliable multicast) being sent.
	 * 
	 * @param	Connection			The connection on which this RPC is queued
	 * @param	TargetObject		The target object of the RPC
	 * @param	Actor				Actor RPC is being called on
	 * @param	Function			Function being called
	 * @param	NumHeaderBits		Number of bits serialized into the header for this RPC
	 * @param	NumParameterBits	Number of bits serialized into parameters of this RPC
	 * @param	NumFooterBits		Number of bits serialized into the footer of this RPC (EndContentBlock)
	 */
	ENGINE_API void TrackQueuedRPC(UNetConnection* Connection, UObject* TargetObject, const AActor* Actor, const UFunction* Function, uint32 NumHeaderBits, uint32 NumParameterBits, uint32 NumFooterBits);

	/**
	 * Writes all queued RPCs for the connection to the profiler stream
	 *
	 * @param Connection The connection for which RPCs are being flushed
	 * @param TargetObject The target object of the RPC
	 */
	ENGINE_API void FlushQueuedRPCs( UNetConnection* Connection, UObject* TargetObject );

	/**
	 * Low level FSocket::Send information.
	 *
	 * @param	SocketDesc				Description of socket data is being sent to
	 * @param	Data					Data sent
	 * @param	BytesSent				Bytes actually being sent
	 */
	ENGINE_API void TrackSocketSend( const FString& SocketDesc, const void* Data, uint16 BytesSent );

	/**
	 * Low level FSocket::SendTo information.
	 *
 	 * @param	SocketDesc				Description of socket data is being sent to
	 * @param	Data					Data sent
	 * @param	BytesSent				Bytes actually being sent
	 * @param	NumPacketIdBits			Number of bits sent for the packet id
	 * @param	NumBunchBits			Number of bits sent in bunches
	 * @param	NumAckBits				Number of bits sent in acks
	 * @param	NumPaddingBits			Number of bits appended to the end to byte-align the data
	 * @param	Connection				Destination address
	 */
	ENGINE_API void TrackSocketSendTo(
		const FString& SocketDesc,
		const void* Data,
		uint16 BytesSent,
		uint16 NumPacketIdBits,
		uint16 NumBunchBits,
		uint16 NumAckBits,
		uint16 NumPaddingBits,
		UNetConnection* Connection );

	/**
	 * Low level FSocket::SendTo information.
	 *
 	 * @param	SocketDesc				Description of socket data is being sent to
	 * @param	Data					Data sent
	 * @param	BytesSent				Bytes actually being sent
	 * @param	Connection				Destination address
	 */
	ENGINE_API void TrackSocketSendToCore(
		const FString& SocketDesc,
		const void* Data,
		uint16 BytesSent,
		uint16 NumPacketIdBits,
		uint16 NumBunchBits,
		uint16 NumAckBits,
		uint16 NumPaddingBits,
		UNetConnection* Connection );

	
	/**
	 * Mid level UChannel::SendBunch information.
	 * 
	 * @param	OutBunch	FOutBunch being sent
	 * @param	NumBits		Num bits to serialize for this bunch (not including merging)
	 */
	ENGINE_API void TrackSendBunch( FOutBunch* OutBunch, uint16 NumBits, UNetConnection* Connection );
	
	/**
	 * Add a sent bunch to the stack. These bunches are not written to the stream immediately,
	 * because they may be merged with another bunch in the future.
	 *
	 * @param Connection The connection on which this bunch was sent
	 * @param OutBunch The bunch being sent
	 * @param NumHeaderBits Number of bits in the bunch header
	 * @param NumPayloadBits Number of bits in the bunch, excluding the header
	 */
	ENGINE_API void PushSendBunch( UNetConnection* Connection, FOutBunch* OutBunch, uint16 NumHeaderBits, uint16 NumPayloadBits );

	/**
	 * Pops the latest bunch for a connection, since it is going to be merged with the next bunch.
	 *
	 * @param Connection the connection which is merging a bunch
	 */
	ENGINE_API void PopSendBunch( UNetConnection* Connection );

	/**
	 * Writes all the outgoing bunches for a connection in the stack to the profiler data stream.
	 *
	 * @param Connection the connection which is about to send any pending bunches over the network
	 */
	ENGINE_API void FlushOutgoingBunches( UNetConnection* Connection );

	/**
	 * Track actor being replicated.
	 *
	 * @param	Actor		Actor being replicated
	 */
	ENGINE_API void TrackReplicateActor( const AActor* Actor, FReplicationFlags RepFlags, uint32 Cycles, UNetConnection* Connection );

	/**
	 * Track a set of metadata for a ReplicateProperties call.
	 *
	 * @param	Object				Object being replicated
	 * @param	InactiveProperties	Bitfield describing the properties that are inactive.
	 * @param	bWasAnythingSent	Whether or not any properties were actually replicated.
	 * @param	bSentAlProperties	Whether or not we're going to try sending all the properties from the beginning of time.
	 * @param	Connection			The connection that we're replicating properties to.
	 */
	ENGINE_API void TrackReplicatePropertiesMetadata(const UObject* Object, TBitArray<>& InactiveProperties, bool bSentAllProperties, UNetConnection* Connection);

	/**
	 * Track time used to compare properties for a given object.
	 *
	 * @param	Object					Object being replicated
	 * @param	Cycles					The number of CPU Cycles we spent comparing the properties for this object.
	 * @param	PropertiesCompared		The properties that were compared (only tracks top level properties).
	 * @param	PropertiesThatChanged	The properties that actually changed (only tracks top level properties).
	 * @param	PropertyNameContainers	Array of items that we can convert to property names if we need to export them (should only happen the first time we see a given object).
	 * @param	PropertyNameProjection	Project that can be used to convery a PropertyNameContainer to a usable property name.
	 */
	template<typename T, typename ProjectionType>
	void TrackCompareProperties(const UObject* Object, uint32 Cycles, TBitArray<>& PropertiesCompared, TBitArray<>& PropertiesThatChanged, const TArray<T>& PropertyNameContainers, ProjectionType PropertyNameProjection)
	{
		if (IsComparisonTrackingEnabled())
		{
			UE::TScopeLock ScopeLock(CriticalSection);

			FString ObjectName = GetNameSafe(Object);
			TArray<uint8> PropertyNameExportData;

			if (!ExportedObjects.Contains(ObjectName))
			{
				uint32 NumProperties = PropertyNameContainers.Num();
				PropertyNameExportData.Reserve(2 + (NumProperties * 2));
				FMemoryWriter PropertyNameAr(PropertyNameExportData);

				PropertyNameAr.SerializeIntPacked(NumProperties);
				for (const T& PropertyNameContainer : PropertyNameContainers)
				{
					uint32 PropertyNameIndex = GetNameTableIndex(PropertyNameProjection(PropertyNameContainer));
					PropertyNameAr.SerializeIntPacked(PropertyNameIndex);
				}
			}

			TrackCompareProperties_Unsafe(ObjectName, Cycles, PropertiesCompared, PropertiesThatChanged, PropertyNameExportData);
		}
	}

	/**
	 * Track property being replicated.
	 *
	 * @param	Property	Property being replicated
	 * @param	NumBits		Number of bits used to replicate this property
	 */
	ENGINE_API void TrackReplicateProperty( const FProperty* Property, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track property header being written.
	 *
	 * @param	Property	Property being replicated
	 * @param	NumBits		Number of bits used in the header
	 */
	ENGINE_API void TrackWritePropertyHeader( const FProperty* Property, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track event occuring, like e.g. client join/ leave
	 *
	 * @param	EventName			Name of the event
	 * @param	EventDescription	Additional description/ information for event
	 */
	ENGINE_API void TrackEvent( const FString& EventName, const FString& EventDescription, UNetConnection* Connection );

	/**
	 * Called when the server first starts listening and on round changes or other
	 * similar game events. We write to a dummy file that is renamed when the current
	 * session ends.
	 *
	 * @param	bShouldContinueTracking		Whether to continue tracking
	 * @param	InURL						URL used for new session
	 */
	ENGINE_API void TrackSessionChange( bool bShouldContinueTracking, const FURL& InURL );

	/**
	 * Track sent acks.
	 *
	 * @param NumBits Number of bits in the ack
	 */
	ENGINE_API void TrackSendAck( uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track NetGUID export bunches.
	 *
	 * @param NumBits Number of bits in the GUIDs
	 */
	ENGINE_API void TrackExportBunch( uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track "must be mapped" GUIDs
	 *
	 * @param NumGuids Number of GUIDs added to the bunch
	 * @param NumBits Number of bits added to the bunch for the GUIDs
	 */
	ENGINE_API void TrackMustBeMappedGuids( uint16 NumGuids, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track actor content block headers
	 *
	 * @param Object the object being replicated (might be a subobject of the actor)
	 * @param NumBits the number of bits in the content block header
	 */
	ENGINE_API void TrackBeginContentBlock( UObject* Object, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track actor content block headers
	 *
	 * @param Object the object being replicated (might be a subobject of the actor)
	 * @param NumBits the number of bits in the content block footer
	 */
	ENGINE_API void TrackEndContentBlock( UObject* Object, uint16 NumBits, UNetConnection* Connection );

	/**
	 * Track property handles
	 *
	 * @param NumBits Number of bits in the property handle
	 */
	ENGINE_API void TrackWritePropertyHandle( uint16 NumBits, UNetConnection* Connection );

	/**
	 * Processes any network profiler specific exec commands
	 *
	 * @param InWorld	The world in this context
	 * @param Cmd		The command to parse
	 * @param Ar		The output device to write data to
	 *
	 * @return			True if processed, false otherwise
	 */
	ENGINE_API bool Exec( UWorld * InWorld, const TCHAR* Cmd, FOutputDevice & Ar );

	bool inline IsTrackingEnabled() const { return bIsTrackingEnabled || bShouldTrackingBeEnabled; }
	bool IsComparisonTrackingEnabled() const { return (bIsTrackingEnabled || bShouldTrackingBeEnabled) && bIsComparisonTrackingEnabled; }

	/** Return the network profile started delegate */
	FOnNetworkProfileStarted& OnNetworkProfileStarted() { return OnNetworkProfileStartedDelegate; }

	/** Return the network profile finished delegate */
	FOnNetworkProfileFinished& OnNetworkProfileFinished() { return OnNetworkProfileFinishedDelegate; }
};

/** Global network profiler instance. */
extern ENGINE_API FNetworkProfiler GNetworkProfiler;

#define NETWORK_PROFILER_IGNORE_PROPERTY_SCOPE const FNetworkProfilerScopedIgnoreReplicateProperties _NetProfilePrivate_IgnoreScope;

/**
 * Can be used to enforce a scope where we don't want to track properties.
 * This is useful to prevent cases where we might inadvertently over count properties.
 */
struct FNetworkProfilerScopedIgnoreReplicateProperties
{
	FNetworkProfilerScopedIgnoreReplicateProperties()
	{
		++GNetworkProfiler.IgnorePropertyCount;
	}

	~FNetworkProfilerScopedIgnoreReplicateProperties()
	{
		--GNetworkProfiler.IgnorePropertyCount;
	}

private:

	FNetworkProfilerScopedIgnoreReplicateProperties(const FNetworkProfilerScopedIgnoreReplicateProperties&) = delete;
	FNetworkProfilerScopedIgnoreReplicateProperties(FNetworkProfilerScopedIgnoreReplicateProperties&&) = delete;
};

#else	// USE_NETWORK_PROFILER

#define NETWORK_PROFILER(x)
#define NETWORK_PROFILER_IGNORE_PROPERTY_SCOPE 

#endif
