// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "OnlineBlueprintCallProxyBase.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FEmptyOnlineDelegate);

UCLASS(MinimalAPI)
class UOnlineBlueprintCallProxyBase : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

};
