// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Online/CoreOnline.h"
#include "OnlineEngineInterface.generated.h"

class FVoicePacket;
class UNetConnection;
struct FWorldContext;

/** Delegate fired when an AutoLogin request is complete */
DECLARE_DELEGATE_ThreeParams(FOnlineAutoLoginComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*Error*/);
/** Delegate fired when an online StartSession call has completed */
DECLARE_DELEGATE_TwoParams(FOnlineSessionStartComplete, FName /*InSessionName*/, bool /*bWasSuccessful*/);
/** Delegate fired when an online EndSession call has completed */
DECLARE_DELEGATE_TwoParams(FOnlineSessionEndComplete, FName /*InSessionName*/, bool /*bWasSuccessful*/);
/** Delegate fired when an external UI is opened or closed */
DECLARE_DELEGATE_OneParam(FOnlineExternalUIChanged, bool /*bInIsOpening*/);
/** Delegate executed when the web url UI has been closed */
DECLARE_DELEGATE_OneParam(FOnlineShowWebUrlClosed, const FString& /*FinalUrl*/);
/** Delegate fired when a PIE login has completed */
DECLARE_DELEGATE_ThreeParams(FOnPIELoginComplete, int32 /*LocalUserNum*/, bool /*bWasSuccessful*/, const FString& /*Error*/);

/**
 * Interface class between engine and OnlineSubsystem to remove dependencies between the two
 * The real work is implemented in UOnlineEngineInterfaceImpl in OnlineSubsystemUtils
 * The expectation is that this basic group of functions will not expand/change and that 
 * OnlineSubsystem can remain independent of the engine
 *
 * Games should not use this interface, use the OnlineSubsystem plugins directly
 *
 * Better functionality descriptions can be found in the OnlineSubsystem interfaces 
 *
 * Adding code here is discouraged, there is probably a better way, talk to the OGS team
 */
UCLASS(config = Engine, MinimalAPI)
class UOnlineEngineInterface : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** @return the singleton accessor of the engine / OnlineSubsystem interface */
	static ENGINE_API UOnlineEngineInterface* Get();

	/**
	 * Subsystem
	 */

	/** @return true if the online subsystem is loaded, false otherwise */
	virtual bool IsLoaded(FName OnlineIdentifier = NAME_None) { return false; }
	/** @return the online identifier that should be used to access the OnlineSubsystem for this context */
	virtual FName GetOnlineIdentifier(FWorldContext& WorldContext) { return NAME_None; }
	/** @return true if a given OnlineSubsystem instance exists */
	virtual bool DoesInstanceExist(FName OnlineIdentifier) { return false; }
	/** Shut down a given online subsystem */
	virtual void ShutdownOnlineSubsystem(FName OnlineIdentifier) {}
	/** Destroy a given online subsystem */
	virtual void DestroyOnlineSubsystem(FName OnlineIdentifier) {}
	/** Returns the name of the default online subsystem, generally the DefaultPlatformService config value. */
	virtual FName GetDefaultOnlineSubsystemName() const { return NAME_None; }
	/** Returns whether the unique net id is compatible */
	virtual bool IsCompatibleUniqueNetId(const FUniqueNetIdWrapper& InUniqueNetId) const { return (InUniqueNetId.IsV1() && InUniqueNetId.GetType() == GetDefaultOnlineSubsystemName()); }

	/**
	 * Utils
	 */

	/** 
	 * Return the replication hash for a given subsystem
	 *
	 * @param InSubsystemName name of subsystem to retrieve hash from
	 *
	 * @return replication hash, or 0 if invalid/unknown
	 */
	virtual uint8 GetReplicationHashForSubsystem(FName InSubsystemName) const { return 0; }

	/**
	 * Return the name of the online subsystem associated with this hash
	 *
	 * @param InHash replication hash for an online subsystem
	 *
	 * @return name of subsystem this hash belongs to
	 */
	virtual FName GetSubsystemFromReplicationHash(uint8 InHash) const { return NAME_None; }

	/**
	 * Identity
	 */

	/** @return a proper FUniqueNetId pointer generated by the default OnlineSubsystem */
	virtual FUniqueNetIdWrapper CreateUniquePlayerIdWrapper(const FString& Str, FName Type = NAME_None) { return FUniqueNetIdWrapper(); }
	/** @return the unique id for a given local user on the given OnlineSubsystem */
	virtual FUniqueNetIdWrapper GetUniquePlayerIdWrapper(UWorld* World, int32 LocalUserNum, FName Type = NAME_None) { return FUniqueNetIdWrapper(); }

	/** @return the stored nickname on the OnlineSubsystem for a player with a given unique id */
	virtual FString GetPlayerNickname(UWorld* World, const FUniqueNetIdWrapper& UniqueId) { return TEXT("INVALID"); }
	/** @return the stored nickname on the secondary/console OnlineSubsystem for a player with a given unique id */
	virtual bool GetPlayerPlatformNickname(UWorld* World, int32 LocalUserNum, FString& OutNickname) { return false; }
	
	/** Auto login a given player based on credentials specified on the command line */
	virtual bool AutoLogin(UWorld* World, int32 LocalUserNum, const FOnlineAutoLoginComplete& InCompletionDelegate) { return false; }
	/** @return true if a local user is logged in, false otherwise */
	virtual bool IsLoggedIn(UWorld* World, int32 LocalUserNum) { return false; }

	/**
	 * Session
	 */

	/** Start a given named online session */
	virtual void StartSession(UWorld* World, FName SessionName, FOnlineSessionStartComplete& InCompletionDelegate) { InCompletionDelegate.ExecuteIfBound(SessionName, false); }
	/** End a given named online session */
	virtual void EndSession(UWorld* World, FName SessionName, FOnlineSessionEndComplete& InCompletionDelegate) { InCompletionDelegate.ExecuteIfBound(SessionName, false); }
	/** @return true if a given session exists, false otherwise */
	virtual bool DoesSessionExist(UWorld* World, FName SessionName) { return false; }

	virtual bool GetSessionJoinability(UWorld* World, FName SessionName, FJoinabilitySettings& OutSettings) { return false; }
	virtual void UpdateSessionJoinability(UWorld* World, FName SessionName, bool bPublicSearchable, bool bAllowInvites, bool bJoinViaPresence, bool bJoinViaPresenceFriendsOnly) {}

	/** Register a given player with the online session */
	virtual void RegisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId, bool bWasInvited) {}
	/** Unregister a given player from the online session */
	virtual void UnregisterPlayer(UWorld* World, FName SessionName, const FUniqueNetIdWrapper& UniqueId) {}
	/** Unregister a given player from the online session */
	virtual void UnregisterPlayers(UWorld* World, FName SessionName, const TArray<FUniqueNetIdWrapper>& Players) {}
	/** @return true if there is a valid URL that can been used to connect to a given session, false otherwise */
	virtual bool GetResolvedConnectString(UWorld* World, FName SessionName, FString& URL) { return false; }

	/**
	 * Voice
	 */
	
	/** @return any ready generated voice packet for a given local user */
	virtual TSharedPtr<FVoicePacket> GetLocalPacket(UWorld* World, uint8 LocalUserNum) { return nullptr; }
	/** @return a valid voice packet submitted over network */
	virtual TSharedPtr<FVoicePacket> SerializeRemotePacket(UWorld* World, const UNetConnection* const RemoteConnection, FArchive& Ar) { return nullptr; }

	/** Start processing networked voice traffic for a given local user */
	virtual void StartNetworkedVoice(UWorld* World, uint8 LocalUserNum) {}
	/** Stop processing networked voice traffic for a given local user */
	virtual void StopNetworkedVoice(UWorld* World, uint8 LocalUserNum) {}
	/** Clears all voice packets currently queued for send */
	virtual void ClearVoicePackets(UWorld* World) {}

	/** Mute a remote user for a given local player */
	virtual bool MuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide) { return false; }
	/** UnMute a remote user for a given local player */
	virtual bool UnmuteRemoteTalker(UWorld* World, uint8 LocalUserNum, const FUniqueNetIdWrapper& PlayerId, bool bIsSystemWide) { return false; }
	/** @return number of available/registered local talkers */
	virtual int32 GetNumLocalTalkers(UWorld* World) { return 0; }

	/**
	 * External UI
	 */

	struct FShowWebUrlParams
	{
		/** presented without a frame if embedded enabled */
		bool bEmbedded;
		/** Show the built in close button */
		bool bShowCloseButton;
		/** Show the built in background */
		bool bShowBackground;
		/** Hide the mouse cursor */
		bool bHideCursor;
		/** Rest cookies before invoking web browser */
		bool bResetCookies;
		/** x offset in pixels from top left */
		int32 OffsetX;
		/** y offset in pixels from top left */
		int32 OffsetY;
		/** x size in pixels */
		int32 SizeX;
		/** y size in pixels */
		int32 SizeY;
		/** if specified then restricted to only navigate within these domains */
		TArray<FString> AllowedDomains;
		/** portion of url for detecting callback.  Eg. "&code=", "redirect=", etc */
		FString CallbackPath;

		/**
		 * Constructor
		 */
		FShowWebUrlParams(bool InbEmbedded, int32 InOffsetX, int32 InOffsetY, int32 InSizeX, int32 InSizeY)
			: bEmbedded(InbEmbedded)
			, bShowCloseButton(false)
			, bShowBackground(false)
			, bHideCursor(false)
			, bResetCookies(false)
			, OffsetX(InOffsetX)
			, OffsetY(InOffsetY)
			, SizeX(InSizeX)
			, SizeY(InSizeY)
		{
		}

		/**
		 * Default Constructor
		 */
		FShowWebUrlParams()
			: bEmbedded(false)
			, bShowCloseButton(false)
			, bShowBackground(false)
			, bHideCursor(false)
			, bResetCookies(false)
			, OffsetX(0)
			, OffsetY(0)
			, SizeX(0)
			, SizeY(0)
		{
		}
	};

	/** Show an online external leaderboard UI if applicable to the platform */
	virtual void ShowLeaderboardUI(UWorld* World, const FString& CategoryName) {}
	/** Show an online external achievements UI if applicable to the platform */
	virtual void ShowAchievementsUI(UWorld* World, int32 LocalUserNum) {}
	/** Bind a delegate to the opening/closing of an online platform's external UI */
	virtual void BindToExternalUIOpening(const FOnlineExternalUIChanged& Delegate) {}
#define OSS_ADDED_SHOW_WEB 1 
	/** Displays a web page in the external UI */
	virtual void ShowWebURL(const FString& CurrentURL, const FShowWebUrlParams& ShowParams, const FOnlineShowWebUrlClosed& CompletionDelegate) {}
	/** Closes the currently active web external UI */
	virtual bool CloseWebURL() { return false; }

	/**
	 * Debug
	 */

	/** Dump session state to the log */
	virtual void DumpSessionState(UWorld* World) {}
	/** Dump party state to the log */
	virtual void DumpPartyState(UWorld* World) {}
	/** Dump voice state to the log */
	virtual void DumpVoiceState(UWorld* World) {}
	/** Dump chat state to the log */
	virtual void DumpChatState(UWorld* World) {}

	/**
	 * PIE Utilities
	 */
	
	/** @return true if the online subsystem supports online Play In Editor (PIE) */
	virtual bool SupportsOnlinePIE() { return false; }
	/** Enable/Disable online PIE at runtime */
	virtual void SetShouldTryOnlinePIE(bool bShouldTry) {}
	/** @return number of logins stored with the editor for logging in via Play In Editor (PIE) */
	virtual int32 GetNumPIELogins() { return 0; }
	/** @return a string with all three components (type, id, token) for the login with the given index for logging in via Play in Editor (PIE) */
	virtual FString GetPIELoginCommandLineArgs(int32 Index) { return TEXT(""); }
	/** Force the online subsystem to treat itself like a dedicated server */
	virtual void SetForceDedicated(FName OnlineIdentifier, bool bForce) {}
	/** Login a local user to the online subsystem before starting a Play In Editor (PIE) instance, using credentials stored in the editor */
	virtual void LoginPIEInstance(FName OnlineIdentifier, int32 LocalUserNum, int32 PIELoginNum, FOnPIELoginComplete& CompletionDelegate) { CompletionDelegate.ExecuteIfBound(LocalUserNum, false, TEXT("Not implemented")); }
	/** Perform any relevant cleanup operations for the online subsystem during Play In Editor (PIE) end/teardown */
	virtual void PIEEndCleanup(FName OnlineIdentifier) {}

private:

	/** One and only instance of this class, should handle normal and PIE operations by disambiguating in the function calls themselves */
	static ENGINE_API UOnlineEngineInterface* Singleton;
};
