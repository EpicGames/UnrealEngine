// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_SERVER_PERF_COUNTERS

	#include "PerfCountersModule.h"

	/** @brief Helper function for setting a performance counter. Compiled out if PerfCounterModule isn't used.
	 */
	void ENGINE_API PerfCountersSet(const FString& Name, float Val, uint32 Flags = 0);

	/** @brief Helper function for setting a performance counter. Compiled out if PerfCounterModule isn't used.
	*/
	void ENGINE_API PerfCountersSet(const FString& Name, double Val, uint32 Flags = 0);

	/** @brief Helper function for setting a performance counter. Compiled out if PerfCounterModule isn't used.
	*/
	void ENGINE_API PerfCountersSet(const FString& Name, int32 Val, uint32 Flags = 0);

	/** @brief Helper function for setting a performance counter. Compiled out if PerfCounterModule isn't used.
	*/
	void ENGINE_API PerfCountersSet(const FString& Name, const FString& Val, uint32 Flags = 0);

	/** @brief Helper function for incrementing a performance counter. Compiled out if PerfCounterModule isn't used.
	 *
	 *  @param Name the name of the counter
	 *  @param Add value of the increment (will be added to the counter, can be negative)
	 *  @param DefaultValue if the counter did not exist or was cleared, this is what it will be initialized to before performing the addition
	 *  @param Flags flags for the counter
	 *
	 *  @return current value (i.e. after the increment)
	 */
	int32 ENGINE_API PerfCountersIncrement(const FString & Name, int32 Add = 1, int32 DefaultValue = 0, uint32 Flags = IPerfCounters::Flags::Transient);

	/** @brief Helper function for getting a performance counter. Compiled out to DefaultValue if PerfCounterModule isn't used.
	 */
	float ENGINE_API PerfCountersGet(const FString& Name, float DefaultVal);

	/** @brief Helper function for getting a performance counter. Compiled out to DefaultValue if PerfCounterModule isn't used.
	 */
	double ENGINE_API PerfCountersGet(const FString& Name, double DefaultVal);

	/** @brief Helper function for getting a performance counter. Compiled out to DefaultValue if PerfCounterModule isn't used.
	 */
	int32 ENGINE_API PerfCountersGet(const FString& Name, int32 DefaultVal);

	/** @brief Helper function for getting a performance counter. Compiled out to DefaultValue if PerfCounterModule isn't used.
	 */
	uint32 ENGINE_API PerfCountersGet(const FString& Name, uint32 DefaultVal);

#else

	/** stub implementations to be used when PerfCounters are unavailable */

	inline void PerfCountersSet(const FString& Name, float Val, uint32 Flags = 0) { /* no-op */ };
	inline void PerfCountersSet(const FString& Name, int32 Val, uint32 Flags = 0) { /* no-op */ };
	inline void PerfCountersSet(const FString& Name, const FString& Val, uint32 Flags = 0) { /* no-op */ };
	inline int32 PerfCountersIncrement(const FString & Name, int32 Add = 1, int32 DefaultValue = 0, uint32 Flags = 0) { return DefaultValue + Add; };
	inline float PerfCountersGet(const FString& Name, float DefaultVal) { return DefaultVal; }
	inline double PerfCountersGet(const FString& Name, double DefaultVal) { return DefaultVal; }
	inline int32 PerfCountersGet(const FString& Name, int32 DefaultVal) { return DefaultVal; }
	inline uint32 PerfCountersGet(const FString& Name, uint32 DefaultVal) { return DefaultVal; }

#endif //USE_SERVER_PERF_COUNTERS
