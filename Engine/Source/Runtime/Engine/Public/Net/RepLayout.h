// Copyright Epic Games, Inc. All Rights Reserved.

/**
* This file contains the core logic and types that support Object and RPC replication.
*
* These types don't dictate how RPCs are triggered or when an Object should be replicated,
* although there are some methods defined here that may be used in those determinations.
*
* Instead, the types here focus on how data from Objects, Structs, Containers, and Properties
* are generically tracked and serialized on both Clients and Servers.
*
* The main class is FRepLayout.
*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/NetworkGuid.h"
#include "UObject/CoreNet.h"
#include "Engine/EngineTypes.h"
#include "UObject/GCObject.h"
#include "Containers/StaticBitArray.h"
#include "Net/Core/Misc/GuidReferences.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/Core/PropertyConditions/RepChangedPropertyTracker.h"
#include "Templates/CopyQualifiersFromTo.h"

class FNetFieldExportGroup;
class FRepLayout;
class UActorChannel;
class UNetConnection;
class UPackageMapClient;

enum class EDiffPropertiesFlags : uint32
{
	None = 0,
	Sync = (1 << 0),							//! Indicates that properties should be updated (synchronized), not just diffed.
	IncludeConditionalProperties = (1 << 1)		//! Whether or not conditional properties should be included.
};

ENUM_CLASS_FLAGS(EDiffPropertiesFlags);

enum class EReceivePropertiesFlags : uint32
{
	None = 0,
	RepNotifies = (1 << 0),						//! Whether or not RepNotifies will be fired due to changed properties.
	SkipRoleSwap = (1 << 1)						//! Whether or not to skip swapping role and remote role.
};

ENUM_CLASS_FLAGS(EReceivePropertiesFlags);

enum class ESerializePropertyType : uint8
{
	Handle,	// Properties are seralized using handles in SendProperties_r
	Name	// Properties are seralized using their names in SendProperties_r
};

enum class ERepDataBufferType
{
	ObjectBuffer,	//! Indicates this buffer is a full object's memory.
	ShadowBuffer	//! Indicates this buffer is a packed shadow buffer.
};

namespace UE_RepLayout_Private
{
	/**
	 * TRepDataBuffer and TConstRepDataBuffer act as wrapper around internal data
	 * buffers that FRepLayout may use. This allows FRepLayout to properly interact
	 * with memory buffers and apply commands to them more easily.
	 */
	template<ERepDataBufferType DataType, typename ConstOrNotType>
	struct TRepDataBufferBase
	{
		static constexpr ERepDataBufferType Type = DataType;
		using ConstOrNotVoid = TCopyQualifiersFromTo_T<ConstOrNotType, void>;

	public:

		TRepDataBufferBase(ConstOrNotVoid* RESTRICT InDataBuffer) :
			Data((ConstOrNotType* RESTRICT)InDataBuffer)
		{}

		friend TRepDataBufferBase operator+(TRepDataBufferBase InBuffer, int32 Offset)
		{
			return InBuffer.Data + Offset;
		}

		explicit operator bool() const
		{
			return Data != nullptr;
		}

		operator ConstOrNotType* () const
		{
			return Data;
		}

		ConstOrNotType* RESTRICT Data;
	};

	template<typename TLayoutCmdType, typename ConstOrNotType>
	static TRepDataBufferBase<ERepDataBufferType::ObjectBuffer, ConstOrNotType> operator+(TRepDataBufferBase<ERepDataBufferType::ObjectBuffer, ConstOrNotType> InBuffer, const TLayoutCmdType& Cmd)
	{
		return InBuffer + Cmd.Offset;
	}

	template<typename TLayoutCmdType, typename ConstOrNotType>
	static TRepDataBufferBase<ERepDataBufferType::ShadowBuffer, ConstOrNotType> operator+(const TRepDataBufferBase<ERepDataBufferType::ShadowBuffer, ConstOrNotType> InBuffer, const TLayoutCmdType& Cmd)
	{
		return InBuffer + Cmd.ShadowOffset;
	}
}

template<ERepDataBufferType DataType> using TRepDataBuffer = UE_RepLayout_Private::TRepDataBufferBase<DataType, uint8>;
template<ERepDataBufferType DataType> using TConstRepDataBuffer = UE_RepLayout_Private::TRepDataBufferBase<DataType, const uint8>;

typedef TRepDataBuffer<ERepDataBufferType::ObjectBuffer> FRepObjectDataBuffer;
typedef TRepDataBuffer<ERepDataBufferType::ShadowBuffer> FRepShadowDataBuffer;
typedef TConstRepDataBuffer<ERepDataBufferType::ObjectBuffer> FConstRepObjectDataBuffer;
typedef TConstRepDataBuffer<ERepDataBufferType::ShadowBuffer> FConstRepShadowDataBuffer;

/** FRepChangedPropertyTracker moved to NetCore module */

class FRepLayout;
class FRepLayoutCmd;

namespace UE::Net
{
	/**
	 * Builds a new ConditionMap given the input RepFlags.
	 * This can be used to determine whether or not a given property should be
	 * considered enabled / disabled based on ELifetimeCondition.
	 */
	inline TStaticBitArray<COND_Max> BuildConditionMapFromRepFlags(const FReplicationFlags& RepFlags)
	{
		TStaticBitArray<COND_Max> ConditionMap;

		// Setup condition map
		const bool bIsInitial = RepFlags.bNetInitial ? true : false;
		const bool bIsOwner = RepFlags.bNetOwner ? true : false;
		const bool bIsSimulated = RepFlags.bNetSimulated ? true : false;
		const bool bIsPhysics = RepFlags.bRepPhysics ? true : false;
		const bool bIsReplay = RepFlags.bReplay ? true : false;

		ConditionMap[COND_None] = true;
		ConditionMap[COND_InitialOnly] = bIsInitial;

		ConditionMap[COND_OwnerOnly] = bIsOwner;
		ConditionMap[COND_SkipOwner] = !bIsOwner;

		ConditionMap[COND_SimulatedOnly] = bIsSimulated;
		ConditionMap[COND_SimulatedOnlyNoReplay] = bIsSimulated && !bIsReplay;
		ConditionMap[COND_AutonomousOnly] = !bIsSimulated;

		ConditionMap[COND_SimulatedOrPhysics] = bIsSimulated || bIsPhysics;
		ConditionMap[COND_SimulatedOrPhysicsNoReplay] = (bIsSimulated || bIsPhysics) && !bIsReplay;

		ConditionMap[COND_InitialOrOwner] = bIsInitial || bIsOwner;
		ConditionMap[COND_ReplayOrOwner] = bIsReplay || bIsOwner;
		ConditionMap[COND_ReplayOnly] = bIsReplay;
		ConditionMap[COND_SkipReplay] = !bIsReplay;

		ConditionMap[COND_Custom] = true;
		ConditionMap[COND_Dynamic] = true;
		ConditionMap[COND_Never] = false;

		return ConditionMap;
	}
}

struct FRepSharedPropertyKey
{
private:
	uint32 CmdIndex = 0;
	uint32 ArrayIndex = 0;
	uint32 ArrayDepth = 0;
	void* DataPtr = nullptr;

public:
	FRepSharedPropertyKey()
		: CmdIndex(0)
		, ArrayIndex(0)
		, ArrayDepth(0)
		, DataPtr(nullptr)
	{
	}

	explicit FRepSharedPropertyKey(uint32 InCmdIndex, uint32 InArrayIndex, uint32 InArrayDepth, void* InDataPtr)
		: CmdIndex(InCmdIndex)
		, ArrayIndex(InArrayIndex)
		, ArrayDepth(InArrayDepth)
		, DataPtr(InDataPtr)
	{
	}

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("{Cmd: %u, Index: %u, Depth: %u, Ptr: 0x%08" UPTRINT_x_FMT "}"), CmdIndex, ArrayIndex, ArrayDepth, (UPTRINT)DataPtr);
	}

	friend bool operator==(const FRepSharedPropertyKey& A, const FRepSharedPropertyKey& B)
	{
		return (A.CmdIndex == B.CmdIndex) && (A.ArrayIndex == B.ArrayIndex) && (A.ArrayDepth == B.ArrayDepth) && (A.DataPtr == B.DataPtr);
	}

	friend uint32 GetTypeHash(const FRepSharedPropertyKey& Key)
	{
		return uint32(CityHash64((char*)&Key, sizeof(FRepSharedPropertyKey)));
	}
};

/** Holds the unique identifier and offsets/lengths of a net serialized property used for Shared Serialization */
struct FRepSerializedPropertyInfo
{
	FRepSerializedPropertyInfo():
		BitOffset(0),
		BitLength(0),
		PropBitOffset(0),
		PropBitLength(0)
	{}

	/** Unique identifier for this property */
	FRepSharedPropertyKey PropertyKey;

	/** Bit offset into shared buffer of the shared data */
	int32 BitOffset;

	/** Length in bits of all serialized data for this property, may include handle and checksum. */
	int32 BitLength;

	/** Bit offset into shared buffer of the property data. */
	int32 PropBitOffset;

	/** Length in bits of net serialized property data only */
	int32 PropBitLength;
};

/** Holds a set of shared net serialized properties */
struct FRepSerializationSharedInfo
{
	FRepSerializationSharedInfo():
		bIsValid(false)
	{}

	~FRepSerializationSharedInfo()
	{
		// Explicitly reset members to improve resilience to double-destruction.
		SharedPropertyInfo = {};
		SerializedProperties = nullptr;
		bIsValid = false;
	}

	void SetValid()
	{
		bIsValid = true;
	}

	void Init()
	{
		if (!SerializedProperties.IsValid())
		{
			SerializedProperties.Reset(new FNetBitWriter(0));
		}
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	void Reset()
	{
		if (bIsValid)
		{
			SharedPropertyInfo.Reset();
			SerializedProperties->Reset();
			NetTokensPendingExport.Reset();

			bIsValid = false;
		}
	}

	/**
	 * Creates a new SharedPropertyInfo and adds it to the SharedPropertyInfo list.
	 *
	 * @param Cmd				The command that represents the property we want to share.
	 * @param PropertyKey		A unique key used to identify the property.
	 * @param CmdIndex			Index of the property command. Only used if bDoChecksum is true.
	 * @param Handle			Relative Handle of the property command. Only used if bWriteHandle is true.
	 * @param Data				Pointer to the raw property memory that will be serialized.
	 * @param bWriteHandle		Whether or not we should write Command handles into the serialized data.
	 * @param bDoChecksum		Whether or not we should do checksums. Only used if ENABLE_PROPERTY_CHECKSUMS is enabled.
	 */
	const FRepSerializedPropertyInfo* WriteSharedProperty(
		const FRepLayoutCmd& Cmd,
		const FRepSharedPropertyKey& PropertyKey,
		const int32 CmdIndex,
		const uint16 Handle,
		const FConstRepObjectDataBuffer Data,
		const bool bWriteHandle,
		const bool bDoChecksum);

	/** Metadata for properties in the shared data blob. */
	TArray<FRepSerializedPropertyInfo> SharedPropertyInfo;

	/** Binary blob of net serialized data to be shared */
	TUniquePtr<FNetBitWriter> SerializedProperties;

	/** NetTokenExports for this RepLayout, will need to index to avoid adding exports that we do not want to use. */
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

	void CountBytes(FArchive& Ar) const;

private:

	/** Whether or not shared serialization data has been successfully built. */
	bool bIsValid = false;
};

/**
 * Represents a single changelist, tracking changed properties.
 *
 * Properties are tracked via Relative Property Command Handles.
 * Valid handles are 1-based, and 0 is reserved as a terminator.
 *
 * Arrays are tracked as a special case inline, where the first entry is the number of array elements,
 * followed by handles for each array element, and ending with their own 0 terminator.
 *
 * Arrays may be nested by continually applying that pattern.
 */
class FRepChangedHistory
{
public:
	FRepChangedHistory():
		Resend(false)
	{}

	void CountBytes(FArchive& Ar) const
	{
		Changed.CountBytes(Ar);
	}

	bool WasSent() const
	{
		return OutPacketIdRange.First != INDEX_NONE;
	}

	void Reset()
	{
		OutPacketIdRange = FPacketIdRange();
		Changed.Empty();
		Resend = false;
	}

	/** Range of the Packets that this changelist was last sent with. Used to track acknowledgments. */
	FPacketIdRange OutPacketIdRange;

	/** List of Property Command Handles that changed in this changelist. */
	TArray<uint16> Changed;

	/** Whether or not this Changelist should be resent due to a Nak. */
	bool Resend = false;
};

/**
 * Holds deep copies of replicated property data for objects.
 * The term "shadow data" is often used in code to refer to memory stored in one of these buffers.
 * Note, dynamic memory allocated by the properties (such as Arrays or Maps) will still be dynamically
 * allocated elsewhere, and the buffer will hold pointers to the dynamic memory (or containers, etc.)
 *
 * When necessary, use FRepShadowDataBuffer or FConstRepShadowDataBuffer to wrap this object's data.
 * Never use FRepObjectDataBuffer or FConstRepObjectDataBuffer as the shadow memory layout is not guaranteed
 * to match an object's layout.
 */
struct FRepStateStaticBuffer : public FNoncopyable
{
private:

	friend class FRepLayout;

	FRepStateStaticBuffer(const TSharedRef<const FRepLayout>& InRepLayout) :
		RepLayout(InRepLayout)
	{
	}

public:

	FRepStateStaticBuffer(FRepStateStaticBuffer&& InStaticBuffer) :
		Buffer(MoveTemp(InStaticBuffer.Buffer)),
		RepLayout(MoveTemp(InStaticBuffer.RepLayout))
	{
	}

	~FRepStateStaticBuffer();

	uint8* GetData()
	{
		return Buffer.GetData();
	}

	const uint8* GetData() const
	{
		return Buffer.GetData();
	}

	int32 Num() const
	{
		return Buffer.Num();
	}

	void CountBytes(FArchive& Ar) const;

	void Empty()
	{
		Buffer.Empty();
	}

private:

	// Properties will be copied in here so memory needs aligned to largest type
	TArray<uint8, TAlignedHeapAllocator<16>> Buffer;
	TSharedRef<const FRepLayout> RepLayout;
};

/**
 * Stores changelist history (that are used to know what properties have changed) for objects.
 *
 * Only a fixed number of history items are kept. Once that limit is reached, old entries are
 * merged into a single monolithic changelist (this happens incrementally each time a new entry
 * is added).
 */
class FRepChangelistState : public FNoncopyable
{
private:

	friend class FReplicationChangelistMgr;

	FRepChangelistState(
		const TSharedRef<const FRepLayout>& InRepLayout,
		const uint8* Source,
		const UObject* InRepresenting,
		struct FCustomDeltaChangelistState* InDeltaChangelistState);

public:

	~FRepChangelistState();

	/** The maximum number of individual changelists allowed.*/
	static const int32 MAX_CHANGE_HISTORY = 64;

	/** Circular buffer of changelists. */
	FRepChangedHistory ChangeHistory[MAX_CHANGE_HISTORY];

	/** Changelist state specific to Custom Delta properties. Only allocated if this RepLayout has Custom Delta Properties. */
	TUniquePtr<struct FCustomDeltaChangelistState> CustomDeltaChangelistState;

	/** Index in the buffer where changelist history starts (i.e., the Oldest changelist). */
	int32 HistoryStart = 0;

	/** Index in the buffer where changelist history ends (i.e., the Newest changelist). */
	int32 HistoryEnd = 0;

	/** Number of times that properties have been compared */
	int32 CompareIndex = 0;

	/** Tracking custom delta sends, for comparison against sending rep state. */
	uint32 CustomDeltaChangeIndex = 0;
	
	/** Latest state of all property data. Not used on Clients, only used on Servers if Shadow State is enabled. */
	FRepStateStaticBuffer StaticBuffer;

	/** Latest state of all shared serialization data. */
	FRepSerializationSharedInfo SharedSerialization;

	void CountBytes(FArchive& Ar) const;

#if WITH_PUSH_MODEL

	const UEPushModelPrivate::FPushModelPerNetDriverHandle& GetPushModelObjectHandle() const
	{
		return PushModelObjectHandle;
	}

	bool HasAnyDirtyProperties() const;

	bool HasValidPushModelHandle() const;

private:
	const UEPushModelPrivate::FPushModelPerNetDriverHandle PushModelObjectHandle;
#endif
};

/**
 *	FReplicationChangelistMgr manages a list of change lists for a particular replicated object that have occurred since the object started replicating
 *	Once the history is completely full, the very first changelist will then be merged with the next one (freeing a slot)
 *		This way we always have the entire history for join in progress players
 *	This information is then used by all connections, to share the compare work needed to determine what to send each connection
 *	Connections will send any changelist that is new since the last time the connection checked
 */
class FReplicationChangelistMgr : public FNoncopyable
{
private:

	friend class FRepLayout;

	FReplicationChangelistMgr(
		const TSharedRef<const FRepLayout>& InRepLayout,
		const uint8* Source,
		const UObject* InRepresenting,
		struct FCustomDeltaChangelistState* CustomDeltaChangelistState);

public:

	~FReplicationChangelistMgr();

	FRepChangelistState* GetRepChangelistState()
	{
		return &RepChangelistState;
	}

	const FRepChangelistState* GetRepChangelistState() const
	{
		return &RepChangelistState;
	}

	void CountBytes(FArchive& Ar) const;

private:

	uint32 LastReplicationFrame = 0;
	uint32 LastInitialReplicationFrame = 0;

	FRepChangelistState RepChangelistState;
};

/** Replication State needed to track received properties. */
class FReceivingRepState : public FNoncopyable
{
private:

	friend class FRepLayout;

	FReceivingRepState(FRepStateStaticBuffer&& InStaticBuffer);

public:

	void CountBytes(FArchive& Ar) const;

	/** Latest state of all property data. Only valid on clients. */
	FRepStateStaticBuffer StaticBuffer;

	/** Map of Absolute Property Offset to GUID Reference for properties. */
	FGuidReferencesMap GuidReferencesMap;

	/** List of properties that have RepNotifies that we will need to call on Clients. */
	TArray<FProperty*> RepNotifies;

	/**
	 * Holds MetaData (such as array index) for RepNotifies.
	 * Only used for CustomDeltaProperties.
	 */
	TMap<FProperty*, TArray<uint8>> RepNotifyMetaData;
};

/** Replication State that is only needed when sending properties. */
class FSendingRepState : public FNoncopyable
{
private:

	friend class FRepLayout;

	FSendingRepState() :
		bOpenAckedCalled(false),
		HistoryStart(0),
		HistoryEnd(0),
		NumNaks(0),
		LastChangelistIndex(0),
		LastCompareIndex(0),
		InactiveChangelist({0})
	{}	

public:

	void CountBytes(FArchive& Ar) const;

	UE_DEPRECATED(5.4, "Use UE::Net::BuildConditionMapFromRepFlags instead")
	static inline TStaticBitArray<COND_Max> BuildConditionMapFromRepFlags(const FReplicationFlags InFlags) { return UE::Net::BuildConditionMapFromRepFlags(InFlags); }

	bool HasAnyPendingRetirements() const;

	/** Whether or not FRepLayout::OpenAcked has been called with this FRepState. */
	bool bOpenAckedCalled;

	// Cache off the RemoteRole and Role per connection to avoid issues with
	// FScopedRoleDowngrade. See UE-66313 (among others).

	TEnumAsByte<ENetRole> SavedRemoteRole = ROLE_MAX;
	TEnumAsByte<ENetRole> SavedRole = ROLE_MAX;

	/** Index in the buffer where changelist history starts (i.e., the Oldest changelist). */
	int32 HistoryStart;

	/** Index in the buffer where changelist history ends (i.e., the Newest changelist). */
	int32 HistoryEnd;

	/** Number of Changelist history entries that have outstanding Naks. */
	int32 NumNaks;

	/**
	 * The last change list history item we replicated from FRepChangelistState.
	 * (If we are caught up to FRepChangelistState::HistoryEnd, there are no new changelists to replicate).
	 */
	int32 LastChangelistIndex;

	/**
	 * Tracks the last time this RepState actually replicated data.
	 * If this matches FRepChangelistState::CompareIndex, then there is definitely no new
	 * information since the last time we checked.
	 *
	 * @see FRepChangelistState::CompareIndex.
	 *
	 * Note, we can't solely rely on on LastChangelistIndex, since changelists are stored in circular buffers.
	 */
	int32 LastCompareIndex;

	/** Tracking custom delta sends, for comparison against the changelist state. */
	uint32 CustomDeltaChangeIndex = 0;

	FReplicationFlags RepFlags;

	TSharedPtr<FRepChangedPropertyTracker> RepChangedPropertyTracker;

	/** The maximum number of individual changelists allowed.*/
	static constexpr int32 MAX_CHANGE_HISTORY = 32;

	/** Circular buffer of changelists. */
	FRepChangedHistory ChangeHistory[MAX_CHANGE_HISTORY];

	/** Array of property retirements that we'll use to track retransmission for Custom Delta Properties. */
	TArray<FPropertyRetirement> Retirement;

	/** List of changelists that were generated before the channel was fully opened.*/
	TArray<FRepChangedHistory> PreOpenAckHistory;

	/** The unique list of properties that have changed since the channel was first opened */
	TArray<uint16> LifetimeChangelist;

	/**
	 * Properties which are inactive through conditions have their changes stored here, so they can be 
	 * applied if/when the property becomes active.
	 *
	 * This should always be a valid changelist, even if no properties are inactive.
	 */
	TArray<uint16> InactiveChangelist;

	/** Cached set of inactive parent commands. */
	TBitArray<> InactiveParents;

	/** This is the delta state we need to compare with when determining what to send to a client for custom delta properties. */
	TArray<TSharedPtr<INetDeltaBaseState>> RecentCustomDeltaState;

	/** Same as RecentCustomDeltaState, but this will always remain as the initial CDO version. We use this to send all properties since channel was first opened (for bResendAllDataSinceOpen). */
	TArray<TSharedPtr<INetDeltaBaseState>>	CDOCustomDeltaState;

	/** Same as RecentCustomDeltaState, but will represent the state at the last checkpoint. */
	TArray<TSharedPtr<INetDeltaBaseState>>	CheckpointCustomDeltaState;
};

/** Replication State that is unique Per Object Per Net Connection. */
class FRepState : public FNoncopyable
{
private:

	friend FRepLayout;

	FRepState() {}

	/** May be null on connections that don't receive properties. */
	TUniquePtr<FReceivingRepState> ReceivingRepState;

	/** May be null on connections that don't send properties. */
	TUniquePtr<FSendingRepState> SendingRepState;

public:

	void CountBytes(FArchive& Ar) const;

	FReceivingRepState* GetReceivingRepState()
	{
		return ReceivingRepState.Get();
	}

	const FReceivingRepState* GetReceivingRepState() const
	{
		return ReceivingRepState.Get();
	}

	FSendingRepState* GetSendingRepState()
	{
		return SendingRepState.Get();
	}

	const FSendingRepState* GetSendingRepState() const
	{
		return SendingRepState.Get();
	}
};

/**
 * Flags used to customize how RepStates are created.
 * @see FRepLayout::CreateRepState.
 */
enum class ECreateRepStateFlags : uint32
{
	None,
	SkipCreateReceivingState = 0x1,	// Don't create a receiving RepState, as we never expect it to be used.
};
ENUM_CLASS_FLAGS(ECreateRepStateFlags);

/** Various types of Properties supported for Replication. */
enum class ERepLayoutCmdType : uint8
{
	DynamicArray			= 0,	//! Dynamic array
	Return					= 1,	//! Return from array, or end of stream
	Property				= 2,	//! Generic property

	PropertyBool			= 3,
	PropertyFloat			= 4,
	PropertyInt				= 5,
	PropertyByte			= 6,
	PropertyName			= 7,
	PropertyObject			= 8,
	PropertyUInt32			= 9,
	PropertyVector			= 10,
	PropertyRotator			= 11,
	PropertyPlane			= 12,
	PropertyVector100		= 13,
	PropertyNetId			= 14,
	RepMovement				= 15,
	PropertyVectorNormal	= 16,
	PropertyVector10		= 17,
	PropertyVectorQ			= 18,
	PropertyString			= 19,
	PropertyUInt64			= 20,
	PropertyNativeBool		= 21,
	PropertySoftObject		= 22,
	PropertyWeakObject		= 23,
	PropertyInterface		= 24,
	NetSerializeStructWithObjectReferences = 25,
};

const TCHAR* LexToString(ERepLayoutCmdType CmdType);

/** Various flags that describe how a Top Level Property should be handled. */
enum class ERepParentFlags : uint32
{
	None						= 0,
	IsLifetime					= (1 << 0),	 //! This property is valid for the lifetime of the object (almost always set).
	IsConditional				= (1 << 1),	 //! This property has a secondary condition to check
	IsConfig					= (1 << 2),	 //! This property is defaulted from a config file
	IsCustomDelta				= (1 << 3),	 //! This property uses custom delta compression. Mutually exclusive with IsNetSerialize.
	IsNetSerialize				= (1 << 4),  //! This property uses a custom net serializer. Mutually exclusive with IsCustomDelta.
	IsStructProperty			= (1 << 5),	 //! This property is a FStructProperty.
	IsZeroConstructible			= (1 << 6),	 //! This property is ZeroConstructible.
	IsFastArray					= (1 << 7),	 //! This property is a FastArraySerializer. This can't be a ERepLayoutCmdType, because
											 //! these Custom Delta structs will have their inner properties tracked.
	HasObjectProperties			= (1 << 8),  //! This property is tracking UObjects (may be through nested properties).
	HasNetSerializeProperties	= (1 << 9),  //! This property contains Net Serialize properties (may be through nested properties).
	HasDynamicArrayProperties   = (1 << 10), //! This property contains Dynamic Array properties (may be through nested properties).
};

ENUM_CLASS_FLAGS(ERepParentFlags)

/**
 * A Top Level Property of a UClass, UStruct, or UFunction (arguments to a UFunction).
 *
 * @see FRepLayout
 */
class FRepParentCmd
{
public:

	FRepParentCmd(FProperty* InProperty, int32 InArrayIndex): 
		Property(InProperty),
		CachedPropertyName(InProperty ? InProperty->GetFName() : NAME_None),
		ArrayIndex(InArrayIndex),
		ShadowOffset(0),
		CmdStart(0),
		CmdEnd(0),
		Condition(COND_None),
		RepNotifyCondition(REPNOTIFY_OnChanged),
		RepNotifyNumParams(INDEX_NONE),
		Flags(ERepParentFlags::None)
	{}

	FProperty* Property;

	const FName CachedPropertyName;

	/**
	 * If the Property is a C-Style fixed size array, then a command will be created for every element in the array.
	 * This is the index of the element in the array for which the command represents.
	 *
	 * This will always be 0 for non array properties.
	 */
	int32 ArrayIndex;

	/** Absolute offset of property in Object Memory. */
	int32 Offset;

	/** Absolute offset of property in Shadow Memory. */
	int32 ShadowOffset;

	/**
	 * CmdStart and CmdEnd define the range of FRepLayoutCommands (by index in FRepLayouts Cmd array) of commands
	 * that are associated with this Parent Command.
	 *
	 * This is used to track and access nested Properties from the parent.
	 */
	uint16 CmdStart;

	/** @see CmdStart */
	uint16 CmdEnd;

	ELifetimeCondition Condition;
	ELifetimeRepNotifyCondition RepNotifyCondition;

	/**
	 * Number of parameters that we need to pass to the RepNotify function (if any).
	 * If this value is INDEX_NONE, it means there is no RepNotify function associated
	 * with the property.
	 */
	int32 RepNotifyNumParams;

	ERepParentFlags Flags;
};

/** Various flags that describe how a Property should be handled. */
enum class ERepLayoutCmdFlags : uint8
{
	None					= 0,		//! No flags.
	IsSharedSerialization	= (1 << 0),	//! Indicates the property is eligible for shared serialization.
	IsStruct				= (1 << 1),	//! This is a struct property.
	IsEmptyArrayStruct		= (1 << 2),	//! This is an ArrayProperty whose InnerProperty has no replicated properties.
};

ENUM_CLASS_FLAGS(ERepLayoutCmdFlags)

/**
 * Represents a single property, which could be either a Top Level Property, a Nested Struct Property,
 * or an element in a Dynamic Array.
 *
 * @see FRepLayout
 */
class FRepLayoutCmd
{
public:

	/** Pointer back to property, used for NetSerialize calls, etc. */
	FProperty* Property;

	/** For arrays, this is the cmd index to jump to, to skip this arrays inner elements. */
	uint16 EndCmd;

	/** For arrays, element size of data. */
	uint16 ElementSize;

	/** Absolute offset of property in Object Memory. */
	int32 Offset;

	/** Absolute offset of property in Shadow Memory. */
	int32 ShadowOffset;

	/** Handle relative to start of array, or top list. */
	uint16 RelativeHandle;

	/** Index into Parents. */
	uint16 ParentIndex;

	/** Used to determine if property is still compatible */
	uint32 CompatibleChecksum;

	ERepLayoutCmdType Type;
	ERepLayoutCmdFlags Flags;
};
	
/** Converts a relative handle to the appropriate index into the Cmds array */
class FHandleToCmdIndex
{
public:
	FHandleToCmdIndex():
		CmdIndex(INDEX_NONE)
	{
	}

	FHandleToCmdIndex(const int32 InHandleToCmdIndex):
		CmdIndex(InHandleToCmdIndex)
	{
	}

	FHandleToCmdIndex(FHandleToCmdIndex&& Other):
		CmdIndex(Other.CmdIndex),
		HandleToCmdIndex(MoveTemp(Other.HandleToCmdIndex))
	{
	}

	FHandleToCmdIndex& operator=(FHandleToCmdIndex&& Other)
	{
		if (this != &Other)
		{
			CmdIndex = Other.CmdIndex;
			HandleToCmdIndex = MoveTemp(Other.HandleToCmdIndex);
		}

		return *this;
	}

	int32 CmdIndex;
	TUniquePtr<TArray<FHandleToCmdIndex>> HandleToCmdIndex;
};

/**
 * Simple helper class to track state while iterating over changelists.
 * This class doesn't actually expose methods to do the iteration, or to retrieve
 * the current value.
 */
class FChangelistIterator
{
public:

	FChangelistIterator(const TArray<uint16>& InChanged, const int32 InChangedIndex):
		Changed(InChanged),
		ChangedIndex(InChangedIndex)
	{}

	/** Changelist that is being iterated. */
	const TArray<uint16>& Changed;

	/** Current index into the changelist. */
	int32 ChangedIndex;
};

/** Iterates over a changelist, taking each handle, and mapping to rep layout index, array index, etc. */
class FRepHandleIterator
{
public:

	FRepHandleIterator(
		UStruct const * const InOwner,
		FChangelistIterator& InChangelistIterator,
		const TArray<FRepLayoutCmd>& InCmds,
		const TArray<FHandleToCmdIndex>& InHandleToCmdIndex,
		const int32 InElementSize,
		const int32 InMaxArrayIndex,
		const int32 InMinCmdIndex,
		const int32 InMaxCmdIndex
	):
		ChangelistIterator(InChangelistIterator),
		Cmds(InCmds),
		HandleToCmdIndex(InHandleToCmdIndex),
		NumHandlesPerElement(HandleToCmdIndex.Num()),
		ArrayElementSize(InElementSize),
		MaxArrayIndex(InMaxArrayIndex),
		MinCmdIndex(InMinCmdIndex),
		MaxCmdIndex(InMaxCmdIndex),
		Owner(InOwner),
		LastSuccessfulCmdIndex(INDEX_NONE)
	{
		ensureMsgf(MaxCmdIndex >= MinCmdIndex, TEXT("Invalid Min / Max Command Indices. Owner=%s, MinCmdIndex=%d, MaxCmdIndex=%d"), *GetPathNameSafe(Owner), MinCmdIndex, MaxCmdIndex);
	}

	/**
	 * Moves the iterator to the next available handle.
	 *
	 * @return True if the move was successful, false otherwise (e.g., end of the iteration range was reached).
	 */
	bool NextHandle();

	/**
	 * Skips all the handles associated with a dynamic array at the iterators current position.
	 *
	 * @return True if the move was successful, false otherwise (this will return True even if the next handle is the end).
	 */
	bool JumpOverArray();

	/**
	 * Gets the handle at the iterators current position without advancing it.
	 *
	 * @return The next Property Command handle.
	 */
	int32 PeekNextHandle() const;

	/** Used to track current state of the iteration. */
	FChangelistIterator& ChangelistIterator;

	/** List of all available Layout Commands. */
	const TArray<FRepLayoutCmd>& Cmds;

	/** Used to map Relative Handles to absolute Property Command Indices. */
	const TArray<FHandleToCmdIndex>& HandleToCmdIndex;

	/**
	 * The number of handles per Command.
	 * This should always be 1, except for Arrays.
	 */
	const int32 NumHandlesPerElement;

	/**
	 * Only used for Dynamic Arrays.
	 * @see FRepLayout ElementSize.
	 */
	const int32	ArrayElementSize;

	/**
	 * Number of elements in a Dynamic array.
	 * Should be 1 when iterating Top Level Properties or non-array properties.
	 */
	const int32	MaxArrayIndex;

	/** Lowest index in Cmds where the iterator can go. */
	const int32	MinCmdIndex;

	/** Highest index in Cmds where the iterator can go. */
	const int32	MaxCmdIndex;

	/** The current Relative Property Command handle. */
	int32 Handle;

	/** The current Property Command index. */
	int32 CmdIndex;

	/** The index of the current element in a dynamic array. */
	int32 ArrayIndex;

	/** The Byte offset of Serialized Property data for a dynamic array to the current element. */
	int32 ArrayOffset;

	UStruct const * const Owner;

private:

	int32 LastSuccessfulCmdIndex;
};

enum class ECreateReplicationChangelistMgrFlags
{
	None,
	SkipDeltaCustomState,	//! Skip creating CustomDeltaState used for tracking.
							//! Only do this if you know you'll never need it (like for replay recording)
};
ENUM_CLASS_FLAGS(ECreateReplicationChangelistMgrFlags);

enum class ECreateRepLayoutFlags
{
	None,
	MaySendProperties,	//! Regardless of whether or not this RepLayout is being created for servers, it may be used to send property data, and needs state to handle that.
};
ENUM_CLASS_FLAGS(ECreateRepLayoutFlags);

enum class ERepLayoutFlags : uint8
{
	None								= 0,
	IsActor 							= (1 << 0),	//! This RepLayout is for AActor or a subclass of AActor.
	PartialPushSupport					= (1 << 1),	//! This RepLayout has some properties that use Push Model and some that don't.
	FullPushSupport						= (1 << 2),	//! All properties and fast arrays in this RepLayout use Push Model.
	HasObjectOrNetSerializeProperties	= (1 << 3),	//! Will be set for any RepLayout that contains Object or Net Serialize property commands.
	NoReplicatedProperties				= (1 << 4), //! Will be set if the RepLayout has no lifetime properties, or they are all disabled.
	FullPushProperties					= (1 << 5), //! All properties in this RepLayout use Push Model.
	HasInitialOnlyProperties			= (1 << 6), //! There is at least 1 Initial Only Lifetime property on this RepLayout.
	HasDynamicConditionProperties		= (1 << 7), //! There is at least 1 Dynamic lifetime property on this RepLayout.
};
ENUM_CLASS_FLAGS(ERepLayoutFlags);

const TCHAR* LexToString(ERepLayoutFlags Flag);

enum class ERepLayoutResult
{
	Success,	// Operation succeeded
	Empty,		// Operation succeeded, but didn't generate (or consume) any data.
	Error,		// Operation failed, but may succeed later.
	FatalError	// Operation failed, and connection should be terminated.
};

/**
 * This class holds all replicated properties for a given type (either a UClass, UStruct, or UFunction).
 * Helpers functions exist to read, write, and compare property state.
 *
 * There is only one FRepLayout for a given type, meaning all instances of the type share the FRepState.
 *
 * COMMANDS:
 *
 * All Properties in a RepLayout are represented as Layout Commands.
 * These commands dictate:
 *		- What the underlying data type is.
 *		- How the data is laid out in memory.
 *		- How the data should be serialized.
 *		- How the data should be compared (between instances of Objects, Structs, etc.).
 *		- Whether or not the data should trigger notifications on change (RepNotifies).
 *		- Whether or not the data is conditional (e.g. may be skipped when sending to some or all connections).
 *
 * Commands are split into 2 main types: Parent Commands (@see FRepParentCmd) and Child Commands (@see FRepLayoutCmd).
 *
 * A Parent Command represents a Top Level Property of the type represented by an FRepLayout.
 * A Child Command represents any Property (even nested properties).
 *
 * E.G.,
 *		Imagine an Object O, with 4 Properties, CA, DA, I, and S.
 *			CA is a fixed size C-Style array. This will generate 1 Parent Command and 1 Child Command for *each* element in the array.
 *			DA is a dynamic array (TArray). This will generate only 1 Parent Command and 1 Child Command, both referencing the array.
 *				Additionally, Child Commands will be added recursively for the element type of the array.
 *			S is a UStruct.
 *				All struct types generate 1 Parent Command for the Struct Property. Additionally:
 *					If the struct has a native NetSerialize method then it will generate 1 Child Command referencing the struct.
 *					If the struct has a native NetDeltaSerialize method then it will generate no Child Commands.
 *					All other structs will recursively generate Child Commands for each nested Net Property in the struct.
 *						Note, in this case there is no Child Command associated with the top level struct property.
 *			I is an integer (or other supported non-Struct type, or object reference). This will generate 1 Parent Command and 1 Child Command.
 *
 * CHANGELISTS
 *
 * Along with Layout Commands that describe the Properties in a type, RepLayout uses changelists to know
 * what Properties have changed between frames. @see FRepChangedHistory.
 *
 * Changelists are arrays of Property Handles that describe what Properties have changed, however they don't
 * track the actual values of the Properties.
 *
 * Changelists can contain "sub-changelists" for arrays. Formally, they can be described as the following grammar:
 *
 *		Terminator			::=	0
 *		Handle				::= Integer between 1 ~ 65535
 *		Number				::= Integer between 0 ~ 65535
 *		Changelist			::=	<Terminator> | <Handle><Changelist> | <Handle><Array-Changelist><Changelist>
 *		Array-Changelist:	::= <Number><Changelist>
 *
 * An important distinction is that Handles do not have a 1:1 mapping with RepLayoutCommands.
 * Handles are 1-based (as opposed to 0-based), and track a relative command index within a single
 * level of a changelist. Each Array Command, regardless of the number of child Commands it has,
 * will only be count as a single handle in its owning changelist. Each time we recurse into an Array-Changelist,
 * our handles restart at 1 for that "depth", and they correspond to the Commands associated with the Array's element type.
 *
 * In order to generate Changelists, Layout Commands are sequentially applied that compare the values
 * of an object's cached state to a object's current state. Any properties that are found to be different
 * will have their handle written into the changelist. This means handles within a changelists are
 * inherently ordered (with arrays inserted whose Handles are also ordered).
 *
 * When we want to replicate properties for an object, merge together any outstanding changelists
 * and then iterate over it using Layout Commands that serialize the necessary property data.
 *
 * Receiving is very similar, except the Handles are baked into the serialized data so no
 * explicit changelist is required. As each Handle is read, a Layout Command is applied
 * that serializes the data from the network bunch and applies it to an object.
 *
 * RETRIES AND RELIABLES
 *
 * @FSendingRepState maintains a circular buffer that tracks recently sent Changelists (@FRepChangedHistory).
 * These history items track the Changelist alongside the Packet ID that the bunches were sent in.
 * 
 * Once we receive ACKs for all associated packets, the history will be removed from the buffer.
 * If NAKs are received for any of the packets, we will merge the changelist into the next set of properties we replicate.
 *
 * If we receive no NAKs or ACKs for an extended period, to prevent overflows in the history buffer,
 * we will merge the entire buffer into a single monolithic changelist which will be sent alongside the next set of properties.
 *
 * In both cases of NAKs or no response, the merged changelists will be tracked in the latest history item
 * alongside with other sent properties.
 *
 * When "net.PartialBunchReliableThreshold" is non-zero and property data bunches are split into partial bunches above
 * the threshold, we will not generate a history item. Instead, we will rely on the reliable bunch framework for resends
 * and replication of the Object will be completely paused until the property bunches are acknowledged.
 * However, this will not affect other history items since they are still unreliable.
 */
class FRepLayout : public FGCObject, public TSharedFromThis<FRepLayout>
{
private:

	friend struct FRepStateStaticBuffer;
	friend class UPackageMapClient;
	friend class FNetSerializeCB;
	friend struct FCustomDeltaPropertyIterator;

	FRepLayout();

public:

	virtual ~FRepLayout();

	/** Creates a new FRepLayout for the given class. */
	ENGINE_API static TSharedPtr<FRepLayout> CreateFromClass(UClass* InObjectClass, const UNetConnection* ServerConnection = nullptr, const ECreateRepLayoutFlags Flags = ECreateRepLayoutFlags::None);

	/** Creates a new FRepLayout for the given struct. */
	ENGINE_API static TSharedPtr<FRepLayout> CreateFromStruct(UStruct * InStruct, const UNetConnection* ServerConnection = nullptr, const ECreateRepLayoutFlags Flags = ECreateRepLayoutFlags::None);

	/** Creates a new FRepLayout for the given function. */
	static TSharedPtr<FRepLayout> CreateFromFunction(UFunction* InFunction, const UNetConnection* ServerConnection = nullptr, const ECreateRepLayoutFlags Flags = ECreateRepLayoutFlags::None);

	/**
	 * Creates and initialize a new Shadow Buffer.
	 *
	 * Shadow Data / Shadow States are used to cache property data so that the Object's state can be
	 * compared between frames to see if any properties have changed. They are also used on clients
	 * to keep track of RepNotify state.
	 *
	 * This includes:
	 *		- Allocating memory for all Properties in the class.
	 *		- Constructing instances of each Property.
	 *		- Copying the values of the Properties from given object.
	 *
	 * @param Source	Memory buffer storing object property data.
	 */
	FRepStateStaticBuffer CreateShadowBuffer(const FConstRepObjectDataBuffer Source) const;

	/**
	 * Creates and initializes a new FReplicationChangelistMgr.
	 *
	 * @param InObject		The Object that is being managed.
	 * @param CreateFlags	Flags modifying how the manager is created.
	 */
	TSharedPtr<FReplicationChangelistMgr> CreateReplicationChangelistMgr(const UObject* InObject, const ECreateReplicationChangelistMgrFlags CreateFlags) const;

	/**
	 * Creates and initializes a new FRepState.
	 *
	 * This includes:
	 *		- Initializing the ShadowData.
	 *		- Associating and validating the appropriate ChangedPropertyTracker.
	 *		- Building initial ConditionMap.
	 *
	 * @param RepState						The RepState to initialize.
	 * @param Class							The class of the object represented by the input memory.
	 * @param Src							Memory buffer storing object property data.
	 * @param InRepChangedPropertyTracker	The PropertyTracker we want to associate with the RepState.
	 *
	 * @return A new RepState.
	 *			Note, maybe a a FRepStateBase or FRepStateSending based on parameters.
	 */
	TUniquePtr<FRepState> CreateRepState(
		const FConstRepObjectDataBuffer Source,
		TSharedPtr<FRepChangedPropertyTracker>& InRepChangedPropertyTracker,
		ECreateRepStateFlags Flags) const;

	UE_DEPRECATED(5.1, "No longer used, trackers are initialized by the replication subsystem.")
	void InitChangedTracker(FRepChangedPropertyTracker * ChangedTracker) const;

	/**
	 * Writes out any changed properties for an Object into the given data buffer,
	 * and does book keeping for the RepState of the object.
	 *
	 * Note, this does not compare properties or send them on the wire, it's only used
	 * to serialize properties.
	 *
	 * @param RepState				RepState for the object.
	 *								This is expected to be valid.
	 * @param RepChangelistState	RepChangelistState for the object.
	 * @param Data					Pointer to memory where property data is stored.
	 * @param ObjectClass			Class of the object.
	 * @param Writer				Writer used to store / write out the replicated properties.
	 * @param RepFlags				Flags used for replication.
	 */
	bool ReplicateProperties(
		FSendingRepState* RESTRICT RepState,
		FRepChangelistState* RESTRICT RepChangelistState,
		const FConstRepObjectDataBuffer Data,
		UClass* ObjectClass,
		UActorChannel* OwningChannel,
		FNetBitWriter& Writer,
		const FReplicationFlags& RepFlags) const;

	/**
	 * Reads all property values from the received buffer, and applies them to the
	 * property memory.
	 *
	 * @param OwningChannel			The channel of the Actor that owns the object whose properties we're reading.
	 * @param InObjectClass			Class of the object.
	 * @param RepState				RepState for the object.
	 *								This is expected to be valid.
	 * @param Data					Pointer to memory where read property data should be stored.
	 * @param InBunch				The data that should be read.
	 * @param bOutHasUnmapped		Whether or not unmapped GUIDs were read.
	 * @param bOutGuidsChanged		Whether or not any GUIDs were changed.
	 * @param Flags					Controls how ReceiveProperties behaves.
	 */
	bool ReceiveProperties(
		UActorChannel* OwningChannel,
		UClass* InObjectClass,
		FReceivingRepState* RESTRICT RepState,
		UObject* Object,
		FNetBitReader& InBunch,
		bool& bOutHasUnmapped,
		bool& bOutGuidsChanged,
		const EReceivePropertiesFlags InFlags) const;

	/**
	 * Finds any properties in the Shadow Buffer of the given Rep State that are currently valid
	 * (mapped or unmapped) references to other network objects, and retrieves the associated
	 * Net GUIDS.
	 *
	 * @param RepState					The RepState whose shadow buffer we'll inspect.
	 *									This is expected to be valid.
	 * @param OutReferencedGuids		Set of Net GUIDs being referenced by the RepState.
	 * @param OutTrackedGuidMemoryBytes	Total memory usage of properties containing GUID references. 
	 */
	void GatherGuidReferences(
		FReceivingRepState* RESTRICT RepState,
		struct FNetDeltaSerializeInfo& Params,
		TSet<FNetworkGUID>& OutReferencedGuids,
		int32& OutTrackedGuidMemoryBytes) const;

	/**
	 * Called to indicate that the object referenced by the FNetworkGUID is no longer mapped.
	 * This can happen if the Object was destroyed, if its level was streamed out, or for any
	 * other reason that may cause the Server (or client) to no longer be able to properly
	 * reference the object. Note, it's possible the object may become valid again later.
	 *
	 * @param RepState	The RepState that holds a reference to the object.
	 *					This is expected to be valid.
	 * @param GUID		The Network GUID of the object to unmap.
	 * @param Params	Delta Serialization Params used for Custom Delta Properties.
	 */
	bool MoveMappedObjectToUnmapped(
		FReceivingRepState* RESTRICT RepState,
		struct FNetDeltaSerializeInfo& Params,
		const FNetworkGUID& GUID) const;

	/**
	 * Attempts to update any unmapped network guids referenced by the RepState.
	 * If any guids become mapped, we will update corresponding properties on the given
	 * object to point to the referenced object.
	 *
	 * @param RepState					The RepState associated with the Object.
	 *									This is expected to be valid.
	 * @param PackageMap				The package map that controls FNetworkGUID associations.
	 * @param Object					The live game object whose properties should be updated if we map any objects.
	 * @param bOutSomeObjectsWereMapped	Whether or not we successfully mapped any references.
	 * @param bOutHasMoreUnamapped		Whether or not there are more unmapped references in the RepState.
	 */
	void UpdateUnmappedObjects(
		FReceivingRepState* RESTRICT RepState,
		UPackageMap* PackageMap,
		UObject* Object,
		struct FNetDeltaSerializeInfo& Params,
		bool& bCalledPreNetReceive,
		bool& bOutSomeObjectsWereMapped,
		bool& bOutHasMoreUnmapped) const;

	/**
	 * Fire any RepNotifies that have been queued for an object while receiving properties.
	 *
	 * @param RepState	The ReceivingRepState associated with the Object.
	 *					This is expected to be valid.
	 * @param Object	The Object that received properties.
	 */
	void CallRepNotifies(FReceivingRepState* RepState, UObject* Object) const;

	template<ERepDataBufferType DataType>
	void ValidateWithChecksum(TConstRepDataBuffer<DataType> Data, FBitArchive & Ar) const;

	uint32 GenerateChecksum(const FRepState* RepState) const;

	/**
	 * Compare all properties between source and destination buffer, and optionally update the destination
	 * buffer to match the state of the source buffer if they don't match.
	 *
	 * @param RepNotifies	RepNotifies that should be fired if we're changing properties.
	 * @param Destination	Destination buffer that will be changed if we're changing properties.
	 * @param Source		Source buffer containing desired property values.
	 * @param Flags			Controls how DiffProperties behaves.
	 *
	 * @return True if there were any properties with different values.
	 */
	template<ERepDataBufferType DstType, ERepDataBufferType SrcType>
	bool DiffProperties(
		TArray<FProperty*>* RepNotifies,
		TRepDataBuffer<DstType> Destination,
		TConstRepDataBuffer<SrcType> Source,
		const EDiffPropertiesFlags Flags) const;

	/**
	 * @see DiffProperties
	 *
	 * The main difference between this method and DiffProperties is that this method will skip
	 * any properties that are:
	 *
	 *	- Transient
	 *	- Point to Actors or ActorComponents
	 *	- Point to Objects that are non-stably named for networking.
	 *
	 * @param RepNotifies	RepNotifies that should be fired if we're changing properties.
	 * @param Destination	Destination buffer that will be changed if we're changing properties.
	 * @param Source		Source buffer containing desired property values.
	 * @param Flags			Controls how DiffProperties behaves.
	 *
	 * @return True if there were any properties with different values.
	 */
	template<ERepDataBufferType DstType, ERepDataBufferType SrcType>
	bool DiffStableProperties(
		TArray<FProperty*>* RepNotifies,
		TArray<UObject*>* ObjReferences,
		TRepDataBuffer<DstType> Destination,
		TConstRepDataBuffer<SrcType> Source) const;

	/** @see SendProperties. */
	void ENGINE_API SendPropertiesForRPC(
		UFunction* Function,
		UActorChannel* Channel,
		FNetBitWriter& Writer,
		const FConstRepObjectDataBuffer Data) const;

	/** @see ReceiveProperties. */
	void ReceivePropertiesForRPC(
		UObject* Object,
		UFunction* Function,
		UActorChannel* Channel,
		FNetBitReader& Reader,
		FRepObjectDataBuffer Data,
		TSet<FNetworkGUID>& UnmappedGuids) const;

	/** Builds shared serialization state for a multicast rpc */
	void ENGINE_API BuildSharedSerializationForRPC(const FConstRepObjectDataBuffer Data, UE::Net::FNetTokenStore* NetTokenStore = nullptr);

	/** Clears shared serialization state for a multicast rpc */
	void ENGINE_API ClearSharedSerializationForRPC();

	// Struct support
	ENGINE_API void SerializePropertiesForStruct(
		UStruct* Struct,
		FBitArchive& Ar,
		UPackageMap* Map,
		FRepObjectDataBuffer Data,
		bool& bHasUnmapped,
		const UObject* OwningObject = nullptr) const;

	/** Serializes all replicated properties of a UObject in or out of an archive (depending on what type of archive it is). */
	ENGINE_API void SerializeObjectReplicatedProperties(UObject* Object, FBitArchive & Ar) const;

	UObject* GetOwner() const { return Owner; }

	/** Currently only used for Replays / with the UDemoNetDriver. */
	void SendProperties_BackwardsCompatible(
		FSendingRepState* RESTRICT RepState,
		FRepChangedPropertyTracker* ChangedTracker,
		const FConstRepObjectDataBuffer Data,
		UNetConnection* Connection,
		FNetBitWriter& Writer,
		TArray<uint16>& Changed) const;

	/** Currently only used for Replays / with the UDemoNetDriver. */
	bool ReceiveProperties_BackwardsCompatible(
		UNetConnection* Connection,
		FReceivingRepState* RESTRICT RepState,
		FRepObjectDataBuffer Data,
		FNetBitReader& InBunch,
		bool& bOutHasUnmapped,
		const bool bEnableRepNotifies,
		bool& bOutGuidsChanged,
		UObject* OwningObject = nullptr) const;

	//~ Begin FGCObject Interface
	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	ENGINE_API virtual FString GetReferencerName() const override;
	//~ End FGCObject Interface

	/**
	 * Gets a pointer to the value of the given property in the Shadow State.
	 *
	 * @return A pointer to the property value in the shadow state, or nullptr if the property wasn't found.
	 */
	template<typename T>
	T* GetShadowStateValue(FRepShadowDataBuffer Data, const FName PropertyName)
	{
		for (const FRepParentCmd& Parent : Parents)
		{
			if (Parent.CachedPropertyName == PropertyName)
			{
				return (T*)((Data + Parent).Data);
			}
		}

		return nullptr;
	}

	template<typename T>
	const T* GetShadowStateValue(FConstRepShadowDataBuffer Data, const FName PropertyName) const
	{
		for (const FRepParentCmd& Parent : Parents)
		{
			if (Parent.CachedPropertyName == PropertyName)
			{
				return (const T*)((Data + Parent).Data);
			}
		}

		return nullptr;
	}

	const ERepLayoutFlags GetFlags() const
	{
		return Flags;
	}

	const bool IsEmpty() const
	{
		return EnumHasAnyFlags(Flags, ERepLayoutFlags::NoReplicatedProperties) || (0 == Parents.Num());
	}

	const int32 GetNumParents() const
	{
		return Parents.Num();
	}

	const FProperty* GetParentProperty(int32 Index) const
	{ 
		return Parents.IsValidIndex(Index) ? Parents[Index].Property : nullptr;
	}

	const int32 GetParentArrayIndex(int32 Index) const
	{
		return Parents.IsValidIndex(Index) ? Parents[Index].ArrayIndex : 0;
	}

	const int32 GetParentCondition(int32 Index) const
	{
		return Parents.IsValidIndex(Index) ? Parents[Index].Condition : COND_None;
	}

	const bool IsCustomDeltaProperty(int32 Index) const
	{
		return Parents.IsValidIndex(Index) ? EnumHasAnyFlags(Parents[Index].Flags, ERepParentFlags::IsCustomDelta) : false;
	}

#if WITH_PUSH_MODEL
	const bool IsPushModelProperty(int32 Index) const
	{
		return PushModelProperties.IsValidIndex(Index) ? PushModelProperties[Index] : false;
	}
#endif

	const uint16 GetCustomDeltaIndexFromPropertyRepIndex(const uint16 PropertyRepIndex) const;

	void CountBytes(FArchive& Ar) const;

private:

	void InitFromClass(UClass* InObjectClass, const UNetConnection* ServerConnection, const ECreateRepLayoutFlags Flags);

	void InitFromStruct(UStruct* InStruct, const UNetConnection* ServerConnection, const ECreateRepLayoutFlags Flags);

	void InitFromFunction(UFunction* InFunction, const UNetConnection* ServerConnection, const ECreateRepLayoutFlags Flags);

	/**
	 * Compare Property Values currently stored in the Changelist State to the Property Values
	 * in the passed in data, generating a new changelist if necessary.
	 *
	 * @param RepState				RepState for the object.
	 * @param RepChangelistState	The FRepChangelistState that contains the last cached values and changelists.
	 * @param Data					The newest Property Data available.
	 * @param RepFlags				Flags that will be used if the object is replicated.
	 * @param bForceCompare			Compare the property even if the dirty flag is not set.
	 */
	ERepLayoutResult CompareProperties(
		FSendingRepState* RESTRICT RepState,
		FRepChangelistState* RESTRICT RepChangelistState,
		const FConstRepObjectDataBuffer Data,
		const FReplicationFlags& RepFlags,
		const bool bForceCompare) const;

	/**
	 * Writes all changed property values from the input owner data to the given buffer.
	 * This is used primarily by ReplicateProperties.
	 *
	 * Note, the changelist is expected to have any conditional properties whose conditions
	 * aren't met filtered out already. See FRepState::ConditionMap and FRepLayout::FilterChangeList
	 *
	 * @param RepState			RepState for the object.
	 *							This is expected to be valid.
	 * @param ChangedTracker	Used to indicate
	 * @param Data				Pointer to the object's memory.
	 * @param ObjectClass		Class of the object.
	 * @param Writer			Writer used to store / write out the replicated properties.
	 * @param Changed			Aggregate list of property handles that need to be written.
	 * @param SharedInfo		Shared Serialization state for properties.
	 */
	void SendProperties(
		FSendingRepState* RESTRICT RepState,
		FRepChangedPropertyTracker* ChangedTracker,
		const FConstRepObjectDataBuffer Data,
		UClass* ObjectClass,
		FNetBitWriter& Writer,
		TArray<uint16>& Changed,
		const FRepSerializationSharedInfo& SharedInfo,
		const ESerializePropertyType SerializePropertyType) const;

	/**
	 * Clamps a changelist so that it conforms to the current size of either an array, or arrays within structs/arrays.
	 *
	 * @param Data				Object memory.
	 * @param Changed			The changelist to prune.
	 * @param PrunedChanged		The resulting pruned changelist.
	 */
	void PruneChangeList(
		const FConstRepObjectDataBuffer Data,
		const TArray<uint16>& Changed,
		TArray<uint16>& PrunedChanged) const;

	void MergeChangeList(
		const FConstRepObjectDataBuffer Data,
		const TArray<uint16>& Dirty1,
		const TArray<uint16>& Dirty2,
		TArray<uint16>& MergedDirty) const;

	void RebuildConditionalProperties(
		FSendingRepState* RepState,
		const FReplicationFlags RepFlags) const;

	void UpdateChangelistHistory(
		FSendingRepState* RepState,
		UClass* ObjectClass,
		const FConstRepObjectDataBuffer Data,
		UNetConnection* Connection,
		TArray<uint16>* OutMerged) const;

	void SendProperties_BackwardsCompatible_r(
		FSendingRepState* RESTRICT RepState,
		UPackageMapClient* PackageMapClient,
		FNetFieldExportGroup* NetFieldExportGroup,
		FRepChangedPropertyTracker* ChangedTracker,
		FNetBitWriter& Writer,
		const bool bDoChecksum,
		FRepHandleIterator& HandleIterator,
		const FConstRepObjectDataBuffer Source) const;

	void SendAllProperties_BackwardsCompatible_r(
		FSendingRepState* RESTRICT RepState,
		FNetBitWriter& Writer,
		const bool bDoChecksum,
		UPackageMapClient* PackageMapClient,
		FNetFieldExportGroup* NetFieldExportGroup,
		const int32 CmdStart,
		const int32 CmdEnd,
		const FConstRepObjectDataBuffer SourceData) const;

	void SendProperties_r(
		FSendingRepState* RESTRICT RepState,
		FNetBitWriter& Writer,
		const bool bDoChecksum,
		FRepHandleIterator& HandleIterator,
		const FConstRepObjectDataBuffer SourceData,
		const int32	 ArrayDepth,
		const FRepSerializationSharedInfo* const RESTRICT SharedInfo,
		const ESerializePropertyType SerializePropertyType) const;

	void BuildSharedSerialization(
		const FConstRepObjectDataBuffer Data,
		TArray<uint16>& Changed,
		const bool bWriteHandle,
		FRepSerializationSharedInfo& SharedInfo, 
		UE::Net::FNetTokenStore* NetTokenStore = nullptr) const;

	void BuildSharedSerialization_r(
		FRepHandleIterator& RepHandleIterator,
		const FConstRepObjectDataBuffer SourceData,
		const bool bWriteHandle,
		const bool bDoChecksum,
		const int32 ArrayDepth,
		FRepSerializationSharedInfo& SharedInfo) const;

	void BuildSharedSerializationForRPC_DynamicArray_r(
		const int32 CmdIndex,
		const FConstRepObjectDataBuffer Data,
		int32 ArrayDepth,
		FRepSerializationSharedInfo& SharedInfo);

	void BuildSharedSerializationForRPC_r(
		const int32 CmdStart,
		const int32 CmdEnd,
		const FConstRepObjectDataBuffer Data,
		int32 ArrayIndex,
		int32 ArrayDepth,
		FRepSerializationSharedInfo& SharedInfo);

	TSharedPtr<FNetFieldExportGroup> CreateNetfieldExportGroup() const;

	int32 FindCompatibleProperty(
		const int32 CmdStart,
		const int32 CmdEnd,
		const uint32 Checksum) const;

	bool ReceiveProperties_BackwardsCompatible_r(
		FReceivingRepState* RESTRICT RepState,
		FNetFieldExportGroup* NetFieldExportGroup,
		FNetBitReader& Reader,
		const int32 CmdStart,
		const int32 CmdEnd,
		FRepShadowDataBuffer ShadowData,
		FRepObjectDataBuffer OldData,
		FRepObjectDataBuffer Data,
		FGuidReferencesMap* GuidReferencesMap,
		bool& bOutHasUnmapped,
		bool& bOutGuidsChanged,
		UObject* OwningObject) const;

	void GatherGuidReferences_r(
		const FGuidReferencesMap* GuidReferencesMap,
		TSet<FNetworkGUID>& OutReferencedGuids,
		int32& OutTrackedGuidMemoryBytes) const;

	bool MoveMappedObjectToUnmapped_r(
		FGuidReferencesMap* GuidReferencesMap,
		const FNetworkGUID& GUID,
		const UObject* const OwningObject) const;

	void UpdateUnmappedObjects_r(
		FReceivingRepState* RESTRICT RepState, 
		FGuidReferencesMap* GuidReferencesMap,
		UObject* OriginalObject,
		UNetConnection* Connection,
		FRepShadowDataBuffer ShadowData, 
		FRepObjectDataBuffer Data, 
		const int32 MaxAbsOffset,
		bool& bCalledPreNetReceive,
		bool& bOutSomeObjectsWereMapped,
		bool& bOutHasMoreUnmapped) const;

	void SanityCheckChangeList_DynamicArray_r(
		const int32 CmdIndex, 
		const FConstRepObjectDataBuffer Data, 
		TArray<uint16>& Changed,
		int32& ChangedIndex) const;

	uint16 SanityCheckChangeList_r(
		const int32 CmdStart, 
		const int32 CmdEnd, 
		const FConstRepObjectDataBuffer Data, 
		TArray<uint16>& Changed,
		int32& ChangedIndex,
		uint16 Handle) const;

	void SanityCheckChangeList(const FConstRepObjectDataBuffer Data, TArray<uint16>& Changed) const;

	void SerializeProperties_DynamicArray_r(
		FBitArchive& Ar, 
		UPackageMap* Map,
		const int32 CmdIndex,
		FRepObjectDataBuffer Data,
		bool& bHasUnmapped,
		const int32 ArrayDepth,
		const FRepSerializationSharedInfo& SharedInfo,
		FNetTraceCollector* Collector,
		const UObject* OwningObject) const;

	void SerializeProperties_r(
		FBitArchive& Ar, 
		UPackageMap* Map,
		const int32 CmdStart, 
		const int32 CmdEnd, 
		FRepObjectDataBuffer Data,
		bool& bHasUnmapped,
		const int32 ArrayIndex,
		const int32 ArrayDepth,
		const FRepSerializationSharedInfo& SharedInfo,
		FNetTraceCollector* Collector,
		const UObject* OwningObject) const;

	void MergeChangeList_r(
		FRepHandleIterator& RepHandleIterator1,
		FRepHandleIterator& RepHandleIterator2,
		const FConstRepObjectDataBuffer SourceData,
		TArray<uint16>& OutChanged) const;

	void PruneChangeList_r(
		FRepHandleIterator& RepHandleIterator,
		const FConstRepObjectDataBuffer SourceData,
		TArray<uint16>& OutChanged) const;

	/**
	 * Splits a given Changelist into an Inactive Change List and an Active Change List.
	 * 
	 * @param Changelist				The Changelist to filter.
	 * @param InactiveParentHandles		The set of ParentCmd Indices that are not active.
	 * @param OutInactiveProperties		The properties found to be inactive.
	 * @param OutActiveProperties		The properties found to be active.
	 */
	void FilterChangeList( 
		const TArray<uint16>& Changelist,
		const TBitArray<>& InactiveParentHandles,
		TArray<uint16>& OutInactiveProperties,
		TArray<uint16>& OutActiveProperties) const;

	/** Same as FilterChangeList, but only populates an Active Change List. */
	void FilterChangeListToActive(
		const TArray<uint16>& Changelist,
		const TBitArray<>& InactiveParentHandles,
		TArray<uint16>& OutActiveProperties) const;

	void BuildChangeList_r(
		const TArray<FHandleToCmdIndex>& HandleToCmdIndex,
		const int32 CmdStart,
		const int32 CmdEnd,
		const FConstRepObjectDataBuffer Data,
		const int32 HandleOffset,
		const bool bForceArraySends,
		TArray<uint16>& Changed) const;

	void BuildHandleToCmdIndexTable_r(
		const int32 CmdStart,
		const int32 CmdEnd,
		TArray<FHandleToCmdIndex>& HandleToCmdIndex);

	ERepLayoutResult UpdateChangelistMgr(
		FSendingRepState* RESTRICT RepState,
		FReplicationChangelistMgr& InChangelistMgr,
		const UObject* InObject,
		const uint32 ReplicationFrame,
		const FReplicationFlags& RepFlags,
		const bool bForceCompare) const;

	void InitRepStateStaticBuffer(FRepStateStaticBuffer& ShadowData, const FConstRepObjectDataBuffer Source) const;
	void ConstructProperties(FRepStateStaticBuffer& ShadowData) const;
	void CopyProperties(FRepStateStaticBuffer& ShadowData, const FConstRepObjectDataBuffer Source) const;
	void DestructProperties(FRepStateStaticBuffer& RepStateStaticBuffer) const;

	/**
	 * @param CustomDeltaIndex	The index of the Custom Delta Property.
	 *							This is not the same as FProperty::RepIndex!
	 *							@see FLifetimeCustomDeltaState.
	 */
	bool SendCustomDeltaProperty(FNetDeltaSerializeInfo& Params, const uint16 CustomDeltaIndex) const;

	/**
	 * Attempts to receive the custom delta property.
	 *
	 *
	 *
	 * @param Params			Params that we'll use to receive.
	 *							The PackageMap member must be non null.
	 *							The Object member must be non null.
	 *							The Reader member must be non null.
	 * @param Property			The Property that we're trying to received.
	 *							This is expected to be a valid Replicated property that is a CustomDelta type.
	 */
	bool ReceiveCustomDeltaProperty(
		FReceivingRepState* RESTRICT ReceivingRepState,
		FNetDeltaSerializeInfo& Params,
		FStructProperty* Property) const;

	ERepLayoutResult DeltaSerializeFastArrayProperty(struct FFastArrayDeltaSerializeParams& Params, FReplicationChangelistMgr* ChangelistMgr) const;

	void GatherGuidReferencesForFastArray(struct FFastArrayDeltaSerializeParams& Params) const;

	bool MoveMappedObjectToUnmappedForFastArray(struct FFastArrayDeltaSerializeParams& Params) const;

	void UpdateUnmappedGuidsForFastArray(struct FFastArrayDeltaSerializeParams& Params) const;

	void PreSendCustomDeltaProperties(
		UObject* Object,
		UNetConnection* Connection,
		FReplicationChangelistMgr& ChangelistMgr,
		uint32 ReplicationFrame,
		TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates) const;

	void PostSendCustomDeltaProperties(
		UObject* Object,
		UNetConnection* Connection,
		FReplicationChangelistMgr& ChangelistMgr,
		TArray<TSharedPtr<INetDeltaBaseState>>& CustomDeltaStates) const;

	const uint16 GetNumLifetimeCustomDeltaProperties() const;

	const uint16 GetLifetimeCustomDeltaPropertyRepIndex(const uint16 RepIndCustomDeltaPropertyIndex) const;

	FProperty* GetLifetimeCustomDeltaProperty(const uint16 CustomDeltaPropertyIndex) const;

	const ELifetimeCondition GetLifetimeCustomDeltaPropertyCondition(const uint16 RepIndCustomDeltaPropertyIndex) const;

	ERepLayoutFlags Flags;

	/** Size (in bytes) needed to allocate a single instance of a Shadow buffer for this RepLayout. */
	int32 ShadowDataBufferSize;

	/** Top level Layout Commands. */
	TArray<FRepParentCmd> Parents;

	/** All Layout Commands. */
	TArray<FRepLayoutCmd> Cmds;

	/** Converts a relative handle to the appropriate index into the Cmds array */
	TArray<FHandleToCmdIndex> BaseHandleToCmdIndex;

	/**
	 * Special state tracking for Lifetime Custom Delta Properties.
	 * Will only ever be valid if the Layout has Lifetime Custom Delta Properties.
	 */
	TUniquePtr<struct FLifetimeCustomDeltaState> LifetimeCustomPropertyState;

	/** UClass, UStruct, or UFunction that this FRepLayout represents.*/
	UStruct* Owner;

	/** Shared serialization state for a multicast rpc */
	FRepSerializationSharedInfo SharedInfoRPC;

	/** Shared comparison to default state for multicast rpc */
	TBitArray<> SharedInfoRPCParentsChanged;

#if WITH_PUSH_MODEL
	/** Properties that have push model enabled. */
	TBitArray<> PushModelProperties;
#endif

	TMap<FRepLayoutCmd*, TArray<FRepLayoutCmd>> NetSerializeLayouts;
};