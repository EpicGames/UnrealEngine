// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/SharedPointer.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/ReplayResult.h"

/**
 * Delegate that is called prior to starting each replay in a playlist,
 * giving game code a chance to update any necessary options or
 * perform other work.
 */
DECLARE_DELEGATE_TwoParams(FPreStartNextPlaylistReplay, const struct FReplayPlaylistTracker& /*ThisPlaylistTracker*/, struct FReplayPlaylistUpdatedOptions& /*OverrideWorldUpdate*/);

/** Set of options that can be updated before starting each replay in a playlist. */
struct FReplayPlaylistUpdatedOptions
{
	TOptional<class UWorld*> NewWorldOverride;
	TOptional<TArray<FString>> NewAdditionalOptions;
};

/** Set of parameters used with UGameInstance::PlayReplayPlatlist to control how the playlist is started. */
struct FReplayPlaylistParams
{
	/** Set of replays to use as the playlist. */
	TArray<FString> Playlist;

	/** Callback that can be used to update playback options after each individual replay has been played. */
	FPreStartNextPlaylistReplay PreStartNextPlaylistReplayDelegate;

	/**
	 * Set of options that will be used to play the very first replay.
	 * These can be updated via PreStartNextPlaylistReplayDelegate before each new replay begins
	 * (including before looping if "Demo.Loop" is enabled.
	 */
	FReplayPlaylistUpdatedOptions InitialOptions;
};

/** Used with UGameInstance::PlayReplayPlaylist to manage playing a set of Replays in succession. */
struct FReplayPlaylistTracker : public TSharedFromThis<FReplayPlaylistTracker>, public FNoncopyable
{
private:

	using ThisClass = FReplayPlaylistTracker;
	friend class FGameInstanceReplayPlaylistHelper;
	friend class FDemoNetDriverReplayPlaylistHelper;

public:

	~FReplayPlaylistTracker()
	{
		// Forcibly reset this in case we end up cancelling requests while trying to start a new replay.
		bIsStartingReplay = false;
		Reset();
	}

	/** Whether or not we're currently on the last replay in a playlist. */
	const int32 IsOnLastReplay() const
	{
		return GetCurrentReplay() == GetNumReplays();
	}

	/** Gets the number of replays in the playlist. */
	const int32 GetNumReplays() const
	{
		return Playlist.Num();
	}

	/** Gets the index of the current replay in the playlist. */
	const int32 GetCurrentReplay() const
	{
		return CurrentReplay;
	}

	/** Gets the actual playlist. */
	const TArrayView<const FString> GetPlaylist() const
	{
		return Playlist;
	}

	/** Gets the last updated value of AdditionalOptions. */
	const TArrayView<const FString> GetAdditionalOptions() const
	{
		return AdditionalOptions;
	}

	/** Gets the last updated value of WorldOverride. */
	class UWorld* GetWorldOverride() const
	{
		return WorldOverride.Get();
	}

	/** Gets the GameInstance used to start replays. */
	class UGameInstance* GetGameInstance() const
	{
		return GameInstance.Get();
	}

private:

	FReplayPlaylistTracker(const FReplayPlaylistParams& Params, class UGameInstance* InGameInstance);

	void Reset();

	bool Start();

	bool Restart();

	void Quit();

	void PlayNextReplay();

	/** Called if an error occurs *after* the replay was requested to start (successfully), but before the replay actually starts. */
	void OnDemoPlaybackFailed(UWorld* InWorld, const UE::Net::TNetResult<EReplayResult>& Result);

	/** Called when the DemoNetDriver hits a Hard Stop, like an error or a user requesting a stop. */
	void OnDemoStopped(UWorld* InWorld);

	/** Called when a replay has reached its end. */
	void OnDemoPlaybackFinished(UWorld* InWorld);

	bool bIsStartingReplay;
	int32 CurrentReplay;
	TArray<FString> Playlist;
	TArray<FString> AdditionalOptions;
	TWeakObjectPtr<class UWorld> WorldOverride;
	TWeakObjectPtr<class UGameInstance> GameInstance;
	TWeakObjectPtr<class UDemoNetDriver> DemoNetDriver;

	FDelegateHandle OnDemoPlaybackFailedHandle;
	FDelegateHandle OnDemoPlaybackFinishedHandle;
	FDelegateHandle OnDemoStoppedHandle;

	FPreStartNextPlaylistReplay PreStartNextReplay;
};