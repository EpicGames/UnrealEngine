// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Connection/NetResult.h"
#include "ReplayResult.generated.h"

UENUM()
enum class EReplayResult : uint32
{
	Success,
	ReplayNotFound,
	Corrupt,
	UnsupportedCheckpoint,
	GameSpecific,
	InitConnect,
	LoadMap,
	Serialization,
	StreamerError,
	ConnectionClosed,
	MissingArchive,
	Unknown,
};

DECLARE_NETRESULT_ENUM(EReplayResult);

ENGINE_API const TCHAR* LexToString(EReplayResult Result);