// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "Net/Core/Misc/NetConditionGroupManager.h"

#include "NetworkSubsystem.generated.h"


UCLASS(MinimalAPI)
class UNetworkSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:

	/** Access the NetConditionGroupManager */
	UE::Net::FNetConditionGroupManager& GetNetConditionGroupManager() { return GroupsManager;  }

	/** Const access the NetConditionGroupManager */
	const UE::Net::FNetConditionGroupManager& GetNetConditionGroupManager() const { return GroupsManager; }

protected:
	
	/** Allow the subsystem on every world type. It's essentially free until it's used but accessed by any world that does netdriver replication. */
	virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override { return true; }

private:

	ENGINE_API virtual void Serialize(FArchive& Ar) override;

	/** Manage the of the subobjects and their relationships to different groups */
	UE::Net::FNetConditionGroupManager GroupsManager;
};
