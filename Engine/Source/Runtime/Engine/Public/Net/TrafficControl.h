// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "CoreMinimal.h"
#include "Net/Util/SequenceNumber.h"
#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "EngineLogs.h"


struct FAckSample;
struct FSeqSample;
class FNetworkTrafficAnalyzer;
class FNetworkCongestionControl;

struct FSeqSample
{
	double Timestamp;
	int32 Seq;
	int32 PacketSize;
};

struct FAckSample
{
	double Timestamp;
	int32 Ack;
};

/**
 * A Network traffic analyzer that keeps track of useful information
 * including the RTT and uplink bandwidth of the latest received packet
 * to help the congestion control module work. It uses seq and
 * ack mechanism to track packet and use timestamp and packet size to
 * calculate RTT and bandwidth.
 */
class FNetworkTrafficAnalyzer
{
public:
	ENGINE_API FNetworkTrafficAnalyzer(FNetworkCongestionControl* InTrafficControlModule);

	ENGINE_API void OnAck(const FAckSample& AckSample);
	ENGINE_API void OnSend(const FSeqSample& SeqSample);

	double GetLatestRTT() const { return LatestRTT; }
	uint32 GetBytesInFlight() const { return BytesInFlight; }
	uint32 GetPacketsInFlight() const { return (uint32)OutPacketRecords.Count(); }

private:
	FNetworkCongestionControl* TrafficControlModule;

	/**
	 * Circular queue that keeps track of useful data of all unacked packets.
	 * Invariant:
	 * The tail of the circular queue is the first unacked packet
	 * whose packet id is OutAckedSeq + 1. The head of the circular
	 * queue is the latest sent packet whose id is equal to OutSeq.
	 */
	TResizableCircularQueue<FSeqSample> OutPacketRecords;

	using SeqT = TSequenceNumber<8, uint8>;

	double LatestUplinkBandwidth;
	double LatestRTT;
	SeqT OutSeq;					// The seq number of last sent packet.
	SeqT OutAckedSeq;				// The seq number of last acked packet.
	// Number of unacked bytes which is the sum of all packet size
	// in the OutPacketRecords.
	uint32 BytesInFlight;
	bool bInitialized;				// Whether the first packet has been sent.
};

/**
 *  One implementation of network traffic control based on the unacked bytes in flight
 *  and the estimated uplink bandwidth and propagation round-trip time. The idea is to
 *  keep the bytes in flight under the BDP(bandwidth round trip time product).
 */
class FNetworkCongestionControl
{
public:
	ENGINE_API FNetworkCongestionControl(double ConfiguredNetSpeed, uint32 MaxPackets);

	ENGINE_API void OnAck(const FAckSample& AckSample);
	ENGINE_API bool IsReadyToSend(double Timestamp) const;
	ENGINE_API void OnSend(const FSeqSample& SeqSample);

private:
	FNetworkTrafficAnalyzer Analyzer;

	double MinRTT;									// Estimated propagation round-trip time in s.
	double NetSpeed;								// Configured uplink bandwidth in bytes/sec.
	uint32 MaxPacketsAllowedInFlight;				// Max allowed packets in flight, typically the HistorySize of TSequenceHistory.

private:
	void UpdateMinRTT(double Timestamp);
	double GetMinRTT() const;
	double GetCongestionWindow() const;
};
