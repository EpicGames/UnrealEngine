// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealNetwork.h: Unreal networking.
=============================================================================*/

/*-----------------------------------------------------------------------------
	Forward declarations.
-----------------------------------------------------------------------------*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/CoreNet.h"
#include "EngineLogs.h"
#include "UObject/UnrealType.h"
#include "Engine/EngineBaseTypes.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/Core/PropertyConditions/PropertyConditions.h"
#include "Net/ReplayResult.h"
#include <type_traits>

class AActor;

/*class	UNetDriver;*/
class	UNetConnection;
class	UPendingNetGame;
struct  FOverridableReplayVersionData;

/*-----------------------------------------------------------------------------
	Types.
-----------------------------------------------------------------------------*/

// Return the value of Max/2 <= Value-Reference+some_integer*Max < Max/2.
inline int32 BestSignedDifference( int32 Value, int32 Reference, int32 Max )
{
	return ((Value-Reference+Max/2) & (Max-1)) - Max/2;
}
inline int32 MakeRelative( int32 Value, int32 Reference, int32 Max )
{
	return Reference + BestSignedDifference(Value,Reference,Max);
}

UE_DEPRECATED(5.1, "No longer used.")
DECLARE_MULTICAST_DELEGATE_OneParam(FPreActorDestroyReplayScrub, AActor*);

/** Fired at the start of replay checkpoint loading */
DECLARE_MULTICAST_DELEGATE_OneParam(FPreReplayScrub, UWorld*);

/** Fired mid-checkpoint load after the previous game state is torn down, and prior to any GC */
DECLARE_MULTICAST_DELEGATE_OneParam(FReplayScrubTeardown, UWorld*);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnWriteGameSpecificDemoHeader, TArray<FString>& /*GameSpecificData*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnProcessGameSpecificDemoHeader, const TArray<FString>& /*GameSpecificData*/, FString& /*Error*/);

typedef TMap<FString, TArray<uint8>> FDemoFrameDataMap;
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWriteGameSpecificFrameData, UWorld* /*World*/, float /*FrameTime*/, FDemoFrameDataMap& /*Data*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnProcessGameSpecificFrameData, UWorld* /*World*/, float /*FrameTime*/, const FDemoFrameDataMap& /*Data*/);

// Games can use these to override Version Data of Replay Headers
DECLARE_DELEGATE_OneParam(FGetOverridableVersionDataForDemoHeaderReadDelegate, FOverridableReplayVersionData& /*OveridableReplayVersionData*/);
DECLARE_DELEGATE_OneParam(FGetOverridableVersionDataForDemoHeaderWriteDelegate, FOverridableReplayVersionData& /*OveridableReplayVersionData*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplayStartedDelegate, UWorld* /*World*/);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.1, "Deprecated in favor of FOnReplayPlaybackFailureDelegate")
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReplayStartFailureDelegate, UWorld* /*World*/, EDemoPlayFailure::Type /*Error*/);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReplayPlaybackFailureDelegate, UWorld* /*World*/, const UE::Net::TNetResult<EReplayResult>& /*Error*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplayScrubCompleteDelegate, UWorld* /*World*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplayPlaybackCompleteDelegate, UWorld* /*World*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplayRecordingStartAttemptDelegate, UWorld* /*World*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnReplayRecordingCompleteDelegate, UWorld* /*World*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPauseChannelsChangedDelegate, UWorld* /*World*/, bool /*bPaused*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReplayIDChangedDelegate, UWorld* /*World*/, const FString& /*ReplayID*/);

struct FNetworkReplayDelegates
{
	/** global delegate called one time prior to scrubbing */
	static ENGINE_API FPreReplayScrub OnPreScrub;
	static ENGINE_API FReplayScrubTeardown OnScrubTeardown;

	/** Game specific demo headers */
	static ENGINE_API FOnWriteGameSpecificDemoHeader OnWriteGameSpecificDemoHeader;
	static ENGINE_API FOnProcessGameSpecificDemoHeader OnProcessGameSpecificDemoHeader;

	/** Game specific per frame data */
	static ENGINE_API FOnWriteGameSpecificFrameData OnWriteGameSpecificFrameData;
	static ENGINE_API FOnProcessGameSpecificFrameData OnProcessGameSpecificFrameData;

	/** Game Overridable Header Version Data */
	static ENGINE_API FGetOverridableVersionDataForDemoHeaderReadDelegate  GetOverridableVersionDataForHeaderRead;
	static ENGINE_API FGetOverridableVersionDataForDemoHeaderWriteDelegate GetOverridableVersionDataForHeaderWrite;

	/** Public delegate for external systems to be notified when a replay begins */
	static ENGINE_API FOnReplayStartedDelegate OnReplayStarted;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Public delegate to be notified when a replay failed to start */
	UE_DEPRECATED(5.1, "Deprecated in favor of OnReplayPlaybackFailure")
	static ENGINE_API FOnReplayStartFailureDelegate OnReplayStartFailure;
	ENGINE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Public delegate to be notified when replay playback failed */
	static FOnReplayPlaybackFailureDelegate OnReplayPlaybackFailure;

	/** Public delegate for external systems to be notified when scrubbing is complete. Only called for successful scrub. */
	static ENGINE_API FOnReplayScrubCompleteDelegate OnReplayScrubComplete;

	/** Delegate for external systems to be notified when playback ends */
	static ENGINE_API FOnReplayPlaybackCompleteDelegate OnReplayPlaybackComplete;

	/** Public Delegate for external systems to be notified when replay recording is about to start. */
	static ENGINE_API FOnReplayRecordingStartAttemptDelegate OnReplayRecordingStartAttempt;

	/** Public Delegate for external systems to be notified when replay recording is about to finish. */
	static ENGINE_API FOnReplayRecordingCompleteDelegate OnReplayRecordingComplete;

	/** Delegate for external systems to be notified when channels are paused during playback, usually waiting for data to be available. */
	static ENGINE_API FOnPauseChannelsChangedDelegate OnPauseChannelsChanged;

	/** Delegate for external systems to be notified when the SessionName has changed (The replay identifier). */
	static ENGINE_API FOnReplayIDChangedDelegate OnReplayIDChanged;
};

/**
 * Struct containing various parameters that can be passed to DOREPLIFETIME_WITH_PARAMS to control
 * how variables are replicated.
 */
struct FDoRepLifetimeParams
{
	/** Replication Condition. The property will only be replicated to connections where this condition is met. */
	ELifetimeCondition Condition = COND_None;
	
	/**
	 * RepNotify Condition. The property will only trigger a RepNotify if this condition is met, and has been
	 * properly set up to handle RepNotifies.
	 */
	ELifetimeRepNotifyCondition RepNotifyCondition = REPNOTIFY_OnChanged;
	
	/** Whether or not this property uses Push Model. See PushModel.h */
	bool bIsPushBased = false;

	/** Function to create and register a ReplicationFragment for the property */
	UE::Net::CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;

};

namespace NetworkingPrivate
{
	struct FRepPropertyDescriptor
	{
		UE_DEPRECATED(5.2, "No longer used")
		FRepPropertyDescriptor(const FProperty* Property)
			: PropertyName(TEXT(""))
			, RepIndex(Property->RepIndex)
			, ArrayDim(Property->ArrayDim)
		{
		}

		FRepPropertyDescriptor(const TCHAR* InPropertyName, const int32 InRepIndex, const int32 InArrayDim)
			: PropertyName(InPropertyName)
			, RepIndex(InRepIndex)
			, ArrayDim(InArrayDim)
		{
		}

		const TCHAR* PropertyName;
		const int32 RepIndex;
		const int32 ArrayDim;

	private:
		UE_NONCOPYABLE(FRepPropertyDescriptor);

		void* operator new(size_t) = delete;
		void* operator new[](size_t) = delete;
		void operator delete(void*) = delete;
		void operator delete[](void*) = delete;
	};

	struct FRepClassDescriptor
	{
		FRepClassDescriptor(const TCHAR* InClassName, const int32 InStartRepIndex, const int32 InEndRepIndex)
			: ClassName(InClassName)
			, StartRepIndex(InStartRepIndex)
			, EndRepIndex(InEndRepIndex)
		{
		}

		const TCHAR* ClassName;
		const int32 StartRepIndex;
		const int32 EndRepIndex;

	private:

		UE_NONCOPYABLE(FRepClassDescriptor);

		void* operator new(size_t) = delete;
		void* operator new[](size_t) = delete;
		void operator delete(void*) = delete;
		void operator delete[](void*) = delete;
	};
}
/*-----------------------------------------------------------------------------
	Replication.
-----------------------------------------------------------------------------*/

template<class BaseClass, class DerivedClass>
constexpr bool ValidateReplicatedClassInheritance()
{
	return std::is_base_of_v<BaseClass, DerivedClass>;
}

/** wrapper to find replicated properties that also makes sure they're valid */
inline FProperty* GetReplicatedProperty(const UClass* CallingClass, const UClass* PropClass, const FName& PropName)
{
	FProperty* TheProperty = FindFieldChecked<FProperty>(PropClass, PropName);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!(TheProperty->PropertyFlags & CPF_Net))
	{
		UE_LOGF(LogNet, Fatal,"Attempt to replicate property '%ls' that was not tagged to replicate! Please use 'Replicated' or 'ReplicatedUsing' keyword in the UPROPERTY() declaration.", *TheProperty->GetFullName());
	}
#endif
	return TheProperty;
}

#define DOREPLIFETIME_WITH_PARAMS_FAST(c,v,params) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v, 1); \
\
	PRAGMA_DISABLE_DEPRECATION_WARNINGS \
	RegisterReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, OutLifetimeProps, FixupParams<decltype(c::v)>(params)); \
	PRAGMA_ENABLE_DEPRECATION_WARNINGS \
}

#define DOREPLIFETIME_WITH_PARAMS_FAST_STATIC_ARRAY(c,v,params) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v##_STATIC_ARRAY, (int32)c::EArrayDims_Private::v); \
	RegisterReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, OutLifetimeProps, params); \
}

#define DOREPLIFETIME_WITH_PARAMS(c,v,params) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	FProperty* ReplicatedProperty = GetReplicatedProperty(StaticClass(), c::StaticClass(), GET_MEMBER_NAME_CHECKED(c,v)); \
	PRAGMA_DISABLE_DEPRECATION_WARNINGS \
	RegisterReplicatedLifetimeProperty(ReplicatedProperty, OutLifetimeProps, FixupParams<decltype(c::v)>(params)); \
	PRAGMA_ENABLE_DEPRECATION_WARNINGS \
}

#define DOREPLIFETIME(c,v) DOREPLIFETIME_WITH_PARAMS(c,v,FDoRepLifetimeParams())

/** This macro is used by nativized code (DynamicClasses), so the Property may be recreated. */
#define DOREPLIFETIME_DIFFNAMES(c,v, n) \
{ \
	static TWeakFieldPtr<FProperty> __swp##v{};							\
	const FProperty* sp##v = __swp##v.Get();								\
	if (nullptr == sp##v)													\
	{																		\
		sp##v = GetReplicatedProperty(StaticClass(), c::StaticClass(), n);	\
		__swp##v = sp##v;													\
	}																		\
	for ( int32 i = 0; i < sp##v->ArrayDim; i++ )							\
	{																		\
		OutLifetimeProps.AddUnique( FLifetimeProperty( sp##v->RepIndex + i ) );	\
	}																		\
}

#define DOREPLIFETIME_CONDITION(c,v,cond) \
{ \
	static_assert(cond != COND_NetGroup, "COND_NetGroup cannot be used on replicated properties. Only when registering subobjects"); \
	FDoRepLifetimeParams LocalDoRepParams; \
	LocalDoRepParams.Condition = cond; \
	DOREPLIFETIME_WITH_PARAMS(c,v,LocalDoRepParams); \
}

/** Allows gamecode to specify RepNotify condition: REPNOTIFY_OnChanged (default) or REPNOTIFY_Always for when repnotify function is called  */
#define DOREPLIFETIME_CONDITION_NOTIFY(c,v,cond,rncond) \
{ \
	static_assert(cond != COND_NetGroup, "COND_NetGroup cannot be used on replicated properties. Only when registering subobjects"); \
	FDoRepLifetimeParams LocalDoRepParams; \
	LocalDoRepParams.Condition = cond; \
	LocalDoRepParams.RepNotifyCondition = rncond; \
	DOREPLIFETIME_WITH_PARAMS(c,v,LocalDoRepParams); \
}

#define DOREPLIFETIME_ACTIVE_OVERRIDE_FAST(c,v,active) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	UE::Net::Private::FNetPropertyConditionManager::Get().SetPropertyActiveOverride(this, (int32)c::ENetFields_Private::v, active); \
}

#define DOREPLIFETIME_ACTIVE_OVERRIDE_FAST_STATIC_ARRAY(c,v,active) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	UE::Net::Private::FNetPropertyConditionManager& PropertyConditionManager = UE::Net::Private::FNetPropertyConditionManager::Get(); \
	for (int32 i = 0; i < (int32)c::EArrayDims_Private::v; ++i) \
	{ \
		PropertyConditionManager.SetPropertyActiveOverride(this, (int32)c::ENetFields_Private::v##_STATIC_ARRAY + i, active); \
	} \
}

#define DOREPLIFETIME_ACTIVE_OVERRIDE(c,v,active) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	static FProperty* sp##v = GetReplicatedProperty(StaticClass(), c::StaticClass(),GET_MEMBER_NAME_CHECKED(c,v)); \
	UE::Net::Private::FNetPropertyConditionManager& PropertyConditionManager = UE::Net::Private::FNetPropertyConditionManager::Get(); \
	for (int32 i = 0; i < sp##v->ArrayDim; i++) \
	{ \
		PropertyConditionManager.SetPropertyActiveOverride(this, sp##v->RepIndex + i, active); \
	} \
}

#define DOREPCUSTOMCONDITION_ACTIVE_FAST(c,v,active) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	OutActiveState.SetActiveState((uint16)c::ENetFields_Private::v, active); \
}

#define DOREPCUSTOMCONDITION_SETACTIVE_FAST(c,v,active) \
{ \
	UE::Net::Private::FNetPropertyConditionManager::Get().SetPropertyActive(this, (uint16)c::ENetFields_Private::v, active); \
}

#define DOREPCUSTOMCONDITION_SETACTIVE_FAST_STATIC_ARRAY(c,v,active) \
{ \
	UE::Net::Private::FNetPropertyConditionManager& PropertyConditionManager = UE::Net::Private::FNetPropertyConditionManager::Get(); \
	for (int32 i = 0; i < (int32)c::EArrayDims_Private::v; ++i) \
	{ \
		PropertyConditionManager.SetPropertyActive(this, (uint16)c::ENetFields_Private::v##_STATIC_ARRAY + i, active); \
	} \
}

/** To be used in a UObject's GetReplicatedCustomConditionState for example. */
#define DOREPDYNAMICCONDITION_INITCONDITION_FAST(c,v,cond) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	OutActiveState.SetDynamicCondition((uint16)c::ENetFields_Private::v, cond); \
}

/**
 * To be used when a condition is to be changed. This should be called as soon as the condition changes, not during PreReplication.
 * @note Changing the condition of a custom delta property, such as a FastArraySerializer, after it has replicated can result in unexpected behavior.
 */
#define DOREPDYNAMICCONDITION_SETCONDITION_FAST(c,v,cond) \
{ \
	UE::Net::Private::FNetPropertyConditionManager::Get().SetPropertyDynamicCondition(this, (uint16)c::ENetFields_Private::v, cond); \
}



ENGINE_API void RegisterReplicatedLifetimeProperty(
	const FProperty* ReplicatedProperty,
	TArray<FLifetimeProperty>& OutLifetimeProps,
	const FDoRepLifetimeParams& Params);

ENGINE_API void RegisterReplicatedLifetimeProperty(
	const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor,
	TArray<FLifetimeProperty>& OutLifetimeProps,
	const FDoRepLifetimeParams& Params);

/*-----------------------------------------------------------------------------
	Capture additional parameters required to bind FastArrays when using Iris
-----------------------------------------------------------------------------*/
struct CGetFastArrayCreateReplicationFragmentFuncable
{
	template <typename T, typename...>
	auto Requires(T* FastArray) -> decltype(FastArray->GetFastArrayCreateReplicationFragmentFunction());
};

template<typename T>
inline typename TEnableIf<TModels_V<CGetFastArrayCreateReplicationFragmentFuncable, T>, const FDoRepLifetimeParams>::Type FixupParams(const FDoRepLifetimeParams& Params)
{
	// Use passed in CreateAndRegisterReplicationFragmentFunction if set
	if (Params.CreateAndRegisterReplicationFragmentFunction)
	{
		return Params;
	}

	FDoRepLifetimeParams NewParams(Params);
	NewParams.CreateAndRegisterReplicationFragmentFunction = T::GetFastArrayCreateReplicationFragmentFunction();
	return NewParams;
}

template<typename T>
inline typename TEnableIf<!TModels_V<CGetFastArrayCreateReplicationFragmentFuncable, T>, const FDoRepLifetimeParams&>::Type FixupParams(const FDoRepLifetimeParams& Params)
{
	return Params;
}

/*-----------------------------------------------------------------------------
	Disable macros.
	Use these macros to state that properties tagged replicated 
	are voluntarily not replicated. This silences an error about missing
	registered properties when class replication is started
-----------------------------------------------------------------------------*/

/** Use this macro in GetLifetimeReplicatedProps to flag a replicated property as not-replicated */
#define DISABLE_REPLICATED_PROPERTY(c,v) \
DisableReplicatedLifetimeProperty(StaticClass(), c::StaticClass(), GET_MEMBER_NAME_CHECKED(c,v), OutLifetimeProps);

/** Use this macro to disable inherited properties that are private. Be careful since it removes a compile-time error when the variable doesn't exist */
#define DISABLE_REPLICATED_PRIVATE_PROPERTY(c,v) \
DisableReplicatedLifetimeProperty(StaticClass(), c::StaticClass(), FName(TEXT(#v)), OutLifetimeProps);

#define DISABLE_REPLICATED_PROPERTY_FAST(c,v) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v, 1); \
	DisableReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, OutLifetimeProps); \
}

#define DISABLE_REPLICATED_PROPERTY_FAST_STATIC_ARRAY(c,v) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v##_STATIC_ARRAY, (int32)c::EArrayDims_Private::v); \
	DisableReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, OutLifetimeProps); \
}

/** Use this macro in GetLifetimeReplicatedProps to flag all replicated properties of a class as not-replicated.
    Use the EFieldIteratorFlags enum to disable all inherited properties or only those of the class specified
*/
#define DISABLE_ALL_CLASS_REPLICATED_PROPERTIES(c, SuperClassBehavior) \
DisableAllReplicatedPropertiesOfClass(StaticClass(), c::StaticClass(), SuperClassBehavior, OutLifetimeProps);

#define DISABLE_ALL_CLASS_REPLICATED_PROPERTIES_FAST(c, SuperClassBehavior) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), "DISABLE_ALL_CLASS_REPLICATED_PROPERTIES"); \
	const TCHAR* DoRepClassName_##c(TEXT(#c)); \
	const NetworkingPrivate::FRepClassDescriptor ClassDescriptor_##c(DoRepClassName_##c, (int32)c::ENetFields_Private::NETFIELD_REP_START, (int32)c::ENetFields_Private::NETFIELD_REP_END); \
	DisableAllReplicatedPropertiesOfClass(ClassDescriptor_##c, SuperClassBehavior, OutLifetimeProps); \
}	

ENGINE_API void DisableReplicatedLifetimeProperty(const UClass* ThisClass, const UClass* PropertyClass, FName PropertyName, TArray< FLifetimeProperty >& OutLifetimeProps);
ENGINE_API void DisableAllReplicatedPropertiesOfClass(const UClass* ThisClass, const UClass* ClassToDisable, EFieldIteratorFlags::SuperClassFlags SuperClassBehavior, TArray< FLifetimeProperty >& OutLifetimeProps);

ENGINE_API void DisableReplicatedLifetimeProperty(const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor, TArray<FLifetimeProperty>& OutLifetimeProps);
ENGINE_API void DisableAllReplicatedPropertiesOfClass(const NetworkingPrivate::FRepClassDescriptor& ClassDescriptor, EFieldIteratorFlags::SuperClassFlags SuperClassBehavior, TArray<FLifetimeProperty>& OutLifetimeProps);

/*-----------------------------------------------------------------------------
	Reset macros.
	Use these to change the replication settings of an inherited property
-----------------------------------------------------------------------------*/

#define RESET_REPLIFETIME_CONDITION(c,v,cond)  ResetReplicatedLifetimeProperty(StaticClass(), c::StaticClass(), GET_MEMBER_NAME_CHECKED(c,v), cond, OutLifetimeProps);

#define RESET_REPLIFETIME(c,v) RESET_REPLIFETIME_CONDITION(c, v, COND_None)

#define RESET_REPLIFETIME_CONDITION_FAST(c,v,cond) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v, 1); \
	ResetReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, cond, OutLifetimeProps); \
}

#define RESET_REPLIFETIME_CONDITION_FAST_STATIC_ARRAY(c,v,cond) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v##_STATIC_ARRAY, (int32)c::EArrayDims_Private::v); \
	ResetReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, cond, OutLifetimeProps); \
}

#define RESET_REPLIFETIME_FAST(c,v) RESET_REPLIFETIME_CONDITION_FAST(c, v, COND_None)
#define RESET_REPLIFETIME_FAST_STATIC_ARRAY(c,v) RESET_REPLIFETIME_FAST_STATIC_ARRAY(c, v, COND_None)

#define RESET_REPLIFETIME_WITH_PARAMS(c,v,params)  ResetReplicatedLifetimeProperty(StaticClass(), c::StaticClass(), GET_MEMBER_NAME_CHECKED(c,v), params, OutLifetimeProps);

#define RESET_REPLIFETIME_FAST_WITH_PARAMS(c,v,params) \
{ \
	static_assert(ValidateReplicatedClassInheritance<c, ThisClass>(), #c "." #v " is not accessible from this class."); \
	const TCHAR* DoRepPropertyName_##c_##v(TEXT(#v)); \
	const NetworkingPrivate::FRepPropertyDescriptor PropertyDescriptor_##c_##v(DoRepPropertyName_##c_##v, (int32)c::ENetFields_Private::v, 1); \
	ResetReplicatedLifetimeProperty(PropertyDescriptor_##c_##v, params, OutLifetimeProps); \
}

ENGINE_API void ResetReplicatedLifetimeProperty(
	const UClass* ThisClass,
	const UClass* PropertyClass,
	FName PropertyName,
	ELifetimeCondition LifetimeCondition,
	TArray< FLifetimeProperty >& OutLifetimeProps);

ENGINE_API void ResetReplicatedLifetimeProperty(
	const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor,
	ELifetimeCondition LifetimeCondition,
	TArray<FLifetimeProperty>& OutLifetimeProps);

ENGINE_API void ResetReplicatedLifetimeProperty(
	const UClass* ThisClass,
	const UClass* PropertyClass,
	FName PropertyName,
	const FDoRepLifetimeParams& Params,
	TArray< FLifetimeProperty >& OutLifetimeProps);

ENGINE_API void ResetReplicatedLifetimeProperty(
	const NetworkingPrivate::FRepPropertyDescriptor& PropertyDescriptor,
	const FDoRepLifetimeParams& Params,
	TArray< FLifetimeProperty >& OutLifetimeProps);


/*-----------------------------------------------------------------------------
	RPC Parameter Validation Helpers
-----------------------------------------------------------------------------*/

// This macro is for RPC parameter validation.
// It handles the details of what should happen if a validation expression fails
#define RPC_VALIDATE( expression )			\
	if ( !( expression ) )					\
	{										\
		UE_LOG( LogNet, Warning,			\
		TEXT("RPC_VALIDATE Failed: ")		\
		TEXT( UE_STRINGIZE( expression ) )	\
		TEXT(" File: ")						\
		TEXT( UE_STRINGIZE( __FILE__ ) )	\
		TEXT(" Line: ")						\
		TEXT( UE_STRINGIZE( __LINE__ ) ) );	\
		return false;						\
	}
