// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/IsSigned.h"
#include "Serialization/BitWriter.h"
#include "Serialization/BitReader.h"

/** Util class to manage history of received sequence numbers */
template <SIZE_T HistorySize>
class TSequenceHistory
{
public:	
	typedef uint32 WordT;
	
	static constexpr SIZE_T BitsPerWord = sizeof(WordT) * 8;
	static constexpr SIZE_T WordCount = HistorySize / BitsPerWord;
	static constexpr SIZE_T MaxSizeInBits = WordCount * BitsPerWord;
	static constexpr SIZE_T Size = HistorySize;

	static_assert(HistorySize > 0, "HistorySize must be > 0");
	static_assert(HistorySize % BitsPerWord == 0, "InMaxHistorySize must be a modulo of the wordsize");

public:	
	TSequenceHistory();

#if WITH_AUTOMATION_WORKER
	explicit TSequenceHistory(WordT Value, SIZE_T Count = 1);
	WordT* Data() { return &Storage[0]; }
	const WordT* Data() const { return &Storage[0]; }
#endif

	/** Reset */
	void Reset();

	/** Store delivery status, oldest will be dropped */
	void AddDeliveryStatus(bool Delivered);

	/** Query the status of a specific index, index 0 is last stored status */
	bool IsDelivered(SIZE_T Index) const;
	
	bool operator==(const TSequenceHistory& Other) const { return FMemory::Memcmp(Storage, Other.Storage, WordCount * sizeof (WordT)) == 0; }

	bool operator!=(const TSequenceHistory& Other) const { return FMemory::Memcmp(Storage, Other.Storage, WordCount * sizeof (WordT)) != 0; }

	/** Write history to BitStream */
	void Write(FBitWriter& Writer, SIZE_T NumWords) const;

	/** Read history from BitStream */
	void Read(FBitReader& Reader, SIZE_T NumWords);

private:
	WordT Storage[WordCount];
};

template<SIZE_T HistorySize>
constexpr SIZE_T TSequenceHistory<HistorySize>::BitsPerWord;

template<SIZE_T HistorySize>
constexpr SIZE_T TSequenceHistory<HistorySize>::WordCount;

template<SIZE_T HistorySize>
constexpr SIZE_T TSequenceHistory<HistorySize>::MaxSizeInBits;

template<SIZE_T HistorySize>
constexpr SIZE_T TSequenceHistory<HistorySize>::Size;

template <SIZE_T HistorySize>
TSequenceHistory<HistorySize>::TSequenceHistory()
{
	Reset();
}

#if WITH_AUTOMATION_WORKER
template <SIZE_T HistorySize>
TSequenceHistory<HistorySize>::TSequenceHistory(WordT Value, SIZE_T Count)
{
	Reset();
	for (SIZE_T CurrentWordIt = 0; CurrentWordIt < Count; ++CurrentWordIt)
	{
		Storage[CurrentWordIt] = Value;
	}	
}
#endif 

template <SIZE_T HistorySize>
void TSequenceHistory<HistorySize>::Reset()
{
	FPlatformMemory::Memset(&Storage[0], 0, WordCount * sizeof(WordT));
}

template <SIZE_T HistorySize>
void TSequenceHistory<HistorySize>::AddDeliveryStatus(bool Delivered)
{
	WordT Carry = Delivered ? 1u : 0u;
	const WordT ValueMask = 1u << (BitsPerWord - 1);
	
	for (SIZE_T CurrentWordIt = 0; CurrentWordIt < WordCount; ++CurrentWordIt)
	{
		const WordT OldValue = Carry;
		
		// carry over highest bit in each word to the next word
		Carry = (Storage[CurrentWordIt] & ValueMask) >> (BitsPerWord - 1);
		Storage[CurrentWordIt] = (Storage[CurrentWordIt] << 1u) | OldValue;
	}
}

template <SIZE_T HistorySize>
bool TSequenceHistory<HistorySize>::IsDelivered(SIZE_T Index) const
{
	check(Index < Size);

	const SIZE_T WordIndex = Index / BitsPerWord;
	const WordT WordMask = (WordT(1) << (Index & (BitsPerWord - 1)));
	
	return (Storage[WordIndex] & WordMask) != 0u;
}

template <SIZE_T HistorySize>
void TSequenceHistory<HistorySize>::Write(FBitWriter& Writer, SIZE_T NumWords) const
{
	NumWords = FPlatformMath::Min(NumWords, WordCount);
	for (SIZE_T CurrentWordIt = 0; CurrentWordIt < NumWords; ++CurrentWordIt)
	{
		WordT temp = Storage[CurrentWordIt];
		Writer << temp;
	}
}

template <SIZE_T HistorySize>
void TSequenceHistory<HistorySize>::Read(FBitReader& Reader, SIZE_T NumWords)
{
	NumWords = FPlatformMath::Min(NumWords, WordCount);
	for (SIZE_T CurrentWordIt = 0; CurrentWordIt < NumWords; ++CurrentWordIt)
	{
		Reader << Storage[CurrentWordIt];
	}
}
