// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/IsSigned.h"

/** Helper class to work with sequence numbers */
template <SIZE_T NumBits, typename SequenceType>
class TSequenceNumber
{
	static_assert(TIsSigned<SequenceType>::Value == false, "The base type for sequence numbers must be unsigned");

public:
	using SequenceT = SequenceType;
	using DifferenceT = int32;

	// Constants
	enum { SeqNumberBits = NumBits };
	enum { SeqNumberCount = SequenceT(1) << NumBits };
	enum { SeqNumberHalf = SequenceT(1) << (NumBits - 1) };
	enum { SeqNumberMax = SeqNumberCount - 1u };
	enum { SeqNumberMask = SeqNumberMax };

	/** Default constructor */
	TSequenceNumber() : Value(0u) {}

	/** Constructor with given value */
	TSequenceNumber(SequenceT ValueIn) : Value(ValueIn & SeqNumberMask) {}
	
	/** Get Current Value */	
	SequenceT Get() const { return Value; }

	/** Diff between sequence numbers (A - B) only valid if (A - B) < SeqNumberHalf */
	static DifferenceT Diff(TSequenceNumber A, TSequenceNumber B);
	
	/** return true if this is > Other, this is only considered to be the case if (A - B) < SeqNumberHalf since we have to be able to detect wraparounds */
	bool operator>(const TSequenceNumber& Other) const { return (Value != Other.Value) && (((Value - Other.Value) & SeqNumberMask) < SeqNumberHalf); }

	/** Check if this is >= Other, See above */
	bool operator>=(const TSequenceNumber& Other) const { return ((Value - Other.Value) & SeqNumberMask) < SeqNumberHalf; }

	/** Pre-increment and wrap around */
	TSequenceNumber& operator++() { Increment(1u); return *this; }
	
	/** Post-increment and wrap around */
	TSequenceNumber operator++(int) { TSequenceNumber Tmp(*this); Increment(1u); return Tmp; }

private:
	void Increment(SequenceT InValue) { *this = TSequenceNumber(Value + InValue); }
	SequenceT Value;

	friend bool operator<(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{
		return !(Lhs >= Rhs);
	}

	friend bool operator<=(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{
		return !(Lhs > Rhs);
	}

	/** Equals, NOTE that sequence numbers wrap around so 0 == 0 + SequenceNumberCount */
	friend bool operator==(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{ 
		return Lhs.Get() == Rhs.Get(); 
	}

	friend bool operator!=(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{
		return Lhs.Get() != Rhs.Get();
	}

	friend const TSequenceNumber operator+(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{
		return TSequenceNumber(Lhs.Get() + Rhs.Get());
	}

	friend const TSequenceNumber operator-(const TSequenceNumber& Lhs, const TSequenceNumber& Rhs)
	{
		return TSequenceNumber(Lhs.Get() - Rhs.Get());
	}

	friend const TSequenceNumber operator+(const TSequenceNumber& Lhs, SequenceType Rhs)
	{
		return TSequenceNumber(Lhs.Get() + Rhs);
	}

	friend const TSequenceNumber operator-(const TSequenceNumber& Lhs, SequenceType Rhs)
	{
		return TSequenceNumber(Lhs.Get() - Rhs);
	}
};

template <SIZE_T NumBits, typename SequenceType>
typename TSequenceNumber<NumBits, SequenceType>::DifferenceT TSequenceNumber<NumBits, SequenceType>::Diff(TSequenceNumber A, TSequenceNumber B) 
{ 
	constexpr SIZE_T ShiftValue = sizeof(DifferenceT)*8 - NumBits;

	const SequenceT ValueA = A.Value;
	const SequenceT ValueB = B.Value;

	return (DifferenceT)((ValueA - ValueB) << ShiftValue) >> ShiftValue;
};
