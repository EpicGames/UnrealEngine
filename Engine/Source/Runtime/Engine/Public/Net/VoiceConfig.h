// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Misc/ConfigCacheIni.h"
#include "DSP/EnvelopeFollower.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundAttenuation.h"
#include "GameFramework/PlayerState.h"
#include "Components/SynthComponent.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VoiceConfig.generated.h"

#if PLATFORM_WINDOWS
#define USE_DEFAULT_VOICE_SAMPLE_RATE 0
#else
#define USE_DEFAULT_VOICE_SAMPLE_RATE 1
#endif

/** Value used to trigger an ensure warning used to tell if UVoipStatics:: */
#define VOICE_CONFIG_LEAK_WARNING_THRESHOLD 48


/** Encoding hints for compression */
enum class EAudioEncodeHint : uint8
{
	/** Best for most VoIP applications where listening quality and intelligibility matter most */
	VoiceEncode_Voice,
	/** Best for broadcast/high-fidelity application where the decoded audio should be as close as possible to the input */
	VoiceEncode_Audio
};

namespace MicSilenceDetectionConfig
{
	static const Audio::EPeakMode::Type LevelDetectionMode = Audio::EPeakMode::Type::Peak;
	static const bool IsAnalog = false;

	static const int32 PacketBufferSlack = 20;
}

class USoundAttenuation;

USTRUCT(BlueprintType)
struct FVoiceSettings
{
	GENERATED_BODY()

	// Optionally attach the voice source to a Scene Component to give the appearance
	// that the voice is coming from that scene component.
	// If this is not set, the voice will not be spatialized.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio|Voice|Spatialization")
	TObjectPtr<USceneComponent> ComponentToAttachTo;

	// Optional attenuation settings to attach to this player's voice.
	// This should only be used when ComponentToAttachTo is set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio|Voice|Spatialization")
	TObjectPtr<USoundAttenuation> AttenuationSettings;
	
	// Optional audio effects to apply to this player's voice.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio|Voice|Effects")
	TObjectPtr<USoundEffectSourcePresetChain> SourceEffectChain;

	FVoiceSettings()
		: ComponentToAttachTo(nullptr)
		, AttenuationSettings(nullptr)
		, SourceEffectChain(nullptr)
	{
	}
};


UCLASS(Blueprintable, BlueprintType, ClassGroup = VOIP, meta = (BlueprintSpawnableComponent, ShortTooltip = "A VOIPTalker is a component that can be used to control the audio characteristics of a player's voice."), MinimalAPI)
class UVOIPTalker : public UActorComponent
{
	GENERATED_BODY()

public:
	// Constructor and destructor.
	ENGINE_API UVOIPTalker(const FObjectInitializer& ObjectInitializer);
	ENGINE_API virtual ~UVOIPTalker();

	// function for creating and registering a UVOIPTalker.
	UFUNCTION(BlueprintCallable, Category = "Audio|Voice|Notification")
	static ENGINE_API UVOIPTalker* CreateTalkerForPlayer(APlayerState* OwningState);

	// This function sets up this talker with a specific player.
	// It is necessary to use this to properly control a specific player's voice
	// and receive events.
	UFUNCTION(BlueprintCallable, Category = "Audio|Voice|Notification")
	ENGINE_API void RegisterWithPlayerState(APlayerState* OwningState);
	
	// Get the current level of how loud this player is speaking. Will return 0.0
	// if player is not talking.
	UFUNCTION(BlueprintCallable, Category = "Audio|Voice|Notification")
	ENGINE_API float GetVoiceLevel();

	// Override this function to implement custom functionality when this player begins talking.
	virtual void OnTalkingBegin(UAudioComponent* AudioComponent) { BPOnTalkingBegin(AudioComponent); };

	// Override this function to implement custom functionality when this player stops talking.
	virtual void OnTalkingEnd() { BPOnTalkingEnd(); };

	// This is used by the VoiceEngineImpl to notify this VOIPTalker what the voice audio component's current level is.
	// This should not be called outside of the voice engine.
	ENGINE_API void OnAudioComponentEnvelopeValue(const UAudioComponent* InAudioComponent, const float EnvelopeValue);

	// Overridden to ensure that instances of UVOIPTalker unregister themselves from the static VoiceTalkerMap.
	ENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;

protected:
	// Blueprint native event for when this player starts speaking.
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "Begin Talking"), Category = "Audio|Voice|Notification")
	ENGINE_API void BPOnTalkingBegin(UAudioComponent* AudioComponent);

	// Blueprint native event for when this player stops speaking.
	UFUNCTION(BlueprintNativeEvent, meta = (DisplayName = "End Talking"), Category = "Audio|Voice|Notification")
	ENGINE_API void BPOnTalkingEnd();

private:
	// This function removes this player component from the VoiceTalkerMap.
	ENGINE_API void UnregisterFromVoiceTalkerMap();

public:
	// Configurable settings for this player's voice. When set, this will update the next time the player speaks.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Voice")
	FVoiceSettings Settings;

private:
	// Cached player ID from the APlayerController.
	FUniqueNetIdWrapper PlayerId;

	// This volume level is set by the Audio Component's OnEnvelopeValueSet delegate.
	float CachedVolumeLevel;

	// whether this component has been registered.
	uint8 bIsRegistered : 1;
};



UCLASS(MinimalAPI)
class UVOIPStatics : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Returns the voice sample rate specified in Audio Settings.
	static ENGINE_API int32 GetVoiceSampleRate();
	static ENGINE_API int32 GetVoiceNumChannels();

	// Returns the max voice data size per packet, uncompressed, and compressed based on the voice sample rate specified in Audio Settings.
	static ENGINE_API uint32 GetMaxVoiceDataSize();
	static ENGINE_API uint32 GetMaxUncompressedVoiceDataSizePerChannel();
	static ENGINE_API uint32 GetMaxCompressedVoiceDataSize();

	// Returns the amount of time that must pass since a VOIP talker has been updated before it's queued data is reset.
	static ENGINE_API float GetRemoteTalkerTimeoutDuration();

	// Returns the desired encoding type. Currently not exposed as a configured settings.
	static ENGINE_API EAudioEncodeHint GetAudioEncodingHint();

	// Returns the amount of seconds to spend buffering specified by AudioSettings.
	static ENGINE_API float GetBufferingDelay();

	// Returns the noise gate threshold we use for incoming audio.
	static ENGINE_API float GetVoiceNoiseGateLevel();

	// returns the amount of packets to allocate memory for based on the buffering delay.
	static ENGINE_API int32 GetNumBufferedPackets();

	// returns pointer to a player state
	static ENGINE_API APlayerState* GetPlayerStateFromUniqueNetId(UWorld* InWorld, const FUniqueNetIdWrapper& InPlayerId);

	// This function sets the Mic threshold for VOIP chat.
	UFUNCTION(BlueprintCallable, Category = "Audio|Voice|Mic")
	static ENGINE_API void SetMicThreshold(float InThreshold);

	// Sets up a VOIP talker to be accessed for a specific player when they are talking.
	static ENGINE_API void SetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InPlayerId, UVOIPTalker* InTalker);

	// These functions retrieve a pointer to a VOIPTalker associated with a specific player, if there is one available.
	static ENGINE_API UVOIPTalker* GetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InUniqueId,  FVoiceSettings& OutSettings, UWorld* InWorld = nullptr, APlayerState** OutPlayerState = nullptr);
	static ENGINE_API UVOIPTalker* GetVOIPTalkerForPlayer(const FUniqueNetIdWrapper& InPlayerId);

	// This is used to check whether a VOIPTalker has been removed from the TalkerMap before using it.
	static ENGINE_API bool IsVOIPTalkerStillAlive(UVOIPTalker* InTalker);
	
	// Removes a player's voice talker. This is done by a UVOIPTalker on destruction.
	static ENGINE_API void ResetPlayerVoiceTalker(APlayerState* InPlayerState);
	static ENGINE_API void ResetPlayerVoiceTalker(const FUniqueNetIdWrapper& InPlayerId);

	//Clear all pointers to VOIPTalkers.
	static ENGINE_API void ClearAllSettings();

private:

	static TMap<FUniqueNetIdWrapper, UVOIPTalker*> VoiceTalkerMap;

};
