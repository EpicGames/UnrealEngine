// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/CoreOnline.h"
#include "Net/VoiceConfig.h"
#ifndef MAX_VOICE_DATA_SIZE
	#define MAX_VOICE_DATA_SIZE 8*1024
#endif

#ifndef MAX_SPLITSCREEN_TALKERS
	#if PLATFORM_WINDOWS || PLATFORM_UNIX || PLATFORM_MAC
		#define MAX_SPLITSCREEN_TALKERS 1
	#else
		#define MAX_SPLITSCREEN_TALKERS 4
	#endif
#endif

#ifndef MAX_REMOTE_TALKERS
	#define MAX_REMOTE_TALKERS 16
#endif

/** Defines the data involved in a voice packet */
class FVoicePacket
{

public:
	/** Zeros members and validates the assumptions */
	FVoicePacket()
	{
	}

	/** Should only be used by TSharedPtr and FVoiceData */
	virtual ~FVoicePacket() = default;

	/**
	 * Copies another packet and inits the ref count
	 *
	 * @param Other packet to copy
	 * @param InRefCount the starting ref count to use
	 */
	FVoicePacket(const FVoicePacket& Other)
	{
	}

	/** @return the amount of space this packet will consume in a buffer */
	virtual uint16 GetTotalPacketSize() = 0;

	/** @return the amount of space used by the internal voice buffer */
	virtual uint16 GetBufferSize() = 0;

	/** @return the sender of this voice packet */
	virtual FUniqueNetIdPtr GetSender() = 0;

	/** @return true if this packet should be sent reliably */
	virtual bool IsReliable() = 0;

	/** 
	 * Serialize the voice packet data to a buffer 
	 *
	 * @param Ar buffer to write into
	 */
	virtual void Serialize(class FArchive& Ar) = 0;

	/** Return the index of the first sample of this packet in the audio stream. */
	virtual uint64 GetSampleCounter() const
	{
		return 0;
	}
};

/** Make the TArray of voice packets a bit more readable */
typedef TArray< TSharedPtr<FVoicePacket> > FVoicePacketList;
