// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// magic number distances used by AI/networking
#define CLOSEPROXIMITY					500.f
#define NEARSIGHTTHRESHOLD				2000.f
#define MEDSIGHTTHRESHOLD				3162.f
#define FARSIGHTTHRESHOLD				8000.f
#define CLOSEPROXIMITYSQUARED			(CLOSEPROXIMITY*CLOSEPROXIMITY)
#define NEARSIGHTTHRESHOLDSQUARED		(NEARSIGHTTHRESHOLD*NEARSIGHTTHRESHOLD)
#define MEDSIGHTTHRESHOLDSQUARED		(MEDSIGHTTHRESHOLD*MEDSIGHTTHRESHOLD)
#define FARSIGHTTHRESHOLDSQUARED		(FARSIGHTTHRESHOLD*FARSIGHTTHRESHOLD)
