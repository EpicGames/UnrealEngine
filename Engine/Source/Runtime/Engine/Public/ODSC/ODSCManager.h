// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "RHIDefinitions.h"
#include "Containers/Ticker.h"
#include "ShaderCompiler.h"

class FODSCThread;
class UMaterialInstance;
class FMaterialShaderMap;
class FMaterialShaderMapId;
class FPrimitiveSceneInfo;

/**
 * Responsible for processing shader compile responses from the ODSC Thread.
 * Interface for submitting shader compile requests to the ODSC Thread.
 */
class FODSCManager
	: public FTSTickerObjectBase
{
public:

	// FODSCManager

	/**
	 * Constructor
	 */
	ENGINE_API FODSCManager();

	/**
	 * Destructor
	 */
	ENGINE_API virtual ~FODSCManager();

	// FTSTickerObjectBase

	/**
	 * FTSTicker callback
	 *
	 * @param DeltaSeconds - time in seconds since the last tick
	 *
	 * @return false if no longer needs ticking
	 */
	ENGINE_API bool Tick(float DeltaSeconds) override;

	/**
	 * Add a request to compile a shader.  The results are submitted and processed in an async manner.
	 *
	 * @param MaterialsToCompile - List of material names to submit compiles for.
	 * @param ShaderPlatform - Which shader platform to compile for.
	 * @param RecompileCommandType - Whether we should recompile changed or global shaders.
	 *
	 * @return false if no longer needs ticking
	 */
	ENGINE_API void AddThreadedRequest(
		const TArray<FString>& MaterialsToCompile,
		const FString& ShaderTypesToLoad,
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		ODSCRecompileCommand RecompileCommandType,
		const FString& RequestedMaterialName = FString(),
		const FShaderCompilerFlags& ExtraCompilerFlags = FShaderCompilerFlags()
	);

	/**
	 * Add a request to compile a pipeline of shaders.  The results are submitted and processed in an async manner.
	 *
	 * @param ShaderPlatform - Which shader platform to compile for.
	 * @param FeatureLevel - Which feature level to compile for.
	 * @param QualityLevel - Which material quality level to compile for.
	 * @param MaterialName - The name of the material to compile.
	 * @param VertexFactoryName - The name of the vertex factory type we should compile.
	 * @param PipelineName - The name of the shader pipeline we should compile.
	 * @param ShaderTypeNames - The shader type names of all the shader stages in the pipeline.
	 * @param PermutationId - The permutation ID of the shader we should compile.
	 *
	 * @return false if no longer needs ticking
	 */
	ENGINE_API void AddThreadedShaderPipelineRequest(
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		const FMaterial* Material,
		const FString& VertexFactoryName,
		const FString& PipelineName,
		const TArray<FString>& ShaderTypeNames,
		int32 PermutationId,
		const TArray<FShaderId>& RequestShaderIds
	);

	UE_DEPRECATED(5.5, "RequestShaderIds is needed for AddThreadedShaderPipelineRequest and need to match the ones from FMaterialShaderMap::GetShaderList")
	void AddThreadedShaderPipelineRequest(
		EShaderPlatform ShaderPlatform,
		ERHIFeatureLevel::Type FeatureLevel,
		EMaterialQualityLevel::Type QualityLevel,
		const FString& MaterialName,
		const FString& VertexFactoryName,
		const FString& PipelineName,
		const TArray<FString>& ShaderTypeNames,
		int32 PermutationId
	) {}

	/** Returns true if we would actually add a request when calling AddThreadedShaderPipelineRequest. */
	inline bool IsHandlingRequests() const { return Thread != nullptr; }

	static void RegisterMaterialInstance(const UMaterialInstance* MI);
	static void UnregisterMaterialInstance(const UMaterialInstance* MI);

	/** Check options to see if ODSC could be active in this session */
	static ENGINE_API bool IsODSCEnabled();

	/** Check if it is currently running */
	static inline bool IsODSCActive();
	static inline bool ShouldForceRecompile(const FMaterialShaderMap* MaterialShaderMap, const FMaterial* Material);

	static void SuspendODSCForceRecompile();
	static void ResumeODSCForceRecompile();
	void TryLoadGlobalShaders(EShaderPlatform ShaderPlatform);

	static void ReportODSCError(const FString& InErrorMessage);
	static bool UseDefaultMaterialOnRecompile();

	bool CheckIfRequestAlreadySent(const TArray<FShaderId>& RequestShaderIds, const FMaterial* Material) const;

	static void UnregisterMaterialName(const FMaterial* Material);
	static void RegisterMaterialShaderMaps(const FString& MaterialName, const TArray<TRefCountPtr<FMaterialShaderMap>>& LoadedShaderMaps);
	static FMaterialShaderMap* FindMaterialShaderMap(const FString& MaterialName, const FMaterialShaderMapId& ShaderMapId);
	ENGINE_API static void SetCurrentPrimitiveSceneInfo(FPrimitiveSceneInfo* PrimitiveSceneInfo);
	ENGINE_API static void ResetCurrentPrimitiveSceneInfo();

private:
	friend class FODSCManagerAccess;

	ENGINE_API void OnEnginePreExit();
	ENGINE_API void StopThread();

	bool HasAsyncLoadingInstances();
	bool ShouldForceRecompileInternal(const FMaterialShaderMap* MaterialShaderMap, const FMaterial* Material);

	void RetrieveErrorMessage(FString& OutErrorMessage);
	void ClearErrorMessage();

	/** Handles communicating directly with the cook on the fly server. */
	FODSCThread* Thread = nullptr;

	FDelegateHandle OnScreenMessagesHandle;
	FCriticalSection MaterialInstancesCachedUniformExpressionsCS;
	TMap<const void*, TWeakObjectPtr<const UMaterialInstance> > MaterialInstancesCachedUniformExpressions;

	FCriticalSection ErrorMessageCS;
	FString ErrorMessage;

	FName MaterialNameToRecompile;
};

struct FODSCPrimitiveSceneInfoScope
{
	FODSCPrimitiveSceneInfoScope(FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		FODSCManager::SetCurrentPrimitiveSceneInfo(PrimitiveSceneInfo);
	}

	~FODSCPrimitiveSceneInfoScope()
	{
		FODSCManager::ResetCurrentPrimitiveSceneInfo();
	}
};

struct FODSCSuspendForceRecompileScope
{
	FODSCSuspendForceRecompileScope()
	{
		FODSCManager::SuspendODSCForceRecompile();
	}

	~FODSCSuspendForceRecompileScope()
	{
		FODSCManager::ResumeODSCForceRecompile();
	}
};

/** The global shader ODSC manager. */
extern ENGINE_API FODSCManager* GODSCManager;

inline bool FODSCManager::IsODSCActive()
{
	return GODSCManager && GODSCManager->IsHandlingRequests();
}

inline bool FODSCManager::ShouldForceRecompile(const FMaterialShaderMap* MaterialShaderMap, const FMaterial* Material)
{
	return FODSCManager::IsODSCActive() && GODSCManager->ShouldForceRecompileInternal(MaterialShaderMap, Material);
}
