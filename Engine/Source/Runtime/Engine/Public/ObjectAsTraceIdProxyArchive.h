// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/NameAsStringProxyArchive.h"

class UObject;

/**
 * Implements a proxy archive that serializes UObjects as Trace Ids with FObjectTrace
 * This requires that a trace is active and the Object trace channel is enabled.
 * This archive only supports writing, and not reading, since this archive needs to work in game runtime code, and ids can only be resolved into objects using editor code dependencies.
 * To read data serialized by this archive, use FObjectAsTraceIdProxyArchiveReader from the GameplayInsights extension
 *
 * Expected use is:
 *    FArchive* SomeAr = CreateAnAr();
 *    ObjectAsTraceIdProxyArchive Ar(*SomeAr);
 *    SomeObject->Serialize(Ar);
 *    FinalizeAr(SomeAr);
 * 
 * @param InInnerArchive The actual FArchive object to serialize normal data types (FStrings, INTs, etc)
 */

struct FObjectAsTraceIdProxyArchive : public FNameAsStringProxyArchive
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive - The inner archive to proxy.
	 */
	ENGINE_API FObjectAsTraceIdProxyArchive(FArchive& InInnerArchive);

	ENGINE_API virtual ~FObjectAsTraceIdProxyArchive() override;

	virtual FArchive& operator<<(UObject*& Obj) override;
	virtual FArchive& operator<<(FWeakObjectPtr& Obj) override;
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(FObjectPtr& Obj) override;

	ENGINE_API void Write(const UObject* Object);
};