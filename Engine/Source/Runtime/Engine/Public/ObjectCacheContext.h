// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "UObject/ObjectKey.h"
#include "Misc/ScopeRWLock.h"

class UObject;
class UPrimitiveComponent;
class UStaticMeshComponent;
class UMaterialInterface;
class UStaticMesh;
class UTexture;
class UTextureCollection;
class USkeletalMesh;
class USkinnedAsset;
class IPrimitiveComponent;
class IStaticMeshComponent;
class ISkinnedMeshComponent;
class UInstancedSkinnedMeshComponent;
class UTransformProviderData;

/**
 *  Class to abstract implementation details of the containers used inside the
 *  object cache so they can be changed later if needed without API changes.
 */
template< class T > class TObjectCacheIterator
{
public:
	/**
	 * Constructor
	 */
	explicit TObjectCacheIterator(TArray<T*>&& InObjectArray)
		: OwnedArray(MoveTemp(InObjectArray))
		, ArrayView(OwnedArray)
	{
	}

	/**
	 * Constructor
	 */
	explicit TObjectCacheIterator(TArrayView<T*> InArrayView)
		: ArrayView(InArrayView)
	{
	}

	inline const int32 IsEmpty() const { return ArrayView.IsEmpty(); }

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline T** begin() const { return ArrayView.begin(); }
	inline T** end() const { return ArrayView.end(); }

protected:
	/** Sometimes, a copy might be necessary to maintain thread-safety */
	TArray<T*> OwnedArray;
	/** The array view is used to do the iteration, the arrayview might point to another object if the array is not owned **/
	TArrayView<T*> ArrayView;
};

/** 
 *  Context containing a lazy initialized ObjectIterator cache along with some useful
 *  reverse lookup tables that can be used during heavy scene updates
 *  of async asset compilation.
 */
class FObjectCacheContext
{
public:
	ENGINE_API TObjectCacheIterator<IPrimitiveComponent>   GetPrimitiveComponents();
	ENGINE_API TObjectCacheIterator<IStaticMeshComponent>  GetStaticMeshComponents();
	ENGINE_API TObjectCacheIterator<ISkinnedMeshComponent> GetSkinnedMeshComponents();
	ENGINE_API TObjectCacheIterator<ISkinnedMeshComponent> GetSkinnedMeshComponents(USkinnedAsset* InSkinnedAsset);
	ENGINE_API TObjectCacheIterator<UInstancedSkinnedMeshComponent> GetInstancedSkinnedMeshComponents();
	ENGINE_API TObjectCacheIterator<UInstancedSkinnedMeshComponent> GetInstancedSkinnedMeshComponents(UTransformProviderData* InTransformProvider);
	ENGINE_API TObjectCacheIterator<IStaticMeshComponent>  GetStaticMeshComponents(UStaticMesh* InStaticMesh);
	ENGINE_API TObjectCacheIterator<UMaterialInterface>    GetMaterialsAffectedByTexture(UTexture* InTexture);
	ENGINE_API TObjectCacheIterator<UMaterialInterface>    GetMaterialsAffectedByTextureCollection(UTextureCollection* InTextureCollection);
	ENGINE_API TObjectCacheIterator<IPrimitiveComponent>   GetPrimitivesAffectedByMaterial(UMaterialInterface* InMaterial);
	ENGINE_API TObjectCacheIterator<IPrimitiveComponent>   GetPrimitivesAffectedByMaterials(TArrayView<UMaterialInterface*> InMaterials);
	ENGINE_API TObjectCacheIterator<UTexture>              GetUsedTextures(UMaterialInterface* InMaterial);
	ENGINE_API TObjectCacheIterator<UTextureCollection>    GetUsedTextureCollections(UMaterialInterface* InMaterial);
	ENGINE_API TObjectCacheIterator<UMaterialInterface>    GetUsedMaterials(IPrimitiveComponent* InComponent);	
	ENGINE_API TObjectCacheIterator<UMaterialInterface>    GetMaterialsAffectedByMaterials(TArrayView<UMaterialInterface*> InMaterials);
#if WITH_EDITOR
	ENGINE_API TObjectCacheIterator<UTexture>              GetTexturesAffectedByTexture(UTexture* InTexture);
#endif

private:
	friend class FObjectCacheContextScope;
	FObjectCacheContext() = default;
	TMap<IPrimitiveComponent*, TSet<UMaterialInterface*>> PrimitiveComponentToMaterial;
	TMap<TObjectKey<UMaterialInterface>, TSet<UTexture*>> MaterialUsedTextures;
	TMap<TObjectKey<UMaterialInterface>, TSet<UTextureCollection*>> MaterialUsedTextureCollections;
	TOptional<TMap<TObjectKey<UTexture>, TSet<UMaterialInterface*>>> TextureToMaterials;
	TOptional<TMap<TObjectKey<UTextureCollection>, TSet<UMaterialInterface*>>> TextureCollectionToMaterials;
#if WITH_EDITOR
	TOptional<TMap<TObjectKey<UTexture>, TSet<UTexture*>>> TextureToTextures;
#endif
	TOptional<TMap<TObjectKey<UMaterialInterface>, TSet<IPrimitiveComponent*>>> MaterialToPrimitives;

	template <typename UComponentType, typename UAssetType>
	struct TComponentTypeCache
	{
		TOptional<TArray<UComponentType*>> Components;
		TOptional<TMap<TObjectKey<UAssetType>, TSet<UComponentType*>>> AssetToComponents;

		template <typename ObjectIteratorType>
		TObjectCacheIterator<UComponentType> Get();

		template <typename ObjectIteratorType, typename GetNumFuncType, typename GetFuncType>
		TObjectCacheIterator<UComponentType> Get(UAssetType* AssetType, GetNumFuncType GetNumFunc, GetFuncType GetFunc);

		template <typename ObjectIteratorType, typename GetNumFuncType, typename GetFuncType>
		const TSet<UComponentType* >& GetSet(UAssetType* AssetType, GetNumFuncType GetNumFunc, GetFuncType GetFunc);
	};

	TComponentTypeCache<ISkinnedMeshComponent, USkinnedAsset> SkinnedMeshCache;
	TComponentTypeCache<UInstancedSkinnedMeshComponent, UTransformProviderData> InstancedSkinnedMeshCache;
	TOptional<TArray<IPrimitiveComponent*>> PrimitiveComponents;
	TComponentTypeCache<IStaticMeshComponent, UStaticMesh> StaticMeshCache;
};

/**
 * A scope that can be used to maintain a FObjectCacheContext active until the scope
 * is destroyed. Should only be used during short periods when there is no new
 * objects created and no object dependency changes. (i.e. Scene update after asset compilation).
 */
class FObjectCacheContextScope
{
public:
	ENGINE_API FObjectCacheContextScope();
	ENGINE_API ~FObjectCacheContextScope();
	ENGINE_API FObjectCacheContext& GetContext();
private:
	// Scopes can be stacked over one another, but only the outermost will
	// own the actual context and destroy it at the end, all inner scopes
	// will feed off the already existing one and will not own it.
	bool bIsOwner = false;
};
