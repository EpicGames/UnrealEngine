// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

class UActorComponent;
class UMaterialInterface;
class UPrimitiveComponent;
class UStaticMeshComponent;

class IPrimitiveComponent;
class IStaticMeshComponent;
class ISkinnedMeshComponent;
class UTexture;

struct  FObjectCacheEventSink
{
	// For parallel processing.  Calling BeginQueueNotifyEvents will prevent notify events from being processed, an instead placed in a list for later processing.
	// While other threads are actively updating the events, the main thread should process those events using this ProcessQueuedNotifyEvents
	// When parallel processing is complete, calling EndQueueNotifyEvents will flush all events, and return to normal in place processing.
	static ENGINE_API void BeginQueueNotifyEvents();
	static ENGINE_API void ProcessQueuedNotifyEvents();
	static ENGINE_API void EndQueueNotifyEvents();

	static ENGINE_API void NotifyUsedMaterialsChanged_Concurrent(const IPrimitiveComponent* PrimitiveComponent, const TArray<UMaterialInterface*>& UsedMaterials);
	static ENGINE_API void NotifyRenderStateChanged_Concurrent(UActorComponent*);
	static ENGINE_API void NotifyRenderStateChanged_Concurrent(IPrimitiveComponent*);
	static ENGINE_API void NotifyMaterialChanged_Concurrent(UMaterialInterface*);
	static inline void NotifyReferencedTextureChanged_Concurrent(UMaterialInterface* InMaterial)
	{
		NotifyMaterialChanged_Concurrent(InMaterial);
	}
	static ENGINE_API void NotifyStaticMeshChanged_Concurrent(IStaticMeshComponent*);
	static ENGINE_API void NotifySkinnedMeshChanged_Concurrent(ISkinnedMeshComponent*);
	static ENGINE_API void NotifyMaterialDestroyed_Concurrent(UMaterialInterface*);
	static ENGINE_API void NotifyCompositeTextureChanged_Concurrent(UTexture*);
};

#endif // #if WITH_EDITOR