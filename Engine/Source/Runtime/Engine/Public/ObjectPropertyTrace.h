// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreTypes.h"
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "ObjectTraceDefines.h"

#define OBJECT_PROPERTY_TRACE_ENABLED (WITH_ENGINE && OBJECT_TRACE_ENABLED && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#if OBJECT_PROPERTY_TRACE_ENABLED

class UObject;

struct FObjectPropertyTrace
{
	/** Initialize object property tracing */
	static ENGINE_API void Init();

	/** Shut down object property tracing */
	static ENGINE_API void Destroy();

	/** Check whether object property tracing is enabled */
	static ENGINE_API bool IsEnabled();

	/** Toggle registration for a UObject being traced by the system */
	static ENGINE_API void ToggleObjectRegistration(const UObject* InObject);

	/** Register a UObject to be traced by the system */
	static ENGINE_API void RegisterObject(const UObject* InObject);

	/** Unregister a UObject to be traced by the system */
	static ENGINE_API void UnregisterObject(const UObject* InObject);

	/** Check whether an object is registered */
	static ENGINE_API bool IsObjectRegistered(const UObject* InObject);
};


#define TRACE_OBJECT_PROPERTIES_BEGIN(Object) FObjectPropertyTrace::RegisterObject(Object);
#define TRACE_OBJECT_PROPERTIES_END(Object) FObjectPropertyTrace::UnregisterObject(Object);

#else // OBJECT_PROPERTY_TRACE_ENABLED

#define TRACE_OBJECT_PROPERTIES_BEGIN(Object)
#define TRACE_OBJECT_PROPERTIES_END(Object)

#endif // OBJECT_PROPERTY_TRACE_ENABLED
