// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Subsystems/WorldSubsystem.h"
#include "ObjectTraceDefines.h"
#include "Trace/Config.h"
#include "TraceFilter.h"

#include "ObjectTrace.generated.h"

/** World subsystem used to track world info */
UCLASS()
class UObjectTraceWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

	UObjectTraceWorldSubsystem()
		: FrameIndex(0), RecordingIndex(0), ElapsedTime(0.0) 
	{}

#if !OBJECT_TRACE_ENABLED
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override
	{
		return false;
	}
#endif

public:
	/** The frame index incremented each tick */
	uint16 FrameIndex;
	/** Trace Recording identifier  (set by RewindDebugger or 0) */
	uint16 RecordingIndex;
	/** Elapsed time since recording started  (or since start of game if RewindDebugger didn't start the trace) */
	double ElapsedTime;
};

#if OBJECT_TRACE_ENABLED

class UClass;
class UObject;
class UWorld;
class FSceneView;

struct FObjectTrace
{
	static constexpr uint64 InvalidObjectId = 0;

	/** Initialize object tracing */
	ENGINE_API static void Init();

	/** Shut down object tracing */
	ENGINE_API static void Destroy();

	/** Reset Caches so a new trace can be started*/
	ENGINE_API static void Reset();

	/** Helper function to output an type (class or struct) */
	ENGINE_API static void OutputType(const UStruct* InType);
	
	/** Helper function to output a class */
	UE_DEPRECATED(5.7, "OutputClass is now deprecated, use OutputType")
	static void OutputClass(const UClass* InClass)
	{
		OutputType(InClass);
	};

	/** Helper function to output an object */
	ENGINE_API static bool OutputObject(const UObject* InObject, UObject* InUserDefinedOuter = nullptr);
	
	/** Helper function to output an "object instance" which is not an actual UObject */
	ENGINE_API static void OutputInstance(const UObject* InOuterObject, uint64 InInstanceId, uint64 InOuterId, UStruct* InType, const FString& InName, const FString& InPathName = FString());
	
	/** Helper function to output an object transform */
	ENGINE_API static void OutputObjectTransform(const UObject* InObject, const FTransform& InTransform, ETeleportType InTeleportType);

	/** Helper function to output object creation event */
	ENGINE_API static void OutputObjectLifetimeBegin(const UObject* InObject, UObject* InUserDefinedOuter = nullptr);
	
	/** Helper function to output object destruction event */
	ENGINE_API static void OutputInstanceLifetimeEnd(const UObject* InOuterObject, uint64 InInstanceId);

	/** Helper function to output object destruction event */
	ENGINE_API static void OutputObjectLifetimeEnd(const UObject* InObject);

	/** Helper function to output camera information for a player */
	ENGINE_API static void OutputView(const UObject* InPlayer, const FSceneView* InView);

	/** Helper function to output an object event */
	ENGINE_API static void OutputObjectEvent(const UObject* InObject, const TCHAR* InEvent);

	/** Helper function to output controller attach event */
	ENGINE_API static void OutputPawnPossess(const UObject* InController, const UObject* InPawn);

	/** Helper function to allocate a new object Id */
	ENGINE_API static uint64 AllocateInstanceId();
	
	/** Helper function to get an object ID from a UObject */
	ENGINE_API static uint64 GetObjectId(const UObject* InObject);

	/** Helper function to get the UObject from an ObjectId, if the UObject still exists */
	ENGINE_API static UObject* GetObjectFromId(uint64 id);

	/** Helper function to get an object's world's tick counter */
	ENGINE_API static uint16 GetObjectWorldTickCounter(const UObject* InObject);

	/** reset the world elapsed time to 0 */
	ENGINE_API static void ResetWorldElapsedTime(const UWorld* InWorld);

	/** Helper function to get a world's elapsed time */
	ENGINE_API static double GetWorldElapsedTime(const UWorld* InWorld);

	/** Helper function to get an object's world's elapsed time */
	ENGINE_API static double GetObjectWorldElapsedTime(const UObject* InObject);

	/** Helper function to set a world's recording index */
	ENGINE_API static void SetWorldRecordingIndex(const UWorld *World, uint16 Index);

	/** Helper function to get a world's recording index */
	ENGINE_API static uint16 GetWorldRecordingIndex(const UWorld *InWorld);

	/** Helper function to get an object's world's recording index */
	ENGINE_API static uint16 GetObjectWorldRecordingIndex(const UObject* InObject);

	/** Helper function to output a world */
	ENGINE_API static void OutputWorld(const UWorld* InWorld);
};

#define TRACE_TYPE(Type) \
	FObjectTrace::OutputType(Type);

#define TRACE_CLASS(Class) \
	FObjectTrace::OutputType(Class);

#define TRACE_OBJECT(Object) \
	FObjectTrace::OutputObject(Object);

#define TRACE_OBJECT_WITH_OUTER(Object, UserDefinedOuter) \
	FObjectTrace::OutputObject(Object, UserDefinedOuter);

#define TRACE_INSTANCE(OuterObject, InstanceId, OuterId, Type, Name) \
	FObjectTrace::OutputInstance(OuterObject, InstanceId, OuterId, Type, Name);

#define TRACE_VIEW(Player, View) \
	FObjectTrace::OutputView(Player, View);

#if TRACE_FILTERING_ENABLED

#define TRACE_OBJECT_EVENT(Object, Event) \
	if (CAN_TRACE_OBJECT(Object)) { UNCONDITIONAL_TRACE_OBJECT_EVENT(Object, Event); }

#define TRACE_OBJECT_TRANSFORM(Object, Transform, Teleport) \
	if (CAN_TRACE_OBJECT(Object)) { FObjectTrace::OutputObjectTransform(Object, Transform, Teleport); }

#define TRACE_OBJECT_LIFETIME_BEGIN(Object) \
	if (CAN_TRACE_OBJECT(Object)) { FObjectTrace::OutputObjectLifetimeBegin(Object); }

#define TRACE_OBJECT_LIFETIME_BEGIN_WITH_OUTER(Object, UserDefinedOuter) \
	if (CAN_TRACE_OBJECT(Object)) { FObjectTrace::OutputObjectLifetimeBegin(Object, UserDefinedOuter); }

#define TRACE_OBJECT_LIFETIME_END(Object) \
	if (CAN_TRACE_OBJECT(Object)) { FObjectTrace::OutputObjectLifetimeEnd(Object); }

#define TRACE_INSTANCE_LIFETIME_END(OuterObject, InstanceId) \
	if (CAN_TRACE_OBJECT(OuterObject)) { FObjectTrace::OutputInstanceLifetimeEnd(OuterObject, InstanceId); }

#define TRACE_PAWN_POSSESS(Controller, Pawn)\
	if (CAN_TRACE_OBJECT(Controller) && (Pawn==nullptr || (CAN_TRACE_OBJECT(Pawn)))) { FObjectTrace::OutputPawnPossess(Controller, Pawn); }

#else // TRACE_FILTERING_ENABLED

#define TRACE_OBJECT_EVENT(Object, Event) \
	UNCONDITIONAL_TRACE_OBJECT_EVENT(Object, Event);

#define TRACE_OBJECT_TRANSFORM(Object, Transform, Teleport) \
	FObjectTrace::OutputObjectTransform(Object, Transform, Teleport);

#define TRACE_OBJECT_LIFETIME_BEGIN(Object) \
	FObjectTrace::OutputObjectLifetimeBegin(Object);

#define TRACE_OBJECT_LIFETIME_BEGIN_WITH_OUTER(Object, UserDefinedOuter) \
	FObjectTrace::OutputObjectLifetimeBegin(Object, UserDefinedOuter);

#define TRACE_OBJECT_LIFETIME_END(Object) \
	FObjectTrace::OutputObjectLifetimeEnd(Object);

#define TRACE_INSTANCE_LIFETIME_END(OuterObject, InstanceId) \
	FObjectTrace::OutputInstanceLifetimeEnd(OuterObject, InstanceId);

#define TRACE_PAWN_POSSESS(Controller, Pawn)\
	FObjectTrace::OutputPawnPossess(Controller, Pawn);

#endif // TRACE_FILTERING_ENABLED
	
#define UNCONDITIONAL_TRACE_OBJECT_EVENT(Object, Event) \
	FObjectTrace::OutputObjectEvent(Object, TEXT(#Event));

#define TRACE_WORLD(World) \
	FObjectTrace::OutputWorld(World);


#else // OBJECT_TRACE_ENABLED

struct FObjectTrace
{
	/** Helper function to get the UObject from an ObjectId, if the UObject still exists */
	static UObject* GetObjectFromId(uint64 id)
	{
		return nullptr;
	}
};

#define TRACE_CLASS(...)
#define TRACE_TYPE(...)
#define TRACE_OBJECT(...)
#define TRACE_INSTANCE(...)
#define TRACE_OBJECT_TRANSFORM(...)
#define TRACE_OBJECT_EVENT(...)
#define TRACE_WORLD(...)
#define TRACE_PAWN_POSSESS(...)
#define TRACE_VIEW(...)
#define TRACE_OBJECT_LIFETIME_BEGIN(...)
#define TRACE_OBJECT_LIFETIME_BEGIN_WITH_OUTER(...)
#define TRACE_OBJECT_LIFETIME_END(...)
#define TRACE_INSTANCE_LIFETIME_END(...)
#define TRACE_CHILD_ELEMENT(...)
#define TRACE_CHILD_ELEMENT_WITH_OUTER(...)
#define TRACE_CHILD_ELEMENT_LIFETIME_BEGIN(...)
#define TRACE_CHILD_ELEMENT_WITH_OUTER_LIFETIME_BEGIN(...)
#define TRACE_CHILD_ELEMENT_LIFETIME_END(...)
#define TRACE_CHILD_ELEMENT_EVENT(...)

#endif // OBJECT_TRACE_ENABLED
