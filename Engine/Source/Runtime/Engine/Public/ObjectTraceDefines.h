// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Config.h"

#ifndef OBJECT_TRACE_ENABLED
#define OBJECT_TRACE_ENABLED ((UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING) || UE_TRACE_MINIMAL_ENABLED)
#endif
