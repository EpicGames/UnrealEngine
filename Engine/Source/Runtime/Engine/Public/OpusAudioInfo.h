// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * DEPRECATED 5.4 OpusAudioInfo.h. please use the module path of Decoders/OpusAudioInfo.h instead.
 */

// HEADER_UNIT_SKIP - Deprecated

#pragma once

UE_DEPRECATED_HEADER(5.4, "OpusAudioInfo.h has moved to its own module OpusAudioDecoder.")

 // Include the new file so that the project still compiles, but has warnings
#include "Decoders/OpusAudioInfo.h"
