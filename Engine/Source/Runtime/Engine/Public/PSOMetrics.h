// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOMetrics.h
=============================================================================*/

#pragma once

#include "Delegates/Delegate.h"

/**
 * Accumulate PSO metric is called for each individual PSO compilation.
 * At the moment only IOS and Android platforms will call this.
 */ 
extern ENGINE_API void AccumulatePSOMetrics(float CompilationDuration);

// retrieves the current metrics and set it to zero
extern ENGINE_API void GetPSOCompilationMetrics(float& DurationSum, int& Count);
