// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecache.h
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "RHIResources.h"
#include "Engine/EngineTypes.h"
#include "PipelineStateCache.h"
#include "PSOPrecacheFwd.h"
#include "Shader.h"

class FVertexFactoryType;
struct FSceneTexturesConfig;

enum class EPSOPrecacheType : uint8 
{
	Global = 0,
	MeshPass,
};

/**
 * Parameters which are needed to collect all possible PSOs used by the PSO collectors
 */
struct FPSOPrecacheParams
{
	FPSOPrecacheParams()
	{
		PrimitiveType = (uint8)PT_TriangleList;
		bDefaultMaterial = false;
		bCanvasMaterial = false;
		bSplineMesh = false;
		bSkinnedMesh = false;
		bRenderInMainPass = true;
		bRenderInDepthPass = true;
		bStaticLighting = true;
		bCastShadow = true;
		bRenderCustomDepth = false;
		bAffectDynamicIndirectLighting = true;
		bReverseCulling = false;
		bDisableBackFaceCulling = false;
		bCastShadowAsTwoSided = false;
		bForceLODModel = false;
		Mobility = (uint8)EComponentMobility::Static;
		bAnyMaterialHasWorldPositionOffset = false;
		StencilWriteMask = (uint8)EStencilMask::SM_Default;
		BasePassPixelFormat = (uint16)PF_Unknown;
		bHasWPODistance = false;
		bHasPixelDistance = false;
		bIgnoreMaskMaterial = false;
		bRequireRTDynamicGeometryUpdate = false;

		Unused = 0;
	}

	bool operator==(const FPSOPrecacheParams& Other) const
	{
		return Data == Other.Data;
	}

	bool operator!=(const FPSOPrecacheParams& rhs) const
	{
		return !(*this == rhs);
	}

	friend uint32 GetTypeHash(const FPSOPrecacheParams& Params)
	{
		return GetTypeHash(Params.Data);
	}

	void SetMobility(EComponentMobility::Type InMobility)
	{
		Mobility = (uint8)InMobility;
	}

	TEnumAsByte<EComponentMobility::Type> GetMobility() const
	{
		return TEnumAsByte<EComponentMobility::Type>((EComponentMobility::Type)Mobility);
	}

	bool IsMoveable() const
	{
		return Mobility == EComponentMobility::Movable || Mobility == EComponentMobility::Stationary;
	}

	void SetStencilWriteMask(EStencilMask InStencilMask)
	{
		StencilWriteMask = (uint8)InStencilMask;
	}

	TEnumAsByte<EStencilMask> GetStencilWriteMask() const
	{
		return TEnumAsByte<EStencilMask>((EStencilMask)StencilWriteMask);
	}

	void SetBassPixelFormat(EPixelFormat InBasePassPixelFormat)
	{
		BasePassPixelFormat = (uint16)InBasePassPixelFormat;
	}

	EPixelFormat GetBassPixelFormat() const
	{
		return (EPixelFormat)BasePassPixelFormat;
	}

	union
	{
		struct
		{
			uint64 PrimitiveType : 6;
			
			uint64 bDefaultMaterial : 1;
			uint64 bCanvasMaterial : 1;

			uint64 bSplineMesh : 1;
			uint64 bSkinnedMesh : 1;
			
			uint64 bRenderInMainPass : 1;
			uint64 bRenderInDepthPass : 1;
			uint64 bStaticLighting : 1;
			uint64 bCastShadow : 1;
			uint64 bRenderCustomDepth : 1;
			
			uint64 bUsesIndirectLightingCache : 1;
			uint64 bAffectDynamicIndirectLighting : 1;
			uint64 bReverseCulling : 1;
			uint64 bDisableBackFaceCulling : 1;
			uint64 bCastShadowAsTwoSided : 1;
			uint64 bForceLODModel : 1;
			
			uint64 Mobility : 4;
			uint64 bAnyMaterialHasWorldPositionOffset : 1;
			uint64 StencilWriteMask : 4;

			uint64 BasePassPixelFormat : 16;

			uint64 bHasWPODistance : 1;
			uint64 bHasPixelDistance : 1;
			uint64 bIgnoreMaskMaterial : 1;

			uint64 bRequireRTDynamicGeometryUpdate : 1;

			uint64 Unused : 14;
		};
		uint64 Data;
	};

};

// Unique ID to find the FVertexDeclarationElementList - these can be shared
using FVertexDeclarationElementListID = uint16;

/**
 * Wraps vertex factory data used during PSO precaching - optional element list ID can be used if manual vertex fetch is not possible for the given vertex factory type
 */
struct FPSOPrecacheVertexFactoryData
{
	FPSOPrecacheVertexFactoryData() = default;
	FPSOPrecacheVertexFactoryData(const FVertexFactoryType* InVertexFactoryType) : VertexFactoryType(InVertexFactoryType), CustomDefaultVertexDeclaration(nullptr) {}
	ENGINE_API FPSOPrecacheVertexFactoryData(const FVertexFactoryType* InVertexFactoryType, const FVertexDeclarationElementList& ElementList);

	const FVertexFactoryType* VertexFactoryType = nullptr;

	// Custom vertex declaration used for EVertexInputStreamType::Default if provided - the others are directly retrieved from the type if needed
	FRHIVertexDeclaration* CustomDefaultVertexDeclaration = nullptr;

	bool operator==(const FPSOPrecacheVertexFactoryData& Other) const
	{
		return VertexFactoryType == Other.VertexFactoryType && CustomDefaultVertexDeclaration == Other.CustomDefaultVertexDeclaration;
	}

	bool operator!=(const FPSOPrecacheVertexFactoryData& rhs) const
	{
		return !(*this == rhs);
	}

	friend uint32 GetTypeHash(const FPSOPrecacheVertexFactoryData& Params)
	{
		return HashCombine(PointerHash(Params.VertexFactoryType), PointerHash(Params.CustomDefaultVertexDeclaration));
	}
};

struct FPSOPrecacheVertexFactoryDataPerMaterialIndex
{
	int16 MaterialIndex;
	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
};

struct FMaterialInterfacePSOPrecacheParams
{
	EPSOPrecachePriority Priority = EPSOPrecachePriority::Medium;
	UMaterialInterface* MaterialInterface = nullptr;
	FPSOPrecacheParams PSOPrecacheParams;
	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
};

extern ENGINE_API void AddMaterialInterfacePSOPrecacheParamsToList(const FMaterialInterfacePSOPrecacheParams& EntryToAdd, FMaterialInterfacePSOPrecacheParamsList& List);

struct FShaderPreloadData
{
	FShaderPreloadData() = default;

	FShaderPreloadData(const TShaderRef<FShader>& InShader)
	{
		Shaders.Emplace(InShader);
	}

	FShaderPreloadData(TArray<TShaderRef<FShader>, TInlineAllocator<3>>&& InShaders)
		: Shaders(MoveTemp(InShaders))
	{
	}

	// Can have 3 shaders at most (vertex, geometry, pixel).
	TArray<TShaderRef<FShader>, TInlineAllocator<3>> Shaders;
};

enum class EPSOPrecacheMode : uint8
{
	PSO = 0,
	PreloadShader = 1,
};

extern ENGINE_API EPSOPrecacheMode GetPSOPrecacheMode();

/**
 * Wrapper class around the initializer to collect some extra validation data during PSO collection on the different collectors
 */
struct FPSOPrecacheData
{
	enum class EType : uint8
	{
		Graphics,
		Compute,
	};
	EType Type = EType::Graphics;
	
	// Is the PSO required to be able render the object or can it provide a fallback path
	// (proxy creation won't wait for these PSOs if enabled)
	bool bRequired = true;

	// Precache priority mode
	EPSOPrecachePriority Priority = EPSOPrecachePriority::Medium;

	// Keep the PSO in memory until the owning material is request deleted
	EPSOPrecacheInMemoryMode InMemoryMode = EPSOPrecacheInMemoryMode::Cached;

	FGraphicsPipelineStateInitializer GraphicsPSOInitializer;
	FComputePipelineStateInitializer ComputeInitializer;

	FShaderPreloadData ShaderPreloadData;

	bool SetComputeShader(const TShaderRef<FShader>& InComputeShader)
	{
		if (GetPSOPrecacheMode() == EPSOPrecacheMode::PreloadShader)
		{
			ShaderPreloadData.Shaders.Emplace(InComputeShader);
		}
		else
		{
			FRHIComputeShader* ComputeShaderRHI = InComputeShader.GetComputeShader(false);
			if (!ComputeShaderRHI)
			{
				return false;
			}
			ComputeInitializer = FComputePipelineStateInitializer(ComputeShaderRHI, InComputeShader.GetSpecializationConstants());
		}
		return true;
	}

#if PSO_PRECACHING_VALIDATE
	FString ResourceName;
	int32 PSOCollectorIndex : 31 = INDEX_NONE;
	int32 bDefaultMaterial : 1 = 0;
	const FVertexFactoryType* VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE
};

struct FMaterialPSOPrecacheParams
{
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
	FMaterial* Material = nullptr;
	FPSOPrecacheVertexFactoryData VertexFactoryData;
	FPSOPrecacheParams PrecachePSOParams;

	bool operator==(const FMaterialPSOPrecacheParams& Other) const
	{
		return FeatureLevel == Other.FeatureLevel &&
			Material == Other.Material &&
			VertexFactoryData == Other.VertexFactoryData &&
			PrecachePSOParams == Other.PrecachePSOParams;
	}

	bool operator!=(const FMaterialPSOPrecacheParams& rhs) const
	{
		return !(*this == rhs);
	}

	friend uint32 GetTypeHash(const FMaterialPSOPrecacheParams& Params)
	{
		return HashCombine(GetTypeHash(Params.FeatureLevel), HashCombine(PointerHash(Params.Material),
			HashCombine(GetTypeHash(Params.VertexFactoryData), GetTypeHash(Params.PrecachePSOParams))));
	}
};

/**
* Predeclared global PSOCollector function
*/
typedef void (*GlobalPSOCollectorFunction)(const FSceneTexturesConfig& SceneTexturesConfig, int32 GlobalPSOCollectorIndex, TArray<FPSOPrecacheData>& PSOInitializers);

/**
* Manages all collect functions of the globally declared PSOCollectorCreateFunction
*/
class FGlobalPSOCollectorManager
{
public:
	constexpr static uint32 MaxPSOCollectorCount = 8;

	static int32 GetPSOCollectorCount() { return PSOCollectorCount; }
	static GlobalPSOCollectorFunction GetCollectFunction(int32 Index)
	{
		check(Index < MaxPSOCollectorCount);
		return PSOCollectors[Index].CollectFunction;
	}
	static const TCHAR* GetName(int32 Index)
	{
		if (Index == INDEX_NONE)
		{
			return TEXT("Unknown");
		}
		check(Index < MaxPSOCollectorCount);
		return PSOCollectors[Index].Name;
	}
	static ENGINE_API int32 GetIndex(const TCHAR* Name);

private:

	// Have to used fixed size array instead of TArray because of order of initialization of static member variables
	static inline int32 PSOCollectorCount = 0;

	struct FPSOCollectorData
	{
		GlobalPSOCollectorFunction CollectFunction;
		const TCHAR* Name = nullptr;
	};
	static ENGINE_API FPSOCollectorData PSOCollectors[MaxPSOCollectorCount];

	friend class FRegisterGlobalPSOCollectorFunction;
};

/**
* Helper class used to register/unregister the GlobalPSOCollectorFunction to the manager at static startup time
*/
class FRegisterGlobalPSOCollectorFunction
{
public:
	FRegisterGlobalPSOCollectorFunction(GlobalPSOCollectorFunction InCollectFunction, const TCHAR* InName)
	{
		Index = FGlobalPSOCollectorManager::PSOCollectorCount;
		check(Index < FGlobalPSOCollectorManager::MaxPSOCollectorCount);

		FGlobalPSOCollectorManager::PSOCollectors[Index].CollectFunction = InCollectFunction;
		FGlobalPSOCollectorManager::PSOCollectors[Index].Name = InName;
		FGlobalPSOCollectorManager::PSOCollectorCount++;
	}

	~FRegisterGlobalPSOCollectorFunction()
	{
		FGlobalPSOCollectorManager::PSOCollectors[Index].CollectFunction = nullptr;
		FGlobalPSOCollectorManager::PSOCollectors[Index].Name = nullptr;
	}

	int32 GetIndex() const { return Index; }

private:
	uint32 Index;
};



/**
 * Precaching PSOs for components?
 */
extern ENGINE_API bool IsComponentPSOPrecachingEnabled();

/**
 * Precaching PSOs for resources?
 */
extern ENGINE_API bool IsResourcePSOPrecachingEnabled();

/**
 * Boost drawn PSO precache request priority
 */
extern ENGINE_API bool ShouldBoostPSOPrecachePriorityOnDraw();

/**
 * Dynamically preload shaders
 */
extern ENGINE_API bool IsPSOShaderPreloadingEnabled();

/**
 * Boost the primitive's precache PSO jobs to highest priority.
 */
extern ENGINE_API void BoostPrecachedPSORequestsOnDraw(const class FPrimitiveSceneInfo* SceneInfo);

/**
 * Precache graphics PSO and return true if PSO is fully precached
 */
extern ENGINE_API bool PrecachePSO(const FGraphicsPipelineStateInitializer& GraphicsPSO);

/**
* Precache compute PSO and return true if PSO is fully precached
 */
extern ENGINE_API bool PrecachePSO(const FComputePipelineStateInitializer& ComputePSO);

/**
 * Start the actual async PSO precache request from the given list of initializers
 */
extern ENGINE_API FPSOPrecacheRequestResultArray RequestPrecachePSOs(EPSOPrecacheType PSOPrecacheType, const FPSOPrecacheDataArray& PSOInitializers);