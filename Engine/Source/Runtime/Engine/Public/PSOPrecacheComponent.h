// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "RHIResources.h"
#include "Engine/EngineTypes.h"
#include "PipelineStateCache.h"
#include "PSOPrecacheFwd.h"

#include "PSOPrecacheComponent.generated.h"

UENUM()
enum class EPSOPrecacheProxyCreationStrategy : uint8
{
	// Always create the render proxy regardless of whether the PSO has finished precaching or not. 
	// This will introduce a blocking wait when the proxy is rendered if the PSO is not ready.
	AlwaysCreate = 0, 

	// Delay the creation of the render proxy until the PSO has finished precaching. 
	// This effectively skips drawing components until the PSO is ready, when the proxy will be created.
	DelayUntilPSOPrecached = 1, 
	
	// Create a render proxy that uses the fallback material if the PSO has not finished precaching by creation time.
	// The proxy will be re-created with the actual materials once the PSO is ready.
	// Currently implemented only for static and skinned mesh components, while Niagara components will skip render proxy creation altogether.
	UseFallbackMaterialUntilPSOPrecached = 2
};

extern ENGINE_API EPSOPrecacheProxyCreationStrategy GetPSOPrecacheProxyCreationStrategy(int32 OverrideValue = -1);

/**
 * Delay component proxy creation when it's requested PSOs are still precaching
 */
UE_DEPRECATED(5.8, "ProxyCreationWhenPSOReady is deprecated - pleas use GetPSOPrecacheProxyCreationStrategy")
extern ENGINE_API bool ProxyCreationWhenPSOReady();

struct FPSOPrecacheComponentData
{			
	void Reset()
	{		
#if UE_WITH_PSO_PRECACHING
		bPSOPrecacheCalled = false;
		PSOPrecacheRequestPriority = EPSOPrecachePriority::Medium;
		MaterialPSOPrecacheRequestIDs.Empty();
#endif // UE_WITH_PSO_PRECACHING
	}

	bool IsPSOPrecaching() const
	{
#if UE_WITH_PSO_PRECACHING
		return LatestPSOPrecacheJobSetCompleted != LatestPSOPrecacheJobSet;
#else
		return false;
#endif // UE_WITH_PSO_PRECACHING
	}

	bool IsPSOPrecacheCalled() const
	{
#if UE_WITH_PSO_PRECACHING
		return bPSOPrecacheCalled;
#else
		return false;
#endif // UE_WITH_PSO_PRECACHING
	}

#if UE_WITH_PSO_PRECACHING
	const TArray<FMaterialPSOPrecacheRequestID>& GetMaterialPSOPrecacheRequestIDs() const
	{
		return MaterialPSOPrecacheRequestIDs;
	}

	void SetLatestJobSetCompleted(int32 JobSetThatJustCompleted)
	{
		int32 CurrJobSetCompleted = LatestPSOPrecacheJobSetCompleted.load();
		while (CurrJobSetCompleted < JobSetThatJustCompleted && !LatestPSOPrecacheJobSetCompleted.compare_exchange_weak(CurrJobSetCompleted, JobSetThatJustCompleted)){}
	}
#endif // UE_WITH_PSO_PRECACHING
	
	bool UsePSOPrecacheFallbackMaterial() const
	{
#if UE_WITH_PSO_PRECACHING
		return IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached;
#else
		return false;
#endif // UE_WITH_PSO_PRECACHING
	}
	
	// Set all the PSO precaching data and mark as precached - will also kick task to run when precaching is done and given callback will be executed then when the component is still valid
	template<typename ComponentType, typename PrecachedFinishedCallback>
	void SetMaterialPSOPrecacheData(ComponentType* Component, TArray<FMaterialPSOPrecacheRequestID>& InMaterialPSOPrecacheRequestIDs, const FGraphEventArray& PSOPrecacheCompileEvents, PrecachedFinishedCallback InCallback);
	
	// Check if PSOs are still compiling, and if so, boost them to requested priority
	ENGINE_API bool CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority);

#if UE_WITH_PSO_PRECACHING

private:

	/** Helper flag to check if PSOs have been precached already */
	uint8 bPSOPrecacheCalled : 1 = false;

	/** PSOs requested priority */
	EPSOPrecachePriority PSOPrecacheRequestPriority : 3 = EPSOPrecachePriority::Medium;
	static_assert((int)EPSOPrecachePriority::Highest < 1 << 3);

	int32 LatestPSOPrecacheJobSet : 28 = 0;

	/** Cached array of material PSO requests which can be used to boost the priority */
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;

	/** Atomic int used to track the last PSO precache events */
	std::atomic<int> LatestPSOPrecacheJobSetCompleted = 0;		

#endif // UE_WITH_PSO_PRECACHING
};

#if UE_WITH_PSO_PRECACHING

template<typename T, typename Enable = void>
struct TPSOWeakPtrTraits
{
	using WeakType = TWeakObjectPtr<T>;

	static WeakType Make(T* Ptr) { return WeakType(Ptr); }
	static T* Get(const WeakType& W) { return W.Get(); }
};

template<typename ComponentType, typename PrecachedFinishedOnComponentCallback>
struct FPSOPrecacheFinishedTask
{
	using WeakType = typename TPSOWeakPtrTraits<ComponentType>::WeakType;

	explicit FPSOPrecacheFinishedTask(ComponentType* Component, int32 InJobSetThatJustCompleted, PrecachedFinishedOnComponentCallback InCallback)
		: WeakComponent(TPSOWeakPtrTraits<ComponentType>::Make(Component)),
		JobSetThatJustCompleted(InJobSetThatJustCompleted),
		Callback(InCallback)
	{
	}

	static TStatId GetStatId() { return TStatId(); }
	static ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if (ComponentType* Component = TPSOWeakPtrTraits<ComponentType>::Get(WeakComponent))
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_PSOPrecacheFinishedTask);
			FPSOPrecacheComponentData& PSOPrecacheComponentData = Component->GetPSOPrecacheComponentData();
			PSOPrecacheComponentData.SetLatestJobSetCompleted(JobSetThatJustCompleted);

			Callback();
		}
	}
		
	WeakType WeakComponent;
	int32 JobSetThatJustCompleted;
	PrecachedFinishedOnComponentCallback Callback;
};

#endif

template<typename ComponentType, typename PrecachedFinishedCallback>
void FPSOPrecacheComponentData::SetMaterialPSOPrecacheData(
	ComponentType* Component, 
	TArray<FMaterialPSOPrecacheRequestID>& InMaterialPSOPrecacheRequestIDs, 
	const FGraphEventArray& PSOPrecacheCompileEvents, 
	PrecachedFinishedCallback InCallback)
{
#if UE_WITH_PSO_PRECACHING
	MaterialPSOPrecacheRequestIDs = MoveTemp(InMaterialPSOPrecacheRequestIDs);
	bPSOPrecacheCalled = true;

	// If the proxy creation strategy relies on knowing when the precached PSO has been compiled,
	// schedule a task to mark the render state dirty when all PSOs are compiled so the proxy gets recreated.
	if (GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
	{
		int32 NewJobID = LatestPSOPrecacheJobSet + 1;
		LatestPSOPrecacheJobSet = NewJobID;
		if(!PSOPrecacheCompileEvents.IsEmpty())
		{
			TGraphTask<FPSOPrecacheFinishedTask<ComponentType, PrecachedFinishedCallback>>::CreateTask(&PSOPrecacheCompileEvents).ConstructAndDispatchWhenReady(Component, NewJobID, InCallback);
		}
		else
		{
			// No graph events to wait on, the job set can be considered complete.
			LatestPSOPrecacheJobSetCompleted = NewJobID;
		}
	}
#endif //UE_WITH_PSO_PRECACHING
}