// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheFwd.h
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "EngineDefines.h"
#include "HAL/Platform.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "PipelineStateCache.h"
#endif

// General switch that decides whether to compile out some PSO precaching code (most importantly, reduce sizeofs of common classes)
#ifndef UE_WITH_PSO_PRECACHING
	#define UE_WITH_PSO_PRECACHING	(PLATFORM_SUPPORTS_PSO_PRECACHING || PLATFORM_SUPPORTS_DYNAMIC_SHADER_PRELOADING) && !WITH_STATE_STREAM_ACTOR
#endif // UE_WITH_PSO_PRECACHING

// Only enable pso validation when the platform support its. Note: UE_WITH_PSO_PRECACHING can be define when dynamic shader preloading is supported. 
#define PSO_PRECACHING_VALIDATE !WITH_EDITOR && (UE_WITH_PSO_PRECACHING && PLATFORM_SUPPORTS_PSO_PRECACHING)

struct FMaterialInterfacePSOPrecacheParams;
struct FPSOPrecacheParams;

typedef TArray<FMaterialInterfacePSOPrecacheParams, TInlineAllocator<4> > FMaterialInterfacePSOPrecacheParamsList;

// Unique request ID of MaterialPSOPrecache which can be used to boost the priority of a PSO precache requests if it's needed for rendering
using FMaterialPSOPrecacheRequestID = uint32;

enum class EPSOPrecachePriority : uint8;

struct FPSOPrecacheData;
struct FPSOPrecacheRequestResult;

typedef TArray<FPSOPrecacheData> FPSOPrecacheDataArray;
typedef TArray<FPSOPrecacheRequestResult, TInlineAllocator<4>> FPSOPrecacheRequestResultArray;

struct FMaterialPSOPrecacheParams;

struct FPSOPrecacheVertexFactoryData;
typedef TArray<FPSOPrecacheVertexFactoryData, TInlineAllocator<2>> FPSOPrecacheVertexFactoryDataList;

struct FPSOPrecacheVertexFactoryDataPerMaterialIndex;
typedef TArray<FPSOPrecacheVertexFactoryDataPerMaterialIndex, TInlineAllocator<4>> FPSOPrecacheVertexFactoryDataPerMaterialIndexList;
