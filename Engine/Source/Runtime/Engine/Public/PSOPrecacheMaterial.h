// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheMaterial.h
=============================================================================*/

#pragma once

#include "PSOPrecache.h"
#include "SceneUtils.h"

struct FSceneTexturesConfig;
class FRHIComputeShader;
class FMaterial;
class FVertexFactoryType;
class FGraphicsPipelineStateInitializer;
enum class EVertexInputStreamType : uint8;


/**
 * Interface class implemented by the mesh pass processor to collect all possible PSOs
 */
class IPSOCollector
{
public:

	IPSOCollector(int32 InPSOCollectorIndex) : PSOCollectorIndex(InPSOCollectorIndex) {}
	virtual ~IPSOCollector() = default;

	// Collect all PSO for given material, vertex factory & params
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) = 0;

	// PSO Collector index used for stats tracking
	int32 PSOCollectorIndex = INDEX_NONE;
};

/**
 * Predeclared IPSOCollector create function
 */
typedef IPSOCollector* (*PSOCollectorCreateFunction)(ERHIFeatureLevel::Type InFeatureLevel);

/**
 * Manages all create functions of the globally defined IPSOCollectors
 */
class FPSOCollectorCreateManager
{
public:
	constexpr static uint32 MaxPSOCollectorCount = 64;

	static int32 GetPSOCollectorCount(EShadingPath ShadingPath) { return PSOCollectorCount[(uint32)ShadingPath]; }
	static PSOCollectorCreateFunction GetCreateFunction(EShadingPath ShadingPath, int32 Index)
	{
		check(Index < MaxPSOCollectorCount);
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		return PSOCollectors[ShadingPathIdx][Index].CreateFunction;
	}
	static const TCHAR* GetName(EShadingPath ShadingPath, int32 Index)
	{
		if (Index == INDEX_NONE)
		{
			return TEXT("Unknown");
		}
		check(Index < MaxPSOCollectorCount);
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		return PSOCollectors[ShadingPathIdx][Index].Name;
	}
	static ENGINE_API int32 GetIndex(EShadingPath ShadingPath, const TCHAR* Name);

private:

	// Have to used fixed size array instead of TArray because of order of initialization of static member variables
	static inline int32 PSOCollectorCount[(uint32)EShadingPath::Num] = {0,0};

	struct FPSOCollectorData
	{
		PSOCollectorCreateFunction CreateFunction;
		const TCHAR* Name = nullptr;
	};
	static ENGINE_API FPSOCollectorData PSOCollectors[(uint32)EShadingPath::Num][MaxPSOCollectorCount];

	friend class FRegisterPSOCollectorCreateFunction;
};

/**
 * Helper class used to register/unregister the IPSOCollector to the manager at static startup time
 */
class FRegisterPSOCollectorCreateFunction
{
public:
	FRegisterPSOCollectorCreateFunction(PSOCollectorCreateFunction InCreateFunction, EShadingPath InShadingPath, const TCHAR* InName)
		: ShadingPath(InShadingPath)
	{
		uint32 ShadingPathIdx = (uint32)ShadingPath;

		Index = FPSOCollectorCreateManager::PSOCollectorCount[ShadingPathIdx];
		check(Index < FPSOCollectorCreateManager::MaxPSOCollectorCount);

		FPSOCollectorCreateManager::PSOCollectors[ShadingPathIdx][Index].CreateFunction = InCreateFunction;
		FPSOCollectorCreateManager::PSOCollectors[ShadingPathIdx][Index].Name = InName;
		FPSOCollectorCreateManager::PSOCollectorCount[ShadingPathIdx]++;
	}

	~FRegisterPSOCollectorCreateFunction()
	{
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		FPSOCollectorCreateManager::PSOCollectors[ShadingPathIdx][Index].CreateFunction = nullptr;
		FPSOCollectorCreateManager::PSOCollectors[ShadingPathIdx][Index].Name = nullptr;
	}

	int32 GetIndex() const { return Index; }

private:
	EShadingPath ShadingPath;
	uint32 Index;
};

/**
 * Precache all PSOs for the given material data.
 */
extern ENGINE_API void PrecacheMaterialPSOs(const FMaterialInterfacePSOPrecacheParamsList& PSOPrecacheParamsList, TArray<FMaterialPSOPrecacheRequestID>& OutMaterialPSOPrecacheRequestIDs, FGraphEventArray& OutGraphEvents);

/**
 * Precache all PSOs for the given material and parameters.
 */
extern ENGINE_API FMaterialPSOPrecacheRequestID PrecacheMaterialPSOs(const FMaterialPSOPrecacheParams& MaterialPSOPrecacheParams, EPSOPrecachePriority Priority, FGraphEventArray& GraphEvents);

/**
 * Preload all shaders for the given material data.
 */
extern ENGINE_API void PreloadMaterialShaderMap(const FMaterial* Material, FGraphEventArray& OutGraphEvents);

/**
 * Release PSO material request data
 */
extern ENGINE_API void ReleasePSOPrecacheData(const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs);

/**
 * Boost priority for all the PSOs still compiling for the request material request IDs
 */
extern ENGINE_API void BoostPSOPriority(EPSOPrecachePriority NewPri, const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs);

/**
 * Boost priority to highest when the PSOs are required for drawing. 
 */
extern ENGINE_API void BoostPSOPriorityOnDraw(const TArray<FMaterialPSOPrecacheRequestID>& MaterialPSORequestIDs);

/**
 * Invalidate & clear all the current material PSO requests
 */
extern ENGINE_API void ClearMaterialPSORequests();

/**
 * Check if all precached PSOs for active materials need to be kept in memory
 */
extern ENGINE_API bool KeepPrecachedPSOsInMemoryForActiveMaterials();

/**
 * Get original FMaterialPSOPrecacheParams from precache request
 */
extern ENGINE_API FMaterialPSOPrecacheParams GetMaterialPSOPrecacheParams(FMaterialPSOPrecacheRequestID RequestID);

/**
 * Get original FPSOPrecacheDataArray from precache request
 */
extern ENGINE_API FPSOPrecacheDataArray GetMaterialPSOPrecacheData(FMaterialPSOPrecacheRequestID RequestID);

/**
 * Dump material PSO precache data to CSV file
 */
extern ENGINE_API void DumpPSOPrecacheMaterialStats();

/**
 * Get the total compile time for all material based PSOs for given pass and/or vertex factory name (if nullptr then collected for all passes and/or vertex factory types)
 */
extern ENGINE_API uint64 GetPSOPrecacheMaterialCompileTime(const TCHAR* InPassName, const TCHAR* InVertexFactoryName);


