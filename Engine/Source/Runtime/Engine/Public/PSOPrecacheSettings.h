// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheSettings.h
=============================================================================*/

#pragma once

#include "Engine/EngineTypes.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "PSOPrecacheFwd.h"

#include "PSOPrecacheSettings.generated.h"

class UMaterial;

/**
 * Settings used to configure what should be precached at runtime
*/
USTRUCT()
struct FPSOPrecacheSettings
{
	GENERATED_BODY()

	FPSOPrecacheSettings() {}

	/** Shader platform for which these settings should be applied */
	UPROPERTY()
	FString ShaderPlatform;

	/** Maximum number of CPU threads for which the settings should be applied */
	UPROPERTY()
	int32 MaxCPUThreadCount = INT32_MAX;

	/**
	 * Precache global shaders 
	 *	0: no precaching
	 *	1: All global compute and subset of global graphics PSOs for which PSO collector is defined (default)
	 *	2: All global compute and global graphics shaders (only available when r.PSOPrecache.Mode == 1 (Preload shaders))
	 */
	UPROPERTY()
	int GlobalShaderMode = 1;

	/** Keep global graphic PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled */
	UPROPERTY()
	bool bKeepGlobalGraphicPSOsInMemory = true;
	
	/** Keep global compute PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled */
	UPROPERTY()
	bool bKeepGlobalComputePSOsInMemory = false;

	/** Precache all PSO permutations for the default material during startup (default true) */
	UPROPERTY()
	bool bAllDefaultMaterialPSOs = true;

	/** Precache slate materials (default true) */
	UPROPERTY()
	bool bSlateMaterials = true;
	
	/** Skip draw of slate material batches when material based PSOs are still compiling and r.PSOPrecache.ProxyCreationStrategy != 0 */
	UPROPERTY()
	bool bSkipSlateMaterialDrawOnPSOTooLate = false;

	/** Keep slate PSOs in memory when r.PSOPrecache.KeepInMemoryForActiveMaterials is enabled */
	UPROPERTY()
	bool bKeepSlatePSOsInMemoryForActiveMaterials = true;
			
	/** Precache all possible light map policies for base pass shaders, otherwise only LMP_NO_LIGHTMAP (default false) */
	UPROPERTY()
	bool bBasePassApplyLightMapPolicies = false;

	/** Precache base pass permutation with alpha channel for planar reflections and scene capture passes (default false) */
	UPROPERTY()	
	bool bBasePassColorWithAlphaChannel = false;
	
	/** Precache permutations for EMeshPass::TranslucencyAll (default false) */
	UPROPERTY()	
	bool bTranslucencyAll = false;
	
	/** Precache permutations for EMeshPass::TranslucencyStandard even when marked as AfterDOF - can be needed when AutoBeforeDOFTranslucencyBoundary is set on the view (default false) */
	UPROPERTY()	
	bool bTranslucencyStandardBeforeDOF = false;
	
	/** Precache translucency under water pass permutations with different render target formats (default false) */
	UPROPERTY()	
	bool bTranslucencyUnderWater = false;
	
	/**
	 * Controls when PSO precaching happens for Niagara/Cascade particles systems:
	 *	0: no precaching
	 *	1: precaching at component proxy creation time only
	 *	2: precaching at component loading time
	 *	3: precaching at asset loading time and component time (default)
	 */
	UPROPERTY()	
	int32 ParticlePrecachingTime = 3;
	
	/** Precache deferred decal pass (default true) **/
	UPROPERTY()
	bool bDeferredDecals = true;
	
	// Precache depth pass permutations with different depth state and render targets used for projected shadows (default false) **/
	UPROPERTY()
	bool bDepthProjectedShadowsState = false;
	
	/** 
	 * Precache custom depth PSO permutations
	 * 	0: No PSOs are compiled
	 *  1: PSOs are compiled for all primitives which explicitly request custom depth rendering (PSOs are precached again on custom depth toggle on component) (default)
	 *  2: PSOs are compiled for all primitives which also request regular depth rendering
	 **/	
	UPROPERTY()
	int32 CustomDepthMode = 1;
	
	/** Precache permutations for EMeshPass::WaterInfoTextureDepthPass (default false) **/
	UPROPERTY()
	bool bWaterInfoDepth = false;

	/** Precache permutations for FLandscapeGrassWeightMeshProcessor (default true) **/
	UPROPERTY()
	bool bLandscapeGrassWeight = true;
	
	/** Precache permutations for EMeshPass::DitheredLODFadingOutMaskPass (default false) **/
	UPROPERTY()
	bool bDitheredLODFadingOutMask = false;
	
	/** Precache permutations for PostProcess material types (default true) **/
	UPROPERTY()
	bool bPostProcessingMaterial = true;
	
	/** Precache permutations for LightType material types which render to translucency light volume textures (default true) **/
	UPROPERTY()
	bool bTranslucencyLightingVolume = true;

	/** Make nanite raster programmable raster PSOs required before the component can be rendered with desired material or fallback to fixed function untill programmable raster PSOs are ready **/
	UPROPERTY()
	bool bNaniteProgrammableRasterRequired = false;
	
	/** Make nanite lumen card PSOs required before the component can be rendered with desired material or otherwise the mesh is not rendered to the LumenScene yet **/
	UPROPERTY()
	bool bNaniteLumenCardRequired = false;
};


UCLASS(MinimalAPI, defaultconfig, Config = Engine)
class UPSOPrecacheSettingsManager : public UObject
{
	GENERATED_BODY()

public:

	static ENGINE_API const FPSOPrecacheSettings& GetSettings();

	static ENGINE_API void SetupFallbackMaterial();
	static ENGINE_API UMaterial* GetFallbackMaterial();

private:
	
	UPROPERTY(globalconfig)
	TArray<FPSOPrecacheSettings> Settings;

	static TStrongObjectPtr<UMaterial> FallbackMaterial;
};