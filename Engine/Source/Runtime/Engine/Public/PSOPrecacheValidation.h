// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PSOPrecacheValidation.h
=============================================================================*/

#pragma once

#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "MaterialShared.h"

// enable detailed tracking of all PSO request for better logging on PSO misses
#define PSO_PRECACHING_TRACKING !WITH_EDITOR && !UE_BUILD_SHIPPING && !UE_BUILD_TEST && UE_WITH_PSO_PRECACHING

#if UE_WITH_PSO_PRECACHING

#if PSO_PRECACHING_VALIDATE

class FPrimitiveSceneProxy;
class FMaterialRenderProxy;

/**
 * Conditional break when PSO precaching a specific material - used for debugging PSO misses
 */
extern ENGINE_API void ConditionalBreakOnPSOPrecacheMaterial(const FMaterial& Material, int32 PSOCollectorIndex);

/**
 * Conditional break when PSO precaching a specific shader - used for debugging PSO misses
 */
extern ENGINE_API void ConditionalBreakOnPSOPrecacheShader(const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer);
extern ENGINE_API void ConditionalBreakOnPSOPrecacheShader(const FComputePipelineStateInitializer& ComputeInitializer);

//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
inline void ConditionalBreakOnPSOPrecacheShader(FRHIComputeShader* ComputeShader)
{
	FComputePipelineStateInitializer Initializer(ComputeShader);
	ConditionalBreakOnPSOPrecacheShader(Initializer);
}

/**
 * Type of PSO precache miss
 */
enum class EPSOPrecacheMissType : uint8 
{
	ShadersOnly = 0,		//< ShaderOnly miss - all other states not checked
	MinimalPSOState,		//< MinimalGraphicsPSO miss - shaders match, but render state is different than precached data
	FullPSO					//< Full PSO precache miss - minimal graphics PSO match, but bound render target and depth stencil state is different
};

/**
 * Log PSO miss information to give optional detailed information on what's causing the miss compared to what's already precached
 */
extern ENGINE_API void LogPSOMissInfo(
	const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer, 
	EPSOPrecacheMissType MissType,
	EPSOPrecacheResult PrecacheResult,
	const FMaterial* Material,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	EPSOPrecacheType PSOPrecacheType,
	int32 PSOCollectorIndex,
	uint64 ShadersOnlyPSOInitializerHash);

/**
 * Log PSO miss information to logg with optional detailed information on what's causing the miss compared to what's already precached
 */
extern ENGINE_API void LogPSOMissInfo(
	const FRHIComputeShader& ComputeShader,
	EPSOPrecacheResult PrecacheResult,
	const FMaterial* Material,
	EPSOPrecacheType PSOPrecacheType,
	int32 PSOCollectorIndex);

/**
 * Wraps all PSO precache state tracking data
 */
namespace PSOCollectorStats
{
	enum class EPSOPrecacheValidationMode : uint8 
	{
		Disabled = 0,
		Lightweight = 1,
		Full = 2
	};

	extern ENGINE_API EPSOPrecacheValidationMode GetPrecachingValidationMode();
	extern ENGINE_API bool IsPrecachingValidationEnabled();
	extern ENGINE_API bool IsFullPrecachingValidationEnabled();

	/**
	* Gets the state PSOs hash only from the grpahics PSOs - should be valid
	*/
	inline uint64 GetPSOPrecacheStateHash(const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer)
	{
		check(GraphicsPSOInitializer.StatePrecachePSOHash != 0);
		return GraphicsPSOInitializer.StatePrecachePSOHash;
	}


	/**
	 * Compute the hash of a graphics PSO initializer to be used by PSO precaching validation.
	 */
	extern ENGINE_API uint64 GetPSOPrecacheHash(const FGraphicsPipelineStateInitializer& GraphicsPSOInitializer);

	/**
	 * Compute the hash of a compute shader to be used by PSO precaching validation.
	 */
	extern ENGINE_API uint64 GetPSOPrecacheHash(const FComputePipelineStateInitializer& ComputeInitializer);

	//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
	inline uint64 GetPSOPrecacheHash(FRHIComputeShader& ComputeShader)
	{
		FComputePipelineStateInitializer ComputeInitializer(&ComputeShader);
		return GetPSOPrecacheHash(ComputeInitializer);
	}

	/**
	 * Create a shaders-only initializer where all state except for shaders is removed from the given initializer.
	 */
	extern ENGINE_API FGraphicsPipelineStateInitializer GetShadersOnlyInitializer(const FGraphicsPipelineStateInitializer& Initializer);
	   
	/**
	 * Check if the full PSO is precached and if not output information on what state is missing
	 */
	extern ENGINE_API void CheckFullPipelineStateInCache(
		const FGraphicsPipelineStateInitializer& Initializer, 
		EPSOPrecacheResult PSOPrecacheResult, 
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FVertexFactoryType* VFType, 
		const FPrimitiveSceneProxy* PrimitiveSceneProxy, 
		int32 PSOCollectorIndex);

	/**
	 * Check if the full PSO is precached and if not output information on what state is missing
	 */
	extern ENGINE_API void CheckFullPipelineStateInCache(
		const FGraphicsPipelineStateInitializer& Initializer,
		EPSOPrecacheResult PSOPrecacheResult,
		const FMaterial* Material,
		const FVertexFactoryType* VFType,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		int32 PSOCollectorIndex);

	/**
	 * Check if the compute PSO is precached and if not output information on what state is missing
	 */
	extern ENGINE_API void CheckComputePipelineStateInCache(
		const FComputePipelineStateInitializer& ComputeInitializer,
		EPSOPrecacheResult PSOPrecacheResult,
		const FMaterialRenderProxy* Material,
		int32 PSOCollectorIndex);

	/**
	 * Check if the global compute PSO is precached and if not output information on what state is missing
	 */
	extern ENGINE_API void CheckGlobalComputePipelineStateInCache(const FComputePipelineStateInitializer& ComputeInitializer);

	//UE_DEPRECATED(5.8, "Identifying compute pipelines specifying only a shader is deprecated, use a FComputePipelineStateInitializer.")
	inline void CheckComputePipelineStateInCache(
		FRHIComputeShader& ComputeShader,
		EPSOPrecacheResult PSOPrecacheResult,
		const FMaterialRenderProxy* Material,
		int32 PSOCollectorIndex)
	{
		FComputePipelineStateInitializer ComputeInitializer(&ComputeShader);
		CheckComputePipelineStateInCache(ComputeInitializer, PSOPrecacheResult, Material, PSOCollectorIndex);
	}

	/**
	 * Check if the global graphics PSO is precached and if not output information on what state is missing
	 */
	extern ENGINE_API void CheckGlobalGraphicsPipelineStateInCache(
		const FGraphicsPipelineStateInitializer& Initializer, int32 GlobalPSOCollectorIndex);

	/*
	 * Update the CSV stats	
	 */
	extern ENGINE_API void UpdateCSVStats(EPSOPrecacheResult PSOPrecacheResult);

	/*
	* Update the global tracking stats
	*/
	extern ENGINE_API void UpdateGlobalStats(EPSOPrecacheType PSOPrecacheType, const FPSOPrecacheData& PrecacheData);

	using VertexFactoryCountTableType = Experimental::TRobinHoodHashMap<const FVertexFactoryType*, uint32>;
	struct FShaderStateUsage
	{
		bool bPrecached = false;
		bool bUsed = false;

#if PSO_PRECACHING_TRACKING
		FString MaterialName;
		int32 PSOCollectorIndex = INDEX_NONE;
		const FVertexFactoryType* VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_TRACKING
	};

	/**
	 * Track a PSO precache stat's total count, and optionally also track the counts at
	 * the mesh pass type and vertex factory type granularity.
	 */
	struct FPrecacheUsageData
	{
		FPrecacheUsageData(FName StatFName = FName())
			: StatFName(StatFName)
		{
			Empty();
		}

		uint64 GetTotalCount() const
		{
			return FPlatformAtomics::AtomicRead(&Count);
		}
		uint64 GetTotalMeshPassCount();
		uint64 GetTotalGlobalCount();

		ENGINE_API void Empty();

		ENGINE_API void UpdateStats(EPSOPrecacheType PrecacheType, int32 PSOCollectorIndex, const FVertexFactoryType* VFType);
		
		bool ShouldRecordFullStats(EPSOPrecacheType PrecacheType, int32 PSOCollectorIndex, const FVertexFactoryType* VFType) const
		{
			if (PrecacheType == EPSOPrecacheType::MeshPass)
			{
				return IsFullPrecachingValidationEnabled() && ((PSOCollectorIndex != INDEX_NONE && PSOCollectorIndex < FPSOCollectorCreateManager::MaxPSOCollectorCount) || VFType != nullptr);
			}
			else
			{
				return IsFullPrecachingValidationEnabled() && PSOCollectorIndex != INDEX_NONE && PSOCollectorIndex < FGlobalPSOCollectorManager::MaxPSOCollectorCount;
			}
		}

		volatile int64 Count;

		FName StatFName;

		// Full stats by mesh pass type and by vertex factory type.
		// Only used when the validation mode is set to EPSOPrecacheValidationMode::Full,
		// and when the mesh pass and vertex factory type are known.
		FCriticalSection StatsLock;
		uint32 PerMeshPassCount[FPSOCollectorCreateManager::MaxPSOCollectorCount];
		uint32 UntrackedMeshPassCount;
		VertexFactoryCountTableType PerVertexFactoryCount;
		uint32 PerGlobalTypeCount[FGlobalPSOCollectorManager::MaxPSOCollectorCount];
		uint32 UntrackedGlobalTypeCount;
	};

	/**
	 * Collect stats for different cache hit states
	 */
	struct FPrecacheStats
	{
		FPrecacheStats(FName UntrackedStatFName = FName(), FName MissStatFName = FName(), FName HitStatFName = FName(), FName UsedStatFName = FName(), FName TooLateStatFName = FName())
			: UsageData(UsedStatFName)
			, HitData(HitStatFName)
			, MissData(MissStatFName)
			, TooLateData(TooLateStatFName)
			, UntrackedData(UntrackedStatFName)
		{
		}

		void Empty()
		{
			PrecacheData.Empty();
			UsageData.Empty();
			HitData.Empty();
			MissData.Empty();
			TooLateData.Empty();
			UntrackedData.Empty();
		}

		FPrecacheUsageData PrecacheData;		//< PSOs which have been precached
		FPrecacheUsageData UsageData;			//< PSOs which are used during rendering 
		FPrecacheUsageData HitData;				//< PSOs which are used during rendering and have been successfully precached
		FPrecacheUsageData MissData;			//< PSOs which are used during rendering and have not been precached (but should have been)
		FPrecacheUsageData TooLateData;			//< PSOs which are used during rendering and are still precaching (will cause hitch - component could wait for it to be done)
		FPrecacheUsageData UntrackedData;		//< PSOs which are used during rendering but are currently not precached because for example the MeshPassProcessor or VertexFactory type don't support PSO precaching yet
	};

	class FPrecacheStatsCollector
	{
	public:
		FPrecacheStatsCollector(FName UntrackedStatFName = FName(), FName MissStatFName = FName(), FName HitStatFName = FName(), FName UsedStatFName = FName(), FName TooLateStatFName = FName())
			: Stats(UntrackedStatFName, MissStatFName, HitStatFName, UsedStatFName, TooLateStatFName)
		{
		}

		void ResetStats()
		{
			Stats.Empty();
		}

		FPrecacheStats& GetStats() { return Stats; }
		const FPrecacheStats& GetStats() const { return Stats; }

		template <typename TPrecacheState>
		bool AddStateToCache(EPSOPrecacheType PrecacheType, const TPrecacheState& PrecacheState, uint64 HashFn(const TPrecacheState&), const FMaterial* Material, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType)
		{
			if (!IsPrecachingValidationEnabled())
			{
				return false;
			}

			uint64 PrecacheStateHash = HashFn(PrecacheState);

			// Only update stats once per state.
			bool bUpdateStats = false;
			{
				FScopeLock Lock(&StateMapLock);

				FShaderStateUsage* Value = HashedStateMap.FindOrAdd(PrecacheStateHash, FShaderStateUsage());
				if (!Value->bPrecached)
				{
					Value->bPrecached = true;
					bUpdateStats = true;

#if PSO_PRECACHING_TRACKING
					Value->MaterialName = Material ? Material->GetAssetName() : FString("Unknown");
					Value->PSOCollectorIndex = PSOCollectorIndex;
					Value->VertexFactoryType = VertexFactoryType;
#endif //PSO_PRECACHING_TRACKING
				}
			}

			if (bUpdateStats)
			{
				Stats.PrecacheData.UpdateStats(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			}

			return bUpdateStats;
		}

		template <typename TPrecacheState>
		bool CheckStateInCache(EPSOPrecacheType PrecacheType, const TPrecacheState& PrecacheState, uint64 HashFn(const TPrecacheState&), int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType, EPSOPrecacheResult& InOutPrecacheResult)
		{
			if (!IsPrecachingValidationEnabled())
			{
				InOutPrecacheResult = EPSOPrecacheResult::Unknown;
				return false;
			}

			uint64 PrecacheStateHash = HashFn(PrecacheState);
			return CheckStateInCacheByHash(PrecacheType, PrecacheStateHash, PSOCollectorIndex, VertexFactoryType, InOutPrecacheResult);
		}

		bool CheckStateInCacheByHash(EPSOPrecacheType PrecacheType, const uint64 PrecacheStateHash, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType, EPSOPrecacheResult& InOutPrecacheResult)
		{
			if (!IsPrecachingValidationEnabled())
			{
				InOutPrecacheResult = EPSOPrecacheResult::Unknown;
				return false;
			}

			bool bTracked = IsStateTracked(PrecacheType, PSOCollectorIndex, VertexFactoryType);
			return UpdatePrecacheStats(PrecacheType, PrecacheStateHash, PSOCollectorIndex, VertexFactoryType, bTracked, InOutPrecacheResult);
		}

		ENGINE_API bool IsPrecached(uint64 PrecacheStateHash);
		ENGINE_API bool IsUsed(uint64 PrecacheStateHash);
#if PSO_PRECACHING_TRACKING
		bool GetPrecacheData(uint64 PrecacheStateHash, FString& OutMaterialName, int32& OutPSOCollectorIndex, const FVertexFactoryType*& OutVertexFactoryType);
#endif

		static ENGINE_API bool IsStateTracked(EPSOPrecacheType PrecacheType, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType);

	private:

		ENGINE_API bool UpdatePrecacheStats(EPSOPrecacheType PrecacheType, uint64 PrecacheHash, int32 PSOCollectorIndex, const FVertexFactoryType* VertexFactoryType, bool bTracked, EPSOPrecacheResult& InOutPrecacheResult);

		FPrecacheStats Stats;

		FCriticalSection StateMapLock;
		Experimental::TRobinHoodHashMap<uint64, FShaderStateUsage> HashedStateMap;
	};

	extern ENGINE_API FPrecacheStatsCollector& GetShadersOnlyPSOPrecacheStatsCollector();
	extern ENGINE_API FPrecacheStatsCollector& GetMinimalPSOPrecacheStatsCollector();
	extern ENGINE_API FPrecacheStatsCollector& GetFullPSOPrecacheStatsCollector();

	extern ENGINE_API void DumpPSOPrecacheValidationStats();
}

#endif // PSO_PRECACHING_VALIDATE

#endif // UE_WITH_PSO_PRECACHING