// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PackedLevelActor/PackedLevelActorTypes.h"

class APackedLevelActor;
class AActor;
class UActorComponent;
class FPackedLevelActorBuilderContext;
class FPackedLevelActorBuilder;

class IPackedLevelActorBuilder
{
public:
	IPackedLevelActorBuilder(FPackedLevelActorBuilder& InOwner) : Owner(InOwner) {}
	virtual ~IPackedLevelActorBuilder() = default;
	virtual FPackedLevelActorBuilderID GetID() const = 0;
	virtual void GetPackClusters(FPackedLevelActorBuilderContext& InContext, AActor* InActor) const = 0;
	virtual uint32 PackActors(FPackedLevelActorBuilderContext& InBuilder, const FPackedLevelActorBuilderClusterID& InClusterID, const TArray<UActorComponent*>& InComponents) const = 0;
protected:
	FPackedLevelActorBuilder& Owner;
};

#endif
