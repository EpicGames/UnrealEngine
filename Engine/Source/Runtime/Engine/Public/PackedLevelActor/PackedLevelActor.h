// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "LevelInstance/LevelInstanceActor.h"

#if WITH_EDITOR
#include "Kismet2/ComponentEditorUtils.h"
#endif

#include "PackedLevelActor.generated.h"

class UInstancedStaticMeshComponent;
class UBlueprint;

/**
 * APackedLevelActor is the result of packing the source level (WorldAsset base class property) into a single actor. See FPackedLevelActorBuilder.
 * 
 * 
 * Other components are unsupported and will result in an incomplete APackedLevelActor. In this case using a regular ALevelInstance is recommended.
 */
UCLASS(MinimalAPI)
class APackedLevelActor : public ALevelInstance
{
	GENERATED_BODY()

public:
	ENGINE_API APackedLevelActor();

	ENGINE_API virtual bool IsLoadingEnabled() const override;

	ENGINE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	ENGINE_API virtual EWorldPartitionActorFilterType GetDetailsFilterTypes() const override;
	ENGINE_API virtual EWorldPartitionActorFilterType GetLoadingFilterTypes() const override;
	ENGINE_API virtual void OnFilterChanged() override;
	void SetShouldLoadForPacking(bool bInLoadForPacking) { bLoadForPacking = bInLoadForPacking; }
	ENGINE_API bool ShouldLoadForPacking() const;
	// When Loading a APackedLevelActor it needs to be fully loaded for packing.
	virtual bool SupportsPartialEditorLoading() const override { return false; }
	virtual bool SupportsPropertyOverrides() const override { return false; }

	ENGINE_API virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	
	UE_DEPRECATED(5.3, "Use FPackedLevelActorUtils::CreateOrUpdateBlueprint instead")
	static bool CreateOrUpdateBlueprint(ALevelInstance* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true) { return false;}
	UE_DEPRECATED(5.3, "Use FPackedLevelActorUtils::CreateOrUpdateBlueprint instead")
	static bool CreateOrUpdateBlueprint(TSoftObjectPtr<UWorld> InWorldAsset, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true) { return false;}
	UE_DEPRECATED(5.3, "Use FPackedLevelActorUtils::UpdateBlueprint instead")
	static void UpdateBlueprint(UBlueprint* InBlueprint, bool bCheckoutAndSave = true) {}

	static ENGINE_API FName GetPackedComponentTag();

	ENGINE_API virtual void PostLoad() override;
	ENGINE_API virtual void UpdateLevelInstanceFromWorldAsset() override;
	ENGINE_API virtual void OnCommit(bool bChanged) override;
	ENGINE_API virtual void OnCommitChild(bool bChanged) override;
	ENGINE_API virtual void OnEdit() override;
	ENGINE_API virtual void OnEditChild() override;

	ENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
		
	void SetPackedVersion(const FGuid& InVersion) { PackedVersion = InVersion; }

	uint32 GetPackedHash() const { return PackedHash; }
	void SetPackedHash(uint32 InHash) { PackedHash = InHash; }

	ENGINE_API virtual bool IsHiddenEd() const override;

	ENGINE_API void DestroyPackedComponents();
	ENGINE_API void GetPackedComponents(TArray<UActorComponent*>& OutPackedComponents) const;

	virtual ELevelInstanceRuntimeBehavior GetDefaultRuntimeBehavior() const override { return ELevelInstanceRuntimeBehavior::None; }

	ENGINE_API virtual void RerunConstructionScripts() override;

	static ENGINE_API bool IsRootBlueprint(UClass* InClass);
	ENGINE_API bool IsRootBlueprintTemplate() const;
	ENGINE_API UBlueprint* GetRootBlueprint() const;

	template<class T>
	T* AddPackedComponent(TSubclassOf<T> ComponentClass)
	{
		Modify();
		FName NewComponentName = *FComponentEditorUtils::GenerateValidVariableName(ComponentClass, this);
		T* NewComponent = NewObject<T>(this, ComponentClass, NewComponentName, RF_Transactional);
		AddInstanceComponent(NewComponent);
		NewComponent->ComponentTags.Add(GetPackedComponentTag());
		return NewComponent;
	}
#endif

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	FGuid PackedVersion;

	UPROPERTY()
	uint32 PackedHash;
#endif

#if WITH_EDITOR
	bool ShouldCookWorldAsset() const override { return false; }

	bool bChildChanged;
	bool bLoadForPacking;
#endif
};


DEFINE_ACTORDESC_TYPE(APackedLevelActor, FPackedLevelActorDesc);
