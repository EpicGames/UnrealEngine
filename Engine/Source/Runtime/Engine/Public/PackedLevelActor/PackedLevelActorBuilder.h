// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PackedLevelActor/PackedLevelActorTypes.h"
#include "PackedLevelActor/IPackedLevelActorBuilder.h"
#include "UObject/SoftObjectPtr.h"
#include "Containers/Set.h"
#include "PreviewScene.h"

class ILevelInstanceInterface;
class APackedLevelActor;
class ALevelInstance;
class AActor;
class UActorComponent;
class UBlueprint;
class FMessageLog;
struct FWorldPartitionActorFilter;

/**
 * FPackedLevelActorBuilder handles packing of ALevelInstance actors into APackedLevelActor actors and Blueprints.
 */
class FPackedLevelActorBuilder
{
public:
	ENGINE_API FPackedLevelActorBuilder();
	
	static ENGINE_API TSharedPtr<FPackedLevelActorBuilder> CreateDefaultBuilder();
	
	/* Packs InPackedLevelActor using InLevelInstanceToPack as its source level actor */
	ENGINE_API bool PackActor(APackedLevelActor* InPackedLevelActor, ILevelInstanceInterface* InLevelInstanceToPack);
	/* Packs InPackedLevelActor using itself as the source level actor */
	ENGINE_API bool PackActor(APackedLevelActor* InPackedLevelActor);
	/* Packs InPackedLevelActor using InWorldAsset as the source level */
	ENGINE_API bool PackActor(APackedLevelActor* InPackedLevelActor, TSoftObjectPtr<UWorld> InWorldAsset);

	/* Creates/Updates a APackedLevelActor Blueprint from InLevelInstance (will overwrite existing asset or show a dialog if InBlueprintAsset is null) */
	ENGINE_API bool CreateOrUpdateBlueprint(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true);
	/* Creates/Updates a APackedLeveInstance Blueprint from InWorldAsset (will overwrite existing asset or show a dialog if InBlueprintAsset is null) */
	ENGINE_API bool CreateOrUpdateBlueprint(TSoftObjectPtr<UWorld> InWorldAsset, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave = true, bool bPromptForSave = true);
	/* Update existing Blueprint */
	ENGINE_API void UpdateBlueprint(UBlueprint* Blueprint, bool bCheckoutAndSave = true);

	static ENGINE_API const FString& GetPackedBPPrefix();
	/* Creates a new APackedLevelActor Blueprint using InPackagePath/InAssetName as hint for path. Prompts the user to input the final asset name. */
	static ENGINE_API UBlueprint* CreatePackedLevelActorBlueprintWithDialog(TSoftObjectPtr<UBlueprint> InBlueprintAsset, TSoftObjectPtr<UWorld> InWorldAsset, bool bInCompile);
	/* Creates a new APackedLevelActor Blueprint using InPackagePath/InAssetName */
	static ENGINE_API UBlueprint* CreatePackedLevelActorBlueprint(TSoftObjectPtr<UBlueprint> InBlueprintAsset, TSoftObjectPtr<UWorld> InWorldAsset, bool bInCompile);
	
	template<class T>
	void AddBuilder()
	{
		check(!Builders.Contains(T::BuilderID));
		Builders.Add(T::BuilderID, MakeUnique<T>(*this));
	}

	void ClusterActor(FPackedLevelActorBuilderContext& InContext, AActor* InActor);
private:
	ENGINE_API bool PackActor(FPackedLevelActorBuilderContext& InContext);

	/* Create/Updates a APackedLevelActor Blueprint from InPackedActor (will overwrite existing asset or show a dialog if InBlueprintAsset is null) */
	ENGINE_API bool CreateOrUpdateBlueprintFromPacked(APackedLevelActor* InPackedActor, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave);
	/* Creates/Updates a APackedLevelActor Blueprint from InLevelInstance (will overwrite existing asset or show a dialog if InBlueprintAsset is null) */
	ENGINE_API bool CreateOrUpdateBlueprintFromUnpacked(ILevelInstanceInterface* InLevelInstance, TSoftObjectPtr<UBlueprint> InBlueprintAsset, bool bCheckoutAndSave, bool bPromptForSave);
	/* Creates and loads a ALevelInstance so it can be used for packing */
	ENGINE_API ALevelInstance* CreateTransientLevelInstanceForPacking(TSoftObjectPtr<UWorld> InWorldAsset, const FVector& InLocation, const FRotator& InRotator, const FWorldPartitionActorFilter& InFilter);

	FPackedLevelActorBuilder(const FPackedLevelActorBuilder&) = delete;
	FPackedLevelActorBuilder& operator=(const FPackedLevelActorBuilder&) = delete;

public:
	friend class FPackedLevelActorBuilderContext;
		
private:
	
	TSet<UClass*> ClassDiscards;
	TMap<FPackedLevelActorBuilderID, TUniquePtr<IPackedLevelActorBuilder>> Builders;
	FPreviewScene PreviewScene;
};


class FPackedLevelActorBuilderContext
{
public:
	FPackedLevelActorBuilderContext(APackedLevelActor* InPackedLevelActor, ILevelInstanceInterface* InLevelInstanceToPack, const TSet<UClass*>& InClassDiscards) 
		: ClassDiscards(InClassDiscards), PackedLevelActor(InPackedLevelActor), LevelInstanceToPack(InLevelInstanceToPack), RelativePivotTransform(FTransform::Identity) {}

	/* Interface for IPackedLevelActorBuilder's to use */
	void FindOrAddCluster(FPackedLevelActorBuilderClusterID&& InClusterID, UActorComponent* InComponent = nullptr);
	void DiscardActor(AActor* InActor);
	void Report(FMessageLog& Log) const;

	const TMap<FPackedLevelActorBuilderClusterID, TArray<UActorComponent*>>& GetClusters() const { return Clusters; }
	
	void SetRelativePivotTransform(const FTransform& InRelativePivotTransform) { RelativePivotTransform = InRelativePivotTransform; }
	const FTransform GetRelativePivotTransform() const { return RelativePivotTransform; }

	bool ShouldPackComponent(UActorComponent* InActorComponent) const;

	APackedLevelActor* GetPackedLevelActor() const { return PackedLevelActor; }
	ILevelInstanceInterface* GetLevelInstanceToPack() const { return LevelInstanceToPack; }
private:
	const TSet<UClass*>& ClassDiscards;

	APackedLevelActor* PackedLevelActor;
	ILevelInstanceInterface* LevelInstanceToPack;
	
	TMap<FPackedLevelActorBuilderClusterID, TArray<UActorComponent*>> Clusters;

	TMap<AActor*, TSet<UActorComponent*>> PerActorClusteredComponents;
	TSet<AActor*> ActorDiscards;

	FTransform RelativePivotTransform;

	friend FPackedLevelActorBuilder;
};

#endif
