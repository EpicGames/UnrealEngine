// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Crc.h"
#include "Logging/LogMacros.h"

using FPackedLevelActorBuilderID = uint32;

class FPackedLevelActorBuilderCluster
{
	FPackedLevelActorBuilderID BuilderID;

public:
	FPackedLevelActorBuilderCluster(FPackedLevelActorBuilderID InBuilderID)
		: BuilderID(InBuilderID) {}
	virtual ~FPackedLevelActorBuilderCluster() = default;

	FPackedLevelActorBuilderID GetBuilderID() const { return BuilderID; }

	virtual uint32 ComputeHash() const
	{
		return FCrc::TypeCrc32(BuilderID);
	}

	virtual bool Equals(const FPackedLevelActorBuilderCluster& Other) const
	{
		return BuilderID == Other.BuilderID;
	}

private:
	FPackedLevelActorBuilderCluster(const FPackedLevelActorBuilderCluster&) = delete;
	FPackedLevelActorBuilderCluster& operator=(const FPackedLevelActorBuilderCluster&) = delete;
};

class FPackedLevelActorBuilderClusterID
{
public:
	static ENGINE_API FPackedLevelActorBuilderClusterID Invalid;

	FPackedLevelActorBuilderClusterID() : Hash(0)
	{
	}

	FPackedLevelActorBuilderClusterID(FPackedLevelActorBuilderClusterID&& Other)
		: Hash(Other.Hash), Data(MoveTemp(Other.Data))
	{
	}

	FPackedLevelActorBuilderClusterID(TUniquePtr<FPackedLevelActorBuilderCluster>&& InData)
		: Data(MoveTemp(InData))
	{
		Hash = Data->ComputeHash();
	}
		
	bool operator==(const FPackedLevelActorBuilderClusterID& Other) const
	{
		return (Hash == Other.Hash) && Data->Equals(*Other.Data);
	}

	bool operator!=(const FPackedLevelActorBuilderClusterID& Other) const
	{
		return !(*this == Other);
	}

	uint32 GetHash() const { return Hash; }

	FPackedLevelActorBuilderID GetBuilderID() const { return Data->GetBuilderID(); }
		
	friend uint32 GetTypeHash(const FPackedLevelActorBuilderClusterID& ID)
	{
		return ID.GetHash();
	}

	FPackedLevelActorBuilderCluster* GetData() const
	{
		return Data.Get();
	}

private:
	FPackedLevelActorBuilderClusterID(const FPackedLevelActorBuilderClusterID&) = delete;
	FPackedLevelActorBuilderClusterID& operator=(const FPackedLevelActorBuilderClusterID&) = delete;

	uint32 Hash;
	TUniquePtr<FPackedLevelActorBuilderCluster> Data;
};

#endif
