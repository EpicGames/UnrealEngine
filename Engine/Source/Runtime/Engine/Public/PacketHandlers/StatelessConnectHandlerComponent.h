// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PacketHandler.h"
#include "IPAddress.h"


DECLARE_LOG_CATEGORY_EXTERN(LogHandshake, Log, All);


// Forward Declarations
class UNetConnection;
class UNetDriver;
enum class EEngineNetworkRuntimeFeatures : uint16;


// Defines
#define SECRET_BYTE_SIZE 64
#define SECRET_COUNT 2
#define COOKIE_BYTE_SIZE 20


namespace UE::Net
{
	/**
	 * The different handshake protocol versions
	 */
	enum class EHandshakeVersion : uint8
	{
		Original				= 0,	// The original/unversioned handshake protocol
		Randomized				= 1,	// The version of the handshake protocol with randomization, versioning and debug/diagnostic tweaks
		NetCLVersion			= 2,	// Added Network CL version, for optional extra-early client rejection

		/** Net compatibility break */

		SessionClientId			= 3,	// Added SessionId (index of server game session) and ClientId (clientside index of NetConnection)
		NetCLUpgradeMessage		= 4,	// Added stateless handshake level NMT_Upgrade message, to allow Network CL checks to be permanently enabled

		Latest					= NetCLUpgradeMessage
	};

	/** List of EHandshakeVersion's which break net compatibility (sorted from earliest to latest) */
	static constexpr EHandshakeVersion HandshakeCompatibilityBreaks[] = { EHandshakeVersion::SessionClientId };


	/**
	 * The different handshake packet types
	 */
	enum class EHandshakePacketType : uint8
	{
		InitialPacket		= 0,
		Challenge			= 1,
		Response			= 2,
		Ack					= 3,
		RestartHandshake	= 4,
		RestartResponse		= 5,
		VersionUpgrade		= 6,

		Last = VersionUpgrade
	};

	/**
	 * Flags/modifiers affecting how handshake packets are handled (e.g. restart handshake packets)
	 */
	enum class EHandshakePacketModifier : uint8
	{
		None				= 0x00,
		RestartHandshake	= 0x01		// Restart handshake packet
	};

	ENUM_CLASS_FLAGS(EHandshakePacketModifier);


	/**
	 * Reasons for handshake failures
	 */
	enum class EHandshakeFailureReason
	{
		None,
		WrongVersion		// Client/Server have incompatible versions
	};

	/**
	 * Contains the information necessary for processing a failed stateless handshake, on the Game Thread.
	 * Designed to be sent from the Receive Thread, to the Game Thread.
	 */
	struct FStatelessHandshakeFailureInfo
	{
		/** The reason the handshake failed */
		EHandshakeFailureReason FailureReason = EHandshakeFailureReason::None;

		/** The Network CL version of the remote side */
		uint32 RemoteNetworkVersion = 0;

		/** The net runtime features of the remote side */
		EEngineNetworkRuntimeFeatures RemoteNetworkFeatures;


	public:
		FStatelessHandshakeFailureInfo();
	};

	/**
	 * Callback function for handling a failed stateless handshake. Triggered when processing incoming packets.
	 *
	 * @param HandshakeFailureInfo		Information about the handshake failure.
	 */
	using FHandshakeFailureFunc = TUniqueFunction<void(FStatelessHandshakeFailureInfo HandshakeFailureInfo)>;
}


/**
 * PacketHandler component for implementing a stateless (non-memory-consuming) connection handshake
 *
 * Partially based on the Datagram Transport Layer Security protocol.
 */
class StatelessConnectHandlerComponent : public HandlerComponent
{
	using EHandshakeVersion = UE::Net::EHandshakeVersion;
	using EHandshakePacketType = UE::Net::EHandshakePacketType;
	using EHandshakePacketModifier = UE::Net::EHandshakePacketModifier;
	using FHandshakeFailureFunc = UE::Net::FHandshakeFailureFunc;

private:
	/** Common parameters for all 'Send*' functions which send packets to clients */
	struct FCommonSendToClientParams
	{
		/** The address of the client to send the packet to */
		const TSharedPtr<const FInternetAddr>& ClientAddress;
		
		/** The handshake format version to use when sending */
		EHandshakeVersion HandshakeVersion;

		/** The client-specified connection id */
		uint32 ClientID;


	public:
		FCommonSendToClientParams(const TSharedPtr<const FInternetAddr>& InClientAddress, EHandshakeVersion InHandshakeVersion, uint32 InClientID);
	};

public:
	/**
	 * Base constructor
	 */
	ENGINE_API StatelessConnectHandlerComponent();

	ENGINE_API virtual void CountBytes(FArchive& Ar) const override;

	virtual bool IsValid() const override { return true; }

	ENGINE_API virtual void NotifyHandshakeBegin() override;

	/**
	 * Initializes a serverside UNetConnection-associated StatelessConnect,
	 * from the connectionless StatelessConnect that negotiated the handshake.
	 *
	 * @param InConnectionlessHandler	The connectionless StatelessConnect we're initializing from
	 */
	ENGINE_API void InitFromConnectionless(StatelessConnectHandlerComponent* InConnectionlessHandler);

	/**
	 * Set a callback for notifying of handshake failure (clientside only).
	 *
	 * @param InHandshakeFailureFunc	The callback to use for notification.
	 */
	ENGINE_API void SetHandshakeFailureCallback(FHandshakeFailureFunc&& InHandshakeFailureFunc);

	/**
	 * Whether or not this HandlerComponent supports running its Outgoing logic on multiple separate connections simultaneously.
	 * Any accesses on a per-connection basis are safe, but writing to Global/NetDriver properties are not
	 *
	 * @return	Whether or not the above is supported
	 */
	virtual bool SupportsParallelConnectionOutgoing() const override;

private:
	/**
	 * Constructs and sends the initial handshake packet, from the client to the server
	 *
	 * @param HandshakeVersion	The handshake format version to use when sending
	 */
	void SendInitialPacket(EHandshakeVersion HandshakeVersion);

	/**
	 * Constructs and sends the server response to the initial connect packet, from the server to the client.
	 *
	 * @param CommonParams						Common parameters for 'Send*' functions to clients
	 * @param ClientSentHandshakePacketCount	The number of handshake packets the client has sent (for debugging/packet-analysis)
	 */
	void SendConnectChallenge(FCommonSendToClientParams CommonParams, uint8 ClientSentHandshakePacketCount);

	/**
	 * Constructs and sends the handshake challenge response packet, from the client to the server
	 *
	 * @param HandshakeVersion	The handshake format version to use when sending
	 * @param InSecretId		Which of the two server HandshakeSecret values this uses
	 * @param InTimestamp		The timestamp value to send
	 * @param InCookie			The cookie value to send
	 */
	void SendChallengeResponse(EHandshakeVersion HandshakeVersion, uint8 InSecretId, double InTimestamp, uint8 InCookie[COOKIE_BYTE_SIZE]);

	/**
	 * Constructs and sends the server ack to a successful challenge response, from the server to the client.
	 *
	 * @param CommonParams						Common parameters for 'Send*' functions to clients
	 * @param ClientSentHandshakePacketCount	The number of handshake packets the client has sent (for debugging/packet-analysis)
	 * @param InCookie							The cookie value to send
	 */
	void SendChallengeAck(FCommonSendToClientParams CommonParams, uint8 ClientSentHandshakePacketCount, uint8 InCookie[COOKIE_BYTE_SIZE]);

	/**
	 * Constructs and sends a request to resend the cookie, from the server to the client.
	 *
	 * @param CommonParams	Common parameters for 'Send*' functions to clients
	 */
	void SendRestartHandshakeRequest(FCommonSendToClientParams CommonParams);

	/**
	 * Constructs and sends a stateless handshake level NMT_Upgrade message
	 *
	 * @param CommonParams	Common parameters for 'Send*' functions to clients
	 */
	void SendVersionUpgradeMessage(FCommonSendToClientParams CommonParams);


	/**
	 * Writes out the base/common parameters for all handshake packets, both to server and client
	 *
	 * @param HandshakePacket							The handshake packet being written.
	 * @param HandshakePacketType						The type of handshake packet.
	 * @param HandshakeVersion							The handshake format version to use when sending.
	 * @param SentHandshakePacketCount_LocalOrRemote	The number of handshake packets sent (for debugging/packet-analysis).
	 * @param ClientID									The client-specified connection id.
	 * @param HandshakePacketModifier					Flags/modifiers affecting how the packet will be handled.
	 */
	void BeginHandshakePacket(FBitWriter& HandshakePacket, EHandshakePacketType HandshakePacketType, EHandshakeVersion HandshakeVersion,
								uint8 SentHandshakePacketCount_LocalOrRemote, uint32 ClientID,
								EHandshakePacketModifier HandshakePacketModifier=EHandshakePacketModifier::None);


	/**
	 * Sends a packet from the client to the server.
	 *
	 * @param HandshakeVersion		The handshake format version to use when sending.
	 * @param PacketType			The type of handshake packet.
	 * @param Packet				The handshake packet being written.
	 */
	void SendToServer(EHandshakeVersion HandshakeVersion, EHandshakePacketType PacketType, FBitWriter& Packet);

	/**
	 * Sends a packet from the server to a client.
	 *
	 * @param CommonParams			Common parameters for 'Send*' functions to clients.
	 * @param PacketType			The type of handshake packet.
	 * @param Packet				The handshake packet being written.
	 */
	void SendToClient(FCommonSendToClientParams CommonParams, EHandshakePacketType PacketType, FBitWriter& Packet);


	/**
	 * Pads the handshake packet, to match the PacketBitAlignment of the PacketHandler, so that it will parse correctly.
	 *
	 * @param HandshakePacket	The handshake packet to be aligned.
	 * @param HandshakeVersion	The handshake format version to use when sending
	 */
	void CapHandshakePacket(FBitWriter& HandshakePacket, EHandshakeVersion HandshakeVersion);

public:
	/**
	 * Whether or not the specified connection address, has just passed the connection handshake challenge.
	 *
	 * @param Address					The address (including port, for UIpNetDriver) being checked
	 * @param bOutRestartedHandshake	Whether or not the passed challenge, was a restarted handshake for an existing NetConnection
	 */
	inline bool HasPassedChallenge(const TSharedPtr<const FInternetAddr>& Address, bool& bOutRestartedHandshake) const
	{
		bOutRestartedHandshake = bRestartedHandshake;

		return (LastChallengeSuccessAddress.IsValid() && Address.IsValid()) ? *LastChallengeSuccessAddress == *Address : false;
	}


	/**
	 * Used for retrieving the initial packet sequence values from the handshake data, after a successful challenge
	 *
	 * @param OutServerSequence		Outputs the initial sequence for the server
	 * @param OutClientSequence		Outputs the initial sequence for the client
	 */
	inline void GetChallengeSequence(int32& OutServerSequence, int32& OutClientSequence) const
	{
		OutServerSequence = LastServerSequence;
		OutClientSequence = LastClientSequence;
	}

	/**
	 * When a restarted handshake is completed, this is used to match it up with the existing NetConnection
	 *
	 * @param NetConnComponent	The NetConnection StatelessConnectHandlerComponent, which is being checked for a match
	 * @return					Whether or not the specified component, belongs to the client which restarted the handshake
	 */
	inline bool DoesRestartedHandshakeMatch(StatelessConnectHandlerComponent& NetConnComponent) const
	{
		return FMemory::Memcmp(AuthorisedCookie, NetConnComponent.AuthorisedCookie, COOKIE_BYTE_SIZE) == 0;
	}

	/**
	 * Used to reset cached handshake success/fail state, when done processing it
	 */
	inline void ResetChallengeData()
	{
		LastChallengeSuccessAddress = nullptr;
		bRestartedHandshake = false;
		LastServerSequence = 0;
		LastClientSequence = 0;
		FMemory::Memzero(AuthorisedCookie, COOKIE_BYTE_SIZE);
	}


	/**
	 * Sets the net driver this handler is associated with
	 *
	 * @param InDriver	The net driver to set
	 */
	ENGINE_API void SetDriver(UNetDriver* InDriver);


protected:
	ENGINE_API virtual void Initialize() override;

	ENGINE_API virtual void Incoming(FBitReader& Packet) override;

	ENGINE_API virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;

	ENGINE_API virtual void IncomingConnectionless(FIncomingPacketRef PacketRef) override;

	virtual bool CanReadUnaligned() const override
	{
		return true;
	}

	ENGINE_API virtual int32 GetReservedPacketBits() const override;

	ENGINE_API virtual void Tick(float DeltaTime) override;

private:

	/**
	 * Handshake data parsed from a packet
	 */
	struct FParsedHandshakeData
	{
		/** The minimum supported handshake protocol version of the remote side */
		EHandshakeVersion RemoteMinVersion = EHandshakeVersion::Latest;

		/** The current handshake protocol version the remote side used for communication */
		EHandshakeVersion RemoteCurVersion = EHandshakeVersion::Latest;

		/** The Network CL version of the remote side */
		uint32 RemoteNetworkVersion = 0;

		/** The net runtime features of the remote side */
		EEngineNetworkRuntimeFeatures RemoteNetworkFeatures;

		/** The type of handshake packet */
		UE::Net::EHandshakePacketType HandshakePacketType = UE::Net::EHandshakePacketType::InitialPacket;

		/** The remote value of 'SentHandshakePacketCount' */
		uint8 RemoteSentHandshakePacketCount = 0;

		/** Whether or not this packet is a restart handshake packet */
		bool bRestartHandshake = false;

		/** Which of the two serverside HandshakeSecret values this is based on */
		uint8 SecretId = 0;

		/** The server timestamp, from the moment the challenge was sent (or 0.f if from the client) */
		double Timestamp = 0.0;

		/** A unique identifier, generated by the server, which the client must reply with (or 0, for initial packet) */
		uint8 Cookie[COOKIE_BYTE_SIZE] = {};

		/** If this is a restart handshake challenge response, this is the original handshake's cookie */
		uint8 OrigCookie[COOKIE_BYTE_SIZE] = {};


	public:
		FParsedHandshakeData();
	};


	/**
	 * Parses an incoming handshake packet (does not parse the handshake bit though)
	 *
	 * @param Packet		The packet the handshake is being parsed from
	 * @param OutResult		The parsed handshake data, if successful
	 * @return				Whether or not the handshake packet was parsed successfully
	 */
	bool ParseHandshakePacket(FBitReader& Packet, FParsedHandshakeData& OutResult) const;

	bool ParseHandshakePacketOriginal(FBitReader& Packet, FParsedHandshakeData& OutResult) const;

	/**
	 * Checks the handshake protocol version and Network CL version for incoming handshake packets, serverside
	 *
	 * @param HandshakeData		The packet handshake data
	 * @param OutTargetVersion	The target handshake protocol version to use, for the new connection
	 * @return					Whether or not the checks successfully validated that the version is correct, and that communication can continue
	 */
	bool CheckVersion(const FParsedHandshakeData& HandshakeData, EHandshakeVersion& OutTargetVersion) const;

	/**
	 * Takes the client address plus server timestamp, and outputs a deterministic cookie value
	 *
	 * @param ClientAddress		The address of the client, including port
	 * @param SecretId			Which of the two HandshakeSecret values the cookie will be based on
	 * @param TimeStamp			The serverside timestamp
	 * @param OutCookie			Outputs the generated cookie value.
	 */
	void GenerateCookie(const TSharedPtr<const FInternetAddr>& ClientAddress, uint8 SecretId, double Timestamp, uint8 (&OutCookie)[COOKIE_BYTE_SIZE]) const;

	/**
	 * Generates a new HandshakeSecret value
	 */
	void UpdateSecret();

	/**
	 * Calculate the adjusted predefined packet size, based on MagicHeader
	 */
	int32 GetAdjustedSizeBits(int32 InSizeBits, EHandshakeVersion HandshakeVersion) const;

#if !UE_BUILD_SHIPPING
	/**
	 * Tracks logs for invalid Session ID's, Connection ID's or Magic Header's, and limits the number of these logs
	 *
	 * @return		Whether or not logging is currently allowed
	 */
	bool TrackValidationLogs();
#endif


private:
	/** The net driver associated with this handler - for performing connectionless sends */
	UNetDriver* Driver;

	/** Cache memory of previously used packets to avoid memory allocations */
	FBitWriter OldPacket;

	/** Serverside variables */

	/** The serverside-only 'secret' value, used to help with generating cookies. */
	TArray<uint8> HandshakeSecret[SECRET_COUNT];

	/** Which of the two secret values above is active (values are changed frequently, to limit replay attacks) */
	uint8 ActiveSecret;

	/** The time of the last secret value update */
	double LastSecretUpdateTimestamp;

	/** The time of the last post-handshake initialization */
	double LastInitTimestamp = 0.0;

	/** The last address to successfully complete the handshake challenge */
	TSharedPtr<const FInternetAddr> LastChallengeSuccessAddress;

	/** The initial server sequence value, from the last successful handshake */
	int32 LastServerSequence;

	/** The initial client sequence value, from the last successful handshake */
	int32 LastClientSequence;

	/** The minimum supported client version, for all connections - for setting minimum bounds for the restart handshake (not aware of version) */
	EHandshakeVersion MinClientHandshakeVersion;

#if !UE_BUILD_SHIPPING
	/** The last time an 'invalid SessionId/ClientId/MagicHeader' log period began */
	double LastValidationLogPeriodStart = 0.0;

	/** The number of 'invalid SessionId/ClientId/MagicHeader' logs in the current log period (for logspam limiting) */
	uint32 ValidationLogCounter = 0;
#endif

	/** The last time an 'upgrade message' send period began */
	double LastUpgradeMessagePeriodStart = 0.0;

	/** The number of upgrade messages sent in the current send period (for rate limiting) */
	uint32 UpgradeMessageCounter = 0;

	/** Callback for processing handshake failures */
	FHandshakeFailureFunc HandshakeFailureCallback;


	/** Clientside variables */

	/** The last time a handshake packet was sent - used for detecting failed sends. */
	double LastClientSendTimestamp;


	/** The local (client) time at which the challenge was last updated */
	double LastChallengeTimestamp;

	/** The local (client) time at which the last restart handshake request was received */
	double LastRestartPacketTimestamp;

	/** The SecretId value of the last challenge response sent */
	uint8 LastSecretId;

	/** The Timestamp value of the last challenge response sent */
	double LastTimestamp;

	/** The Cookie value of the last challenge response sent. Will differ from AuthorisedCookie, if a handshake retry is triggered. */
	uint8 LastCookie[COOKIE_BYTE_SIZE];


	/** Both Serverside and Clientside variables */

	/** Client: Whether or not we are in the middle of a restarted handshake. Server: Whether or not the last handshake was a restarted handshake. */
	bool bRestartedHandshake;

	/** The cookie which completed the connection handshake. */
	uint8 AuthorisedCookie[COOKIE_BYTE_SIZE];

	/** The magic header which is prepended to all packets */
	TBitArray<> MagicHeader;

	/** Integer representation of the magic header, for direct comparison */
	uint32 MagicHeaderUint = 0;

	/** The number of sent handshake packets, added to packets to aid packet dump debugging (may roll over) */
	uint8 SentHandshakePacketCount = 0;

	/** The last remote handshake version sent by the client/server */
	EHandshakeVersion LastRemoteHandshakeVersion;

	/** Cached version of the serverside UEngine.GlobalNetTravelCount value, for writing session id's */
	uint32 CachedGlobalNetTravelCount = 0;

	/** Cached version of the clientside per-NetDriverDefinition 'ClientID' value, for writing client connection id's */
	uint32 CachedClientID = 0;

	/** The size of the session id in packets (WARNING: Adjusting this is a net compatibility break) */
	static constexpr uint32 SessionIDSizeBits = 2;

	/** The size of the connection id in packets (WARNING: Adjusting this is a net compatibility break) */
	static constexpr uint32 ClientIDSizeBits = 3;
};

