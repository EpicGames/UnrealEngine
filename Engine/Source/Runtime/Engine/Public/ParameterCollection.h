// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParameterCollection.h: 
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "RHI.h"

/** 
 * Maximum number of parameter collections referenced by a material.  
 * Limited to a fairly low count for now, can be raised later.  
 * D3D11 allows 15 constant buffers per shader, but many are used by code, and there's state setting overhead to setup each one per material being drawn.
 */
static const uint32 MaxNumParameterCollectionsPerMaterial = 2;

/** 
 * Rendering thread mirror of UMaterialParameterCollectionInstance. 
 * Stores data needed to render a material that references a UMaterialParameterCollection.
 */
class FMaterialParameterCollectionInstanceResource
{
public:
	
	/** 
	 * Update the contents of the uniform buffer, called from the game thread. 
	 * bRecreateUniformBuffer must be false if any components have their rendering state created which might reference this!
	 * The calling code is responsible for issuing a FGlobalComponentRecreateRenderStateContext before modifying.  Otherwise cached mesh draw commands will be left with a dangling pointer.
	 */
	void GameThread_UpdateContents(const FGuid& InId, const TArray<FVector4f>& Data, const FName& InOwnerName, bool bRecreateUniformBuffer);

	/** Destroy, called from the game thread. */
	void GameThread_Destroy();

	FGuid GetId() const 
	{ 
		return Id; 
	}

	FName GetOwnerName() const
	{
		return OwnerName;
	}

	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer;
	}

	FMaterialParameterCollectionInstanceResource();

private:
	~FMaterialParameterCollectionInstanceResource();

	/** Unique identifier for the UMaterialParameterCollection that material shaders were compiled with. */
	FGuid Id;

	FName OwnerName;

	/** Uniform buffer containing the UMaterialParameterCollection default parameter values and UMaterialParameterCollectionInstance instance overrides. */
	FUniformBufferRHIRef UniformBuffer;

	FUniformBufferLayoutRHIRef UniformBufferLayout;

	void UpdateContents(const FGuid& InId, const TArray<FVector4f>& Data, const FName& InOwnerName, bool bRecreateUniformBuffer);
};

// Default instance resources used when rendering a material using a parameter collection but there's no FScene present to get a FMaterialParameterCollectionInstanceResource
extern ENGINE_API TMultiMap<FGuid, FMaterialParameterCollectionInstanceResource*> GDefaultMaterialParameterCollectionInstances;
