// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreFwd.h"

namespace ParseExecCommands
{
	ENGINE_API TArray<FString> ParseExecCmds(const FString& Line);
	ENGINE_API TArray<FString> ParseExecCmdsFromCommandLine(const FString& InKey);
	ENGINE_API void QueueDeferredCommands(const TArray<FString>& CommandArray);
	ENGINE_API void QueueDeferredCommands(const FString& Line);
}