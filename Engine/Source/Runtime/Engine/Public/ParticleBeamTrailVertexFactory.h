// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleBeamTrailVertexFactory.h: Shared Particle Beam and Trail vertex 
										factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "ParticleVertexFactory.h"

class FMaterial;

/**
 * Uniform buffer for particle beam/trail vertex factories.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FParticleBeamTrailUniformParameters, ENGINE_API)
	SHADER_PARAMETER( FVector4f, CameraRight )
	SHADER_PARAMETER( FVector4f, CameraUp )
	SHADER_PARAMETER( FVector4f, ScreenAlignment )
	SHADER_PARAMETER( uint32, bUseLocalSpace)
	SHADER_PARAMETER( FVector3f, LWCTile)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FParticleBeamTrailUniformParameters> FParticleBeamTrailUniformBufferRef;

/**
 * Beam/Trail particle vertex factory.
 */
class FParticleBeamTrailVertexFactory : public FParticleVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE(FParticleBeamTrailVertexFactory);

public:

	/** Default constructor. */
	FParticleBeamTrailVertexFactory( EParticleVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel )
		: FParticleVertexFactoryBase(InType, InFeatureLevel)
		, IndexBuffer(nullptr)
		, FirstIndex(0)
		, OutTriangleCount(0)
		, bUsesDynamicParameter(true)
	{}

	FParticleBeamTrailVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FParticleVertexFactoryBase(PVFT_MAX, InFeatureLevel)
		, IndexBuffer(nullptr)
		, FirstIndex(0)
		, OutTriangleCount(0)
		, bUsesDynamicParameter(true)
	{}

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	 */
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Get vertex elements used when during PSO precaching materials using this vertex factory type
	 */
	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static ENGINE_API FRHIVertexDeclaration* GetPSOPrecacheVertexDeclaration(bool bUsesDynamicParameter);

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/**
	 * Set the uniform buffer for this vertex factory.
	 */
	inline void SetBeamTrailUniformBuffer( FParticleBeamTrailUniformBufferRef InSpriteUniformBuffer )
	{
		BeamTrailUniformBuffer = InSpriteUniformBuffer;
	}

	/**
	 * Retrieve the uniform buffer for this vertex factory.
	 */
	inline FParticleBeamTrailUniformBufferRef GetBeamTrailUniformBuffer()
	{
		return BeamTrailUniformBuffer;
	}	
	
	/**
	 * Set the source vertex buffer.
	 */
	ENGINE_API void SetVertexBuffer(const FVertexBuffer* InBuffer, uint32 StreamOffset, uint32 Stride);

	/**
	 * Set the source vertex buffer that contains particle dynamic parameter data.
	 */
	ENGINE_API void SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride);
	inline void SetUsesDynamicParameter(bool bInUsesDynamicParameter)
	{
		bUsesDynamicParameter = bInUsesDynamicParameter;
	}

	FIndexBuffer*& GetIndexBuffer()
	{
		return IndexBuffer;
	}

	uint32& GetFirstIndex()
	{
		return FirstIndex;
	}

	int32& GetOutTriangleCount()
	{
		return OutTriangleCount;
	}

private:

	/** Uniform buffer with beam/trail parameters. */
	FParticleBeamTrailUniformBufferRef BeamTrailUniformBuffer;

	/** Used to hold the index buffer allocation information when we call GDME more than once per frame. */
	FIndexBuffer* IndexBuffer;
	uint32 FirstIndex;
	int32 OutTriangleCount;
	bool bUsesDynamicParameter;
};
