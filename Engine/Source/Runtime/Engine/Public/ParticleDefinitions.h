// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This file is just used to indirectly include EngineParticleClasses.h with all of the C++ headers it depends on included first.
 */

#pragma once

// IWYU pragma: begin_keep

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "PixelFormat.h"
#include "ParticleVertexFactory.h"
#include "ConvexVolume.h"
#include "HitProxies.h"
#include "UnrealClient.h"
#include "TextureResource.h"
#include "SceneTypes.h"
#include "StaticParameterSet.h"
#include "MaterialShared.h"
#include "ShowFlags.h"
#include "EngineLogs.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "EngineDefines.h"
#include "ComponentInstanceDataCache.h"
#include "TimerManager.h"
#include "Math/GenericOctreePublic.h"
#include "Math/GenericOctree.h"
#include "BlendableManager.h"
#include "FinalPostProcessSettings.h"
#include "SceneInterface.h"
#include "DebugViewModeHelpers.h"
#include "SceneView.h"
#include "StaticBoundShaderState.h"
#include "BatchedElements.h"
#include "PrimitiveUniformShaderParameters.h"
#include "MeshBatch.h"
#include "SceneUtils.h"
#include "SceneManagement.h"
#include "MeshParticleVertexFactory.h"
#include "ParticleHelper.h"
#include "Distributions.h"
#include "ParticleEmitterInstances.h"

// IWYU pragma: end_keep
