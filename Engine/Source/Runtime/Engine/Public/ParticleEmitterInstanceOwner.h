// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/MathFwd.h"
#include "UObject/WeakObjectPtr.h"

class FFXSystem;
class FPrimitiveSceneProxy;
class FSceneInterface;
class UMaterialInterface;
class UParticleSystem;
class UParticleSystemComponent;
class UPhysicalMaterial;
class UObject;
class UWorld;
struct FParticleSysParam;
struct FParticleEmitterInstance;
struct FParticleEventBurstData;
struct FParticleEventDeathData;
struct FParticleEventCollideData;
struct FParticleEventKismetData;
struct FParticleEventSpawnData;
struct FRandomStream;


class IParticleEmitterInstanceOwner
{
public:
	virtual const FTransform& GetAsyncComponentToWorld() const = 0;

	virtual UObject* GetDistributionData() const = 0;
	virtual const FTransform& GetComponentTransform() const = 0;
	virtual FRotator GetComponentRotation() const = 0;
	virtual const FTransform& GetComponentToWorld() const = 0;
	virtual const FBoxSphereBounds& GetBounds() const = 0;
	virtual TWeakObjectPtr<UWorld> GetWeakWorld() const = 0;
	virtual bool HasWorld() const = 0;
	virtual bool HasWorldSettings() const = 0;
	virtual bool IsGameWorld() const = 0;
	virtual float GetWorldTimeSeconds() const = 0;
	virtual float GetWorldEffectiveTimeDilation() const = 0;
	virtual FIntVector GetWorldOriginLocation() const = 0;
	virtual FSceneInterface* GetScene() const = 0;
	virtual bool GetFloatParameter(const FName InName, float& OutFloat) = 0;
	virtual const FVector3f& GetLWCTile() const = 0;
	virtual FString GetName() const = 0;
	virtual FString GetFullName() const = 0;
	virtual FString GetPathName() const = 0;
	virtual bool IsActive() const = 0;
	virtual bool IsValidLowLevel() const = 0;
	virtual TArrayView<const FParticleSysParam> GetAsyncInstanceParameters() = 0;
	virtual int32 GetCurrentDetailMode() const = 0;
	virtual int32 GetCurrentLODIndex() const = 0;
	virtual const FVector& GetPartSysVelocity() const = 0;
	virtual const FVector& GetOldPosition() const = 0;
	virtual FFXSystem* GetFXSystem() const = 0;
	virtual const UParticleSystem* GetTemplate() const = 0;
	virtual TArrayView<const FParticleSysParam> GetInstanceParameters() const = 0;
	virtual TArrayView<FParticleEmitterInstance*> GetEmitterInstances() const = 0;
	virtual TArrayView<TObjectPtr<UMaterialInterface>> GetEmitterMaterials() const = 0;
	virtual FPrimitiveSceneProxy* GetSceneProxy() const = 0;
	virtual bool GetIsWarmingUp() const = 0;
	virtual bool GetJustRegistered() const = 0;
	virtual float GetWarmupTime() const = 0;
	virtual float GetEmitterDelay() const = 0;
	virtual FRandomStream& GetRandomStream() = 0;

	virtual void SetComponentToWorld(const FTransform& NewComponentToWorld) = 0;
	virtual void DeactivateNextTick() = 0;

	virtual UParticleSystemComponent* AsComponent() const = 0;

	virtual void ReportEventSpawn(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData) = 0;
	virtual void ReportEventDeath(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime) = 0;
	virtual void ReportEventCollision(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InDirection, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat) = 0;
	virtual void ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 ParticleCount, const FVector InLocation, const TArray<class UParticleModuleEventSendToGame*>& InEventData) = 0;

	virtual TArrayView<FParticleEventSpawnData> GetSpawnEvents() const = 0;
	virtual TArrayView<FParticleEventDeathData> GetDeathEvents() const = 0;
	virtual TArrayView<FParticleEventCollideData> GetCollisionEvents() const = 0;
	virtual TArrayView<FParticleEventBurstData> GetBurstEvents() const = 0;
	virtual TArrayView<FParticleEventKismetData> GetKismetEvents() const = 0;

protected:
	virtual ~IParticleEmitterInstanceOwner() = default;
};
