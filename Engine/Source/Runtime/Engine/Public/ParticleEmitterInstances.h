// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnParticleEmitterInstances.h: 
	Particle emitter instance definitions/ macros.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialRelevance.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Distributions.h"
#include "Distributions/DistributionFloat.h"
#include "ParticleHelper.h"
#include "Distributions/DistributionVector.h"
#include "Particles/Orientation/ParticleModuleOrientationAxisLock.h"

//Temporary define to allow switching on and off of some trail optimizations until bugs can be worked out.
#define ENABLE_TRAILS_START_END_INDEX_OPTIMIZATION (0)

class UParticleEmitter;
class UParticleLODLevel;
class UParticleModuleBeamModifier;
class UParticleModuleBeamNoise;
class UParticleModuleBeamSource;
class UParticleModuleBeamTarget;
class UParticleModuleSpawnPerUnit;
class UParticleModuleTrailSource;
class UParticleModuleTypeDataAnimTrail;
class UParticleModuleTypeDataBeam2;
class UParticleModuleTypeDataMesh;
class UParticleModuleTypeDataRibbon;
class UParticleSystemComponent;

/*-----------------------------------------------------------------------------
	Forward declarations
-----------------------------------------------------------------------------*/
class IParticleEmitterInstanceOwner;
//	Emitter and module types
class UParticleEmitter;
class UParticleSpriteEmitter;
class UParticleModule;
// Data types
class UParticleModuleTypeDataMesh;
class UParticleModuleTypeDataBeam2;
class UParticleModuleTypeDataRibbon;
class UParticleModuleTypeDataAnimTrail;

class UStaticMeshComponent;
class UParticleSystemComponent;

class UParticleModuleBeamSource;
class UParticleModuleBeamTarget;
class UParticleModuleBeamNoise;

class UParticleModuleTrailSource;

class UParticleModuleSpawnPerUnit;

class UParticleModuleOrientationAxisLock;

class UParticleLODLevel;

enum ETrailWidthMode : int;

struct FLODBurstFired
{
	TArray<bool> Fired;
};

/*-----------------------------------------------------------------------------
	Information compiled from modules to build runtime emitter data.
-----------------------------------------------------------------------------*/

struct FParticleEmitterBuildInfo
{
	/** The required module. */
	class UParticleModuleRequired* RequiredModule;
	/** The spawn module. */
	class UParticleModuleSpawn* SpawnModule;
	/** The spawn-per-unit module. */
	class UParticleModuleSpawnPerUnit* SpawnPerUnitModule;
	/** List of spawn modules that need to be invoked at runtime. */
	TArray<class UParticleModule*> SpawnModules;

	/** The accumulated orbit offset. */
	FComposableVectorDistribution OrbitOffset;
	/** The accumulated orbit initial rotation. */
	FComposableVectorDistribution OrbitInitialRotation;
	/** The accumulated orbit rotation rate. */
	FComposableVectorDistribution OrbitRotationRate;

	/** The color scale of a particle over time. */
	FComposableVectorDistribution ColorScale;
	/** The alpha scale of a particle over time. */
	FComposableFloatDistribution AlphaScale;

	/** An additional color scale for allowing parameters to be used for ColorOverLife modules. */
	FRawDistributionVector DynamicColor;
	/** An additional alpha scale for allowing parameters to be used for ColorOverLife modules. */
	FRawDistributionFloat DynamicAlpha;

	/** An additional color scale for allowing parameters to be used for ColorScaleOverLife modules. */
	FRawDistributionVector DynamicColorScale;
	/** An additional alpha scale for allowing parameters to be used for ColorScaleOverLife modules. */
	FRawDistributionFloat DynamicAlphaScale;
	
	/** How to scale a particle's size over time. */
	FComposableVectorDistribution SizeScale;
	/** The maximum size of a particle. */
	FVector2D MaxSize;
	/** How much to scale a particle's size based on its speed. */
	FVector2D SizeScaleBySpeed;
	/** The maximum amount by which to scale a particle based on its speed. */
	FVector2D MaxSizeScaleBySpeed;

	/** The sub-image index over the particle's life time. */
	FComposableFloatDistribution SubImageIndex;

	/** Drag coefficient. */
	FComposableFloatDistribution DragCoefficient;
	/** Drag scale over life. */
	FComposableFloatDistribution DragScale;

	/** Enable collision? */
	bool bEnableCollision;
	/** How particles respond to collision. */
	uint8 CollisionResponse;
	uint8 CollisionMode;
	/** Radius scale applied to friction. */
	float CollisionRadiusScale;
	/** Bias applied to the collision radius. */
	float CollisionRadiusBias;
	/** Factor reflection spreading cone when colliding. */
	float CollisionRandomSpread;
	/** Random distribution across the reflection spreading cone when colliding. */
	float CollisionRandomDistribution;
	/** Friction. */
	float Friction;
	/** Collision damping factor. */
	FComposableFloatDistribution Resilience;
	/** Collision damping factor scale over life. */
	FComposableFloatDistribution ResilienceScaleOverLife;

	/** Location of a point source attractor. */
	FVector PointAttractorPosition;
	/** Radius of the point source attractor. */
	float PointAttractorRadius;
	/** Strength of the point attractor. */
	FComposableFloatDistribution PointAttractorStrength;

	/** The per-particle vector field scale. */
	FComposableFloatDistribution VectorFieldScale;
	/** The per-particle vector field scale-over-life. */
	FComposableFloatDistribution VectorFieldScaleOverLife;
	/** Global vector field scale. */
	float GlobalVectorFieldScale;
	/** Global vector field tightness. */
	float GlobalVectorFieldTightness;

	/** Local vector field. */
	class UVectorField* LocalVectorField;
	/** Local vector field transform. */
	FTransform LocalVectorFieldTransform;
	/** Local vector field intensity. */
	float LocalVectorFieldIntensity;
	/** Tightness tweak for local vector fields. */
	float LocalVectorFieldTightness;
	/** Minimum initial rotation applied to local vector fields. */
	FVector LocalVectorFieldMinInitialRotation;
	/** Maximum initial rotation applied to local vector fields. */
	FVector LocalVectorFieldMaxInitialRotation;
	/** Local vector field rotation rate. */
	FVector LocalVectorFieldRotationRate;

	/** Constant acceleration to apply to particles. */
	FVector ConstantAcceleration;

	/** The maximum lifetime of any particle that will spawn. */
	float MaxLifetime;
	/** The maximum rotation rate of particles. */
	float MaxRotationRate;
	/** The estimated maximum number of particles for this emitter. */
	int32 EstimatedMaxActiveParticleCount;

	int32 ScreenAlignment;

	/** An offset in UV space for the positioning of a sprites vertices. */
	FVector2D PivotOffset;

	/** If true, local vector fields ignore the component transform. */
	uint32 bLocalVectorFieldIgnoreComponentTransform : 1;
	/** Tile vector field in x axis? */
	uint32 bLocalVectorFieldTileX : 1;
	/** Tile vector field in y axis? */
	uint32 bLocalVectorFieldTileY : 1;
	/** Tile vector field in z axis? */
	uint32 bLocalVectorFieldTileZ : 1;
	/** Use fix delta time in the simulation? */
	uint32 bLocalVectorFieldUseFixDT : 1;

	uint32 bUseVelocityForMotionBlur : 1;

	/** Particle alignment overrides */
	uint32 bRemoveHMDRoll : 1;
	float MinFacingCameraBlendDistance;
	float MaxFacingCameraBlendDistance;
	
	/** Default constructor. */
	FParticleEmitterBuildInfo();
};

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FParticleEmitterInstanceFixLayout
{
	virtual ~FParticleEmitterInstanceFixLayout() = default;
};


/*-----------------------------------------------------------------------------
	FParticleEmitterInstance
-----------------------------------------------------------------------------*/
struct FParticleEmitterInstance : FParticleEmitterInstanceFixLayout
{
public:
	/** The maximum DeltaTime allowed for updating PeakActiveParticle tracking.
	 *	Any delta time > this value will not impact active particle tracking.
	 */
	static ENGINE_API const float PeakActiveParticleUpdateDelta;

	/** The template this instance is based on.							*/
	UParticleEmitter* SpriteTemplate;
	/** The component who owns it.										*/
	IParticleEmitterInstanceOwner& Component;
	/** The currently set LOD level.									*/
	UParticleLODLevel* CurrentLODLevel;
	/** The index of the currently set LOD level.						*/
	int32 CurrentLODLevelIndex;
	/** The offset to the TypeData payload in the particle data.		*/
	int32 TypeDataOffset;
	/** The offset to the TypeData instance payload.					*/
	int32 TypeDataInstanceOffset;
	/** The offset to the SubUV payload in the particle data.			*/
	int32 SubUVDataOffset;
	/** The offset to the dynamic parameter payload in the particle data*/
	int32 DynamicParameterDataOffset;
	/** Offset to the light module data payload.						*/
	int32 LightDataOffset;
	float LightVolumetricScatteringIntensity;
	/** The offset to the Orbit module payload in the particle data.	*/
	int32 OrbitModuleOffset;
	/** The offset to the Camera payload in the particle data.			*/
	int32 CameraPayloadOffset;
	/** The offset to the particle data.								*/
	int32 PayloadOffset;
	/** The location of the emitter instance							*/
	FVector Location;
	/** Transform from emitter local space to simulation space.			*/
	FMatrix EmitterToSimulation;
	/** Transform from simulation space to world space.					*/
	FMatrix SimulationToWorld;
	/** Component can disable Tick and Rendering of this emitter. */
	uint8 bEnabled : 1;
	/** If true, kill this emitter instance when it is deactivated.		*/
	uint8 bKillOnDeactivate:1;
	/** if true, kill this emitter instance when it has completed.		*/
	uint8 bKillOnCompleted:1;
	/** Whether this emitter requires sorting as specified by artist.	*/
	uint8 bRequiresSorting:1;
	/** If true, halt spawning for this instance.						*/
	uint8 bHaltSpawning : 1;
	/** If true, this emitter has been disabled by game code and some systems to re-enable are not allowed. */
	uint8 bHaltSpawningExternal : 1;
	/** If true, the emitter has modules that require loop notification.*/
	uint8 bRequiresLoopNotification:1;
	/** If true, the emitter ignores the component's scale. (Mesh emitters only). */
	uint8 bIgnoreComponentScale:1;
	/** Hack: Make sure this is a Beam type to avoid casting from/to wrong types. */
	uint8 bIsBeam:1;
	/** Whether axis lock is enabled, cached here to avoid finding it from the module each frame */
	uint8 bAxisLockEnabled : 1;
	/** When true and spawning is supressed, the bursts will be faked so that when spawning is enabled again, the bursts don't fire late. */
	uint8 bFakeBurstsWhenSpawningSupressed : 1;
	/** true if the emitter has no active particles and will no longer spawn any in the future */
	uint8 bEmitterIsDone:1;
	/** Axis lock flags, cached here to avoid finding it from the module each frame */
	TEnumAsByte<EParticleAxisLock> LockAxisFlags;
	/** The sort mode to use for this emitter as specified by artist.	*/
	int32 SortMode;
	/** Pointer to the particle data array.								*/
	uint8* ParticleData;
	/** Pointer to the particle index array.							*/
	uint16* ParticleIndices;
	/** Pointer to the instance data array.								*/
	uint8* InstanceData;
	/** The size of the Instance data array.							*/
	int32 InstancePayloadSize;
	/** The total size of a particle (in bytes).						*/
	int32 ParticleSize;
	/** The stride between particles in the ParticleData array.			*/
	int32 ParticleStride;
	/** The number of particles currently active in the emitter.		*/
	int32 ActiveParticles;
	/** Monotonically increasing counter. */
	uint32 ParticleCounter;
	/** The maximum number of active particles that can be held in 
	 *	the particle data array.
	 */
	int32 MaxActiveParticles;
	/** The fraction of time left over from spawning.					*/
	float SpawnFraction;
	/** The number of seconds that have passed since the instance was
	 *	created.
	 */
	float SecondsSinceCreation;
	/** */
	float EmitterTime;
	/** The amount of time simulated in the previous time step. */
	float LastDeltaTime;
	/** how long did the last tick take? */
#if WITH_EDITOR
	float LastTickDurationMs;
#endif
	/** The previous location of the instance.							*/
	FVector OldLocation;
	/** The bounding box for the particles.								*/
	FBox ParticleBoundingBox;
	/** The BurstFire information.										*/
	TArray<struct FLODBurstFired> BurstFired;
	/** The number of loops completed by the instance.					*/
	int32 LoopCount;
	/** Flag indicating if the render data is dirty.					*/
	int32 IsRenderDataDirty;
	/** The current duration fo the emitter instance.					*/
	float EmitterDuration;
	/** The emitter duration at each LOD level for the instance.		*/
	TArray<float> EmitterDurations;
	/** The emitter's delay for the current loop		*/
	float CurrentDelay;

	/** The number of triangles to render								*/
	int32	TrianglesToRender;
	int32 MaxVertexIndex;

	/** The material to render this instance with.						*/
	UMaterialInterface* CurrentMaterial;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Number of events this emitter has generated... */
	int32 EventCount;
	int32 MaxEventCount;
#endif	//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	/** Position offset for each particle. Will be reset to zero at the end of the tick	*/
	FVector PositionOffsetThisTick;
	
	/** The PivotOffset applied to the vertex positions 			*/
	FVector2D PivotOffset;

	TArray<class UPointLightComponent*> HighQualityLights;

	/** Constructor	*/
	ENGINE_API FParticleEmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	/** Destructor	*/
	ENGINE_API virtual ~FParticleEmitterInstance();

#if STATS
	ENGINE_API void PreDestructorCall();
#endif

	//
	ENGINE_API virtual void InitParameters(UParticleEmitter* InTemplate);
	ENGINE_API virtual void Init();

	/**
	 * Ensures enough memory is allocated for the requested number of particles.
	 *
	 * @param NewMaxActiveParticles		The number of particles for which memory must be allocated.
	 * @param bSetMaxActiveCount		If true, update the peak active particles for this LOD.
	 * @returns bool					true if memory is allocated for at least NewMaxActiveParticles.
	 */
	ENGINE_API virtual bool Resize(int32 NewMaxActiveParticles, bool bSetMaxActiveCount = true);

	ENGINE_API virtual void Tick(float DeltaTime, bool bSuppressSpawning);
	ENGINE_API void CheckEmitterFinished();

	/** Advances the bursts as though they were fired with out actually firing them. */
	ENGINE_API void FakeBursts();

	/**
	 *	Tick sub-function that handles EmitterTime setup, looping, etc.
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 *
	 *	@return	float				The EmitterDelay
	 */
	ENGINE_API virtual float Tick_EmitterTimeSetup(float DeltaTime, UParticleLODLevel* CurrentLODLevel);
	/**
	 *	Tick sub-function that handles spawning of particles
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 *	@param	bSuppressSpawning	true if spawning has been suppressed on the owning particle system component
	 *	@param	bFirstTime			true if this is the first time the instance has been ticked
	 *
	 *	@return	float				The SpawnFraction remaining
	 */
	ENGINE_API virtual float Tick_SpawnParticles(float DeltaTime, UParticleLODLevel* CurrentLODLevel, bool bSuppressSpawning, bool bFirstTime);
	/**
	 *	Tick sub-function that handles module updates
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	ENGINE_API virtual void Tick_ModuleUpdate(float DeltaTime, UParticleLODLevel* CurrentLODLevel);
	/**
	 *	Tick sub-function that handles module post updates
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	ENGINE_API virtual void Tick_ModulePostUpdate(float DeltaTime, UParticleLODLevel* CurrentLODLevel);
	/**
	 *	Tick sub-function that handles module FINAL updates
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	ENGINE_API virtual void Tick_ModuleFinalUpdate(float DeltaTime, UParticleLODLevel* CurrentLODLevel);

	/**
	 *	Set the LOD to the given index
	 *
	 *	@param	InLODIndex			The index of the LOD to set as current
	 *	@param	bInFullyProcess		If true, process burst lists, etc.
	 */
	ENGINE_API virtual void SetCurrentLODIndex(int32 InLODIndex, bool bInFullyProcess);

	ENGINE_API virtual void Rewind();
	ENGINE_API virtual FBox GetBoundingBox();
	ENGINE_API virtual void UpdateBoundingBox(float DeltaTime);
	ENGINE_API virtual void ForceUpdateBoundingBox();
	ENGINE_API virtual uint32 RequiredBytes();
	/** Get offset for particle payload data for a particular module */
	ENGINE_API uint32 GetModuleDataOffset(UParticleModule* Module);
	/** Get pointer to emitter instance payload data for a particular module */
	ENGINE_API uint8* GetModuleInstanceData(UParticleModule* Module);
	/** Get pointer to emitter instance random seed payload data for a particular module */
	ENGINE_API FParticleRandomSeedInstancePayload* GetModuleRandomSeedInstanceData(UParticleModule* Module);
	ENGINE_API virtual uint8* GetTypeDataModuleInstanceData();
	ENGINE_API virtual uint32 CalculateParticleStride(uint32 ParticleSize);
	ENGINE_API virtual void ResetBurstList();
	ENGINE_API virtual float GetCurrentBurstRateOffset(float& DeltaTime, int32& Burst);
	ENGINE_API virtual void ResetParticleParameters(float DeltaTime);
	ENGINE_API void CalculateOrbitOffset(FOrbitChainModuleInstancePayload& Payload, 
		FVector& AccumOffset, FVector& AccumRotation, FVector& AccumRotationRate, 
		float DeltaTime, FVector& Result, FMatrix& RotationMat);
	ENGINE_API virtual void UpdateOrbitData(float DeltaTime);
	ENGINE_API virtual void ParticlePrefetch();

	/**
	 *	Spawn particles for this emitter instance
	 *	@param	DeltaTime		The time slice to spawn over
	 *	@return	float			The leftover fraction of spawning
	 */
	ENGINE_API virtual float Spawn(float DeltaTime);

	/**
	 * Spawn the indicated number of particles.
	 *
	 * @param Count The number of particles to spawn.
	 * @param StartTime			The local emitter time at which to begin spawning particles.
	 * @param Increment			The time delta between spawned particles.
	 * @param InitialLocation	The initial location of spawned particles.
	 * @param InitialVelocity	The initial velocity of spawned particles.
	 * @param EventPayload		Event generator payload if events should be triggered.
	 */
	ENGINE_API void SpawnParticles( int32 Count, float StartTime, float Increment, const FVector& InitialLocation, const FVector& InitialVelocity, struct FParticleEventInstancePayload* EventPayload );

	/**
	 *	Spawn/burst the given particles...
	 *
	 *	@param	DeltaTime		The time slice to spawn over.
	 *	@param	InSpawnCount	The number of particles to forcibly spawn.
	 *	@param	InBurstCount	The number of particles to forcibly burst.
	 *	@param	InLocation		The location to spawn at.
	 *	@param	InVelocity		OPTIONAL velocity to have the particle inherit.
	 *
	 */
	ENGINE_API virtual void ForceSpawn(float DeltaTime, int32 InSpawnCount, int32 InBurstCount, FVector& InLocation, FVector& InVelocity);
	ENGINE_API void CheckSpawnCount(int32 InNewCount, int32 InMaxCount);

	/**
	 * Handle any pre-spawning actions required for particles
	 *
	 * @param Particle			The particle being spawned.
	 * @param InitialLocation	The initial location of the particle.
	 * @param InitialVelocity	The initial velocity of the particle.
	 */
	ENGINE_API virtual void PreSpawn(FBaseParticle* Particle, const FVector& InitialLocation, const FVector& InitialVelocity);

	/**
	 * Handle any post-spawning actions required by the instance
	 *
	 * @param	Particle					The particle that was spawned
	 * @param	InterpolationPercentage		The percentage of the time slice it was spawned at
	 * @param	SpawnTIme					The time it was spawned at
	 */
	ENGINE_API virtual void PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime);

	ENGINE_API virtual bool HasCompleted();
	ENGINE_API virtual void KillParticles();
	/**
	 *	Kill the particle at the given instance
	 *
	 *	@param	Index		The index of the particle to kill.
	 */
	ENGINE_API virtual void KillParticle(int32 Index);

	/**
	 *	Force kill all particles in the emitter.
	 *
	 *	@param	bFireEvents		If true, fire events for the particles being killed.
	 */
	ENGINE_API virtual void KillParticlesForced(bool bFireEvents = false);

	/** Set the HaltSpawning flag */
	virtual void SetHaltSpawning(bool bInHaltSpawning)
	{
		bHaltSpawning = bInHaltSpawning;
	}

	/** Set the bHaltSpawningExternal flag */
	virtual void SetHaltSpawningExternal(bool bInHaltSpawning)
	{
		bHaltSpawningExternal = bInHaltSpawning;
	}

	inline void SetFakeBurstWhenSpawningSupressed(bool bInFakeBurstsWhenSpawningSupressed)
	{
		bFakeBurstsWhenSpawningSupressed = bInFakeBurstsWhenSpawningSupressed;
	}

	/** Get the offset of the orbit payload. */
	ENGINE_API int32 GetOrbitPayloadOffset();

	/** Get the position of the particle taking orbit in to account. */
	ENGINE_API FVector GetParticleLocationWithOrbitOffset(FBaseParticle* Particle);

	ENGINE_API virtual FBaseParticle* GetParticle(int32 Index);
	/**
	 *	Get the physical index of the particle at the given index
	 *	(ie, the contents of ParticleIndices[InIndex])
	 *
	 *	@param	InIndex			The index of the particle of interest
	 *
	 *	@return	int32				The direct index of the particle
	 */
	inline int32 GetParticleDirectIndex(int32 InIndex)
	{
		if (InIndex < MaxActiveParticles)
		{
			return ParticleIndices[InIndex];
		}
		return -1;
	}
	/**
	 *	Get the particle at the given direct index
	 *
	 *	@param	InDirectIndex		The direct index of the particle of interest
	 *
	 *	@return	FBaseParticle*		The particle, if valid index was given; NULL otherwise
	 */
	ENGINE_API virtual FBaseParticle* GetParticleDirect(int32 InDirectIndex);

	/**
	 *	Calculates the emitter duration for the instance.
	 */
	ENGINE_API void SetupEmitterDuration();
	
	/**
	 * Returns whether the system has any active particles.
	 *
	 * @return true if there are active particles, false otherwise.
	 */
	bool HasActiveParticles()
	{
		return ActiveParticles > 0;
	}

	/**
	 *	Checks some common values for GetDynamicData validity
	 *
	 *	@return	bool		true if GetDynamicData should continue, false if it should return NULL
	 */
	ENGINE_API virtual bool IsDynamicDataRequired(UParticleLODLevel* CurrentLODLevel);

	/**
	 *	Retrieves the dynamic data for the emitter
	 */
	virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel)
	{
		return NULL;
	}

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	virtual FDynamicEmitterReplayDataBase* GetReplayData()
	{
		return NULL;
	}

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	virtual void GetAllocatedSize(int32& OutNum, int32& OutMax)
	{
		OutNum = 0;
		OutMax = 0;
	}
	
	/** Returns resource size, similar to UObject function */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
	{
	}

	/**
	 *	Process received events.
	 */
	ENGINE_API virtual void ProcessParticleEvents(float DeltaTime, bool bSuppressSpawning);

	/**
	 *	Called when the particle system is deactivating...
	 */
	virtual void OnDeactivateSystem()
	{
	}

	/**
	 * Returns the offset to the mesh rotation payload, if any.
	 */
	virtual int32 GetMeshRotationOffset() const
	{
		return 0;
	}

	/**
	 * Returns true if mesh rotation is active.
	 */
	virtual bool IsMeshRotationActive() const
	{
		return false;
	}

	/**
	 * Sets the materials with which mesh particles should be rendered.
	 * @param InMaterials - The materials.
	 */
	virtual void SetMeshMaterials( const TArray<UMaterialInterface*>& InMaterials )
	{
	}

	/**
	 * Gathers material relevance flags for this emitter instance.
	 * @param OutMaterialRelevance - Pointer to where material relevance flags will be stored.
	 * @param LODLevel - The LOD level for which to compute material relevance flags.
	 */
	UE_DEPRECATED(5.7, "Please use GatherMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual void GatherMaterialRelevance( FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, ERHIFeatureLevel::Type InFeatureLevel ) const;
	ENGINE_API virtual void GatherMaterialRelevance( FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, EShaderPlatform InShaderPlatform ) const;

	/**
	 * When an emitter is killed, this will check other emitters and clean up anything pointing to this one
	 */
	virtual void OnEmitterInstanceKilled(FParticleEmitterInstance* Instance)
	{

	}

	// Beam interface
	virtual void SetBeamEndPoint(FVector NewEndPoint) {};
	virtual void SetBeamSourcePoint(FVector NewSourcePoint,int32 SourceIndex) {};
	virtual void SetBeamSourceTangent(FVector NewTangentPoint,int32 SourceIndex) {};
	virtual void SetBeamSourceStrength(float NewSourceStrength,int32 SourceIndex) {};
	virtual void SetBeamTargetPoint(FVector NewTargetPoint,int32 TargetIndex) {};
	virtual void SetBeamTargetTangent(FVector NewTangentPoint,int32 TargetIndex) {};
	virtual void SetBeamTargetStrength(float NewTargetStrength,int32 TargetIndex) {};

	//Beam get interface
	virtual bool GetBeamEndPoint(FVector& OutEndPoint) const { return false; }
	virtual bool GetBeamSourcePoint(int32 SourceIndex, FVector& OutSourcePoint) const { return false; }
	virtual bool GetBeamSourceTangent(int32 SourceIndex, FVector& OutTangentPoint) const { return false; }
	virtual bool GetBeamSourceStrength(int32 SourceIndex, float& OutSourceStrength) const { return false; }
	virtual bool GetBeamTargetPoint(int32 TargetIndex, FVector& OutTargetPoint) const { return false; }
	virtual bool GetBeamTargetTangent(int32 TargetIndex, FVector& OutTangentPoint) const { return false; }
	virtual bool GetBeamTargetStrength(int32 TargetIndex, float& OutTargetStrength) const { return false; }
	
	// Called on world origin changes
	ENGINE_API virtual void ApplyWorldOffset(FVector InOffset, bool bWorldShift);
		
	virtual bool IsTrailEmitter()const{ return false; }

	/**
	* Begins the trail.
	*/
	virtual void BeginTrail(){}

	/**
	* Ends the trail.
	*/
	virtual void EndTrail(){}

	/**
	* Sets the data that defines this trail.
	*
	* @param	InFirstSocketName	The name of the first socket for the trail.
	* @param	InSecondSocketName	The name of the second socket for the trail.
	* @param	InWidthMode			How the width value is applied to the trail.
	* @param	InWidth				The width of the trail.
	*/
	virtual void SetTrailSourceData(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth){}

	/** 
	Ticks the emitter's material overrides.
	@return True if there were material overrides. Otherwise revert to default behaviour.
	*/
	ENGINE_API virtual void Tick_MaterialOverrides(int32 EmitterIndex);

	/**
	* True if this emitter emits in local space
	*/
	ENGINE_API bool UseLocalSpace();

	/**
	* returns the screen alignment and scale of the component.
	*/
	ENGINE_API void GetScreenAlignmentAndScale(int32& OutScreenAlign, FVector& OutScale);

protected:

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	ENGINE_API virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData );

	/**
	 * Updates all internal transforms.
	 */
	ENGINE_API void UpdateTransforms();

	/**
	* Retrieves the current LOD level and asserts that it is valid.
	*/
	ENGINE_API class UParticleLODLevel* GetCurrentLODLevelChecked();

	/**
	 * Get the current material to render with.
	 */
	ENGINE_API UMaterialInterface* GetCurrentMaterial();


	/**
	 * Fixup particle indices to only have valid entries.
	 */
	ENGINE_API void FixupParticleIndices();

};

#if STATS
struct FScopeCycleCounterEmitter : public FCycleCounter
{
	/**
	* Constructor, starts timing
	*/
	template<class T>
	inline FScopeCycleCounterEmitter(const T *Object)
	{
		if (Object)
		{
			TStatId SpriteStatId = Object->SpriteTemplate->GetStatID();
			if (FThreadStats::IsCollectingData(SpriteStatId))
			{
				Start(SpriteStatId);
			}
		}
	}
	/**
	* Constructor, starts timing with an alternate enable stat to use high performance disable for only SOME UObject stats
	*/
	template<class T>
	inline FScopeCycleCounterEmitter(const T *Object, TStatId OtherStat)
	{
		if (FThreadStats::IsCollectingData(OtherStat) && Object)
		{
			TStatId StatId = Object->SpriteTemplate->GetStatID();
			if (!StatId.IsNone())
			{
				Start(StatId);
			}
		}
	}
	/**
	* Updates the stat with the time spent
	*/
	inline ~FScopeCycleCounterEmitter()
	{
		Stop();
	}
};
#else
struct FScopeCycleCounterEmitter
{
	inline FScopeCycleCounterEmitter(const FParticleEmitterInstance *Object)
	{
	}
	inline FScopeCycleCounterEmitter(const FParticleEmitterInstance *Object, TStatId OtherStat)
	{
	}
};
#endif

/*-----------------------------------------------------------------------------
	ParticleSpriteEmitterInstance
-----------------------------------------------------------------------------*/
struct FParticleSpriteEmitterInstance : public FParticleEmitterInstance
{
	/** Constructor	*/
	FParticleSpriteEmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	/** Destructor	*/
	virtual ~FParticleSpriteEmitterInstance();

	/**
	 *	Retrieves the dynamic data for the emitter
	 */
	virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel) override;

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	virtual FDynamicEmitterReplayDataBase* GetReplayData() override;

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	virtual void GetAllocatedSize(int32& OutNum, int32& OutMax) override;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
	 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

protected:

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData ) override;

};

/*-----------------------------------------------------------------------------
	ParticleMeshEmitterInstance
-----------------------------------------------------------------------------*/
struct FParticleMeshEmitterInstance : public FParticleEmitterInstance
{
	UParticleModuleTypeDataMesh* MeshTypeData;
	bool MeshRotationActive;
	int32 MeshRotationOffset;
	int32 MeshMotionBlurOffset;

	/** The materials to render this instance with.	*/
	TArray<UMaterialInterface*> CurrentMaterials;

	/** Constructor	*/
	ENGINE_API FParticleMeshEmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	ENGINE_API virtual void InitParameters(UParticleEmitter* InTemplate) override;
	ENGINE_API virtual void Init() override;
	ENGINE_API virtual bool Resize(int32 NewMaxActiveParticles, bool bSetMaxActiveCount = true) override;
	ENGINE_API virtual void Tick(float DeltaTime, bool bSuppressSpawning) override;
	ENGINE_API virtual void UpdateBoundingBox(float DeltaTime) override;
	ENGINE_API virtual uint32 RequiredBytes() override;
	ENGINE_API virtual void PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime) override;
	ENGINE_API virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel) override;
	ENGINE_API virtual bool IsDynamicDataRequired(UParticleLODLevel* CurrentLODLevel) override;

	ENGINE_API virtual void Tick_MaterialOverrides(int32 EmitterIndex) override;

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	ENGINE_API virtual FDynamicEmitterReplayDataBase* GetReplayData() override;

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	ENGINE_API virtual void GetAllocatedSize(int32& OutNum, int32& OutMax) override;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
	 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	ENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	/**
	 * Returns the offset to the mesh rotation payload, if any.
	 */
	virtual int32 GetMeshRotationOffset() const override
	{
		return MeshRotationOffset;
	}

	/**
	 * Returns true if mesh rotation is active.
	 */
	virtual bool IsMeshRotationActive() const override
	{
		return MeshRotationActive;
	}

	/**
	 * Sets the materials with which mesh particles should be rendered.
	 * @param InMaterials - The materials.
	 */
	ENGINE_API virtual void SetMeshMaterials( const TArray<UMaterialInterface*>& InMaterials ) override;

	/**
	 * Gathers material relevance flags for this emitter instance.
	 * @param OutMaterialRelevance - Pointer to where material relevance flags will be stored.
	 * @param LODLevel - The LOD level for which to compute material relevance flags.
	 */
	UE_DEPRECATED(5.7, "Please use GatherMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	ENGINE_API virtual void GatherMaterialRelevance(FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, ERHIFeatureLevel::Type InFeatureLevel) const override;
	ENGINE_API virtual void GatherMaterialRelevance(FMaterialRelevance* OutMaterialRelevance, const UParticleLODLevel* LODLevel, EShaderPlatform InShaderPlatform) const override;

	/**
	 * Gets the materials applied to each section of a mesh.
	 */
	UE_DEPRECATED(5.7, "Please use GatherMaterialRelevance without ERHIFeatureLevel::Type")
	ENGINE_API void GetMeshMaterials(TArray<UMaterialInterface*,TInlineAllocator<2> >& OutMaterials, const UParticleLODLevel* LODLevel, ERHIFeatureLevel::Type InFeatureLevel, bool bLogWarnings = false) const;
	ENGINE_API void GetMeshMaterials(TArray<UMaterialInterface*,TInlineAllocator<2> >& OutMaterials, const UParticleLODLevel* LODLevel, bool bLogWarnings = false) const;

protected:

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	ENGINE_API virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData ) override;
};

/*-----------------------------------------------------------------------------
	ParticleBeam2EmitterInstance
-----------------------------------------------------------------------------*/
struct FParticleBeam2EmitterInstance : public FParticleEmitterInstance
{
	UParticleModuleTypeDataBeam2*	BeamTypeData;

	UParticleModuleBeamSource*		BeamModule_Source;
	UParticleModuleBeamTarget*		BeamModule_Target;
	UParticleModuleBeamNoise*		BeamModule_Noise;
	UParticleModuleBeamModifier*	BeamModule_SourceModifier;
	int32								BeamModule_SourceModifier_Offset;
	UParticleModuleBeamModifier*	BeamModule_TargetModifier;
	int32								BeamModule_TargetModifier_Offset;

	bool							FirstEmission;
	int32								TickCount;
	int32								ForceSpawnCount;
	/** The method to utilize when forming the beam.							*/
	int32								BeamMethod;
	/** How many times to tile the texture along the beam.						*/
	TArray<int32>						TextureTiles;
	/** The number of live beams												*/
	int32								BeamCount;
	/** The actor to get the source point from.									*/
	AActor*							SourceActor;
	/** The emitter to get the source point from.								*/
	FParticleEmitterInstance*		SourceEmitter;
	/** User set Source points of each beam - primarily for weapon effects.		*/
	TArray<FVector>					UserSetSourceArray;
	/** User set Source tangents of each beam - primarily for weapon effects.	*/
	TArray<FVector>					UserSetSourceTangentArray;
	/** User set Source strengths of each beam - primarily for weapon effects.	*/
	TArray<float>					UserSetSourceStrengthArray;
	/** The distance of each beam, if utilizing the distance method.			*/
	TArray<float>					DistanceArray;
	/** The target point of each beam, when using the end point method.			*/
	TArray<FVector>					TargetPointArray;
	/** The target tangent of each beam, when using the end point method.		*/
	TArray<FVector>					TargetTangentArray;
	/** User set Target strengths of each beam - primarily for weapon effects.	*/
	TArray<float>					UserSetTargetStrengthArray;
	/** The actor to get the target point from.									*/
	AActor*							TargetActor;
	/** The emitter to get the Target point from.								*/
	FParticleEmitterInstance*		TargetEmitter;
	/** The target point sources of each beam, when using the end point method.	*/
	TArray<FName>					TargetPointSourceNames;
	/** User set target points of each beam - primarily for weapon effects.		*/
	TArray<FVector>					UserSetTargetArray;
	/** User set target tangents of each beam - primarily for weapon effects.	*/
	TArray<FVector>					UserSetTargetTangentArray;

	/** The number of vertices and triangles, for rendering						*/
	int32								VertexCount;
	int32								TriangleCount;
	TArray<int32>						BeamTrianglesPerSheet;

	/** Constructor	*/
	FParticleBeam2EmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	/** Destructor	*/
	virtual ~FParticleBeam2EmitterInstance();

	// Beam interface.
	virtual void SetBeamEndPoint(FVector NewEndPoint) override;
	virtual void SetBeamSourcePoint(FVector NewSourcePoint,int32 SourceIndex) override;
	virtual void SetBeamSourceTangent(FVector NewTangentPoint,int32 SourceIndex) override;
	virtual void SetBeamSourceStrength(float NewSourceStrength,int32 SourceIndex) override;
	virtual void SetBeamTargetPoint(FVector NewTargetPoint,int32 TargetIndex) override;
	virtual void SetBeamTargetTangent(FVector NewTangentPoint,int32 TargetIndex) override;
	virtual void SetBeamTargetStrength(float NewTargetStrength,int32 TargetIndex) override;
	virtual void ApplyWorldOffset(FVector InOffset, bool bWorldShift) override;
	
	virtual bool GetBeamEndPoint(FVector& OutEndPoint) const override;
	virtual bool GetBeamSourcePoint(int32 SourceIndex, FVector& OutSourcePoint) const override;
	virtual bool GetBeamSourceTangent(int32 SourceIndex, FVector& OutTangentPoint) const override;
	virtual bool GetBeamSourceStrength(int32 SourceIndex, float& OutSourceStrength) const override;
	virtual bool GetBeamTargetPoint(int32 TargetIndex, FVector& OutTargetPoint) const override;
	virtual bool GetBeamTargetTangent(int32 TargetIndex, FVector& OutTangentPoint) const override;
	virtual bool GetBeamTargetStrength(int32 TargetIndex, float& OutTargetStrength) const override;
	//
	virtual void InitParameters(UParticleEmitter* InTemplate) override;
	virtual void Init() override;
	virtual void Tick(float DeltaTime, bool bSuppressSpawning) override;
	/**
	 *	Tick sub-function that handles module post updates
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	virtual void Tick_ModulePostUpdate(float DeltaTime, UParticleLODLevel* CurrentLODLevel) override;

	/**
	 *	Set the LOD to the given index
	 *
	 *	@param	InLODIndex			The index of the LOD to set as current
	 *	@param	bInFullyProcess		If true, process burst lists, etc.
	 */
	virtual void SetCurrentLODIndex(int32 InLODIndex, bool bInFullyProcess) override;

	/**
	 * Handle any post-spawning actions required by the instance
	 *
	 * @param	Particle					The particle that was spawned
	 * @param	InterpolationPercentage		The percentage of the time slice it was spawned at
	 * @param	SpawnTIme					The time it was spawned at
	 */
	virtual void PostSpawn(FBaseParticle* Particle, float InterpolationPercentage, float SpawnTime) override;

	virtual void UpdateBoundingBox(float DeltaTime) override;
	virtual void ForceUpdateBoundingBox() override;
	virtual uint32 RequiredBytes() override;
	float SpawnBeamParticles(float OldLeftover, float Rate, float DeltaTime, int32 Burst = 0, float BurstTime = 0.0f);
	virtual void KillParticles() override;
	/**
	 *	Setup the offsets to the BeamModifier modules...
	 *	This must be done after the base Init call as that inserts modules into the offset map.
	 */
	void SetupBeamModifierModulesOffsets();
	void ResolveSource();
	void ResolveTarget();
	void DetermineVertexAndTriangleCount();

	/**
	 *	Retrieves the dynamic data for the emitter
	 */
	virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel) override;

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	virtual FDynamicEmitterReplayDataBase* GetReplayData() override;

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	virtual void GetAllocatedSize(int32& OutNum, int32& OutMax) override;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
	 * @return	Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	/**
	 * When an emitter is killed, this will check other emitters and clean up anything pointing to this one
	 */
	virtual void OnEmitterInstanceKilled(FParticleEmitterInstance* Instance) override
	{
		if (SourceEmitter == Instance)
		{
			SourceEmitter = NULL;
		}
		if (TargetEmitter == Instance)
		{
			TargetEmitter = NULL;
		}
	}

protected:
	/**
	 * Get the current material to render with.
	 */
	UMaterialInterface* GetCurrentMaterial();

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData ) override;

};

/*-----------------------------------------------------------------------------
	FParticleRibbonEmitterInstance.
-----------------------------------------------------------------------------*/
struct FParticleTrailsEmitterInstance_Base : public FParticleEmitterInstance
{
	/** The vertex count for this emitter */
	int32 VertexCount;
	/** The triangle count for this emitter */
	int32 TriangleCount;
	/** The number of active trails in this emitter */
	int32 TrailCount;
	/** The max number of trails this emitter is allowed to have */
	int32 MaxTrailCount;
	/** The running time for this instance w/ ActiveParticles > 0 */
	float RunningTime;
	/** The last time the emitter instance was ticked */
	float LastTickTime;
	/** If true, mark trails dead on deactivate */
	uint32 bDeadTrailsOnDeactivate:1;

	/** The Spawn times for each trail in this emitter */
	TArray<float> TrailSpawnTimes;
	/** The last time a spawn happened for each trail in this emitter */
	TArray<float> LastSpawnTime;
	/** The distance traveled by each source of each trail in this emitter */
	TArray<float> SourceDistanceTraveled;
	/** The distance traveled by each source of each trail in this emitter */
	TArray<float> TiledUDistanceTraveled;
	/** If true, this emitter has not been updated yet... */
	uint32 bFirstUpdate:1;
	/**
	 *	If true, when the system checks for particles to kill, it
	 *	will use elapsed gametime to make the determination. This
	 *	will result in emitters that were inactive due to not being
	 *	rendered killing off old particles.
	 */
	uint32 bEnableInactiveTimeTracking:1;

#if ENABLE_TRAILS_START_END_INDEX_OPTIMIZATION
	/** The direct index of the particle that is the start of each ribbon */
	int32 CurrentStartIndices[128];
	/** The direct index of the particle that is the end of each ribbon */
	int32 CurrentEndIndices[128];

	void SetStartIndex(int32 TrailIndex, int32 ParticleIndex)
	{
		CurrentStartIndices[TrailIndex] = ParticleIndex;
		if (CurrentEndIndices[TrailIndex] == ParticleIndex)
		{
			CurrentEndIndices[TrailIndex] = INDEX_NONE;
		}
		CheckIndices(TrailIndex);
	}

	template<typename TrailDataType> void GetTrailStart(const int32 TrailIdx, int32 &OutStartIndex, TrailDataType *&OutTrailData, FBaseParticle *&OutParticle)
	{
		if (TrailIdx != INDEX_NONE)
		{
			OutStartIndex = CurrentStartIndices[TrailIdx];
			if (OutStartIndex != INDEX_NONE)
			{
				DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * OutStartIndex);
				TrailDataType* CheckTrailData = ((TrailDataType*)((uint8*)CheckParticle + TypeDataOffset));
				check(TRAIL_EMITTER_IS_START(CheckTrailData->Flags));
				OutTrailData = CheckTrailData;
				OutParticle = CheckParticle;
			}
		}
	}

	template<typename TrailDataType> void GetTrailEnd(const int32 TrailIdx, int32 &OutEndIndex, TrailDataType *&OutTrailData, FBaseParticle *&OutParticle)
	{
		if (TrailIdx != INDEX_NONE)
		{
			OutEndIndex = CurrentEndIndices[TrailIdx];
			if (OutEndIndex != INDEX_NONE)
			{
				DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * OutEndIndex);
				TrailDataType* CheckTrailData = ((TrailDataType*)((uint8*)CheckParticle + TypeDataOffset));
				check(TRAIL_EMITTER_IS_END(CheckTrailData->Flags));
				OutTrailData = CheckTrailData;
				OutParticle = CheckParticle;
			}
		}
	}


	void SetEndIndex(int32 TrailIndex, int32 ParticleIndex)
	{
		CurrentEndIndices[TrailIndex] = ParticleIndex;
		if (CurrentStartIndices[TrailIndex] == ParticleIndex)
		{
			CurrentStartIndices[TrailIndex] = INDEX_NONE;
		}
		CheckIndices(TrailIndex);
	}

	void SetDeadIndex(int32 TrailIndex, int32 ParticleIndex)
	{
		if (CurrentStartIndices[TrailIndex] == ParticleIndex)
		{
			CurrentStartIndices[TrailIndex] = INDEX_NONE;
		}

		if (CurrentEndIndices[TrailIndex] == ParticleIndex)
		{
			CurrentEndIndices[TrailIndex] = INDEX_NONE;
		}
		CheckIndices(TrailIndex);
	}

	void ClearIndices(int32 TrailIndex, int32 ParticleIndex)
	{
		SetDeadIndex(TrailIndex, ParticleIndex);
		CheckIndices(TrailIndex);
	}

	void CheckIndices(int32 TrailIdx)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
 		if (CurrentEndIndices[TrailIdx] != INDEX_NONE)
 		{
 			DECLARE_PARTICLE_PTR(EndParticle, ParticleData + ParticleStride * CurrentEndIndices[TrailIdx]);
			FTrailsBaseTypeDataPayload* EndTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)EndParticle + TypeDataOffset));
 			check(TRAIL_EMITTER_IS_END(EndTrailData->Flags));
 		}
 
 		if (CurrentStartIndices[TrailIdx] != INDEX_NONE)
 		{
 			DECLARE_PARTICLE_PTR(StartParticle, ParticleData + ParticleStride * CurrentStartIndices[TrailIdx]);
			FTrailsBaseTypeDataPayload* StartTrailData = ((FTrailsBaseTypeDataPayload*)((uint8*)StartParticle + TypeDataOffset));
 			check(TRAIL_EMITTER_IS_START(StartTrailData->Flags));
 		}
#endif
	}

	void CheckAllIndices()
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (uint32 TrailIdx = 0; TrailIdx < 128; TrailIdx++)
		{
			if (CurrentEndIndices[TrailIdx] != INDEX_NONE)
			{
				DECLARE_PARTICLE_PTR(EndParticle, ParticleData + ParticleStride * CurrentEndIndices[TrailIdx]);
				FRibbonTypeDataPayload* EndTrailData = ((FRibbonTypeDataPayload*)((uint8*)EndParticle + TypeDataOffset));
				check(TRAIL_EMITTER_IS_END(EndTrailData->Flags));
			}

			if (CurrentStartIndices[TrailIdx] != INDEX_NONE)
			{
				DECLARE_PARTICLE_PTR(StartParticle, ParticleData + ParticleStride * CurrentStartIndices[TrailIdx]);
				FRibbonTypeDataPayload* StartTrailData = ((FRibbonTypeDataPayload*)((uint8*)StartParticle + TypeDataOffset));
				check(TRAIL_EMITTER_IS_START(StartTrailData->Flags));
			}
		}
#endif
	}

#else //ENABLE_TRAILS_START_END_INDEX_OPTIMIZATION

template<typename TrailDataType> void GetTrailStart(const int32 TrailIdx, int32 &OutStartIndex, TrailDataType *&OutTrailData, FBaseParticle *&OutParticle)
{
	for (int32 FindTrailIdx = 0; FindTrailIdx < ActiveParticles; FindTrailIdx++)
	{
		int32 CheckIndex = ParticleIndices[FindTrailIdx];
		DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * CheckIndex);
		TrailDataType* CheckTrailData = ((TrailDataType*)((uint8*)CheckParticle + TypeDataOffset));
		if (TRAIL_EMITTER_IS_START(CheckTrailData->Flags))
		{
			if (CheckTrailData->TrailIndex == TrailIdx)
			{
				OutStartIndex = CheckIndex;
				OutParticle = CheckParticle;
				OutTrailData = CheckTrailData;				
				break;
			}
		}
	}
}
template<typename TrailDataType> void GetTrailEnd(const int32 TrailIdx, int32 &OutEndIndex, TrailDataType *&OutTrailData, FBaseParticle *&OutParticle)
{
	for (int32 FindTrailIdx = 0; FindTrailIdx < ActiveParticles; FindTrailIdx++)
	{
		int32 CheckIndex = ParticleIndices[FindTrailIdx];
		DECLARE_PARTICLE_PTR(CheckParticle, ParticleData + ParticleStride * CheckIndex);
		TrailDataType* CheckTrailData = ((TrailDataType*)((uint8*)CheckParticle + TypeDataOffset));
		if (TRAIL_EMITTER_IS_END(CheckTrailData->Flags))
		{
			if (CheckTrailData->TrailIndex == TrailIdx)
			{
				OutEndIndex = CheckIndex;
				OutParticle = CheckParticle;
				OutTrailData = CheckTrailData;
				break;
			}
		}
	}
}
void SetStartIndex(int32 TrailIndex, int32 ParticleIndex){}
void SetEndIndex(int32 TrailIndex, int32 ParticleIndex){}
void SetDeadIndex(int32 TrailIndex, int32 ParticleIndex){}
void ClearIndices(int32 TrailIndex, int32 ParticleIndex){}
void CheckIndices(int32 TrailIdx){}
void CheckAllIndices(){}

#endif

	void DumpCircularTrailsSpam();
#if !UE_BUILD_SHIPPING
	bool CheckForCircularTrail(FBaseParticle* StartParticle, FBaseParticle* CheckParticle)
	{
		if (StartParticle == CheckParticle)
		{
			DumpCircularTrailsSpam();
			return true;
		}
		return false;
	}
#else
	bool CheckForCircularTrail(FBaseParticle* StartParticle, FBaseParticle* CheckParticle) { return StartParticle == CheckParticle; }
#endif

	/** Constructor	*/
	FParticleTrailsEmitterInstance_Base(IParticleEmitterInstanceOwner& InComponent) :
		  FParticleEmitterInstance(InComponent)
		, VertexCount(0)
		, TriangleCount(0)
		, TrailCount(0)
		, MaxTrailCount(0)
		, RunningTime(0.0f)
		, LastTickTime(0.0f)
		, bDeadTrailsOnDeactivate(false)
		, bFirstUpdate(true)
		, bEnableInactiveTimeTracking(false)
	{
		TrailSpawnTimes.Empty();
		LastSpawnTime.Empty();
		SourceDistanceTraveled.Empty();
		TiledUDistanceTraveled.Empty();

		for (int32 TrailIdx = 0; TrailIdx < 128; TrailIdx++)
		{
#if ENABLE_TRAILS_START_END_INDEX_OPTIMIZATION
			CurrentStartIndices[TrailIdx] = INDEX_NONE;
			CurrentEndIndices[TrailIdx] = INDEX_NONE;
#endif
		}
	}

	/** Destructor	*/
	virtual ~FParticleTrailsEmitterInstance_Base() = default;

	virtual void Init() override;
	virtual void InitParameters(UParticleEmitter* InTemplate) override;
	virtual void Tick(float DeltaTime, bool bSuppressSpawning) override;

	bool AddParticleHelper(int32 InTrailIdx,
		int32 StartParticleIndex, FTrailsBaseTypeDataPayload* StartTrailData,
		int32 ParticleIndex, FTrailsBaseTypeDataPayload* TrailData);

	/**
	 *	Tick sub-function that handles recalculation of tangents
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	virtual void Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* CurrentLODLevel);
	virtual void UpdateBoundingBox(float DeltaTime) override;
	virtual void ForceUpdateBoundingBox() override;
	virtual void KillParticles() override;

	/**
	 *	Kill the given number of particles from the end of the trail.
	 *
	 *	@param	InTrailIdx		The trail to kill particles in.
	 *	@param	InKillCount		The number of particles to kill off.
	 */
	virtual void KillParticles(int32 InTrailIdx, int32 InKillCount);

	virtual void SetupTrailModules() {};
	virtual void UpdateSourceData(float DeltaTime, bool bFirstTime);
	// Update the start particles of the trail, if needed
//	virtual void UpdateStartParticles(float DeltaTime, bool bFirstTime) {}

	/**
	 *	Called when the particle system is deactivating...
	 */
	virtual void OnDeactivateSystem() override;

protected:
	/**
	 * Get the current material to render with.
	 */
	UMaterialInterface* GetCurrentMaterial();

	enum EGetTrailDirection
	{
		GET_Prev,
		GET_Next
	};
	enum EGetTrailParticleOption
	{
		GET_Any,			// Grab the prev/next particle
		GET_Spawned,		// Grab the first prev/next particle that was true spawned
		GET_Interpolated,	// Grab the first prev/next particle that was interpolation spawned
		GET_Start,			// Grab the start particle for the trail the particle is in
		GET_End,			// Grab the end particle for the trail the particle is in
	};

	/**
	 *	Retrieve the particle in the trail that meets the given criteria
	 *
	 *	@param	bSkipStartingParticle		If true, don't check the starting particle for meeting the criteria
	 *	@param	InStartingFromParticle		The starting point for the search.
	 *	@param	InStartingTrailData			The trail data for the starting point.
	 *	@param	InGetDirection				Direction to search the trail.
	 *	@param	InGetOption					Options for defining the type of particle.
	 *	@param	OutParticle					The particle that meets the criteria.
	 *	@param	OutTrailData				The trail data of the particle that meets the criteria.
	 *
	 *	@return	bool						true if found, false if not.
	 */
	bool GetParticleInTrail(
		bool bSkipStartingParticle,
		FBaseParticle* InStartingFromParticle,
		FTrailsBaseTypeDataPayload* InStartingTrailData,
		EGetTrailDirection InGetDirection,
		EGetTrailParticleOption InGetOption,
		FBaseParticle*& OutParticle,
		FTrailsBaseTypeDataPayload*& OutTrailData);

	/** Prints out info for a single particle. */
	virtual void PrintParticleData(FBaseParticle* Particle, FTrailsBaseTypeDataPayload* TrailData, int32 CurrentIndex, int32 TrailIndex){}
	/** Prints out info for all active particles. */
	virtual void PrintAllActiveParticles(){}
	/** Traverses all trails and prints out debugging info. */
	virtual void PrintTrails(){}
};

struct FParticleRibbonEmitterInstance : public FParticleTrailsEmitterInstance_Base
{
	/** The TypeData module for this trail emitter */
	UParticleModuleTypeDataRibbon*	TrailTypeData;

	/** SpawnPerUnit module (hijacking it for trails here) */
	UParticleModuleSpawnPerUnit*	SpawnPerUnitModule;

	/** Source module */
	UParticleModuleTrailSource*		SourceModule;
	/** Payload offset for source module */
	int32								TrailModule_Source_Offset;

	/** The current source position for each trail in this emitter */
	TArray<FVector> CurrentSourcePosition;
	/** The current source rotation for each trail in this emitter */
	TArray<FQuat> CurrentSourceRotation;
	/** The current source up for each trail in this emitter */
	TArray<FVector> CurrentSourceUp;
	/** The current source tangent for each trail in this emitter */
	TArray<FVector> CurrentSourceTangent;
	/** The current source tangent strength for each trail in this emitter */
	TArray<float> CurrentSourceTangentStrength;
	/** The previous source position for each trail in this emitter */
	TArray<FVector> LastSourcePosition;
	/** The last source rotation for each trail in this emitter */
	TArray<FQuat> LastSourceRotation;
	/** The previous source up for each trail in this emitter */
	TArray<FVector> LastSourceUp;
	/** The previous source tangent for each trail in this emitter */
	TArray<FVector> LastSourceTangent;
	/** The previous source tangent strength for each trail in this emitter */
	TArray<float> LastSourceTangentStrength;
	/** If the source is an actor, this is it */
	AActor* SourceActor;
	/** The offset from the source for each trail in this emitter */
	TArray<FVector> SourceOffsets;
	/** If the source is an emitter, this is it */
	FParticleEmitterInstance* SourceEmitter;
	/** The last selected source index (for sequential selection) */
	int32 LastSelectedParticleIndex;
	/** The indices for the source of each trail (if required) */
	TArray<int32> SourceIndices;
	/** The time of the last partice source update */
	TArray<float> SourceTimes;
	/** The time of the last partice source update */
	TArray<float> LastSourceTimes;
	/** The lifetime to use for each ribbon */
	TArray<float> CurrentLifetimes;
	/** The size to use for each ribbon */
	TArray<float> CurrentSizes;
	/** The direct index of the particle that is the start of each ribbon */
//	TArray<int32> CurrentStartIndices;
	/** If true, then do not render the source (a real spawn just happened) */
//	TArray<bool> SkipSourceSegment;

	/** The number of "head only" active particles */
	int32 HeadOnlyParticles;

	/** Constructor	*/
	FParticleRibbonEmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	/** Destructor	*/
	virtual ~FParticleRibbonEmitterInstance();

	virtual void InitParameters(UParticleEmitter* InTemplate) override;

	/**
	 *	Tick sub-function that handles recalculation of tangents
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	virtual void Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* CurrentLODLevel) override;

	virtual bool GetSpawnPerUnitAmount(float DeltaTime, int32 InTrailIdx, int32& OutCount, float& OutRate);

	/**
	 *	Get the lifetime and size for a particle being added to the given trail
	 *	
	 *	@param	InTrailIdx				The index of the trail the particle is being added to
	 *	@param	InParticle				The particle that is being added
	 *	@param	bInNoLivingParticles	true if there are no particles in the trail, false if there already are
	 *	@param	OutOneOverMaxLifetime	The OneOverMaxLifetime value to use for the particle
	 *	@param	OutSize					The Size value to use for the particle
	 */
	void GetParticleLifetimeAndSize(int32 InTrailIdx, const FBaseParticle* InParticle, bool bInNoLivingParticles, float& OutOneOverMaxLifetime, float& OutSize);

	/**
	 *	Spawn particles for this emitter instance
	 *	@param	DeltaTime		The time slice to spawn over
	 *	@return	float			The leftover fraction of spawning
	 */
	virtual float Spawn(float DeltaTime) override;

	/**
	 *	Spawn source-based ribbon particles.
	 *
	 *	@param	DeltaTime			The current time slice
	 *
	 *	@return	bool				true if SpawnRate should be processed.
	 */
	bool Spawn_Source(float DeltaTime);
	/**
	 *	Spawn ribbon particles from SpawnRate and Burst settings.
	 *
	 *	@param	DeltaimTime			The current time slice
	 *	
	 *	@return	float				The spawnfraction left over from this time slice
	 */
	float Spawn_RateAndBurst(float DeltaTime);
	
	virtual void SetupTrailModules() override;
	void ResolveSource();
	virtual void UpdateSourceData(float DeltaTime, bool bFirstTime) override;
	// Update the start particles of the trail, if needed
//	virtual void UpdateStartParticles(float DeltaTime, bool bFirstTime);
	bool ResolveSourcePoint(int32 InTrailIdx, FVector& OutPosition, FQuat& OutRotation, FVector& OutUp, FVector& OutTangent, float& OutTangentStrength);

	/** Determine the number of vertices and triangles in each trail */
	void DetermineVertexAndTriangleCount();

	/**
	 *	Checks some common values for GetDynamicData validity
	 *
	 *	@return	bool		true if GetDynamicData should continue, false if it should return NULL
	 */
	virtual bool IsDynamicDataRequired(UParticleLODLevel* CurrentLODLevel) override;

	/**
	 *	Retrieves the dynamic data for the emitter
	 */
	virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel) override;

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	virtual FDynamicEmitterReplayDataBase* GetReplayData() override;

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	virtual void GetAllocatedSize(int32& OutNum, int32& OutMax) override;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
	 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	/**
	 * When an emitter is killed, this will check other emitters and clean up anything pointing to this one
	 */
	virtual void OnEmitterInstanceKilled(FParticleEmitterInstance* Instance) override
	{
		if (SourceEmitter == Instance)
		{
			SourceEmitter = NULL;
		}
	}
	
	/** Called on world origin changes */
	virtual void ApplyWorldOffset(FVector InOffset, bool bWorldShift) override;

protected:

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData ) override;
};

/*-----------------------------------------------------------------------------
	FParticleAnimTrailEmitterInstance.
-----------------------------------------------------------------------------*/

struct FParticleAnimTrailEmitterInstance : public FParticleTrailsEmitterInstance_Base
{
	/** The TypeData module for this trail emitter */
	UParticleModuleTypeDataAnimTrail*	TrailTypeData;

	/** SpawnPerUnit module (hijacking it for trails here) */
	UParticleModuleSpawnPerUnit*		SpawnPerUnitModule;

	/**
	 *	The name of the socket that supplies the first edge for this emitter.
	 */
	FName FirstSocketName;

	/**
	 *	The name of the socket that supplies the second edge for this emitter.
	 */
	FName SecondSocketName;

	/**
	*	The width of the trail.
	*/
	float Width;

	/**
	*	How the width is applied to the trail.
	*/
	ETrailWidthMode WidthMode;

	/**
	*	The owner of this trail. Used only for assosiating trail with it's creator in some cases. E.g. AnimNotifyState_Trail. Do not use.
	*/
	const void* Owner;
	
	/**
	*	When set, the current trail will be marked as dead in the next tick.
	*/
	bool bTagTrailAsDead;

	/**
	*	Whether new particles should be spawned.
	*/
	bool bTrailEnabled;

	/** Editor only variables controlling the debug rendering for trails.*/
#if WITH_EDITORONLY_DATA
	uint32 bRenderGeometry : 1;
	uint32 bRenderSpawnPoints : 1;
	uint32 bRenderTangents : 1;
	uint32 bRenderTessellation : 1;
#endif

	/** The number of particles in the trail which are either marked */
	int32 HeadOnlyParticles;

	/** Constructor	*/
	FParticleAnimTrailEmitterInstance(IParticleEmitterInstanceOwner& InComponent);

	/** Destructor	*/
	virtual ~FParticleAnimTrailEmitterInstance();

	virtual void InitParameters(UParticleEmitter* InTemplate) override;

	/**
	 *	Helper function for recalculating tangents and the spline interpolation parameter...
	 *
	 *	@param	PrevParticle		The previous particle in the trail
	 *	@param	PrevTrailData		The payload of the previous particle in the trail
	 *	@param	CurrParticle		The current particle in the trail
	 *	@param	CurrTrailData		The payload of the current particle in the trail
	 *	@param	NextParticle		The next particle in the trail
	 *	@param	NextTrailData		The payload of the next particle in the trail
	 */
	virtual void RecalculateTangentAndInterpolationParam(
		FBaseParticle* PrevParticle, FAnimTrailTypeDataPayload* PrevTrailData, 
		FBaseParticle* CurrParticle, FAnimTrailTypeDataPayload* CurrTrailData, 
		FBaseParticle* NextParticle, FAnimTrailTypeDataPayload* NextTrailData);

	/**
	 *	Tick sub-function that handles recalculation of tangents
	 *
	 *	@param	DeltaTime			The current time slice
	 *	@param	CurrentLODLevel		The current LOD level for the instance
	 */
	virtual void Tick_RecalculateTangents(float DeltaTime, UParticleLODLevel* CurrentLODLevel) override;

	virtual bool GetSpawnPerUnitAmount(float DeltaTime, int32 InTrailIdx, int32& OutCount, float& OutRate);

	/**
	 *	Spawn particles for this emitter instance
	 *	@param	DeltaTime		The time slice to spawn over
	 *	@return	float			The leftover fraction of spawning
	 */
	virtual float Spawn(float DeltaTime) override;

	virtual void SetupTrailModules() override;
	void ResolveSource();
	virtual void UpdateSourceData(float DeltaTime, bool bFirstTime) override;

	void UpdateBoundingBox(float DeltaTime) override;
	void ForceUpdateBoundingBox() override;

	/** Determine the number of vertices and triangles in each trail */
	void DetermineVertexAndTriangleCount();

	virtual bool HasCompleted() override;
	/**
	 *	Retrieves the dynamic data for the emitter
	 */
	virtual FDynamicEmitterDataBase* GetDynamicData(bool bSelected, ERHIFeatureLevel::Type InFeatureLevel) override;

	/**
	 *	Retrieves replay data for the emitter
	 *
	 *	@return	The replay data, or NULL on failure
	 */
	virtual FDynamicEmitterReplayDataBase* GetReplayData() override;

	/**
	 *	Retrieve the allocated size of this instance.
	 *
	 *	@param	OutNum			The size of this instance
	 *	@param	OutMax			The maximum size of this instance
	 */
	virtual void GetAllocatedSize(int32& OutNum, int32& OutMax) override;

	/**
	 * Returns the size of the object/ resource for display to artists/ LDs in the Editor.
	 *
	 * @param	Mode	Specifies which resource size should be displayed. ( see EResourceSizeMode::Type )
	 * @return  Size of resource as to be displayed to artists/ LDs in the Editor.
	 */
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;

	virtual bool IsTrailEmitter() const override { return true; }

	/**
	 * Begins the trail.
	 */
	virtual void BeginTrail() override;

	/**
	 * Ends the trail.
	 */
	virtual void EndTrail() override;

	/**
	* Sets the date that defines this trail.
	*
	* @param	InFirstSocketName	The name of the first socket for the trail.
	* @param	InSecondSocketName	The name of the second socket for the trail.
	* @param	InWidthMode			How the width value is applied to the trail.
	* @param	InWidth				The width of the trail.
	*/
	virtual void SetTrailSourceData(FName InFirstSocketName, FName InSecondSocketName, ETrailWidthMode InWidthMode, float InWidth) override;

	bool IsTrailActive()const;

#if WITH_EDITORONLY_DATA
	/**
	* Sets various debug variables for trails.
	*
	* @param	bRenderGeometry		When true, the trails geometry is rendered.
	* @param	bRenderSpawnPoints	When true, the trails spawn points are rendered.
	* @param	bRenderTessellation	When true, the trails tessellated points are rendered.
	* @param	bRenderTangents		When true, the trails tangents are rendered.
	*/
	void SetTrailDebugData(bool bInRenderGeometry, bool bInRenderSpawnPoints, bool bInRenderTessellation, bool bInRenderTangents);
#endif

protected:

	/**
	 * Captures dynamic replay data for this particle system.
	 *
	 * @param	OutData		[Out] Data will be copied here
	 *
	 * @return Returns true if successful
	 */
	virtual bool FillReplayData( FDynamicEmitterReplayDataBase& OutData ) override;

	/** 
	*	Helper to spawn a trail particle during SpawnParticles(). 
	*	@param	StartParticleIndex	Index of the particle at the start of the trail. Is altered in the function.
	*	@param	Params			Various parameters for spawning.
	*/
	void SpawnParticle( int32& StartParticleIndex, const struct FAnimTrailParticleSpawnParams& Params );

	/** Prints out info for a single particle. */
	virtual void PrintParticleData(FBaseParticle* Particle, FTrailsBaseTypeDataPayload* TrailData, int32 CurrentIndex, int32 TrailIndex) override;
	/** Prints out info for all active particles. */
	virtual void PrintAllActiveParticles() override;
	/** Traverses all trails and prints out debugging info. */
	virtual void PrintTrails() override;
};
