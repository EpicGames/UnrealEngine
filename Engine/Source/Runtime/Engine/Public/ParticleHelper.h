// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleHelper.h: Particle helper definitions/ macros.
=============================================================================*/

#pragma once

#include "Math/Quat.h"
#include "Math/RandomStream.h"
#include "Math/Vector4.h"
#include "ParticleVertexFactory.h"
#include "Particles/ParticlePerfStats.h"
#include "ParticleHelper.generated.h"

#define _ENABLE_PARTICLE_LOD_INGAME_

class FGlobalDynamicIndexBuffer;
class FGlobalDynamicVertexBuffer;
class FMaterialRenderProxy;
class FMeshElementCollector;
class FMeshParticleVertexFactory;
class FParticleSystemSceneProxy;
class FPrimitiveDrawInterface;
class FSceneView;
class FSceneViewFamily;
class FSimpleLightArray;
class UMaterialInterface;
class UParticleModuleRequired;
class UStaticMesh;
namespace ERHIFeatureLevel { enum Type : int; }
struct FBaseParticle;
struct FParticleMeshEmitterInstance;
struct FStaticMeshLODResources;
struct FGlobalDynamicIndexBufferAllocation;
struct FGlobalDynamicVertexBufferAllocation;

DECLARE_LOG_CATEGORY_EXTERN(LogParticles, Log, All);

/** Detail mode for scene component rendering. */
UENUM()
enum EParticleDetailMode : int
{
	PDM_Low UMETA(DisplayName = "Low"),
	PDM_Medium UMETA(DisplayName = "Medium"),
	PDM_High UMETA(DisplayName = "High"),
	PDM_Epic UMETA(DisplayName = "Epic"),
	PDM_MAX UMETA(Hidden),
};
inline const int32 PDM_DefaultValue = 0xFFFF;


/*-----------------------------------------------------------------------------
	Helper macros.
-----------------------------------------------------------------------------*/
//	Macro fun.
#define _PARTICLES_USE_PREFETCH_
#if defined(_PARTICLES_USE_PREFETCH_)
	#define	PARTICLE_PREFETCH(Index)					FPlatformMisc::Prefetch( ParticleData, ParticleStride * ParticleIndices[Index] )
	#define PARTICLE_INSTANCE_PREFETCH(Instance, Index)	FPlatformMisc::Prefetch( Instance->ParticleData, Instance->ParticleStride * Instance->ParticleIndices[Index] )
	#define	PARTICLE_OWNER_PREFETCH(Index)				FPlatformMisc::Prefetch( Owner->ParticleData, Owner->ParticleStride * Owner->ParticleIndices[Index] )
#else	//#if defined(_PARTICLES_USE_PREFETCH_)
	#define	PARTICLE_PREFETCH(Index)					
	#define	PARTICLE_INSTANCE_PREFETCH(Instance, Index)	
	#define	PARTICLE_OWNER_PREFETCH(Index)				
#endif	//#if defined(_PARTICLES_USE_PREFETCH_)

#define DECLARE_PARTICLE(Name,Address)		\
	FBaseParticle& Name = *((FBaseParticle*) (Address));

#define DECLARE_PARTICLE_CONST(Name,Address)		\
	const FBaseParticle& Name = *((const FBaseParticle*) (Address));

#define DECLARE_PARTICLE_PTR(Name,Address)		\
	FBaseParticle* Name = (FBaseParticle*) (Address);

#define BEGIN_UPDATE_LOOP																								\
	{																													\
		int32&			ActiveParticles = Context.Owner.ActiveParticles;												\
		int32			Offset	= Context.Offset;																		\
		uint32			CurrentOffset	= Offset;																		\
		float			DeltaTime = Context.DeltaTime;																	\
		const uint8*	ParticleData	= Context.Owner.ParticleData;													\
		const uint32	ParticleStride	= Context.Owner.ParticleStride;													\
		uint16*			ParticleIndices	= Context.Owner.ParticleIndices;												\
		for(int32 i=ActiveParticles-1; i>=0; i--)																		\
		{																												\
			const int32	CurrentIndex	= ParticleIndices[i];															\
			const uint8* ParticleBase	= ParticleData + CurrentIndex * ParticleStride;									\
			FBaseParticle& Particle		= *((FBaseParticle*) ParticleBase);												\
			if ((Particle.Flags & STATE_Particle_Freeze) == 0)															\
			{																											\

#define END_UPDATE_LOOP																									\
			}																											\
			CurrentOffset				= Offset;																		\
		}																												\
	}

#define CONTINUE_UPDATE_LOOP																							\
		CurrentOffset = Offset;																							\
		continue;

#define SPAWN_INIT																										\
	const int32		ActiveParticles	= Context.Owner.ActiveParticles;													\
	const uint32	ParticleStride	= Context.Owner.ParticleStride;														\
	uint32			CurrentOffset	= Context.Offset;																	\
	FBaseParticle*	ParticleBase	= Context.ParticleBase;																\
	FBaseParticle&	Particle		= *(ParticleBase);

#define PARTICLE_ELEMENT(Type,Name)																						\
	Type& Name = *((Type*)((uint8*)ParticleBase + CurrentOffset));																\
	CurrentOffset += sizeof(Type);

#define KILL_CURRENT_PARTICLE																							\
	{																													\
		ParticleIndices[i]					= ParticleIndices[ActiveParticles-1];										\
		ParticleIndices[ActiveParticles-1]	= CurrentIndex;																\
		ActiveParticles--;																								\
	}

/*-----------------------------------------------------------------------------
	Helper functions.
-----------------------------------------------------------------------------*/

inline void Particle_SetColorFromVector(const FVector3f& InColorVec, const float InAlpha, FLinearColor& OutColor)
{
	OutColor.R = InColorVec.X;
	OutColor.G = InColorVec.Y;
	OutColor.B = InColorVec.Z;
	OutColor.A = InAlpha;
}

/*-----------------------------------------------------------------------------
	Forward declarations
-----------------------------------------------------------------------------*/
//	Emitter and module types
class UParticleEmitter;
class UParticleSpriteEmitter;
class UParticleModule;
// Data types
class UParticleModuleTypeDataMesh;
class UParticleModuleTypeDataBeam2;
class UParticleModuleTypeDataTrail2;

class UParticleSystem;

class UParticleModuleBeamSource;
class UParticleModuleBeamTarget;
class UParticleModuleBeamNoise;
class UParticleModuleBeamModifier;

class UParticleModuleTrailSource;
class UParticleModuleTrailSpawn;
class UParticleModuleTrailTaper;

class UParticleModuleOrientationAxisLock;

class UParticleLODLevel;

struct FDynamicBeam2EmitterData;
struct FDynamicTrail2EmitterData;

struct FParticleSpriteEmitterInstance;
struct FParticleMeshEmitterInstance;
struct FParticleBeam2EmitterInstance;

struct FStaticMeshLODResources;

// Special module indices...
#define INDEX_TYPEDATAMODULE	(INDEX_NONE - 1)
#define INDEX_REQUIREDMODULE	(INDEX_NONE - 2)
#define INDEX_SPAWNMODULE		(INDEX_NONE - 3)

/*-----------------------------------------------------------------------------
	FBaseParticle
-----------------------------------------------------------------------------*/
// Mappings for 'standard' particle data
// Only used when required.
struct FBaseParticle
{
	// 48 bytes
	FVector		OldLocation;			// Last frame's location, used for collision
	FVector		Location;				// Current location

	// 16 bytes
	FVector3f		BaseVelocity;			// Velocity = BaseVelocity at the start of each frame.
	float			Rotation;				// Rotation of particle (in Radians)

	// 16 bytes
	FVector3f		Velocity;				// Current velocity, gets reset to BaseVelocity each frame to allow 
	float			BaseRotationRate;		// Initial angular velocity of particle (in Radians per second)

	// 16 bytes
	FVector3f		BaseSize;				// Size = BaseSize at the start of each frame
	float			RotationRate;			// Current rotation rate, gets reset to BaseRotationRate each frame

	// 16 bytes
	FVector3f		Size;					// Current size, gets reset to BaseSize each frame
	int32			Flags;					// Flags indicating various particle states

	// 16 bytes
	FLinearColor	Color;					// Current color of particle.

	// 16 bytes
	FLinearColor	BaseColor;				// Base color of the particle

	// 16 bytes
	float			RelativeTime;			// Relative time, range is 0 (==spawn) to 1 (==death)
	float			OneOverMaxLifetime;		// Reciprocal of lifetime
	float			Placeholder0;
	float			Placeholder1;
};

/*-----------------------------------------------------------------------------
	Particle State Flags
-----------------------------------------------------------------------------*/
enum EParticleStates
{
	/** Ignore updates to the particle						*/
	STATE_Particle_JustSpawned			= 0x02000000,
	/** Ignore updates to the particle						*/
	STATE_Particle_Freeze				= 0x04000000,
	/** Ignore collision updates to the particle			*/
	STATE_Particle_IgnoreCollisions		= 0x08000000,
	/**	Stop translations of the particle					*/
	STATE_Particle_FreezeTranslation	= 0x10000000,
	/**	Stop rotations of the particle						*/
	STATE_Particle_FreezeRotation		= 0x20000000,
	/** Combination for a single check of 'ignore' flags	*/
	STATE_Particle_CollisionIgnoreCheck	= STATE_Particle_Freeze |STATE_Particle_IgnoreCollisions | STATE_Particle_FreezeTranslation| STATE_Particle_FreezeRotation,
	/** Delay collision updates to the particle				*/
	STATE_Particle_DelayCollisions		= 0x40000000,
	/** Flag indicating the particle has had at least one collision	*/
	STATE_Particle_CollisionHasOccurred	= 0x80000000,
	/** State mask. */
	STATE_Mask = 0xFE000000,
	/** Counter mask. */
	STATE_CounterMask = (~STATE_Mask)
};

/*-----------------------------------------------------------------------------
	FParticlesStatGroup
-----------------------------------------------------------------------------*/

DECLARE_STATS_GROUP(TEXT("ParticlesOverview"), STATGROUP_ParticlesOverview, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("GT Total"), STAT_ParticlesOverview_GT, STATGROUP_ParticlesOverview, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GT Concurrent Total"), STAT_ParticlesOverview_GT_CNC, STATGROUP_ParticlesOverview, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RT Total"), STAT_ParticlesOverview_RT, STATGROUP_ParticlesOverview, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("RT Concurrent Total"), STAT_ParticlesOverview_RT_CNC, STATGROUP_ParticlesOverview, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprite Particles"),STAT_SpriteParticles,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprite Ptcls Spawned"),STAT_SpriteParticlesSpawned,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprite Ptcls Updated"),STAT_SpriteParticlesUpdated,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprite Ptcls Killed"),STAT_SpriteParticlesKilled,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Particle Draw Calls"),STAT_ParticleDrawCalls,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sort Time RT"),STAT_SortingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Render Time RT"),STAT_SpriteRenderingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Tick Time GT"),STAT_SpriteTickTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Spawn Time GT"),STAT_SpriteSpawnTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PSys Comp Tick Time GT"),STAT_PSysCompTickTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Collision Time GT"),STAT_ParticleCollisionTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle SkelMeshSurf Time GT"),STAT_ParticleSkelMeshSurfTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Pool Time RT"),STAT_ParticlePoolTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Compute Time GT,AT"),STAT_ParticleComputeTickTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Finalize Time GT"),STAT_ParticleFinalizeTickTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle GT Stall Time"),STAT_GTSTallTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Render Time RT"),STAT_ParticleRenderingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Packing Time RT"),STAT_ParticlePackingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetTemplate Time GT"),STAT_ParticleSetTemplateTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Initialize Time GT"),STAT_ParticleInitializeTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Activate Time GT"),STAT_ParticleActivateTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async Work Time"),STAT_ParticleAsyncTime,STATGROUP_Particles, );           // regardless of if it is actually performed on other threads or not
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait For ASync Time"),STAT_ParticleAsyncWaitTime,STATGROUP_Particles, );   // can be either performed on this thread or a true wait
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Bounds Time GT"),STAT_ParticleUpdateBounds,STATGROUP_Particles, );

DECLARE_MEMORY_STAT_EXTERN(TEXT("Ptcls Data GT Mem"),STAT_GTParticleData,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ptcls Data GT Mem MAX"),STAT_GTParticleData_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ptcls Data RT Mem"),STAT_RTParticleData,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ptcls Data RT Mem MAX"),STAT_RTParticleData_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ptcls Data RT Largest"),STAT_RTParticleData_Largest,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ptcls Data RT Largest MAX"),STAT_RTParticleData_Largest_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynPSysComp Mem"),STAT_DynamicPSysCompMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynPSysComp Mem MAX"),STAT_DynamicPSysCompMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter Mem"),STAT_DynamicEmitterMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter Mem MAX"),STAT_DynamicEmitterMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter GTMem Waste"),STAT_DynamicEmitterGTMem_Waste,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter GTMem Largest"),STAT_DynamicEmitterGTMem_Largest,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter GTMem Waste MAX"),STAT_DynamicEmitterGTMem_Waste_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter GTMem Largest MAX"),STAT_DynamicEmitterGTMem_Largest_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynPSysComp Count"),STAT_DynamicPSysCompCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynPSysComp Count MAX"),STAT_DynamicPSysCompCount_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter Count"),STAT_DynamicEmitterCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynEmitter Count Max"),STAT_DynamicEmitterCount_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSprite Count"),STAT_DynamicSpriteCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSprite Max"),STAT_DynamicSpriteCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynSprite GT Mem"),STAT_DynamicSpriteGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSprite GT Mem Max"),STAT_DynamicSpriteGTMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSubUV Count"),STAT_DynamicSubUVCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSubUV Max"),STAT_DynamicSubUVCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynSubUV GT Mem"),STAT_DynamicSubUVGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynSubUV GT Mem Max"),STAT_DynamicSubUVGTMem_Max,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynMesh Count"),STAT_DynamicMeshCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynMesh Max"),STAT_DynamicMeshCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynMesh GT Mem"),STAT_DynamicMeshGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynMesh GT Mem Max"),STAT_DynamicMeshGTMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynBeam Count"),STAT_DynamicBeamCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynBeam Max"),STAT_DynamicBeamCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynBeam GT Mem"),STAT_DynamicBeamGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynBeam GT Mem Max"),STAT_DynamicBeamGTMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynRibbon Count"),STAT_DynamicRibbonCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynRibbon Max"),STAT_DynamicRibbonCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynRibbon GT Mem"),STAT_DynamicRibbonGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynRibbon GT Mem Max"),STAT_DynamicRibbonGTMem_MAX,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynAnimTrail Count"),STAT_DynamicAnimTrailCount,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynAnimTrail Max"),STAT_DynamicAnimTrailCount_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynAnimTrail GT Mem"),STAT_DynamicAnimTrailGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynAnimTrail GT Mem Max"),STAT_DynamicAnimTrailGTMem_MAX,STATGROUP_ParticleMem, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("DynUntracked Mem"),STAT_DynamicUntrackedGTMem,STATGROUP_ParticleMem, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DynUntracked Mem Max"),STAT_DynamicUntrackedGTMem_MAX,STATGROUP_ParticleMem, );

// GPU Particle stats.
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprites"),STAT_GPUSpriteParticles,STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sprites Spawned"),STAT_GPUSpritesSpawned,STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sorted Emitters"),STAT_SortedGPUEmitters,STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Sorted Particles"),STAT_SortedGPUParticles,STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Free Tiles"),STAT_FreeGPUTiles,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Tick Time"),STAT_GPUSpriteTickTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Spawn Time"),STAT_GPUSpriteSpawnTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite PreRender Time"),STAT_GPUSpritePreRenderTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sprite Render Time"),STAT_GPUSpriteRenderingTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("GPU Particle Tick Time"),STAT_GPUParticleTickTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Build Sim Commands"),STAT_GPUParticleBuildSimCmdsTime,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Misc1"),STAT_GPUParticleMisc1,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Misc2"),STAT_GPUParticleMisc2,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Misc3"),STAT_GPUParticleMisc3,STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Injection Tick Time"), STAT_GPUParticlesInjectionTime, STATGROUP_GPUParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Execute Simulation Commands Time"), STAT_GPUParticlesSimulationCommands, STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Single Iteration Emitters"), STAT_GPUSingleIterationEmitters, STATGROUP_GPUParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Multi Iterations Emitters"), STAT_GPUMultiIterationsEmitters, STATGROUP_GPUParticles, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Mesh Particles"),STAT_MeshParticles,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Mesh Render Time RT"),STAT_MeshRenderingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Mesh Tick Time GT,AT"),STAT_MeshTickTime,STATGROUP_Particles, );

/**
 * Per-particle data sent to the GPU.
 */
struct FParticleSpriteVertex
{
	/** The position of the particle. */
	FVector3f Position;
	/** The relative time of the particle. */
	float RelativeTime;
	/** The previous position of the particle. */
	FVector3f	OldPosition;
	/** Value that remains constant over the lifetime of a particle. */
	float ParticleId;
	/** The size of the particle. */
	FVector2f Size;
	/** The rotation of the particle. */
	float Rotation;
	/** The sub-image index for the particle. */
	float SubImageIndex;
	/** The color of the particle. */
	FLinearColor Color;
};

/**
 * Per-particle data sent to the GPU.
 */
struct FParticleSpriteVertexNonInstanced
{
	/** The texture UVs. */
	FVector2f UV;
	/** The position of the particle. */
	FVector3f Position;
	/** The relative time of the particle. */
	float RelativeTime;
	/** The previous position of the particle. */
	FVector3f	OldPosition;
	/** Value that remains constant over the lifetime of a particle. */
	float ParticleId;
	/** The size of the particle. */
	FVector2f Size;
	/** The rotation of the particle. */
	float Rotation;
	/** The sub-image index for the particle. */
	float SubImageIndex;
	/** The color of the particle. */
	FLinearColor Color;
};


//	FParticleSpriteVertexDynamicParameter
struct FParticleVertexDynamicParameter
{
	/** The dynamic parameter of the particle			*/
	float			DynamicValue[4];
};

//	FParticleBeamTrailVertex
struct FParticleBeamTrailVertex
{
	/** The position of the particle. */
	FVector3f Position;
	/** The relative time of the particle. */
	float RelativeTime;
	/** The previous position of the particle. */
	FVector3f	OldPosition;
	/** Value that remains constant over the lifetime of a particle. */
	float ParticleId;
	/** The size of the particle. */
	FVector2f Size;
	/** The rotation of the particle. */
	float Rotation;
	/** The sub-image index for the particle. */
	float SubImageIndex;
	/** The color of the particle. */
	FLinearColor Color;

	float			Tex_U;
	float			Tex_V;

	/** The second UV set for the particle				*/
	float			Tex_U2;
	float			Tex_V2;
};

//	FParticleBeamTrailVertexDynamicParameter
struct FParticleBeamTrailVertexDynamicParameter
{
	/** The dynamic parameter of the particle			*/
	float			DynamicValue[4];
};

// Per-particle data sent to the GPU.
struct FMeshParticleInstanceVertex
{
	/** The color of the particle. */
	FLinearColor Color;

	/** The instance to world transform of the particle. Translation vector is packed into W components. */
	FVector4f Transform[3];

	/** The velocity of the particle, XYZ: direction, W: speed. */
	FVector4f Velocity;

	/** The sub-image texture offsets for the particle. */
	int16 SubUVParams[4];

	/** The sub-image lerp value for the particle. */
	float SubUVLerp;

	/** The relative time of the particle. */
	float RelativeTime;
};

struct FMeshParticleInstanceVertexDynamicParameter
{
	/** The dynamic parameter of the particle. */
	float DynamicValue[4];
};

struct FMeshParticleInstanceVertexPrevTransform
{
	FVector4f PrevTransform0;
	FVector4f PrevTransform1;
	FVector4f PrevTransform2;
};

//
//  Trail emitter flags and macros
//
// ForceKill: Indicates all the particles in the trail should be killed in the next KillParticles call.
#define TRAIL_EMITTER_FLAG_FORCEKILL	0x00000000
// DeadTrail: indicates that the particle is the start of a trail than should no longer spawn.
//			  It should just fade out as the particles die...
#define TRAIL_EMITTER_FLAG_DEADTRAIL	0x10000000
// Middle: indicates the particle is in the middle of a trail.
#define TRAIL_EMITTER_FLAG_MIDDLE       0x20000000
// Start: indicates the particle is the start of a trail.
#define TRAIL_EMITTER_FLAG_START        0x40000000
// End: indicates the particle is the end of a trail.
#define TRAIL_EMITTER_FLAG_END          0x80000000

//#define TRAIL_EMITTER_FLAG_ONLY	        (TRAIL_EMITTER_FLAG_START | TRAIL_EMITTER_FLAG_END)
#define TRAIL_EMITTER_FLAG_MASK         0xf0000000
#define TRAIL_EMITTER_PREV_MASK         0x0fffc000
#define TRAIL_EMITTER_PREV_SHIFT        14
#define TRAIL_EMITTER_NEXT_MASK         0x00003fff
#define TRAIL_EMITTER_NEXT_SHIFT        0

#define TRAIL_EMITTER_NULL_PREV			(TRAIL_EMITTER_PREV_MASK >> TRAIL_EMITTER_PREV_SHIFT)
#define TRAIL_EMITTER_NULL_NEXT			(TRAIL_EMITTER_NEXT_MASK >> TRAIL_EMITTER_NEXT_SHIFT)

// Helper macros
#define TRAIL_EMITTER_CHECK_FLAG(val, mask, flag)				((val & mask) == flag)
#define TRAIL_EMITTER_SET_FLAG(val, mask, flag)					((val & ~mask) | flag)
#define TRAIL_EMITTER_GET_PREVNEXT(val, mask, shift)			((val & mask) >> shift)
#define TRAIL_EMITTER_SET_PREVNEXT(val, mask, shift, setval)	((val & ~mask) | ((setval << shift) & mask))

// Start/end accessor macros
#define TRAIL_EMITTER_IS_START(index)       TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_START)
#define TRAIL_EMITTER_SET_START(index)      TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_START)

#define TRAIL_EMITTER_IS_END(index)			TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_END)
#define TRAIL_EMITTER_SET_END(index)		TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_END)

#define TRAIL_EMITTER_IS_MIDDLE(index)		TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_MIDDLE)
#define TRAIL_EMITTER_SET_MIDDLE(index)		TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_MIDDLE)

// Only is used for the first emission from the emitter
#define TRAIL_EMITTER_IS_ONLY(index)		(TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_START)	&& \
											(TRAIL_EMITTER_GET_NEXT(index) == TRAIL_EMITTER_NULL_NEXT))
#define TRAIL_EMITTER_SET_ONLY(index)		TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_START)

#define TRAIL_EMITTER_IS_FORCEKILL(index)	TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_FORCEKILL)
#define TRAIL_EMITTER_SET_FORCEKILL(index)	TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_FORCEKILL)

#define TRAIL_EMITTER_IS_DEADTRAIL(index)	TRAIL_EMITTER_CHECK_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_DEADTRAIL)
#define TRAIL_EMITTER_SET_DEADTRAIL(index)	TRAIL_EMITTER_SET_FLAG(index, TRAIL_EMITTER_FLAG_MASK, TRAIL_EMITTER_FLAG_DEADTRAIL)

#define TRAIL_EMITTER_IS_HEAD(index)		(TRAIL_EMITTER_IS_START(index) || TRAIL_EMITTER_IS_DEADTRAIL(index))
#define TRAIL_EMITTER_IS_HEADONLY(index)	((TRAIL_EMITTER_IS_START(index) || TRAIL_EMITTER_IS_DEADTRAIL(index)) && \
											(TRAIL_EMITTER_GET_NEXT(index) == TRAIL_EMITTER_NULL_NEXT))

// Prev/Next accessor macros
#define TRAIL_EMITTER_GET_PREV(index)       TRAIL_EMITTER_GET_PREVNEXT(index, TRAIL_EMITTER_PREV_MASK, TRAIL_EMITTER_PREV_SHIFT)
#define TRAIL_EMITTER_SET_PREV(index, prev) TRAIL_EMITTER_SET_PREVNEXT(index, TRAIL_EMITTER_PREV_MASK, TRAIL_EMITTER_PREV_SHIFT, prev)
#define TRAIL_EMITTER_GET_NEXT(index)       TRAIL_EMITTER_GET_PREVNEXT(index, TRAIL_EMITTER_NEXT_MASK, TRAIL_EMITTER_NEXT_SHIFT)
#define TRAIL_EMITTER_SET_NEXT(index, next) TRAIL_EMITTER_SET_PREVNEXT(index, TRAIL_EMITTER_NEXT_MASK, TRAIL_EMITTER_NEXT_SHIFT, next)

/**
 * Particle trail stats
 */
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Particles"),STAT_TrailParticles,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Ptcl Render Calls"),STAT_TrailParticlesRenderCalls,STATGROUP_TrailParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Ptcls Spawned"),STAT_TrailParticlesSpawned,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Tick Calls"),STAT_TrailParticlesTickCalls,STATGROUP_TrailParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Ptcls Killed"),STAT_TrailParticlesKilled,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Trail Ptcl Tris"),STAT_TrailParticlesTrianglesRendered,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trail FillVertex Time RT"),STAT_TrailFillVertexTime,STATGROUP_TrailParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trail FillIndex Time RT"),STAT_TrailFillIndexTime,STATGROUP_TrailParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trail Render Time RT"),STAT_TrailRenderingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trail Tick Time GT"),STAT_TrailTickTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("AnimTrail Notify Time GT"),STAT_AnimTrailNotifyTime,STATGROUP_Particles, );

/**
 * Beam particle stats
 */
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Particles"),STAT_BeamParticles,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcl Render Calls"),STAT_BeamParticlesRenderCalls,STATGROUP_BeamParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcls Spawned"),STAT_BeamParticlesSpawned,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcl Update Calls"),STAT_BeamParticlesUpdateCalls,STATGROUP_BeamParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcls Updated"),STAT_BeamParticlesUpdated,STATGROUP_BeamParticles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcls Killed"),STAT_BeamParticlesKilled,STATGROUP_Particles, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Beam Ptcl Tris"),STAT_BeamParticlesTrianglesRendered,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Beam Spawn Time GT"),STAT_BeamSpawnTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Beam FillVertex Time RT"),STAT_BeamFillVertexTime,STATGROUP_BeamParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Beam FillIndex Time RT"),STAT_BeamFillIndexTime,STATGROUP_BeamParticles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Beam Render Time RT"),STAT_BeamRenderingTime,STATGROUP_Particles, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Beam Tick Time GT"),STAT_BeamTickTime,STATGROUP_Particles, );

/**
* Mesh Particle Stats
*/

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Mesh Particle Polys"), STAT_MeshParticlePolys, STATGROUP_Particles, );

//
//	Helper structures for payload data...
//

//
//	SubUV-related payloads
//
struct FFullSubUVPayload
{
	// The integer portion indicates the sub-image index.
	// The fractional portion indicates the lerp factor.
	float ImageIndex;
	float RandomImageTime;
};

//
//	AttractorParticle
//
struct FAttractorParticlePayload
{
	int32			SourceIndex;
	uint32		SourcePointer;
	FVector3f		SourceVelocity;
};

struct FLightParticlePayload
{
	FVector3f		ColorScale;
	uint64		LightId;
	float		RadiusScale;
	float		LightExponent;
	float		InverseExposureBlend;
	bool		bValid;
	bool		bAffectsTranslucency;
	bool		bHighQuality;
};

//
//	TypeDataBeam2 payload
//
#define BEAM2_TYPEDATA_LOCKED_MASK					0x80000000
#define	BEAM2_TYPEDATA_LOCKED(x)					((x & BEAM2_TYPEDATA_LOCKED_MASK) != 0)
#define	BEAM2_TYPEDATA_SETLOCKED(x, Locked)			(x = Locked ? (x | BEAM2_TYPEDATA_LOCKED_MASK) : (x & ~BEAM2_TYPEDATA_LOCKED_MASK))

#define BEAM2_TYPEDATA_FREQUENCY_MASK				0x00fff000
#define BEAM2_TYPEDATA_FREQUENCY_SHIFT				12
#define	BEAM2_TYPEDATA_FREQUENCY(x)					((x & BEAM2_TYPEDATA_FREQUENCY_MASK) >> BEAM2_TYPEDATA_FREQUENCY_SHIFT)
#define BEAM2_TYPEDATA_SETFREQUENCY(x, Freq)		(x = ((x & ~BEAM2_TYPEDATA_FREQUENCY_MASK) | (Freq << BEAM2_TYPEDATA_FREQUENCY_SHIFT)))

struct FBeam2TypeDataPayload
{
	/** The source of this beam											*/
	FVector		SourcePoint;
	/** The source tangent of this beam									*/
	FVector		SourceTangent;
	/** The stength of the source tangent of this beam					*/
	float		SourceStrength;

	/** The target of this beam											*/
	FVector		TargetPoint;
	/** The target tangent of this beam									*/
	FVector		TargetTangent;
	/** The stength of the Target tangent of this beam					*/
	float		TargetStrength;

	/** Target lock, extreme max, Number of noise points				*/
	int32		Lock_Max_NumNoisePoints;

	/** Number of segments to render (steps)							*/
	int32		InterpolationSteps;

	/** Direction to step in											*/
	FVector		Direction;
	/** StepSize (for each segment to be rendered)						*/
	double		StepSize;
	/** Number of segments to render (steps)							*/
	int32		Steps;
	/** The 'extra' amount to travel (partial segment)					*/
	float		TravelRatio;

	/** The number of triangles to render for this beam					*/
	int32		TriangleCount;

	/**
	 *	Type and indexing flags
	 * 3               1              0
	 * 1...|...|...|...5...|...|...|..0
	 * TtPppppppppppppppNnnnnnnnnnnnnnn
	 * Tt				= Type flags --> 00 = Middle of Beam (nothing...)
	 * 									 01 = Start of Beam
	 * 									 10 = End of Beam
	 * Ppppppppppppppp	= Previous index
	 * Nnnnnnnnnnnnnnn	= Next index
	 * 		int32				Flags;
	 * 
	 * NOTE: These values DO NOT get packed into the vertex buffer!
	 */
	int32			Flags;
};

/**	Particle Source/Target Data Payload									*/
struct FBeamParticleSourceTargetPayloadData
{
	int32			ParticleIndex;
};

/**	Particle Source Branch Payload										*/
struct FBeamParticleSourceBranchPayloadData
{
	int32			NoiseIndex;
};

/** Particle Beam Modifier Data Payload */
struct FBeamParticleModifierPayloadData
{
	uint32	bModifyPosition:1;
	uint32	bScalePosition:1;
	uint32	bModifyTangent:1;
	uint32	bScaleTangent:1;
	uint32	bModifyStrength:1;
	uint32	bScaleStrength:1;
	FVector	Position;
	FVector	Tangent;
	float	Strength;

	// Helper functions
	inline void UpdatePosition(FVector& Value)
	{
		if (bModifyPosition == true)
		{
			if (bScalePosition == false)
			{
				Value += Position;
			}
			else
			{
				Value *= Position;
			}
		}
	}

	inline void UpdateTangent(FVector& Value, bool bAbsolute)
	{
		if (bModifyTangent == true)
		{
			FVector ModTangent;
			if (bAbsolute == false)
			{
				// Transform the modified tangent so it is relative to the real tangent
				const FQuat RotQuat = FQuat::FindBetweenNormals(FVector(1.0f, 0.0f, 0.0f), Value);
				ModTangent = RotQuat.RotateVector(Tangent);
			}
			else
			{
				ModTangent = Tangent;
			}

			if (bScaleTangent == false)
			{
				Value += ModTangent;
			}
			else
			{
				Value *= ModTangent;
			}
		}
	}

	inline void UpdateStrength(float& Value)
	{
		if (bModifyStrength == true)
		{
			if (bScaleStrength == false)
			{
				Value += Strength;
			}
			else
			{
				Value *= Strength;
			}
		}
	}
};

/** Trails Base data payload */
struct FTrailsBaseTypeDataPayload
{
	/**
	 * TRAIL_EMITTER_FLAG_MASK         0xf0000000
	 * TRAIL_EMITTER_PREV_MASK         0x0fffc000
	 * TRAIL_EMITTER_PREV_SHIFT        14
	 * TRAIL_EMITTER_NEXT_MASK         0x00003fff
	 * TRAIL_EMITTER_NEXT_SHIFT        0

	 *	Type and indexing flags
	 *	3               1              0
	 *	1...|...|...|...5...|...|...|..0
	 *	TtttPpppppppppppppNnnnnnnnnnnnnn
	 *
	 *	Tttt = Type flags
	 *		0x0 = ForceKill	- the trail should be completely killed in the next KillParticles call.
	 *		0x1	= DeadTrail	- the trail should no longer spawn particles. Just let it die out as particles in it fade.
	 *		0x2	= Middle	- indicates this is a particle in the middle of a trail.
	 *		0x4	= Start		- indicates this is the first particle in a trail.
	 *		0x8	= End		- indicates this is the last particle in a trail.
	 *	Pppppppppppppp	= Previous index
	 *	Nnnnnnnnnnnnnn	= Next index
	 */
	int32 Flags;
	/** The trail index - valid in a START particle only */
	int32 TrailIndex;
	/** The number of triangles in the trail - valid in a START particle only */
	int32 TriangleCount;
	/** The time that the particle was spawned */
	float SpawnTime;
	/** The time slice when the particle was spawned */
	float SpawnDelta;
	/** The starting tiled U value for this particle */
	float TiledU;
	/** The tessellated spawn points between this particle and the next one */
	int32 SpawnedTessellationPoints;
	/** The number of points to interpolate between this particle and the next when rendering */
	int32 RenderingInterpCount;
	/** The scale factor to use to shrink up in tight curves */
	float PinchScaleFactor;
	/** true if the particle is an interpolated spawn, false if true position based. */
	uint32 bInterpolatedSpawn:1;
	/** true if the particle was spawn via movement, false if not. */
	uint32 bMovementSpawned:1;
};

struct FRibbonTypeDataPayload : public FTrailsBaseTypeDataPayload
{
	/**	Tangent for the trail segment */
	FVector3f Tangent;
	/**	The 'up' for the segment (render plane) */
	FVector3f Up;
	/** The source index tracker (particle index, etc.) */
	int32 SourceIndex;
};

/** AnimTrail payload */
struct FAnimTrailTypeDataPayload : public FTrailsBaseTypeDataPayload
{
	//Direction from the first socket sample to the second.
	FVector3f Direction;
	//Tangent of the curve.
	FVector3f Tangent;
	//Half length between the sockets. First vertex = Location - Dir * Length; Second vertex = Location + Dir * Lenght
	float Length;
	/** Parameter of this knot on the spline*/
	float InterpolationParameter;
};

/** Mesh rotation data payload										*/
struct FMeshRotationPayloadData
{
	FVector3f	 InitialOrientation;		// from mesh data module
	FVector3f  InitRotation;				// from init rotation module
	FVector3f  Rotation;
	FVector3f	 CurContinuousRotation;
	FVector3f  RotationRate;
	FVector3f  RotationRateBase;
};

struct FMeshMotionBlurPayloadData
{
	FVector3f BaseParticlePrevVelocity;
	FVector3f BaseParticlePrevSize;
	FVector3f PayloadPrevRotation;
	FVector3f PayloadPrevOrbitOffset;
	float   BaseParticlePrevRotation;
	float   PayloadPrevCameraOffset;
};

/** ModuleLocationEmitter instance payload							*/
struct FLocationEmitterInstancePayload
{
	int32		LastSelectedIndex;
};

/** Helper class that provides a basic interface for an inline and presized array within a contiguous memory block */
template <typename ElementType>
class TPreallocatedArrayProxy
{
public:

	/** Constructor, just zeros everything  */
	TPreallocatedArrayProxy() : 
		ArrayMax(0),
		Array(NULL)
	{}

	/** Constructor, just sets up the array pointer and size
	*   @param ArrayStart The array pointer
	*   @param ArraySize  The maximum number of elements that the array  
	*/
	TPreallocatedArrayProxy( uint8* ArrayStart, int32 ArraySize ) :
		ArrayMax( ArraySize ),
		Array( (ElementType*) ArrayStart )
	{}

	/** Accesses the identified element's value. */
	inline ElementType& operator[]( int32 Index )
	{
		check((Index >= 0) & ((Index < ArrayMax) | ((Index == 0) & (ArrayMax == 0)))); // & and | for one branch
		return Array[Index];
	}

	/** Accesses the identified element's value. */
	inline const ElementType& operator[]( int32 Index ) const
	{
		check((Index >= 0) & ((Index < ArrayMax) | ((Index == 0) & (ArrayMax == 0)))); // & and | for one branch
		return Array[Index];
	}

	/**
	 * Finds an element with the given value in the array.
	 * @param Item - The value to search for.
	 * @return The index of an element to the given value.  If no element in the array has the given value, this will return INDEX_NONE.
	 */
	int32 Find( const ElementType& Item ) const
	{
		const ElementType* const RESTRICT DataEnd = Array + ArrayMax;
		for(const ElementType* RESTRICT Data = Array; Data < DataEnd; ++Data)
		{
			if( *Data==Item )
			{
				return (int32)(Data - Array);
			}
		}
		return INDEX_NONE;
	}

	/**
	 * Checks if the array contains an element with the given value.
	 * @param Item - The item to check for.
	 * @return true if the array contains an element with the given value.
	 */
	bool Contains( const ElementType& Item ) const
	{
		return ( Find(Item) != INDEX_NONE );
	}

	int32 Num() const
	{
		return ArrayMax;
	}

private:
	/** The maximum number of elements in the array. This is cannot dynamically change.*/
	int32 ArrayMax;
	/** Pointer to an array, stored within a contiguous memory block.*/
	ElementType* Array;
};

/** ModuleLocationBoneSocket per-particle payload */
struct FModuleLocationBoneSocketParticlePayload
{
	/** The index of the socket this particle is 'attached' to */
	int32 SourceIndex;
};

/**
 *	Chain-able Orbit module instance payload
 */
struct FOrbitChainModuleInstancePayload
{
	/** The base offset of the particle from it's tracked location	*/
	FVector3f	BaseOffset;
	/** The offset of the particle from it's tracked location		*/
	FVector3f	Offset;
	/** The rotation of the particle at it's offset location		*/
	FVector3f	Rotation;
	/** The base rotation rate of the particle offset				*/
	FVector3f	BaseRotationRate;
	/** The rotation rate of the particle offset					*/
	FVector3f	RotationRate;
	/** The offset of the particle from the last frame				*/
	FVector3f	PreviousOffset;
};

/**
 *	Payload for instances which use the SpawnPerUnit module.
 */
struct FParticleSpawnPerUnitInstancePayload
{
	float	CurrentDistanceTravelled;
};

/**
 *	Collision module particle payload
 */
struct FParticleCollisionPayload
{
	FVector3f	UsedDampingFactor;
	FVector3f	UsedDampingFactorRotation;
	int32		UsedCollisions;
	float	Delay;
};

/** Collision module per instance payload */
struct FParticleCollisionInstancePayload
{
	/** Count for tracking how many times the bounds checking was skipped */
	uint8 CurrentLODBoundsCheckCount;
	/** Padding for potential future expansion */
	uint8 Padding1;
	uint8 Padding2;
	uint8 Padding3;
};

/**
 *	General event instance payload.
 */
struct FParticleEventInstancePayload
{
	uint32 bSpawnEventsPresent:1;
	uint32 bDeathEventsPresent:1;
	uint32 bCollisionEventsPresent:1;
	uint32 bBurstEventsPresent:1;

	int32 SpawnTrackingCount;
	int32 DeathTrackingCount;
	int32 CollisionTrackingCount;
	int32 BurstTrackingCount;
};

/**
 *	DynamicParameter particle payload.
 */
struct FEmitterDynamicParameterPayload
{
	/** The float4 value to assign to the dynamic parameter. */
	float DynamicParameterValue[4];
};

/**
 *	Helper function for retrieving the dynamic payload of a particle.
 *
 *	@param	InDynamicPayloadOffset		The offset to the payload
 *	@param	InParticle					The particle being processed
 *	@param	OutDynamicData				The dynamic data from the particle
 */
inline void GetDynamicValueFromPayload(int32 InDynamicPayloadOffset, const FBaseParticle& InParticle, FVector4f& OutDynamicData)
{
	checkSlow(InDynamicPayloadOffset > 0);
	const FEmitterDynamicParameterPayload* DynPayload = ((const FEmitterDynamicParameterPayload*)((uint8*)(&InParticle) + InDynamicPayloadOffset));
	OutDynamicData.X = DynPayload->DynamicParameterValue[0];
	OutDynamicData.Y = DynPayload->DynamicParameterValue[1];
	OutDynamicData.Z = DynPayload->DynamicParameterValue[2];
	OutDynamicData.W = DynPayload->DynamicParameterValue[3];
}

/** Camera offset particle payload */
struct FCameraOffsetParticlePayload
{
	/** The base amount to offset the particle towards the camera */
	float	BaseOffset;
	/** The amount to offset the particle towards the camera */
	float	Offset;
};

/** Random-seed instance payload */
struct FParticleRandomSeedInstancePayload
{
	FRandomStream	RandomStream;
};

/*-----------------------------------------------------------------------------
	Particle Sorting Helper
-----------------------------------------------------------------------------*/
struct FParticleOrder
{
	int32 ParticleIndex;

	union
	{
		float Z;
		uint32 C;
	};
	
	FParticleOrder(int32 InParticleIndex,float InZ):
		ParticleIndex(InParticleIndex),
		Z(InZ)
	{}

	FParticleOrder(int32 InParticleIndex,uint32 InC):
		ParticleIndex(InParticleIndex),
		C(InC)
	{}
};


/*-----------------------------------------------------------------------------
	Async Fill Organizational Structure
-----------------------------------------------------------------------------*/

struct FAsyncBufferFillData
{
	/** Local to world transform. */
	FMatrix LocalToWorld;
	/** World to local transform. */
	FMatrix WorldToLocal;
	/** View for this buffer fill task   */
	const FSceneView*					View;
	/** Number of verts in VertexData   */
	int32									VertexCount;
	/** Stride of verts, used only for error checking   */
	int32									VertexSize; 
	/** Pointer to vertex data   */
	void*								VertexData;
	/** Number of indices in IndexData   */
	int32									IndexCount;
	/** Pointer to index data   */
	void*								IndexData;
	/** Number of triangles filled in   */
	int32									OutTriangleCount;
	/** Pointer to dynamic parameter data */
	void*								DynamicParameterData;

	/** Constructor, just zeros everything   */
	FAsyncBufferFillData()
	{
		// this is all POD
		FMemory::Memzero(this,sizeof(FAsyncBufferFillData));
	}
	/** Destructor, frees memory and zeros everything   */
	~FAsyncBufferFillData()
	{
		FMemory::Memzero(this,sizeof(FAsyncBufferFillData));
	}
};

/*-----------------------------------------------------------------------------
	Particle vertex factory pools
-----------------------------------------------------------------------------*/
class FParticleVertexFactoryBase;

/*-----------------------------------------------------------------------------
	Particle Dynamic Data
-----------------------------------------------------------------------------*/

/**
 * Dynamic particle emitter types
 *
 * NOTE: These are serialized out for particle replay data, so be sure to update all appropriate
 *    when changing anything here.
 */
enum EDynamicEmitterType
{
	DET_Unknown = 0,
	DET_Sprite,
	DET_Mesh,
	DET_Beam2,
	DET_Ribbon,
	DET_AnimTrail,
	DET_Custom
};

struct FParticleDataContainer
{
	int32 MemBlockSize;
	int32 ParticleDataNumBytes;
	int32 ParticleIndicesNumShorts;
	uint8* ParticleData; // this is also the memory block we allocated
	uint16* ParticleIndices; // not allocated, this is at the end of the memory block

	FParticleDataContainer()
		: MemBlockSize(0)
		, ParticleDataNumBytes(0)
		, ParticleIndicesNumShorts(0)
		, ParticleData(nullptr)
		, ParticleIndices(nullptr)
	{
	}
	~FParticleDataContainer()
	{
		Free();
	}
	void Alloc(int32 InParticleDataNumBytes, int32 InParticleIndicesNumShorts);
	void Free();
};

struct FMacroUVOverride
{
	FMacroUVOverride() : bOverride(false), Radius(0.f), Position(0.f,0.f,0.f) {}

	bool	bOverride;
	float   Radius;
	FVector3f Position;

	friend inline FArchive& operator<<(FArchive& Ar, FMacroUVOverride& O)
	{
		Ar << O.bOverride;
		Ar << O.Radius;
		Ar << O.Position;
		return Ar;
	}
};

/** Source data base class for all emitter types */
struct FDynamicEmitterReplayDataBase
{
	/**	The type of emitter. */
	EDynamicEmitterType	eEmitterType;

	/**	The number of particles currently active in this emitter. */
	int32 ActiveParticleCount;

	int32 ParticleStride;
	FParticleDataContainer DataContainer;

	FVector3f Scale;

	/** Whether this emitter requires sorting as specified by artist.	*/
	int32 SortMode;

	/** MacroUV (override) data **/
	FMacroUVOverride MacroUVOverride;

	/** Constructor */
	FDynamicEmitterReplayDataBase()
		: eEmitterType( DET_Unknown ),
		  ActiveParticleCount( 0 ),
		  ParticleStride( 0 ),
		  Scale( FVector3f( 1.0f ) ),
		  SortMode(0)	// Default to PSORTMODE_None		  
	{
	}

	virtual ~FDynamicEmitterReplayDataBase() = default;

	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		int32 EmitterTypeAsInt = eEmitterType;
		Ar << EmitterTypeAsInt;
		eEmitterType = static_cast< EDynamicEmitterType >( EmitterTypeAsInt );

		Ar << ActiveParticleCount;
		Ar << ParticleStride;
		
		TArray<uint8> ParticleData;
		TArray<uint16> ParticleIndices;

		if (!Ar.IsLoading() && !Ar.IsObjectReferenceCollector())
		{
			if (DataContainer.ParticleDataNumBytes)
			{
				ParticleData.AddUninitialized(DataContainer.ParticleDataNumBytes);
				FMemory::Memcpy(ParticleData.GetData(), DataContainer.ParticleData, DataContainer.ParticleDataNumBytes);
			}
			if (DataContainer.ParticleIndicesNumShorts)
			{
				ParticleIndices.AddUninitialized(DataContainer.ParticleIndicesNumShorts);
				FMemory::Memcpy(ParticleIndices.GetData(), DataContainer.ParticleIndices, DataContainer.ParticleIndicesNumShorts * sizeof(uint16));
			}
		}

		Ar << ParticleData;
		Ar << ParticleIndices;

		if (Ar.IsLoading())
		{
			DataContainer.Free();
			if (ParticleData.Num())
			{
				DataContainer.Alloc(ParticleData.Num(), ParticleIndices.Num());
				FMemory::Memcpy(DataContainer.ParticleData, ParticleData.GetData(), DataContainer.ParticleDataNumBytes);
				if (DataContainer.ParticleIndicesNumShorts)
				{
					FMemory::Memcpy(DataContainer.ParticleIndices, ParticleIndices.GetData(), DataContainer.ParticleIndicesNumShorts * sizeof(uint16));
				}
			}
			else
			{
				check(!ParticleIndices.Num());
			}
		}

		Ar << Scale;
		Ar << SortMode;
		Ar << MacroUVOverride;
	}

};



/** Base class for all emitter types */
struct FDynamicEmitterDataBase
{
	FDynamicEmitterDataBase(const class UParticleModuleRequired* RequiredModule);
	
	virtual ~FDynamicEmitterDataBase() = default;

	/** Custom new/delete with recycling */
	void* operator new(size_t Size);
	void operator delete(void *RawMemory, size_t Size);

	/**
	 *	Create the render thread resources for this emitter data
	 *
	 *	@param	InOwnerProxy	The proxy that owns this dynamic emitter data
	 */
	virtual void UpdateRenderThreadResourcesEmitter(const FParticleSystemSceneProxy* InOwnerProxy)
	{
	}

	/**
	 *	Release the render thread resources for this emitter data
	 *
	 *	@param	InOwnerProxy	The proxy that owns this dynamic emitter data
	 */
	virtual void ReleaseRenderThreadResources(const FParticleSystemSceneProxy* InOwnerProxy)
	{
	}

	virtual void GetDynamicMeshElementsEmitter(const FParticleSystemSceneProxy* Proxy, const FSceneView* View, const FSceneViewFamily& ViewFamily, int32 ViewIndex, FMeshElementCollector& Collector) const {}

	/**
	 *	Retrieve the material render proxy to use for rendering this emitter. PURE VIRTUAL
	 *
	 *	@param	bSelected				Whether the object is selected
	 *
	 *	@return	FMaterialRenderProxy*	The material proxt to render with.
	 */
	virtual const FMaterialRenderProxy* GetMaterialRenderProxy() = 0;

	/** Callback from the renderer to gather simple lights that this proxy wants renderered. */
	virtual void GatherSimpleLights(const FParticleSystemSceneProxy* Proxy, const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const {}

	/** Returns the source data for this particle system */
	virtual const FDynamicEmitterReplayDataBase& GetSource() const = 0;

	/** Returns the current macro uv override. Specialized by FGPUSpriteDynamicEmitterData  */
	virtual const FMacroUVOverride& GetMacroUVOverride() const { return GetSource().MacroUVOverride; }

	/** Stat id of this object, 0 if nobody asked for it yet */
	mutable TStatId StatID;
	/** true if this emitter is currently selected */
	uint32	bSelected:1;
	/** true if this emitter has valid rendering data */
	uint32	bValid:1;

	int32  EmitterIndex;
};

/** Source data base class for Sprite emitters */
struct FDynamicSpriteEmitterReplayDataBase
	: public FDynamicEmitterReplayDataBase
{
	UMaterialInterface*				MaterialInterface;
	struct FParticleRequiredModule	*RequiredModule;
	FVector3f							NormalsSphereCenter;
	FVector3f							NormalsCylinderDirection;
	float							InvDeltaSeconds;
	FVector3f						LWCTile;
	int32							MaxDrawCount;
	int32							OrbitModuleOffset;
	int32							DynamicParameterDataOffset;
	int32							LightDataOffset;
	float							LightVolumetricScatteringIntensity;
	int32							CameraPayloadOffset;
	int32							SubUVDataOffset;
	int32							SubImages_Horizontal;
	int32							SubImages_Vertical;
	bool						bUseLocalSpace;
	bool						bLockAxis;
	uint8						ScreenAlignment;
	uint8						LockAxisFlag;
	uint8						EmitterRenderMode;
	uint8						EmitterNormalsMode;
	FVector2f					PivotOffset;
	bool						bUseVelocityForMotionBlur;
	bool						bRemoveHMDRoll;
	float						MinFacingCameraBlendDistance;
	float						MaxFacingCameraBlendDistance;
	
	/** Constructor */
	FDynamicSpriteEmitterReplayDataBase();
	~FDynamicSpriteEmitterReplayDataBase();

	/** Serialization */
	virtual void Serialize( FArchive& Ar );

};

/** Base class for Sprite emitters and other emitter types that share similar features. */
struct FDynamicSpriteEmitterDataBase : public FDynamicEmitterDataBase
{
	FDynamicSpriteEmitterDataBase(const UParticleModuleRequired* RequiredModule) : 
		FDynamicEmitterDataBase(RequiredModule),
		bUsesDynamicParameter( false )
	{
		MaterialResource = nullptr;
	}

	virtual ~FDynamicSpriteEmitterDataBase() = default;

	/**
	 *	Retrieve the material render proxy to use for rendering this emitter. PURE VIRTUAL
	 *
	 *	@param	bSelected				Whether the object is selected
	 *
	 *	@return	FMaterialRenderProxy*	The material proxt to render with.
	 */
	const FMaterialRenderProxy* GetMaterialRenderProxy() 
	{ 
		return MaterialResource;
	}

	/**
	 *	Sort the given sprite particles
	 *
	 *	@param	SorceMode			The sort mode to utilize (EParticleSortMode)
	 *	@param	bLocalSpace			true if the emitter is using local space
	 *	@param	ParticleCount		The number of particles
	 *	@param	ParticleData		The actual particle data
	 *	@param	ParticleStride		The stride between entries in the ParticleData array
	 *	@param	ParticleIndices		Indirect index list into ParticleData
	 *	@param	View				The scene view being rendered
	 *	@param	LocalToWorld		The local to world transform of the component rendering the emitter
	 *	@param	ParticleOrder		The array to fill in with ordered indices
	 */
	void SortSpriteParticles(int32 SortMode, bool bLocalSpace, 
		int32 ParticleCount, const uint8* ParticleData, int32 ParticleStride, const uint16* ParticleIndices,
		const FSceneView* View, const FMatrix& LocalToWorld, FParticleOrder* ParticleOrder) const;

	/**
	 *	Get the vertex stride for the dynamic rendering data
	 */
	virtual int32 GetDynamicVertexStride(ERHIFeatureLevel::Type /*InFeatureLevel*/) const
	{
		checkf(0, TEXT("GetDynamicVertexStride MUST be overridden"));
		return 0;
	}

	/**
	 *	Get the vertex stride for the dynamic parameter rendering data
	 */
	virtual int32 GetDynamicParameterVertexStride() const
	{
		checkf(0, TEXT("GetDynamicParameterVertexStride MUST be overridden"));
		return 0;
	}

	/**
	 *	Get the source replay data for this emitter
	 */
	virtual const FDynamicSpriteEmitterReplayDataBase* GetSourceData() const
	{
		checkf(0, TEXT("GetSourceData MUST be overridden"));
		return NULL;
	}
	
	/**
	 *	Gets the information required for allocating this emitters indices from the global index array.
	 */
	virtual void GetIndexAllocInfo(int32& OutNumIndices, int32& OutStride ) const
	{
		checkf(0, TEXT("GetIndexAllocInfo is not valid for this class."));
	}

	/**
	 *	Debug rendering
	 *
	 *	@param	Proxy		The primitive scene proxy for the emitter.
	 *	@param	PDI			The primitive draw interface to render with
	 *	@param	View		The scene view being rendered
	 *	@param	bCrosses	If true, render Crosses at particle position; false, render points
	 */
	virtual void RenderDebug(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI, const FSceneView* View, bool bCrosses) const;

	virtual void DoBufferFill(FAsyncBufferFillData& Me) const
	{
		// Must be overridden if called
		check(0);
	}

	/**
	 *	Set up an buffer for async filling
	 *
	 *	@param	Proxy					The primitive scene proxy for the emitter.
	 *	@param	InView					View for this buffer
	 *	@param	InVertexCount			Count of verts for this buffer
	 *	@param	InVertexSize			Stride of these verts, only used for verification
	 *	@param	InDynamicParameterVertexStride	Stride of the dynamic parameter
	 */
	void BuildViewFillData(
		const FParticleSystemSceneProxy* Proxy, 
		const FSceneView *InView, 
		int32 InVertexCount, 
		int32 InVertexSize, 
		int32 InDynamicParameterVertexSize, 
		FGlobalDynamicIndexBuffer& DynamicIndexBuffer,
		FGlobalDynamicVertexBuffer& DynamicVertexBuffer,
		FGlobalDynamicVertexBufferAllocation& DynamicVertexAllocation,
		FGlobalDynamicIndexBufferAllocation& DynamicIndexAllocation,
		FGlobalDynamicVertexBufferAllocation* DynamicParameterAllocation,
		FAsyncBufferFillData& Data) const;

	/** The material render proxy for this emitter */
	const FMaterialRenderProxy*	MaterialResource;
	/** true if the particle emitter utilizes the DynamicParameter module */
	uint32 bUsesDynamicParameter:1;
};

/** Source data for Sprite emitters */
struct FDynamicSpriteEmitterReplayData
	: public FDynamicSpriteEmitterReplayDataBase
{
	/** Constructor */
	FDynamicSpriteEmitterReplayData()
	{
	}


	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		// Call parent implementation
		FDynamicSpriteEmitterReplayDataBase::Serialize( Ar );

		// ...
	}

};

/** Dynamic emitter data for sprite emitters */
struct FDynamicSpriteEmitterData : public FDynamicSpriteEmitterDataBase
{
	FDynamicSpriteEmitterData(const UParticleModuleRequired* RequiredModule) :
		FDynamicSpriteEmitterDataBase(RequiredModule)
	{
	}

	~FDynamicSpriteEmitterData() = default;

	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	void Init( bool bInSelected );

	/**
	 *	Get the vertex stride for the dynamic rendering data
	 */
	virtual int32 GetDynamicVertexStride(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return sizeof(FParticleSpriteVertex);
	}

	/**
	 *	Get the vertex stride for the dynamic parameter rendering data
	 */
	virtual int32 GetDynamicParameterVertexStride() const override
	{
		return sizeof(FParticleVertexDynamicParameter);
	}

	/**
	 *	Get the source replay data for this emitter
	 */
	virtual const FDynamicSpriteEmitterReplayDataBase* GetSourceData() const override
	{
		return &Source;
	}

	/**
	 *	Retrieve the vertex and (optional) index required to render this emitter.
	 *	Render-thread only
	 *
	 *	@param	VertexData			The memory to fill the vertex data into
	 *	@param	FillIndexData		The index data to fill in
	 *	@param	ParticleOrder		The (optional) particle ordering to use
	 *	@param	InCameraPosition	The position of the camera in world space.
	 *	@param	InLocalToWorld		Transform from local to world space.
	 *	@param	InstanceFactor		The factor to duplicate instances by.
	 *
	 *	@return	bool			true if successful, false if failed
	 */
	bool GetVertexAndIndexData(void* VertexData, void* DynamicParameterVertexData, void* FillIndexData, FParticleOrder* ParticleOrder, const FVector& InCameraPosition, const FMatrix& InLocalToWorld, uint32 InstanceFactor) const;

	/** Gathers simple lights for this emitter. */
	virtual void GatherSimpleLights(const FParticleSystemSceneProxy* Proxy, const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const override;

	virtual void GetDynamicMeshElementsEmitter(const FParticleSystemSceneProxy* Proxy, const FSceneView* View, const FSceneViewFamily& ViewFamily, int32 ViewIndex, FMeshElementCollector& Collector) const override;

	/**
	 *	Create the render thread resources for this emitter data
	 *
	 *	@param	InOwnerProxy	The proxy that owns this dynamic emitter data
	 *
	 *	@return	bool			true if successful, false if failed
	 */
	virtual void UpdateRenderThreadResourcesEmitter(const FParticleSystemSceneProxy* InOwnerProxy) override;

	/** Returns the source data for this particle system */
	virtual const FDynamicEmitterReplayDataBase& GetSource() const override
	{
		return Source;
	}

	/** The frame source data for this particle system.  This is everything needed to represent this
		this particle system frame.  It does not include any transient rendering thread data.  Also, for
		non-simulating 'replay' particle systems, this data may have come straight from disk! */
	FDynamicSpriteEmitterReplayData Source;

	/** Uniform parameters. Most fields are filled in when updates are sent to the rendering thread, some are per-view! */
	FParticleSpriteUniformParameters UniformParameters;
};

/** Source data for Mesh emitters */
struct FDynamicMeshEmitterReplayData
	: public FDynamicSpriteEmitterReplayDataBase
{
	int32	SubUVInterpMethod;
	int32	SubUVDataOffset;
	int32	SubImages_Horizontal;
	int32	SubImages_Vertical;
	bool	bScaleUV;
	int32	MeshRotationOffset;
	int32	MeshMotionBlurOffset;
	uint8	MeshAlignment;
	bool	bMeshRotationActive;
	FVector3f	LockedAxis;	

	/** Constructor */
	FDynamicMeshEmitterReplayData() : 
		SubUVInterpMethod( 0 ),
		SubUVDataOffset( 0 ),
		SubImages_Horizontal( 0 ),
		SubImages_Vertical( 0 ),
		bScaleUV( false ),
		MeshRotationOffset( 0 ),
		MeshMotionBlurOffset( 0 ),
		MeshAlignment( 0 ),
		bMeshRotationActive( false ),
		LockedAxis(1.0f, 0.0f, 0.0f)
	{
	}


	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		// Call parent implementation
		FDynamicSpriteEmitterReplayDataBase::Serialize( Ar );
		
		Ar << SubUVInterpMethod;
		Ar << SubUVDataOffset;
		Ar << SubImages_Horizontal;
		Ar << SubImages_Vertical;
		Ar << bScaleUV;
		Ar << MeshRotationOffset;
		Ar << MeshMotionBlurOffset;
		Ar << MeshAlignment;
		Ar << bMeshRotationActive;
		Ar << LockedAxis;
	}

};


/** Dynamic emitter data for Mesh emitters */
struct FDynamicMeshEmitterData : public FDynamicSpriteEmitterDataBase
{
	FDynamicMeshEmitterData(const UParticleModuleRequired* RequiredModule);

	virtual ~FDynamicMeshEmitterData();

	uint32 GetMeshLODIndexFromProxy(const FParticleSystemSceneProxy *InOwnerProxy) const;
	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	void Init(	bool bInSelected,
				const FParticleMeshEmitterInstance* InEmitterInstance,
				UStaticMesh* InStaticMesh,
				bool InUseStaticMeshLODs,
				float InLODSizeScale,
				ERHIFeatureLevel::Type InFeatureLevel);

	/**
	 *	Create the render thread resources for this emitter data
	 *
	 *	@param	InOwnerProxy	The proxy that owns this dynamic emitter data
	 *
	 *	@return	bool			true if successful, false if failed
	 */
	virtual void UpdateRenderThreadResourcesEmitter(const FParticleSystemSceneProxy* InOwnerProxy) override;

	/**
	 *	Release the render thread resources for this emitter data
	 *
	 *	@param	InOwnerProxy	The proxy that owns this dynamic emitter data
	 *
	 *	@return	bool			true if successful, false if failed
	 */
	virtual void ReleaseRenderThreadResources(const FParticleSystemSceneProxy* InOwnerProxy) override;

	virtual void GetDynamicMeshElementsEmitter(const FParticleSystemSceneProxy* Proxy, const FSceneView* View, const FSceneViewFamily& ViewFamily, int32 ViewIndex, FMeshElementCollector& Collector) const override;

	/**
	 *	Retrieve the instance data required to render this emitter.
	 *	Render-thread only
	 *
	 *	@param	InstanceData            The memory to fill the vertex data into
	 *	@param	DynamicParameterData    The memory to fill the vertex dynamic parameter data into
	 *	@param	PrevTransformBuffer     The memory to fill the vertex prev transform data into. May be null
	 *	@param	Proxy                   The scene proxy for the particle system that owns this emitter
	 *	@param	View                    The scene view being rendered
	 *	@param	InstanceFactor			The factor to duplicate instances by
	 */
	void GetInstanceData(void* InstanceData, void* DynamicParameterData, void* PrevTransformBuffer, const FParticleSystemSceneProxy* Proxy, const FSceneView* View, uint32 InstanceFactor) const;

	/**
	 *	Helper function for retrieving the particle transform.
	 *
	 *	@param	InParticle					The particle being processed
	 *  @param	Proxy					    The scene proxy for the particle system that owns this emitter
	 *	@param	View						The scene view being rendered
	 *	@param	OutTransform				The InstanceToWorld transform matrix for the particle
	 */
	void GetParticleTransform(const FBaseParticle& InParticle, const FParticleSystemSceneProxy* Proxy, const FSceneView* View, FMatrix& OutTransformMat) const;

	void GetParticlePrevTransform(const FBaseParticle& InParticle, const FParticleSystemSceneProxy* Proxy, const FSceneView* View, FMatrix& OutTransformMat) const;

	void CalculateParticleTransform(
		const FMatrix& ProxyLocalToWorld,
		const FVector& ParticleLocation,
			  float    ParticleRotation,
		const FVector3f& ParticleVelocity,
		const FVector3f& ParticleSize,
		const FVector3f& ParticlePayloadInitialOrientation,
		const FVector3f& ParticlePayloadRotation,
		const FVector& ParticlePayloadCameraOffset,
		const FVector3f& ParticlePayloadOrbitOffset,
		const FVector& ViewOrigin,
		const FVector3f& ViewDirection,
		FMatrix& OutTransformMat
		) const;

	/** Gathers simple lights for this emitter. */
	virtual void GatherSimpleLights(const FParticleSystemSceneProxy* Proxy, const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const override;

	/**
	 *	Get the vertex stride for the dynamic rendering data
	 */
	virtual int32 GetDynamicVertexStride(ERHIFeatureLevel::Type /*InFeatureLevel*/) const override
	{
		return sizeof(FMeshParticleInstanceVertex);
	}

	virtual int32 GetDynamicParameterVertexStride() const override 
	{
		return sizeof(FMeshParticleInstanceVertexDynamicParameter);
	}

	/**
	 *	Get the source replay data for this emitter
	 */
	virtual const FDynamicSpriteEmitterReplayDataBase* GetSourceData() const override
	{
		return &Source;
	}

	/**
	 *	 Initialize this emitter's vertex factory with the vertex buffers from the mesh's rendering data.
	 */
	void SetupVertexFactory( FRHICommandListBase& RHICmdList, FMeshParticleVertexFactory* InVertexFactory, const FStaticMeshLODResources& LODResources, uint32 LODIdx) const;

	/** Returns the source data for this particle system */
	virtual const FDynamicEmitterReplayDataBase& GetSource() const override
	{
		return Source;
	}

	/** The frame source data for this particle system.  This is everything needed to represent this
		this particle system frame.  It does not include any transient rendering thread data.  Also, for
		non-simulating 'replay' particle systems, this data may have come straight from disk! */
	FDynamicMeshEmitterReplayData Source;

	int32					LastFramePreRendered;

	UStaticMesh*		StaticMesh;
	TArray<FMaterialRenderProxy*, TInlineAllocator<2>> MeshMaterials;	

	/** offset to FMeshTypeDataPayload */
	uint32 MeshTypeDataOffset;

	// 'orientation' items...
	// These don't need to go into the replay data, as they are constant over the life of the emitter
	/** If true, apply the 'pre-rotation' values to the mesh. */
	uint32 bApplyPreRotation:1;
	/** If true, then use the locked axis setting supplied. Trumps locked axis module and/or TypeSpecific mesh settings. */
	uint32 bUseMeshLockedAxis:1;
	/** If true, then use the camera facing options supplied. Trumps all other settings. */
	uint32 bUseCameraFacing:1;
	/** 
	 *	If true, apply 'sprite' particle rotation about the orientation axis (direction mesh is pointing).
	 *	If false, apply 'sprite' particle rotation about the camera facing axis.
	 */
	uint32 bApplyParticleRotationAsSpin:1;	
	/** 
	*	If true, all camera facing options will point the mesh against the camera's view direction rather than pointing at the cameras location. 
	*	If false, the camera facing will point to the cameras position as normal.
	*/
	uint32 bFaceCameraDirectionRatherThanPosition:1;
	/** The EMeshCameraFacingOption setting to use if bUseCameraFacing is true. */
	uint8 CameraFacingOption;

	bool bUseStaticMeshLODs;
	float LODSizeScale;
	mutable int32 LastCalculatedMeshLOD;
	const FParticleMeshEmitterInstance* EmitterInstance;
};

/** Source data for Beam emitters */
struct FDynamicBeam2EmitterReplayData
	: public FDynamicSpriteEmitterReplayDataBase
{
	int32									VertexCount;
	int32									IndexCount;
	int32									IndexStride;

	TArray<int32>							TrianglesPerSheet;
	int32									UpVectorStepSize;

	// Offsets to particle data
	int32									BeamDataOffset;
	int32									InterpolatedPointsOffset;
	int32									NoiseRateOffset;
	int32									NoiseDeltaTimeOffset;
	int32									TargetNoisePointsOffset;
	int32									NextNoisePointsOffset;
	int32									TaperValuesOffset;
	int32									NoiseDistanceScaleOffset;

	bool								bLowFreqNoise_Enabled;
	bool								bHighFreqNoise_Enabled;
	bool								bSmoothNoise_Enabled;
	bool								bUseSource;
	bool								bUseTarget;
	bool								bTargetNoise;
	int32									Sheets;
	int32									Frequency;
	int32									NoiseTessellation;
	float								NoiseRangeScale;
	float								NoiseTangentStrength;
	FVector3f								NoiseSpeed;
	float								NoiseLockTime;
	float								NoiseLockRadius;
	float								NoiseTension;

	int32									TextureTile;
	float								TextureTileDistance;
	uint8								TaperMethod;
	int32									InterpolationPoints;

	/** Debugging rendering flags												*/
	bool								bRenderGeometry;
	bool								bRenderDirectLine;
	bool								bRenderLines;
	bool								bRenderTessellation;

	/** Constructor */
	FDynamicBeam2EmitterReplayData()
		: VertexCount(0)
		, IndexCount(0)
		, IndexStride(0)
		, TrianglesPerSheet()
		, UpVectorStepSize(0)
		, BeamDataOffset(-1)
		, InterpolatedPointsOffset(-1)
		, NoiseRateOffset(-1)
		, NoiseDeltaTimeOffset(-1)
		, TargetNoisePointsOffset(-1)
		, NextNoisePointsOffset(-1)
		, TaperValuesOffset(-1)
		, NoiseDistanceScaleOffset(-1)
		, bLowFreqNoise_Enabled( false )
		, bHighFreqNoise_Enabled( false )
		, bSmoothNoise_Enabled( false )
		, bUseSource( false )
		, bUseTarget( false )
		, bTargetNoise( false )
		, Sheets(1)
		, Frequency(1)
		, NoiseTessellation(1)
		, NoiseRangeScale(1)
		, NoiseTangentStrength( 0.0f )
		, NoiseSpeed( 0.0f, 0.0f, 0.0f )
		, NoiseLockTime( 0.0f )
		, NoiseLockRadius( 0.0f )
		, NoiseTension( 0.0f )
		, TextureTile(0)
		, TextureTileDistance(0)
		, TaperMethod(0)
		, InterpolationPoints(0)
		, bRenderGeometry(true)
		, bRenderDirectLine(false)
		, bRenderLines(false)
		, bRenderTessellation(false)
	{
	}


	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		// Call parent implementation
		FDynamicSpriteEmitterReplayDataBase::Serialize( Ar );

		Ar << VertexCount;
		Ar << IndexCount;
		Ar << IndexStride;

		Ar << TrianglesPerSheet;
		Ar << UpVectorStepSize;
		Ar << BeamDataOffset;
		Ar << InterpolatedPointsOffset;
		Ar << NoiseRateOffset;
		Ar << NoiseDeltaTimeOffset;
		Ar << TargetNoisePointsOffset;
		Ar << NextNoisePointsOffset;
		Ar << TaperValuesOffset;
		Ar << NoiseDistanceScaleOffset;

		Ar << bLowFreqNoise_Enabled;
		Ar << bHighFreqNoise_Enabled;
		Ar << bSmoothNoise_Enabled;
		Ar << bUseSource;
		Ar << bUseTarget;
		Ar << bTargetNoise;
		Ar << Sheets;
		Ar << Frequency;
		Ar << NoiseTessellation;
		Ar << NoiseRangeScale;
		Ar << NoiseTangentStrength;
		Ar << NoiseSpeed;
		Ar << NoiseLockTime;
		Ar << NoiseLockRadius;
		Ar << NoiseTension;

		Ar << TextureTile;
		Ar << TextureTileDistance;
		Ar << TaperMethod;
		Ar << InterpolationPoints;

		Ar << bRenderGeometry;
		Ar << bRenderDirectLine;
		Ar << bRenderLines;
		Ar << bRenderTessellation;
	}

};



/** Dynamic emitter data for Beam emitters */
struct FDynamicBeam2EmitterData : public FDynamicSpriteEmitterDataBase 
{
	static const uint32 MaxBeams = 2 * 1024;
	static const uint32 MaxInterpolationPoints = 250;
	static const uint32 MaxNoiseFrequency = 250;

	FDynamicBeam2EmitterData(const UParticleModuleRequired* RequiredModule)
		: 
		  FDynamicSpriteEmitterDataBase(RequiredModule)
		, LastFramePreRendered(-1)
	{
	}

	~FDynamicBeam2EmitterData();

	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	void Init( bool bInSelected );


	virtual void GetDynamicMeshElementsEmitter(const FParticleSystemSceneProxy* Proxy, const FSceneView* View, const FSceneViewFamily& ViewFamily, int32 ViewIndex, FMeshElementCollector& Collector) const override;

	// Debugging functions
	virtual void RenderDirectLine(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI,const FSceneView* View) const;
	virtual void RenderLines(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI,const FSceneView* View) const;

	virtual void RenderDebug(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI, const FSceneView* View, bool bCrosses) const override;

	// Data fill functions
	int32 FillIndexData(struct FAsyncBufferFillData& Data) const;
	int32 FillVertexData_NoNoise(struct FAsyncBufferFillData& Data) const;
	int32 FillData_Noise(struct FAsyncBufferFillData& Data) const;
	int32 FillData_InterpolatedNoise(struct FAsyncBufferFillData& Data) const;

	/** Returns the source data for this particle system */
	virtual const FDynamicEmitterReplayDataBase& GetSource() const override
	{
		return Source;
	}

	/** Perform the actual work of filling the buffer */
	virtual void DoBufferFill(FAsyncBufferFillData& Me) const override;

	/**
	 *	Get the vertex stride for the dynamic rendering data
	 */
	virtual int32 GetDynamicVertexStride(ERHIFeatureLevel::Type /*InFeatureLevel*/) const override
	{
		return sizeof(FParticleBeamTrailVertex);
	}

	/**
	 *	Get the vertex stride for the dynamic parameter rendering data
	 */
	virtual int32 GetDynamicParameterVertexStride() const override
	{
		return sizeof(FParticleBeamTrailVertexDynamicParameter);
	}
		
	/**
	 *	Gets the information required for allocating this emitters indices from the global index array.
	 */
	virtual void GetIndexAllocInfo(int32& OutNumIndices, int32& OutStride ) const override;

	/** The frame source data for this particle system.  This is everything needed to represent this
		this particle system frame.  It does not include any transient rendering thread data.  Also, for
		non-simulating 'replay' particle systems, this data may have come straight from disk! */
	FDynamicBeam2EmitterReplayData Source;

	int32									LastFramePreRendered;
};

/** Source data for trail-type emitters */
struct FDynamicTrailsEmitterReplayData : public FDynamicSpriteEmitterReplayDataBase
{
	int32					PrimitiveCount;
	int32					VertexCount;
	int32					IndexCount;
	int32					IndexStride;

	// Payload offsets
	int32					TrailDataOffset;

	int32					MaxActiveParticleCount;
	int32					TrailCount;
	int32					Sheets;

	/** Constructor */
	FDynamicTrailsEmitterReplayData()
		: PrimitiveCount(0)
		, VertexCount(0)
		, IndexCount(0)
		, IndexStride(0)
		, TrailDataOffset(-1)
		, MaxActiveParticleCount(0)
		, TrailCount(1)
		, Sheets(1)
	{
	}

	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		// Call parent implementation
		FDynamicSpriteEmitterReplayDataBase::Serialize( Ar );

		Ar << PrimitiveCount;
		Ar << VertexCount;
		Ar << IndexCount;
		Ar << IndexStride;

		Ar << TrailDataOffset;

		Ar << MaxActiveParticleCount;
		Ar << TrailCount;
		Ar << Sheets;
	}
};

/** Source data for Ribbon emitters */
struct FDynamicRibbonEmitterReplayData : public FDynamicTrailsEmitterReplayData
{
	// Payload offsets
	int32 MaxTessellationBetweenParticles;

	/** Constructor */
	FDynamicRibbonEmitterReplayData()
		: FDynamicTrailsEmitterReplayData()
		, MaxTessellationBetweenParticles(0)
	{
	}

	/** Serialization */
	virtual void Serialize( FArchive& Ar )
	{
		// Call parent implementation
		FDynamicTrailsEmitterReplayData::Serialize( Ar );
		Ar << MaxTessellationBetweenParticles;
	}
};

/** Dynamic emitter data for Ribbon emitters */
struct FDynamicTrailsEmitterData : public FDynamicSpriteEmitterDataBase 
{
	FDynamicTrailsEmitterData(const UParticleModuleRequired* RequiredModule) : 
		  FDynamicSpriteEmitterDataBase(RequiredModule)
		, LastFramePreRendered(-1)
		, bClipSourceSegement(false)
		, bRenderGeometry(true)
		, bRenderParticles(false)
		, bRenderTangents(false)
		, bRenderTessellation(false)
		, bTextureTileDistance(false)
		, DistanceTessellationStepSize(12.5f)
		, TangentTessellationScalar(25.0f)
		, TextureTileDistance(0.0f)
	{
	}

	~FDynamicTrailsEmitterData();

	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	virtual void Init(bool bInSelected);

	virtual void GetDynamicMeshElementsEmitter(const FParticleSystemSceneProxy* Proxy, const FSceneView* View, const FSceneViewFamily& ViewFamily, int32 ViewIndex, FMeshElementCollector& Collector) const override;

	virtual void RenderDebug(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI, const FSceneView* View, bool bCrosses) const override;

	// Data fill functions
	virtual int32 FillIndexData(struct FAsyncBufferFillData& Data) const;
	virtual int32 FillVertexData(struct FAsyncBufferFillData& Data) const;

	/** Returns the source data for this particle system */
	virtual const FDynamicEmitterReplayDataBase& GetSource() const override
	{
		check(SourcePointer);
		return *SourcePointer;
	}

	virtual const FDynamicTrailsEmitterReplayData* GetSourceData() const override
	{
		check(SourcePointer);
		return SourcePointer;
	}

	virtual void DoBufferFill(FAsyncBufferFillData& Me) const override
	{
		if( Me.VertexCount <= 0 || Me.IndexCount <= 0 || Me.VertexData == NULL || Me.IndexData == NULL )
		{
			return;
		}

		FillIndexData(Me);
		FillVertexData(Me);
	}
		
	/**
	 *	Get the vertex stride for the dynamic rendering data
	 */
	virtual int32 GetDynamicVertexStride(ERHIFeatureLevel::Type /*InFeatureLevel*/) const override
	{
		return sizeof(FParticleBeamTrailVertex);
	}

	/**
	 *	Get the vertex stride for the dynamic parameter rendering data
	 */
	virtual int32 GetDynamicParameterVertexStride() const override
	{
		return sizeof(FParticleBeamTrailVertexDynamicParameter);
	}
	
	/**
	 *	Gets the number of indices to be allocated for this emitter.
	 */
	virtual void GetIndexAllocInfo(int32& OutNumIndices, int32& OutStride ) const override;

	FDynamicTrailsEmitterReplayData*	SourcePointer;
	/**	The sprite particle data.										*/
	int32									LastFramePreRendered;

	uint32	bClipSourceSegement:1;
	uint32	bRenderGeometry:1;
	uint32	bRenderParticles:1;
	uint32	bRenderTangents:1;
	uint32	bRenderTessellation:1;
	uint32	bTextureTileDistance:1;

	float DistanceTessellationStepSize;
	float TangentTessellationScalar;
	float TextureTileDistance;
};

/** Dynamic emitter data for Ribbon emitters */
struct FDynamicRibbonEmitterData : public FDynamicTrailsEmitterData
{
	FDynamicRibbonEmitterData(const UParticleModuleRequired* RequiredModule) : 
		  FDynamicTrailsEmitterData(RequiredModule)
	{
	}

	virtual ~FDynamicRibbonEmitterData() = default;

	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	virtual void Init(bool bInSelected);

	virtual void RenderDebug(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI, const FSceneView* View, bool bCrosses) const;

	// Data fill functions
	virtual int32 FillVertexData(struct FAsyncBufferFillData& Data) const;
		
	/**
	 *	Get the source replay data for this emitter
	 */
	virtual const FDynamicRibbonEmitterReplayData* GetSourceData() const
	{
		return &Source;
	}

	/** 
	 *	The frame source data for this particle system.  This is everything needed to represent this
	 *	this particle system frame.  It does not include any transient rendering thread data.  Also, for
	 *	non-simulating 'replay' particle systems, this data may have come straight from disk!
	 */
	FDynamicRibbonEmitterReplayData Source;

	/**	The sprite particle data.										*/
	uint32	RenderAxisOption:2;
};

/** Dynamic emitter data for AnimTrail emitters */
struct FDynamicAnimTrailEmitterData : public FDynamicTrailsEmitterData
{
	FDynamicAnimTrailEmitterData(const UParticleModuleRequired* RequiredModule) : 
		  FDynamicTrailsEmitterData(RequiredModule)
	{
	}

	virtual ~FDynamicAnimTrailEmitterData() = default;

	/** Initialize this emitter's dynamic rendering data, called after source data has been filled in */
	virtual void Init(bool bInSelected);

	virtual void RenderDebug(const FParticleSystemSceneProxy* Proxy, FPrimitiveDrawInterface* PDI, const FSceneView* View, bool bCrosses) const;

	// Data fill functions
	virtual int32 FillVertexData(struct FAsyncBufferFillData& Data) const;

	/** 
	 *	The frame source data for this particle system.  This is everything needed to represent this
	 *	this particle system frame.  It does not include any transient rendering thread data.  Also, for
	 *	non-simulating 'replay' particle systems, this data may have come straight from disk!
	 */
	FDynamicTrailsEmitterReplayData Source;
};

/*-----------------------------------------------------------------------------
 *	Particle dynamic data
 *	This is a copy of the particle system data needed to render the system in
 *	another thread.
 ----------------------------------------------------------------------------*/
class FParticleDynamicData
{
public:
	FParticleDynamicData()
		: DynamicEmitterDataArray()
	{
	}

	~FParticleDynamicData()
	{
		ClearEmitterDataArray();
	}

	/** Custom new/delete with recycling */
	ENGINE_API void* operator new(size_t Size);
	ENGINE_API void operator delete(void *RawMemory, size_t Size);

	void ClearEmitterDataArray()
	{
		for (int32 Index = 0; Index < DynamicEmitterDataArray.Num(); Index++)
		{
			FDynamicEmitterDataBase* Data =	DynamicEmitterDataArray[Index];
			delete Data;
		}
		DynamicEmitterDataArray.Reset();
	}

	uint32 GetMemoryFootprint( void ) const { return( sizeof( *this ) + DynamicEmitterDataArray.GetAllocatedSize() ); }

	/** The Current Emmitter we are rendering **/
	uint32 EmitterIndex;

	// Variables
	TArray<FDynamicEmitterDataBase*, TInlineAllocator<12> >	DynamicEmitterDataArray;

	/** World space position that UVs generated with the ParticleMacroUV material node will be centered on. */
	FVector SystemPositionForMacroUVs;

	/** World space radius that UVs generated with the ParticleMacroUV material node will tile based on. */
	float SystemRadiusForMacroUVs;

#if WITH_PARTICLE_PERF_STATS
	FParticlePerfStatsContext PerfStatContext;
#endif
};


#if STATS
/*-----------------------------------------------------------------------------
 *	FParticleMemoryStatManager
 *	Handles the collection of various ParticleSystemComponents memory stats
 ----------------------------------------------------------------------------*/
struct FParticleMemoryStatManager
{
public:
	static uint32 DynamicPSysCompCount;
	static uint32 DynamicPSysCompMem;
	static uint32 DynamicEmitterCount;
	static uint32 DynamicEmitterMem;
	static uint32 TotalGTParticleData;
	static uint32 TotalRTParticleData;

	static uint32 DynamicSpriteCount;
	static uint32 DynamicSubUVCount;
	static uint32 DynamicMeshCount;
	static uint32 DynamicBeamCount;
	static uint32 DynamicRibbonCount;
	static uint32 DynamicAnimTrailCount;

	static uint32 DynamicSpriteGTMem;
	static uint32 DynamicSubUVGTMem;
	static uint32 DynamicMeshGTMem;
	static uint32 DynamicBeamGTMem;
	static uint32 DynamicRibbonGTMem;
	static uint32 DynamicAnimTrailGTMem;
	static uint32 DynamicUntrackedGTMem;

	static uint32 DynamicPSysCompCount_MAX;
	static uint32 DynamicPSysCompMem_MAX;
	static uint32 DynamicEmitterCount_MAX;
	static uint32 DynamicEmitterMem_MAX;
	static uint32 DynamicEmitterGTMem_Waste_MAX;
	static uint32 DynamicEmitterGTMem_Largest_MAX;
	static uint32 TotalGTParticleData_MAX;
	static uint32 TotalRTParticleData_MAX;
	static uint32 LargestRTParticleData_MAX;

	static uint32 DynamicSpriteCount_MAX;
	static uint32 DynamicSubUVCount_MAX;
	static uint32 DynamicMeshCount_MAX;
	static uint32 DynamicBeamCount_MAX;
	static uint32 DynamicRibbonCount_MAX;
	static uint32 DynamicAnimTrailCount_MAX;

	static uint32 DynamicSpriteGTMem_MAX;
	static uint32 DynamicSubUVGTMem_MAX;
	static uint32 DynamicMeshGTMem_MAX;
	static uint32 DynamicBeamGTMem_MAX;
	static uint32 DynamicRibbonGTMem_MAX;
	static uint32 DynamicAnimTrailGTMem_MAX;
	static uint32 DynamicUntrackedGTMem_MAX;

	static void ResetParticleMemoryMaxValues();

	static void DumpParticleMemoryStats(FOutputDevice& Ar);

	/**
	 *	Update the stats for all particle system components
	 */
	static void UpdateStats();
};

#endif

class FNullDynamicParameterVertexBuffer : public FVertexBuffer
{
public:
	/** 
	* Initialize the RHI for this rendering resource 
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// create a static vertex buffer
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FNullDynamicParameterVertexBuffer"), sizeof(FParticleVertexDynamicParameter))
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		FParticleVertexDynamicParameter Value;
		Value.DynamicValue[0] = Value.DynamicValue[1] = Value.DynamicValue[2] = Value.DynamicValue[3] = 1.0f;

		FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer.WriteData(&Value, sizeof(Value));

		VertexBufferRHI = Initializer.Finalize();
	}
};

/** The global null color vertex buffer, which is set with a stride of 0 on meshes without a color component. */
extern ENGINE_API TGlobalResource<FNullDynamicParameterVertexBuffer> GNullDynamicParameterVertexBuffer;

inline FVector3f GetParticleBaseSize(const FBaseParticle& Particle, bool bKeepFlipScale = false)
{
	return bKeepFlipScale ? Particle.BaseSize : FVector3f(FMath::Abs(Particle.BaseSize.X), FMath::Abs(Particle.BaseSize.Y), FMath::Abs(Particle.BaseSize.Z));
}

inline FVector2D GetParticleSizeWithUVFlipInSign(const FBaseParticle& Particle, const FVector2D& ScaledSize)
{
	return FVector2D(
		Particle.BaseSize.X >= 0.0f ? ScaledSize.X : -ScaledSize.X,
		Particle.BaseSize.Y >= 0.0f ? ScaledSize.Y : -ScaledSize.Y);
}


/** A level of significance for a particle system. Used by game code to enable/disable emitters progressively as they move away from the camera or are occluded/off screen. */
UENUM()
enum class EParticleSignificanceLevel : uint8
{
	/** Low significance emitter. Culled first. */
	Low,
	/** Medium significance emitter. */
	Medium,
	/** High significance emitter. Culled last. */
	High,
	/** Critical emitter. Never culled. */
	Critical,

	Num UMETA(Hidden),
};

/** Determines what a particle system will do when all of it's emitters become insignificant. */
UENUM()
enum class EParticleSystemInsignificanceReaction: uint8
{
	/** Looping systems will DisableTick. Non-looping systems will Complete.*/
	Auto,
	/** The system will be considered complete and will auto destroy if desired etc.*/
	Complete,
	/** The system will simply stop ticking. Tick will be re-enabled when any emitters become significant again. This is useful for persistent fx such as environmental fx.  */
	DisableTick,
	/** As DisableTick but will also kill all particles. */
	DisableTickAndKill UMETA(Hidden), //Hidden for now until I make it useful i.e. Killing particles saves memory.

	Num UMETA(Hidden),
};

/** Helper class to reset and recreate all PSCs with specific templates on their next tick. */
class FParticleResetContext
{
public:

	TArray<class UParticleSystem*, TInlineAllocator<32>> SystemsToReset;
	ENGINE_API void AddTemplate(class UParticleSystem* Template);
	ENGINE_API void AddTemplate(class UParticleModule* Module);
	ENGINE_API void AddTemplate(class UParticleEmitter* Emitter);
	ENGINE_API ~FParticleResetContext();
};


struct FParticleSystemCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		SkipCookingEmittersBasedOnDetailMode,	// skip emitter cooking if their detail mode doesn't match predefined
		FixLegacySpawningBugs,					// fixing some spawning bugs but must keep old behavior around for existing systems.
		AddEpicDetailMode,						// adding another bitmask entry to EParticleDetailMode

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FParticleSystemCustomVersion() {}
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Math/RandomStream.h"
#include "RHI.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/MaterialInterface.h"
#include "MeshBatch.h"
#include "MeshParticleVertexFactory.h"
#include "PrimitiveSceneProxy.h"
#include "ParticleSystemSceneProxy.h"
#endif
