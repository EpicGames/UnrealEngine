// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleResources.h: Declaration of global particle resources.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"

/** The number of sprites to support per-instanced draw. */
enum { MAX_PARTICLES_PER_INSTANCE = 16 };
/** The size of the scratch vertex buffer. */
extern const int32 GParticleScratchVertexBufferSize;

/**
 * Vertex buffer containing texture coordinates for the four corners of a sprite.
 */
class FParticleTexCoordVertexBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Global particle texture coordinate vertex buffer. */
extern ENGINE_API TGlobalResource<FParticleTexCoordVertexBuffer> GParticleTexCoordVertexBuffer;

class FParticleEightTexCoordVertexBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

extern TGlobalResource<FParticleEightTexCoordVertexBuffer> GParticleEightTexCoordVertexBuffer;

/**
 * Index buffer for drawing an individual sprite.
 */
class FParticleIndexBuffer : public FIndexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/**
 * Index buffer for drawing an individual sprite.
 */
class FSixTriangleParticleIndexBuffer : public FIndexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Global particle index buffer. */
ENGINE_API extern TGlobalResource<FParticleIndexBuffer> GParticleIndexBuffer;
ENGINE_API extern TGlobalResource<FSixTriangleParticleIndexBuffer> GSixTriangleParticleIndexBuffer;

typedef FRHIShaderResourceView* FParticleShaderParamRef;
typedef FRHIBuffer* FParticleBufferParamRef;

/**
 * Scratch vertex buffer available for dynamic draw calls.
 */
class FParticleScratchVertexBuffer : public FVertexBuffer
{
public:

	FParticleShaderParamRef GetShaderParam();
	FParticleBufferParamRef GetBufferParam();

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	static const uint32 Stride = sizeof(FVector4f);

private:

	/** SRV in to the buffer as an array of FVector4f values. */
	FShaderResourceViewRHIRef VertexBufferSRV_A32B32G32R32F;
};

/** The global scratch vertex buffer. */
extern TGlobalResource<FParticleScratchVertexBuffer> GParticleScratchVertexBuffer;
