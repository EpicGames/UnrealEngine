// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialRelevance.h"
#include "Particles/ParticlePerfStats.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneProxyDesc.h"

class FColoredMaterialRenderProxy;
class FParticleDynamicData;
class UParticleSystemComponent;
struct FDynamicEmitterDataBase;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FParticleSystemSceneProxyDesc : public FPrimitiveSceneProxyDesc
{
	ENGINE_API FParticleSystemSceneProxyDesc();
	ENGINE_API FParticleSystemSceneProxyDesc(UParticleSystemComponent& Component, FParticleDynamicData* InDynamicData, bool InbCanBeOccluded);

	UParticleSystem* SystemAsset;
	FParticleDynamicData* DynamicData;
	FMaterialRelevance MaterialRelevance;
	int32 VisualizeLODIndex; // Only used in the LODColoration view mode.
	int32 LODMethod;
	uint8 bCanBeOccluded : 1;
	uint8 bManagingSignificance : 1;
	uint8 bAlwaysHasVelocity : 1;

	ENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FParticleSystemSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;

	/** Initialization constructor. */
	ENGINE_API FParticleSystemSceneProxy(const FParticleSystemSceneProxyDesc& Desc);
	virtual ~FParticleSystemSceneProxy();

	// FPrimitiveSceneProxy interface.
	virtual bool CanBeOccluded() const override
	{
		return  bCanBeOccluded ? !MaterialRelevance.bDisableDepthTest : false;
	}

	/**
	*	Returns whether the proxy utilizes custom occlusion bounds or not
	*
	*	@return	bool		true if custom occlusion bounds are used, false if not;
	*/
	virtual bool HasCustomOcclusionBounds() const override
	{
		return bCanBeOccluded ? bHasCustomOcclusionBounds : FPrimitiveSceneProxy::HasCustomOcclusionBounds();
	}

	/**
	*	Return the custom occlusion bounds for this scene proxy.
	*
	*	@return	FBoxSphereBounds		The custom occlusion bounds.
	*/
	virtual FBoxSphereBounds GetCustomOcclusionBounds() const override
	{
		return bCanBeOccluded ? OcclusionBounds.TransformBy(GetLocalToWorld()) : FPrimitiveSceneProxy::GetCustomOcclusionBounds();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;

	/** Gathers simple lights for this emitter. */
	virtual void GatherSimpleLights(const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const override;

	/**
	 *	Called when the rendering thread adds the proxy to the scene.
	 *	This function allows for generating renderer-side resources.
	 */
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;

	/**
	 *	Called when the rendering thread removes the dynamic data from the scene.
	 */
	void ReleaseRenderThreadResources();

	ENGINE_API void UpdateData(FParticleDynamicData* NewDynamicData);
	void UpdateData_RenderThread(FParticleDynamicData* NewDynamicData);

	FParticleDynamicData* GetDynamicData()
	{
		return DynamicData;
	}

	FParticleDynamicData* GetLastDynamicData()
	{
		return LastDynamicData;
	}

	void SetLastDynamicData(FParticleDynamicData* InLastDynamicData)
	{
		LastDynamicData  = InLastDynamicData;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const 
	{ 
		uint32 AdditionalSize = (uint32)FPrimitiveSceneProxy::GetAllocatedSize();

		return( AdditionalSize ); 
	}

	// @param FrameNumber from ViewFamily.FrameNumber
	void DetermineLODDistance(const FSceneView* View, int32 FrameNumber);

	/**
	 * Called by dynamic emitter data during initialization to make sure the
	 * world space primitive uniform buffer is up-to-date.
	 * Only called in the rendering thread.
	 */
	void UpdateWorldSpacePrimitiveUniformBuffer(FRHICommandListBase& RHICmdList) const;

	/** Object position in post projection space. */
	void GetObjectPositionAndScale(const FSceneView& View, FVector2D& ObjectNDCPosition, FVector2D& ObjectMacroUVScales) const;

	// While this isn't good OO design, access to everything is made public.
	// This is to allow custom emitter instances to easily be written when extending the engine.
	FMatrix GetWorldToLocal() const		{	return GetLocalToWorld().Inverse();	}
	bool GetCastShadow() const			{	return bCastShadow;				}
	const FMaterialRelevance& GetMaterialRelevance() const
	{
		return MaterialRelevance;
	}
	float GetPendingLODDistance() const	{	return PendingLODDistance;		}
	void SetVisualizeLODIndex(int32 InVisualizeLODIndex) { VisualizeLODIndex = InVisualizeLODIndex; }
	int32  GetVisualizeLODIndex() const { return VisualizeLODIndex; }

	inline FRHIUniformBuffer* GetWorldSpacePrimitiveUniformBuffer() const { return WorldSpacePrimitiveUniformBuffer.GetUniformBufferRHI(); }

	const FColoredMaterialRenderProxy* GetDeselectedWireframeMatInst() const { return DeselectedWireframeMaterialInstance; }

	/** Gets a mesh batch from the pool. */
	FMeshBatch* GetPooledMeshBatch();

	// persistent proxy storage for mesh emitter LODs; need to store these here, because GDME needs to calc the index,
	// but VF needs to be init'ed with the correct LOD, and DynamicData goes away every frame
	mutable TArray<int32> MeshEmitterLODIndices;
	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel;  }
protected:

	/**
	 * Allows dynamic emitter data to create render thread resources.
	 */
	void CreateRenderThreadResourcesForEmitterData();

	/**
	 * Allows dynamic emitter data to release render thread resources.
	 */
	void ReleaseRenderThreadResourcesForEmitterData();

#if STATS
	double LastStatCaptureTime;
	uint8 bCountedThisFrame:1;
#endif

	uint8 bCastShadow : 1;
	uint8 bManagingSignificance : 1;

private:
	uint8	bCanBeOccluded : 1;
	uint8	bHasCustomOcclusionBounds : 1;

protected:
	TEnumAsByte<ERHIFeatureLevel::Type> FeatureLevel;

	FMaterialRelevance MaterialRelevance;

	FParticleDynamicData* DynamicData;			// RENDER THREAD USAGE ONLY
	FParticleDynamicData* LastDynamicData;		// RENDER THREAD USAGE ONLY

	FColoredMaterialRenderProxy* DeselectedWireframeMaterialInstance;

	int32 LODMethod;
	float PendingLODDistance;
	int32 VisualizeLODIndex; // Only used in the LODColoration view mode.

	// from ViewFamily.FrameNumber
	int32 LastFramePreRendered;

	/** The primitive's uniform buffer.  Mutable because it is cached state during GetDynamicMeshElements. */
	mutable TUniformBuffer<FPrimitiveUniformShaderParameters> WorldSpacePrimitiveUniformBuffer;
	mutable uint32 WorldSpaceUBHash = 0;
	mutable UE::FMutex WorldSpacePrimitiveUniformBufferMutex;

	/** Pool for holding FMeshBatches to reduce allocations. */
	TIndirectArray<FMeshBatch, TInlineAllocator<4> > MeshBatchPool;
	int32 FirstFreeMeshBatch;

private:
	/** Bounds for occlusion rendering. */
	FBoxSphereBounds OcclusionBounds;

protected:
	mutable TArray<FDynamicEmitterDataBase*> DynamicDataForThisFrame;

	friend struct FDynamicSpriteEmitterDataBase;

#if WITH_PARTICLE_PERF_STATS
public:
	FParticlePerfStatsContext PerfStatContext;
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////
