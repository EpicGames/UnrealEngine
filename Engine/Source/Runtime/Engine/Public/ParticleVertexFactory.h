// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactory.h: Particle vertex factory definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"

class FMaterial;

/**
 * Enum identifying the type of a particle vertex factory.
 */
enum EParticleVertexFactoryType
{
	PVFT_Sprite,
	PVFT_BeamTrail,
	PVFT_Mesh,
	PVFT_MAX
};

/**
 * Base class for particle vertex factories.
 */
class FParticleVertexFactoryBase : public FVertexFactory
{
public:
	explicit FParticleVertexFactoryBase(ERHIFeatureLevel::Type InFeatureLevel)
		: FVertexFactory(InFeatureLevel)
	{
	}

	/** Default constructor. */
	explicit FParticleVertexFactoryBase( EParticleVertexFactoryType Type, ERHIFeatureLevel::Type InFeatureLevel )
		: FVertexFactory(InFeatureLevel)
	{
	}

	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) 
	{
		FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("PARTICLE_FACTORY"),TEXT("1"));
	}

	/** Return the vertex factory type */
	inline EParticleVertexFactoryType GetParticleFactoryType() const
	{
		return ParticleFactoryType;
	}

	inline void SetParticleFactoryType(EParticleVertexFactoryType InType)
	{
		ParticleFactoryType = InType;
	}

	ERHIFeatureLevel::Type GetFeatureLevel() const { check(HasValidFeatureLevel());  return FRenderResource::GetFeatureLevel(); }

private:
	/** The type of the vertex factory. */
	EParticleVertexFactoryType ParticleFactoryType;
};

/**
 * Uniform buffer for particle sprite vertex factories.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FParticleSpriteUniformParameters, ENGINE_API)
	SHADER_PARAMETER_EX( FVector4f, AxisLockRight, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, AxisLockUp, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, TangentSelector, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, NormalsSphereCenter, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, NormalsCylinderUnitDirection, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector4f, SubImageSize, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( FVector3f, CameraFacingBlend, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, RemoveHMDRoll, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER( FVector4f, MacroUVParameters )
	SHADER_PARAMETER_EX( float, RotationScale, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, RotationBias, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, NormalsType, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER_EX( float, InvDeltaSeconds, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER( FVector3f, LWCTile )
	SHADER_PARAMETER_EX( FVector2f, PivotOffset, EShaderPrecisionModifier::Half )
	SHADER_PARAMETER(float, UseVelocityForMotionBlur)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
typedef TUniformBufferRef<FParticleSpriteUniformParameters> FParticleSpriteUniformBufferRef;

/**
 * Vertex factory for rendering particle sprites.
 */
class FParticleSpriteVertexFactory : public FParticleVertexFactoryBase
{
	DECLARE_VERTEX_FACTORY_TYPE(FParticleSpriteVertexFactory);

public:

	/** Default constructor. */
	FParticleSpriteVertexFactory( EParticleVertexFactoryType InType, ERHIFeatureLevel::Type InFeatureLevel )
		: FParticleVertexFactoryBase(InType, InFeatureLevel),
		NumVertsInInstanceBuffer(0),
		NumCutoutVerticesPerFrame(0),
		CutoutGeometrySRV(nullptr),
		bCustomAlignment(false),
		bUsesDynamicParameter(true),
		DynamicParameterStride(0)
	{}

	FParticleSpriteVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FParticleVertexFactoryBase(PVFT_MAX, InFeatureLevel),
		NumVertsInInstanceBuffer(0),
		NumCutoutVerticesPerFrame(0),
		CutoutGeometrySRV(nullptr),
		bCustomAlignment(false),
		bUsesDynamicParameter(true),
		DynamicParameterStride(0)
	{}

	// FRenderResource interface.
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual bool RendersPrimitivesAsCameraFacingSprites() const override { return true; }

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static ENGINE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	 * Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	 */
	static ENGINE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	 * Get vertex elements used when during PSO precaching materials using this vertex factory type
	 */
	static ENGINE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	static ENGINE_API FRHIVertexDeclaration* GetPSOPrecacheVertexDeclaration(bool bUsesDynamicParameter);

	/**
	 * Set the source vertex buffer that contains particle instance data.
	 */
	ENGINE_API void SetInstanceBuffer(const FVertexBuffer* InInstanceBuffer, uint32 StreamOffset, uint32 Stride);

	ENGINE_API void SetTexCoordBuffer(const FVertexBuffer* InTexCoordBuffer);

	inline void SetNumVertsInInstanceBuffer(int32 InNumVertsInInstanceBuffer)
	{
		NumVertsInInstanceBuffer = InNumVertsInInstanceBuffer;
	}

	/**
	 * Set the source vertex buffer that contains particle dynamic parameter data.
	 */
	ENGINE_API void SetDynamicParameterBuffer(const FVertexBuffer* InDynamicParameterBuffer, uint32 StreamOffset, uint32 Stride);
	inline void SetUsesDynamicParameter(bool bInUsesDynamicParameter, uint32 Stride)
	{
		bUsesDynamicParameter = bInUsesDynamicParameter;
		DynamicParameterStride = Stride;
	}

	/**
	 * Set the uniform buffer for this vertex factory.
	 */
	inline void SetSpriteUniformBuffer( const FParticleSpriteUniformBufferRef& InSpriteUniformBuffer )
	{
		SpriteUniformBuffer = InSpriteUniformBuffer;
	}

	/**
	 * Retrieve the uniform buffer for this vertex factory.
	 */
	inline FRHIUniformBuffer* GetSpriteUniformBuffer()
	{
		return SpriteUniformBuffer;
	}

	void SetCutoutParameters(int32 InNumCutoutVerticesPerFrame, FRHIShaderResourceView* InCutoutGeometrySRV)
	{
		NumCutoutVerticesPerFrame = InNumCutoutVerticesPerFrame;
		CutoutGeometrySRV = InCutoutGeometrySRV;
	}

	inline int32 GetNumCutoutVerticesPerFrame() const { return NumCutoutVerticesPerFrame; }
	inline FRHIShaderResourceView* GetCutoutGeometrySRV() const { return CutoutGeometrySRV; }

	void SetCustomAlignment(bool bAlign)
	{
		bCustomAlignment = bAlign;
	}

	bool GetCustomAlignment()
	{
		return bCustomAlignment;
	}

protected:
	/** Initialize streams for this vertex factory. */
	ENGINE_API void InitStreams();

private:

	int32 NumVertsInInstanceBuffer;

	/** Uniform buffer with sprite paramters. */
	FUniformBufferRHIRef SpriteUniformBuffer;

	int32 NumCutoutVerticesPerFrame;
	FRHIShaderResourceView* CutoutGeometrySRV;
	bool bCustomAlignment;
	bool bUsesDynamicParameter;
	uint32 DynamicParameterStride;
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneView.h"
#endif
