// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Redirect PerPlatformProperties.h include to UObject
=============================================================================*/

#pragma once

#include "UObject/PerPlatformProperties.h"

UE_DEPRECATED_HEADER(5.5, "WARNING: Do not #include PerPlatformProperties.h. Please use #include UObject/PerPlatformProperties.h instead.")

