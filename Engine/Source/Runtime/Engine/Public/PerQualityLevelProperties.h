// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
FPerQualityLevelProperties.h: Property types that can be overridden on a quality level basis at cook time
=============================================================================*/

#pragma once

#include "Serialization/Archive.h"
#include "Containers/Map.h"
#include "Algo/Find.h"
#include "Serialization/MemoryLayout.h"
#include "Scalability.h"
#include "CoreMinimal.h"

#include "PerQualityLevelProperties.generated.h"

#if WITH_EDITOR
class UObject;
typedef TSet<int32> FSupportedQualityLevelArray;
#endif


UENUM(BlueprintType)
enum class EPerQualityLevels : uint8
{
	Low,
	Medium,
	High,
	Epic,
	Cinematic,
	Num
};

namespace QualityLevelProperty
{
	enum class UE_DEPRECATED(5.1, "Use EPerQualityLevels instead since we need to expose as an ENUM in blueprint.") EQualityLevels : uint8
	{
		Low,
		Medium,
		High,
		Epic,
		Cinematic,
		Num
	};

	ENGINE_API FName QualityLevelToFName(int32 QL);
	ENGINE_API int32 FNameToQualityLevel(FName QL);
	template<typename _ValueType>
	ENGINE_API TMap<int32, _ValueType> ConvertQualityLevelData(const TMap<EPerQualityLevels, _ValueType>& Data);
	template<typename _ValueType>
	UE_DEPRECATED(5.5, "Use ConvertQualityLevelData.")
	TMap<int32, _ValueType> ConvertQualtiyLevelData(const TMap<EPerQualityLevels, _ValueType>& Data)
	{
		return ConvertQualityLevelData(Data);
	}
	template<typename _ValueType>
	ENGINE_API TMap<EPerQualityLevels, _ValueType> ConvertQualityLevelData(const TMap<int32, _ValueType>& Data);
	template<typename _ValueType>
	UE_DEPRECATED(5.5, "Use ConvertQualityLevelData.")
	TMap<EPerQualityLevels, _ValueType> ConvertQualtiyLevelData(const TMap<int32, _ValueType>& Data)
	{
		return ConvertQualityLevelData(Data);
	}

	template <typename ValueType>
	struct FSavedData
	{
		ValueType Default = 0;
		TMap<int32, ValueType> PerQuality;
	};

#if WITH_EDITOR
	ENGINE_API TArray<FName> GetEnginePlatformsForPlatformOrGroupName(const FString& InPlatformName);
	ENGINE_API FSupportedQualityLevelArray PerPlatformOverrideMapping(FString& InPlatformName, UObject* RequestingAsset = nullptr);
#endif
};

template<typename _StructType, typename _ValueType, EName _BasePropertyName>
struct FPerQualityLevelProperty
{
	typedef _ValueType ValueType;
	typedef _StructType StructType;

	FPerQualityLevelProperty() 
	{
	}
	~FPerQualityLevelProperty() = default;

	_ValueType GetValueForQualityLevel(int32 QualityLevel) const
	{
		const _StructType* This = StaticCast<const _StructType*>(this);
		if (This->PerQuality.Num() == 0 || QualityLevel < 0)
		{
			return This->Default;
		}

		_ValueType* Value = (_ValueType*)This->PerQuality.Find(QualityLevel);
		
		if (Value)
		{
			return *Value;
		}
		else
		{
			return This->Default;
		}
	}

#if WITH_EDITOR
	int32 GetValueForPlatform(const ITargetPlatform* TargetPlatform) const;
	FSupportedQualityLevelArray GetSupportedQualityLevels(const TCHAR* InPlatformName = nullptr) const;
	void StripQualityLevelForCooking(const TCHAR* InPlatformName = nullptr);
	UE_DEPRECATED(5.5, "Use StripQualityLevelForCooking")
	void StripQualtiyLevelForCooking(const TCHAR* InPlatformName = nullptr)
	{
		StripQualityLevelForCooking(InPlatformName);
	}
	bool IsQualityLevelValid(int32 QualityLevel) const;
	void ConvertQualityLevelData(const TMap<FName, _ValueType>& PlatformData, const TMultiMap<FName, FName>& PerPlatformToQualityLevel, _ValueType Default);
	UE_DEPRECATED(5.5, "Use ConvertQualityLevelData")
	void ConvertQualtiyLevelData(TMap<FName, _ValueType>& PlatformData, TMultiMap<FName, FName>& PerPlatformToQualityLevel, _ValueType Default)
	{
		ConvertQualityLevelData(PlatformData, PerPlatformToQualityLevel, Default);
	}
	// Use the CVar set by SetQualityLevelCVarForCooking to convert from PlatformData.
	// This method will do nothing if bRequireAllPlatformsKnown and some of the keys in PlatformData are unrecognized as either Platform names or PlatformGroup names.
	void ConvertQualityLevelDataUsingCVar(const TMap<FName, _ValueType>& PlatformData, _ValueType Default, bool bRequireAllPlatformsKnown);
#endif

	// Set Cvar to be able to scan ini files at cook-time and only have the supported ranges of quality levels relevant to the platform.
	// Unsupported quality levels will be stripped.
	void SetQualityLevelCVarForCooking(const TCHAR* InCVarName, const TCHAR* InSection)
	{
#if WITH_EDITOR
		ScalabilitySection = FString(InSection);
#endif
		CVarName = FString(InCVarName);
	}

	UE_DEPRECATED(5.4, "If no cvar is associated with the property, all quality levels will be keept when cooking. Call SetQualityLevelCVarForCooking to strip unsupported quality levels when cooking")
	void Init(const TCHAR* InCVarName, const TCHAR* InSection)
	{
		SetQualityLevelCVarForCooking(InCVarName, InSection);
	}

	_ValueType GetDefault() const
	{
		const _StructType* This = StaticCast<const _StructType*>(this);
		return This->Default;
	}

	_ValueType GetValue(int32 QualityLevel) const
	{
		return GetValueForQualityLevel(QualityLevel);
	}

	_ValueType GetLowestValue() const
	{
		const _StructType* This = StaticCast<const _StructType*>(this);
		_ValueType Value = This->Default;

		for (const TPair<int32, _ValueType>& Pair : This->PerQuality)
		{
			if (Pair.Value < Value)
			{
				Value = Pair.Value;
			}
		}
		return Value;
	}

	/* Load old properties that have been converted to FPerQualityLevel */
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FArchive& Ar)
	{
		if (Tag.Type == _BasePropertyName)
		{
			_StructType* This = StaticCast<_StructType*>(this);
			_ValueType OldValue;
			Ar << OldValue;
			*This = _StructType(OldValue);
			return true;
		}
		return false;
	}

	/* Serialization */
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	/* Serialization */
	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}

#if WITH_EDITOR
	FString ScalabilitySection;
#endif
	FString CVarName;

private:

	/* Make this operator friend so it can access private member SavedValue. */
	friend FArchive& operator<<(FArchive& Ar, FPerQualityLevelProperty& Property) { Property.StreamArchive(Ar); return Ar; } 
	friend void operator<<(FStructuredArchive::FSlot Slot, FPerQualityLevelProperty& Property) { Property.StreamStructuredArchive(Slot); }

	ENGINE_API void StreamArchive(FArchive& Ar);
	ENGINE_API void StreamStructuredArchive(FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	/** 
	* Store values here during harvesting so they can be restored during PostSave. 
	* Why is this a TPimplPtr<...>. The idea is to only restore the state if it was first saved in SavedValue.
	* To detect if a value was saved, I simply check if the object was allocated. Using a TUniquePtr<...> would be better
	* but compilation fails because this class has a compiler generated copy operator and a unique ptr can't be copied. So using
	* a TPimplPtr is equivalent to using a unique ptr but it can be deep copied and the compiler is happy.
	*/
	TPimplPtr<QualityLevelProperty::FSavedData<ValueType>, EPimplPtrMode::DeepCopy> SavedValue;
#endif

};

USTRUCT(BlueprintType)
struct FPerQualityLevelInt 
#if CPP
	:	public FPerQualityLevelProperty<FPerQualityLevelInt, int32, NAME_IntProperty>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = PerQualityLevel)
	int32 Default;

	UPROPERTY(EditAnywhere, Category = PerQualityLevel)
	TMap<int32, int32> PerQuality;

	FPerQualityLevelInt()
	{
		Default = 0;
	}

	FPerQualityLevelInt(int32 InDefaultValue)
	{
		Default = InDefaultValue;
	}

	ENGINE_API FString ToString() const;
	int32 MaxType() const { return MAX_int32; }
};

template<>
struct TStructOpsTypeTraits<FPerQualityLevelInt>
	: public TStructOpsTypeTraitsBase2<FPerQualityLevelInt>
{
	enum
	{
		WithSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

USTRUCT(BlueprintType)
struct FPerQualityLevelFloat
#if CPP
	:	public FPerQualityLevelProperty<FPerQualityLevelFloat, float, NAME_FloatProperty>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = PerQualityLevel)
	float Default;

	UPROPERTY(EditAnywhere, Category = PerQualityLevel)
	TMap<int32, float> PerQuality;

	FPerQualityLevelFloat()
	{
		Default = 0.0f;
	}

	FPerQualityLevelFloat(float InDefaultValue)
	{
		Default = InDefaultValue;
	}

	ENGINE_API FString ToString() const;
	float MaxType() const { return UE_MAX_FLT; }
};

template<>
struct TStructOpsTypeTraits<FPerQualityLevelFloat>
	: public TStructOpsTypeTraitsBase2<FPerQualityLevelFloat>
{
	enum
	{
		WithSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

