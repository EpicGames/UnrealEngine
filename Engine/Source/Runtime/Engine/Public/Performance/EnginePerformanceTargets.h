// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FEnginePerformanceTargets
{
public:
	// The target threshold for frame time in miliseconds (*cosmetic only* and used for fps/stat display, should not be used in scalability code)
	// Values below this will be green, values between this and the unacceptable threshold will be yellow, and values above that will be red
	static ENGINE_API float GetTargetFrameTimeThresholdMS();

	// The maximum threshold for an 'OK' frame time in miliseconds (*cosmetic only* and used for fps/stat display, should not be used in scalability code)
	// Values above this will be red, values between this and the acceptable limit will be yellow, and values below will be green.
	static ENGINE_API float GetUnacceptableFrameTimeThresholdMS();

	// The threshold that would be considered so bad that it would cause a hitch in gameplay
	// (*cosmetic only* for reporting purposes such as FPS charts, should not be used in scalability code)
	static ENGINE_API float GetHitchFrameTimeThresholdMS();

	// Minimum time passed before we'll record a new hitch
	static ENGINE_API float GetMinTimeBetweenHitchesMS();

	// For the current frame to be considered a hitch, it must have run at least this many times slower than the previous frame
	static ENGINE_API float GetHitchToNonHitchRatio();

private:
	FEnginePerformanceTargets() {}
};
