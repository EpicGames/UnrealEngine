// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

class ILatencyMarkerModule : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("LatencyMarker");
		return FeatureName;
	}

	virtual void Initialize() = 0;

	virtual void SetEnabled(bool bInEnabled) = 0;
	virtual bool GetEnabled() = 0;
	virtual bool GetAvailable() = 0;
	virtual void SetFlashIndicatorEnabled(bool bInEnabled) = 0;
	virtual bool GetFlashIndicatorEnabled() = 0;

	virtual void SetInputSampleLatencyMarker(uint64 FrameNumber) = 0;
	virtual void SetSimulationLatencyMarkerStart(uint64 FrameNumber) = 0;
	virtual void SetSimulationLatencyMarkerEnd(uint64 FrameNumber) = 0;
	virtual void SetPresentLatencyMarkerStart(uint64 FrameNumber) = 0;
	virtual void SetPresentLatencyMarkerEnd(uint64 FrameNumber) = 0;
	virtual void SetRenderSubmitLatencyMarkerStart(uint64 FrameNumber) = 0;
	virtual void SetRenderSubmitLatencyMarkerEnd(uint64 FrameNumber) = 0;
	virtual void SetFlashIndicatorLatencyMarker(uint64 FrameNumber) = 0;
	virtual void SetCustomLatencyMarker(uint32 MarkerId, uint64 FrameNumber) = 0;

	virtual float GetTotalLatencyInMs() = 0;
	virtual float GetGameLatencyInMs() = 0; // This is defined as "Game simulation start to driver submission end"
	virtual float GetRenderLatencyInMs() = 0; // This is defined as "OS render queue start to GPU render end"

	virtual float GetSimulationLatencyInMs() = 0;
	virtual float GetRenderSubmitLatencyInMs() = 0;
	virtual float GetPresentLatencyInMs() = 0;
	virtual float GetDriverLatencyInMs() = 0;
	virtual float GetOSRenderQueueLatencyInMs() = 0;
	virtual float GetGPURenderLatencyInMs() = 0;

	virtual float GetRenderSubmitOffsetFromFrameStartInMs() = 0;
	virtual float GetPresentOffsetFromFrameStartInMs() = 0;
	virtual float GetDriverOffsetFromFrameStartInMs() = 0;
	virtual float GetOSRenderQueueOffsetFromFrameStartInMs() = 0;
	virtual float GetGPURenderOffsetFromFrameStartInMs() = 0;
};
