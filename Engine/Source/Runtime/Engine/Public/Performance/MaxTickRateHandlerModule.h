// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"

class IMaxTickRateHandlerModule : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("MaxTickRateHandler");
		return FeatureName;
	}

	virtual void Initialize() = 0;
	virtual void SetEnabled(bool bInEnabled) = 0;
	virtual bool GetEnabled() = 0;
	virtual bool GetAvailable() = 0;

	virtual void SetFlags(uint32 Flags) = 0;
	virtual uint32 GetFlags() = 0;

	// Return true if waiting occurred in the plugin, if false engine will use the default sleep setup
	virtual bool HandleMaxTickRate(float DesiredMaxTickRate) = 0;
};
