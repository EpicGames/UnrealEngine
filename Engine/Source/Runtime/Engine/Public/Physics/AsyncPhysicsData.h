// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncPhysicsData.generated.h"

/**
	The base class for async physics data. Inherit from this to create custom data for async physics tick.
	When no data is available (say due to massive latency or packet loss) we fall back on the default constructed data.
	This means you should set the default values to something equivalent to no input (for example bPlayerWantsToJump should probably default to false)
*/
UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UAsyncPhysicsData : public UObject
{
	GENERATED_BODY()
public:
	virtual ~UAsyncPhysicsData() = default;
	int32 GetServerFrame() const { return ServerFrame; }

private:
	UPROPERTY()
	int32 ServerFrame = INDEX_NONE;
protected:

	//Determines how many times we redundantly send data to server. The higher this number the less packet loss, but more bandwidth is used
	UPROPERTY(EditDefaultsOnly, Category = AsyncPhysicsData)
	int32 ReplicationRedundancy = 4;

	friend class UAsyncPhysicsInputComponent;
};
