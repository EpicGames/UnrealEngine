// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Components/ActorComponent.h"
#include "Physics/AsyncPhysicsData.h"

#include "AsyncPhysicsInputComponent.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UAsyncPhysicsInputComponent : public UActorComponent
{
	GENERATED_BODY()
public:
	ENGINE_API UAsyncPhysicsInputComponent();
	
	/** Sets the data class used. Can only call this once after the component has been created */
	ENGINE_API void SetDataClass(TSubclassOf<UAsyncPhysicsData> InDataClass);

	ENGINE_API virtual void AsyncPhysicsTickComponent(float DeltaTime, float SimTime) override;

	UFUNCTION(Server, unreliable)
	ENGINE_API void ServerRPCBufferInput(UAsyncPhysicsData* AsyncPhysicsData);

	ENGINE_API void OnDispatchPhysicsTick(int32 PhysicsStep, int32 NumSteps, int32 ServerFrame);

	ENGINE_API APlayerController* GetPlayerController();

	/** Get the async physics data to write to. This data will make its way to the async physics tick on client and server. Should not be called from async tick */
	UFUNCTION(BlueprintPure, Category = PlayerController)
	ENGINE_API UAsyncPhysicsData* GetDataToWrite() const;

	/** Get the async physics data to execute logic off of. This data should not be modified and will NOT make its way back. Should be called from async tick */
	UFUNCTION(BlueprintPure, Category = PlayerController)
	ENGINE_API const UAsyncPhysicsData* GetDataToConsume() const;


private:
	UPROPERTY()
	TSubclassOf<UAsyncPhysicsData> DataClass;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UAsyncPhysicsData>> BufferedData;

	UPROPERTY(Transient)
	TObjectPtr<UAsyncPhysicsData> DataToConsume;

	UPROPERTY(Transient)
	TObjectPtr<UAsyncPhysicsData> DataToWrite;
};
