// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsQueryHandler.h"

#include "DefaultPhysicsQueryHandler.generated.h"

UCLASS(hidecategories=(Object), MinimalAPI)
// This does default handling for physics queries (as if there was no handler).
class UDefaultPhysicsQueryHandler : public UPhysicsQueryHandler
{
public:
	GENERATED_BODY()

	ENGINE_API virtual bool Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps) override;

	ENGINE_API virtual bool RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits) override;
	ENGINE_API virtual bool RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits) override;
	ENGINE_API virtual bool RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits) override;

	ENGINE_API virtual bool SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits) override;
	ENGINE_API virtual bool SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits) override;
	ENGINE_API virtual bool SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits) override;
};
