// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Misc/Timeout.h"

// Interface used by FPhysScene_AsyncPhysicsStateJobQueue

class UObject;
class UBodySetup;

class IAsyncPhysicsStateProcessor
{
public:
	/** Returns whether this component allows having its physics state being created asynchronously (outside of the GameThread). */
	virtual bool AllowsAsyncPhysicsStateCreation() const { return false; }
	
	/** Returns whether this component allows having its physics state being destroyed asynchronously (outside of the GameThread). */
	virtual bool AllowsAsyncPhysicsStateDestruction() const { return false; }

	/** Returns whether the physics state is created. */
	virtual bool IsAsyncPhysicsStateCreated() const { return false; }

	/** Returns the associated UObject for this processor. */
	virtual UObject* GetAsyncPhysicsStateObject() const { return nullptr; }

	/** Returns body setups that needs to create their physics meshes before physics state asynchronous creation */
	virtual void CollectBodySetupsWithPhysicsMeshesToCreate(TSet<UBodySetup*>& OutBodySetups) const {}

	/** 
	 * Called on the GameThread before the component's physics state is created.
	 * Allows preparing caches for asynchronous physics state creation and adding
	 * any objects that must remain rooted during the async process.
	 * 
	 * @param OutRootedObjects Objects that must be rooted for the duration of the async creation.
	 */
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) {}

	/** Used to create any physics state for this component outside of the GameThread. */
	virtual bool OnAsyncCreatePhysicsState(const UE::FTimeout& TimeOut) { return true; }

	/**
	 * Called on the GameThread once the component's physics state is created.
	 * If necessary, free caches.
	 */
	virtual void OnAsyncCreatePhysicsStateEnd_GameThread() {}

	/** 
	 * Called on the GameThread before the component's physics state is destroyed.
	 * Allows preparing caches for asynchronous physics state destruction and adding
	 * any objects that must remain rooted during the async process.
	 * 
	 * @param OutRootedObjects Objects that must be rooted for the duration of the async destruction.
	 */
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread(TSet<UObject*>& OutRootedObjects) {}

	/** Used to destroy any physics state for this component outside of the GameThread. */
	virtual bool OnAsyncDestroyPhysicsState(const UE::FTimeout& TimeOut) { return true; }

	/**
	 * Called on the GameThread once the component's physics state is destroyed. 
	 * If necessary, free caches.
	 */
	virtual void OnAsyncDestroyPhysicsStateEnd_GameThread() {}

	UE_DEPRECATED(5.8, "Use OnAsyncCreatePhysicsStateBegin_GameThread with parameters instead")
	virtual void OnAsyncCreatePhysicsStateBegin_GameThread() {}

	UE_DEPRECATED(5.8, "Use OnAsyncDestroyPhysicsStateBegin_GameThread with parameters instead")
	virtual void OnAsyncDestroyPhysicsStateBegin_GameThread() {}
};