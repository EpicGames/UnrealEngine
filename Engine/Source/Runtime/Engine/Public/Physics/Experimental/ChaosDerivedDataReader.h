// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "PhysicsEngine/BodySetup.h"
namespace Chaos
{
	class FImplicitObject;

	class FTriangleMeshImplicitObject;

	class FConvex;
}

template<typename T, int d>
class FChaosDerivedDataReader
{
public:

	// Only valid use is to explicitly read chaos bulk data
	explicit FChaosDerivedDataReader(FBulkData* InBulkData);

	UE_DEPRECATED(5.4, "Please use ConvexGeometries instead")
	TArray<TSharedPtr<Chaos::FConvex, ESPMode::ThreadSafe>> ConvexImplicitObjects;
	
	UE_DEPRECATED(5.4, "Please use TrimeshGeometries instead")
	TArray<TSharedPtr<Chaos::FTriangleMeshImplicitObject, ESPMode::ThreadSafe>> TrimeshImplicitObjects;

	TArray<Chaos::FConvexPtr> ConvexGeometries;
	TArray<Chaos::FTriangleMeshImplicitObjectPtr> TriMeshGeometries;
	FBodySetupUVInfo UVInfo;
	TArray<int32> FaceRemap;

private:
	FChaosDerivedDataReader() = delete;
	FChaosDerivedDataReader(const FChaosDerivedDataReader& Other) = delete;
	FChaosDerivedDataReader(FChaosDerivedDataReader&& Other) = delete;
	FChaosDerivedDataReader& operator =(const FChaosDerivedDataReader& Other) = delete;
	FChaosDerivedDataReader& operator =(FChaosDerivedDataReader&& Other) = delete;

	bool bReadSuccessful;

};

extern template class FChaosDerivedDataReader<float, 3>;
