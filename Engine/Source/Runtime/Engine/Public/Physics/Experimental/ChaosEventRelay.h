// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/Experimental/ChaosEventType.h"
#include "PhysicsPublic.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"

#include "ChaosEventRelay.generated.h"



DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FCollisionEventSignature, const TArray<FCollisionChaosEvent>&, CollisionEvents);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FBreakEventSignature, const TArray<FChaosBreakEvent>&, BreakEvents);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FRemovalEventSignature, const TArray<FChaosRemovalEvent>&, RemovalEvents);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FCrumblingEventSignature, const TArray<FChaosCrumblingEvent>&, CrumblingEvents);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrailingEventSignature, const TArray<FChaosTrailingEvent>&, TrailingEvents);


/**
* An object managing events
*/
UCLASS(MinimalAPI)
class UChaosEventRelay : public UObject
{
	GENERATED_BODY()

public:

	ENGINE_API UChaosEventRelay();

	ENGINE_API void DispatchPhysicsCollisionEvents(const TArray<FCollisionChaosEvent>& CollisionEvents);
	ENGINE_API void DispatchPhysicsBreakEvents(const TArray<FChaosBreakEvent>& BreakEvents);
	ENGINE_API void DispatchPhysicsRemovalEvents(const TArray<FChaosRemovalEvent>& RemovalEvents);
	ENGINE_API void DispatchPhysicsCrumblingEvents(const TArray<FChaosCrumblingEvent>& CrumblingEvents);
	ENGINE_API void DispatchPhysicsTrailingEvents(const TArray<FChaosTrailingEvent>& TrailingEvents);
	
	UPROPERTY(BlueprintAssignable, Category = "Physics Events")
	FCollisionEventSignature OnCollisionEvent;

	UPROPERTY(BlueprintAssignable, Category = "Physics Events")
	FBreakEventSignature OnBreakEvent;

	UPROPERTY(BlueprintAssignable, Category = "Physics Events")
	FRemovalEventSignature OnRemovalEvent;

	UPROPERTY(BlueprintAssignable, Category = "Physics Events")
	FCrumblingEventSignature OnCrumblingEvent;

	UPROPERTY(BlueprintAssignable, Category = "Physics Events")
	FTrailingEventSignature OnTrailingEvent;
};
