// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsPublic.h"
#include "Math/MathFwd.h"

#include "ChaosEventType.generated.h"


namespace Chaos
{
	struct FBreakingData;
	struct FCollidingData;
}

USTRUCT(BlueprintType)
struct FCollisionChaosEventBodyInfo
{
	GENERATED_BODY()

public:
	ENGINE_API FCollisionChaosEventBodyInfo();

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector Velocity;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector DeltaVelocity;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector AngularVelocity;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	float Mass;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	TObjectPtr<class UPhysicalMaterial> PhysMaterial;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	TWeakObjectPtr<UPrimitiveComponent> Component;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	int32 BodyIndex;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FName BoneName;
};



USTRUCT(BlueprintType)
struct FCollisionChaosEvent
{
	GENERATED_BODY()

public:
	ENGINE_API FCollisionChaosEvent();
	ENGINE_API FCollisionChaosEvent(const Chaos::FCollidingData& CollisionData);

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector Location;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector AccumulatedImpulse;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FVector Normal;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	float PenetrationDepth;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FCollisionChaosEventBodyInfo Body1;

	UPROPERTY(BlueprintReadOnly, Category = "Collision Event")
	FCollisionChaosEventBodyInfo Body2;

};


USTRUCT(BlueprintType)
struct FChaosBreakEvent
{
	GENERATED_BODY()

public:

	ENGINE_API FChaosBreakEvent();
	ENGINE_API FChaosBreakEvent(const Chaos::FBreakingData& BreakingData);

	/** primitive component involved in the break event */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	TObjectPtr<UPrimitiveComponent> Component = nullptr;

	/** World location of the breaking piece */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	FVector Location;

	/** World orientation of the breaking piece */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	FQuat Orientation;

	/** Linear Velocity of the breaking particle  */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	FVector Velocity;

	/** Angular Velocity of the breaking particle  */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	FVector AngularVelocity;

	/** Extents of the bounding box in local space of the breaking piece */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	FVector Extents;

	/** Mass of the breaking particle  */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	float Mass;

	/** Index of the geometry collection bone if positive */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	int32 Index;

	/** Whether the break event originated from a crumble event */
	UPROPERTY(BlueprintReadOnly, Category = "Break Event")
	bool bFromCrumble;
};

USTRUCT(BlueprintType)
struct FChaosRemovalEvent
{
	GENERATED_BODY()

public:

	ENGINE_API FChaosRemovalEvent();

	UPROPERTY(BlueprintReadOnly, Category = "Removal Event")
	TObjectPtr<UPrimitiveComponent> Component = nullptr;

	UPROPERTY(BlueprintReadOnly, Category = "Removal Event")
	FVector Location;

	UPROPERTY(BlueprintReadOnly, Category = "Removal Event")
	float Mass;
};


USTRUCT(BlueprintType)
struct FChaosCrumblingEvent
{
	GENERATED_BODY()

public:
	FChaosCrumblingEvent()
		: Component(nullptr)
		, Location(FVector::ZeroVector)
		, Orientation(FQuat::Identity)
		, LinearVelocity(FVector::ZeroVector)
		, AngularVelocity(FVector::ZeroVector)
		, Mass(0)
		, LocalBounds(ForceInitToZero)
	{}

	/** primitive component involved in the crumble event */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	TObjectPtr<UPrimitiveComponent> Component = nullptr;

	/** World location of the crumbling cluster */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	FVector Location;

	/** World orientation of the crumbling cluster */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	FQuat Orientation;

	/** Linear Velocity of the crumbling cluster */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	FVector LinearVelocity;

	/** Angular Velocity of the crumbling cluster  */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	FVector AngularVelocity;

	/** Mass of the crumbling cluster  */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	float Mass;

	/** Local bounding box of the crumbling cluster  */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	FBox LocalBounds;

	/** List of children indices released (optional : see geometry collection component bCrumblingEventIncludesChildren) */
	UPROPERTY(BlueprintReadOnly, Category = "Crumble Event")
	TArray<int32> Children;
};

USTRUCT(BlueprintType)
struct FChaosTrailingEvent
{
	GENERATED_BODY()

public:

	FChaosTrailingEvent()
		: Component(nullptr)
		, Location(FVector::ZeroVector)
		, Orientation(FQuat::Identity)
		, LinearVelocity(FVector::ZeroVector)
		, AngularVelocity(FVector::ZeroVector)
		, Mass(0)
		, LocalBounds(ForceInitToZero)
		, Index(INDEX_NONE)
	{}

	/** Primitive component involved in the trailing event */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	TObjectPtr<UPrimitiveComponent> Component = nullptr;

	/** World location of the trailing piece  */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	FVector Location;

	/** World orientation of the trailing piece */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	FQuat Orientation;

	/** Linear Velocity of the trailing piece  */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	FVector LinearVelocity;

	/** Angular Velocity of the trailing piece */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	FVector AngularVelocity;

	/** Mass of the trailing piece in Kilograms */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	float Mass;

	/** Local bounding box of the trailing piece   */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	FBox LocalBounds;

	/** Index of the geometry collection bone if positive */
	UPROPERTY(BlueprintReadOnly, Category = "Trailing Event")
	int32 Index;
};


