// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Declares.h"
#include "Containers/ContainersFwd.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Templates/UniquePtr.h"
#include "BodySetupEnums.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Chaos/GeometryParticles.h"

struct FGeometryAddParams;

namespace ChaosInterface
{
	Chaos::EChaosCollisionTraceFlag ConvertCollisionTraceFlag(ECollisionTraceFlag Flag);
	
	/**
	 * Create the Chaos Geometry based on the geometry parameters.
	 */
	void ENGINE_API CreateGeometry(const FGeometryAddParams& InParams, TArray<Chaos::FImplicitObjectPtr>& OutGeoms, Chaos::FShapesArray& OutShapes);

	UE_DEPRECATED(5.4, "Use CreateGeometry with FImplicitObjectPtr instead")
	void ENGINE_API CreateGeometry(const FGeometryAddParams& InParams, TArray<TUniquePtr<Chaos::FImplicitObject>>& OutGeoms, Chaos::FShapesArray& OutShapes);


	/**
	 * Generate the mass properties for a set of shapes in the space of the shapes' owner. 
	 * Rotation will be built into the inertia matrix (it may not be diagonal) and RotationOfMass will be identity.
	*/
	void ENGINE_API CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties, const TArray<FPhysicsShapeHandle>& InShapes, float InDensityKGPerCM);
	void ENGINE_API CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties, const Chaos::FShapesArray& InShapes, const TArray<bool>& bContributesToMass, float InDensityKGPerCM);
}
