// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosInterfaceWrapperCore.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/SpatialAccelerationFwd.h"
#include "PhysicsInterfaceUtilsCore.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "SceneQueryCommonParams.h"

class FPhysScene_Chaos;
class UPhysicalMaterial;
struct FBodyInstance;
struct FCollisionFilterData;
struct FCollisionQueryParams;
struct FExternalSpatialAccelerationPayload;
using IExternalSpatialAcceleration = Chaos::ISpatialAcceleration<FExternalSpatialAccelerationPayload, Chaos::FReal, 3>;

namespace Chaos
{
	class FPerShapeData;
	struct FPhysicsObject;
}

namespace ChaosInterface
{
	struct FPTOverlapHit;

// Needed by low level SQ calls.
struct FScopedSceneReadLock
{
	FScopedSceneReadLock(FPhysScene_Chaos& SceneIn);
	~FScopedSceneReadLock();

	Chaos::FPBDRigidsSolver* Solver;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.8, "Use ChaosInterface::FSceneQueryCommonParams constructor instead.")
inline FQueryFilterData MakeQueryFilterData(const FCollisionFilterData& FilterData, EQueryFlags QueryFlags, const FCollisionQueryParams& Params)
{
	return FChaosQueryFilterData(U2CFilterData(FilterData), U2CQueryFlags(QueryFlags));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FBodyInstance* GetUserData(const Chaos::FGeometryParticle& Actor);
UPhysicalMaterial* GetUserData(const Chaos::FChaosPhysicsMaterial& Material);
UPrimitiveComponent* GetPrimitiveComponentFromUserData(const Chaos::FGeometryParticle& Actor);
Chaos::FPhysicsObject* GetPhysicsObjectFromUserData(const Chaos::FGeometryParticle& Actor);
bool HasValidUserData(const Chaos::FGeometryParticle& Actor);

}

namespace Chaos::Private
{
	template<typename TContainer, typename THit>
	UE_INTERNAL void LowLevelRaycast(const TContainer& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams);

	template<typename TContainer, typename THit>
	UE_INTERNAL void LowLevelSweep(const TContainer& Container, const FPhysicsGeometry& Geom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THit>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams);

	template<typename TContainer, typename THit>
	UE_INTERNAL void LowLevelOverlap(const TContainer& Container, const FPhysicsGeometry& Geom, const FTransform& GeomPose, FPhysicsHitCallback<THit>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams);

	// Export raycast functions
#define DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void LowLevelRaycast(const TACCEL& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams);
#define DECLARE_LOW_LEVEL_RAYCAST_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, FHitRaycast); \
DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, ChaosInterface::FPTRaycastHit);
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(FPhysScene)
#undef DECLARE_LOW_LEVEL_RAYCAST_ACCEL
#undef DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT

// Export sweep functions
#define DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void LowLevelSweep(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, const ChaosInterface::FSceneQueryCommonParams& CommonParams);
#define DECLARE_LOW_LEVEL_SWEEP_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, FHitSweep); \
DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, ChaosInterface::FPTSweepHit);
DECLARE_LOW_LEVEL_SWEEP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_SWEEP_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_SWEEP_ACCEL(FPhysScene)
#undef DECLARE_LOW_LEVEL_SWEEP_ACCEL
#undef DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT

// Export overlap functions
#define DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void LowLevelOverlap(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THIT>& HitBuffer, const ChaosInterface::FSceneQueryCommonParams& CommonParams)
#define DECLARE_LOW_LEVEL_OVERLAP_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, FHitOverlap); \
DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, ChaosInterface::FPTOverlapHit);
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(FPhysScene)
#undef DECLARE_LOW_LEVEL_OVERLAP_ACCEL
#undef DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
namespace Chaos::Private
{
	template<typename TContainer, typename THit>
	UE_DEPRECATED(5.8, "This is meant to be internal only")
	void LowLevelRaycast(const TContainer& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THit>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {});

	template<typename TContainer, typename THit>
	UE_DEPRECATED(5.8, "This is meant to be internal only")
	void LowLevelSweep(const TContainer& Container, const FPhysicsGeometry& Geom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THit>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {});

	template<typename TContainer, typename THit>
	UE_DEPRECATED(5.8, "This is meant to be internal only")
	void LowLevelOverlap(const TContainer& Container, const FPhysicsGeometry& Geom, const FTransform& GeomPose, FPhysicsHitCallback<THit>& HitBuffer, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams = {});
}

// Export raycast functions
#define DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void Chaos::Private::LowLevelRaycast(const TACCEL& Container, const FVector& Start, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams);
#define DECLARE_LOW_LEVEL_RAYCAST_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, FHitRaycast); \
DECLARE_LOW_LEVEL_RAYCAST_ACCEL_HIT(TACCEL, ChaosInterface::FPTRaycastHit);
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_RAYCAST_ACCEL(FPhysScene)

// Export sweep functions
#define DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void Chaos::Private::LowLevelSweep(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& StartTM, const FVector& Dir, float DeltaMag, FPhysicsHitCallback<THIT>& HitBuffer, EHitFlags OutputFlags, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams);
#define DECLARE_LOW_LEVEL_SWEEP_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, FHitSweep); \
DECLARE_LOW_LEVEL_SWEEP_ACCEL_HIT(TACCEL, ChaosInterface::FPTSweepHit);
DECLARE_LOW_LEVEL_SWEEP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_SWEEP_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_SWEEP_ACCEL(FPhysScene)

// Export overlap functions
#define DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, THIT) template<> UE_INTERNAL ENGINE_API void Chaos::Private::LowLevelOverlap(const TACCEL& Container, const FPhysicsGeometry& QueryGeom, const FTransform& GeomPose, FPhysicsHitCallback<THIT>& HitBuffer, FQueryFlags QueryFlags, const FCollisionFilterData& Filter, const ChaosInterface::FQueryFilterData& QueryFilterData, ICollisionQueryFilterCallbackBase* QueryCallback, const ChaosInterface::FQueryDebugParams& DebugParams)
#define DECLARE_LOW_LEVEL_OVERLAP_ACCEL(TACCEL) \
DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, FHitOverlap); \
DECLARE_LOW_LEVEL_OVERLAP_ACCEL_HIT(TACCEL, ChaosInterface::FPTOverlapHit);
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(Chaos::IDefaultChaosSpatialAcceleration)
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(IExternalSpatialAcceleration)
DECLARE_LOW_LEVEL_OVERLAP_ACCEL(FPhysScene)

PRAGMA_ENABLE_DEPRECATION_WARNINGS
