// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsInterfaceDeclaresCore.h"
#include "Chaos/PhysicsObject.h"
#include "Framework/Threading.h"

enum class EPhysicsInterfaceScopedLockType : uint8
{
	Read,
	Write
};

enum class EPhysicsInterfaceScopedThreadContext : uint8
{
	External,
	Internal
};

enum class EPhysicsInterfaceScopedTransactionMode : uint8
{
	Normal,
	MultiServer
};

class USkeletalMeshComponent;
class FChaosScene;

namespace Chaos
{
	class FPBDRigidsSolver;
}

struct FScopedSceneLock_Chaos
{
	ENGINE_API FScopedSceneLock_Chaos(const FPhysicsActorHandle& InActorHandle, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(const FPhysicsActorHandle& InActorHandleA, const FPhysicsActorHandle& InActorHandleB, EPhysicsInterfaceScopedLockType InLockType);
	// TODO_CHAOSAPI: Deprecate pointer-to-handle API
	ENGINE_API FScopedSceneLock_Chaos(FPhysicsActorHandle const* InActorHandle, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(FPhysicsActorHandle const* InActorHandleA, FPhysicsActorHandle const* InActorHandleB, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(FPhysicsConstraintHandle const* InConstraintHandle, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(USkeletalMeshComponent* InSkelMeshComp, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(Chaos::FPhysicsObjectHandle InObjectA, Chaos::FPhysicsObjectHandle InObjectB, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API FScopedSceneLock_Chaos(FChaosScene* InScene, EPhysicsInterfaceScopedLockType InLockType);
	ENGINE_API ~FScopedSceneLock_Chaos();

	FScopedSceneLock_Chaos(FScopedSceneLock_Chaos& Other) = delete;
	FScopedSceneLock_Chaos& operator=(FScopedSceneLock_Chaos& Other) = delete;

	ENGINE_API FScopedSceneLock_Chaos(FScopedSceneLock_Chaos&& Other);
	ENGINE_API FScopedSceneLock_Chaos& operator=(FScopedSceneLock_Chaos&& Other);

	ENGINE_API void Release();

protected:

	FScopedSceneLock_Chaos(EPhysicsInterfaceScopedLockType InLockType) : Solver(nullptr), LockType(InLockType)
	{
	}
	
	void LockSceneForConstraint(FPhysicsConstraintHandle const* InConstraintHandle);

	ENGINE_API void LockScene();
	ENGINE_API void UnlockScene();

	ENGINE_API FChaosScene* GetSceneForActor(const FPhysicsActorHandle& InActorHandle);
	ENGINE_API FChaosScene* GetSceneForActor(FPhysicsConstraintHandle const* InConstraintHandle);

	Chaos::FPhysSceneLock* GetSolverLock(Chaos::FPBDRigidsSolver* InSolver);

	bool bHasLock = false;
	Chaos::FPBDRigidsSolver* Solver;
	EPhysicsInterfaceScopedLockType LockType;

	EPhysicsInterfaceScopedThreadContext ThreadContext = EPhysicsInterfaceScopedThreadContext::External;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** By default, any physics write operation needs to be done as part of a multi-server commit transaction */
	EPhysicsInterfaceScopedTransactionMode TransactionMode = EPhysicsInterfaceScopedTransactionMode::MultiServer;
#endif
};

struct UE_INTERNAL FScopedSceneLockWithContext_Chaos : FScopedSceneLock_Chaos
{
	ENGINE_API FScopedSceneLockWithContext_Chaos(const FPhysicsActorHandle& InActorHandle, EPhysicsInterfaceScopedLockType InLockType, EPhysicsInterfaceScopedThreadContext InThreadContext, EPhysicsInterfaceScopedTransactionMode InTransactionMode);
	ENGINE_API FScopedSceneLockWithContext_Chaos(FPhysicsConstraintHandle const* InConstraintHandle, EPhysicsInterfaceScopedLockType InLockType, EPhysicsInterfaceScopedThreadContext InThreadContext, EPhysicsInterfaceScopedTransactionMode InTransactionMode);
};

#if UE_WITH_REMOTE_OBJECT_HANDLE
namespace Chaos::Private
{
	struct UE_INTERNAL FDstmScopedWriteLockInvalidationIgnore
	{
		ENGINE_API FDstmScopedWriteLockInvalidationIgnore(bool bInvalidate);
		ENGINE_API ~FDstmScopedWriteLockInvalidationIgnore();

		ENGINE_API static bool WriteLocksInvalidationAreIgnored();

	private:
		bool bOldState;
	};
} // namespace Chaos::Private
#endif //  UE_WITH_REMOTE_OBJECT_HANDLE
