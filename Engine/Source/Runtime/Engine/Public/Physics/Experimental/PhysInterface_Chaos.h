// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosInterfaceWrapper.h"
#include "Chaos/Declares.h"
#include "Chaos/PhysicsObject.h"
#include "PhysicsEngine/ConstraintTypes.h"
#include "PhysicsInterfaceWrapperShared.h"
#include "Physics/PhysicsInterfaceDeclares.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "Physics/GenericPhysicsInterface.h"
#include "Physics/Experimental/PhysicsUserData_Chaos.h"
#include "Chaos/ChaosEngineInterface.h"

//NOTE: Do not include Chaos headers directly as it means recompiling all of engine. This should be reworked to avoid allocations

enum class EPhysicsInterfaceScopedTransactionMode : uint8;
enum class EPhysicsInterfaceScopedThreadContext : uint8;
static int32 NextBodyIdValue = 0;
static int32 NextConstraintIdValue = 0;

class FPhysInterface_Chaos;
struct FBodyInstance;
struct FPhysxUserData;
struct FConstraintDrive;
struct FLinearDriveConstraint;
struct FAngularDriveConstraint;


class AWorldSettings;
class UPhysicsConstraintComponent;

class FPhysInterface_Chaos : public FGenericPhysicsInterface, public FChaosEngineInterface
{
public:
    ENGINE_API FPhysInterface_Chaos(const AWorldSettings* Settings=nullptr);
    ENGINE_API ~FPhysInterface_Chaos();

	// Describe the interface to identify it to the caller
	static FString GetInterfaceDescription() { return TEXT("Chaos"); }

	static ENGINE_API const FBodyInstance* ShapeToOriginalBodyInstance(const FBodyInstance* InCurrentInstance, const Chaos::FPerShapeData* InShape);

	// Material mask functions 
	static ENGINE_API FPhysicsMaterialMaskHandle CreateMaterialMask(const UPhysicalMaterialMask* InMaterialMask);
	static ENGINE_API void UpdateMaterialMask(FPhysicsMaterialMaskHandle& InHandle, const UPhysicalMaterialMask* InMaterialMask);

	// Actor interface functions

	static ENGINE_API FPhysScene* GetCurrentSceneForComponent(USceneComponent* InComponent);

	static ENGINE_API void FlushScene(FPhysScene* InScene);
	static ENGINE_API bool IsInScene(const FPhysicsActorHandle& InActorReference);

	static ENGINE_API void SetLinearMotionLimitType_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, PhysicsInterfaceTypes::ELimitAxis InAxis, ELinearConstraintMotion InMotion);
	static ENGINE_API void SetAngularMotionLimitType_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, PhysicsInterfaceTypes::ELimitAxis InAxis, EAngularConstraintMotion InMotion);

	static ENGINE_API void UpdateLinearLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InLimit, float InAverageMass, const FLinearConstraint& InParams);
	static ENGINE_API void UpdateConeLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InAverageMass, const FConeConstraint& InParams);
	static ENGINE_API void UpdateTwistLimitParams_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, float InAverageMass, const FTwistConstraint& InParams);
	static ENGINE_API void UpdateLinearDrive_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FLinearDriveConstraint& InDriveParams, bool InInitialize = false);
	static ENGINE_API void UpdateAngularDrive_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FAngularDriveConstraint& InDriveParams, bool InInitialize = false);
	static ENGINE_API void UpdateDriveTarget_AssumesLocked(const FPhysicsConstraintHandle& InConstraintRef, const FLinearDriveConstraint& InLinDrive, const FAngularDriveConstraint& InAngDrive, bool InInitialize = false);
	

	static ENGINE_API bool ExecuteOnUnbrokenConstraintReadOnly(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle&)> Func);
	static ENGINE_API bool ExecuteOnUnbrokenConstraintReadWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle&)> Func);

    /////////////////////////////////////////////

    // Interface needed for cmd
    static ENGINE_API bool ExecuteRead(const FPhysicsActorHandle& InActorReference, TFunctionRef<void(const FPhysicsActorHandle& Actor)> InCallable);
    static ENGINE_API bool ExecuteRead(USkeletalMeshComponent* InMeshComponent, TFunctionRef<void()> InCallable);
    static ENGINE_API bool ExecuteRead(const FPhysicsActorHandle& InActorReferenceA, const FPhysicsActorHandle& InActorReferenceB, TFunctionRef<void(const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)> InCallable);
    static ENGINE_API bool ExecuteRead(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable);
    static ENGINE_API bool ExecuteRead(FPhysScene* InScene, TFunctionRef<void()> InCallable);
	static ENGINE_API bool ExecuteRead(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB, TFunctionRef<void(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB)> InCallable);

    static ENGINE_API bool ExecuteWrite(const FPhysicsActorHandle& InActorReference, TFunctionRef<void(const FPhysicsActorHandle& Actor)> InCallable);
	// TODO_CHAOSAPI: Deprecate non-const ref API
	static ENGINE_API bool ExecuteWrite(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable);
    static ENGINE_API bool ExecuteWrite(USkeletalMeshComponent* InMeshComponent, TFunctionRef<void()> InCallable);
    static ENGINE_API bool ExecuteWrite(const FPhysicsActorHandle& InActorReferenceA, const FPhysicsActorHandle& InActorReferenceB, TFunctionRef<void(const FPhysicsActorHandle& ActorA, const FPhysicsActorHandle& ActorB)> InCallable);
    static ENGINE_API bool ExecuteWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable);
    static ENGINE_API bool ExecuteWrite(FPhysScene* InScene, TFunctionRef<void()> InCallable);
	static ENGINE_API bool ExecuteWrite(FPhysScene* InScene, TFunctionRef<void(FPhysScene* Scene)> InCallable);
	static ENGINE_API bool ExecuteWrite(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB, TFunctionRef<void(Chaos::FPhysicsObject* InObjectA, Chaos::FPhysicsObject* InObjectB)> InCallable);

    static ENGINE_API void ExecuteShapeWrite(FBodyInstance* InInstance, FPhysicsShapeHandle& InShape, TFunctionRef<void(FPhysicsShapeHandle& InShape)> InCallable);

	UE_INTERNAL static ENGINE_API bool ExecuteWrite(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext, EPhysicsInterfaceScopedTransactionMode TransactionMode);
	UE_INTERNAL static ENGINE_API bool ExecuteRead(FPhysicsActorHandle& InActorReference, TFunctionRef<void(FPhysicsActorHandle& Actor)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext);
	UE_INTERNAL static ENGINE_API bool ExecuteWrite(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext, EPhysicsInterfaceScopedTransactionMode TransactionMode);
	UE_INTERNAL static ENGINE_API bool ExecuteRead(const FPhysicsConstraintHandle& InConstraintRef, TFunctionRef<void(const FPhysicsConstraintHandle& Constraint)> InCallable, EPhysicsInterfaceScopedThreadContext ThreadContext);

	// Misc

	static ENGINE_API bool ExecPhysCommands(const TCHAR* Cmd, FOutputDevice* Ar, UWorld* InWorld);

	static ENGINE_API void CalculateMassPropertiesFromShapeCollection(Chaos::FMassProperties& OutProperties, const TArray<FPhysicsShapeHandle>& InShapes, float InDensityKGPerCM);

	// Shape interface functions
	static ENGINE_API FPhysicsShapeHandle CreateShape(physx::PxGeometry* InGeom, bool bSimulation = true, bool bQuery = true, UPhysicalMaterial* InSimpleMaterial = nullptr, TArray<UPhysicalMaterial*>* InComplexMaterials = nullptr);
	
	static ENGINE_API void AddGeometry(const FPhysicsActorHandle& InActor, const FGeometryAddParams& InParams, TArray<FPhysicsShapeHandle>* OutOptShapes = nullptr);
	static ENGINE_API void CreateGeometry(const FGeometryAddParams& InParams, TArray<Chaos::FImplicitObjectPtr>& OutGeoms, Chaos::FShapesArray& OutShapes, TArray<FPhysicsShapeHandle>* OutOptShapes);
	
	UE_DEPRECATED(5.4, "Use CreateGeometry with FImplicitObjectPtr instead")
	static void CreateGeometry(const FGeometryAddParams& InParams, TArray<TUniquePtr<Chaos::FImplicitObject>>& OutGeoms, Chaos::FShapesArray& OutShapes, TArray<FPhysicsShapeHandle>* OutOptShapes){}
	
	// Trace functions for testing specific geometry (not against a world)
	static ENGINE_API bool LineTrace_Geom(FHitResult& OutHit, const FBodyInstance* InInstance, const FVector& InStart, const FVector& InEnd, bool bTraceComplex, bool bExtractPhysMaterial = false);
	static ENGINE_API bool Sweep_Geom(FHitResult& OutHit, const FBodyInstance* InInstance, const FVector& InStart, const FVector& InEnd, const FQuat& InShapeRotation, const FCollisionShape& InShape, bool bSweepComplex);
	static ENGINE_API bool Overlap_Geom(const FBodyInstance* InBodyInstance, const FPhysicsGeometryCollection& InGeometry, const FTransform& InShapeTransform, FMTDResult* OutOptResult = nullptr, bool bTraceComplex = false);
	static ENGINE_API bool Overlap_Geom(const FBodyInstance* InBodyInstance, const FCollisionShape& InCollisionShape, const FQuat& InShapeRotation, const FTransform& InShapeTransform, FMTDResult* OutOptResult = nullptr, bool bTraceComplex = false);
	static ENGINE_API bool GetSquaredDistanceToBody(const FBodyInstance* InInstance, const FVector& InPoint, float& OutDistanceSquared, FVector* OutOptPointOnBody = nullptr);

	static ENGINE_API void SetMaterials(const FPhysicsShapeHandle& InShape, const TArrayView<UPhysicalMaterial*>InMaterials);
	static ENGINE_API void SetMaterials(const FPhysicsShapeHandle& InShape, const TArrayView<UPhysicalMaterial*> InMaterials, const TArrayView<FPhysicalMaterialMaskParams>& InMaterialMasks);
};

/*
FORCEINLINE ECollisionShapeType GetType(const Chaos::FImplicitObject& Geom)
{
	if (Geom.GetType() == Chaos::ImplicitObjectType::Box)
	{
		return ECollisionShapeType::Box;
	}
	if (Geom.GetType() == Chaos::ImplicitObjectType::Sphere)
	{
		return ECollisionShapeType::Sphere;
	}
	if (Geom.GetType() == Chaos::ImplicitObjectType::Plane)
	{
		return ECollisionShapeType::Plane;
	}
	return ECollisionShapeType::None;
}
*/
ECollisionShapeType GetGeometryType(const Chaos::FPerShapeData& Shape);

/*
FORCEINLINE float GetRadius(const Chaos::FCapsule& Capsule)
{
	return Capsule.GetRadius();
}

FORCEINLINE float GetHalfHeight(const Chaos::FCapsule& Capsule)
{
	return Capsule.GetHeight() / 2.f;
}
*/

inline void DrawOverlappingTris(const UWorld* World, const ChaosInterface::FLocationHit& Hit, const Chaos::FImplicitObject& Geom, const FTransform& QueryTM)
{
	//TODO_SQ_IMPLEMENTATION
}

inline void ComputeZeroDistanceImpactNormalAndPenetration(const UWorld* World, const ChaosInterface::FLocationHit& Hit, const Chaos::FImplicitObject& Geom, const FTransform& QueryTM, FHitResult& OutResult)
{
	//TODO_SQ_IMPLEMENTATION
}

inline void DrawOverlappingTris(const UWorld* World, const ChaosInterface::FPTLocationHit& Hit, const Chaos::FImplicitObject& Geom, const FTransform& QueryTM)
{
	//TODO_SQ_IMPLEMENTATION
}

inline void ComputeZeroDistanceImpactNormalAndPenetration(const UWorld* World, const ChaosInterface::FPTLocationHit& Hit, const Chaos::FImplicitObject& Geom, const FTransform& QueryTM, FHitResult& OutResult)
{
	//TODO_SQ_IMPLEMENTATION
}

Chaos::FChaosPhysicsMaterial* GetMaterialFromInternalFaceIndex(const FPhysicsShape& Shape, const FPhysicsActor& Actor, uint32 InternalFaceIndex);

Chaos::FChaosPhysicsMaterial* GetMaterialFromInternalFaceIndexAndHitLocation(const FPhysicsShape& Shape, const FPhysicsActor& Actor, uint32 InternalFaceIndex, const FVector& HitLocation);

uint32 GetTriangleMeshExternalFaceIndex(const FPhysicsShape& Shape, uint32 InternalFaceIndex);

inline void GetShapes(const FPhysActorDummy& RigidActor, FPhysTypeDummy** ShapesBuffer, uint32 NumShapes)
{
	
}

inline void SetShape(FPhysTypeDummy& Hit, FPhysTypeDummy* Shape)
{

}
