// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "Async/Fundamental/Task.h"
#include "UObject/Object.h"
#include "UObject/GCObject.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Physics/Experimental/AsyncPhysicsStateProcessorInterface.h"

struct FPhysScene_AsyncPhysicsStateJobQueue : public FGCObject
{
	FPhysScene_AsyncPhysicsStateJobQueue(FPhysScene* InPhysScene);
	~FPhysScene_AsyncPhysicsStateJobQueue();

	enum class EJobType
	{
		CreatePhysicsState,
		DestroyPhysicsState
	};

	struct FJob
	{
		FJob(IAsyncPhysicsStateProcessor* InProcessor, EJobType InType)
			: Processor(InProcessor)
			, Type(InType)
		{}

		bool IsValid() const
		{
			// No need to test for Processor->GetAsyncPhysicsStateObject() as it's rooted by OnPreExecute_GameThread
			return (Processor != nullptr);
		}

		bool Execute(UE::FTimeout& Timeout) const;
		void OnPreExecute_GameThread() const;
		void OnPostExecute_GameThread() const;

		inline bool operator == (const FJob& Other) const
		{
			return (Processor == Other.Processor) && (Type == Other.Type);
		}

		friend inline uint32 GetTypeHash(const FJob& InJob)
		{
			return GetTypeHash(TPair<IAsyncPhysicsStateProcessor*, EJobType>{InJob.Processor, InJob.Type});
		}

		mutable TSet<UObject*> RootedInputs;
		IAsyncPhysicsStateProcessor* Processor;
		EJobType Type;
	private:
		mutable FPhysScene_AsyncPhysicsStateJobQueue* Owner = nullptr;
		friend struct FPhysScene_AsyncPhysicsStateJobQueue;
	};

	struct FScopedDeferAsyncPhysicsStateJobs
	{
		explicit FScopedDeferAsyncPhysicsStateJobs(FPhysScene* InPhysScene);
		~FScopedDeferAsyncPhysicsStateJobs();
		
		FScopedDeferAsyncPhysicsStateJobs(const FScopedDeferAsyncPhysicsStateJobs&) = delete;
		FScopedDeferAsyncPhysicsStateJobs& operator=(const FScopedDeferAsyncPhysicsStateJobs&) = delete;

	private:
		FPhysScene* PhysScene = nullptr;
	};

	void Tick(bool bWaitForCompletion = false);
	void AddJob(const FJob& Job);
	void RemoveJob(const FJob& Job);
	bool IsCompleted() const;

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FPhysScene_AsyncPhysicsStateJobQueue"); }
	//~ End FGCObject interface

private:

	void BeginDeferringNewJobs();
	void EndDeferringNewJobs();

	void LaunchAsyncJobTask();
	void ExecuteJobsAsync(double TimeBudgetSeconds);
	void OnUpdateLevelStreamingDone();

	void AddRootObject(UObject* Obj);
	void RemoveRootObject(UObject* Obj);

	// GameThread variables
	FPhysScene* PhysScene = nullptr; // The Physics scene
	UE::Tasks::FTask AsyncJobTask; // The async task

	// Async Task variables
	int32 TaskEpoch = 0; // Last epoch used by the async task
	double UsedAsyncTaskTimeBudgetSec = 0; // Time used in the async task since the last epoch update

	// Variables access/modified by both the GameThread and the async task
	mutable FRWLock JobsLock; // Lock used to protect jobs to execute, executing and completed
	TArray<FJob> JobsToExecute;
	TOptional<FJob> ExecutingJob;
	TArray<FJob> CompletedJobs;

	// When greater than zero, new jobs are added to DeferredJobs until counter goes back to 0 
	// This allows to temporarily pause adding to JobsToExecute.
	int32 DeferNewJobsScopeCount = 0;
	TArray<FJob> DeferredJobs;

	std::atomic<bool> bIsBlocking = false; // Notifies to the async task that we are block waiting for the task to complete
	std::atomic<int32> GameThreadEpoch = 0; // Epoch updated by the GameThread at every frame and used by the async task to reset UsedAsyncTaskTimeBudgetSec

	// GT-only: tracks roots across jobs (counts to handle duplicate/additional jobs)
	TMap<TObjectPtr<UObject>, int32> RootCounts;

	friend struct FScopedDeferAsyncPhysicsStateJobs;
};