// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tickable.h"
#include "EventsData.h"
#include "Physics/PhysScene.h"
#include "Physics/Experimental/ChaosEventType.h"
#include "GameFramework/Actor.h"
#include "PhysicsPublic.h"
#include "PhysInterface_Chaos.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "Chaos/ChaosScene.h"
#include "Chaos/ContactModification.h"
#include "Chaos/Real.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "Chaos/ChaosDebugNameDefines.h"
#include "UObject/ObjectKey.h"

#ifndef CHAOS_WITH_PAUSABLE_SOLVER
#define CHAOS_WITH_PAUSABLE_SOLVER 1
#endif

// Currently compilation issue with Incredibuild when including headers required by event template functions
#define XGE_FIXED 0

class UPrimitiveComponent;

class IAsyncPhysicsStateProcessor;
class AdvanceOneTimeStepTask;
class IPhysicsReplication;
class IPhysicsReplicationLOD;
class FPhysicsReplicationCache;
class FPhysicsReplicationQuantization;
class FPhysInterface_Chaos;
class FChaosSolversModule;
struct FForceFieldProxy;
struct FSolverStateStorage;

class FSkeletalMeshPhysicsProxy;
class FStaticMeshPhysicsProxy;
class FPerSolverFieldSystem;

class IPhysicsProxyBase;

class UWorld;
class UChaosEventRelay;
class AWorldSettings;
class FPhysicsReplicationFactory;
class FContactModifyCallbackFactory;
struct FConstraintInstanceBase;
struct FPhysScene_AsyncPhysicsStateJobQueue;

namespace Chaos
{
	class FPhysicsProxy;
	class FClusterUnionPhysicsProxy;

	struct FCollisionEventData;

	enum class EEventType : int32;

	template<typename PayloadType, typename HandlerType>
	class TRawEventHandler;

	class FAccelerationStructureHandle;

	template <typename TPayload, typename T, int d>
	class ISpatialAcceleration;

	template <typename TPayload, typename T, int d>
	class ISpatialAccelerationCollection;

	template <typename T>
	class TArrayCollectionArray;

	struct FSerializedDataBuffer;
}

extern int32 GEnableKinematicDeferralStartPhysicsCondition;

struct FConstraintBrokenDelegateWrapper
{
	FConstraintBrokenDelegateWrapper(FConstraintInstanceBase* ConstraintInstance);

	void DispatchOnBroken();

	FOnConstraintBroken OnConstraintBrokenDelegate;
	int32 ConstraintIndex;
};

struct FConstraintViolatedDelegateWrapper
{
	FConstraintViolatedDelegateWrapper(FConstraintInstanceBase* ConstraintInstance);

	void DispatchOnViolated(float LinearViolation, float AngularViolation);

	FOnConstraintViolated OnConstraintViolatedDelegate;
	int32 ConstraintIndex;
};

struct FPlasticDeformationDelegateWrapper
{
	FPlasticDeformationDelegateWrapper(FConstraintInstanceBase* ConstraintInstance);

	void DispatchPlasticDeformation();

	FOnPlasticDeformation OnPlasticDeformationDelegate;
	int32 ConstraintIndex;
};

/**
* Low level Chaos scene used when building custom simulations that don't exist in the main world physics scene.
*/
class FPhysScene_Chaos : public FChaosScene
{
public:

	using Super = FChaosScene;
	
	ENGINE_API FPhysScene_Chaos(AActor* InSolverActor=nullptr
#if CHAOS_SOLVER_DEBUG_NAME
	, const FName& DebugName=NAME_None
#endif
);

	ENGINE_API virtual ~FPhysScene_Chaos();

	/** Returns the actor that owns this solver. */
	ENGINE_API AActor* GetSolverActor() const;

	ENGINE_API void RegisterForCollisionEvents(UPrimitiveComponent* Component);
	ENGINE_API void UnRegisterForCollisionEvents(UPrimitiveComponent* Component);

	ENGINE_API void RegisterForGlobalCollisionEvents(UPrimitiveComponent* Component);
	ENGINE_API void UnRegisterForGlobalCollisionEvents(UPrimitiveComponent* Component);

	ENGINE_API void RegisterForGlobalRemovalEvents(UPrimitiveComponent* Component);
	ENGINE_API void UnRegisterForGlobalRemovalEvents(UPrimitiveComponent* Component);

	ENGINE_API void RegisterAsyncPhysicsTickComponent(UActorComponent* Component);
	ENGINE_API void UnregisterAsyncPhysicsTickComponent(UActorComponent* Component);

	ENGINE_API void RegisterAsyncPhysicsTickActor(AActor* Actor);
	ENGINE_API void UnregisterAsyncPhysicsTickActor(AActor* Actor);

	ENGINE_API void EnqueueAsyncPhysicsCommand(int32 PhysicsStep, UObject* OwningObject, const TFunction<void()>& Command, const bool bEnableResim = false);


	/**
	 * Called during creation of the physics state for gamethread objects to pass off an object to the physics thread
	 */
	ENGINE_API void AddObject(UPrimitiveComponent* Component, FSkeletalMeshPhysicsProxy* InObject);
	ENGINE_API void AddObject(UPrimitiveComponent* Component, FStaticMeshPhysicsProxy* InObject);
	ENGINE_API void AddObject(UPrimitiveComponent* Component, Chaos::FSingleParticlePhysicsProxy* InObject);
	ENGINE_API void AddObject(UPrimitiveComponent* Component, FGeometryCollectionPhysicsProxy* InObject);
	ENGINE_API void AddObject(UPrimitiveComponent* Component, Chaos::FClusterUnionPhysicsProxy* InObject);
	
	ENGINE_API void AddToComponentMaps(UPrimitiveComponent* Component, IPhysicsProxyBase* InObject);
	ENGINE_API void RemoveFromComponentMaps(IPhysicsProxyBase* InObject);

	/**
	 * Called during physics state destruction for the game thread to remove objects from the simulation
	 * #BG TODO - Doesn't actually remove from the evolution at the moment
	 */
	ENGINE_API void RemoveObject(FSkeletalMeshPhysicsProxy* InObject);
	ENGINE_API void RemoveObject(FStaticMeshPhysicsProxy* InObject);
	ENGINE_API void RemoveObject(Chaos::FSingleParticlePhysicsProxy* InObject);
	ENGINE_API void RemoveObject(FGeometryCollectionPhysicsProxy* InObject);
	ENGINE_API void RemoveObject(Chaos::FClusterUnionPhysicsProxy* InObject);

	void PushPendingSerializedStateForObject(UObject* InComponent, Chaos::FSerializedDataBufferPtr&& InState);
	Chaos::FSerializedDataBufferPtr PopPendingSerializedStateForObject(UObject* InComponent);

	bool IsComponentMigrated(UPrimitiveComponent* PrimComp) const;
	void ResetComponentOwnerMigratedSet();
	void AddToComponentOwnerMigratedSet(const FRemoteObjectId CompOwnerId);
	void RemoveFromComponentOwnerMigratedSet(const FRemoteObjectId CompOwnerId);

	ENGINE_API IPhysicsReplication* GetPhysicsReplication();

	ENGINE_API const IPhysicsReplication* GetPhysicsReplication() const;

	ENGINE_API IPhysicsReplication* CreatePhysicsReplication();

	ENGINE_API IPhysicsReplicationLOD* GetPhysicsReplicationLOD();

	ENGINE_API IPhysicsReplicationLOD* CreatePhysicsReplicationLOD();

	ENGINE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	/** Given a solver object, returns its associated component. */
	template<class OwnerType>
	OwnerType* GetOwningComponent(const IPhysicsProxyBase* PhysicsProxy) const
	{ 
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(PhysicsProxyComponentMapsLock);
		const TWeakObjectPtr<UPrimitiveComponent>* ComponentWeak = PhysicsProxyToComponentMap.Find(PhysicsProxy);
		return (ComponentWeak != nullptr && ComponentWeak->IsValid()) ? Cast<OwnerType>(ComponentWeak->Get()) : nullptr;
	}

	template<>
	ENGINE_API UPrimitiveComponent* GetOwningComponent(const IPhysicsProxyBase* PhysicsProxy) const;

	/** Given a component, returns its associated solver objects. */
	const TArray<IPhysicsProxyBase*>* GetOwnedPhysicsProxies(UPrimitiveComponent* Comp) const
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(PhysicsProxyComponentMapsLock);
		return ComponentToPhysicsProxyMap.Find(Comp);
	}

	/** Given a physics proxy, returns its associated body instance if any */
	ENGINE_API FBodyInstance* GetBodyInstanceFromProxy(const IPhysicsProxyBase* PhysicsProxy) const;
	ENGINE_API const FBodyInstance* GetBodyInstanceFromProxyAndShape(IPhysicsProxyBase* InProxy, int32 InShapeIndex) const;
	/**
	 * Callback when a world ends, to mark updated packages dirty. This can't be done in final
	 * sync as the editor will ignore packages being dirtied in PIE. Also used to clean up any other references
	 */
	ENGINE_API void OnWorldEndPlay();
	ENGINE_API void OnWorldBeginPlay();

	ENGINE_API void AddAggregateToScene(const FPhysicsAggregateHandle& InAggregate);

	ENGINE_API void SetOwningWorld(UWorld* InOwningWorld);

	ENGINE_API UWorld* GetOwningWorld();
	ENGINE_API const UWorld* GetOwningWorld() const;

	ENGINE_API void ResimNFrames(int32 NumFrames);

	ENGINE_API void RemoveBodyInstanceFromPendingLists_AssumesLocked(FBodyInstance* BodyInstance, int32 SceneType);
	ENGINE_API void AddCustomPhysics_AssumesLocked(FBodyInstance* BodyInstance, FCalculateCustomPhysics& CalculateCustomPhysics);
	ENGINE_API void AddForce_AssumesLocked(FBodyInstance* BodyInstance, const FVector& Force, bool bAllowSubstepping, bool bAccelChange);
	ENGINE_API void AddForceAtPosition_AssumesLocked(FBodyInstance* BodyInstance, const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce = false);
	ENGINE_API void AddRadialForceToBody_AssumesLocked(FBodyInstance* BodyInstance, const FVector& Origin, const float Radius, const float Strength, const uint8 Falloff, bool bAccelChange, bool bAllowSubstepping);
	ENGINE_API void ClearForces_AssumesLocked(FBodyInstance* BodyInstance, bool bAllowSubstepping);
	ENGINE_API void AddTorque_AssumesLocked(FBodyInstance* BodyInstance, const FVector& Torque, bool bAllowSubstepping, bool bAccelChange);
	ENGINE_API void ClearTorques_AssumesLocked(FBodyInstance* BodyInstance, bool bAllowSubstepping);
	ENGINE_API void SetKinematicTarget_AssumesLocked(FBodyInstance* BodyInstance, const FTransform& TargetTM, bool bAllowSubstepping);
	ENGINE_API bool GetKinematicTarget_AssumesLocked(const FBodyInstance* BodyInstance, FTransform& OutTM) const;

	ENGINE_API bool MarkForPreSimKinematicUpdate(USkeletalMeshComponent* InSkelComp, ETeleportType InTeleport, bool bNeedsSkinning);
	ENGINE_API void ClearPreSimKinematicUpdate(USkeletalMeshComponent* InSkelComp);

	ENGINE_API void AddPendingOnConstraintBreak(FConstraintInstance* ConstraintInstance, int32 SceneType);
	ENGINE_API void AddPendingSleepingEvent(FBodyInstance* BI, ESleepEvent SleepEventType, int32 SceneType);

	ENGINE_API int32 DirtyElementCount(Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>& Collection);

	ENGINE_API TArray<FCollisionNotifyInfo>& GetPendingCollisionNotifies(int32 SceneType);

	static ENGINE_API bool SupportsOriginShifting();
	ENGINE_API void ApplyWorldOffset(FVector InOffset);
	ENGINE_API virtual float OnStartFrame(float InDeltaTime) override;

	ENGINE_API bool HandleExecCommands(const TCHAR* Cmd, FOutputDevice* Ar);
	ENGINE_API void ListAwakeRigidBodies(bool bIncludeKinematic);
	ENGINE_API int32 GetNumAwakeBodies() const;

	static ENGINE_API TSharedPtr<IPhysicsReplicationFactory> PhysicsReplicationFactory;

	ENGINE_API void StartAsync();
	ENGINE_API bool HasAsyncScene() const;
	ENGINE_API void SetPhysXTreeRebuildRate(int32 RebuildRate);
	ENGINE_API void EnsureCollisionTreeIsBuilt(UWorld* World);
	ENGINE_API void KillVisualDebugger();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPhysScenePreTick, FPhysScene_Chaos*, float /*DeltaSeconds*/);
	FOnPhysScenePreTick OnPhysScenePreTick;
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPhysSceneStep, FPhysScene_Chaos*, float /*DeltaSeconds*/);
	FOnPhysSceneStep OnPhysSceneStep;

	ENGINE_API bool ExecPxVis(uint32 SceneType, const TCHAR* Cmd, FOutputDevice* Ar);
	ENGINE_API bool ExecApexVis(uint32 SceneType, const TCHAR* Cmd, FOutputDevice* Ar);

	static ENGINE_API Chaos::FCollisionModifierCallback CollisionModifierCallback;

	ENGINE_API void DeferPhysicsStateCreation(UPrimitiveComponent* Component);
	ENGINE_API void RemoveDeferredPhysicsStateCreation(UPrimitiveComponent* Component);
	ENGINE_API void ProcessDeferredCreatePhysicsState();

	ENGINE_API static bool SupportsAsyncPhysicsStateCreation();
	ENGINE_API static bool SupportsAsyncPhysicsStateDestruction();
	ENGINE_API FPhysScene_AsyncPhysicsStateJobQueue& GetOrCreateAsyncPhysicsStateJobQueue();
	ENGINE_API bool PushAsyncCreatePhysicsStateJob(IAsyncPhysicsStateProcessor* Processor);
	ENGINE_API bool PushAsyncDestroyPhysicsStateJob(IAsyncPhysicsStateProcessor* Processor);
	ENGINE_API void RemoveAsyncPhysicsStateJobs(IAsyncPhysicsStateProcessor* Processor);
	ENGINE_API void ProcessAsyncPhysicsStateJobs(bool bWaitForCompletion = false);
	ENGINE_API bool HasAsyncPhysicsStateJobs() const;

	UChaosEventRelay* GetChaosEventRelay() const { return ChaosEventRelay.Get(); }

	/** Get cached state for replication, if no state is cached RegisterForReplicationCache() is called */
	ENGINE_API const FRigidBodyState* GetStateFromReplicationCache(UPrimitiveComponent* RootComponent, int& ServerFrame);
	
	/** Get cached state for replication, if no state is cached RegisterForReplicationCache() is called */
	ENGINE_API const FRigidBodyState* GetStateFromReplicationCache(Chaos::FPhysicsObjectHandle PhysicsObject, int& ServerFrame);
	
	/** Register a component for physics replication state caching, the component will deregister automatically if cache is not accessed within timelimit set by CVar: np2.ReplicationCache.LingerForNSeconds */
	ENGINE_API void RegisterForReplicationCache(UPrimitiveComponent* RootComponent);

	UE_DEPRECATED(5.5, "Deprecated, ReplicationCache has been moved to FPhysicsReplicationCache and it's getting populated automatically. Use GetStateFromReplicationCache() and RegisterForReplicationCache() to use the replication cache. ")
	void PopulateReplicationCache(const int32 PhysicsStep) { }

	/** Get the FPhysicsReplicationCache */
	FPhysicsReplicationCache* GetPhysicsReplicationCache()
	{
		// Create the physics replication cache if not already created
		CreatePhysicsReplicationCache();

		return PhysicsReplicationCache.Get();
	}

	/** Get the FPhysicsReplicationQuantization. Created on demand without forcing
	 *  the replication cache to be created. */
	FPhysicsReplicationQuantization* GetPhysicsReplicationQuantization()
	{
		CreatePhysicsReplicationQuantization();
		return PhysicsReplicationQuantization.Get();
	}

	/** Create the quantization manager if one doesn't exist. */
	ENGINE_API void CreatePhysicsReplicationQuantization();

	/** Create the replication cache if one doesn't exist.
	 *  @note The quantization manager is created first to keep its PostSolve sim
	 *  callback registered ahead of the cache's. Sim callback execution order follows
	 *  registration order. Reordering this can cause the cache to record pre snap
	 *  state and break the rewind cache's bit equality with the wire grid. */
	ENGINE_API void CreatePhysicsReplicationCache();

	struct UE_DEPRECATED(5.5, "Deprecated, ReplicationCache has been moved to FPhysicsReplicationCache")
		FReplicationCacheData
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FReplicationCacheData() = default;
		FReplicationCacheData(UPrimitiveComponent * InRootComponent, Chaos::FReal InAccessTime);
		~FReplicationCacheData() = default;
		FReplicationCacheData(const FReplicationCacheData&) = default;
		FReplicationCacheData(FReplicationCacheData&&) = default;
		FReplicationCacheData& operator=(const FReplicationCacheData&) = default;
		FReplicationCacheData& operator=(FReplicationCacheData&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UPrimitiveComponent* GetRootComponent() { return RootComponent.Get(); }
		FRigidBodyState& GetState() { return State; }
		void SetAccessTime(Chaos::FReal Time) { AccessTime = Time; }
		Chaos::FReal GetAccessTime() { return AccessTime; }
		void SetIsCached(bool InIsCached) { bValidStateCached = InIsCached; }
		bool IsCached() { return bValidStateCached; }
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	private:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TWeakObjectPtr<UPrimitiveComponent> RootComponent;
		Chaos::FReal AccessTime;
		bool bValidStateCached;
		FRigidBodyState State;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	struct UE_DEPRECATED(5.5, "Deprecated, ReplicationCache has been moved to FPhysicsReplicationCache")
		FPrimitiveComponentReplicationCache
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPrimitiveComponentReplicationCache() = default;
		~FPrimitiveComponentReplicationCache() = default;
		FPrimitiveComponentReplicationCache(const FPrimitiveComponentReplicationCache&) = default;
		FPrimitiveComponentReplicationCache(FPrimitiveComponentReplicationCache&&) = default;
		FPrimitiveComponentReplicationCache& operator=(const FPrimitiveComponentReplicationCache&) = default;
		FPrimitiveComponentReplicationCache& operator=(FPrimitiveComponentReplicationCache&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		int32 ServerFrame = 0;
		TMap<FObjectKey, FReplicationCacheData> Map;
		void Reset()
		{
			ServerFrame = 0;
			Map.Reset();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

private:
	
	TMap<UObject*, Chaos::FSerializedDataBufferPtr> PendingMigratedPhysicsStateByComponent;

	TSet<FRemoteObjectId> MigratedPrimitiveComponentOwnerIds;

	TUniquePtr<FPhysicsReplicationCache> PhysicsReplicationCache;
	TUniquePtr<FPhysicsReplicationQuantization> PhysicsReplicationQuantization;

	TSet<UPrimitiveComponent*> CollisionEventRegistrations;
	TSet<UPrimitiveComponent*> GlobalCollisionEventRegistrations;
	TSet<UPrimitiveComponent*> GlobalRemovalEventRegistrations;

	// contains the set of properties that uniquely identifies a reported collision
	// Note that order matters, { Body0, Body1 } is not the same as { Body1, Body0 }
	struct FUniqueContactPairKey
	{
		const void* Body0;
		const void* Body1;

		friend bool operator==(const FUniqueContactPairKey& Lhs, const FUniqueContactPairKey& Rhs)
		{
			return Lhs.Body0 == Rhs.Body0 && Lhs.Body1 == Rhs.Body1;
		}

		friend inline uint32 GetTypeHash(FUniqueContactPairKey const& P)
		{
			return (uint32)((PTRINT)P.Body0 ^ ((PTRINT)P.Body1 << 18));
		}
	};

	ENGINE_API FCollisionNotifyInfo& GetPendingCollisionForContactPair(const void* P0, const void* P1, Chaos::FReal SolverTime, bool& bNewEntry);
	/** Key is the unique pair, value is index into PendingNotifies array */
	TMultiMap<FUniqueContactPairKey, int32> ContactPairToPendingNotifyMap;

	/** Holds the list of pending legacy notifies that are to be processed */
	TArray<FCollisionNotifyInfo> PendingCollisionNotifies;

	// Chaos Event Handlers
	ENGINE_API void HandleEachCollisionEvent(const TArray<int32>& CollisionIndices, IPhysicsProxyBase* PhysicsProxy0, UPrimitiveComponent* const Comp0, Chaos::FCollisionDataArray const& CollisionData, Chaos::FReal MinDeltaVelocityThreshold);
	ENGINE_API void HandleGlobalCollisionEvent(Chaos::FCollisionDataArray const& CollisionData);
	ENGINE_API void HandleCollisionEvents(const Chaos::FCollisionEventData& CollisionData);
	ENGINE_API void DispatchPendingCollisionNotifies();

	ENGINE_API void HandleBreakingEvents(const Chaos::FBreakingEventData& Event);
	ENGINE_API void HandleRemovalEvents(const Chaos::FRemovalEventData& Event);
	ENGINE_API void HandleCrumblingEvents(const Chaos::FCrumblingEventData& Event);
	ENGINE_API void HandleTrailingEvents(const Chaos::FTrailingEventData& Event);

	/** Physics Replication manager which updates physics particles state to replicate towards networked target states */
	TUniquePtr<IPhysicsReplication> PhysicsReplication;

	/** Physics Replication "Level of Detail" that recommends physics replication mode and how much forward prediction should be applied for replicated physics particles */
	TUniquePtr<IPhysicsReplicationLOD> PhysicsReplicationLOD;

#if CHAOS_WITH_PAUSABLE_SOLVER
	/** Callback that checks the status of the world settings for this scene before pausing/unpausing its solver. */
	ENGINE_API void OnUpdateWorldPause();
#endif


#if WITH_EDITOR
	ENGINE_API bool IsOwningWorldEditor() const;
#endif

	ENGINE_API virtual void OnSyncBodies(Chaos::FPhysicsSolverBase* Solver) override;

	ENGINE_API void EnableAsyncPhysicsTickCallback();

#if 0
	void SetKinematicTransform(FPhysicsActorHandle& InActorReference,const Chaos::TRigidTransform<float,3>& NewTransform)
	{
		// #todo : Initialize
		// Set the buffered kinematic data on the game and render thread
		// InActorReference.GetPhysicsProxy()->SetKinematicData(...)
	}

	void EnableCollisionPair(const TTuple<int32,int32>& CollisionPair)
	{
		// #todo : Implement
	}

	void DisableCollisionPair(const TTuple<int32,int32>& CollisionPair)
	{
		// #todo : Implement
	}
#endif

	ENGINE_API FPhysicsConstraintHandle AddSpringConstraint(const TArray< TPair<FPhysicsActorHandle,FPhysicsActorHandle> >& Constraint);
	ENGINE_API void RemoveSpringConstraint(const FPhysicsConstraintHandle& Constraint);

#if 0
	void AddForce(const Chaos::FVec3& Force,FPhysicsActorHandle& Handle)
	{
		// #todo : Implement
	}

	void AddTorque(const Chaos::FVec3& Torque,FPhysicsActorHandle& Handle)
	{
		// #todo : Implement
	}
#endif

	/** Process kinematic updates on any deferred skeletal meshes */
	ENGINE_API void UpdateKinematicsOnDeferredSkelMeshes();

	/** Information about how to perform kinematic update before physics */
	struct FDeferredKinematicUpdateInfo
	{
		/** Whether to teleport physics bodies or not */
		ETeleportType	TeleportType;
		/** Whether to update skinning info */
		bool			bNeedsSkinning;
	};

	/** Map of SkeletalMeshComponents that need their bone transforms sent to the physics engine before simulation. */
	TArray<TPair<TWeakObjectPtr<USkeletalMeshComponent>, FDeferredKinematicUpdateInfo>> DeferredKinematicUpdateSkelMeshes;

	TSet<UPrimitiveComponent*> DeferredCreatePhysicsStateComponents;

	// Queue used to create/destroy physics state asynchronously.
	TUniquePtr<FPhysScene_AsyncPhysicsStateJobQueue> AsyncPhysicsStateJobQueue;

	// RWLock for thread safe access to PhysicsProxyToComponentMap and ComponentToPhysicsProxyMap.
	mutable FRWLock PhysicsProxyComponentMapsLock;

	// RWLock for thread safe access to CollisionEventRegistrations
	mutable FRWLock CollisionEventRegistrationsLock;

	//Body Instances
	TUniquePtr<Chaos::TArrayCollectionArray<FBodyInstance*>> BodyInstances;
	// Temp Interface
	TArray<FCollisionNotifyInfo> MNotifies;

	// Maps PhysicsProxy to Component that created the PhysicsProxy.
	// Keeps a weak ptr so that we can track validity of the primitive.
	TMap<IPhysicsProxyBase*, TWeakObjectPtr<UPrimitiveComponent>> PhysicsProxyToComponentMap;

	// Maps Component to PhysicsProxy that is created
	TMap<UPrimitiveComponent*, TArray<IPhysicsProxyBase*>> ComponentToPhysicsProxyMap;

	//TSet<UActorComponent*> FixedTickComponents;
	//TSet<AActor*> FixedTickActors;

	/** The SolverActor that spawned and owns this scene */
	TWeakObjectPtr<AActor> SolverActor;

	TObjectPtr<UChaosEventRelay> ChaosEventRelay;

	Chaos::FReal LastEventDispatchTime;
	Chaos::FReal LastBreakEventDispatchTime;
	Chaos::FReal LastRemovalEventDispatchTime;
	Chaos::FReal LastCrumblingEventDispatchTime;
	Chaos::FReal LastTrailingEventDispatchTime;
	
	class FAsyncPhysicsTickCallback* AsyncPhysicsTickCallback = nullptr;

#if WITH_EDITOR
	// Counter used to check a match with the single step status.
	int32 SingleStepCounter;
#endif
#if CHAOS_WITH_PAUSABLE_SOLVER
	// Cache the state of the game pause in order to avoid sending extraneous commands to the solver.
	bool bIsWorldPaused;
#endif

	// Allow other code to obtain read-locks when needed
	friend struct ChaosInterface::FScopedSceneReadLock;
	friend struct FScopedSceneLock_Chaos;
};
