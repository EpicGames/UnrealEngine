// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/CollisionProfile.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsThreadLibrary.generated.h"

UCLASS(meta = (ScriptName = "PhysicsThreadLibrary"), MinimalAPI)
class UPhysicsThreadLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
	 *	Add a force to a single rigid body.
	 *  This is like a 'thruster'. Good for adding a burst over some (non zero) time. Should be called every frame for the duration of the force.
	 *
	 *	@param	Force		 Force vector to apply. Magnitude indicates strength of force.
	 *  @param  bAccelChange If true, Force is taken as a change in acceleration instead of a physical force (i.e. mass will have no effect).
	 */
	UFUNCTION(BlueprintCallable, Category = "Utilities")
	static void AddForce(FBodyInstanceAsyncPhysicsTickHandle Handle, FVector Force, bool bAccelChange = false)
	{
		if (Handle)
		{
			if(bAccelChange)
			{
				Handle->AddForce(Force * Handle->M());
			}
			else
			{
				Handle->AddForce(Force);
			}
		}
	}

};
