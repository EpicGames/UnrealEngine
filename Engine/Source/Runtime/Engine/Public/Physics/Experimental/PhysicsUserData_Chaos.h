// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Physics/PhysicsInterfaceDeclares.h"

class UPhysicalMaterial;
struct FBodyInstance;
struct FConstraintInstanceBase;

// Note: this must match the enum EChaosUserDataType in PhysicsInterfaceTypesCore
class FPhysicsUserData_Chaos
{
private:
	enum EType
	{
		Invalid,
		BodyInstance,
		PhysicalMaterial,
		PhysScene,
		ConstraintInstanceBase
	};

	EType Type;
	void* Payload;

public:
	FPhysicsUserData_Chaos() : Type(EType::Invalid), Payload(nullptr) { }
	FPhysicsUserData_Chaos(FBodyInstance* InPayload) : Type(EType::BodyInstance), Payload(InPayload) { }
	FPhysicsUserData_Chaos(UPhysicalMaterial* InPayload) : Type(EType::PhysicalMaterial), Payload(InPayload) { }
	FPhysicsUserData_Chaos(FPhysScene* InPayload) : Type(EType::PhysScene), Payload(InPayload) { }
	FPhysicsUserData_Chaos(FConstraintInstanceBase* InPayload) : Type(EType::ConstraintInstanceBase), Payload(InPayload) { }

	template <class T> static T* Get(void* UserData);
	template <class T> static void Set(void* UserData, T* InPayload);

	/*
	template <class T> static T* Get(intptr_t UserData)
	{
		return Get<T>((void*)UserData);
	}

	template <class T> static void Set(intptr_t UserData, T* InPayload)
	{
		return Get<T>((void*)UserData);
	}
	*/

private:

	template <class T, EType TType> static T* Get(void* UserData)
	{
		FPhysicsUserData_Chaos* ChaosUserData = (FPhysicsUserData_Chaos*)UserData;
		return UserData && ChaosUserData->Type == TType ? (T*)(ChaosUserData->Payload) : nullptr;
	}
};

template<> inline FBodyInstance* FPhysicsUserData_Chaos::Get(void* UserData)
{
	return Get<FBodyInstance, EType::BodyInstance>(UserData);
}

template<> inline UPhysicalMaterial* FPhysicsUserData_Chaos::Get(void* UserData)
{
	return Get<UPhysicalMaterial, EType::PhysicalMaterial>(UserData);
}

template<> inline FPhysScene* FPhysicsUserData_Chaos::Get(void* UserData)
{
	return Get<FPhysScene, EType::PhysScene>(UserData);
}

template<> inline FConstraintInstanceBase* FPhysicsUserData_Chaos::Get(void* UserData)
{
	return Get<FConstraintInstanceBase, EType::ConstraintInstanceBase>(UserData);
}
