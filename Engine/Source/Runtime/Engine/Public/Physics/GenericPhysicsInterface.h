// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/SpatialAccelerationFwd.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"

/** Generic interface for physics APIs in the engine. Some common functionality is defined here, but APIs can override behavior as needed. See FGenericPlatformMisc for a similar pattern */

class UWorld;

namespace Chaos
{
	enum class EQueryInfo : uint8;
	enum class EThreadQueryContext : uint8;

	struct FCommonQueryData;
	struct FOverlapQueryData;
	struct FRayQueryData;
	struct FSweepQueryData;
}

struct FGenericPhysicsInterface
{
	/** Trace a ray against the world and return if a blocking hit is found */
	static ENGINE_API bool RaycastTest(const UWorld* World, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Trace a ray against the world and return the first blocking hit */
	static ENGINE_API bool RaycastSingle(const UWorld* World, struct FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/**
	*  Trace a ray against the world and return touching hits and then first blocking hit
	*  Results are sorted, so a blocking hit (if found) will be the last element of the array
	*  Only the single closest blocking result will be generated, no tests will be done after that
	*/
	static ENGINE_API bool RaycastMulti(const UWorld* World, TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Function used for sweeping a supplied shape against the world as a test */
	static ENGINE_API bool GeomSweepTest(const UWorld* World, const FCollisionShape& CollisionShape, const FQuat& Rot, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Function for sweeping a supplied shape against the world */
	template<typename GeomWrapper>
	static bool GeomSweepSingle(const UWorld* World, const GeomWrapper& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Sweep a supplied shape against the world and do not stop until the first blocking hit */
	template <typename GeomWrapper>
	static bool GeomSweepMulti(const UWorld* World, const GeomWrapper& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Find overlapping shapes with a given shape */
	template<typename GeomWrapper>
	static bool GeomOverlapMulti(const UWorld* World, const GeomWrapper& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

	/** Function for testing overlaps between a supplied PxGeometry and the world. Returns true if at least one overlapping shape is blocking*/
	static ENGINE_API bool GeomOverlapBlockingTest(const UWorld* World, const FCollisionShape& CollisionShape, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

	/** Function for testing overlaps between a supplied PxGeometry and the world. Returns true if anything is overlapping (blocking or touching)*/
	static ENGINE_API bool GeomOverlapAnyTest(const UWorld* World, const FCollisionShape& CollisionShape, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
};

template<typename TAccel>
struct FGenericRaycastPhysicsInterfaceUsingSpatialAcceleration
{
	static bool RaycastTest(const TAccel& Accel, const UWorld* World, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool RaycastSingle(const TAccel& Accel, const UWorld* World, struct FHitResult& OutHit, const FVector Start, const FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool RaycastMulti(const TAccel& Accel, const UWorld* World, TArray<struct FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
};

template<typename TAccel, typename TGeom>
struct FGenericGeomPhysicsInterfaceUsingSpatialAcceleration
{
	static bool GeomSweepTest(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FQuat& Rot, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool GeomSweepSingle(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool GeomSweepMulti(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool GeomOverlapMulti(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);
	static bool GeomOverlapBlockingTest(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	static bool GeomOverlapAnyTest(const TAccel& Accel, const UWorld* World, const TGeom& InGeom, const FVector& Pos, const FQuat& Rot, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
};

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const FCollisionShape& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const FPhysicsGeometry& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FCollisionShape& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FPhysicsGeometry& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FCollisionShape& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FPhysicsGeometry& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepSingle(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FQuat& Rot, FHitResult& OutHit, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomSweepMulti(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

template<>
ENGINE_API bool FGenericPhysicsInterface::GeomOverlapMulti(const UWorld* World, const FPhysicsGeometryCollection& InGeom, const FVector& InPosition, const FQuat& InRotation, TArray<FOverlapResult>& OutOverlaps, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams);

namespace Chaos::Private
{
	struct FGenericPhysicsInterface_Internal
	{
		/**
		*  INTERNAL USE ONLY
		*  Physics thread sphere query:
		*  Trace a sphere against the world and return touching hits and then first blocking hit
		*  Results are sorted, so a blocking hit (if found) will be the last element of the array
		*  Only the single closest blocking result will be generated, no tests will be done after that.
		*  Falls back to a raycast is the query radius is less than or equal to zero.
		*/
		static ENGINE_API bool SpherecastMulti(const UWorld* World, float QueryRadius, TArray<FHitResult>& OutHits, FVector Start, FVector End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

		/**
		*  INTERNAL USE ONLY
		*  Physics thread shape sweep:
		*  Trace a FCollisionShape against the world and return touching hits and then first blocking hit
		*  Results are sorted, so a blocking hit (if found) will be the last element of the array
		*  Only the single closest blocking result will be generated, no tests will be done after that.
		*  Falls back to a raycast is the shape size is less than or equal to UE_KINDA_SMALL_NUMBER.
		*/
		UE_EXPERIMENTAL(5.6, "Physics Thread sweep trace, might have API changes")
		static ENGINE_API bool GeomSweepMulti(const UWorld* World, const FCollisionShape& InGeom, const FQuat& InGeomRot, TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);

		/**
		*  INTERNAL USE ONLY
		*  Physics thread raycast query:
		*  Trace a ray against the world and return touching hits and then first blocking hit
		*  Results are sorted, so a blocking hit (if found) will be the last element of the array
		*  Only the single closest blocking result will be generated, no tests will be done after that.
		*/
		UE_EXPERIMENTAL(5.6, "Physics Thread raycast, might have API changes")
		static ENGINE_API bool RaycastMulti(const UWorld* World, TArray<FHitResult>& OutHits, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, const FCollisionQueryParams& Params, const FCollisionResponseParams& ResponseParams, const FCollisionObjectQueryParams& ObjectParams = FCollisionObjectQueryParams::DefaultObjectQueryParam);
	};

	// This is for internal use so that the QueryInterface API can resume queries.
	struct FQueryInterface_Internal
	{
		static bool Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps);

		static bool RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits);
		static bool RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits);
		static bool RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits);

		static bool SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits);
		static bool SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits);
		static bool SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits);
	};

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Scoped object that allows the physics query handle to be ignored for the current thread.
	struct FScopedIgnoreQueryHandler
	{
		FScopedIgnoreQueryHandler(bool bIgnore);
		~FScopedIgnoreQueryHandler();

	private:
		bool bOldState;
	};
#endif
}
