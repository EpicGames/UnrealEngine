// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "Chaos/ChaosEngineInterface.h"

#include "Engine/EngineTypes.h"

struct FConeConstraint;
struct FConstraintProfileProperties;
struct FLinearConstraint;
struct FTwistConstraint;
struct FLinearDriveConstraint;
struct FAngularDriveConstraint;

namespace ImmediatePhysics
{
	// Make an ActorSetup from a BodyInstance
	ENGINE_API FActorSetup MakeStaticActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform);

	// Make an ActorSetup from a BodyInstance
	ENGINE_API FActorSetup MakeKinematicActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform);

	// Make an ActorSetup from a BodyInstance
	ENGINE_API FActorSetup MakeDynamicActorSetup(const FBodyInstance* InBodyInstance, const FTransform& InTransform);

	// Make an ActorSetup from a BodyInstance
	ENGINE_API FActorSetup MakeActorSetup(EActorType InActorType, const FBodyInstance* InBodyInstance, const FTransform& InTransform);

	// Make a JointSetup from a ConstraintInstance
	ENGINE_API FJointSetup MakeJointSetup(const FConstraintInstance* InConstraintInstance, FActorHandle* InActor1, FActorHandle* InActor2);

	// Transfers parameters from a constraint profile into the joint settings
	ENGINE_API void UpdateJointSettingsFromConstraintProfile(const FConstraintProfileProperties& Profile, Chaos::FPBDJointSettings& JointSettings);

	// Extracts parameters from a constraint constraint into the joint settings
	ENGINE_API void UpdateJointSettingsFromLinearConstraint(const FLinearConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings);
	
	// Extracts parameters from a cone constraint into the joint settings
	ENGINE_API void UpdateJointSettingsFromConeConstraint(const FConeConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings);
	
	// Extracts parameters from a twist constraint into the joint settings
	ENGINE_API void UpdateJointSettingsFromTwistConstraint(const FTwistConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings);

	// Extracts parameters from a linear drive constraint into the joint settings
	ENGINE_API void UpdateJointSettingsFromLinearDriveConstraint(const FLinearDriveConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings);

	// Extracts parameters from an angular drive constraint into the joint settings
	ENGINE_API void UpdateJointSettingsFromAngularDriveConstraint(const FAngularDriveConstraint& Constraint, Chaos::FPBDJointSettings& JointSettings);
}
