// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsCore_Chaos.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/ImplicitObject.h"

#include "Engine/EngineTypes.h"

namespace ImmediatePhysics_Chaos
{
	struct FActorSetup
	{
	public:
		ENGINE_API FActorSetup();
		ENGINE_API FActorSetup(FActorSetup&& R);
		ENGINE_API ~FActorSetup();

		FTransform Transform;

		// NOTE: Shapes and Material are Moved out the setup structure when
		// ued to create an Actor.
		Chaos::FImplicitObjectPtr Geometry;
		TArray<TUniquePtr<Chaos::FPerShapeData>> Shapes;
		TUniquePtr<Chaos::FChaosPhysicsMaterial> Material;

		FReal Mass;
		FVector Inertia;
		FTransform CoMTransform;

		FReal LinearDamping;
		FReal AngularDamping;

		EActorType ActorType;
		ECollisionEnabled::Type CollisionEnabled;
		int8 GravityGroupIndex;
		uint8 bInertiaConditioningEnabled : 1;
		uint8 bEnableGravity : 1;
		uint8 bUpdateKinematicFromSimulation : 1;
		uint8 bGyroscopicTorqueEnabled : 1;
	};

	/** handle associated with a physics actor. This is the proper way to read/write to the physics simulation */
	struct FActorHandle
	{
	public:
		ENGINE_API ~FActorHandle();

		void SetName(const FName& InName) { Name = InName; }
		const FName& GetName() const { return Name; }

		ENGINE_API bool GetEnabled() const;

		/**
		 * Note that this will not update the colliding pairs in the simulation, so better to change the kinematic
		 * state there.
		 */
		ENGINE_API void SetEnabled(bool bEnabled);

		ENGINE_API bool GetHasCollision() const;

		/**
		 * Note that this will not update the colliding pairs in the simulation, so better to change the kinematic
		 * state there.
		 */
		ENGINE_API void SetHasCollision(bool bHasCollision);

		/**
		 * Sets the world transform, zeroes velocity, etc. This also initialises the bounds, which may be
		 * important since otherwise they would only get updated if the object actually moves.
		 */
		ENGINE_API void InitWorldTransform(const FTransform& WorldTM);

		/** Sets the world transform, maintains velocity etc.*/
		ENGINE_API void SetWorldTransform(const FTransform& WorldTM);

		/** 
		 * Normally used to modify a EParticleType::Rigid between kinematic and dynamic. Returns true if that 
		 * was possible (even if nothing changed). Returns false if it fails - e.g. because there was no current 
		 * ParticleHandle, or if the particle type was not EParticleType::Rigid (unless the current type was
		 * EParticleType::Kinematic and bKinematic was true)
		 * 
		 * Note that this will not update the colliding pairs in the simulation, so better to change the kinematic
		 * state there.
		 */
		ENGINE_API bool SetIsKinematic(bool bKinematic);

		/** Is the actor kinematic */
		ENGINE_API bool GetIsKinematic() const;

		/** Gets the kinematic target (next transform) for the actor if one is set (check HasKinematicTarget() to see if a target is available) */
		ENGINE_API const FKinematicTarget& GetKinematicTarget() const;

		/** Sets the kinematic target. This will affect velocities as expected*/
		ENGINE_API void SetKinematicTarget(const FTransform& WorldTM);

		/** Does this actor have a kinematic target (next kinematic transform to be applied) */
		ENGINE_API bool HasKinematicTarget() const;

		/** Returns true if the world gravity applies to this actor */
		ENGINE_API bool IsGravityEnabled() const;

		/** Returns true if the output transform is set to the final body transform when it is kinematic. Will return false if the body is not a rigid. */
		ENGINE_API bool GetUpdateKinematicFromSimulation() const;

		/** Sets whether or not the output transform is set to the final body transform when it is kinematic. */
		ENGINE_API void SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation);

		/** Returns true the body is or could be dynamic (i.e. currently simulated, or a rigid body that is currently kinematic) */
		ENGINE_API bool CouldBeDynamic() const;

		/** Sets whether world gravity should apply to this actor, assuming it is dynamic */
		ENGINE_API void SetGravityEnabled(bool bEnable);

		/** Whether the body is simulating */
		ENGINE_API bool IsSimulated() const;

		/** Get the world transform */
		ENGINE_API FTransform GetWorldTransform() const;

		/** Set the linear velocity */
		ENGINE_API void SetLinearVelocity(const FVector& NewLinearVelocity);

		/** Get the linear velocity */
		ENGINE_API FVector GetLinearVelocity() const;

		/** Set the angular velocity */
		ENGINE_API void SetAngularVelocity(const FVector& NewAngularVelocity);

		/** Get the angular velocity */
		ENGINE_API FVector GetAngularVelocity() const;

		/** Adds a force at the centre of mass */
		ENGINE_API void AddForce(const FVector& Force);

		/** Adds a "linear push" at the centre of mass, which can be a force, acceleration, impulse or velocity change */
		ENGINE_API void AddForce(const FVector& Linear, EForceType ForceType);

		/** Adds a "linear push" at the specified world location, which can be a force, acceleration, impulse or velocity change */
		ENGINE_API void AddForceAtLocation(const FVector& Linear, const FVector& Location, EForceType ForceType);

		/** Adds a linear impulse at the specified world location */
		ENGINE_API void AddImpulseAtLocation(FVector Impulse, FVector Location);

		/** Adds a torque to the body */
		ENGINE_API void AddTorque(const FVector& Torque);

		/** Adds a "rotation push" to the body, which can be an angular force, acceleration, impulse or velocity change */
		ENGINE_API void AddTorque(const FVector& Angular, EForceType ForceType);

		/** Adds a "linear push" (depending on the ForceType) to the centre of mass of the body that is directed away from, and decays with distance from (up to Radius), Origin */
		ENGINE_API void AddRadialForce(const FVector& Origin, FReal Strength, FReal Radius, ERadialImpulseFalloff Falloff, EForceType ForceType);

		/** Set the linear damping*/
		ENGINE_API void SetLinearDamping(FReal NewLinearDamping);

		/** Get the linear damping*/
		ENGINE_API FReal GetLinearDamping() const;

		/** Set the angular damping*/
		ENGINE_API void SetAngularDamping(FReal NewAngularDamping);

		/** Get the angular damping*/
		ENGINE_API FReal GetAngularDamping() const;

		/** Set the max linear velocity squared*/
		ENGINE_API void SetMaxLinearVelocitySquared(FReal NewMaxLinearVelocitySquared);

		/** Get the max linear velocity squared*/
		ENGINE_API FReal GetMaxLinearVelocitySquared() const;

		/** Set the max angular velocity squared*/
		ENGINE_API void SetMaxAngularVelocitySquared(FReal NewMaxAngularVelocitySquared);

		/** Get the max angular velocity squared*/
		ENGINE_API FReal GetMaxAngularVelocitySquared() const;

		/** Set the inverse mass. 0 indicates kinematic object */
		ENGINE_API void SetInverseMass(FReal NewInverseMass);

		/** Get the inverse mass. */
		ENGINE_API FReal GetInverseMass() const;
		ENGINE_API FReal GetMass() const;

		/** Set the inverse inertia. Mass-space inverse inertia diagonal vector */
		ENGINE_API void SetInverseInertia(const FVector& NewInverseInertia);

		/** Get the inverse inertia. Mass-space inverse inertia diagonal vector */
		ENGINE_API FVector GetInverseInertia() const;
		ENGINE_API FVector GetInertia() const;

		/** Set the max depenetration velocity*/
		ENGINE_API void SetMaxDepenetrationVelocity(FReal NewMaxDepenetrationVelocity);

		/** Get the max depenetration velocity*/
		ENGINE_API FReal GetMaxDepenetrationVelocity() const;

		/** Set the max contact impulse*/
		ENGINE_API void SetMaxContactImpulse(FReal NewMaxContactImpulse);

		/** Get the max contact impulse*/
		ENGINE_API FReal GetMaxContactImpulse() const;

		/** Get the actor-space centre of mass offset */
		ENGINE_API FTransform GetLocalCoMTransform() const;

		/** Get the actor-space centre of mass offset (location only) */
		ENGINE_API FVector GetLocalCoMLocation() const;

		/** Enable/disable CCD */
		ENGINE_API void SetCCDEnabled(bool bEnable);

		/** returns whether CCD is enabled for the particle */
		ENGINE_API bool GetCCDEnabled() const;

		ENGINE_API Chaos::FGeometryParticleHandle* GetParticle();
		ENGINE_API const Chaos::FGeometryParticleHandle* GetParticle() const;

		ENGINE_API int32 GetLevel() const;
		ENGINE_API void SetLevel(int32 InLevel);

	private:
		FKinematicTarget& GetKinematicTarget();

		void CreateParticleHandle(
			FActorSetup&& ActorSetup);

		friend struct FSimulation;
		friend struct FJointHandle;

		FActorHandle(
			Chaos::FPBDRigidsSOAs& InParticles,
			Chaos::TArrayCollectionArray<Chaos::FVec3>& InParticlePrevXs,
			Chaos::TArrayCollectionArray<Chaos::FRotation3>& InParticlePrevRs,
			FActorSetup&& ActorSetup);


		Chaos::FGenericParticleHandle Handle() const;

		FName Name;
		Chaos::FPBDRigidsSOAs& Particles;
		Chaos::FGeometryParticleHandle* ParticleHandle;
		Chaos::TArrayCollectionArray<Chaos::FVec3>& ParticlePrevXs;
		Chaos::TArrayCollectionArray<Chaos::FRotation3>& ParticlePrevRs;
		int32 Level;
	};
}
