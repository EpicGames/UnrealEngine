// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsDeclares_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsShared/ImmediatePhysicsCore.h"

#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"

namespace Chaos
{
	class FImplicitObject;
	class FBasicBroadPhase;
	class FBasicCollisionDetector;
	class FKinematicTarget;
	class FPBDJointConstraintHandle;
	class FPBDJointConstraints;
	struct FPBDJointCombinedConstraints;
	class FPerShapeData;
	template<class T> class TArrayCollectionArray;
	struct FKinematicGeometryParticleParameters;
	template<typename T> class TPBDConstraintIslandRule;
	struct FPBDRigidParticleParameters;
	class FPBDRigidsSOAs;
	template<typename T> class TSimpleConstraintRule;

}

namespace ImmediatePhysics_Chaos
{
	using FReal = Chaos::FReal;
	using FRealSingle = Chaos::FRealSingle;
	//const int Dimensions = 3;

	using EActorType = ImmediatePhysics_Shared::EActorType;
	using EForceType = ImmediatePhysics_Shared::EForceType;

	using FKinematicTarget = Chaos::FKinematicTarget;
}

struct FBodyInstance;
struct FConstraintInstance;

// Used to define out code that still has to be implemented to match PhysX
#define IMMEDIATEPHYSICS_CHAOS_TODO 0
