// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace Chaos
{
	class FPBDJointSettings;
}

namespace ImmediatePhysics_Chaos
{
	struct FActorHandle;
	struct FActorSetup;
	struct FJointHandle;
	struct FJointSetup;
	struct FSimulation;
}
