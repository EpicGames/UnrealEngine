// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsCore_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsActorHandle_Chaos.h"

#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/Vector.h"

#include "Engine/EngineTypes.h"

namespace ImmediatePhysics_Chaos
{
	struct FJointSetup
	{
	public:
		FJointSetup()
			: ConstraintSettings()
			, Actor1(nullptr)
			, Actor2(nullptr)
		{
		}

		FJointSetup(const Chaos::FPBDJointSettings& InConstraintSettings, FActorHandle* InActor1, FActorHandle* InActor2)
			: ConstraintSettings(InConstraintSettings)
			, Actor1(InActor1)
			, Actor2(InActor2)
		{
		}

		Chaos::FPBDJointSettings ConstraintSettings;
		FActorHandle* Actor1;
		FActorHandle* Actor2;
	};

	/** handle associated with a physics joint. This is the proper way to read/write to the physics simulation */
	struct FJointHandle
	{
	public:
		using FChaosConstraintContainer = Chaos::FPBDJointConstraints;
		using FChaosConstraintHandle = typename Chaos::FPBDJointConstraintHandle;

		using FPBDJointSettings UE_DEPRECATED(5.6, "Not used") = Chaos::FPBDJointSettings;

		UE_DEPRECATED(5.6, "Use constructor which takes FJointSetup")
		ENGINE_API FJointHandle(FChaosConstraintContainer* InConstraints, FConstraintInstance* ConstraintInstance, FActorHandle* InActor1, FActorHandle* InActor2);

		UE_DEPRECATED(5.6, "Use constructor which takes FJointSetup")
		ENGINE_API FJointHandle(FChaosConstraintContainer* InConstraints, const Chaos::FPBDJointSettings& ConstraintSettings, FActorHandle* InActor1, FActorHandle* InActor2);

		ENGINE_API FJointHandle(FChaosConstraintContainer* InConstraints, const FJointSetup& JointSetup);
		ENGINE_API ~FJointHandle();

		ENGINE_API FChaosConstraintHandle* GetConstraint();
		ENGINE_API const FChaosConstraintHandle* GetConstraint() const;

		ENGINE_API const Chaos::TVec2<FActorHandle*>& GetActorHandles();
		ENGINE_API const Chaos::TVec2<const FActorHandle*>& GetActorHandles() const;

		ENGINE_API void SetSoftLinearSettings(bool bLinearSoft, FReal LinearStiffness, FReal LinearDamping);

	private:
		void CreateJoint(FChaosConstraintContainer* InConstraints, const Chaos::FPBDJointSettings& ConstraintSettings, FActorHandle* const Actor1, FActorHandle* const Actor2);
		void SetActorInertiaConditioningDirty();
		
		Chaos::TVec2<FActorHandle*> ActorHandles;
		FChaosConstraintContainer* Constraints;
		FChaosConstraintHandle* ConstraintHandle;
	};
}

// TODO: REMOVE - only for deprecated function
struct FConstraintProfileProperties;

namespace ImmediatePhysics_Chaos
{
	UE_DEPRECATED(5.6, "Moved to ImmediatePhysics namespace")
	ENGINE_API void UpdateJointSettingsFromConstraintProfile(const FConstraintProfileProperties& Profile, Chaos::FPBDJointSettings& JointSettings);
}
