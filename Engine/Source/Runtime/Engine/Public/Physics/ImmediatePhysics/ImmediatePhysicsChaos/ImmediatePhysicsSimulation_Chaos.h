// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsCore_Chaos.h"

#include "Chaos/Collision/CollisionVisitor.h"
#include "Engine/EngineTypes.h"
#include "Templates/UniquePtr.h"
#include "Chaos/ChaosDebugNameDefines.h"

namespace Chaos
{
	class FPBDJointSettings;
	class FPBDJointSolverSettings;
	class FPBDCollisionSolverSettings;
	class FCollisionDetectorSettings;
	class FSimulationSpaceSettings;
}

namespace ImmediatePhysics_Chaos
{
	/** Owns all the data associated with the simulation. Can be considered a single scene or world */
	struct FSimulation
	{
	public:
		ENGINE_API FSimulation();
		ENGINE_API ~FSimulation();

		ENGINE_API int32 NumActors() const;

		ENGINE_API FActorHandle* GetActorHandle(int32 ActorHandleIndex);
		ENGINE_API const FActorHandle* GetActorHandle(int32 ActorHandleIndex) const;

		UE_DEPRECATED(5.6, "Use CreateActor with FActorSetup")
		ENGINE_API FActorHandle* CreateStaticActor(FBodyInstance* BodyInstance);
		
		UE_DEPRECATED(5.6, "Use CreateActor with FActorSetup")
		ENGINE_API FActorHandle* CreateKinematicActor(FBodyInstance* BodyInstance, const FTransform& Transform);
		
		UE_DEPRECATED(5.6, "Use CreateActor with FActorSetup")
		ENGINE_API FActorHandle* CreateDynamicActor(FBodyInstance* BodyInstance, const FTransform& Transform);

		UE_DEPRECATED(5.6, "Use CreateActor with FActorSetup")
		ENGINE_API FActorHandle* CreateActor(EActorType ActorType, FBodyInstance* BodyInstance, const FTransform& Transform);

		ENGINE_API FActorHandle* CreateActor(FActorSetup&& ActorSetup);

		ENGINE_API void DestroyActor(FActorHandle* ActorHandle);

		ENGINE_API void DestroyActorCollisions(FActorHandle* ActorHandle);

		ENGINE_API void SetIsKinematic(FActorHandle* ActorHandle, bool bKinematic);

		ENGINE_API void SetEnabled(FActorHandle* ActorHandle, bool bEnable);

		ENGINE_API void SetHasCollision(FActorHandle* ActorHandle, bool bHasCollision);

		/** Create a physical joint and add it to the simulation */
		UE_DEPRECATED(5.6, "Use CreateActor with FJointSetup")
		ENGINE_API FJointHandle* CreateJoint(FConstraintInstance* ConstraintInstance, FActorHandle* Body1, FActorHandle* Body2);

		/** Create a physical joint and add it to the simulation */
		ENGINE_API FJointHandle* CreateJoint(const FJointSetup& JointSetup);

		ENGINE_API void DestroyJoint(FJointHandle* JointHandle);

		/** Sets the number of active bodies. This number is reset any time a new simulated body is created */
		ENGINE_API void SetNumActiveBodies(int32 NumActiveBodies, TArray<int32> ActiveBodyIndices);

		/** An array of actors to ignore. */
		struct FIgnorePair
		{
			FActorHandle* A;
			FActorHandle* B;
		};

		/** Set pair of bodies to ignore collision for */
		ENGINE_API void SetIgnoreCollisionPairTable(const TArray<FIgnorePair>& InIgnoreCollisionPairTable);

		/** Set bodies that require no collision */
		ENGINE_API void SetIgnoreCollisionActors(const TArray<FActorHandle*>& InIgnoreCollisionActors);

		/** Set up potential collisions between the actor and all other dynamic actors */
		ENGINE_API void AddToCollidingPairs(FActorHandle* ActorHandle);

		/** 
		 * Sets whether velocities should be rewound when simulating - this may happen when the requested 
		 * step size is smaller than the fixed simulation step.
		 */
		ENGINE_API void SetRewindVelocities(bool bRewindVelocities);

		/** 
		 * Advance the simulation by DeltaTime. If settings are passed in they will be used, 
		 * otherwise they will be taken from CVars 
		 */
		ENGINE_API void Simulate(
			FReal DeltaTime, FReal MaxStepTime, int32 MaxSubSteps, const FVector& InGravity,
			Chaos::FPBDJointSolverSettings* JointSolverSettings = nullptr, 
			Chaos::FPBDCollisionSolverSettings* CollisionSolverSettings = nullptr,
			Chaos::FCollisionDetectorSettings* CollisionDetectorSettings = nullptr);

		void Simulate_AssumesLocked(FReal DeltaTime, FReal MaxStepTime, int32 MaxSubSteps, const FVector& InGravity,
			Chaos::FPBDJointSolverSettings* JointSolverSettings = nullptr,
			Chaos::FPBDCollisionSolverSettings* CollisionSolverSettings = nullptr,
			Chaos::FCollisionDetectorSettings* CollisionDetectorSettings = nullptr)
		{
			Simulate(DeltaTime, MaxStepTime, MaxSubSteps, InGravity, JointSolverSettings, CollisionSolverSettings, CollisionDetectorSettings);
		}

		ENGINE_API void InitSimulationSpace(
			const FTransform& Transform);

		ENGINE_API void UpdateSimulationSpace(
			const FTransform& Transform,
			const FVector& LinearVel,
			const FVector& AngularVel,
			const FVector& LinearAcc,
			const FVector& AngularAcc);

		ENGINE_API void SetSimulationSpaceSettings(
			const bool bEnabled, 
			const FReal DampingAlpha,
			const FVector& ExternalLinearEtherDrag);

		ENGINE_API const Chaos::FSimulationSpaceSettings& GetSimulationSpaceSettings() const;

		ENGINE_API void SetSimulationSpaceSettings(
			const Chaos::FSimulationSpaceSettings& SimulationSpaceSettings);

		ENGINE_API const Chaos::FPBDCollisionSolverSettings& GetCollisionSolverSettings() const;

		ENGINE_API const Chaos::FCollisionDetectorSettings& GetCollisionDetectorSettings() const;

		ENGINE_API void SetCollisionDetectorSettings(const Chaos::FCollisionDetectorSettings& Settings);

		/** Set settings. Invalid (negative) values will leave that value unchanged from defaults */
		ENGINE_API void SetSolverSettings(
			const FReal FixedDt,
			const FReal CullDistance,
			const FReal MaxDepenetrationVelocity,
			const int32 UseLinearJointSolver,
			const int32 PositionIts,
			const int32 VelocityIts,
			const int32 ProjectionIts,
			const int32 bUseManifolds);

		// Additional settings for less commonly adjusted parameters. As in SetSolverSettings,
		// invalid (negative) values will leave that value unchanged from the default.
		UE_INTERNAL
		ENGINE_API void SetMaxNumRollingAverageStepTimes(const int32 MaxNumRollingAverageStepTimes);

		// Sets whether or not to use the minimum step time, which is itself set with a cvar p.Chaos.ImmPhys.MinStepTime
		UE_INTERNAL
		ENGINE_API void SetUseMinStepTime(bool bUse);

		// Sets whether or not to use the fixed timestep tolerance for rewinding, which is itself set with a cvar p.Chaos.ImmPhys.FixedStepTolerance
		UE_INTERNAL
		ENGINE_API void SetUseFixedStepTolerance(bool bUse);

		/** Explicit debug draw path if the use case needs it to happen at a point outside of the simulation **/
		ENGINE_API void DebugDraw();

		// Data relating to contacts returned by VisitCollisions
		struct FCollisionData
		{
		public:
			// The overall accumulated impulse applied due to this collision/contact. Only valid if
			// called after the solve has completed.
			ENGINE_API Chaos::FVec3 GetCollisionAccumulatedImpulse() const;

			ENGINE_API int GetNumManifoldPoints() const;

			// The contact represents a point attached to one particle, and the plane attached to
			// the other. These will be calculated and returned relative to where the particles are now.
			ENGINE_API const void GetManifoldPointData(
				const int32              ManifoldPointIndex,
				FRealSingle&             Depth, // The initial penetration depth
				Chaos::FVec3&            PlaneNormal,
				Chaos::FVec3&            PointLocation,
				Chaos::FVec3&            PlaneLocation) const;

		private:
			friend struct FSimulation;
			// This is constructed by Chaos, so it passes in the original CollisionConstraint
			FCollisionData(const Chaos::FPBDCollisionConstraint& InCollisionConstraint) 
				: Collision(InCollisionConstraint) {}

			const Chaos::FPBDCollisionConstraint& Collision;
		};

		// Access to collisions detected during the previous solve. 
		ENGINE_API void VisitCollisions(
			TFunction<void(const FCollisionData&)> Visitor, Chaos::ECollisionVisitorFlags VisitorFlags) const;

	private:
		void RemoveFromCollidingPairs(FActorHandle* ActorHandle);
		void UpdateInertiaConditioning(const FVector& Gravity);
		void PackCollidingPairs();
		void UpdateActivePotentiallyCollidingPairs();
		void EnableDisableJoints();
		FReal UpdateStepTime(const FReal DeltaTime, const FReal MaxStepTime);

		void UpdateStatCounters();
		void DebugDrawStaticParticles();
		void DebugDrawKinematicParticles();
		void DebugDrawDynamicParticles();
		void DebugDrawConstraints();
		void DebugDrawSimulationSpace();

		struct FImplementation;
		TUniquePtr<FImplementation> Implementation;

#if CHAOS_SOLVER_DEBUG_NAME
	public:
		void SetDebugName(const FName& Name)
		{
			DebugName = Name;
		}

		const FName& GetDebugName() const
		{
			return DebugName;
		}

	private:
	FName DebugName;
#endif

	private:

#if WITH_CHAOS_VISUAL_DEBUGGER
	private:
		FChaosVDContext CVDContextData;

	public:
		int32 GetCVDFrameNumber() const { return INDEX_NONE; }

		FChaosVDContext& GetChaosVDContextData()
		{
			return CVDContextData;
		};
#endif

#if CHAOS_DEBUG_DRAW
	public:
		ENGINE_API void SetDebugDrawScene(const FString& SceneName, const ChaosDD::Private::FChaosDDScenePtr& InScene);

	private:
		ChaosDD::Private::FChaosDDTimelinePtr DDSimulationTimeline;
#endif

	};

}
