// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsDeclares_Chaos.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsShared/ImmediatePhysicsCore.h"

namespace ImmediatePhysics
{
	using EActorType = ImmediatePhysics_Shared::EActorType;
	using EForceType = ImmediatePhysics_Shared::EForceType;
	using FSimulation = ImmediatePhysics_Chaos::FSimulation;
	using FActorHandle = ImmediatePhysics_Chaos::FActorHandle;
	using FActorSetup = ImmediatePhysics_Chaos::FActorSetup;
	using FJointHandle = ImmediatePhysics_Chaos::FJointHandle;
	using FJointSetup = ImmediatePhysics_Chaos::FJointSetup;
}