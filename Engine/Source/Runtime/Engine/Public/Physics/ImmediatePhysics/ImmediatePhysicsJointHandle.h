// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Physics/ImmediatePhysics/ImmediatePhysicsPhysX/ImmediatePhysicsJointHandle_PhysX.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsJointHandle_Chaos.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
