// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * ImediatePhysics Edit mode module interface
 */
class IImediatePhysicsModule : public IModuleInterface
{
public:
};
