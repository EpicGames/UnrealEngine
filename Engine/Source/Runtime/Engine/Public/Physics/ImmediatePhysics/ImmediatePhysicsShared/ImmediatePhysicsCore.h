// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * @file ImmediatePhysicsCore.h
 *
 * This is for declarations that are shared by both PhysX and Chaos implementations.
 * This will be included in the engine-specific implementations and therefore should never pull in any 
 * headers or declare any types that are in the engine-specific namespaces (which would results
 * in some types from each engine being brought into the other).
 */

namespace ImmediatePhysics_Shared
{
	enum class EActorType
	{
		StaticActor,		//collision but no movement
		KinematicActor,		//collision + movement but no dynamics (forces, mass, etc...)
		DynamicActor		//collision + movement + dynamics
	};

	enum class EForceType
	{
		AddForce,			//use mass and delta time
		AddAcceleration,	//use delta time, ignore mass
		AddImpulse,			//use mass, ignore delta time
		AddVelocity			//ignore mass, ignore delta time
	};
}