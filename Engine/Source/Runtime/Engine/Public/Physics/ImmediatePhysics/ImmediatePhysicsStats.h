// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("Immediate Physics"), STATGROUP_ImmediatePhysics, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Immediate Physics Counters"), STATGROUP_ImmediatePhysicsCounters, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("RigidBodyNodeInitTime"), STAT_RigidBodyNodeInitTime, STATGROUP_ImmediatePhysics,);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RigidBodyNodeInitTime_SetupSimulation"), STAT_RigidBodyNodeInitTime_SetupSimulation, STATGROUP_ImmediatePhysics,);
