// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/EngineTypes.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "RewindData.h"
#include "Components/ActorComponent.h"
#include "GameFramework/PawnMovementComponent.h"
#include "GameFramework/PlayerController.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Subsystems/WorldSubsystem.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Chaos/PhysicsObject.h"
#include "Physics/NetworkPhysicsSettingsComponent.h"
#include "StructUtils/InstancedStruct.h"
#include "Iris/ReplicationState/PropertyReplicationState.h"
#include "Iris/ReplicationSystem/ReplicationFragment.h"

#include "NetworkPhysicsComponent.generated.h"

#ifndef DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION
#define DEBUG_NETWORK_PHYSICS_DELTASERIALIZATION 0
#endif

class FAsyncNetworkPhysicsComponent;
struct FAsyncNetworkPhysicsComponentInput;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreProcessInputsInternal, const int32);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostProcessInputsInternal, const int32);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInjectInputsExternal, const int32 /* PhysicsStep */, const int32 /* NumSteps */);
DECLARE_MULTICAST_DELEGATE_OneParam(FAddResimulationRequestInternal, const int32 /* PhysicsStep */);


namespace NetworkPhysicsComponentConstants
{
	/** Size of the array caching Delta Sources for delta serialization.
	* NOTE: This is also used to set the number of bits to serialize when sending the delta source index and we need one extra value for "invalid" delta source 
	* So if changing this, set it to a PowerOfTwo - 1 to use all bits available for example: 3 will give index values of 0, 1, 2 and we will use 3 as the invalid value meaning we will have 4 values and 4 values fits int 2 bits */
	static constexpr int32 DeltaSourceBufferSize = 3;

	/** Max number of elements when replicating inputs and states. Example, latest input along with the last N inputs will be sent in an array to cover for packet loss.
	* See CVar: np2.Resim.RedundantInputs or np2.Resim.DynamicInputReplicationScaling.Enabled */
	static constexpr uint16 MaxNumberOfElementsToNetwork = 16;
}

namespace UE::Net
{
	FReplicationFragment* CreateAndRegisterNetworkPhysicsRewindDataProxyReplicationFragment(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);

	/** ReplicationFragment to handle assigning Owner to replicated properties based on FNetworkPhysicsRewindDataProxy
	* Info: A ReplicationFragment is an extension / override to the NetSerializer logic */
	class FNetworkPhysicsRewindDataProxyReplicationFragment : public FReplicationFragment
	{
	public:
		FNetworkPhysicsRewindDataProxyReplicationFragment(EReplicationFragmentTraits InTraits, UObject* InOwner, const FReplicationStateDescriptor* InDescriptor);

		void Register(FFragmentRegistrationContext& Context);

	protected:
		// Set replicated property as dirty if needed
		virtual bool PollReplicatedState(EReplicationFragmentPollFlags PollOption) override;
		// On receive, assign Owner to the target struct and then call Dequantize
		virtual void ApplyReplicatedState(FReplicationStateApplyContext& Context) const override;
		// Manually call OnRep_ functions
		virtual void CallRepNotifies(FReplicationStateApplyContext& Context) override;
		virtual void CollectOwner(FReplicationStateOwnerCollector* Owners) const override;

	private:
		TRefCountPtr<const FReplicationStateDescriptor> ReplicationStateDescriptor;
		TUniquePtr<FPropertyReplicationState> SrcReplicationState;
		UObject* Owner = nullptr;
	};
}


namespace UE::NetworkPhysicsUtils
{
	// Returns the index of the frame about to be simulated, in the server timeline
	ENGINE_API int32 GetUpcomingServerFrame_External(UWorld* World);

	// Returns the network physics frame offset on the game thread
	ENGINE_API const int32 GetNetworkPhysicsTickOffset_External(const UWorld* World);

	// Returns the network physics frame offset on the physics thread
	ENGINE_API const int32 GetNetworkPhysicsTickOffset_Internal(const Chaos::FPhysicsSolverBase* Solver);

	// Returns the owning UObject of a UObject
	ENGINE_API const UObject* GetOwnerFromUObject_External(const UObject* SourceObject);

	// Returns a network stable hash for a UObject, requires it to be either replicated or baked in the level, else returns 0
	ENGINE_API const uint32 GetNetworkStableHash_External(const UObject* InSourceObject);
}

/** Templated data history, holding a data buffer */
template<typename DataType, bool bLegacyData = false>
struct TNetRewindHistory : public Chaos::TDataRewindHistory<DataType>
{
	using Super = Chaos::TDataRewindHistory<DataType>;

	TNetRewindHistory(const int32 FrameCount, const bool bIsHistoryLocal) :
		Super(FrameCount, bIsHistoryLocal)
	{
	}

	TNetRewindHistory(const int32 FrameCount) :
		Super(FrameCount)
	{
	}

	virtual ~TNetRewindHistory() = default;

	virtual TUniquePtr<Chaos::FBaseRewindHistory> CreateNew() const
	{
		TUniquePtr<TNetRewindHistory> Copy = MakeUnique<TNetRewindHistory>(0, Super::bIsLocalHistory);

		return Copy;
	}

	virtual TUniquePtr<Chaos::FBaseRewindHistory> Clone() const
	{
		return MakeUnique<TNetRewindHistory>(*this);
	}

	virtual void ValidateDataInHistory(const void* ActorComponent) override
	{
		if constexpr (bLegacyData)
		{
			const UActorComponent* NetworkComponent = static_cast<const UActorComponent*>(ActorComponent);
			for (int32 FrameIndex = 0; FrameIndex < Super::NumFrames; ++FrameIndex)
			{
				DataType& FrameData = Super::DataHistory[FrameIndex];
				FrameData.ValidateData(NetworkComponent);
			}
		}
	}

	virtual int32 CountValidData(const uint32 StartFrame, const uint32 EndFrame, const bool bIncludeUnimportant = true, const bool bIncludeImportant = false) override
	{
		// Find how many entries are valid in frame range
		DataType FrameData;
		int32 Count = 0;
		for (uint32 Frame = StartFrame; Frame <= EndFrame; ++Frame)
		{
			const int32 Index = Super::GetFrameIndex(Frame);
			if (Frame == Super::DataHistory[Index].LocalFrame)
			{
				FrameData = Super::DataHistory[Index];

				// Check if we should include unimportant and/or important data
				if ((!FrameData.bImportant && bIncludeUnimportant) || (FrameData.bImportant && bIncludeImportant))
				{
					Count++;
				}
			}
		}
		return Count;
	}

	virtual int32 CountAlteredData(const bool bIncludeUnimportant = true, const bool bIncludeImportant = false) override
	{
		DataType FrameData;
		int32 Count = 0;
		for (int32 Index = 0; Index < Super::NumFrames; ++Index)
		{
			FrameData = Super::DataHistory[Index];

			// Check if we should include unimportant and/or important data
			if (FrameData.bDataAltered && ((!FrameData.bImportant && bIncludeUnimportant) || (FrameData.bImportant && bIncludeImportant)))
			{
				Count++;
			}
		}
		return Count;
	}

	virtual void SetImportant(const bool bImportant, const int32 Frame = INDEX_NONE) override
	{
		if (Frame > INDEX_NONE)
		{
			if (Super::EvalData(Frame))
			{
				// Set importance on specified frame
				Super::DataHistory[Super::CurrentIndex].bImportant = bImportant;
			}
		}
		else
		{
			// Set importance on all frames
			for (int32 Index = 0; Index < Super::NumFrames; ++Index)
			{
				Super::DataHistory[Index].bImportant = bImportant;
			}
		}

	}

	virtual void ApplyDataRange(const int32 FromFrame, const int32 ToFrame, void* ActorComponent, const bool bOnlyImportant = false) override
	{
		if constexpr (bLegacyData)
		{
			UActorComponent* NetworkComponent = static_cast<UActorComponent*>(ActorComponent);

			for (int32 ApplyFrame = FromFrame; ApplyFrame <= ToFrame; ++ApplyFrame)
			{
				const int32 ApplyIndex = Super::GetFrameIndex(ApplyFrame);
				DataType& FrameData = Super::DataHistory[ApplyIndex];
				if (ApplyFrame == FrameData.LocalFrame && (!bOnlyImportant || FrameData.bImportant))
				{
					FrameData.ApplyData(NetworkComponent);
				}
			}
		}
	}

	virtual bool CopyAllData(Chaos::FBaseRewindHistory& OutHistory, bool bIncludeUnimportant = true, bool bIncludeImportant = false) override
	{
		TNetRewindHistory& OutNetHistory = static_cast<TNetRewindHistory&>(OutHistory);
		bool bHasCopiedData = false;

		DataType FrameData;
		for (int32 CopyIndex = 0; CopyIndex < Super::NumFrames; ++CopyIndex)
		{
			FrameData = Super::DataHistory[CopyIndex];

			// Check if we should include unimportant and/or important data
			if ((!FrameData.bImportant && bIncludeUnimportant) || (FrameData.bImportant && bIncludeImportant))
			{
				OutNetHistory.RecordData(FrameData.LocalFrame, &FrameData);
				bHasCopiedData = true;
			}
		}
		return bHasCopiedData;
	}

	virtual bool CopyAlteredData(Chaos::FBaseRewindHistory& OutHistory, bool bIncludeUnimportant = true, bool bIncludeImportant = false) override
	{
		TNetRewindHistory& OutNetHistory = static_cast<TNetRewindHistory&>(OutHistory);
		bool bHasCopiedData = false;

		DataType FrameData;
		for (int32 CopyIndex = 0; CopyIndex < Super::NumFrames; ++CopyIndex)
		{
			FrameData = Super::DataHistory[CopyIndex];

			// Check if we should include unimportant and/or important data
			if (FrameData.bDataAltered && ((!FrameData.bImportant && bIncludeUnimportant) || (FrameData.bImportant && bIncludeImportant)))
			{
				OutNetHistory.RecordData(FrameData.LocalFrame, &FrameData);
				bHasCopiedData = true;
			}
		}
		return bHasCopiedData;
	}

	virtual bool CopyData(Chaos::FBaseRewindHistory& OutHistory, const uint32 StartFrame, const uint32 EndFrame, bool bIncludeUnimportant = true, bool bIncludeImportant = false) override
	{
		TNetRewindHistory& OutNetHistory = static_cast<TNetRewindHistory&>(OutHistory);
		bool bHasCopiedData = false;

		DataType FrameData;
		for (uint32 CopyFrame = StartFrame; CopyFrame <= EndFrame; ++CopyFrame)
		{
			const int32 CopyIndex = Super::GetFrameIndex(CopyFrame);
			if (CopyFrame == Super::DataHistory[CopyIndex].LocalFrame)
			{
				FrameData = Super::DataHistory[CopyIndex];

				// Check if we should include unimportant and/or important data
				if ((!FrameData.bImportant && bIncludeUnimportant) || (FrameData.bImportant && bIncludeImportant))
				{
					OutNetHistory.RecordData(CopyFrame, &FrameData);
					bHasCopiedData = true;
				}
			}
		}
		return bHasCopiedData;
	}

	virtual TUniquePtr<Chaos::FBaseRewindHistory> CopyFramesWithOffset(const uint32 StartFrame, const uint32 EndFrame, const int32 FrameOffset) override
	{
		uint32 FramesCount = (uint32)Super::NumValidData(StartFrame, EndFrame);

		TUniquePtr<TNetRewindHistory> Copy = MakeUnique<TNetRewindHistory>(FramesCount, Super::bIsLocalHistory);

		DataType FrameData;
		for (uint32 CopyFrame = StartFrame; CopyFrame < EndFrame; ++CopyFrame)
		{
			const int32 CopyIndex = Super::GetFrameIndex(CopyFrame);
			if (CopyFrame == Super::DataHistory[CopyIndex].LocalFrame)
			{
				FrameData = Super::DataHistory[CopyIndex];
				FrameData.ServerFrame = FrameData.LocalFrame + FrameOffset;
				Copy->RecordData(CopyFrame, &FrameData);
			}
		}

		return Copy;
	}

	virtual int32 ReceiveNewData(Chaos::FBaseRewindHistory& NewData, const int32 FrameOffset, bool CompareDataForRewind = false, const bool bImportant = false, int32 TryInjectAtFrame = INDEX_NONE) override
	{
		TNetRewindHistory& NetNewData = static_cast<TNetRewindHistory&>(NewData);

		int32 RewindFrame = INDEX_NONE;
		DataType* ReceiveData = &NetNewData.GetAndLoadEarliestData();

		for (int32 Itr = 0; Itr < NetNewData.GetHistorySize(); Itr++)
		{
			DataType* NextReceiveData = NetNewData.GetAndLoadNextIncrementalData();

			ReceiveData->bImportant = bImportant;
			ReceiveData->bReceivedData = true; // Received data is marked to differentiate from locally predicted data

			ReceiveData->LocalFrame = ReceiveData->ServerFrame - FrameOffset;

			if (ShouldRecordReceivedDataOnFrame(*ReceiveData, NextReceiveData))
			{
				if (CompareDataForRewind && ReceiveData->LocalFrame > RewindFrame && TriggerRewindFromNewData(*ReceiveData))
				{
					RewindFrame = ReceiveData->LocalFrame;
				}

				Super::RecordData(ReceiveData->LocalFrame, ReceiveData);
			}

			if (NextReceiveData)
			{
				ReceiveData = NextReceiveData;
			}
			else
			{
				// Record a copy of the last data at specified frame if the history doesn't have data for that frame yet
				if (TryInjectAtFrame > Super::GetLatestFrame())
				{
#if DEBUG_NETWORK_PHYSICS
					UE_LOGF(LogChaos, Log, "SERVER | PT | Input Buffer Empty, Injecting Received Input at frame %d || LocalFrame = %d || ServerFrame = %d || bDataAltered = %d || Data: %ls"
						, TryInjectAtFrame, ReceiveData.LocalFrame, ReceiveData.ServerFrame, ReceiveData.bDataAltered, *ReceiveData.DebugData());
#endif

					DataType InjectData = *ReceiveData;
					InjectData.bReceivedData = false;
					InjectData.bDataAltered = true;
					InjectData.LocalFrame = TryInjectAtFrame;
					InjectData.ServerFrame = TryInjectAtFrame + FrameOffset;

					Super::RecordData(TryInjectAtFrame, &InjectData);
				}

				break;
			}
		}

		return RewindFrame;
	}

	/** Check if we should record received data into history.
	* Can for example block received data from client from overriding server authoritative data */
	virtual bool ShouldRecordReceivedDataOnFrame(const DataType& ReceivedData, DataType* NextReceivedData = nullptr)
	{
		if (ReceivedData.LocalFrame < 0)
		{
			return false;
		}

		// Get the cached data at the index slot for the received data 
		Super::LoadData(ReceivedData.LocalFrame);
		DataType& Data = Super::DataHistory[Super::CurrentIndex];

		if (Data.LocalFrame < ReceivedData.LocalFrame)
		{
			// Allow recording the received data if it's newer than the already recorded data on this index
			return true;
		}
		else if (!Data.bReceivedData && Data.LocalFrame == ReceivedData.LocalFrame)
		{
			// If the data exist but is not marked as received and marked as altered, the server has produced this input via extrapolation or interpolation, don't overwrite it with data from the client.
			if (Data.bDataAltered)
			{
				// If we have a newer received data (since we receive multiple data at the same time) merge this data into the newer which can either get recorded or injected at the front of the input buffer on the server.
				if (NextReceivedData != nullptr)
				{
					NextReceivedData->MergeData(ReceivedData);
					NextReceivedData->bDataAltered = true;
				}

				// Mark the already cached data as received so that we don't perform this merge again when receiving redundant inputs / states
				Data.bReceivedData = true;
				return false;
			}
			else
			{
				// Allow recording the received data if we already have data for the same frame cached, not marked as altered, meaning it was predicted data on the client
				return true;
			}
		}
		return false;
	}

	/** Compares new received data with local predicted data and returns true if they differ enough to trigger a resimulation  */
	virtual bool TriggerRewindFromNewData(DataType& NewData)
	{
		if (Super::EvalData(NewData.LocalFrame) && !Super::DataHistory[Super::CurrentIndex].bReceivedData)
		{
			return !NewData.CompareData(Super::DataHistory[Super::CurrentIndex]);
		}

		return false;
	}

	UE_DEPRECATED(5.6, "Deprecated, use the NetSerialize call with parameter that takes a function DataSetupFunction, pass in nullptr to opt out of implementing a function.")
	virtual void NetSerialize(FArchive& Ar, UPackageMap* InPackageMap) override
	{
		NetSerialize(Ar, InPackageMap, [](void* Data, const int32 DataIndex) {});
	}

	virtual void NetSerialize(FArchive& Ar, UPackageMap* InPackageMap, TUniqueFunction<void(void* Data, const int32 DataIndex)> DataSetupFunction) override
	{
		bool bOneEntry = Super::NumFrames == 1;
		if (Ar.IsSaving())
		{
			Ar.SerializeBits(&bOneEntry, 1);

			if (!bOneEntry)
			{
				uint32 NumFramesUnsigned = static_cast<uint32>(Super::NumFrames);

				constexpr uint32 NumBitsNumFramesUnsigned = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork);
				Ar.SerializeBits(&NumFramesUnsigned, NumBitsNumFramesUnsigned);

				Super::NumFrames = static_cast<int32>(NumFramesUnsigned);
			}
			else
			{
				Super::NumFrames = 1;
			}

			if (Super::NumFrames > GetMaxArraySize() || Super::NumFrames <= 0)
			{
				UE_LOGF(LogChaos, Warning, "TNetRewindHistory: serialized array of size %d exceeds maximum size %d.", Super::NumFrames, GetMaxArraySize());
				Ar.SetError();
				return;
			}

			// Note that we use a for-loop to iterate over all entries instead of relying on GetAndLoadNextIncrementalData because the data can have Frame 0 which makes incremental not work.
			DataType* LastData = nullptr;
			DataType* CurrentData = &Super::GetAndLoadEarliestData();
			for (int32 Itr = 0; Itr < Super::GetHistorySize(); Itr++)
			{
				// Set the implementation component pointer and stateful delta serialization source
				if (DataSetupFunction)
				{
					DataSetupFunction(CurrentData, Itr);
				}

				// Set the internal delta serialization source (between data entries in the collection)
				if constexpr (bLegacyData)
				{
					if (LastData)
					{
						CurrentData->SetDeltaSourceData(LastData);
					}
				}

				NetSerializeData(*CurrentData, Ar, InPackageMap);

				// Get next incremental data
				if (DataType* NextData = Super::GetAndLoadNextData())
				{
					LastData = CurrentData;
					CurrentData = NextData;
				}
				else
				{
					ensureMsgf(false, TEXT("TNetRewindHistory: Serialization failed to get the next data."));
				}
			}

			for (DataType& Data : Super::DataHistory)
			{
				// Clear delta source and strong pointer to implementation component after serialization
				if constexpr (bLegacyData)
				{
					Data.ClearImplementationComponent();
					Data.ClearDeltaSourceData();
				}
			}
		}
		else // IsLoading
		{
			Ar.SerializeBits(&bOneEntry, 1);

			if (!bOneEntry)
			{
				uint32 NumFramesUnsigned = static_cast<uint32>(Super::NumFrames);

				constexpr uint32 NumBitsNumFramesUnsigned = FMath::CeilLogTwo(NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork);
				Ar.SerializeBits(&NumFramesUnsigned, NumBitsNumFramesUnsigned);

				Super::NumFrames = static_cast<int32>(NumFramesUnsigned);
			}
			else
			{
				Super::NumFrames = 1;
			}

			if (Super::NumFrames > GetMaxArraySize())
			{
				UE_LOGF(LogChaos, Warning, "TNetRewindHistory: serialized array of size %d exceeds maximum size %d.", Super::NumFrames, GetMaxArraySize());
				Ar.SetError();
				return;
			}

			Super::DataHistory.SetNum(Super::NumFrames);

			for (int32 I = 0; I < Super::DataHistory.Num(); I++)
			{
				DataType& Data = Super::DataHistory[I];

				// Set the implementation component pointer and stateful delta serialization source
				if (DataSetupFunction)
				{
					DataSetupFunction(&Data, I);
				}

				// Set the internal delta serialization source (between data entries in the collection)
				if constexpr (bLegacyData)
				{
					if (I > 0)
					{
						Data.SetDeltaSourceData(&Super::DataHistory[I - 1]);
					}
				}

				NetSerializeData(Data, Ar, InPackageMap);
			}

			for (DataType& Data : Super::DataHistory)
			{
				// Clear delta source and strong pointer to implementation component after serialization
				if constexpr (bLegacyData)
				{
					Data.ClearImplementationComponent();
					Data.ClearDeltaSourceData();
				}
			}
		}

		Super::Initialize();
	}

	/** Debug the data from the archive */
	virtual void DebugData(const Chaos::FBaseRewindHistory& DebugHistory, TArray<int32>& LocalFrames, TArray<int32>& ServerFrames, TArray<int32>& InputFrames) override
	{
		const TNetRewindHistory& NetDebugHistory = static_cast<const TNetRewindHistory&>(DebugHistory);

		if(NetDebugHistory.NumFrames >= 0)
		{
			LocalFrames.SetNum(NetDebugHistory.NumFrames);
			ServerFrames.SetNum(NetDebugHistory.NumFrames);
			InputFrames.SetNum(NetDebugHistory.NumFrames);

			DataType FrameData;
			for (int32 FrameIndex = 0; FrameIndex < NetDebugHistory.NumFrames; ++FrameIndex)
			{
				FrameData = NetDebugHistory.DataHistory[FrameIndex];
				LocalFrames[FrameIndex] = FrameData.LocalFrame;
				ServerFrames[FrameIndex] = FrameData.ServerFrame;
				InputFrames[FrameIndex] = FrameData.bDataAltered ? 1 : 0; // For now we show the altered state inside the InputFrames array, since that was the main usecase for it when it was implemented
			}
		}
	}

	/** Print custom string along with values for each entry in history */
	virtual void DebugData(const FString& DebugText) override
	{
		UE_LOGF(LogChaos, Log, "%ls", *DebugText);
		UE_LOGF(LogChaos, Log, "	NumFrames in data collection: %d", Super::NumFrames);

		if (Super::NumFrames >= 0)
		{
			for (int32 FrameIndex = 0; FrameIndex < Super::NumFrames; ++FrameIndex)
			{
				UE_LOGF(LogChaos, Log, "		Index: %d || LocalFrame = %d || ServerFrame = %d || bDataAltered = %d || bReceivedData = %d || bImportant = %d  ||  Data: %ls"
				, FrameIndex
				, Super::DataHistory[FrameIndex].LocalFrame
				, Super::DataHistory[FrameIndex].ServerFrame
				, Super::DataHistory[FrameIndex].bDataAltered
				, Super::DataHistory[FrameIndex].bReceivedData
				, Super::DataHistory[FrameIndex].bImportant
				, *Super::DataHistory[FrameIndex].DebugData());
			}
		}
	}

private :

	/** Serialized array size limit to guard against invalid network data */
	static int32 GetMaxArraySize()
	{
		static int32 MaxArraySize = UPhysicsSettings::Get()->GetPhysicsHistoryCount() * 4;
		return MaxArraySize;
	}

	/** Use net serialize path to serialize data  */
	bool NetSerializeData(DataType& FrameData, FArchive& Ar, UPackageMap* PackageMap) const 
	{
		bool bOutSuccess = false;
		UScriptStruct* ScriptStruct = DataType::StaticStruct();
		if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
		{
			ScriptStruct->GetCppStructOps()->NetSerialize(Ar, PackageMap, bOutSuccess, &FrameData);
		}
		else
		{
			UE_LOGF(LogChaos, Error, "TNetRewindHistory::NetSerializeData called on data struct %ls without a native NetSerialize", *ScriptStruct->GetName());

			// Not working for now since the packagemap could be null
			// UNetConnection* Connection = CastChecked<UPackageMapClient>(PackageMap)->GetConnection();
			// UNetDriver* NetDriver = Connection ? Connection->GetDriver() : nullptr;
			// TSharedPtr<FRepLayout> RepLayout = NetDriver ? NetDriver->GetStructRepLayout(ScriptStruct) : nullptr;
			//
			// if (RepLayout.IsValid())
			// {
			// 	bool bHasUnmapped = false;
			// 	RepLayout->SerializePropertiesForStruct(ScriptStruct, Ar, PackageMap, &FrameData, bHasUnmapped);
			//
			// 	bOutSuccess = true;
			// }
		}
		return bOutSuccess;
	}
};


/**
*	Start Implementing Action Structs and helpers
* 
*	Action "history" is setup like this:
*	FActionHistory - Circular frame buffer
*		Frame 1 - FActionHistoryFrameData
*			Action 1 - FActionHistoryEntry
*				FActionInstance ActionInstance
*			Action 2 - FActionHistoryEntry
*				FActionInstance ActionInstance
*		Frame 2 - FActionHistoryFrameData
*			Action 1 - FActionHistoryEntry
*				FActionInstance ActionInstance
*/

USTRUCT()
struct FNetworkPhysicsActionPayload
{
	GENERATED_BODY()

	virtual ~FNetworkPhysicsActionPayload() = default;

	enum EActionAuthorStyle : uint8
	{
		Authority = 0, // Performed server-side, replicated and applied to clients
		Predicted = 1, // Predicted and applied client-side, performed server-side, replicated to clients where the predicted action gets overwritten
		PredictedAutonomousOnly = 2, // Predicted and applied on the autonomous client, performed server-side, replicated to clients where the predicted action gets overwritten
		Proposed = 3, // Predicted and applied client-side, replicated to server, performed server-side and compared with the proposed action from the autonomous proxy and if close enough the server uses the clients proposed action, then replicated to where the proposed action gets overwritten
		ProposedAutonomousOnly = 4 // Predicted and applied on the autonomous client, replicated to server, performed server-side and compared with the proposed action from the autonomous proxy and if close enough the server uses the clients proposed action, then replicated to where the proposed action gets overwritten
	};
	virtual const EActionAuthorStyle GetAuthorStyle() const { return EActionAuthorStyle::Authority; };

	UPROPERTY()
	int32 ServerFrame = INDEX_NONE;

	UPROPERTY()
	uint32 SourceId = 0u;

	UPROPERTY()
	uint16 ActionId = 0u;

	virtual bool IsNearlyEqual(const FNetworkPhysicsActionPayload& Other) const
	{
		ensureMsgf(false, TEXT("FNetworkPhysicsActionPayload::IsNearlyEqual is not overridden for an action."));
		return false;
	}

	virtual void DebugAction() const { UE_LOGF(LogChaos, Log, "		DEBUG || ActionPayload || DebugAction() Not Implemented!"); }
};

// Struct holding the Action Instance along with helper data
struct FActionHistoryEntry
{
	using FActionInstance = TInstancedStruct<FNetworkPhysicsActionPayload>;

	FActionInstance ActionInstance;
	bool bActive = false;
	bool bApplied = false;
	bool bHasBeenQueuedForReplication = false;
	bool bReliable = false;
	bool bClaimedThisResim = false; // Transient, cleared at the start of each resim pass.

	void Reset()
	{
		ActionInstance.Reset();
		bActive = false;
		bApplied = false;
		bHasBeenQueuedForReplication = false;
		bReliable = false;
		bClaimedThisResim = false;
	}
};

// Struct holding actions per-frame in FActionHistory
struct FActionHistoryFrameData
{
	int32 LocalFrame = INDEX_NONE;
	TArray<FActionHistoryEntry> Entries;

	void ResetForFrame(const int32 InLocalFrame)
	{
		LocalFrame = InLocalFrame;
		Entries.Reset();
	}

	bool IsForFrame(const int32 InLocalFrame) const
	{
		return LocalFrame == InLocalFrame;
	}
};

// Struct that points to a specific action in FActionHistory
struct FActionHistoryLocation
{
	int32 FrameIndex = INDEX_NONE;
	int32 EntryIndex = INDEX_NONE;

	bool IsValid() const
	{
		return FrameIndex != INDEX_NONE && EntryIndex != INDEX_NONE;
	}
};

struct FActionHistory
{
	using FActionInstance = TInstancedStruct<FNetworkPhysicsActionPayload>;

	int32 HistorySize = 0;
	TArray<FActionHistoryFrameData> FrameHistory;

	void Initialize(const int32 InHistorySize)
	{
		HistorySize = FMath::Max(1, InHistorySize);
		FrameHistory.SetNum(HistorySize);

		for (FActionHistoryFrameData& FrameData : FrameHistory)
		{
			FrameData.ResetForFrame(INDEX_NONE);
		}
	}

	void Reset()
	{
		for (FActionHistoryFrameData& FrameData : FrameHistory)
		{
			FrameData.ResetForFrame(INDEX_NONE);
		}
	}

	bool IsInitialized() const
	{
		return HistorySize > 0 && FrameHistory.Num() == HistorySize;
	}

	int32 GetFrameIndex(const int32 LocalFrame) const
	{
		check(IsInitialized());

		int32 FrameIndex = LocalFrame % HistorySize;
		if (FrameIndex < 0)
		{
			FrameIndex += HistorySize;
		}

		return FrameIndex;
	}

	FActionHistoryFrameData* FindFrameData(const int32 LocalFrame)
	{
		if (IsInitialized() == false)
		{
			return nullptr;
		}

		FActionHistoryFrameData& FrameData = FrameHistory[GetFrameIndex(LocalFrame)];
		if (FrameData.IsForFrame(LocalFrame))
		{
			return &FrameData;
		}

		return nullptr;
	}

	const FActionHistoryFrameData* FindFrameData(const int32 LocalFrame) const
	{
		if (IsInitialized() == false)
		{
			return nullptr;
		}

		const FActionHistoryFrameData& FrameData = FrameHistory[GetFrameIndex(LocalFrame)];
		if (FrameData.IsForFrame(LocalFrame))
		{
			return &FrameData;
		}

		return nullptr;
	}

	FActionHistoryFrameData& FindOrAddFrameData(const int32 LocalFrame)
	{
		check(IsInitialized());

		FActionHistoryFrameData& FrameData = FrameHistory[GetFrameIndex(LocalFrame)];
		if (FrameData.IsForFrame(LocalFrame) == false)
		{
			// Override old entry in circle buffer with new frame
			FrameData.ResetForFrame(LocalFrame);
		}

		return FrameData;
	}

	/**
	*	Start static helper functions
	*/
	static const FNetworkPhysicsActionPayload& GetRef(const FActionInstance& ActionInstance)
	{
		return ActionInstance.Get<FNetworkPhysicsActionPayload>();
	}

	static FNetworkPhysicsActionPayload& GetRefMutable(FActionInstance& ActionInstance)
	{
		return ActionInstance.GetMutable<FNetworkPhysicsActionPayload>();
	}

	/* Check if active, not applied and valid.
	* @param bAllowApplied - If false, return false if action has been applied. */
	static bool IsUsableEntry(const FActionHistoryEntry& Entry, const bool bAllowApplied)
	{
		if (Entry.bActive == false)
		{
			return false;
		}

		// Check if we care about the entry already being applied or not, for example during resimulation we allow re-application of actions
		if (bAllowApplied == false && Entry.bApplied == true)
		{
			return false;
		}

		return Entry.ActionInstance.IsValid();
	}

	static bool SameConcreteType(const FActionInstance& AActionInstance, const FActionInstance& BActionInstance)
	{
		if (AActionInstance.IsValid() == false || BActionInstance.IsValid() == false)
		{
			return false;
		}

		return AActionInstance.GetScriptStruct() == BActionInstance.GetScriptStruct();
	}

	static bool HasSameValidActionId(const FActionInstance& AActionInstance, const FActionInstance& BActionInstance)
	{
		const FNetworkPhysicsActionPayload& AAction = GetRef(AActionInstance);
		const FNetworkPhysicsActionPayload& BAction = GetRef(BActionInstance);

		if (AAction.ActionId == 0u)
		{
			return false;
		}

		return AAction.ActionId == BAction.ActionId;
	}

	// Check if two actions are of the same type and produced by the same source
	static bool AreSourceEquivalent(const FActionInstance& ExistingActionInstance, const FActionInstance& CandidateActionInstance)
	{
		if (SameConcreteType(ExistingActionInstance, CandidateActionInstance) == false)
		{
			return false;
		}

		const FNetworkPhysicsActionPayload& ExistingAction = GetRef(ExistingActionInstance);
		const FNetworkPhysicsActionPayload& CandidateAction = GetRef(CandidateActionInstance);

		return ExistingAction.SourceId == CandidateAction.SourceId;
	}

	// Check if two actions are equal enough to be considered the same action
	static bool AreEquivalent(const FActionInstance& ExistingActionInstance, const FActionInstance& CandidateActionInstance)
	{
		if (SameConcreteType(ExistingActionInstance, CandidateActionInstance) == false)
		{
			return false;
		}

		const FNetworkPhysicsActionPayload& ExistingAction = GetRef(ExistingActionInstance);
		const FNetworkPhysicsActionPayload& CandidateAction = GetRef(CandidateActionInstance);

		if (ExistingAction.SourceId != CandidateAction.SourceId)
		{
			return false;
		}

		return ExistingAction.IsNearlyEqual(CandidateAction);
	}

	/**
	*	End static helper functions
	*/


	FActionInstance MakeCopy(const FActionInstance& ActionInstance, const int32 LocalFrame, const int32 TickOffset, const uint16 OverrideActionId = 0u, const bool bClearActionId = false) const
	{
		FActionInstance ActionInstanceCopy = ActionInstance;
		FNetworkPhysicsActionPayload& ActionCopy = GetRefMutable(ActionInstanceCopy);

		// If no frame number is set, apply LocalFrame converted into ServerFrame (i.e. the Action will be applied ASAP)
		if (ActionCopy.ServerFrame == INDEX_NONE)
		{
			ActionCopy.ServerFrame = LocalFrame + TickOffset;
		}

		if (OverrideActionId != 0u)
		{
			ActionCopy.ActionId = OverrideActionId;
		}
		else if (bClearActionId)
		{
			ActionCopy.ActionId = 0u;
		}

		return ActionInstanceCopy;
	}

	FActionHistoryLocation FindActiveActionById(const uint16 ActionId, const bool bIncludeApplied = false) const
	{
		FActionHistoryLocation FoundLocation;

		if (IsInitialized() == false || ActionId == 0u)
		{
			return FoundLocation;
		}

		for (int32 FrameIndex = 0; FrameIndex < FrameHistory.Num(); ++FrameIndex)
		{
			const FActionHistoryFrameData& FrameData = FrameHistory[FrameIndex];
			if (FrameData.LocalFrame == INDEX_NONE)
			{
				continue;
			}

			for (int32 EntryIndex = 0; EntryIndex < FrameData.Entries.Num(); ++EntryIndex)
			{
				const FActionHistoryEntry& Entry = FrameData.Entries[EntryIndex];
				if (IsUsableEntry(Entry, bIncludeApplied) == false)
				{
					continue;
				}

				if (GetRef(Entry.ActionInstance).ActionId == ActionId)
				{
					FoundLocation.FrameIndex = FrameIndex;
					FoundLocation.EntryIndex = EntryIndex;
					return FoundLocation;
				}
			}
		}

		return FoundLocation;
	}

	FActionHistoryLocation FindEquivalentActionInRange(const FActionInstance& CandidateActionInstance, const int32 MinLocalFrame, const int32 MaxLocalFrame, const bool bIncludeApplied = false, const bool bCheckSourceEquivalent = false, const bool bSkipClaimedThisResim = false) const
	{
		FActionHistoryLocation FoundLocation;

		if (IsInitialized() == false || CandidateActionInstance.IsValid() == false)
		{
			return FoundLocation;
		}

		const int32 StartLocalFrame = FMath::Min(MinLocalFrame, MaxLocalFrame);
		const int32 EndLocalFrame = FMath::Max(MinLocalFrame, MaxLocalFrame);

		for (int32 QueryLocalFrame = StartLocalFrame; QueryLocalFrame <= EndLocalFrame; ++QueryLocalFrame)
		{
			const FActionHistoryFrameData* FrameData = FindFrameData(QueryLocalFrame);
			if (FrameData == nullptr)
			{
				continue;
			}

			for (int32 EntryIndex = 0; EntryIndex < FrameData->Entries.Num(); ++EntryIndex)
			{
				const FActionHistoryEntry& Entry = FrameData->Entries[EntryIndex];
				if (IsUsableEntry(Entry, bIncludeApplied) == false)
				{
					continue;
				}

				if (bSkipClaimedThisResim && Entry.bClaimedThisResim)
				{
					continue;
				}

				if (bCheckSourceEquivalent)
				{
					if (AreSourceEquivalent(Entry.ActionInstance, CandidateActionInstance))
					{
						FoundLocation.FrameIndex = GetFrameIndex(QueryLocalFrame);
						FoundLocation.EntryIndex = EntryIndex;
						return FoundLocation;
					}
				}
				else
				{
					if (AreEquivalent(Entry.ActionInstance, CandidateActionInstance))
					{
						FoundLocation.FrameIndex = GetFrameIndex(QueryLocalFrame);
						FoundLocation.EntryIndex = EntryIndex;
						return FoundLocation;
					}
				}
			}
		}

		return FoundLocation;
	}

	bool HasEquivalentActionInRange(const FActionInstance& CandidateActionInstance, const int32 MinLocalFrame, const int32 MaxLocalFrame, const bool bIncludeApplied = false, const bool bCheckSourceEquivalent = false, const bool bSkipClaimedThisResim = false) const
	{
		return FindEquivalentActionInRange(CandidateActionInstance, MinLocalFrame, MaxLocalFrame, bIncludeApplied, bCheckSourceEquivalent, bSkipClaimedThisResim).IsValid();
	}

	bool AddAction(const FActionInstance& ActionInstance, const int32 LocalFrame, const int32 TickOffset
		, const uint16 OverrideActionId = 0u, const bool bClearActionId = false, const bool bInReliable = false)
	{
		if (IsInitialized() == false || ActionInstance.IsValid() == false)
		{
			return false;
		}

		FActionInstance ActionInstanceCopy = MakeCopy(ActionInstance, LocalFrame, TickOffset, OverrideActionId, bClearActionId);
		const FNetworkPhysicsActionPayload& ActionCopy = GetRef(ActionInstanceCopy);

		// If we already have registered an action with this ActionId, early out. We might have received the same replicated action twice in a row.
		if (ActionCopy.ActionId != 0u && FindActiveActionById(ActionCopy.ActionId, true).IsValid())
		{
			return false;
		}

		FActionHistoryFrameData& FrameData = FindOrAddFrameData(LocalFrame);

		FActionHistoryEntry& NewEntry = FrameData.Entries.Emplace_GetRef();
		NewEntry.ActionInstance = MoveTemp(ActionInstanceCopy);
		NewEntry.bActive = true;
		NewEntry.bApplied = false;
		NewEntry.bHasBeenQueuedForReplication = false;
		NewEntry.bReliable = bInReliable;
		NewEntry.bClaimedThisResim = false;
		return true;
	}

	// Reset bClaimedThisResim on every entry. Call once per resim pass.
	void ClearResimClaims()
	{
		if (IsInitialized() == false)
		{
			return;
		}

		for (FActionHistoryFrameData& FrameData : FrameHistory)
		{
			for (FActionHistoryEntry& Entry : FrameData.Entries)
			{
				Entry.bClaimedThisResim = false;
			}
		}
	}

	// Mark the entry at Location as claimed
	bool MarkClaimed(const FActionHistoryLocation& Location)
	{
		if (Location.IsValid() == false)
		{
			return false;
		}

		if (FrameHistory.IsValidIndex(Location.FrameIndex) == false)
		{
			return false;
		}

		FActionHistoryFrameData& FrameData = FrameHistory[Location.FrameIndex];
		if (FrameData.Entries.IsValidIndex(Location.EntryIndex) == false)
		{
			return false;
		}

		FrameData.Entries[Location.EntryIndex].bClaimedThisResim = true;
		return true;
	}

	const FActionInstance* GetAction(const FActionHistoryLocation& Location) const
	{
		if (Location.IsValid() == false)
		{
			return nullptr;
		}

		if (FrameHistory.IsValidIndex(Location.FrameIndex) == false)
		{
			return nullptr;
		}

		const FActionHistoryFrameData& FrameData = FrameHistory[Location.FrameIndex];

		if (FrameData.Entries.IsValidIndex(Location.EntryIndex) == false)
		{
			return nullptr;
		}

		const FActionHistoryEntry& Entry = FrameData.Entries[Location.EntryIndex];

		if (Entry.ActionInstance.IsValid() == false)
		{
			return nullptr;
		}

		return &Entry.ActionInstance;
	}

	int32 GetLocalFrame(const FActionHistoryLocation& Location) const
	{
		if (Location.IsValid() == false)
		{
			return INDEX_NONE;
		}

		if (FrameHistory.IsValidIndex(Location.FrameIndex) == false)
		{
			return INDEX_NONE;
		}

		return FrameHistory[Location.FrameIndex].LocalFrame;
	}

	bool Deactivate(const FActionHistoryLocation& Location)
	{
		if (Location.IsValid() == false)
		{
			return false;
		}

		if (FrameHistory.IsValidIndex(Location.FrameIndex) == false)
		{
			return false;
		}

		FActionHistoryFrameData& FrameData = FrameHistory[Location.FrameIndex];

		if (FrameData.Entries.IsValidIndex(Location.EntryIndex) == false)
		{
			return false;
		}

		FrameData.Entries[Location.EntryIndex].bActive = false;
		return true;
	}

	bool DeactivateAllByActionId(const uint16 ActionId)
	{
		bool bDeactivatedAny = false;

		if (IsInitialized() == false || ActionId == 0u)
		{
			return false;
		}

		for (FActionHistoryFrameData& FrameData : FrameHistory)
		{
			if (FrameData.LocalFrame == INDEX_NONE)
			{
				continue;
			}

			for (FActionHistoryEntry& Entry : FrameData.Entries)
			{
				if (Entry.ActionInstance.IsValid() == false)
				{
					continue;
				}

				if (GetRef(Entry.ActionInstance).ActionId == ActionId)
				{
					Entry.bActive = false;
					bDeactivatedAny = true;
				}
			}
		}

		return bDeactivatedAny;
	}

	bool DeactivateEquivalentActionsInRange(const FActionInstance& CandidateActionInstance, const int32 MinLocalFrame, const int32 MaxLocalFrame, const bool bCheckSourceEquivalent = false)
	{
		bool bDeactivatedAny = false;

		if (IsInitialized() == false || CandidateActionInstance.IsValid() == false)
		{
			return false;
		}

		const int32 StartLocalFrame = FMath::Min(MinLocalFrame, MaxLocalFrame);
		const int32 EndLocalFrame = FMath::Max(MinLocalFrame, MaxLocalFrame);
		const uint16 CandidateActionId = GetRef(CandidateActionInstance).ActionId;

		for (int32 QueryLocalFrame = StartLocalFrame; QueryLocalFrame <= EndLocalFrame; ++QueryLocalFrame)
		{
			FActionHistoryFrameData* FrameData = FindFrameData(QueryLocalFrame);
			if (FrameData == nullptr)
			{
				continue;
			}

			for (FActionHistoryEntry& Entry : FrameData->Entries)
			{
				if (IsUsableEntry(Entry, /*bAllowApplied*/false) == false)
				{
					continue;
				}

				bool bMatches = CandidateActionId != 0u && GetRef(Entry.ActionInstance).ActionId == CandidateActionId;
				if (!bMatches)
				{
					bMatches = bCheckSourceEquivalent
						? AreSourceEquivalent(Entry.ActionInstance, CandidateActionInstance)
						: AreEquivalent(Entry.ActionInstance, CandidateActionInstance);
				}

				if (bMatches)
				{
					Entry.bActive = false;
					bDeactivatedAny = true;
				}
			}
		}

		return bDeactivatedAny;
	}

	/** Split actions to replicate into unreliable and reliable output arrays based on FActionHistoryEntry::bReliable */
	void GetActionsToReplicate(TArray<FActionInstance>& OutUnreliableActions, TArray<FActionInstance>& OutReliableActions, const bool bResetOutArray = true, const bool bMarkAsQueuedForReplication = true, const bool bOnlyActiveActions = true)
	{
		if (IsInitialized() == false)
		{
			return;
		}

		if (bResetOutArray == true)
		{
			OutUnreliableActions.Reset();
			OutReliableActions.Reset();
		}

		struct FPendingReplicationLocation
		{
			int32 FrameIndex = INDEX_NONE;
			int32 EntryIndex = INDEX_NONE;
			int32 LocalFrame = INDEX_NONE;
		};

		TArray<FPendingReplicationLocation, TInlineAllocator<16>> PendingReplicationLocations;

		for (int32 FrameIndex = 0; FrameIndex < FrameHistory.Num(); ++FrameIndex)
		{
			const FActionHistoryFrameData& FrameData = FrameHistory[FrameIndex];

			if (FrameData.LocalFrame == INDEX_NONE)
			{
				continue;
			}

			for (int32 EntryIndex = 0; EntryIndex < FrameData.Entries.Num(); ++EntryIndex)
			{
				const FActionHistoryEntry& Entry = FrameData.Entries[EntryIndex];

				if (Entry.bHasBeenQueuedForReplication == true)
				{
					continue;
				}
				if (bOnlyActiveActions == true && Entry.bActive == false)
				{
					continue;
				}
				if (Entry.ActionInstance.IsValid() == false)
				{
					continue;
				}

				FPendingReplicationLocation& Location = PendingReplicationLocations.Emplace_GetRef();
				Location.FrameIndex = FrameIndex;
				Location.EntryIndex = EntryIndex;
				Location.LocalFrame = FrameData.LocalFrame;
			}
		}

		// The circular buffer storage order is not chronological, sort by local frame, this makes delta serialization work better when multiple actions are being replicated
		PendingReplicationLocations.Sort([](const FPendingReplicationLocation& A, const FPendingReplicationLocation& B)
			{
				if (A.LocalFrame != B.LocalFrame)
				{
					return A.LocalFrame < B.LocalFrame;
				}
				if (A.FrameIndex != B.FrameIndex)
				{
					return A.FrameIndex < B.FrameIndex;
				}
				return A.EntryIndex < B.EntryIndex;
			});

		for (const FPendingReplicationLocation& Location : PendingReplicationLocations)
		{
			FActionHistoryFrameData& FrameData = FrameHistory[Location.FrameIndex];
			FActionHistoryEntry& Entry = FrameData.Entries[Location.EntryIndex];

			if (Entry.bReliable)
			{
				OutReliableActions.Add(Entry.ActionInstance);
			}
			else
			{
				OutUnreliableActions.Add(Entry.ActionInstance);
			}

			if (bMarkAsQueuedForReplication == true)
			{
				Entry.bHasBeenQueuedForReplication = true;
			}
		}
	}
};

/**
* End Implementing Action Structs and helpers
*/


/**
 * Base struct for replicated rewind history properties
 */
USTRUCT()
struct FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()
	virtual ~FNetworkPhysicsRewindDataProxy(){};
	FNetworkPhysicsRewindDataProxy& operator=(const FNetworkPhysicsRewindDataProxy& Other);

	/** Causes the history to be serialized every time. If implemented, would prevent serializing if the history hasn't changed. */
	bool operator==(const FNetworkPhysicsRewindDataProxy& Other) const { return false; }

protected:
	virtual bool NetSerializeBase(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess, TUniqueFunction<TUniquePtr<Chaos::FBaseRewindHistory>()> CreateHistoryFunction, TUniqueFunction<const uint32()> GetLatestDeltaSourceIndex, TUniqueFunction<FNetworkPhysicsData*(const int32)> GetDeltaSourceData);

public:
	/** The history to be serialized */
	TUniquePtr<Chaos::FBaseRewindHistory> History;

	/** Component that utilizes this data */
	UPROPERTY()
	TObjectPtr<UNetworkPhysicsComponent> Owner = nullptr;

	/** If a delta serialization issue was detected, i.e. the data might be corrupt if this is true*/
	bool bDeltaSerializationIssue = false;
};

/**
 * Base struct for replicated rewind history RPCs
 */
USTRUCT()
struct FNetworkPhysicsRewindDataProxyRPC : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()
	virtual ~FNetworkPhysicsRewindDataProxyRPC() {};
	FNetworkPhysicsRewindDataProxyRPC& operator=(const FNetworkPhysicsRewindDataProxyRPC& Other);

	/** Causes the history to be serialized every time. If implemented, would prevent serializing if the history hasn't changed. */
	bool operator==(const FNetworkPhysicsRewindDataProxyRPC& Other) const { return false; }

protected:
	virtual bool NetSerializeBase(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess
		, TUniqueFunction<TUniquePtr<Chaos::FBaseRewindHistory>()> CreateHistoryFunction, TUniqueFunction<const uint32()> GetLatestDeltaSourceIndex, TUniqueFunction<FNetworkPhysicsData* (const int32)> GetDeltaSourceData) override
	{
		Ar << Owner;
		return Super::NetSerializeBase(Ar, Map, bOutSuccess, MoveTemp(CreateHistoryFunction), MoveTemp(GetLatestDeltaSourceIndex), MoveTemp(GetDeltaSourceData));
	}
};

/**
 * Struct suitable for use as a replicated RPC parameter to replicate input rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataInputProxy : public FNetworkPhysicsRewindDataProxyRPC
{
	GENERATED_BODY()
		
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataInputProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataInputProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * Struct suitable for use as a replicated property to replicate input rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataOwnerInputProxy : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()
		
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataOwnerInputProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataOwnerInputProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * Struct suitable for use as a replicated property to replicate input rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataRemoteInputProxy : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()
		
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataRemoteInputProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataRemoteInputProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * Struct suitable for use as a replicated property to replicate state rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataStateProxy : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataStateProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataStateProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};
/**
 * Struct suitable for use as a replicated RPC parameter to replicate input rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataImportantInputProxy : public FNetworkPhysicsRewindDataProxyRPC
{
	GENERATED_BODY()
		
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataImportantInputProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataImportantInputProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * Struct suitable for use as a replicated RPC parameter to replicate state rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataImportantStateProxy : public FNetworkPhysicsRewindDataProxyRPC
{
	GENERATED_BODY()

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataImportantStateProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataImportantStateProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};


/**
 * Struct suitable for use as a replicated property to replicate state rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataDeltaSourceStateProxy : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	UPROPERTY()
	uint32 Index = 0;
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataDeltaSourceStateProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataDeltaSourceStateProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};

/**
 * Struct suitable for use as a replicated property to replicate state rewind history
 */
USTRUCT()
struct FNetworkPhysicsRewindDataDeltaSourceInputProxy : public FNetworkPhysicsRewindDataProxy
{
	GENERATED_BODY()

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	UPROPERTY()
	uint32 Index = 0;
};

template<>
struct TStructOpsTypeTraits<FNetworkPhysicsRewindDataDeltaSourceInputProxy> : public TStructOpsTypeTraitsBase2<FNetworkPhysicsRewindDataDeltaSourceInputProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdenticalViaEquality = true
	};
};


/**
 * Network physics rewind callback to manage all the sim callbacks rewind functionalities
 */
struct FNetworkPhysicsCallback : public Chaos::IRewindCallback
{
	FNetworkPhysicsCallback(UWorld* InWorld) : World(InWorld) 
	{ }

	// Delegate on the internal inputs process
	FOnPreProcessInputsInternal PreProcessInputsInternal;
	FOnPostProcessInputsInternal PostProcessInputsInternal;
	FAddResimulationRequestInternal AddResimulationRequestInternal;

	// Bind to this for additional processing on the GT during InjectInputs_External()
	FOnInjectInputsExternal InjectInputsExternal;

	// Rewind API
	virtual void InjectInputs_External(int32 PhysicsStep, int32 NumSteps) override;
	virtual void ProcessInputs_External(int32 PhysicsStep, const TArray<Chaos::FSimCallbackInputAndObject>& SimCallbackInputs);
	virtual void PreProcessInputs_Internal(int32 PhysicsStep) override;
	virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<Chaos::FSimCallbackInputAndObject>& SimCallbackInputs) override;
	virtual void PostProcessInputs_Internal(int32 PhysicsStep) override;
	virtual void PreResimStep_Internal(int32 PhysicsStep, bool bFirst) override;
	virtual void PostResimStep_Internal(int32 PhysicsStep) override;
	virtual int32 TriggerRewindIfNeeded_Internal(int32 LatestStepCompleted) override;
	virtual void RegisterRewindableSimCallback_Internal(Chaos::ISimCallbackObject* SimCallbackObject) override
	{
		if (SimCallbackObject && SimCallbackObject->HasOption(Chaos::ESimCallbackOptions::Rewind))
		{
			RewindableCallbackObjects.Add(SimCallbackObject);
		}
	}

	virtual void UnregisterRewindableSimCallback_Internal(Chaos::ISimCallbackObject* SimCallbackObject) override
	{
		RewindableCallbackObjects.Remove(SimCallbackObject);
	}

	/** Add potential resimulation requests to the rewind data */
	virtual void AddResimulationRequest_Internal(const int32 PhysicsStep, const float DeltaSeconds) override;

	// World owning that callback
	UWorld* World = nullptr;

	// List of rewindable sim callback objects
	TArray<Chaos::ISimCallbackObject*> RewindableCallbackObjects;
};


/**
 * Network physics manager to initialize data required for rewind/resim
 */
UCLASS(MinimalAPI)
class UNetworkPhysicsSystem : public UWorldSubsystem
{
public:

	GENERATED_BODY()
	ENGINE_API UNetworkPhysicsSystem();

	friend struct FNetworkPhysicsCallback;

	// Subsystem Init/Deinit
	ENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ENGINE_API virtual void Deinitialize() override;

	// Delegate at world init 
	ENGINE_API void OnWorldPostInit(UWorld* World, const UWorld::InitializationValues);
};


/** START Iris Compliant Data */

USTRUCT()
struct FNetworkPhysicsPayload
{
	GENERATED_BODY()

	virtual ~FNetworkPhysicsPayload() = default;

	UPROPERTY()
	int32 ServerFrame = 0;

	mutable int32 LocalFrame = 0; // TEMP: Mutable because we need to set this after receiving it via RPC where it's const

	/** Define how to interpolate between two data points if we have a gap between known data.
	* @param MinData is data from a previous frame.
	* @param MaxData is data from a future frame.
	* @param LerpAlpha is the 0.0 - 1.0 value of where *this* data is between MinData and MaxData calculated as so: float LerpAlpha = (LocalFrame - MinData.LocalFrame) / (MaxData.LocalFrame - MinData.LocalFrame)
 	* EXAMPLE: We have input data for frame 1 and 4 and we need to interpolate data for frame 2 and 3 based on frame 1 as MinData and frame 4 as MaxData, for frame 2 LerpAlpha will be 0.33 and for frame 3 LerpAlpha will be 0.66
	*/
	virtual void InterpolateData(const FNetworkPhysicsPayload& MinData, const FNetworkPhysicsPayload& MaxData, float LerpAlpha) {}

	/** Define how to merge data from a previous adjacent frame into the current data.
	* Called when multiple frames need to be combined into a single simulation tick, e.g. when a simulated proxy
	* receives multiple inputs at once, or during resim frame coalescing where skipped frames are merged into the
	* frame that actually simulates.
	* FromData is always from an older adjacent frame; *this is always the newer/accumulated frame.
	* Implementations can rely on the following conventions:
	*   - Floats/vectors: Value = (Value + From.Value) * 0.5
	*   - Booleans: bValue |= From.bValue (true if either frame had it active)
	*   - Discrete/enum: keep *this value (*this is always the newer/accumulated frame)
	* @param FromData Data from the immediately preceding (older) frame being folded into *this.
	*/
	virtual void MergeData(const FNetworkPhysicsPayload& FromData) {}

	/** Use to decay desired data during resimulation if data is forward predicted.
	* @param DecayAmount = Total amount of decay as a multiplier. 10% decay = 0.1.
	* NOTE: Decay is not accumulated, the data will be in its original state each time DecayData is called. DecayAmount will increase each time the input is predicted (reused).
	* EXAMPLE: Use to decay steering inputs to make resimulation not predict too much with a high steering value. Use DecayAmount of 0.1 to turn a steering value of 0.5 into 0.45 for example.
	*/
	virtual void DecayData(float DecayAmount) {}

	/** Define how to compare client and server data for the same frame, returning false means the data differ enough to trigger a resimulation.
	* @param PredictedData is data predicted on the client to compare with the current data received from the server.
	* NOTE: To use this function, CVars np2.Resim.CompareStateToTriggerRewind and/or np2.Resim.CompareInputToTriggerRewind needs to be set to true
	* or the equivalent settings overridden on the actor via UNetworkPhysicsSettingsComponent.
	*/
	virtual bool CompareData(const FNetworkPhysicsPayload& PredictedData) const { return true; }

	/** Return string with custom debug data */
	virtual const FString DebugData() { return FString(" - DebugData() not implemented - "); } // TODO, Deprecate non-const version
	virtual const FString DebugData() const { return FString(" - DebugData() not implemented - "); }

	// If this data was altered so that it doesn't correspond the produced source data (from merging, interpolating or extrapolating)
	bool bDataAltered = false;

	// If this data was received over the network or locally predicted
	bool bReceivedData = false;

	// If this data is marked as important (replicated reliably)
	bool bImportant = false;

	void PrepareFrame(int32 CurrentFrame, bool bIsServer, int32 ClientFrameOffset)
	{
		LocalFrame = CurrentFrame;
		ServerFrame = bIsServer ? CurrentFrame : CurrentFrame + ClientFrameOffset;
		bDataAltered = false;
		bReceivedData = false;
		bImportant = false;
	}

	/** Temporary virtual function for backwards compatibility with FNetworkPhysicsData.Use the InterpolateData function that passes the LerpAlpha instead */
	UE_DEPRECATED(5.7, "Temporary virtual function for backwards compatibility with FNetworkPhysicsData. Use the InterpolateData function that passes the LerpAlpha instead")
	virtual void DoInterpolateData(const FNetworkPhysicsPayload& MinData, const FNetworkPhysicsPayload& MaxData)
	{
		float LerpAlpha = 0.5f;
		if (MaxData.LocalFrame != MinData.LocalFrame)
		{
			LerpAlpha = static_cast<float>(LocalFrame - MinData.LocalFrame) / static_cast<float>(MaxData.LocalFrame - MinData.LocalFrame);
			LerpAlpha = FMath::Clamp(LerpAlpha, 0.0f, 1.0f);
		}
		InterpolateData(MinData, MaxData, LerpAlpha);
	}
};

USTRUCT()
struct FNetworkPhysicsActionCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TInstancedStruct<FNetworkPhysicsActionPayload>> DataArray;
};

USTRUCT()
struct FNetworkPhysicsDataCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TInstancedStruct<FNetworkPhysicsPayload>> DataArray;

	// TEMP, handle this via FNetworkPhysicsPayloadNetSerializer (but without offset, just set LocalFrame = ServerFrame)
	void UpdateLocalFrameFromServerFrame(const int32 ClientFrameOffset) const
	{
		for (const TInstancedStruct<FNetworkPhysicsPayload>& DataInstance : DataArray)
		{
			if (const FNetworkPhysicsPayload* Data = DataInstance.GetPtr<FNetworkPhysicsPayload>())
			{
				Data->LocalFrame = Data->ServerFrame - ClientFrameOffset;
			}
		}
	}

	void DebugCollection(const FString& DebugText) const
	{
		UE_LOGF(LogChaos, Log, "%ls", *DebugText);
		UE_LOGF(LogChaos, Log, "	NumFrames in data collection: %d", DataArray.Num());

		if (DataArray.Num() >= 0)
		{
			for (int32 FrameIndex = 0; FrameIndex < DataArray.Num(); ++FrameIndex)
			{
				const TInstancedStruct<FNetworkPhysicsPayload>& DataInstance = DataArray[FrameIndex];
				const FNetworkPhysicsPayload* Data = DataInstance.GetPtr<FNetworkPhysicsPayload>();

				if (Data)
				{
					UE_LOGF(LogChaos, Log, "		Index: %d || LocalFrame = %d || ServerFrame = %d || bDataAltered = %d || bReceivedData = %d || bImportant = %d  ||  Data: %ls"
						, FrameIndex
						, Data->LocalFrame
						, Data->ServerFrame
						, Data->bDataAltered
						, Data->bReceivedData
						, Data->bImportant
						, *Data->DebugData());
				}
				else
				{
					UE_LOGF(LogChaos, Log, "		No data in collection");
				}
			}
		}
	}

};

// Game Thread Input and State Interface API
class INetworkPhysicsInputState_External
{
protected:
	virtual ~INetworkPhysicsInputState_External() = default;

public:
	virtual void ValidateInput(FNetworkPhysicsPayload& Input) const = 0;
};

template<typename InputType, typename StateType>
class TNetworkPhysicsInputState_External : public INetworkPhysicsInputState_External
{
protected:
	virtual ~TNetworkPhysicsInputState_External() = default;

private:
	// Wrapper for non-templated base
	void ValidateInput(FNetworkPhysicsPayload& Input) const override { ValidateInput_External(static_cast<InputType&>(Input)); }

public:
	/** Validate data received on the server from clients
	* EXAMPLE: Validate incoming inputs from clients on the server and correct any invalid input commands.
	* NOTE: Changes to the data in this callback will be sent from server to clients. */
	virtual void ValidateInput_External(InputType& Input) const = 0;
};

// Physics Thread Action Handler Interface API
class INetworkPhysicsActionHandler_Internal
{
protected:
	virtual ~INetworkPhysicsActionHandler_Internal() = default;

public:
	virtual void ApplyAction_Internal(const TInstancedStruct<FNetworkPhysicsActionPayload>& ActionInstance) = 0;
};

// Physics Thread Input and State Interface API
class INetworkPhysicsInputState_Internal
{
protected:
	virtual ~INetworkPhysicsInputState_Internal() = default;

public:
	virtual void BuildInput(FNetworkPhysicsPayload& Input) const = 0;
	virtual void ValidateInput(FNetworkPhysicsPayload& Input) const = 0;
	virtual void ApplyInput(const FNetworkPhysicsPayload& Input) = 0;

	virtual void BuildState(FNetworkPhysicsPayload& State) const = 0;
	virtual void ApplyState(const FNetworkPhysicsPayload& State) = 0;
};

template<typename InputType, typename StateType>
class TNetworkPhysicsInputState_Internal : public INetworkPhysicsInputState_Internal
{
protected:
	virtual ~TNetworkPhysicsInputState_Internal() = default;

private:
	// Wrapper for non-templated base
	void BuildInput(FNetworkPhysicsPayload& Input) const override { BuildInput_Internal(static_cast<InputType&>(Input)); }
	void ValidateInput(FNetworkPhysicsPayload& Input) const override { ValidateInput_Internal(static_cast<InputType&>(Input)); }
	void ApplyInput(const FNetworkPhysicsPayload& Input) override { ApplyInput_Internal(static_cast<const InputType&>(Input)); }

	void BuildState(FNetworkPhysicsPayload& State) const override { BuildState_Internal(static_cast<StateType&>(State)); }
	void ApplyState(const FNetworkPhysicsPayload& State) override { ApplyState_Internal(static_cast<const StateType&>(State)); }

public:
	/** Populate the input struct with current input data */
	virtual void BuildInput_Internal(InputType& Input) const = 0;

	/** Validate data received on the server from clients
	* EXAMPLE: Validate incoming inputs from clients on the server and correct any invalid input commands.
	* NOTE: Changes to the data in this callback will be sent from server to clients. */
	virtual void ValidateInput_Internal(InputType& Input) const = 0;

	/** Apply input struct to implementation */
	virtual void ApplyInput_Internal(const InputType& Input) = 0;

	/** Populate the state struct with current state data */
	virtual void BuildState_Internal(StateType& State) const = 0;

	/** Apply state struct to implementation */
	virtual void ApplyState_Internal(const StateType& State) = 0;
};

/** END Iris Compliant Data */


/**
 * Base network physics data that will be used by physics
 */
USTRUCT()
struct FNetworkPhysicsData : public FNetworkPhysicsPayload
{
	GENERATED_USTRUCT_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS // Needed while we have a deprecated property inside the struct
	FNetworkPhysicsData() = default;
	virtual ~FNetworkPhysicsData() = default;
	FNetworkPhysicsData(const FNetworkPhysicsData&) = default;
	FNetworkPhysicsData(FNetworkPhysicsData&&) = default;
	FNetworkPhysicsData& operator=(const FNetworkPhysicsData&) = default;
	FNetworkPhysicsData& operator=(FNetworkPhysicsData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	// InputFrame is no longer replicated or used. Use bDataAltered to check if an input has been altered
	UE_DEPRECATED(5.6, "InputFrame is no longer replicated or used. Use bDataAltered to check if an input has been altered.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "InputFrame is no longer replicated or populated. Use bDataAltered to check if an input has been altered"))
	int32 InputFrame_DEPRECATED = INDEX_NONE;
#endif

	/** Thread safe pointer to the UActorComponent that implements the derived type of this data
	* Note: This pointer can be accessed on both Game Thread and Physics Thread but you still need to ensure the read and write calls inside the UActorComponent are thread safe. */
	TStrongObjectPtr<UActorComponent> ImplementationComponent = nullptr;
	
	void SetImplementationComponent(UActorComponent* InImplementationComponent)
	{
		ImplementationComponent = TStrongObjectPtr<UActorComponent>(InImplementationComponent);
	}
	void ClearImplementationComponent() { ImplementationComponent = nullptr; }

	/** Pointer to a previous FNetworkPhysicsData which is valid during NetSerialize() to be used for delta serialization */
	FNetworkPhysicsData* DeltaSourceData = nullptr;

	bool bIsUsingDeltaSerialization = false;

	void SetDeltaSourceData(FNetworkPhysicsData* InDeltaSourceData)
	{
		DeltaSourceData = InDeltaSourceData;
		bIsUsingDeltaSerialization = DeltaSourceData != nullptr;
	}
	void ClearDeltaSourceData()
	{
		DeltaSourceData = nullptr;
		bIsUsingDeltaSerialization = false;
	}

	// Serialize the data into/from the archive
	void SerializeFrames(FArchive& Ar)
	{
		// Delta Serialization
		if (DeltaSourceData) 
		{
			uint32 ServerFrameUnsigned = 0;

			bool bIncrememtalFrame = false;
			if (Ar.IsSaving())
			{
				bIncrememtalFrame = ServerFrame == (DeltaSourceData->ServerFrame + 1);
				{
					Ar.SerializeBits(&bIncrememtalFrame, 1); // Write if the frame delta is just +1, which is most common for internal deltas
				}

				if (!bIncrememtalFrame)
				{
					int32 FrameDelta = ServerFrame - DeltaSourceData->ServerFrame; // Get the frame delta
					bool bFrameDeltaNegative = FrameDelta < 0;
					{
						Ar.SerializeBits(&bFrameDeltaNegative, 1); // Write if the delta is negative
					}
					uint32 FrameDeltaUnsigned = FMath::Abs(FrameDelta);
					{
						Ar.SerializeIntPacked(FrameDeltaUnsigned); // Write the unsigned delta frame
					}
				}
			}
			else // IsLoading
			{
				Ar.SerializeBits(&bIncrememtalFrame, 1);
				if (!bIncrememtalFrame)
				{
					bool bFrameDeltaNegative = false;
					Ar.SerializeBits(&bFrameDeltaNegative, 1); // Get if the delta is negative
					uint32 FrameDeltaUnsigned = 0;
					Ar.SerializeIntPacked(FrameDeltaUnsigned); // Get the frame delta

					// Apply the frame delta to the delta source to get the ServerFrame value
					ServerFrame = bFrameDeltaNegative ? (DeltaSourceData->ServerFrame - FrameDeltaUnsigned) : (DeltaSourceData->ServerFrame + FrameDeltaUnsigned);
				}
				else
				{
					// Increment the delta source ServerFrame once to get the ServerFrame value 
					ServerFrame = DeltaSourceData->ServerFrame + 1;
				}

				LocalFrame = ServerFrame; // Temporarily set LocalFrame to ServerFrame, it will get recalculated later in TRewindHistory::ReceiveNewData
			}
		}
		else // Standard Serialization
		{
			uint32 ServerFrameUnsigned = 0;
			if (Ar.IsSaving())
			{
				check((ServerFrame + 1) >= 0);
				ServerFrameUnsigned = static_cast<uint32>(ServerFrame + 1);
				{
					Ar.SerializeIntPacked(ServerFrameUnsigned);
				}
			}
			else // IsLoading
			{
				Ar.SerializeIntPacked(ServerFrameUnsigned);
				ServerFrame = static_cast<int32>(ServerFrameUnsigned) - 1;
				LocalFrame = ServerFrame; // Temporarily set LocalFrame to ServerFrame, it will get recalculated later in TRewindHistory::ReceiveNewData
			}
		}
	}

	/** Set if this data is important(replicated reliably) or unimportant(replicated unreliably)
	* NOTE: Default is to handle all inputs as unimportant, while one time events can be marked as important. */
	void SetImportant(bool bIsImportant)
	{
		bImportant = bIsImportant;
	}

	// Apply the data onto the network physics component
	virtual void ApplyData(UActorComponent* NetworkComponent) const { }

	// Build the data from the network physics component
	virtual void BuildData(const UActorComponent* NetworkComponent) { }
	
	/** Define how to interpolate between two data points if we have a gap between known data.
	* @param MinData is data from a previous frame.
	* @param MaxData is data from a future frame.
	* EXAMPLE: We have input data for frame 1 and 4 and we need to interpolate data for frame 2 and 3 based on frame 1 as MinData and frame 4 as MaxData.
	*/
	virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) { }

	/** Use to decay desired data during resimulation if data is forward predicted.
	* @param DecayAmount = Total amount of decay as a multiplier. 10% decay = 0.1.
	* NOTE: Decay is not accumulated, the data will be in its original state each time DecayData is called. DecayAmount will increase each time the input is predicted (reused).
	* EXAMPLE: Use to decay steering inputs to make resimulation not predict too much with a high steering value. Use DecayAmount of 0.1 to turn a steering value of 0.5 into 0.45 for example.
	*/ 
	virtual void DecayData(float DecayAmount) override { }
	
	/** Define how to merge data from a previous adjacent frame into the current data.
	* Called when multiple frames need to be combined into a single simulation tick, e.g. when a simulated proxy
	* receives multiple inputs at once, or during resim frame coalescing where skipped frames are merged into the
	* frame that actually simulates.
	* FromData is always from an older adjacent frame; *this is always the newer/accumulated frame.
	* Implementations can rely on the following conventions:
	*   - Floats/vectors: Value = (Value + From.Value) * 0.5
	*   - Booleans: bValue |= From.bValue (true if either frame had it active)
	*   - Discrete/enum: keep *this value (*this is always the newer/accumulated frame)
	* @param FromData Data from the immediately preceding (older) frame being folded into *this.
	*/
	virtual void MergeData(const FNetworkPhysicsData& FromData) { }

	/** Validate data received on the server from clients
	* EXAMPLE: Validate incoming inputs from clients and correct any invalid input commands.
	* NOTE: Changes to the data in this callback will be sent from server to clients.
	*/
	virtual void ValidateData(const UActorComponent* NetworkComponent) { }

	/** Define how to compare client and server data for the same frame, returning false means the data differ enough to trigger a resimulation.
	* @param PredictedData is data predicted on the client to compare with the current data received from the server.
	* NOTE: To use this function, CVars np2.Resim.CompareStateToTriggerRewind and/or np2.Resim.CompareInputToTriggerRewind needs to be set to true
	* or the equivalent settings overridden on the actor via UNetworkPhysicsSettingsComponent. 
	*/
	virtual bool CompareData(const FNetworkPhysicsData& PredictedData) { return true; }
	
	/** Return string with custom debug data */
	virtual const FString DebugData() { return FString(" - DebugData() not implemented - "); }

	bool operator==(const FNetworkPhysicsData& Other) const
	{
		return ServerFrame == Other.ServerFrame && LocalFrame == Other.LocalFrame;
	}

	// Base API Wrapper
	UE_DEPRECATED(5.7, "Temporary virtual function for backwards compatibility with FNetworkPhysicsData. Use the InterpolateData function that passes the LerpAlpha instead")
	void DoInterpolateData(const FNetworkPhysicsPayload& MinData, const FNetworkPhysicsPayload& MaxData) override { this->InterpolateData(static_cast<const FNetworkPhysicsData&>(MinData), static_cast<const FNetworkPhysicsData&>(MaxData)); }
	void MergeData(const FNetworkPhysicsPayload& FromData) override { this->MergeData(static_cast<const FNetworkPhysicsData&>(FromData)); }
	bool CompareData(const FNetworkPhysicsPayload& PredictedData) const override { return const_cast<FNetworkPhysicsData*>(this)->CompareData(static_cast<const FNetworkPhysicsData&>(PredictedData)); }
	using FNetworkPhysicsPayload::InterpolateData; // Silence warning for FNetworkPhysicsData::InterpolateData not overriding a function in FNetworkPhysicsPayload, which is intended
};

/** Base for helper, to create data and data history */
struct FNetworkPhysicsDataHelper
{
	virtual ~FNetworkPhysicsDataHelper() = default;
	virtual const void* GetTypeId() const = 0;
	virtual TUniquePtr<FNetworkPhysicsDataHelper> Clone() const = 0;
	virtual TUniquePtr<FNetworkPhysicsPayload> CreateUniqueData() const = 0;
	virtual TUniquePtr<Chaos::FBaseRewindHistory> CreateUniqueRewindHistory(const int32 Size) const = 0;
	virtual bool IsUsingLegacyData() = 0;

	/** Copy data from the networked data collection to the rewind history */
	virtual void CopyData(const FNetworkPhysicsDataCollection& From, Chaos::FBaseRewindHistory* To) = 0;

	/** Copy data from the rewind history to the networked data collection */
	virtual void CopyData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) = 0;

	/** Copy data from the rewind history to the networked data collection and only copy data that is newer than the already existing data in the data collection */
	virtual void CopyIncrementalData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) = 0;

	/** Copy data from the rewind history to the networked data collection and only copy data that has been altered since it was created (i.e. the client created an input that the server later altered) */
	virtual bool CopyAlteredData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) = 0;

	/** Copy data from the networked data collection to the rewind history and store the data ordered instead of circular and grow the history to fit all unique data */
	virtual void CopyDataGrowingOrdered(const FNetworkPhysicsDataCollection& From, Chaos::FBaseRewindHistory* To) = 0;

	/** Call the ValidateData callback on the external implementation interface for data in the provided rewind history */
	virtual void ValidateData(Chaos::FBaseRewindHistory* History, INetworkPhysicsInputState_External& Interface) = 0;

	/** Call the ValidateData callback on the internal implementation interface for data in the provided rewind history */
	virtual void ValidateData(Chaos::FBaseRewindHistory* History, INetworkPhysicsInputState_Internal& Interface) = 0;

/* // Importance is not implemented into the new network flow (yet?)
	virtual void ApplyDataRange(const Chaos::FBaseRewindHistory* History, const int32 FromFrame, const int32 ToFrame, INetworkPhysicsInputState_Internal& Interface) = 0;
*/
};

/** Helper for the creation of state / input data and history with correct derived type */
template<typename DataType, bool bLegacyData = false>
struct TNetworkPhysicsDataHelper : FNetworkPhysicsDataHelper
{
	~TNetworkPhysicsDataHelper() = default;

	// Implement a static type id, all structs with the same templated types will share the same address to this
	inline static const int32 StaticTypeId = 0;

	// Get the address to the static type id
	virtual const void* GetTypeId() const override { return &StaticTypeId; }

	TUniquePtr<FNetworkPhysicsDataHelper> Clone() const override { return MakeUnique<TNetworkPhysicsDataHelper>(*this); }
	TUniquePtr<FNetworkPhysicsPayload> CreateUniqueData() const override { return MakeUnique<DataType>(); }
	TUniquePtr<Chaos::FBaseRewindHistory> CreateUniqueRewindHistory(const int32 Size) const override { return MakeUnique<TNetRewindHistory<DataType, bLegacyData>>(Size); }
	bool IsUsingLegacyData() { return bLegacyData; }

	/** Copy data from the networked data collection to the rewind history */
	void CopyData(const FNetworkPhysicsDataCollection& From, Chaos::FBaseRewindHistory* To) override
	{
		Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<Chaos::TDataRewindHistory<DataType>*>(To);

		// Record at same index as taken from
		int32 Idx = 0;
		for (const TInstancedStruct<FNetworkPhysicsPayload>& DataInstance : From.DataArray)
		{
			const DataType& Data = DataInstance.Get<DataType>();
			NetHistory->RecordData(Idx, &Data);
			Idx++;
		}
	}

	/** Copy data from the rewind history to the networked data collection */
	void CopyData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) override
	{
		if (ensure(To.DataArray.Num() > 0) == false)
		{
			return;
		}
		
		const Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<const Chaos::TDataRewindHistory<DataType>*>(From);
		const TArray<DataType>& HistoryArray = NetHistory->GetDataHistoryConst();
		for (int32 FromIdx = 0; FromIdx < NetHistory->GetHistorySize(); FromIdx++)
		{
			const int32 ToIdx = FromIdx % To.DataArray.Num();
			To.DataArray[ToIdx] = TInstancedStruct<DataType>::Make(HistoryArray[FromIdx]);
		}
	}

	/** Copy data from the rewind history to the networked data collection and only copy data that is newer than the already existing data in the data collection */
	void CopyIncrementalData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) override
	{
		if (ensure(To.DataArray.Num() > 0) == false)
		{
			return;
		}
		const Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<const Chaos::TDataRewindHistory<DataType>*>(From);

		const TArray<DataType>& HistoryArray = NetHistory->GetDataHistoryConst();
		for (int32 FromIdx = 0; FromIdx < NetHistory->GetHistorySize(); FromIdx++)
		{
			const DataType& FromData = HistoryArray[FromIdx];

			const int32 ToIdx = FromIdx % To.DataArray.Num();
			const FNetworkPhysicsPayload* ToData = To.DataArray[ToIdx].GetPtr<FNetworkPhysicsPayload>();

			// Only copy the data if it's newer than the already cached data
			if (!ToData || ToData->LocalFrame < FromData.LocalFrame)
			{
				To.DataArray[ToIdx] = TInstancedStruct<DataType>::Make(FromData);
			}
		}
	}

	/** Copy data from the rewind history to the networked data collection and only copy data that has been altered since it was created (i.e. the client created an input that the server later altered) */
	bool CopyAlteredData(const Chaos::FBaseRewindHistory* From, FNetworkPhysicsDataCollection& To) override
	{
		bool bHasCopiedData = false;
		if (ensure(To.DataArray.Num() > 0) == false)
		{
			return bHasCopiedData;
		}
		
		const Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<const Chaos::TDataRewindHistory<DataType>*>(From);

		const TArray<DataType>& HistoryArray = NetHistory->GetDataHistoryConst();
		for (int32 FromIdx = 0; FromIdx < NetHistory->GetHistorySize(); FromIdx++)
		{
			const DataType& FromData = HistoryArray[FromIdx];

			// Only copy the data that has been altered
			if (FromData.bDataAltered)
			{
				const int32 ToIdx = FromIdx % To.DataArray.Num();
				To.DataArray[ToIdx] = TInstancedStruct<DataType>::Make(FromData);
				bHasCopiedData = true;
			}
		}

		return bHasCopiedData;
	}

	/** Copy data from the networked data collection to the rewind history and store the data ordered instead of circular and grow the history to fit all unique data */
	void CopyDataGrowingOrdered(const FNetworkPhysicsDataCollection& From, Chaos::FBaseRewindHistory* To) override
	{
		Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<Chaos::TDataRewindHistory<DataType>*>(To);

		for (const TInstancedStruct<FNetworkPhysicsPayload>& DataInstance : From.DataArray)
		{
			NetHistory->RecordDataGrowingOrdered(&DataInstance.Get<DataType>());
		}
	}

	/** Call the ValidateData callback on the external implementation interface for data in the provided rewind history */
	void ValidateData(Chaos::FBaseRewindHistory* History, INetworkPhysicsInputState_External& Interface) override
	{
		Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<Chaos::TDataRewindHistory<DataType>*>(History);
		TArray<DataType>& HistoryArray = NetHistory->GetDataHistory();

		for (int32 Idx = 0; Idx < NetHistory->GetHistorySize(); Idx++)
		{
			FNetworkPhysicsPayload& Data = HistoryArray[Idx];
			if (Data.LocalFrame > LatestValidatedInput_External)
			{
				Interface.ValidateInput(Data);
			}
		}
		LatestValidatedInput_External = FMath::Max(LatestValidatedInput_External, NetHistory->GetLatestFrame());
	}

	/** Call the ValidateData callback on the internal implementation interface for data in the provided rewind history */
	void ValidateData(Chaos::FBaseRewindHistory* History, INetworkPhysicsInputState_Internal& Interface) override
	{
		Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<Chaos::TDataRewindHistory<DataType>*>(History);
		TArray<DataType>& HistoryArray = NetHistory->GetDataHistory();

		for (int32 Idx = 0; Idx < NetHistory->GetHistorySize(); Idx++)
		{
			FNetworkPhysicsPayload& Data = HistoryArray[Idx];
			if (Data.LocalFrame > LatestValidatedInput_Internal)
			{
				Interface.ValidateInput(Data);
			}
		}
		LatestValidatedInput_Internal = FMath::Max(LatestValidatedInput_Internal, NetHistory->GetLatestFrame());
	}

/* // Important is not implemented into the new network flow (yet?)
	void ApplyDataRange(const Chaos::FBaseRewindHistory* History, const int32 FromFrame, const int32 ToFrame, INetworkPhysicsInputState_Internal& Interface) override
	{
		const Chaos::TDataRewindHistory<DataType>* NetHistory = static_cast<const Chaos::TDataRewindHistory<DataType>*>(History);
		const TArray<DataType>& HistoryArray = NetHistory->GetDataHistoryConst();

		for (int32 ApplyFrame = FromFrame; ApplyFrame <= ToFrame; ++ApplyFrame)
		{
			const int32 ApplyIndex = NetHistory->GetFrameIndex(ApplyFrame);
			const DataType& Data = HistoryArray[ApplyIndex];
			if (ApplyFrame == Data.LocalFrame)
			{
				Interface.ApplyInput(Data);
			}
		}
	}
*/

private:
	int32 LatestValidatedInput_External = 0;
	int32 LatestValidatedInput_Internal = 0;
};

/**
 * Network physics component to add to actors or pawns that control their physic simulation through applying inputs,
 * and should support networking through physics resimulation.
 */
UCLASS(BlueprintType, MinimalAPI)
class UNetworkPhysicsComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()
public:
	ENGINE_API UNetworkPhysicsComponent();

	// Get the player controller
	ENGINE_API virtual APlayerController* GetPlayerController() const;

	// Get the controller for this pawn
	ENGINE_API virtual AController* GetController() const;
	
	// Init the network physics component 
	ENGINE_API void InitPhysics();

	// Called every frame
	ENGINE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	// Function to init the replicated properties
	ENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifeTimeProps) const override;

	// Function called just before replication, alter the active state of replicated properties
	ENGINE_API virtual void PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker) override;

	// Used to create any physics engine information for this component 
	ENGINE_API virtual void BeginPlay() override;

	// Register the component into the network manager. Triggers initialization only during
	// seamless travel; normal init uses InitializeComponent instead.
	virtual void OnRegister() override;

	// Called after all sibling components have registered, ensuring NetworkPhysicsSettingsComponent
	// data is available. Handles normal spawning, level load, and dynamic component addition.
	virtual void InitializeComponent() override;

	// Unregister the component from the network manager
	virtual void OnUnregister() override;

	// Remove state/input history from rewind data
	ENGINE_API void RemoveDataHistory();

	// Add state/input history to rewind data
	ENGINE_API void AddDataHistory();

	// Get the GameThread state history (not guaranteed to be the exact data used in physics, for that use GetStateHistory_Internal on PhysicsThread)
	TSharedPtr<Chaos::FBaseRewindHistory>& GetStateHistory_External() { return StateHistory; }
	
	// Get the PhysicsThread state history (if there is none it returns the GameThread history)
	ENGINE_API TSharedPtr<Chaos::FBaseRewindHistory>& GetStateHistory_Internal();
	
	// Get the GameThread input history (not guaranteed to be the exact data used in physics, for that use GetInputHistory_Internal on PhysicsThread)
	TSharedPtr<Chaos::FBaseRewindHistory>& GetInputHistory_External() { return InputHistory; }
	
	// Get the PhysicsThread input history (if there is none it returns the GameThread history)
	ENGINE_API TSharedPtr<Chaos::FBaseRewindHistory>& GetInputHistory_Internal();

	// Check if the world is on server
	ENGINE_API bool HasServerWorld() const;
	
	// Check if this is controlled locally through relayed inputs or an existing local player controller
	ENGINE_API bool IsLocallyControlled() const;
	
	// Check if networked physics is setup with a synchronized physics tick offset
	ENGINE_API bool IsNetworkPhysicsTickOffsetAssigned() const;

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	ENGINE_API void EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const bool bReliable = false);

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	template<typename TActionType>
	void EnqueueImmediateAction_External(TActionType& Action, const UObject* SourceObject, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueImmediateAction_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceObject, bReliable);
	}

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueImmediateAction_External(TActionType& Action, const uint32 SourceId, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueImmediateAction_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueImmediateActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, bReliable);
	}

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	ENGINE_API void EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const float DelaySeconds, const bool bReliable = false);

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const float DelaySeconds, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	template<typename TActionType>
	void EnqueueScheduledAction_External(TActionType& Action, const UObject* SourceObject, const float DelaySeconds, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledAction_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceObject, DelaySeconds, bReliable);
	}

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueScheduledAction_External(TActionType& Action, const uint32 SourceId, const float DelaySeconds, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledAction_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstance_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, DelaySeconds, bReliable);
	}

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	ENGINE_API void EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const UObject* SourceObject, const int32 LocalFrame, const bool bReliable = false);

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const int32 LocalFrame, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame.
	* @param SourceObject is the object that is the source of this action, not necessarily the object that the action should run on.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication. */
	template<typename TActionType>
	void EnqueueScheduledActionAtFrame_External(TActionType& Action, const UObject* SourceObject, const int32 LocalFrame, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledActionAtFrame_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceObject, LocalFrame, bReliable);
	}

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueScheduledActionAtFrame_External(TActionType& Action, const uint32 SourceId, const int32 LocalFrame, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledActionAtFrame_External - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstanceAtFrame_External(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, LocalFrame, bReliable);
	}

	/** Mark this as controlled through locally relayed inputs rather than controlled as a pawn through a player controller.
	* Set if NetworkPhysicsComponent is implemented on an AActor instead of APawn and it's currently being fed inputs, or if this is controlled by the server. 
	* NOTE: The actor for this NetworkPhysicsComponent also needs to be owned by the local client if this is used client-side. */
	void SetIsRelayingLocalInputs(bool bInRelayingLocalInputs)
	{
		bIsRelayingLocalInputs = bInRelayingLocalInputs;
	}

	/** Stop relaying local inputs after next network send.
	* Deferred version of SetIsRelayingLocalInputs(false) to ensure that the last replicated data gets sent.
	* This does not work on locally controlled APawns, see SetIsRelayingLocalInputs() for description. */
	void StopRelayingLocalInputsDeferred()
	{
		if (bIsRelayingLocalInputs)
		{
			bStopRelayingLocalInputsDeferred = true;
		}
	}

	/** Check if this is controlled locally through relayed inputs from autonomous proxy. It's recommended to use IsLocallyControlled() when checking if this is locally controlled. */
	const bool GetIsRelayingLocalInputs() const { return bIsRelayingLocalInputs; }

	/** Override the initial value set by CVar np2.Resim.CompareStateToTriggerRewind on initialize -- When true, cache the clients FNetworkPhysicsPayload state in rewind history for autonomous proxies and compare the predicted state with incoming server state to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData. 
	* @param bInIncludeSimProxies overrides value set by CVar np2.Resim.CompareStateToTriggerRewind.IncludeSimProxies on initialize -- When true CompareStateToTriggerRewind is also done for simulated proxies. */
	ENGINE_API void SetCompareStateToTriggerRewind(const bool bInCompareStateToTriggerRewind, const bool bInIncludeSimProxies = false);

	/** Override the initial value set by CVar np2.Resim.CompareInputToTriggerRewind on initialize -- When true, compare autonomous proxies predicted FNetworkPhysicsPayload inputs with incoming server inputs to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData.*/
	ENGINE_API void SetCompareInputToTriggerRewind(const bool bInCompareInputToTriggerRewind);

	/** Get FAsyncNetworkPhysicsComponent on the Physics Thread */
	FAsyncNetworkPhysicsComponent* GetNetworkPhysicsComponent_Internal() { return NetworkPhysicsComponent_Internal; }

	/** Set the physics object that gets affected by this NetworkPhysicsComponents inputs / states, default is the physics object from the root primitive component.
	* Used to link a desynced physics object that needs resimulation to its input and state history to be able to ensure there is valid data to rewind to. */
	ENGINE_API void SetPhysicsObject(Chaos::FConstPhysicsObjectHandle InPhysicsObject);

protected: 

	// repnotify for input, used for delta serialization
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedDeltaSourceInput();

	// repnotify for state, used for delta serialization
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedDeltaSourceState();

	// replicated physics input, used for delta serialization
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedDeltaSourceInput)
	FNetworkPhysicsRewindDataDeltaSourceInputProxy ReplicatedDeltaSourceInput;

	// replicated physics states, used for delta serialization
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedDeltaSourceState)
	FNetworkPhysicsRewindDataDeltaSourceStateProxy ReplicatedDeltaSourceState;

	/** Server RPC to acknowledge which Delta Source Input Frame the owning client has received */
	UFUNCTION(Server, reliable)
	ENGINE_API void ServerReceiveDeltaSourceInputFrame(const uint32 Index, const int32 Frame);
	
	/** Server RPC to acknowledge which Delta Source State Frame the owning client has received */
	UFUNCTION(Server, reliable)
	ENGINE_API void ServerReceiveDeltaSourceStateFrame(const uint32 Index, const int32 Frame);


	// Server RPC to receive inputs from client
	UFUNCTION(Server, unreliable)
	ENGINE_API void ServerReceiveInputData(const FNetworkPhysicsRewindDataInputProxy& ClientInputs);

	// Server RPC to receive important inputs from client
	UFUNCTION(Server, reliable)
	ENGINE_API void ServerReceiveImportantInputData(const FNetworkPhysicsRewindDataImportantInputProxy& ClientInputs);

	// Client RPC to receive important inputs from server
	UFUNCTION(NetMulticast, reliable)
	ENGINE_API void MulticastReceiveImportantInputData(const FNetworkPhysicsRewindDataImportantInputProxy& ServerInputs);

	// Client RPC to receive important states from server
	UFUNCTION(NetMulticast, reliable)
	ENGINE_API void MulticastReceiveImportantStateData(const FNetworkPhysicsRewindDataImportantStateProxy& ServerStates);

	// replicated client inputs for rpc
	UPROPERTY(Transient)
	FNetworkPhysicsRewindDataInputProxy ReplicateClientInputs;

	// replicated important physics input
	UPROPERTY(Transient)
	FNetworkPhysicsRewindDataImportantInputProxy ReplicatedImportantInput;

	// replicated important physics state
	UPROPERTY(Transient)
	FNetworkPhysicsRewindDataImportantStateProxy ReplicatedImportantState;

	// repnotify for inputs on owner client
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedOwnerInputs();

	// repnotify for inputs on remote clients
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedRemoteInputs();

	// repnotify for the states on the client
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedStates();

	// replicated physics inputs for owner client
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedOwnerInputs)
	FNetworkPhysicsRewindDataOwnerInputProxy ReplicatedOwnerInputs;

	// replicated physics inputs for remote clients
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedRemoteInputs)
	FNetworkPhysicsRewindDataRemoteInputProxy ReplicatedRemoteInputs;

	// replicated physics states 
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedStates)
	FNetworkPhysicsRewindDataStateProxy ReplicatedStates;

	/** Iris Replication */
	// Server RPC to receive inputs from client
	UFUNCTION(Server, unreliable)
	ENGINE_API void ServerReceiveInputCollection(const FNetworkPhysicsDataCollection& ClientInputCollection);

	// Server RPC to receive unreliable actions from client
	UFUNCTION(Server, unreliable)
	ENGINE_API void ServerReceiveUnreliableActionCollection(const FNetworkPhysicsActionCollection& ClientActionCollection);

	// Server RPC to receive reliable actions from client
	UFUNCTION(Server, reliable)
	ENGINE_API void ServerReceiveReliableActionCollection(const FNetworkPhysicsActionCollection& ClientActionCollection);

	// Client RPC to receive reliable actions from server
	UFUNCTION(NetMulticast, reliable)
	ENGINE_API void MulticastReceiveReliableActionCollection(const FNetworkPhysicsActionCollection& ServerActionCollection);

	// repnotify for inputs on owner client
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedInputCollection();

	// repnotify for inputs on remote clients
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedRemoteInputCollection();

	// repnotify for the states on the client
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedStateCollection();

	// repnotify for actions on the client
	UFUNCTION()
	ENGINE_API void OnRep_SetReplicatedActionsCollection();

	// replicated physics inputs for owner client
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedInputCollection)
	FNetworkPhysicsDataCollection ReplicatedInputCollection;

	// replicated physics inputs for remote clients
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedRemoteInputCollection)
	FNetworkPhysicsDataCollection ReplicatedRemoteInputCollection;

	// replicated physics states 
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedStateCollection)
	FNetworkPhysicsDataCollection ReplicatedStateCollection;

	// replicated physics actions 
	UPROPERTY(Transient, ReplicatedUsing = OnRep_SetReplicatedActionsCollection)
	FNetworkPhysicsActionCollection ReplicatedActionsCollection;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ENGINE_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext) override;
#endif

private:
	/** Marshal received action data (from RPCs or property replication) to the physics thread */
	void ReceiveActionCollectionData(const FNetworkPhysicsActionCollection& ActionCollection);

	/** Send network data from marshaled data */
	void NetworkMarshaledData();

	/** Update the async component with GT properties */
	ENGINE_API void UpdateAsyncComponent(const bool bFullUpdate);

	/** Setup data type and history in async input and trigger the creation of data history in the async component */
	ENGINE_API void CreateAsyncDataHistory();

	/** Set default number of inputs to send over the network with each message, clamped between 1 and NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	* From owning client to server.
	* NOTE: this will be overridden if np2.Resim.DynamicInputReplicationScaling.Enabled is enabled. */
	void SetNumberOfInputsToNetwork(uint16 NumInputs)
	{
		NumInputs = FMath::Clamp(NumInputs, 1, NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork);
		InputsToNetwork_OwnerDefault = NumInputs;
		InputsToNetwork_Owner = InputsToNetwork_OwnerDefault;
	}

	/** Set number of inputs to send over the network with each message, clamped between 1 and NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	* From server to remote clients (simulated proxies) */
	void SetNumberOfRemoteInputsToNetwork(uint16 NumInputs)
	{
		NumInputs = FMath::Clamp(NumInputs, 1, NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork);
		InputsToNetwork_Simulated = NumInputs;
	}

	/** Set number of states to send over the network with each message, clamped between 1 and NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	* From server to all clients */
	void SetNumberOfStatesToNetwork(uint16 NumInputs)
	{
		NumInputs = FMath::Clamp(NumInputs, 1, NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork);
		StatesToNetwork = NumInputs;
	}

	/** Inject the current delta source data into array */
	void AddDeltaSourceInput();

	/** Inject the current delta source data into array */
	void AddDeltaSourceState();

	/** Returns the latest acknowledged delta source index  */
	const uint32 GetLatestAcknowledgedDeltaSourceInputIndex() const { return LatestAcknowledgedDeltaSourceInputIndex; }

	/** Returns the latest acknowledged delta source index  */
	const uint32 GetLatestAcknowledgedDeltaSourceStateIndex() const { return LatestAcknowledgedDeltaSourceStateIndex; }

	/** Returns the next index in the delta source input array */
	const int32 GetNextDeltaSourceInputIndex() const { return GetDeltaSourceIndexForFrame(LatestCachedDeltaSourceInputIndex + 1); }

	/** Returns the next index in the delta source state array */
	const int32 GetNextDeltaSourceStateIndex() const { return GetDeltaSourceIndexForFrame(LatestCachedDeltaSourceStateIndex + 1); }

	/** Returns if this frame is valid to store at the next index in the delta source input array, frame value needs to match with the modulo of the array size */
	const bool IsValidNextDeltaSourceInput(const int32 Frame) const { return GetDeltaSourceIndexForFrame(Frame) == GetNextDeltaSourceInputIndex(); }

	/** Returns if this frame is valid to store at the next index in the delta source state array, frame value needs to match with the modulo of the array size */
	const bool IsValidNextDeltaSourceState(const int32 Frame) const { return GetDeltaSourceIndexForFrame(Frame) == GetNextDeltaSourceStateIndex(); }

	/** Initializes necessary data for NetworkPhysicsComponent_Internal and callbacks */
	void InitializePhysicsReplication();

	/** Uninitialize NetworkPhysicsComponent_Internal and callbacks */
	void UninitializePhysicsReplication();

public:
	/** Get the delta source stored at index or with frame, @param bValueIsIndex switches between grabbing frame vs index
	* Note, passing in @param Value as -1 returns the latest data
	* Note, passing in @param Value as -2 returns default data */
	FNetworkPhysicsData* GetDeltaSourceInput(const int32 Value, const bool bValueIsIndexElseFrame);

	/** Get the delta source stored at index or with frame, @param bValueIsIndex switches between grabbing frame vs index
	* Note, passing in @param Value as - 1 returns the latest data
	* Note, passing in @param Value as - 2 returns default data */
	FNetworkPhysicsData* GetDeltaSourceState(const int32 Value, const bool bValueIsIndexElseFrame);

	/** Convert frame number to its corresponding index it would hold in the delta sources array */
	static const int32 GetDeltaSourceIndexForFrame(const int32 Frame) { return FMath::Abs(Frame % NetworkPhysicsComponentConstants::DeltaSourceBufferSize); }

	/** Register and create both state and input to be both networked and cached in history */
	template<typename PhysicsTraits>
	void CreateDataHistory(UActorComponent* HistoryComponent);

	/**  Register and create input history
	* Please use CreateDataHistory() if both input and custom state are supposed to be networked and cached in history.
	* NOTE: Registering Input without State requires networking push-model to be enabled to take advantage of all the CPU and Network Bandwidth savings, CVar: Net.IsPushModelEnabled 1 */
	template<class InputsType>
	void CreateInputHistory(UActorComponent* HistoryComponent);

	/** Register state and input to be networked and cached in history along with the interface implementation to interact with the input and state */
	template<typename Input, typename State>
	void CreateDataHistory(TNetworkPhysicsInputState_Internal<Input, State>* Implementation_Internal, TNetworkPhysicsInputState_External<Input, State>* Implementation_External = nullptr);

	/** Set action handler interface for receiving and applying actions on the physics thread.
	* This can be used standalone (without registering input/state types) for action-only components,
	* or alongside CreateDataHistory() for components that use both input/state and actions. */
	ENGINE_API void SetActionHandler(INetworkPhysicsActionHandler_Internal* InActionHandler);

private:

	friend FNetworkPhysicsCallback;
	friend struct FNetworkPhysicsRewindDataInputProxy;
	friend struct FNetworkPhysicsRewindDataOwnerInputProxy;
	friend struct FNetworkPhysicsRewindDataRemoteInputProxy;
	friend struct FNetworkPhysicsRewindDataStateProxy;
	friend struct FNetworkPhysicsRewindDataImportantInputProxy;
	friend struct FNetworkPhysicsRewindDataImportantStateProxy;
	friend struct FNetworkPhysicsRewindDataDeltaSourceInputProxy;
	friend struct FNetworkPhysicsRewindDataDeltaSourceStateProxy;

	INetworkPhysicsInputState_Internal* ImplementationInterface_Internal = nullptr;
	INetworkPhysicsInputState_External* ImplementationInterface_External = nullptr;
	INetworkPhysicsActionHandler_Internal* ActionImplementationInterface_Internal = nullptr;

	bool bIsUsingLegacyData = false;
	bool bHasRegisteredInputState = false;
	bool bHasPendingActionsToReplicate = false;

	// Network Physics Component data internal to the physics thread
	FAsyncNetworkPhysicsComponent* NetworkPhysicsComponent_Internal;

	// States history on GameThread
	TSharedPtr<Chaos::FBaseRewindHistory> StateHistory;

	// Inputs history on GameThread
	TSharedPtr<Chaos::FBaseRewindHistory> InputHistory;

	// Helper for the creation of input data and history with correct derived type
	TUniquePtr<FNetworkPhysicsDataHelper> InputHelper;

	// Helper for the creation of state data and history with correct derived type
	TUniquePtr<FNetworkPhysicsDataHelper> StateHelper;

	// Local default input data in legacy type
	TUniquePtr<FNetworkPhysicsData> InputDataDefault_Legacy;

	// Local default state data in legacy type
	TUniquePtr<FNetworkPhysicsData> StateDataDefault_Legacy;

	// The number of inputs the owning client should send to the server with each RPC, replicated from the server. This is dynamically scaled based on when there are holes in the inputs buffer if np2.Resim.DynamicInputReplicationScaling.Enabled is enabled. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	UPROPERTY( Replicated )
	uint16 InputsToNetwork_Owner = 3;

	// The default value for number for InputsToNetwork_Owner, acts as the initial value and the cap when dynamically adjusting InputsToNetwork_Owner. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	uint16 InputsToNetwork_OwnerDefault = 3;

	// Send last N number of inputs each replication call from server to remote clients. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	uint16 InputsToNetwork_Simulated = 2;

	// Send last N number of states each replication call from server to remote clients. Clamped by NetworkPhysicsComponentConstants::MaxNumberOfElementsToNetwork.
	uint16 StatesToNetwork = 1;

	// Set by InitializePhysicsReplication when a NetworkPhysicsSettingsComponent was found but
	// its internal (physics thread) data was not yet initialized, i.e. the settings component
	// registered after us during seamless travel. Cleared on the first tick to re-send a full
	// async component update once all sibling components have finished their OnRegister.
	bool bNeedsFullAsyncComponentUpdate = false;

	// Array of delta sources, used as a base for delta serialization
	TArray<TUniquePtr<FNetworkPhysicsData>> DeltaSourceInputs;
	int32 LatestAcknowledgedDeltaSourceInputIndex = 0;
	int32 LatestCachedDeltaSourceInputIndex = 0;
	double TimeToSyncDeltaSourceInput = 0;
	
	// Array of delta sources, used as a base for delta serialization
	TArray<TUniquePtr<FNetworkPhysicsData>> DeltaSourceStates;
	int32 LatestAcknowledgedDeltaSourceStateIndex = 0;
	int32 LatestCachedDeltaSourceStateIndex = 0;
	double TimeToSyncDeltaSourceState = 0;

public:
	// Actor component that will be used to fill the histories
	TWeakObjectPtr<UActorComponent> ActorComponent = nullptr;

private:
	// Root components physics object
	Chaos::FConstPhysicsObjectHandle PhysicsObject = nullptr;

	// Locally relayed inputs makes this component act as if it's a locally controlled pawn.
	bool bIsRelayingLocalInputs = false;
	
	// If we are currently relaying inputs and will stop after next network send.
	bool bStopRelayingLocalInputsDeferred = false;

	// Compare state / input to trigger rewind via FNetworkPhysicsPayload::CompareData
	bool bCompareStateToTriggerRewind = false;
	bool bCompareStateToTriggerRewindIncludeSimProxies = false; // Include simulated proxies when bCompareStateToTriggerRewind is enabled
	bool bCompareInputToTriggerRewind = false;

	// ToDo, retrieve from NetworkPhysicsSettingsComponent so changes at runtime gets picked up
	bool bEnableUnreliableFlow = true;
	bool bEnableReliableFlow = false;
	bool bValidateDataOnGameThread = false;
};

template<typename Input, typename State>
void UNetworkPhysicsComponent::CreateDataHistory(TNetworkPhysicsInputState_Internal<Input, State>* Implementation_Internal, TNetworkPhysicsInputState_External<Input, State>* Implementation_External)
{
	bIsUsingLegacyData = false;
	bHasRegisteredInputState = true;

	ImplementationInterface_Internal = Implementation_Internal;
	ImplementationInterface_External = Implementation_External;

	InputHelper = MakeUnique<TNetworkPhysicsDataHelper<Input, /*bLegacyData*/ false>>();
	StateHelper = MakeUnique<TNetworkPhysicsDataHelper<State, /*bLegacyData*/ false>>();

	CreateAsyncDataHistory();
}

template<typename PhysicsTraits>
inline void UNetworkPhysicsComponent::CreateDataHistory(UActorComponent* HistoryComponent)
{
	bIsUsingLegacyData = true;
	bHasRegisteredInputState = true;

	InputHelper = MakeUnique<TNetworkPhysicsDataHelper<typename PhysicsTraits::InputsType, /*bLegacyData*/ true>>();
	StateHelper = MakeUnique<TNetworkPhysicsDataHelper<typename PhysicsTraits::StatesType, /*bLegacyData*/ true>>();
	
	InputDataDefault_Legacy = MakeUnique<typename PhysicsTraits::InputsType>();
	StateDataDefault_Legacy = MakeUnique<typename PhysicsTraits::StatesType>();

	// Initialize delta source arrays
	for (int32 I = 0; I < NetworkPhysicsComponentConstants::DeltaSourceBufferSize; I++)
	{
		DeltaSourceInputs.Add(MakeUnique<typename PhysicsTraits::InputsType>());
		DeltaSourceStates.Add(MakeUnique<typename PhysicsTraits::StatesType>());
	}

	ReplicateClientInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_OwnerDefault);
	ReplicateClientInputs.Owner = this;

	ReplicatedOwnerInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_OwnerDefault);
	ReplicatedOwnerInputs.Owner = this;

	ReplicatedRemoteInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_Simulated);
	ReplicatedRemoteInputs.Owner = this;

	ReplicatedStates.History = StateHelper->CreateUniqueRewindHistory(StatesToNetwork);
	ReplicatedStates.Owner = this;

	ReplicatedImportantInput.History = InputHelper->CreateUniqueRewindHistory(1);
	ReplicatedImportantInput.Owner = this;

	ReplicatedImportantState.History = StateHelper->CreateUniqueRewindHistory(1);
	ReplicatedImportantState.Owner = this;

	ReplicatedDeltaSourceInput.History = InputHelper->CreateUniqueRewindHistory(1);
	ReplicatedDeltaSourceInput.Owner = this;

	ReplicatedDeltaSourceState.History = StateHelper->CreateUniqueRewindHistory(1);
	ReplicatedDeltaSourceState.Owner = this;

	ActorComponent = TWeakObjectPtr<UActorComponent>(HistoryComponent);

	CreateAsyncDataHistory();
}

template<class InputsType>
inline void UNetworkPhysicsComponent::CreateInputHistory(UActorComponent* HistoryComponent)
{
	bIsUsingLegacyData = true;
	bHasRegisteredInputState = true;

	InputHelper = MakeUnique<TNetworkPhysicsDataHelper<InputsType, /*bLegacyData*/ true>>();
	
	InputDataDefault_Legacy = MakeUnique<InputsType>();

	// Initialize delta source array
	for (int32 I = 0; I < NetworkPhysicsComponentConstants::DeltaSourceBufferSize; I++)
	{
		DeltaSourceInputs.Add(MakeUnique<InputsType>());
	}

	ReplicateClientInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_OwnerDefault);
	ReplicateClientInputs.Owner = this;

	ReplicatedOwnerInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_OwnerDefault);
	ReplicatedOwnerInputs.Owner = this;

	ReplicatedRemoteInputs.History = InputHelper->CreateUniqueRewindHistory(InputsToNetwork_Simulated);
	ReplicatedRemoteInputs.Owner = this;

	ReplicatedImportantInput.History = InputHelper->CreateUniqueRewindHistory(1);
	ReplicatedImportantInput.Owner = this;

	ReplicatedDeltaSourceInput.History = InputHelper->CreateUniqueRewindHistory(1);
	ReplicatedDeltaSourceInput.Owner = this;

	ActorComponent = TWeakObjectPtr<UActorComponent>(HistoryComponent);

	CreateAsyncDataHistory();
}


// --------------------------- PhysicsThread Network Physics Component ---------------------------

struct FAsyncNetworkPhysicsComponentInput : public Chaos::FSimCallbackInput
{
	TOptional<bool> bIsLocallyControlled;
	TOptional<ENetMode> NetMode;
	TOptional<ENetRole> NetRole;
	TOptional<uint16> InputsToNetwork_Owner;
	TOptional<EPhysicsReplicationMode> PhysicsReplicationMode;
	TOptional<TWeakObjectPtr<UActorComponent>> ActorComponent;
	TOptional<INetworkPhysicsInputState_Internal*> ImplementationInterface_Internal;
	TOptional<Chaos::FConstPhysicsObjectHandle> PhysicsObject;
	TOptional<FString> ActorName;
	TOptional<TUniquePtr<FNetworkPhysicsDataHelper>> InputHelper;
	TOptional<TUniquePtr<FNetworkPhysicsDataHelper>> StateHelper;
	TOptional<bool> bRegisterDataHistoryInRewindData;
	TOptional<bool> bUnregisterDataHistoryFromRewindData;
	TOptional<bool> bCompareStateToTriggerRewind;
	TOptional<bool> bCompareStateToTriggerRewindIncludeSimProxies;
	TOptional<bool> bCompareInputToTriggerRewind;
	TOptional<TWeakPtr<const FNetworkPhysicsSettingsData>> SettingsComponent;
	TOptional<INetworkPhysicsActionHandler_Internal*> ActionHandler_Internal;

	TUniquePtr<Chaos::FBaseRewindHistory> InputData;
	TUniquePtr<Chaos::FBaseRewindHistory> StateData;

	TArray<TUniquePtr<Chaos::FBaseRewindHistory>> InputDataImportant;
	TArray<TUniquePtr<Chaos::FBaseRewindHistory>> StateDataImportant;

	TArray<TInstancedStruct<FNetworkPhysicsActionPayload>> LocalActionData;
	TArray<bool> LocalActionDataReliable; // Parallel array to LocalActionData, indicates if each action should be replicated reliably
	TArray<TInstancedStruct<FNetworkPhysicsActionPayload>> ReplicatedActionData;

	void Reset()
	{
		bIsLocallyControlled.Reset();
		NetMode.Reset();
		NetRole.Reset();
		InputsToNetwork_Owner.Reset();
		PhysicsReplicationMode.Reset();
		ActorComponent.Reset();
		ImplementationInterface_Internal.Reset();
		PhysicsObject.Reset();
		ActorName.Reset();
		InputHelper.Reset();
		StateHelper.Reset();
		bRegisterDataHistoryInRewindData.Reset();
		bUnregisterDataHistoryFromRewindData.Reset();
		bCompareStateToTriggerRewind.Reset();
		bCompareStateToTriggerRewindIncludeSimProxies.Reset();
		bCompareInputToTriggerRewind.Reset();
		SettingsComponent.Reset();
		ActionHandler_Internal.Reset();

		if (InputData)
		{
			InputData->ResizeDataHistory(0, EAllowShrinking::No);
			InputData->ResetFast();
		}
		if (StateData)
		{
			StateData->ResizeDataHistory(0, EAllowShrinking::No);
			StateData->ResetFast();
		}

		// Todo, optimize
		InputDataImportant.Reset();
		StateDataImportant.Reset();

		LocalActionData.Reset();
		LocalActionDataReliable.Reset();
		ReplicatedActionData.Reset();
	}
};

struct FAsyncNetworkPhysicsComponentOutput : public Chaos::FSimCallbackOutput
{
	TOptional<uint16> InputsToNetwork_Owner;
	TOptional<float> TargetBufferSizeMs;

	TUniquePtr<Chaos::FBaseRewindHistory> InputData;
	TUniquePtr<Chaos::FBaseRewindHistory> StateData;

	TArray<TUniquePtr<Chaos::FBaseRewindHistory>> InputDataImportant;
	TArray<TUniquePtr<Chaos::FBaseRewindHistory>> StateDataImportant;

	TArray<TInstancedStruct<FNetworkPhysicsActionPayload>> UnreliableActionData;
	TArray<TInstancedStruct<FNetworkPhysicsActionPayload>> ReliableActionData;

	void Reset()
	{
		InputsToNetwork_Owner.Reset();
		TargetBufferSizeMs.Reset();

		if (InputData)
		{
			InputData->ResetFast();
		}
		if (StateData)
		{
			StateData->ResetFast();
		}

		// Todo, optimize
		InputDataImportant.Reset();
		StateDataImportant.Reset();

		UnreliableActionData.Reset();
		ReliableActionData.Reset();
	}
};

class FAsyncNetworkPhysicsComponent : public Chaos::TSimCallbackObject<
	FAsyncNetworkPhysicsComponentInput,
	FAsyncNetworkPhysicsComponentOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::PostSolve | Chaos::ESimCallbackOptions::PhysicsObjectUnregister>
{
	friend UNetworkPhysicsComponent;

public:
	FAsyncNetworkPhysicsComponent();
	~FAsyncNetworkPhysicsComponent() = default;

	// Get reference to async output for current internal frame and initialize it if not already done
	FAsyncNetworkPhysicsComponentOutput& GetAsyncOutput_Internal();

	// If this network physics component is locally controlled, can be either server or autonomous proxy
	const bool IsLocallyControlled() const { return bIsLocallyControlled; }

	// If we are on the server
	const bool IsServer() const { return (NetMode == ENetMode::NM_DedicatedServer || NetMode == ENetMode::NM_ListenServer); }

	// Get the ENetRole
	const ENetRole GetNetRole() const { return NetRole; }

	// Get actor name
	const FString GetActorName() const { return ActorName; }

	// Get the physics tick offset (add to clients physics tick to get the servers corresponding physics tick)
	const int32 GetNetworkPhysicsTickOffset() const
	{
		return UE::NetworkPhysicsUtils::GetNetworkPhysicsTickOffset_Internal(GetSolver());
	}

	/** Get the physics replication mode used
	* TODO: FPhysicsReplication can change replication mode temporarily on the physics thread, make this correspond to the actual replication mode being used*/
	const EPhysicsReplicationMode GetPhysicsReplicationMode() const { return PhysicsReplicationMode; }

	// Add state/input history to rewind data
	void RegisterDataHistoryInRewindData();

	// Remove state/input history from rewind data
	void UnregisterDataHistoryFromRewindData();

	// Enable RewindData history caching and return the history size
	const int32 SetupRewindData();

	// Return true if this is a simulated proxy and it's not currently replicating via physics resimulation
	const bool IsNonResimSimProxy() const { return NetRole == ENetRole::ROLE_SimulatedProxy && PhysicsReplicationMode != EPhysicsReplicationMode::Resimulation; }

	// Get the frame number of the last server update
	ENGINE_API int32 GetLatestReceivedStateFrame() const;

	/** Returns the time in seconds between the frame of the latest received state and the latest frame we have reached.
	  * This can be used as a quantity that is stable over the course of a resimulation (from rewind to last frame of resim) and grows with the amount of time being resimulated
	  * We use this to indicate how much more we should decay input for instance when we have high RTT compared to some baseline decay
	  */
	ENGINE_API const float GetForwardPredictionTime() const;

	/** Returns the current amount of input decay during resimulation as a magnitude from 0.0 to 1.0. Returns 0 if not currently resimulating. */
	ENGINE_API const float GetCurrentInputDecay() const;

	/** Returns the current amount of input decay for sim proxies during simulation as a magnitude from 0.0 to 1.0. */
	ENGINE_API const float GetCurrentSimProxyInputDecayAtRuntime() const;

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueImmediateActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed as soon as possible.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueImmediateAction_Internal(TActionType& Action, const uint32 SourceId, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueImmediateAction_Internal - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueImmediateActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, bReliable);
	}

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueScheduledActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const float DelaySeconds, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed after a delay of @param DelaySeconds.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueScheduledAction_Internal(TActionType& Action, const uint32 SourceId, const float DelaySeconds, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledAction_Internal - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstance_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, DelaySeconds, bReliable);
	}

	/** Enqueue a TInstancedStruct with an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame.
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note: On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	ENGINE_API void EnqueueScheduledActionInstanceAtFrame_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>&& ActionInstance, const uint32 SourceId, const int32 LocalFrame, const bool bReliable = false);

	/** Enqueue an action based on FNetworkPhysicsActionPayload, to be executed at @param LocalFrame
	* @param SourceId can be left at 0 for actions triggered by the owner of this implementation, else if triggered by an external actor pass in a network stable hash.
	* @param bReliable if true, the action will be sent via a reliable RPC instead of unreliable property replication.
	* Note:	On the game thread a network stable hash can be retrieved from replicated UObjects or level baked UObjects (for example actors) via UE::NetworkPhysicsUtils::GetNetworkStableHash_External(UObject); */
	template<typename TActionType>
	void EnqueueScheduledActionAtFrame_Internal(TActionType& Action, const uint32 SourceId, const int32 LocalFrame, const bool bReliable = false)
	{
		static_assert(TIsDerivedFrom<TActionType, FNetworkPhysicsActionPayload>::Value, "EnqueueScheduledActionAtFrame_Internal - Action needs to derive from FNetworkPhysicsActionPayload");
		EnqueueScheduledActionInstanceAtFrame_Internal(TInstancedStruct<FNetworkPhysicsActionPayload>::Make<TActionType>(Action), SourceId, LocalFrame, bReliable);
	}

private:
	// Initialize, bind delegates etc.
	void OnInitialize_Internal();

	// Uninitialize, unbind delegates etc.
	void OnUninitialize_Internal();

	virtual void OnPreSimulate_Internal() override;

	virtual void OnPostSolve_Internal() override;

	virtual void OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;

	// Delegate ran at the start of FNetworkPhysicsCallback::ProcessInputs_Internal, used to receive and apply inputs and states if needed.
	void OnPreProcessInputs_Internal(const int32 PhysicsStep);

	// Delegate ran at the end of FNetworkPhysicsCallback::ProcessInputs_Internal, used to record and send inputs and states if needed.
	void OnPostProcessInputs_Internal(const int32 PhysicsStep);

	// Delegate ran to potentially trigger resimulation if needed
	void OnAddResimulationRequest_Internal(const int32 PhysicsStep);

	// Consume data from async input
	void ConsumeAsyncInput_Internal(const int32 PhysicsStep, const bool bTriggerResim);

	// Add new action and handle related proposed actions
	void RegisterAction_Internal(const TInstancedStruct<FNetworkPhysicsActionPayload>& NewActionInstance, const bool bWasReceivedFromReplication, const bool bReliable = false);

	// Apply actions
	void ApplyActionsForFrame(const int32 LocalFrame);

	// Get a new unique ActionId
	uint16 GetUniqueActionId_Internal();

	/** Get the rigid solver */
	Chaos::FPBDRigidsSolver* GetRigidSolver();

	/** Get the rigid solver evolution */
	Chaos::FPBDRigidsEvolution* GetEvolution();

	/** Get the settings for this NetworkPhysicsComponent */
	const FNetworkPhysicsSettingsNetworkPhysicsComponent& GetComponentSettings() const;

	/** Trigger a resimulation on frame */
	void TriggerResimulation(int32 ResimFrame);

	/** Rewind data has resized, also resize the Input and State histories */
	void OnRewindDataResize_Internal(int32 InNumFrames);

	/** Returns the current amount of input decay during resimulation as a magnitude from 0.0 to 1.0. Returns 0 if not currently resimulating. */
	UE_DEPRECATED(5.8, "Use function that doesn't take a parameter, the decay curve will be based on the current frame number compared with the latest input frame number available")
	const float GetCurrentInputDecay(const FNetworkPhysicsPayload* PhysicsData) { return GetCurrentInputDecay(); }

	/** Populate input data in AsyncOutput to send over the network */
	void SendInputData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep);

	/** Populate state data in AsyncOutput to send over the network */
	void SendStateData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep);
	
	/** Populate action data in AsyncOutput to send over the network */
	void SendActionData_Internal(FAsyncNetworkPhysicsComponentOutput& AsyncOutput, const int32 PhysicsStep);

	/** Update function on the server to manage how many inputs the owning client should send with each RPC */
	void UpdateDynamicInputReplicationScaling();

	/** Update the dynamic input buffer and return the new size */
	float UpdateDynamicInputBufferScaling();

private:
	bool bIsLocallyControlled;
	ENetMode NetMode;
	ENetRole NetRole;
	EPhysicsReplicationMode PhysicsReplicationMode;
	FString ActorName;
	bool bIsUsingLegacyData;

	int32 LastInputSendFrame = INDEX_NONE;
	int32 LastStateSendFrame = INDEX_NONE;
	int32 NewImportantInputFrame = INT_MAX;

	int32 LastReceivedStateFrame = INDEX_NONE;

	// Local iterator to produce a unique ID for an Action
	uint16 ActionIdIterator = 0;

	int32 LastAppliedActionLocalFrame = INDEX_NONE;

	// Component settings
	TWeakPtr<const FNetworkPhysicsSettingsData> SettingsComponent;
	static const FNetworkPhysicsSettingsNetworkPhysicsComponent SettingsNetworkPhysicsComponent_Default;

	// Simulation Decay Settings
	TWeakPtr<FParticleSimDecaySettings> SimDecaySettings = nullptr;

	// PhysicsObject handle used when SimDecaySettings was registered with FPhysicsReplicationAsync. Kept separate from the live PhysicsObject so cleanup always targets the originally-registered entry, even if the live handle has since changed.
	Chaos::FConstPhysicsObjectHandle SimDecayRegisteredHandle = nullptr;

	// Actor component that will be used to fill the histories
	TWeakObjectPtr<UActorComponent> ActorComponent;

	// Implementation of input / state interface
	INetworkPhysicsInputState_Internal* ImplementationInterface_Internal;

	// Implementation of action handler interface
	INetworkPhysicsActionHandler_Internal* ActionHandler_Internal = nullptr;

	// Root components physics object
	Chaos::FConstPhysicsObjectHandle PhysicsObject;

	// Helper for the creation of input data and history with correct derived type
	TUniquePtr<FNetworkPhysicsDataHelper> InputHelper;

	// Helper for the creation of state data and history with correct derived type
	TUniquePtr<FNetworkPhysicsDataHelper> StateHelper;

	// States history uses to rewind simulation 
	TSharedPtr<Chaos::FBaseRewindHistory> StateHistory;

	// Inputs history used during simulation
	TSharedPtr<Chaos::FBaseRewindHistory> InputHistory;

	// Local temporary inputs data used by pre/post process inputs functions
	TUniquePtr<FNetworkPhysicsPayload> InputData;
	
	// Local temporary inputs data used by ConsumeAsyncInput
	TUniquePtr<FNetworkPhysicsPayload> LatestInputReceiveData;

	// Local temporary states data used by pre/post process inputs functions
	TUniquePtr<FNetworkPhysicsPayload> StateData;

	// Confirmed actions for past, present and future frames
	FActionHistory ConfirmedActions;

	// Predicted actions for present and future frames
	FActionHistory PredictedActions;

	// Proposed actions pending confirmation
	FActionHistory PendingProposedActions;

	// Send last N number of inputs each replication call
	uint16 InputsToNetwork_OwnerDefault = 3; // Default value for owning client
	uint16 InputsToNetwork_Owner = 3; // From owning client, i.e. autonomous proxy or client owning an actor with bIsRelayingLocalInputs enabled
	uint16 InputsToNetwork_Simulated = 2; // To simulated proxies
	
	// Properties for dynamic scaling of number of inputs to replicate
	float TimeOfLastDynamicInputReplicationScaling = 0.0f;
	float DynamicInputReplicationScalingAverageInputs = 0.0f;
	int32 MissingInputCount = 0;

	// Send last N number of states each replication call
	uint16 StatesToNetwork = 1;

	// Last input frame applied from history. Used to compute NextExpectedInputFrame for gap detection and input merging.
	// Updated unconditionally during both sim and resim so adaptive coalescing can detect skipped-frame gaps.
	// For sim-proxies the pre-resim value is cached and restored after resim so late-arriving authority inputs
	// are still picked up via MergeData.
	int32 LastInputFrameApplied = -1;

	// Cached pre-resim value of LastInputFrameApplied for sim-proxies.
	// Set on first resim frame, restored on first non-resim frame after resim ends.
	TOptional<int32> SimProxyCachedLastInputFrame;

	// Properties for dynamic scaling of buffered inputs
	float InputBufferAverageMs = 90.0;
	float InputBufferTargetSizeMs = 90.0;
	float InputBufferDynamicMinSizeMs = 0.0f;

	// Simulation Decay properties
	float InputPredictionFramesAverage = 0.0f;

	// Cache predicted states and then compare incoming states via FNetworkPhysicsData::CompareData to trigger a resim if they desync
	bool bCompareStateToTriggerRewind;

	// Include simulated proxies when bCompareStateToTriggerRewind is enabled
	bool bCompareStateToTriggerRewindIncludeSimProxies;

	// Cache compare incoming inputs with locally predicted inputs via FNetworkPhysicsData::CompareData to trigger a resim if they desync
	bool bCompareInputToTriggerRewind;

	FDelegateHandle DelegateOnRewindDataResize_Internal;
	FDelegateHandle DelegateOnPreProcessInputs_Internal;
	FDelegateHandle DelegateOnPostProcessInputs_Internal;
	FDelegateHandle DelegateOnAddResimulationRequest_Internal;
};
