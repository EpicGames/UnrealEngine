// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "NetworkPhysicsComponentNetSerializers.generated.h"

// These serializers handle FNetworkPhysicsDataCollection and FNetworkPhysicsActionCollection,
// providing three compression benefits over the default Iris path:
//   1. ServerFrame delta-encoding: 1 bit when frame == prev+1 (consecutive frames)
//   2. Intra-array delta chaining: element[i] uses element[i-1] as its delta source instead of
//      Iris's tracked per-slot previous state, so consecutive frames in the same batch share
//      the first element's delta baseline across the entire array.
//   3. Always-delta serialization: element[0] (and action type boundaries) delta against a
//      zero-initialized default rather than falling back to full serialization. This avoids
//      relying on Iris's baseline acknowledgment, which rarely arrives in time for per-frame data.


// ---- FNetworkPhysicsPayloadNetSerializer ----------------------------------------
// Handles the FNetworkPhysicsPayload base struct which replicate Inputs and States
// Quantized form: uint32 QuantizedServerFrame = ServerFrame + 1 (unsigned, +1 because ServerFrame is normally an int32 with a default value of INDEX_NONE).
// Full serialize: packed uint32.
// Delta serialize: 1 bit bIncremental if delta==+1, else sign + SerializeIntPacked(|delta|).
USTRUCT()
struct FNetworkPhysicsPayloadNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};


// ---- FNetworkPhysicsDataCollectionNetSerializer ---------------------------------
// Handles FNetworkPhysicsDataCollection (TArray<TInstancedStruct<FNetworkPhysicsPayload>>).
// Owns the full pipeline: type dispatch, descriptor caching, array sizing, and the
// intra-array delta chain described above.
USTRUCT()
struct FNetworkPhysicsDataCollectionNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};


// ---- FNetworkPhysicsActionPayloadNetSerializer ----------------------------------
// Handles the FNetworkPhysicsActionPayload base struct.
// Mirrors FNetworkPhysicsPayloadNetSerializer for efficient ServerFrame delta encoding.
USTRUCT()
struct FNetworkPhysicsActionPayloadNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};


// ---- FNetworkPhysicsActionCollectionNetSerializer -------------------------------
// Handles FNetworkPhysicsActionCollection (TArray<TInstancedStruct<FNetworkPhysicsActionPayload>>).
// Delta strategy for heterogeneous arrays: if curr[i] has the same concrete type as curr[i-1]
// use curr[i-1] as delta source; otherwise delta against a zero-initialized default.
USTRUCT()
struct FNetworkPhysicsActionCollectionNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FNetworkPhysicsPayloadNetSerializer, ENGINE_API);
UE_NET_DECLARE_SERIALIZER(FNetworkPhysicsDataCollectionNetSerializer, ENGINE_API);
UE_NET_DECLARE_SERIALIZER(FNetworkPhysicsActionPayloadNetSerializer, ENGINE_API);
UE_NET_DECLARE_SERIALIZER(FNetworkPhysicsActionCollectionNetSerializer, ENGINE_API);

} // namespace UE::Net
