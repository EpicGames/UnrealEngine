// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NetworkPhysicsSettingsComponent.h
	Manage networked physics settings per actor through ActorComponent and the subsequent physics-thread data flow for the settings.
=============================================================================*/

#pragma once
#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Chaos/SimCallbackObject.h"
#include "Curves/CurveFloat.h"
#include "Engine/DataAsset.h"

#include "NetworkPhysicsSettingsComponent.generated.h"


class FNetworkPhysicsSettingsComponentAsync;

//  Alias
using FDefaultReplicationSettings		= FNetworkPhysicsSettingsDefaultReplication;
using FPredictiveInterpolationSettings	= FNetworkPhysicsSettingsPredictiveInterpolation;
using FResimulationSettings				= FNetworkPhysicsSettingsResimulation;
using FRenderInterpolationSettings		= FNetworkPhysicsSettingsResimulationErrorCorrection;

namespace PhysicsReplicationCVars
{
	namespace DefaultReplicationCVars
	{
		extern bool bHardsnapLegacyInPT;
		extern bool bCorrectConnectedBodies;
		extern bool bCorrectConnectedBodiesFriction;
	}
	
	namespace PredictiveInterpolationCVars
	{
		extern bool bVelocityBased;
		extern bool bCorrectionAsVelocity;
		extern float PosCorrectionTimeBase;
		extern float PosCorrectionTimeMin;
		extern float PosCorrectionTimeMultiplier;
		extern float RotCorrectionTimeBase;
		extern float RotCorrectionTimeMin;
		extern float RotCorrectionTimeMultiplier;
		extern float PosInterpolationTimeMultiplier;
		extern float RotInterpolationTimeMultiplier;
		extern float SoftSnapPosStrength;
		extern float SoftSnapRotStrength;
		extern bool bSoftSnapToSource;
		extern bool bSkipVelocityRepOnPosEarlyOut;
		extern bool bPostResimWaitForUpdate;
		extern bool bDisableSoftSnap;
		extern bool bCorrectConnectedBodies;
		extern bool bCorrectConnectedBodiesFriction;
	}

	namespace ResimulationCVars
	{
		extern bool bRuntimeCorrectionEnabled;
		extern bool bRuntimeVelocityCorrection;
		extern float PosStabilityMultiplier;
		extern float RotStabilityMultiplier;
		extern float VelStabilityMultiplier;
		extern float AngVelStabilityMultiplier;
		extern bool bRuntimeCorrectConnectedBodies;
		extern bool bEnableUnreliableFlow;
		extern bool bEnableReliableFlow;
		extern bool bApplyDataInsteadOfMergeData;
		extern bool bAllowInputExtrapolation;
		extern bool bValidateDataOnGameThread;
		extern int32 RedundantInputs;
		extern int32 RedundantRemoteInputs;
		extern int32 RedundantStates;
		extern bool bDynamicInputBufferScalingEnabled;
		extern bool bCompareStateToTriggerRewind;
		extern bool bCompareStateToTriggerRewindIncludeSimProxies;
		extern bool bCompareInputToTriggerRewind;
		extern bool bApplySimProxyStateAtRuntime;
		extern bool bApplySimProxyInputAtRuntime;
		extern bool bTriggerResimOnInputReceive;
		extern bool bApplyInputDecaySimProxyInputAtRuntime;
		extern bool bEnableLagScalingSimProxyRuntimeInputDecay;
		extern float InputDecaySimProxyInputAtRuntime;
		extern bool bApplyInputDecayOverSetTime;
		extern float InputDecaySetTime;
		extern bool bEnableLagScalingInputDecay;
		extern float InputDecayReferenceLagMs;
	}
}


USTRUCT()
struct FNetworkPhysicsSettings
{
	GENERATED_BODY()

	FNetworkPhysicsSettings() { }

	// Override properties
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	bool bOverrideSimProxyRepMode = false;

	// Override the EPhysicsReplicationMode for Actors with ENetRole::ROLE_SimulatedProxy.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideSimProxyRepMode"))
	EPhysicsReplicationMode SimProxyRepMode = EPhysicsReplicationMode::PredictiveInterpolation;
	
	// Register this actor's Autonomous Proxy as a focal point / focal particle in Physics Replication LOD.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides")
	bool bFocalParticleInPhysicsReplicationLOD = false;

	// Minimum delay added, in seconds, when scheduling an event far enough in the future to ensure server and all peers that event at the same time
	// This value should be large enough to accommodate the largest network Round Trip Time, beyond which it is understood that the simulation will suffer from corrections
	UPROPERTY(EditDefaultsOnly, Category = "General")
	float EventSchedulingMinDelaySeconds = 0.3f;
};

USTRUCT()
struct FNetworkPhysicsSettingsDefaultReplication
{
	GENERATED_BODY()

	FNetworkPhysicsSettingsDefaultReplication()
		: bOverrideMaxLinearHardSnapDistance(0)
		, bOverrideDefaultLegacyHardsnapInPT(0)
		, bOverrideCorrectConnectedBodies(0)
		, bOverrideCorrectConnectedBodiesFriction(0)
	{}

	// Override properties
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideMaxLinearHardSnapDistance : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideDefaultLegacyHardsnapInPT : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCorrectConnectedBodies : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCorrectConnectedBodiesFriction : 1;

	// Overrides CVar: p.MaxLinearHardSnapDistance -- Hardsnap if distance between current position and extrapolated target position is larger than this value.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideMaxLinearHardSnapDistance"))
	float MaxLinearHardSnapDistance = 400.0f;
	float GetMaxLinearHardSnapDistance(float DefaultValue) const { return bOverrideMaxLinearHardSnapDistance ? MaxLinearHardSnapDistance : DefaultValue; }

	// Overrides CVar: p.DefaultReplication.Legacy.HardsnapInPT -- If default replication is used and it's running the legacy flow through Game Thread, allow hardsnapping to be performed on Physics Thread if async physics is enabled.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideDefaultLegacyHardsnapInPT"))
	bool bHardsnapInPhysicsThread = PhysicsReplicationCVars::DefaultReplicationCVars::bHardsnapLegacyInPT;
	bool GetHardsnapDefaultLegacyInPT() const { return bOverrideDefaultLegacyHardsnapInPT ? bHardsnapInPhysicsThread : PhysicsReplicationCVars::DefaultReplicationCVars::bHardsnapLegacyInPT; }

	// Overrides CVar: p.DefaultReplication.CorrectConnectedBodies -- When true, transform corrections will also apply to any connected physics object.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCorrectConnectedBodies"))
	bool bCorrectConnectedBodies = PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodies;
	bool GetCorrectConnectedBodies() const { return bOverrideCorrectConnectedBodies ? bCorrectConnectedBodies : PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodies; }

	// Overrides CVar: p.DefaultReplication.CorrectConnectedBodiesFriction -- When true, transform correction on any connected physics object will also recalculate their friction.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCorrectConnectedBodiesFriction"))
	bool bCorrectConnectedBodiesFriction = PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodiesFriction;
	bool GetCorrectConnectedBodiesFriction() const { return bOverrideCorrectConnectedBodiesFriction ? bCorrectConnectedBodiesFriction : PhysicsReplicationCVars::DefaultReplicationCVars::bCorrectConnectedBodiesFriction; }
};

USTRUCT()
struct FNetworkPhysicsSettingsPredictiveInterpolation
{
	GENERATED_BODY()

	FNetworkPhysicsSettingsPredictiveInterpolation() 
		: bOverrideVelocityBased(0)
		, bOverrideCorrectionAsVelocity(0)
		, bOverridePosCorrectionTimeBase(0)
		, bOverridePosCorrectionTimeMin(0)
		, bOverridePosCorrectionTimeMultiplier(0)
		, bOverrideRotCorrectionTimeBase(0)
		, bOverrideRotCorrectionTimeMin(0)
		, bOverrideRotCorrectionTimeMultiplier(0)
		, bOverridePosInterpolationTimeMultiplier(0)
		, bOverrideRotInterpolationTimeMultiplier(0)
		, bOverrideSoftSnapPosStrength(0)
		, bOverrideSoftSnapRotStrength(0)
		, bOverrideSoftSnapToSource(0)
		, bOverrideDisableSoftSnap(0)
		, bOverrideSkipVelocityRepOnPosEarlyOut(0)
		, bOverridePostResimWaitForUpdate(0)
		, bOverrideCorrectConnectedBodies(0)
		, bOverrideCorrectConnectedBodiesFriction(0)
	{}

	// Override properties
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideVelocityBased : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCorrectionAsVelocity : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePosCorrectionTimeBase : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePosCorrectionTimeMin : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePosCorrectionTimeMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotCorrectionTimeBase : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotCorrectionTimeMin : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotCorrectionTimeMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePosInterpolationTimeMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotInterpolationTimeMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideSoftSnapPosStrength : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideSoftSnapRotStrength : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideSoftSnapToSource : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideDisableSoftSnap : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideSkipVelocityRepOnPosEarlyOut : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePostResimWaitForUpdate : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCorrectConnectedBodies : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCorrectConnectedBodiesFriction : 1;

	// Overrides CVar: np2.PredictiveInterpolation.VelocityBased -- When true, predictive interpolation replication mode will move objects replicate by applying linear velocity and angular velocity. When false, objects will move by transform shifting towards the target state.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideVelocityBased"))
	bool bVelocityBased = PhysicsReplicationCVars::PredictiveInterpolationCVars::bVelocityBased;
	bool GetVelocityBased() const { return bOverrideVelocityBased ? bVelocityBased : PhysicsReplicationCVars::PredictiveInterpolationCVars::bVelocityBased; }

	// Overrides CVar: np2.PredictiveInterpolation.CorrectionAsVelocity -- When true, predictive interpolation will apply positional and rotational offset correction as a velocity instead of as a transform shift, while np2.PredictiveInterpolation.VelocityBased is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCorrectionAsVelocity"))
	bool bCorrectionAsVelocity = PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectionAsVelocity;
	bool GetCorrectionAsVelocity() const { return bOverrideCorrectionAsVelocity ? bCorrectionAsVelocity : PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectionAsVelocity; }

	// Overrides CVar: np2.PredictiveInterpolation.PosCorrectionTimeBase -- Base time to correct positional offset over. RoundTripTime * PosCorrectionTimeMultiplier is added on top of this.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePosCorrectionTimeBase"))
	float PosCorrectionTimeBase = PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeBase;
	float GetPosCorrectionTimeBase() const { return bOverridePosCorrectionTimeBase ? PosCorrectionTimeBase : PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeBase; }

	// Overrides CVar: np2.PredictiveInterpolation.PosCorrectionTimeMin -- Min time to correct positional offset over. DeltaSeconds is added on top of this.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePosCorrectionTimeMin"))
	float PosCorrectionTimeMin = PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeMin;
	float GetPosCorrectionTimeMin() const { return bOverridePosCorrectionTimeMin ? PosCorrectionTimeMin : PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeMin; }

	// Overrides CVar: np2.PredictiveInterpolation.PosCorrectionTimeMultiplier -- Multiplier to adjust how much of RoundTripTime to add to positional offset correction.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePosCorrectionTimeMultiplier"))
	float PosCorrectionTimeMultiplier = PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeMultiplier;
	float GetPosCorrectionTimeMultiplier() const { return bOverridePosCorrectionTimeMultiplier ? PosCorrectionTimeMultiplier : PhysicsReplicationCVars::PredictiveInterpolationCVars::PosCorrectionTimeMultiplier; }

	// Overrides CVar: np2.PredictiveInterpolation.RotCorrectionTimeBase -- Base time to correct rotational offset over. RoundTripTime * RotCorrectionTimeMultiplier is added on top of this.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRotCorrectionTimeBase"))
	float RotCorrectionTimeBase = PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeBase;
	float GetRotCorrectionTimeBase() const { return bOverrideRotCorrectionTimeBase ? RotCorrectionTimeBase : PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeBase; }

	// Overrides CVar: np2.PredictiveInterpolation.RotCorrectionTimeMin -- Min time to correct rotational offset over. DeltaSeconds is added on top of this.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRotCorrectionTimeMin"))
	float RotCorrectionTimeMin = PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeMin;
	float GetRotCorrectionTimeMin() const { return bOverrideRotCorrectionTimeMin ? RotCorrectionTimeMin : PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeMin; }

	// Overrides CVar: np2.PredictiveInterpolation.RotCorrectionTimeMultiplier -- Multiplier to adjust how much of RoundTripTime to add to rotational offset correction.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRotCorrectionTimeMultiplier"))
	float RotCorrectionTimeMultiplier = PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeMultiplier;
	float GetRotCorrectionTimeMultiplier() const { return bOverrideRotCorrectionTimeMultiplier ? RotCorrectionTimeMultiplier : PhysicsReplicationCVars::PredictiveInterpolationCVars::RotCorrectionTimeMultiplier; }

	// Overrides CVar: np2.PredictiveInterpolation.InterpolationTimeMultiplier -- Multiplier to adjust the interpolation time which is based on the sendrate of state data from the server.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePosInterpolationTimeMultiplier"))
	float PosInterpolationTimeMultiplier = PhysicsReplicationCVars::PredictiveInterpolationCVars::PosInterpolationTimeMultiplier;
	float GetPosInterpolationTimeMultiplier() const { return bOverridePosInterpolationTimeMultiplier ? PosInterpolationTimeMultiplier : PhysicsReplicationCVars::PredictiveInterpolationCVars::PosInterpolationTimeMultiplier; }

	// Overrides CVar: np2.PredictiveInterpolation.RotInterpolationTimeMultiplier -- Multiplier to adjust the rotational interpolation time which is based on the sendrate of state data from the server.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRotInterpolationTimeMultiplier"))
	float RotInterpolationTimeMultiplier = PhysicsReplicationCVars::PredictiveInterpolationCVars::RotInterpolationTimeMultiplier;
	float GetRotInterpolationTimeMultiplier() const { return bOverrideRotInterpolationTimeMultiplier ? RotInterpolationTimeMultiplier : PhysicsReplicationCVars::PredictiveInterpolationCVars::RotInterpolationTimeMultiplier; }

	// Overrides CVar: np2.PredictiveInterpolation.SoftSnapPosStrength -- Value in percent between 0.0 - 1.0 representing how much to softsnap each tick of the remaining positional distance.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideSoftSnapPosStrength"))
	float SoftSnapPosStrength = PhysicsReplicationCVars::PredictiveInterpolationCVars::SoftSnapPosStrength;
	float GetSoftSnapPosStrength() const { return bOverrideSoftSnapPosStrength ? SoftSnapPosStrength : PhysicsReplicationCVars::PredictiveInterpolationCVars::SoftSnapPosStrength; }

	// Overrides CVar: np2.PredictiveInterpolation.SoftSnapRotStrength -- Value in percent between 0.0 - 1.0 representing how much to softsnap each tick of the remaining rotational distance.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideSoftSnapRotStrength"))
	float SoftSnapRotStrength = PhysicsReplicationCVars::PredictiveInterpolationCVars::SoftSnapRotStrength;
	float GetSoftSnapRotStrength() const { return bOverrideSoftSnapRotStrength ? SoftSnapRotStrength : PhysicsReplicationCVars::PredictiveInterpolationCVars::SoftSnapRotStrength; }

	// Overrides CVar: np2.PredictiveInterpolation.SoftSnapToSource -- If true, softsnap will be performed towards the source state of the current target instead of the predicted state of the current target.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideSoftSnapToSource"))
	bool bSoftSnapToSource = PhysicsReplicationCVars::PredictiveInterpolationCVars::bSoftSnapToSource;
	bool GetSoftSnapToSource() const { return bOverrideSoftSnapToSource ? bSoftSnapToSource : PhysicsReplicationCVars::PredictiveInterpolationCVars::bSoftSnapToSource; }

	// Overrides CVar: np2.PredictiveInterpolation.DisableSoftSnap -- When true, predictive interpolation will not use softsnap to correct the replication with when velocity fails. Hardsnap will still eventually kick in if replication can't reach the target.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideDisableSoftSnap"))
	bool bDisableSoftSnap = PhysicsReplicationCVars::PredictiveInterpolationCVars::bDisableSoftSnap;
	bool GetDisableSoftSnap() const { return bOverrideDisableSoftSnap ? bDisableSoftSnap : PhysicsReplicationCVars::PredictiveInterpolationCVars::bDisableSoftSnap; }

	// Overrides CVar: np2.PredictiveInterpolation.SkipVelocityRepOnPosEarlyOut -- If true, don't run linear velocity replication if position can early out but angular can't early out.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideSkipVelocityRepOnPosEarlyOut"))
	bool bSkipVelocityRepOnPosEarlyOut = PhysicsReplicationCVars::PredictiveInterpolationCVars::bSkipVelocityRepOnPosEarlyOut;
	bool GetSkipVelocityRepOnPosEarlyOut() const { return bOverrideSkipVelocityRepOnPosEarlyOut ? bSkipVelocityRepOnPosEarlyOut : PhysicsReplicationCVars::PredictiveInterpolationCVars::bSkipVelocityRepOnPosEarlyOut; }

	// Overrides CVar: np2.PredictiveInterpolation.PostResimWaitForUpdate -- After a resimulation, wait for replicated states that correspond to post-resim state before processing replication again.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePostResimWaitForUpdate"))
	bool bPostResimWaitForUpdate = PhysicsReplicationCVars::PredictiveInterpolationCVars::bPostResimWaitForUpdate;
	bool GetPostResimWaitForUpdate() const { return bOverridePostResimWaitForUpdate ? bPostResimWaitForUpdate : PhysicsReplicationCVars::PredictiveInterpolationCVars::bPostResimWaitForUpdate; }

	// Overrides CVar: np2.PredictiveInterpolation.CorrectConnectedBodies -- When true, transform corrections will also apply to any connected physics object.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCorrectConnectedBodies"))
	bool bCorrectConnectedBodies = PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectConnectedBodies;
	bool GetCorrectConnectedBodies() const { return bOverrideCorrectConnectedBodies ? bCorrectConnectedBodies : PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectConnectedBodies; }

	// Overrides CVar: np2.PredictiveInterpolation.CorrectConnectedBodiesFriction -- When true, transform correction on any connected physics object will also recalculate their friction.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCorrectConnectedBodiesFriction"))
	bool bCorrectConnectedBodiesFriction = PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectConnectedBodiesFriction;
	bool GetCorrectConnectedBodiesFriction() const { return bOverrideCorrectConnectedBodiesFriction ? bCorrectConnectedBodiesFriction : PhysicsReplicationCVars::PredictiveInterpolationCVars::bCorrectConnectedBodiesFriction; }
};

USTRUCT()
struct FNetworkPhysicsSettingsResimulationErrorCorrection
{
	FNetworkPhysicsSettingsResimulationErrorCorrection()
		: bOverrideResimErrorInterpolationSettings(0)
		, ResimErrorCorrectionDuration(0.3f)
		, ResimErrorMaximumDistanceBeforeSnapping(250.0f)
		, ResimErrorMaximumDesyncTimeBeforeSnapping(0.6f)
		, ResimErrorDirectionalDecayMultiplier(0.0f)
		, bRenderInterpApplyExponentialDecay(false)
		, RenderInterpExponentialDecayLinearHalfLife(0.06f)
		, RenderInterpExponentialDecayAngularHalfLife(0.06f)
		, RenderInterpMinimumLinearThreshold(0.1f)
		, RenderInterpMinimumAngularThreshold(0.001f)
	{};

	GENERATED_BODY()

	/** Apply these settings to Physics Object */
	ENGINE_API void ApplySettings_External(Chaos::FPhysicsObjectHandle PhysicsObject, const bool bInForceOverride = false) const;

	ENGINE_API static void ResetSettings_External(Chaos::FPhysicsObjectHandle PhysicsObject);

	ENGINE_API static bool FromPhysicObject(Chaos::FPhysicsObjectHandle PhysicsObject, FNetworkPhysicsSettingsResimulationErrorCorrection& OutValue);

	/** Enable override for post-resimulation error correction settings during render interpolation
	* NOTE: This currently does not work if the experimental p.RenderInterp.ErrorVelocityCorrection CVar is set to true (false by default) */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides")
	uint32 bOverrideResimErrorInterpolationSettings : 1;

	/** Overrides CVar: p.RenderInterp.ErrorCorrectionDuration -- How long in seconds to apply error correction over */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float ResimErrorCorrectionDuration;

	/** Overrides CVar : p.RenderInterp.MaximumErrorCorrectionBeforeSnapping -- Maximum error correction in cm before we stop interpolating and snap to target */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float ResimErrorMaximumDistanceBeforeSnapping;

	/** Overrides CVar: p.RenderInterp.MaximumErrorCorrectionDesyncTimeBeforeSnapping -- Time multiplied by the particles velocity to get the distance that error correction will be performed within without snapping, disable by setting a negative value
	* NOTE: ResimErrorMaximumDistanceBeforeSnapping will act as a lowest distance clamp. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "-1.0", UIMin = "-1.0"))
	float ResimErrorMaximumDesyncTimeBeforeSnapping;

	/** Overrides CVar: p.RenderInterp.DirectionalDecayMultiplier -- Decay error offset in the direction that the physics object is moving, value is multiplier of projected offset direction, 0.25 means a 25 % decay of the magnitude in the direction of physics travel.Deactivate by setting to 0 */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float ResimErrorDirectionalDecayMultiplier;
	
	/** Overrides CVar: p.RenderInterp.ApplyExponentialDecay -- When enabled a post-resim error will decay exponentially (instead of linearly) based on half-life time set in ExponentialDecayLinearHalfLife and ExponentialDecayAngularHalfLife. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings"))
	bool bRenderInterpApplyExponentialDecay;
	
	/** Overrides CVar: p.RenderInterp.ExponentialDecayLinearHalfLife -- Sets the positional half-life time for when bApplyExponentialDecay is enabled. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float RenderInterpExponentialDecayLinearHalfLife;

	/** Overrides CVar: p.RenderInterp.ExponentialDecayAngularHalfLife -- Sets the rotational half-life time for when bApplyExponentialDecay is enabled. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float RenderInterpExponentialDecayAngularHalfLife;

	/** Overrides CVar: p.RenderInterp.MinimumLinearThreshold -- Squared value, when the remaining render error is below this we clear it, if ApplyExponentialDecay is enabled. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float RenderInterpMinimumLinearThreshold;

	/** Overrides CVar: p.RenderInterp.MinimumAngularThreshold -- When the remaining render error angle is below this we clear it, if ApplyExponentialDecay is enabled. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimErrorInterpolationSettings", ClampMin = "0.0", UIMin = "0.0"))
	float RenderInterpMinimumAngularThreshold;
};

USTRUCT()
struct FNetworkPhysicsSettingsResimulation
{
	FNetworkPhysicsSettingsResimulation()
		: bOverrideResimulationErrorPositionThreshold(0)
		, bOverrideResimulationErrorRotationThreshold(0)
		, bOverrideResimulationErrorLinearVelocityThreshold(0)
		, bOverrideResimulationErrorAngularVelocityThreshold(0)
		, bOverrideRuntimeCorrectionEnabled(0)
		, bOverrideRuntimeVelocityCorrection(0)
		, bOverrideRuntimeCorrectConnectedBodies(0)
		, bOverridePosStabilityMultiplier(0)
		, bOverrideRotStabilityMultiplier(0)
		, bOverrideVelStabilityMultiplier(0)
		, bOverrideAngVelStabilityMultiplier(0)
		, ResimulationErrorCorrectionSettings()
	{};

	GENERATED_BODY()

	// Override properties
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideResimulationErrorPositionThreshold : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideResimulationErrorRotationThreshold : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideResimulationErrorLinearVelocityThreshold : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideResimulationErrorAngularVelocityThreshold : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRuntimeCorrectionEnabled : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRuntimeVelocityCorrection : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRuntimeCorrectConnectedBodies : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridePosStabilityMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRotStabilityMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideVelStabilityMultiplier : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideAngVelStabilityMultiplier : 1;

	// Overrides Project Settings -> Physics -> Replication -> Physics Prediction -> Resimulation Error Position Threshold -- Distance that the object is allowed to desync from the server before triggering a resimulation, within this threshold runtime correction can be performed if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimulationErrorPositionThreshold"))
	float ResimulationErrorPositionThreshold = 10.0f;
	float GetResimulationErrorPositionThreshold(float DefaultValue) const { return bOverrideResimulationErrorPositionThreshold ? ResimulationErrorPositionThreshold : DefaultValue; }

	// Overrides Project Settings -> Physics -> Replication -> Physics Prediction -> Resimulation Error Rotation Threshold -- Rotation difference in degrees that the object is allowed to desync from the server before triggering a resimulation, within this threshold runtime correction can be performed if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimulationErrorRotationThreshold"))
	float ResimulationErrorRotationThreshold = 4.0f;
	float GetResimulationErrorRotationThreshold(float DefaultValue) const { return bOverrideResimulationErrorRotationThreshold ? ResimulationErrorRotationThreshold : DefaultValue; }

	// Overrides Project Settings -> Physics -> Replication -> Physics Prediction -> Resimulation Error Linear Velocity Threshold -- Velocity difference in centimeters / second that the object is allowed to desync from the server before triggering a resimulation, within this threshold runtime correction can be performed if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimulationErrorLinearVelocityThreshold"))
	float ResimulationErrorLinearVelocityThreshold = 5.0f;
	float GetResimulationErrorLinearVelocityThreshold(float DefaultValue) const { return bOverrideResimulationErrorLinearVelocityThreshold ? ResimulationErrorLinearVelocityThreshold : DefaultValue; }

	// Overrides Project Settings -> Physics -> Replication -> Physics Prediction -> Resimulation Error Angular Velocity Threshold -- Degrees / second that the object is allowed to desync from the server before triggering a resimulation, within this threshold runtime correction can be performed if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideResimulationErrorAngularVelocityThreshold"))
	float ResimulationErrorAngularVelocityThreshold = 2.0f;
	float GetResimulationErrorAngularVelocityThreshold(float DefaultValue) const { return bOverrideResimulationErrorAngularVelocityThreshold ? ResimulationErrorAngularVelocityThreshold : DefaultValue; }

	// Overrides CVar: np2.Resim.RuntimeCorrectionEnabled -- Apply positional and rotational runtime corrections while within resim trigger distance.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRuntimeCorrectionEnabled"))
	bool bRuntimeCorrectionEnabled = PhysicsReplicationCVars::ResimulationCVars::bRuntimeCorrectionEnabled;
	bool GetRuntimeCorrectionEnabled() const { return bOverrideRuntimeCorrectionEnabled ? bRuntimeCorrectionEnabled : PhysicsReplicationCVars::ResimulationCVars::bRuntimeCorrectionEnabled; }

	// Overrides CVar: np2.Resim.RuntimeVelocityCorrection -- Apply linear and angular velocity corrections in runtime while within resim trigger distance. Used if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRuntimeVelocityCorrection"))
	bool bRuntimeVelocityCorrection = PhysicsReplicationCVars::ResimulationCVars::bRuntimeVelocityCorrection;
	bool GetRuntimeVelocityCorrectionEnabled() const { return bOverrideRuntimeVelocityCorrection ? bRuntimeVelocityCorrection : PhysicsReplicationCVars::ResimulationCVars::bRuntimeVelocityCorrection; }

	// Overrides CVar: np2.Resim.RuntimeCorrectConnectedBodies -- If true runtime position and rotation correction will also shift transform of any connected physics objects. Used if RuntimeCorrectionEnabled is true.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRuntimeCorrectConnectedBodies"))
	bool bRuntimeCorrectConnectedBodies = PhysicsReplicationCVars::ResimulationCVars::bRuntimeCorrectConnectedBodies;
	bool GetRuntimeCorrectConnectedBodies() const { return bOverrideRuntimeCorrectConnectedBodies ? bRuntimeCorrectConnectedBodies : PhysicsReplicationCVars::ResimulationCVars::bRuntimeCorrectConnectedBodies; }

	// Overrides CVar: np2.Resim.PosStabilityMultiplier -- Recommended range between 0.0-1.0. Lower value means more stable positional corrections.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridePosStabilityMultiplier"))
	float PosStabilityMultiplier = PhysicsReplicationCVars::ResimulationCVars::PosStabilityMultiplier;
	float GetPosStabilityMultiplier() { return bOverridePosStabilityMultiplier ? PosStabilityMultiplier : PhysicsReplicationCVars::ResimulationCVars::PosStabilityMultiplier; }

	// Overrides CVar: np2.Resim.RotStabilityMultiplier -- Recommended range between 0.0-1.0. Lower value means more stable rotational corrections.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRotStabilityMultiplier"))
	float RotStabilityMultiplier = PhysicsReplicationCVars::ResimulationCVars::RotStabilityMultiplier;
	float GetRotStabilityMultiplier() const { return bOverrideRotStabilityMultiplier ? RotStabilityMultiplier : PhysicsReplicationCVars::ResimulationCVars::RotStabilityMultiplier; }

	// Overrides CVar: np2.Resim.VelStabilityMultiplier -- Recommended range between 0.0-1.0. Lower value means more stable linear velocity corrections.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideVelStabilityMultiplier"))
	float VelStabilityMultiplier = PhysicsReplicationCVars::ResimulationCVars::VelStabilityMultiplier;
	float GetVelStabilityMultiplier() const { return bOverrideVelStabilityMultiplier ? VelStabilityMultiplier : PhysicsReplicationCVars::ResimulationCVars::VelStabilityMultiplier; }

	// Overrides CVar: np2.Resim.AngVelStabilityMultiplier -- Recommended range between 0.0-1.0. Lower value means more stable angular velocity corrections.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideAngVelStabilityMultiplier"))
	float AngVelStabilityMultiplier = PhysicsReplicationCVars::ResimulationCVars::AngVelStabilityMultiplier;
	float GetAngVelStabilityMultiplier() const { return bOverrideAngVelStabilityMultiplier ? AngVelStabilityMultiplier : PhysicsReplicationCVars::ResimulationCVars::AngVelStabilityMultiplier; }

	UPROPERTY(EditDefaultsOnly, Category = "Overrides", meta = (DisplayName = "Post-Resimulation Error Correction Render Interpolation Settings"))
	FNetworkPhysicsSettingsResimulationErrorCorrection ResimulationErrorCorrectionSettings;
};

USTRUCT()
struct FNetworkPhysicsSettingsNetworkPhysicsComponent
{
	FNetworkPhysicsSettingsNetworkPhysicsComponent()
		: bOverrideRedundantInputs(0)
		, bOverrideRedundantRemoteInputs(0)
		, bOverrideRedundantStates(0)
		, bOverrideDynamicInputBufferScalingEnabled(0)
		, bOverrideCompareStateToTriggerRewind(0)
		, bOverridebCompareStateToTriggerRewindIncludeSimProxies(0)
		, bOverrideCompareInputToTriggerRewind(0)
		, bOverrideEnableUnreliableFlow(0)
		, bOverrideEnableReliableFlow(0)
		, bOverrideApplyDataInsteadOfMergeData(0)
		, bOverrideAllowInputExtrapolation(0)
		, bOverrideValidateDataOnGameThread(0)
		, bOverrideTriggerResimOnInputReceive(0)
		, bOverrideApplySimProxyStateAtRuntime(0)
		, bOverrideApplySimProxyInputAtRuntime(0)
		, bOverrideApplyInputDecaySimProxyInputAtRuntime(0)
		, bOverrideInputDecaySimProxyInputAtRuntime(0)
		, bOverrideEnableLagScalingSimProxyRuntimeInputDecay(0)
		, bOverrideApplyInputDecayOverSetTime(0)
		, bOverrideInputDecaySetTime(0)
		, bOverrideEnableLagScalingInputDecay(0)
		, bOverrideInputDecayReferenceLagMs(0)
	{};

	GENERATED_BODY()

	void Initialize()
	{
		if (InputDecayCurve.EditorCurveData.GetNumKeys() == 0)
		{
			InputDecayCurve.GetRichCurve()->AddKey(0.0f, 0.0f);
			InputDecayCurve.GetRichCurve()->AddKey(0.5f, 1.0f);
			InputDecayCurve.GetRichCurve()->AddKey(1.0f, 1.0f);
		}
	}

	// Override properties
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRedundantInputs : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRedundantRemoteInputs : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideRedundantStates : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideDynamicInputBufferScalingEnabled : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCompareStateToTriggerRewind : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverridebCompareStateToTriggerRewindIncludeSimProxies : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideCompareInputToTriggerRewind : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideEnableUnreliableFlow : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideEnableReliableFlow : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideApplyDataInsteadOfMergeData : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideAllowInputExtrapolation : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideValidateDataOnGameThread : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideTriggerResimOnInputReceive : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideApplySimProxyStateAtRuntime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideApplySimProxyInputAtRuntime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideApplyInputDecaySimProxyInputAtRuntime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideInputDecaySimProxyInputAtRuntime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideEnableLagScalingSimProxyRuntimeInputDecay : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideApplyInputDecayOverSetTime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideInputDecaySetTime : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideEnableLagScalingInputDecay : 1;
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (InlineEditConditionToggle))
	uint32 bOverrideInputDecayReferenceLagMs : 1;

	/* Overrides CVar : np2.Resim.RedundantInputs -- How many extra inputs to send with each unreliable network message, to account for packetloss.From owning client to server and server to owning client.
	 * NOTE: This is disabled while np2.Resim.DynamicInputReplicationScaling.Enabled is enabled. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRedundantInputs"))
	uint16 RedundantInputs = 2;
	const uint16 GetRedundantInputs() const { return bOverrideRedundantInputs ? RedundantInputs : (uint16)PhysicsReplicationCVars::ResimulationCVars::RedundantInputs; }

	// Overrides CVar: np2.Resim.RedundantRemoteInputs -- How many extra inputs to send with each unreliable network message, to account for packetloss. From Server to remote clients.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRedundantRemoteInputs"))
	uint16 RedundantRemoteInputs = 1;
	const uint16 GetRedundantRemoteInputs() const { return bOverrideRedundantRemoteInputs ? RedundantRemoteInputs : (uint16)PhysicsReplicationCVars::ResimulationCVars::RedundantRemoteInputs; }

	// Overrides CVar: np2.Resim.RedundantStates -- How many extra states to send with each unreliable network message, to account for packetloss. From Server to remote clients.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideRedundantStates"))
	uint16 RedundantStates = 0;
	const uint16 GetRedundantStates() const { return bOverrideRedundantStates ? RedundantStates : (uint16)PhysicsReplicationCVars::ResimulationCVars::RedundantStates; }

	// Overrides CVar: np2.Resim.DynamicInputBufferScaling.Enabled -- Enable dynmic scaling of input buffer on the server.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideDynamicInputBufferScalingEnabled"))
	bool bDynamicInputBufferScalingEnabled = PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingEnabled;
	const bool GetDynamicInputBufferScalingEnabled() const { return bOverrideDynamicInputBufferScalingEnabled ? bDynamicInputBufferScalingEnabled : PhysicsReplicationCVars::ResimulationCVars::bDynamicInputBufferScalingEnabled; }

	// Overrides CVar: np2.Resim.CompareStateToTriggerRewind -- When true, cache local FNetworkPhysicsData state in rewind history and compare the predicted state with incoming server state to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCompareStateToTriggerRewind"))
	bool bCompareStateToTriggerRewind = PhysicsReplicationCVars::ResimulationCVars::bCompareStateToTriggerRewind;
	const bool GetCompareStateToTriggerRewind(const bool bDefaultValue) const { return bOverrideCompareStateToTriggerRewind ? bCompareStateToTriggerRewind : bDefaultValue; }

	// Overrides CVar: np2.Resim.CompareStateToTriggerRewind.IncludeSimProxies -- When true, include simulated proxies when np2.Resim.CompareStateToTriggerRewind is enabled.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverridebCompareStateToTriggerRewindIncludeSimProxies"))
	bool bCompareStateToTriggerRewindIncludeSimProxies = PhysicsReplicationCVars::ResimulationCVars::bCompareStateToTriggerRewindIncludeSimProxies;
	const bool GetCompareStateToTriggerRewindIncludeSimProxies(const bool bDefaultValue) const { return bOverridebCompareStateToTriggerRewindIncludeSimProxies ? bCompareStateToTriggerRewindIncludeSimProxies : bDefaultValue; }

	// Overrides CVar: np2.Resim.CompareInputToTriggerRewind -- When true, compare local predicted FNetworkPhysicsData input with incoming server input to trigger resimulations if they differ, comparison done through FNetworkPhysicsData::CompareData.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideCompareInputToTriggerRewind"))
	bool bCompareInputToTriggerRewind = PhysicsReplicationCVars::ResimulationCVars::bCompareInputToTriggerRewind;
	const bool GetCompareInputToTriggerRewind(const bool bDefaultValue) const { return bOverrideCompareInputToTriggerRewind ? bCompareInputToTriggerRewind : bDefaultValue; }

	// Overrides CVar: np2.Resim.EnableUnreliableFlow -- When true, allow data to be sent unreliably. Also sends FNetworkPhysicsData not marked with FNetworkPhysicsData::bimportant unreliably over the network.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideEnableUnreliableFlow"))
	bool bEnableUnreliableFlow = PhysicsReplicationCVars::ResimulationCVars::bEnableUnreliableFlow;
	const bool GetEnableUnreliableFlow() const { return bOverrideEnableUnreliableFlow ? bEnableUnreliableFlow : PhysicsReplicationCVars::ResimulationCVars::bEnableUnreliableFlow; }

	// Overrides CVar: np2.Resim.EnableReliableFlow -- EXPERIMENTAL -- When true, allow data to be sent reliably. Also send FNetworkPhysicsData marked with FNetworkPhysicsData::bimportant reliably over the network.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideEnableReliableFlow"))
	bool bEnableReliableFlow = PhysicsReplicationCVars::ResimulationCVars::bEnableReliableFlow;
	const bool GetEnableReliableFlow() const { return bOverrideEnableReliableFlow ? bEnableReliableFlow : PhysicsReplicationCVars::ResimulationCVars::bEnableReliableFlow; }

	// Overrides CVar: np2.Resim.ApplyDataInsteadOfMergeData -- When true, call ApplyData for each data instead of MergeData when having to use multiple data entries in one frame.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideApplyDataInsteadOfMergeData"))
	bool bApplyDataInsteadOfMergeData = PhysicsReplicationCVars::ResimulationCVars::bApplyDataInsteadOfMergeData;
	const bool GetApplyDataInsteadOfMergeData() const { return bOverrideApplyDataInsteadOfMergeData ? bApplyDataInsteadOfMergeData : PhysicsReplicationCVars::ResimulationCVars::bApplyDataInsteadOfMergeData; }

	// Overrides CVar: np2.Resim.AllowInputExtrapolation -- When true and not locally controlled, allow inputs to be extrapolated from last known and if there is a gap allow interpolation between two known inputs.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideAllowInputExtrapolation"))
	bool bAllowInputExtrapolation = PhysicsReplicationCVars::ResimulationCVars::bAllowInputExtrapolation;
	const bool GetAllowInputExtrapolation() const { return bOverrideAllowInputExtrapolation ? bAllowInputExtrapolation : PhysicsReplicationCVars::ResimulationCVars::bAllowInputExtrapolation; }

	// Overrides CVar: np2.Resim.ValidateDataOnGameThread -- When true, perform server-side input validation through FNetworkPhysicsData::ValidateData on the Game Thread. If false, perform the call on the Physics Thread.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideValidateDataOnGameThread"))
	bool bValidateDataOnGameThread = PhysicsReplicationCVars::ResimulationCVars::bValidateDataOnGameThread;
	const bool GetValidateDataOnGameThread() const { return bOverrideValidateDataOnGameThread ? bValidateDataOnGameThread : PhysicsReplicationCVars::ResimulationCVars::bValidateDataOnGameThread; }

	// Overrides CVar: np2.Resim.TriggerResimOnInputReceive -- When true, a resimulation will be requested to the frame of the latest frame of received inputs this frame
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideTriggerResimOnInputReceive"))
	bool bTriggerResimOnInputReceive = PhysicsReplicationCVars::ResimulationCVars::bTriggerResimOnInputReceive;
	const bool GetTriggerResimOnInputReceive() const { return bOverrideTriggerResimOnInputReceive ? bTriggerResimOnInputReceive : PhysicsReplicationCVars::ResimulationCVars::bTriggerResimOnInputReceive; }

	// Overrides CVar: np2.Resim.ApplySimProxyStateAtRuntime -- When true, call ApplyData on received states for simulated proxies at runtime.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideApplySimProxyStateAtRuntime"))
	bool bApplySimProxyStateAtRuntime = PhysicsReplicationCVars::ResimulationCVars::bApplySimProxyStateAtRuntime;
	const bool GetApplySimProxyStateAtRuntime() const { return bOverrideApplySimProxyStateAtRuntime ? bApplySimProxyStateAtRuntime : PhysicsReplicationCVars::ResimulationCVars::bApplySimProxyStateAtRuntime; }

	// Overrides CVar: np2.Resim.ApplySimProxyInputAtRuntime -- When true, call ApplyData on received inputs for simulated proxies at runtime.
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideApplySimProxyInputAtRuntime"))
	bool bApplySimProxyInputAtRuntime = PhysicsReplicationCVars::ResimulationCVars::bApplySimProxyInputAtRuntime;
	const bool GetApplySimProxyInputAtRuntime() const { return bOverrideApplySimProxyInputAtRuntime ? bApplySimProxyInputAtRuntime : PhysicsReplicationCVars::ResimulationCVars::bApplySimProxyInputAtRuntime; }

	// Overrides CVar: np2.Resim.ApplyInputDecaySimProxyInputAtRuntime -- When true, apply input decay on inputs applied on simulated proxies at runtime (outside of resimulation)
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideApplyInputDecaySimProxyInputAtRuntime"))
	bool bApplyInputDecaySimProxyInputAtRuntime = PhysicsReplicationCVars::ResimulationCVars::bApplyInputDecaySimProxyInputAtRuntime;
	const bool GetApplyInputDecaySimProxyInputAtRuntime() const { return bOverrideApplyInputDecaySimProxyInputAtRuntime ? bApplyInputDecaySimProxyInputAtRuntime : PhysicsReplicationCVars::ResimulationCVars::bApplyInputDecaySimProxyInputAtRuntime; }

	// Overrides CVar: np2.Resim.InputDecaySimProxyInputAtRuntime -- The amount of input decay to apply on simulated proxy inputs at runtime (outside of resim), if np2.Resim.ApplyInputDecaySimProxyInputAtRuntime is true
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideInputDecaySimProxyInputAtRuntime"))
	float InputDecaySimProxyInputAtRuntime = PhysicsReplicationCVars::ResimulationCVars::InputDecaySimProxyInputAtRuntime;
	const float GetInputDecaySimProxyInputAtRuntime() const { return bOverrideInputDecaySimProxyInputAtRuntime ? InputDecaySimProxyInputAtRuntime : PhysicsReplicationCVars::ResimulationCVars::InputDecaySimProxyInputAtRuntime; }

	// Overrides CVar: np2.Resim.EnableLagScalingSimProxyRuntimeInputDecay -- When true, scales the input decay applied on simulated proxies at runtime (outside of resimulation) as a function of measured lag (forward prediction time)
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideEnableLagScalingSimProxyRuntimeInputDecay"))
	bool bEnableLagScalingSimProxyRuntimeInputDecay = PhysicsReplicationCVars::ResimulationCVars::bEnableLagScalingSimProxyRuntimeInputDecay;
	const bool GetEnableLagScalingSimProxyRuntimeInputDecay() const { return bOverrideEnableLagScalingSimProxyRuntimeInputDecay ? bEnableLagScalingSimProxyRuntimeInputDecay : PhysicsReplicationCVars::ResimulationCVars::bEnableLagScalingSimProxyRuntimeInputDecay; }

	// Overrides CVar: np2.Resim.ApplyInputDecayOverSetTime -- When true, apply the Input Decay Curve over a set amount of time instead of over the start of input prediction and end of resim which is variable each resimulation
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideApplyInputDecayOverSetTime"))
	bool bApplyInputDecayOverSetTime = PhysicsReplicationCVars::ResimulationCVars::bApplyInputDecayOverSetTime;
	const bool GetApplyInputDecayOverSetTime() const { return bOverrideApplyInputDecayOverSetTime ? bApplyInputDecayOverSetTime : PhysicsReplicationCVars::ResimulationCVars::bApplyInputDecayOverSetTime; }

	// Overrides CVar: np2.Resim.InputDecaySetTime -- Applied when np2.Resim.ApplyInputDecayOverSetTime is true, read there for more info. Set time to apply Input Decay Curve over while predicting inputs during resimulation
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideInputDecaySetTime"))
	float InputDecaySetTime = PhysicsReplicationCVars::ResimulationCVars::InputDecaySetTime;
	const float GetInputDecaySetTime() const { return bOverrideInputDecaySetTime ? InputDecaySetTime : PhysicsReplicationCVars::ResimulationCVars::InputDecaySetTime; }

	/* Curve for input decay during resimulation if input is being reused. 
	* XAxis = Alpha value from 0.0 to 1.0 where 0 is the start of reusing input and 1 is the last time we will reuse the input this resimulation.
	* YAxis = The Input Decay value from 0.0 to 1.0 (as a percentage where 1.0 = 100% decay) for the given Alpha. */
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (XAxisName="LerpAlpha", YAxisName="DecayValue"))
	FRuntimeFloatCurve InputDecayCurve;
	const FRuntimeFloatCurve& GetInputDecayCurve() const { return InputDecayCurve; }

	/* Overrides CVar: np2.Resim.EnableLagScalingInputDecay -- When true, scales input decay as a function of measured lag (i.e. forward prediction time). The greater the lag, the greater the decay. 
	 * The decay will be exactly the tuned InputDecay when the measured lag is equal to the reference lag (InputDecayReferenceLagMs)
	 **/
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideEnableLagScalingInputDecay"))
	bool bEnableLagScalingInputDecay = PhysicsReplicationCVars::ResimulationCVars::bEnableLagScalingInputDecay;
	const bool GetEnableLagScalingInputDecay() const { return bOverrideEnableLagScalingInputDecay ? bEnableLagScalingInputDecay : PhysicsReplicationCVars::ResimulationCVars::bEnableLagScalingInputDecay; }

	/** Overrides CVar: np2.Resim.InputDecayReferenceLagMs -- Applied when np2.Resim.EnableLagScalingInputDecay is true. Reference lag at which the decay will be exactly InputDecay. Linearly interpolated below, reaching 0 decay at 0 lag.
	*   Tries to maintain behavior above that lag, by decaying input more to travel the same distance over greater time for the same input.
	*/
	UPROPERTY(EditDefaultsOnly, Category = "Overrides", Meta = (EditCondition = "bOverrideInputDecayReferenceLagMs"))
	float InputDecayReferenceLagMs = PhysicsReplicationCVars::ResimulationCVars::InputDecayReferenceLagMs;
	const float GetInputDecayReferenceLagMs() const { return bOverrideInputDecayReferenceLagMs ? InputDecayReferenceLagMs : PhysicsReplicationCVars::ResimulationCVars::InputDecayReferenceLagMs; }
};

/*
USTRUCT()
struct FNetworkPhysicsSettingsRewindData
{
	GENERATED_BODY()
};

USTRUCT()
struct FNetworkPhysicsSettingsRenderInterpolation
{
	GENERATED_BODY()
};
*/

USTRUCT()
struct FNetworkPhysicsSettingsData
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettings GeneralSettings;

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsDefaultReplication DefaultReplicationSettings;

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsPredictiveInterpolation PredictiveInterpolationSettings;

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsResimulation ResimulationSettings;

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsNetworkPhysicsComponent NetworkPhysicsComponentSettings;
	/*
	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsRewindData RewindSettings;

	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsRenderInterpolation RenderInterpolationSettings;
	*/
};

UCLASS(BlueprintType)
class UNetworkPhysicsSettingsDataAsset : public UDataAsset
{
	GENERATED_BODY()

	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			Settings.NetworkPhysicsComponentSettings.Initialize();
		}
	}

public:
	
	void InitializeInternalSettings()
	{
		if (bInitialized)
		{
			return;
		}

		Settings_Internal = MakeShared<FNetworkPhysicsSettingsData>(Settings);

		bInitialized = true;
	}

	void MarkUninitialized()
	{
		bInitialized = false;
	}

	/** Get the settings, on the game thread */
	const FNetworkPhysicsSettingsData& GetSettings_External() const { return Settings; }
	
	/** Get the settings, on the physics thread */
	TWeakPtr<const FNetworkPhysicsSettingsData> GetSettings_Internal() const { return Settings_Internal.ToWeakPtr(); }

private:
	bool bInitialized = false;

	/** Game Thread Settings*/
	// Network Physics Settings
	UPROPERTY(EditDefaultsOnly, Category = "Networked Physics Settings")
	FNetworkPhysicsSettingsData Settings;

	/** Physics Thread Settings */
	TSharedPtr<FNetworkPhysicsSettingsData> Settings_Internal = nullptr;
};

/** Settings Component for network replicated physics actors
* Overrides default settings, CVar settings and project settings. */
UCLASS(BlueprintType, MinimalAPI, meta = (BlueprintSpawnableComponent))
class UNetworkPhysicsSettingsComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ENGINE_API UNetworkPhysicsSettingsComponent();
	~UNetworkPhysicsSettingsComponent() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	virtual void BeginPlay() override;

	/** Get the settings internal to the PhysicsThread (Only access construct on the Physics Thread) */
	FNetworkPhysicsSettingsComponentAsync* GetNetworkPhysicsSettings_Internal() const { return NetworkPhysicsSettings_Internal; };

private:
	UFUNCTION()
	void OnComponentPhysicsStateChanged(UPrimitiveComponent* ChangedComponent, EComponentPhysicsStateChange StateChange);
	
	/** Perform logic to register this actor into physics replication LOD, depending on settings */
	void RegisterInPhysicsReplicationLOD();

	/** Unregister this actor from physics replication LOD */
	void UnregisterFromPhysicsReplicationLOD();

	/** Callback for pawn controller changes to register/unregister as focal particle */
	UFUNCTION()
	void OnControllerChanged(APawn* InPawn, AController* OldController, AController* NewController);

public:
	/** Get the settings, on the game thread */
	const FNetworkPhysicsSettingsData& GetSettings() const { return SettingsDataAsset ? SettingsDataAsset->GetSettings_External() : SettingsNetworkPhysicsData_Default; }

	/** Get the settings, on the physics thread */
	TWeakPtr<const FNetworkPhysicsSettingsData> GetSettings_Internal() const { return SettingsDataAsset ? SettingsDataAsset->GetSettings_Internal() : nullptr; }

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Networked Physics Settings")
	TObjectPtr<UNetworkPhysicsSettingsDataAsset> SettingsDataAsset;

// Deprecated properties
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "FNetworkPhysicsSettings GeneralSettings has been deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead"))
	FNetworkPhysicsSettings GeneralSettings;
	
	UE_DEPRECATED(5.7, "FNetworkPhysicsSettingsDefaultReplication DefaultReplicationSettings has been deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead"))
	FNetworkPhysicsSettingsDefaultReplication DefaultReplicationSettings;

	UE_DEPRECATED(5.7, "FNetworkPhysicsSettingsPredictiveInterpolation PredictiveInterpolationSettings has been deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead"))
	FNetworkPhysicsSettingsPredictiveInterpolation PredictiveInterpolationSettings;
	
	UE_DEPRECATED(5.7, "FNetworkPhysicsSettingsResimulation ResimulationSettings has been deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead"))
	FNetworkPhysicsSettingsResimulation ResimulationSettings;
	
	UE_DEPRECATED(5.7, "FNetworkPhysicsSettingsNetworkPhysicsComponent NetworkPhysicsComponentSettings has been deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Deprecated, create a DataAsset of UNetworkPhysicsSettingsDataAsset type and reference that in the UNetworkPhysicsSettingsComponent instead"))
	FNetworkPhysicsSettingsNetworkPhysicsComponent NetworkPhysicsComponentSettings;
#endif

private:
	FNetworkPhysicsSettingsComponentAsync* NetworkPhysicsSettings_Internal;
	ENGINE_API static const FNetworkPhysicsSettingsData SettingsNetworkPhysicsData_Default;

	/* --- Static API --- */
private:
	// Game Thread map of settings component per actor
	static TMap<AActor*, UNetworkPhysicsSettingsComponent*> ObjectToSettings_External;

public:
	/** Get the settings component for a specified actor  */
	static UNetworkPhysicsSettingsComponent* GetSettingsForActor(AActor* Owner);
};






#pragma region // FNetworkPhysicsSettingsComponentAsync

struct FNetworkPhysicsSettingsAsync
{
	FNetworkPhysicsSettings GeneralSettings;
	FNetworkPhysicsSettingsDefaultReplication DefaultReplicationSettings;
	FNetworkPhysicsSettingsPredictiveInterpolation PredictiveInterpolationSettings;
	FNetworkPhysicsSettingsResimulation ResimulationSettings;
	FNetworkPhysicsSettingsNetworkPhysicsComponent NetworkPhysicsComponentSettings;
};

struct FNetworkPhysicsSettingsAsyncInput : public Chaos::FSimCallbackInput
{
	TOptional<Chaos::FConstPhysicsObjectHandle> PhysicsObject;
	TOptional<TWeakPtr<const FNetworkPhysicsSettingsData>> Settings_Internal;

	UE_DEPRECATED(5.7, "Deprecated, use Settings_Internal instead")
	TOptional<FNetworkPhysicsSettingsAsync> Settings;

	void Reset()
	{
		PhysicsObject.Reset();
		Settings_Internal.Reset();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Settings.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
};

class FNetworkPhysicsSettingsComponentAsync : public Chaos::TSimCallbackObject<FNetworkPhysicsSettingsAsyncInput, Chaos::FSimCallbackNoOutput, Chaos::ESimCallbackOptions::Presimulate>
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNetworkPhysicsSettingsComponentAsync();
	~FNetworkPhysicsSettingsComponentAsync() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void OnPreSimulate_Internal() override;

private:
	void ConsumeAsyncInput();
	void RegisterSettingsInPhysicsReplication();

public:
	TSharedPtr<const FNetworkPhysicsSettingsData> Settings_Internal;

	UE_DEPRECATED(5.7, "Deprecated, use Settings_Internal instead")
	FNetworkPhysicsSettingsAsync Settings;

private:
	Chaos::FConstPhysicsObjectHandle PhysicsObject;
};

#pragma endregion // FNetworkPhysicsSettingsComponentAsync
