// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/Tuple.h"

template<typename ImplType>
class FPhysScene_Base
{
public: 

	void Init() 
	{
		ConcreteScene.Init();
	}

	void Tick(float InDeltaSeconds)
	{
		ConcreteScene.Tick(InDeltaSeconds);
	}

	void SetKinematicUpdateFunction(TFunction<void(typename ImplType::DataType&, const float, const float, const int32)> KinematicUpdate)
	{
		ConcreteScene.SetKinematicUpdateFunction(KinematicUpdate);
	}

	void SetStartFrameFunction(TFunction<void(const float)> StartFrame)
	{
		ConcreteScene.SetStartFrameFunction(StartFrame);
	}

	void SetEndFrameFunction(TFunction<void(const float)> EndFrame)
	{
		ConcreteScene.SetEndFrameFunction(EndFrame);
	}

	void SetCreateBodiesFunction(TFunction<void(typename ImplType::DataType&)> CreateBodies)
	{
		ConcreteScene.SetCreateBodiesFunction(CreateBodies);
	}
	
	void SetParameterUpdateFunction(TFunction<void(typename ImplType::DataType&, const float, const int32)> ParameterUpdate)
	{
		ConcreteScene.SetParameterUpdateFunction(ParameterUpdate);
	}

	void SetDisableCollisionsUpdateFunction(TFunction<void(TSet<TTuple<int32, int32>>&)> DisableCollisionsUpdate)
	{
		ConcreteScene.SetDisableCollisionsUpdateFunction(DisableCollisionsUpdate);
	}

	void AddPBDConstraintFunction(TFunction<void(typename ImplType::DataType&, const float)> ConstraintFunction)
	{
		ConcreteScene.AddPBDConstraintFunction(ConstraintFunction);
	}

	void AddForceFunction(TFunction<void(typename ImplType::DataType&, const float, const int32)> ForceFunction)
	{
		ConcreteScene.AddForceFunction(ForceFunction);
	}

	ImplType& GetImpl()
	{
		return ConcreteScene;
	}

	const ImplType& GetImpl() const
	{
		return ConcreteScene;
	}

private:

	ImplType ConcreteScene;

};