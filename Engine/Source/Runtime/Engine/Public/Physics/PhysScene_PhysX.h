// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineGlobals.h"
#include "PhysicsPublic.h"
#include "PhysxUserData.h"
#include "PhysicsEngine/RigidBodyIndexPair.h"
#include "Physics/PhysicsInterfaceTypes.h"

/** Buffers used as scratch space for PhysX to avoid allocations during simulation */
struct FSimulationScratchBuffer2
{
	FSimulationScratchBuffer2()
		: Buffer(nullptr)
		, BufferSize(0)
	{}

	// The scratch buffer
	uint8* Buffer;

	// Allocated size of the buffer
	int32 BufferSize;
};
