// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "EdMode.h"

class FPhysicsDebugMode : public FEdMode
{
public:
    FPhysicsDebugMode();
    ~FPhysicsDebugMode();

protected:
    // FEdMode interface
    virtual void Initialize() override;
    virtual void Enter() override;
    virtual void Exit() override;
    virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
    virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
    virtual bool InputAxis(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) override;
    virtual bool InputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& Drag, FRotator& Rotation, FVector& Scale) override;
    virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
    virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
    virtual void Render(const FSceneView* SceneView, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
    virtual void PostUndo() override;
    virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) override;
    virtual bool ShouldDrawWidget() const override;
};

#endif
