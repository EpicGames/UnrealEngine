// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "PhysicsPublic.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "Chaos/ChaosEngineInterface.h"

/** 
 * Set of flags stored in the PhysX FilterData
 *
 * When this flag is saved in CreateShapeFilterData or CreateQueryFilterData, we only use 23 bits
 * If you plan to use more than 23 bits, you'll also need to change the format of ShapeFilterData,QueryFilterData
 * Make sure you also change preFilter/SimFilterShader where it's used
 */
enum EPhysXFilterDataFlags
{

	EPDF_SimpleCollision	=	0x0001,
	EPDF_ComplexCollision	=	0x0002,
	EPDF_CCD				=	0x0004,
	EPDF_ContactNotify		=	0x0008,
	EPDF_StaticShape		=	0x0010,
	EPDF_ModifyContacts		=   0x0020,
	EPDF_KinematicKinematicPairs = 0x0040,
};


// Bit counts for Word3 of filter data.
// (ExtraFilter (top NumExtraFilterBits) + MyChannel (next NumCollisionChannelBits) as ECollisionChannel + Flags (remaining NumFilterDataFlagBits)
// [NumExtraFilterBits] [NumCollisionChannelBits] [NumFilterDataFlagBits] = 32 bits
enum { NumCollisionChannelBits = 5 };
enum { NumFilterDataFlagBits = 32 - NumExtraFilterBits - NumCollisionChannelBits };


struct FPhysicsFilterBuilder
{
	UE_INTERNAL FPhysicsFilterBuilder() = default;
	FPhysicsFilterBuilder(const FPhysicsFilterBuilder&) = default;
	FPhysicsFilterBuilder(FPhysicsFilterBuilder&&) = default;
	ENGINE_API FPhysicsFilterBuilder(const FChaosScene* InScene);
	UE_INTERNAL ENGINE_API FPhysicsFilterBuilder(const Chaos::Filter::FShapeFilterData& InShapeFilter, const Chaos::Filter::FInstanceData& InInstanceData);
	ENGINE_API FPhysicsFilterBuilder(const FChaosScene* InScene, const Chaos::Filter::FShapeFilterData& InShapeFilter, const Chaos::Filter::FInstanceData& InInstanceData);
	UE_DEPRECATED(5.8, "Use The constructor that takes a scene instead")
	ENGINE_API FPhysicsFilterBuilder(TEnumAsByte<enum ECollisionChannel> InObjectType, FMaskFilter MaskFilter, const struct FCollisionResponseContainer& ResponseToChannels);

	FPhysicsFilterBuilder& operator=(const FPhysicsFilterBuilder&) = default;
	FPhysicsFilterBuilder& operator=(FPhysicsFilterBuilder&&) = default;

	ENGINE_API FPhysicsFilterBuilder& SetCollisionChannelIndex(const TEnumAsByte<enum ECollisionChannel> InObjectType);
	ENGINE_API FPhysicsFilterBuilder& SetResponses(const FCollisionResponseContainer& InResponses);
	ENGINE_API FPhysicsFilterBuilder& SetMaskFilter(const FMaskFilter InMaskFilter);
	ENGINE_API FPhysicsFilterBuilder& SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled);
	ENGINE_API FPhysicsFilterBuilder& SetOwnerID(const uint32 InOwnerID);
	ENGINE_API FPhysicsFilterBuilder& SetComponentID(const uint32 InComponentID);

	ENGINE_API Chaos::Filter::FShapeFilterData BuildShapeFilterData() const;
	ENGINE_API Chaos::Filter::FInstanceData BuildInstanceData() const;

	UE_DEPRECATED(5.8, "Use SetFlags")
	inline void ConditionalSetFlags(EPhysXFilterDataFlags Flag, bool bEnabled)
	{
		if (bEnabled)
		{
			SetFlags((Chaos::EFilterFlags)Flag, bEnabled);
		}
	}

	UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead")
	inline void GetQueryData(uint32 SourceObjectID, uint32& OutWord0, uint32& OutWord1, uint32& OutWord2, uint32& OutWord3) const
	{
		/**
		 * Format for QueryData : 
		 *		word0 (object ID)
		 *		word1 (blocking channels)
		 *		word2 (touching channels)
		 *		word3 (ExtraFilter (top NumExtraFilterBits) + MyChannel (next NumCollisionChannelBits) as ECollisionChannel + Flags (remaining NumFilterDataFlagBits)
		 */
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData& FilterData = BuildShapeFilterData();
		OutWord0 = SourceObjectID;
		OutWord1 = (uint32)FilterData.GetBlockChannels();
		OutWord2 = (uint32)FilterData.GetOverlapChannels();
		OutWord3 = FilterData.GetLegacyWord3();
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	}

	UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead")
	inline void GetSimData(uint32 BodyIndex, uint32 ComponentID, uint32& OutWord0, uint32& OutWord1, uint32& OutWord2, uint32& OutWord3) const
	{
		/**
		 * Format for SimData : 
		 * 		word0 (body index)
		 *		word1 (blocking channels)
		 *		word2 (skeletal mesh component ID)
		 *		word3 (ExtraFilter (top NumExtraFilterBits) + MyChannel (next NumCollisionChannelBits) as ECollisionChannel + Flags (remaining NumFilterDataFlagBits)
		 */
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData& FilterData = BuildShapeFilterData();
		OutWord0 = BodyIndex;
		OutWord1 = (uint32)FilterData.GetBlockChannels();
		OutWord2 = ComponentID;
		OutWord3 = FilterData.GetLegacyWord3();
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	}

	UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead")
	inline void GetCombinedData(uint32& OutBlockingBits, uint32& OutTouchingBits, uint32& OutObjectTypeAndFlags) const
	{
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData& FilterData = BuildShapeFilterData();
		OutBlockingBits = (uint32)FilterData.GetBlockChannels();
		OutTouchingBits = (uint32)FilterData.GetOverlapChannels();
		OutObjectTypeAndFlags = FilterData.GetLegacyWord3();
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	}

private:
	Chaos::Filter::FInstanceData InstanceData;
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Chaos::Filter::FShapeFilterBuilder ShapeFilterBuilder;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
};

struct FPhysicsObjectQueryFilterBuilder
{
	UE_INTERNAL FPhysicsObjectQueryFilterBuilder() = default;
	FPhysicsObjectQueryFilterBuilder(const FPhysicsObjectQueryFilterBuilder&) = default;
	FPhysicsObjectQueryFilterBuilder(FPhysicsObjectQueryFilterBuilder&&) = default;
	ENGINE_API FPhysicsObjectQueryFilterBuilder(const FChaosScene* InScene);

	FPhysicsObjectQueryFilterBuilder& operator=(const FPhysicsObjectQueryFilterBuilder&) = default;
	FPhysicsObjectQueryFilterBuilder& operator=(FPhysicsObjectQueryFilterBuilder&&) = default;

	ENGINE_API FPhysicsObjectQueryFilterBuilder& SetObjectTypes(const uint64 ObjectTypes);
	ENGINE_API FPhysicsObjectQueryFilterBuilder& SetMultiQuery(const bool bMultiQuery);
	ENGINE_API FPhysicsObjectQueryFilterBuilder& SetMaskFilter(const uint8 InMaskFilter);
	ENGINE_API FPhysicsObjectQueryFilterBuilder& SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled);

	ENGINE_API Chaos::Filter::FQueryFilterData Build() const;

private:
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Chaos::Filter::FQueryObjectFilterBuilder Builder;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
};

struct FPhysicsTraceQueryFilterBuilder
{
	UE_INTERNAL FPhysicsTraceQueryFilterBuilder() = default;
	FPhysicsTraceQueryFilterBuilder(const FPhysicsTraceQueryFilterBuilder&) = default;
	FPhysicsTraceQueryFilterBuilder(FPhysicsTraceQueryFilterBuilder&&) = default;
	ENGINE_API FPhysicsTraceQueryFilterBuilder(const FChaosScene* InScene);

	FPhysicsTraceQueryFilterBuilder& operator=(const FPhysicsTraceQueryFilterBuilder&) = default;
	FPhysicsTraceQueryFilterBuilder& operator=(FPhysicsTraceQueryFilterBuilder&&) = default;

	ENGINE_API FPhysicsTraceQueryFilterBuilder& SetCollisionChannelIndex(const TEnumAsByte<enum ECollisionChannel> InObjectType);
	ENGINE_API FPhysicsTraceQueryFilterBuilder& SetResponses(const FCollisionResponseContainer& InResponses);
	ENGINE_API FPhysicsTraceQueryFilterBuilder& SetMaskFilter(const FMaskFilter InMaskFilter);
	ENGINE_API FPhysicsTraceQueryFilterBuilder& SetFlags(const Chaos::EFilterFlags InFlags, bool bEnabled);

	ENGINE_API Chaos::Filter::FQueryFilterData Build() const;

private:
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	Chaos::Filter::FQueryTraceFilterBuilder Builder;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
};

/** Utility for creating a FCollisionFilterData for filtering query (trace) and sim (physics) from the Unreal filtering info. */

UE_DEPRECATED(5.8, "Use FPhysicsFilterBuilder directly instead")
inline void CreateShapeFilterData(
	const uint8 MyChannel,
	const FMaskFilter MaskFilter,
	const int32 SourceObjectID,
	const FCollisionResponseContainer& ResponseToChannels,
	uint32 ComponentID,
	uint16 BodyIndex,
	FCollisionFilterData& OutQueryData,
	FCollisionFilterData& OutSimData,
	bool bEnableCCD,
	bool bEnableContactNotify,
	bool bStaticShape,
	bool bModifyContacts = false)
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	FPhysicsFilterBuilder Builder;
	Builder.SetCollisionChannelIndex((ECollisionChannel)MyChannel);
	Builder.SetMaskFilter(MaskFilter);
	Builder.SetResponses(ResponseToChannels);
	Builder.SetFlags(Chaos::EFilterFlags::CCD, bEnableCCD);
	Builder.SetFlags(Chaos::EFilterFlags::ContactNotify, bEnableContactNotify);
	Builder.SetFlags(Chaos::EFilterFlags::StaticShape, bStaticShape);
	Builder.SetFlags(Chaos::EFilterFlags::ModifyContacts, bModifyContacts);
	Builder.SetOwnerID(SourceObjectID);
	Builder.SetComponentID(ComponentID);

	const Chaos::Filter::FCombinedShapeFilterData CombinedShapeFilter(Builder.BuildShapeFilterData(), Builder.BuildInstanceData());
	Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeFilter(CombinedShapeFilter, OutQueryData, OutSimData);
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}

ENGINE_API ECollisionChannel GetCollisionChannel(const Chaos::Filter::FShapeFilterData& ShapeFilterData);
ENGINE_API ECollisionChannel GetCollisionChannel(const Chaos::Filter::FQueryFilterData& QueryFilterData);
ENGINE_API bool HasCollisionChannel(const Chaos::Filter::FShapeFilterData& ShapeFilterData, ECollisionChannel Channel);
ENGINE_API bool HasCollisionChannel(const Chaos::Filter::FQueryFilterData& QueryFilterData, ECollisionChannel Channel);

inline ECollisionChannel GetCollisionChannel(uint32 Word3)
{
	uint32 ChannelMask = (Word3 << NumExtraFilterBits) >> (32 - NumCollisionChannelBits);
	return (ECollisionChannel)ChannelMask;
}

UE_DEPRECATED(5.8, "Use the methods on Chaos::Filter::FShapeFilterData instead")
inline ECollisionChannel GetCollisionChannelAndExtraFilter(uint32 Word3, FMaskFilter& OutMaskFilter)
{
	uint32 ChannelMask = GetCollisionChannel(Word3);
	OutMaskFilter = Word3 >> (32 - NumExtraFilterBits);
	return (ECollisionChannel)ChannelMask;
}

UE_DEPRECATED(5.8, "Use FPhysicsFilterBuilder directly instead")
inline uint32 CreateChannelAndFilter(ECollisionChannel CollisionChannel, FMaskFilter MaskFilter)
{
	uint32 ResultMask = (uint32(MaskFilter) << NumCollisionChannelBits) | (uint32)CollisionChannel;
	return ResultMask << NumFilterDataFlagBits;
}

UE_DEPRECATED(5.8, "Use FPhysicsFilterBuilder directly instead")
inline void UpdateMaskFilter(uint32& Word3, FMaskFilter NewMaskFilter)
{
	static_assert(NumExtraFilterBits <= 8, "Only up to 8 extra filter bits are supported.");
	Word3 &= (0xFFFFFFFFu >> NumExtraFilterBits);	//we drop the top NumExtraFilterBits bits because that's where the new mask filter is going
	Word3 |= uint32(NewMaskFilter) << (32 - NumExtraFilterBits);
}

UE_DEPRECATED(5.8, "Use ExtractCollisionResponseContainer instead")
inline FCollisionResponseContainer ExtractSimCollisionResponseContainer(const FCollisionFilterData& InSimFilterData)
{
	FCollisionResponseContainer CollisionResponseContainer;
	
	for (int32 ChannelIndex = 0; ChannelIndex < UE_ARRAY_COUNT(CollisionResponseContainer.EnumArray); ++ChannelIndex)
	{
		const uint64 ChannelBit = 1LLU << ChannelIndex;
		if (uint64(InSimFilterData.Word1) & ChannelBit)
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Block;
		}
		else
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Ignore;
		}
	}

	return CollisionResponseContainer;
}

UE_DEPRECATED(5.8, "Use ExtractCollisionResponseContainer instead")
inline FCollisionResponseContainer ExtractQueryCollisionResponseContainer(const FCollisionFilterData& InQueryFilterData)
{
	FCollisionResponseContainer CollisionResponseContainer;
	
	for (int32 ChannelIndex = 0; ChannelIndex < UE_ARRAY_COUNT(CollisionResponseContainer.EnumArray); ++ChannelIndex)
	{
		const uint64 ChannelBit = 1LLU << ChannelIndex;
		if (uint64(InQueryFilterData.Word1) & ChannelBit)
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Block;
		}
		else if (uint64(InQueryFilterData.Word2) & ChannelBit)
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Overlap;
		}
		else
		{
			CollisionResponseContainer.EnumArray[ChannelIndex] = ECR_Ignore;
		}
	}

	return CollisionResponseContainer;
}

UE_INTERNAL ENGINE_API FCollisionResponseContainer ExtractCollisionResponseContainer(const Chaos::Filter::FShapeFilterData& ShapeFilterData);
