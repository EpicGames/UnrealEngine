// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/AggregateGeom.h"
#include "Containers/ArrayView.h"

struct FKSphereElem;

class UBodySetup;

/** Helper struct for iterating over shapes in a body setup.*/
struct FBodySetupShapeIterator
{
	ENGINE_API FBodySetupShapeIterator();

private:
	FVector Scale3D;
	const FTransform& RelativeTM;

	float MinScaleAbs;
	float MinScale;
	FVector ShapeScale3DAbs;
	FVector ShapeScale3D;

	float ContactOffsetFactor;
	float MinContactOffset;
	float MaxContactOffset;

	bool bDoubleSidedTriMeshGeo;
};
