// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineGlobals.h"
#include "Engine/EngineTypes.h"
#include "PhysicsInterfaceDeclares.h"
#include "BodySetupEnums.h"
#include "PhysicsInterfaceTypesCore.h"
#include "Chaos/Serializable.h"

namespace Chaos
{
	class FImplicitObjectUnion;
	class FImplicitObject;
	class FTriangleMeshImplicitObject;
}

class UPhysicalMaterialMask;
class UMaterialInterface;

// Defines for enabling hitch repeating (see ScopedSQHitchRepeater.h)
#if !UE_BUILD_SHIPPING
#define DETECT_SQ_HITCHES 1
#endif

#ifndef DETECT_SQ_HITCHES
#define DETECT_SQ_HITCHES 0
#endif

struct FKAggregateGeom;

struct FPhysicalMaterialMaskParams
{
	/** Physical materials mask */
	UPhysicalMaterialMask* PhysicalMaterialMask;

	/** Pointer to material which contains the physical material map */
	UMaterialInterface* PhysicalMaterialMap;
};

struct FGeometryAddParams
{
	FGeometryAddParams() = default;

	FGeometryAddParams(const FGeometryAddParams& Other)
	: bDoubleSided(Other.bDoubleSided)
	, CollisionData(Other.CollisionData)
	, CollisionTraceType(Other.CollisionTraceType)
	, Scale(Other.Scale)
	, SimpleMaterial(Other.SimpleMaterial)
	, ComplexMaterials(Other.ComplexMaterials)
	, ComplexMaterialMasks(Other.ComplexMaterialMasks)
	, LocalTransform(Other.LocalTransform)
	, WorldTransform(Other.WorldTransform)
	, Geometry(Other.Geometry)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, ChaosTriMeshes(Other.ChaosTriMeshes)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, TriMeshGeometries(Other.TriMeshGeometries)
	{}

	bool bDoubleSided;
	FBodyCollisionData CollisionData;
	ECollisionTraceFlag CollisionTraceType;
	FVector Scale;
	UPhysicalMaterial* SimpleMaterial;
	TArrayView<UPhysicalMaterial*> ComplexMaterials;
	TArrayView<FPhysicalMaterialMaskParams> ComplexMaterialMasks;
	FTransform LocalTransform;
	FTransform WorldTransform;
	FKAggregateGeom* Geometry;

	UE_DEPRECATED(5.4, "Please use TriMeshGeometries instead")
	TArrayView<TSharedPtr<Chaos::FTriangleMeshImplicitObject, ESPMode::ThreadSafe>> ChaosTriMeshes;

	TArrayView<Chaos::FTriangleMeshImplicitObjectPtr> TriMeshGeometries;
};