// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysXPublic.h"
#include "Containers/Union.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PhysicsInterfaceUtilsCore.h"
#include "PhysicsReplicationInterface.h"
#include "PhysicsReplicationLODInterface.h"
#include "Misc/CoreMiscDefines.h"

class FPhysScene_PhysX;
struct FConstraintInstance;

// FILTER DATA

/** Utility for creating a filter data object for performing a query (trace) against the scene */
UE_DEPRECATED(5.7, "Use CreateChaosQueryFilterData instead") ENGINE_API FCollisionFilterData CreateQueryFilterData(const uint8 MyChannel, const bool bTraceComplex, const FCollisionResponseContainer& InCollisionResponseContainer, const struct FCollisionQueryParams& QueryParam, const struct FCollisionObjectQueryParams & ObjectParam, const bool bMultitrace);
ENGINE_API Chaos::Filter::FQueryFilterData CreateChaosQueryFilterData(const uint8 MyChannel, const bool bTraceComplex, const FCollisionResponseContainer& InCollisionResponseContainer, const struct FCollisionQueryParams& QueryParam, const struct FCollisionObjectQueryParams & ObjectParam, const bool bMultitrace);

struct FConstraintBrokenDelegateData
{
	FConstraintBrokenDelegateData(FConstraintInstance* ConstraintInstance);

	void DispatchOnBroken()
	{
		OnConstraintBrokenDelegate.ExecuteIfBound(ConstraintIndex);
	}

	FOnConstraintBroken OnConstraintBrokenDelegate;
	int32 ConstraintIndex;
};

/** Interface for the creation of customized physics replication.*/
class IPhysicsReplicationFactory
{
public:

	virtual TUniquePtr<IPhysicsReplication> CreatePhysicsReplication(FPhysScene* OwningPhysScene) { return nullptr; }
	virtual TUniquePtr<IPhysicsReplicationLOD> CreatePhysicsReplicationLOD(FPhysScene* OwningPhysScene) { return nullptr; }
};
