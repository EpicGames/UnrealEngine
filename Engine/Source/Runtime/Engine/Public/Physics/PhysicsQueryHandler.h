// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Physics/SceneQueryData.h"

#include "PhysicsQueryHandler.generated.h"

struct FOverlapResult;
struct FHitResult;
struct FTraceHandle;

UENUM()
enum class EPhysicsQueryKind : uint8
{
	/** A physics query that is only a test (no results are returned). */
	Test = 0,
	/** A physics query that only returns the first hit. */
	Single,
	/** A physics query that returns all hits up to a blocking hit. */
	Multi,
};

UCLASS(hidecategories=(Object), MinimalAPI)
// Allows custom handling of physics queries. Primarily for handling network/server redirection.
class UPhysicsQueryHandler : public UObject
{
	GENERATED_BODY()

public:
	using EQueryInfo = Chaos::EQueryInfo;
	using EThreadQueryContext = Chaos::EThreadQueryContext;

	using FCommonQueryData = Chaos::FCommonQueryData;
	using FOverlapQueryData = Chaos::FOverlapQueryData;
	using FRayQueryData = Chaos::FRayQueryData;
	using FSweepQueryData = Chaos::FSweepQueryData;

	ENGINE_API virtual bool Overlap(const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData, TArray<FOverlapResult>& OutOverlaps);

	ENGINE_API virtual bool RaycastTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits);
	ENGINE_API virtual bool RaycastSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, FHitResult& OutHits);
	ENGINE_API virtual bool RaycastMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits);

	ENGINE_API virtual bool SweepTest(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits);
	ENGINE_API virtual bool SweepSingle(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, FHitResult& OutHits);
	ENGINE_API virtual bool SweepMulti(const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData, TArray<FHitResult>& OutHits);

	// Async
	ENGINE_API virtual void QueueAsyncOverlap(const FTraceHandle& TraceHandle, const EQueryInfo InfoType, const EThreadQueryContext ThreadContext, const UWorld* World, const FOverlapQueryData& OverlapData, const FCommonQueryData& CommonData);
	ENGINE_API virtual void QueueAsyncRaycast(const FTraceHandle& TraceHandle, const EPhysicsQueryKind QueryKind, const EThreadQueryContext ThreadContext, const UWorld* World, const FRayQueryData& RayData, const FCommonQueryData& CommonData);
	ENGINE_API virtual void QueueAsyncSweep(const FTraceHandle& TraceHandle, const EPhysicsQueryKind QueryKind, const EThreadQueryContext ThreadContext, const UWorld* World, const FSweepQueryData& SweepData, const FCommonQueryData& CommonData);

	ENGINE_API virtual bool AreAsyncRequestsAdded();
	ENGINE_API virtual void VerifyAsyncRequestsAreCompletedOrAbort();
};
