// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsReplicationCache.h 
	Cache physics state from PT on GT for actors using physics replication.
=============================================================================*/

#pragma once

#include "Chaos/SimCallbackObject.h"
#include "Engine/ReplicatedState.h"

class FPhysScene_Chaos;
class UPrimitiveComponent;

// -------------- Game Thread --------------

class FPhysicsReplicationCache
{
public:
	FPhysicsReplicationCache(FPhysScene_Chaos* InPhysicsScene);
	
	virtual ~FPhysicsReplicationCache();

	/** Get cached state for replication */
	const FRigidBodyState* GetStateFromReplicationCache(UPrimitiveComponent* RootComponent, int32& OutSolverFrame);

	/** Get cached state for replication */
	const FRigidBodyState* GetStateFromReplicationCache(Chaos::FPhysicsObjectHandle PhysicsObject, int32& OutSolverFrame);

	/** Register a component for physics replication state caching, the component will unregister automatically if cache is not accessed within time limit set by CVar: np2.ReplicationCache.LingerForNSeconds */
	void RegisterForReplicationCache(UPrimitiveComponent* RootComponent);
	
	/** Unregister a component from physics replication state caching */
	void UnregisterForReplicationCache(UPrimitiveComponent* RootComponent);

private:
	/** Process marshaled data from the physics thread */
	void ProcessAsyncOutput();

private:
	int32 SolverFrame = 0;
	TMap<Chaos::FConstPhysicsObjectHandle, FRigidBodyState> ReplicationCache_External;
	
	FPhysScene_Chaos* PhysicsScene;
	FDelegateHandle DelegateInjectInputs_External;

	/** Async part of the replication cache, only access on Physics Thread except for initialize and uninitialize */
	class FPhysicsReplicationCacheAsync* AsyncPhysicsReplicationCache;
};



// -------------- Async Marshaling --------------

/** Async Marshal Input */
struct FPhysicsReplicationCacheAsyncInput : public Chaos::FSimCallbackInput
{
	mutable TArray<Chaos::FConstPhysicsObjectHandle> AccessedObjects;
	TArray<Chaos::FConstPhysicsObjectHandle> UnregisterObjects;

	void Reset()
	{
		AccessedObjects.Reset();
		UnregisterObjects.Reset();
	}
};

/** Async Marshal Output */
struct FPhysicsReplicationCacheAsyncOutput : public Chaos::FSimCallbackOutput
{
	int32 SolverFrame;
	TArray<Chaos::FConstPhysicsObjectHandle> ReplicationCache_Key_Marshal;
	TArray<FRigidBodyState> ReplicationCache_Value_Marshal;

	void Reset()
	{
		SolverFrame = 0;
	}
};



// -------------- Physics Thread --------------

/** Replication cache data holding state and access time */
struct FPhysicsReplicationCacheData
{
public:
	FPhysicsReplicationCacheData()
	{ }

	void SetAccessTime(double Time) { AccessTime = Time; }
	const double GetAccessTime() const { return AccessTime; }
	FRigidBodyState& GetState() { return StateData; }

private:
	double AccessTime;
	FRigidBodyState StateData;
};

/** Async replication cache class */
class FPhysicsReplicationCacheAsync : public Chaos::TSimCallbackObject<
	FPhysicsReplicationCacheAsyncInput,
	FPhysicsReplicationCacheAsyncOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::PostSolve | Chaos::ESimCallbackOptions::PhysicsObjectUnregister>
{
	virtual void OnPreSimulate_Internal() override;
	virtual void OnPostSolve_Internal() override;

	/** Callback when a physics object is removed */
	virtual void OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;

	/** Process incoming async input data from GameThread, register and unregister particles to/from cache */
	void ProcessAsyncInputs();

	/** Populate replication cache by iterating over particles and populating async output with data */
	void PopulateReplicationCache_Internal();

private:
	TMap<Chaos::FConstPhysicsObjectHandle, FPhysicsReplicationCacheData> ReplicationCache_Internal;
	TArray<Chaos::FConstPhysicsObjectHandle> PendingDeleteFromReplicationCache_Internal;
	bool bUpdateAfterRemoval = false;
};

