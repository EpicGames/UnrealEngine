// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsReplicationQuantization.h
	Snaps replicated rigid particle state to the wire format quantization grid
	at PostSolve so server and client integrate from the same grid each step.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Chaos/SimCallbackObject.h"
#include "Engine/ReplicatedState.h"

class AActor;
class FPhysScene_Chaos;
class FPhysicsReplicationQuantizationAsync;
class UPrimitiveComponent;

namespace Chaos
{
	class FPhysicsSolverBase;
}

// Profile types

/** Per axis quantization grid. Step equals 1.0/Scale, zero scale disables this axis.
 *  Bits is informational (the wire ceiling) and is not enforced as a runtime clamp.
 *  bTruncate is reserved for truncation based wire format, not implemented, current format is rounding to nearest int */
struct FQuantizationAxisGrid
{
	int32 Scale = 0;
	int32 Bits = 30;
	bool bTruncate = false;
};

/** What to quantize for one replicated rigid particle. */
struct FQuantizationProfile
{
	enum class ERotationMode : uint8
	{
		None,
		Euler8Bit,
		Euler16Bit,
	};

	FQuantizationAxisGrid Location;
	FQuantizationAxisGrid LinearVelocity;
	FQuantizationAxisGrid AngularVelocity;
	ERotationMode RotationMode = ERotationMode::None;

	bool HasAnyAxis() const
	{
		return Location.Scale != 0
			|| LinearVelocity.Scale != 0
			|| AngularVelocity.Scale != 0
			|| RotationMode != ERotationMode::None;
	}
};

// Helpers

/** EVectorQuantization to FQuantizationAxisGrid
 *  Returns Scale=0 for the engine default RoundWholeNumber so default tier actors are skipped */
ENGINE_API FQuantizationAxisGrid VectorQuantizationToGrid(EVectorQuantization Q);

/** Get a FQuantizationAxisGrid with the bit size calculated based in the precision wanted and max absolute range */
ENGINE_API FQuantizationAxisGrid MakeAxisGrid(double Precision, double MaxAbsRange);

/** Get FQuantizationProfile for AActor based on FRepMovement. Returns an empty profile (HasAnyAxis() == false) for actors with engine defaults */
ENGINE_API FQuantizationProfile BuildProfileFromActor(const AActor* Actor);

/** Canonicalize a quaternion's sign for logging and comparison. Q and the negated
 *  quaternion represent the same rotation, so two endpoints that are rotationally
 *  identical can print with all four components negated and look like a huge diff.
 *  Force W to be non negative (and use X then Y then Z as tiebreakers when W==0) so any
 *  two equal rotations print identically. The returned value is only for display.
 *  Do not feed it back into the simulation. */
inline FQuat CanonicalizeQuatForLog(const FQuat& Q)
{
	// Tiebreaker chain handles the W==0 (180 degree) edge case.
	// Beyond that just to flip when W is negative.
	bool bFlip = false;
	if (Q.W < 0.0) { bFlip = true; }
	else if (Q.W == 0.0 && Q.X < 0.0) { bFlip = true; }
	else if (Q.W == 0.0 && Q.X == 0.0 && Q.Y < 0.0) { bFlip = true; }
	else if (Q.W == 0.0 && Q.X == 0.0 && Q.Y == 0.0 && Q.Z < 0.0) { bFlip = true; }
	return bFlip ? FQuat(-Q.X, -Q.Y, -Q.Z, -Q.W) : Q;
}

class FPhysicsReplicationQuantization
{
public:
	FPhysicsReplicationQuantization(FPhysScene_Chaos* InPhysicsScene);
	~FPhysicsReplicationQuantization();

	/** Register an actor's primary physics body using a profile derived from FRepMovement */
	void RegisterActor(AActor* Actor);

	/** Re derive the profile and overwrite the existing registration */
	void RefreshActor(AActor* Actor);

	/** Unregister an actor's primary physics body */
	void UnregisterActor(AActor* Actor);

	/** Register an explicit profile against a physics object handle. For callers using
	 *  non FRepMovement replication paths who need to match their own wire grid.
	 *  DebugName is used in PT log lines so client and server entries can be matched
	 *  cross process. Pass a name that is stable on both ends */
	ENGINE_API void RegisterCustomProfile(Chaos::FConstPhysicsObjectHandle Handle, const FQuantizationProfile& Profile, const FString& DebugName = FString());

	/** Re push an explicit profile (overwrites) */
	void RefreshCustomProfile(Chaos::FConstPhysicsObjectHandle Handle, const FQuantizationProfile& Profile, const FString& DebugName = FString());

	/** Unregister a handle registered via RegisterCustomProfile */
	ENGINE_API void UnregisterHandle(Chaos::FConstPhysicsObjectHandle Handle);

	/** Async sim callback accessor used by the static IsHandleManaged_Internal query */
	FPhysicsReplicationQuantizationAsync* GetAsync() const { return AsyncCallback; }

private:
	Chaos::FConstPhysicsObjectHandle GetActorHandle(AActor* Actor) const;

	// Per tick refresh of bIsServer onto the AsyncInput.
	void OnPhysScenePreTick(FPhysScene_Chaos* Scene, float DeltaSeconds);

	void PushPerTickStateIfChanged();

	FPhysScene_Chaos* PhysicsScene = nullptr;
	FPhysicsReplicationQuantizationAsync* AsyncCallback = nullptr;
	FDelegateHandle PreTickHandle;

	// GT side mirror so Unregister can find the handle even if the actor's component has already torn down by the time Unregister is called
	TMap<TWeakObjectPtr<AActor>, Chaos::FConstPhysicsObjectHandle> ActorHandles_External;

	// Last role pushed to the PT side
	bool bLastPushedIsServer = false;
	bool bHasPushedPerTickStateOnce = false;
};

namespace UE::PhysicsReplicationQuantization
{
	/** Component lifecycle helper called from primitive component physics state create
	 *  and destroy paths. Self must be the actor's root primitive component */
	ENGINE_API void NotifyComponentPhysicsState(UPrimitiveComponent* Self, bool bRegister);

	/** Get the PT side async callback for PhysicsReplicationQuantization */
	ENGINE_API FPhysicsReplicationQuantizationAsync* GetAsyncFromSolver(Chaos::FPhysicsSolverBase* Solver);

	/** PT thread safe query to check if a physics object is handled by PhysicsReplicationQuantization */
	ENGINE_API bool IsHandleManaged_Internal(Chaos::FPhysicsSolverBase* Solver, Chaos::FConstPhysicsObjectHandle Handle);

	/** Verify a network-received target state's quantization against the registered quantization for the physics object */
	ENGINE_API bool VerifyTargetOnGrid_Internal(
		Chaos::FPhysicsSolverBase* Solver,
		Chaos::FConstPhysicsObjectHandle Handle,
		int32 ServerFrame,
		const FVector& TargetPosition,
		const FQuat& TargetRotation,
		const FVector& TargetLinVel,
		const FVector& TargetAngVelDeg);

	/** Force a quantization pass on all registered particle */
	ENGINE_API void ForceSnapAllRegistered_Internal(Chaos::FPhysicsSolverBase* Solver, bool bIncludeCurrentXR = true);
}

// Async marshaling

struct FQuantizationAsyncInput : public Chaos::FSimCallbackInput
{
	struct FAdd
	{
		Chaos::FConstPhysicsObjectHandle Handle;
		FQuantizationProfile Profile;
		FString DebugName;
	};

	TArray<FAdd> Adds;
	TArray<Chaos::FConstPhysicsObjectHandle> Removes;

	// Set on every input from the GT shell so the PT side can stamp logs with role.
	TOptional<bool> SetIsServer;

	void Reset()
	{
		Adds.Reset();
		Removes.Reset();
		SetIsServer.Reset();
	}
};

struct FQuantizationAsyncOutput : public Chaos::FSimCallbackOutput
{
	void Reset() {}
};

// PT sim callback

class FPhysicsReplicationQuantizationAsync : public Chaos::TSimCallbackObject<FQuantizationAsyncInput, FQuantizationAsyncOutput,
	Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::Rewind | Chaos::ESimCallbackOptions::PostSolve | Chaos::ESimCallbackOptions::PhysicsObjectUnregister>
{
public:
	enum class EVerificationState : uint8
	{
		Unverified, // Have not yet observed a target for this axis class
		Verified, // A target landed on the expected wire grid
		MismatchReported, // A received target was off grid, ensure already fired
	};

	/** True if the handle is registered and quantization is enabled */
	ENGINE_API bool IsHandleManaged_Internal(Chaos::FConstPhysicsObjectHandle Handle) const;

	/** True if the handle is present in the internal profile map */
	bool IsHandleRegistered_Internal(Chaos::FConstPhysicsObjectHandle Handle) const;

	/** Verifies a received target state's P, Q, V, W components against the quantization grids we assume for this handle
	 *  Returns true if all checked axis classes are Verified or deferred
	 *  Returns false if any axis detected a mismatch in quantization */
	bool VerifyTargetOnGrid_Internal(Chaos::FConstPhysicsObjectHandle Handle, int32 ServerFrame,
		const FVector& TargetPosition, const FQuat& TargetRotation,
		const FVector& TargetLinVel, const FVector& TargetAngVelDeg);

	/** Snap every registered particle's PQVW state to the quantization grid
	 *  @param bIncludeCurrentXR: also snap X and R
	 *  @param FrameLabelDelta: added to the solver's CurrentFrame when stamping log lines
	 *      Pass minus one from a pre step caller (FirstPreResimStep) so the log represents "end of step N minus 1", matching a call in PostSolve. */
	void ForceSnapAllRegistered_Internal(bool bIncludeCurrentXR, bool bAllowLogging, int32 FrameLabelDelta = 0);

private:
	// Called by ProcessAsyncInputs or OnPhysicsObjectUnregistered_Internal before a slot is removed.
	void NotifySlotPreRemove_Internal(Chaos::FConstPhysicsObjectHandle Handle);

	// PreSimulate processes Adds and Removes early so any other PostSolve callback that
	// queries IsHandleManaged_Internal sees current registration state for the same step.
	virtual void OnPreSimulate_Internal() override;

	// Fires after FRewindData::ApplyTargets has injected the cached server target onto the particle and BEFORE the resim's AdvanceSolver runs any other sim callback.
	// We quantize registered particles at this point due to floating point inconsistencies when converting R and W in the network serialize flow
	virtual void FirstPreResimStep_Internal(int32 PhysicsStep) override;

	// PostSolve performs the actual quantization on each registered handle.
	virtual void OnPostSolve_Internal() override;

	virtual void OnPhysicsObjectUnregistered_Internal(Chaos::FConstPhysicsObjectHandle PhysicsObject) override;

	void ProcessAsyncInputs();

	void EmitCallerHeaderLog_Internal(const TCHAR* CallerName, int32 FrameLabelDelta = 0);

	struct FProfileSlot
	{
		FQuantizationProfile Profile;
		FString DebugName;
		EVerificationState LocationVerifyState = EVerificationState::Unverified;
		EVerificationState RotationVerifyState = EVerificationState::Unverified;
		EVerificationState LinVelVerifyState = EVerificationState::Unverified;
		EVerificationState AngVelVerifyState = EVerificationState::Unverified;

		// Number of non trivial targets we ran the verify against, per axis class.
		// Used at unregister time to distinguish "never had data to check" from "had data but never validated"
		// the latter is suspicious because the network quantization is most likely less precise than the registered one but on the same grid, i.e. scale 10 while we have registered 100.
		uint32 LocationTestsAttempted = 0;
		uint32 RotationTestsAttempted = 0;
		uint32 LinVelTestsAttempted = 0;
		uint32 AngVelTestsAttempted = 0;
	};
	TMap<Chaos::FConstPhysicsObjectHandle, FProfileSlot> Profiles_Internal;

	// Names already reported by NotifySlotPreRemove_Internal
	TSet<FString> UnverifiedReportedNames_Internal;

	bool bIsServer_Internal = false;
};
