// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Math/MathFwd.h"
#include "CollisionShape.h"
#include "CollisionQueryParams.h"
#include "Chaos/AABB.h"
#include "Chaos/Core.h"

namespace Chaos
{
	enum class EQueryInfo : uint8
	{
		GatherAll,		//get all data and actually return it
		IsBlocking,		//is any of the data blocking? only return a bool so don't bother collecting
		IsAnything		//is any of the data blocking or touching? only return a bool so don't bother collecting
	};

	enum class EThreadQueryContext : uint8
	{
		GTData,		//use interpolated GT data
		PTDataWithGTObjects,	//use pt data, but convert back to GT when possible
		PTOnlyData,	//use only the PT data and don't try to convert anything back to GT
	};

	struct FCommonQueryData
	{
		ECollisionChannel TraceChannel;
		FCollisionQueryParams Params;
		FCollisionResponseParams ResponseParams;
		FCollisionObjectQueryParams ObjectParams;
	};

	struct FQueryShape
	{
		bool IsConvexShape() const
		{
			return !ConvexData.IsEmpty();
		}

		FCollisionShape CollisionShape;
		TArray<uint8> ConvexData;
		FAABB3 LocalBoundingBox;
	};

	struct FOverlapQueryData
	{
		FQueryShape QueryShape;
		FTransform GeomPose;
	};

	struct FRayQueryData
	{
		FVector Start;
		FVector End;
	};

	struct FSweepQueryData
	{
		FVector Start;
		FVector End;
		FQueryShape QueryShape;
		FQuat GeomRot;
	};
}
