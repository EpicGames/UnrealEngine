// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/UnrealMath.h"
#include "Templates/Function.h"
#include "Containers/Array.h"

struct FSimpleSuspensionSpringParams
{
	FSimpleSuspensionSpringParams()
		: Stiffness(0.f)
		, Damping(0.f)
		, Length(0.f)
		, Mass(0.f)
	{ }

	float Stiffness;
	float Damping;
	float Length;
	float Mass;
};

struct FSimpleSuspensionParams
{
	FSimpleSuspensionParams()
		: LocalCenterOfMass(FVector::ZeroVector)
		, LocalSuspensionNormal(FVector::UpVector)
		, LocalGroundDistance(0.f)
		, NaturalFrequency(10.f)
		, DampingRatio(.1f)
	{ }

	TArray<FSimpleSuspensionSpringParams> SpringParams;
	TArray<FVector> LocalSpringOrigins;
	FVector LocalCenterOfMass;
	FVector LocalSuspensionNormal;
	float LocalGroundDistance;
	float NaturalFrequency;
	float DampingRatio;
};

struct FSimpleSuspensionState
{
	FSimpleSuspensionState()
		: TotalForce(FVector::ZeroVector)
		, TotalTorqueRad(FVector::ZeroVector)
		, WorldCenterOfMass(FVector::ZeroVector)
		, WorldSuspensionNormal(FVector::UpVector)
	{ }

	TArray<float> SpringDisplacements;
	TArray<float> SpringForces;
	TArray<bool> SpringContacts;
	TArray<FVector> WorldSpringOrigins;
	FVector TotalForce;
	FVector TotalTorqueRad;
	FVector WorldCenterOfMass;
	FVector WorldSuspensionNormal;
};

using FSimpleSuspensionRaycast = TFunction<bool(const FVector& RayBegin, const FVector& RayEnd, const int32 SpringIndex, FVector& OutRayHitLocation, FVector& OutRayHitNormal)>;

const FSimpleSuspensionRaycast SimpleSuspensionRaycastGroundPlane = [](const FVector& RayBegin, const FVector& RayEnd, const int32 SpringIndex, FVector& OutRayHitLocation, FVector& OutRayHitNormal)
{
	// Do an intersection with the z-plane at 0,0,0.
	const FVector Norm = FVector::UpVector;
	const FVector Diff = RayEnd - RayBegin;
	const FVector::FReal ProjA = FVector::DotProduct(RayBegin, Norm);
	const FVector::FReal ProjDiff = FVector::DotProduct(Diff, Norm);
	if (FMath::Abs(ProjDiff) > UE_SMALL_NUMBER)
	{
		const FVector::FReal Lambda = -ProjA / ProjDiff;
		if (Lambda > 0.f && Lambda <= 1.f)
		{
			OutRayHitLocation = RayBegin + (Lambda * Diff);
			OutRayHitNormal = Norm;
			return true;
		}
	}
	return false;
};

struct FSimpleSuspension
{
	FSimpleSuspension()
	{ }

	FSimpleSuspension(const FSimpleSuspensionParams& SuspensionParams)
	{
		Setup(SuspensionParams);
	}

	FSimpleSuspensionParams SuspensionParams;
	FSimpleSuspensionState SuspensionState;

	/** Take full set of parameters. They might be invalid, who cares. */
	void Setup(const FSimpleSuspensionParams& InSuspensionParams);

	/** Copy in a full set of parameters, and then compute the spring params
	    and overwrite whatever we got for that */
	void Setup(const FSimpleSuspensionParams& InSuspensionParams, const float TotalMass, const float Gravity);

	/** Compute sprung masses, stiffnesses, damping, etc. */
	void ComputeSpringParams(const float TotalMass, const float Gravity);

	/** Compute suspension compressions, individual spring forces, and
		the total force that the suspension system wants to apply to the
		body. Actual application of this force must be done by someone else. */
	void Update(
		const FTransform& LocalToWorld,
		const FVector& LinearVelocity,
		const FVector& AngularVelocityRad,
		const FSimpleSuspensionRaycast& RaycastFunction = SimpleSuspensionRaycastGroundPlane);
};

struct FSimpleSuspensionHelpers
{
	//
	// Setup functions
	//

	/** Compute the distribution of the mass of a body among springs.
		This method assumes that spring positions are given relative
		to the center of mass of the body, and that gravity occurs
		in the local -Z direction.

		Returns true if it was able to find a valid mass configuration.
		If only one or two springs are included, then a valid
		configuration may not result in a stable suspension system -
		a bicycle or pogostick, for example, which is not perfectly centered
		may have a valid sprung mass configuration without being stable. */
	ENGINE_API static bool ComputeSprungMasses(const TArray<FVector>& MassSpringPositions, const float TotalMass, TArray<float>& OutSprungMasses, FString* ErrMsg = nullptr);

	/** Same as above, but allows the caller to specify spring locations
		in a local space which is not necessarily originated at the center
		of mass. */
	ENGINE_API static bool ComputeSprungMasses(const TArray<FVector>& LocalSpringPositions, const FVector& LocalCenterOfMass, const float TotalMass, TArray<float>& OutSprungMasses, FString* ErrMsg = nullptr);

	/** Calculates the Lambdas in the single axis case*/
	static bool ComputeSingleAxisLambda(const FVector::FReal AxisDot, const FVector::FReal SumAxis, const uint32 Count, TArray<FVector::FReal, TFixedAllocator<2>>& Lambdas, FString* ErrMsg = nullptr);

	/** Given a sprung mass and a spring stiffness, compute the natural
		frequency of the spring */
	static void ComputeSpringNaturalFrequencyAndDampingRatio(const float SprungMass, const float SpringStiffness, const float SpringDamping, float& OutNaturalFrequency, float& OutDampingRatio);

	/** Given a sprung mass (not just a sprung mass percent!), a desired
		natural frequency, compute the stiffness of the spring. */
	static float ComputeSpringStiffness(const float SprungMass, const float NaturalFrequency);

	/** Given a sprung mass (not just a sprung mass percent!), a desired
		natural frequency, and a damping ratio, compute the damping
		coefficient of the spring. */
	static float ComputeSpringDamping(const float SprungMass, const float NaturalFrequency, const float DampingRatio);

	/** Given a sprung mass and a natural frequency, return the damping
		coefficient which will critically damp the spring. */
	static float ComputeSpringCriticalDamping(const float SprungMass, const float NaturalFrequency);

	/** Based on spring stiffness, gravity, and the desired length of the
		spring at rest under load, compute its uncompressed rest-length */
	static float ComputeSpringRestLength(const float SpringStiffness, const float SprungMass, const float SuspendedLength, const float Gravity);

	/** Compute an array of FSimpleSupportParams objects */
	static void ComputeSpringParams(const TArray<FVector>& LocalSuspensionOrigins, const FVector& LocalCenterOfMass, const float TotalMass, const FVector& LocalSuspensionNormal, const float LocalGroundDistance, const float NaturalFrequency, const float DampingRatio, const float Gravity, TArray<FSimpleSuspensionSpringParams>& OutSpringParams);

	/** Compute an array of FSimpleSupportParams objects, based on data from a SuspensionParams object */
	static void ComputeSpringParams(const FSimpleSuspensionParams& SuspensionParams, const float TotalMass, const float Gravity, TArray<FSimpleSuspensionSpringParams>& OutSpringParams);

	//
	// Dynamics functions
	//

	static void ComputeWorldSuspensionCoordinates(const FVector& LocalCenterOfMass, const FVector& LocalSuspensionNormal, const TArray<FVector>& LocalSuspensionOrigins, const FTransform& LocalToWorld, FVector& OutWorldCenterOfMass, FVector& OutWorldSuspensionNormal, TArray<FVector>& OutWorldSuspensionOrigins);

	static void ComputeWorldSuspensionCoordinates(const FSimpleSuspensionParams& SuspensionParams, const FTransform& LocalToWorld, FSimpleSuspensionState& OutSuspensionState);

	/** Perform raycasts to compute spring displacements. This helper
		doesn't know about "world", so it takes a lambda. Consumers of
		this class may just want to do this themselves to avoid this */
	static void ComputeSuspensionDisplacements(const TArray<FVector> WorldSuspensionOrigins, const TArray<FSimpleSuspensionSpringParams>& SpringParams, const FVector& WorldSuspensionNormal, const FSimpleSuspensionRaycast& RaycastFunction, TArray<float>& OutSpringDisplacements);

	static void ComputeSuspensionDisplacements(const FSimpleSuspensionParams& SuspensionParams, const FSimpleSuspensionRaycast& RaycastFunction, FSimpleSuspensionState InOutSuspensionState);

	/** Compute the total force that a spring ought to apply */
	static float ComputeSpringForce(const float SpringStiffness, const float SpringDamping, const float SpringDisplacement, const float SpringVelocity);

	/** Based on the current state of the system, compute the total force
		and torque that the suspension system wants to apply to the
		suspended body, as well as the spring force magnitudes of each
		support. */
	static void ComputeSuspensionForces(const FVector& LinearVelocity, const FVector& AngularVelocityRad, const FVector& WorldCenterOfMass, const FVector& WorldSuspensionNormal, const TArray<FVector> WorldSuspensionOrigins, const TArray<FSimpleSuspensionSpringParams>& SpringParams, const TArray<float>& SpringDisplacements, FVector& OutTotalForce, FVector& OutTotalTorque, TArray<float>& OutSpringForces);

	static void ComputeSuspensionForces(const FVector& LinearVelocity, const FVector& AngularVelocityRad, const FSimpleSuspensionState& SuspensionState, const TArray<FSimpleSuspensionSpringParams>& SpringParams, FSimpleSuspensionState& OutSuspensionState);

	/** Integrate a single spring. */
	static void IntegrateSpring(const float DeltaTime, const float SpringDisplacement, const float SpringVelocity, const FSimpleSuspensionSpringParams& SpringParams, const float SprungMass, float& OutNewSpringDisplacement, float& OutNewSpringVelocity);

	/** Performs a fully implicit damped spring integration to compute next
		positions and velocities. */
	static void IntegrateSprings(const float DeltaTime, const TArray<float>& SpringDisplacements, const TArray<float>& SpringVelocities, const TArray<FSimpleSuspensionSpringParams>& SuspensionParams, const TArray<float>& SprungMasses, TArray<float>& OutNewSpringDisplacements, TArray<float>& OutNewSpringVelocities);

private:

	static bool ErrCheck(const bool bCondition, const FString& Message, FString* ErrMsg);
};
