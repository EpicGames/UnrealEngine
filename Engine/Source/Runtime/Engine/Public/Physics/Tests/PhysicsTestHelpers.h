// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineGlobals.h"
#include "Engine/Engine.h"

struct FPhysicsTestHelpers
{
	static UWorld* GetWorld()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			return GWorld;
		}
#endif // WITH_EDITOR
		return GEngine->GetWorldContexts()[0].World();
	}
};
