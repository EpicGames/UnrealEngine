// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsPublic.h
	Rigid Body Physics Public Types
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Engine/EngineTypes.h"
#include "Misc/CoreMisc.h"
#include "Misc/App.h"
#include "EngineDefines.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsPublicCore.h"
//#include "StaticMeshResources.h"

class AActor;
class ULineBatchComponent;
class UPhysicalMaterial;
class UPhysicsAsset;
class UPrimitiveComponent;
class USkeletalMeshComponent;
struct FConstraintInstance;
struct FBodyInstance;
struct FStaticMeshVertexBuffers;
class FPhysScene_PhysX;
class FLocalVertexFactory;
class FDynamicMeshIndexBuffer32;

/** Delegate for applying custom physics forces upon the body. Can be passed to "AddCustomPhysics" so 
* custom forces and torques can be calculated individually for every physics substep.
* The function provides delta time for a physics step and pointer to body instance upon which forces must be added.
* 
* Do not expect this callback to be called from the main game thread! It may get called from a physics simulation thread. */
DECLARE_DELEGATE_TwoParams(FCalculateCustomPhysics, float, FBodyInstance*);

/** Delegate for applying custom physics projection upon the body. When this is set for the body instance,
* it will be called whenever component transformation is requested from the physics engine. If
* projection is required (for example, visual position of an object must be different to the one in physics engine,
* e.g. the box should not penetrate the wall visually) the transformation of body must be updated to account for it.
* Since this could be called many times by GetWorldTransform any expensive computations should be cached if possible.*/
DECLARE_DELEGATE_TwoParams(FCalculateCustomProjection, const FBodyInstance*, FTransform&);

/** Delegate for when the mass properties of a body instance have been re-calculated. This can be useful for systems that need to set specific physx settings on actors, or systems that rely on the mass information in some way*/
DECLARE_MULTICAST_DELEGATE_OneParam(FRecalculatedMassProperties, FBodyInstance*);

/**
 * Physics stats
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("FetchAndStart Time (all)"), STAT_TotalPhysicsTime, STATGROUP_Physics, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Cloth Actor Count"), STAT_NumCloths, STATGROUP_Physics, ENGINE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Simulated Cloth Verts"), STAT_NumClothVerts, STATGROUP_Physics, ENGINE_API);

/** Pointer to PhysX Command Handler */
extern ENGINE_API class FPhysCommandHandler* GPhysCommandHandler;

/** Information about a specific object involved in a rigid body collision */
struct FRigidBodyCollisionInfo
{
	/** Actor involved in the collision */
	TWeakObjectPtr<AActor>					Actor;

	/** Component of Actor involved in the collision. */
	TWeakObjectPtr<UPrimitiveComponent>		Component;

	/** Index of body if this is in a PhysicsAsset. INDEX_NONE otherwise. */
	int32									BodyIndex;

	/** Name of bone if a PhysicsAsset */
	FName									BoneName;

	/** Amount by which the linear velocity at the center of mass of this body has changed in the frame of this contact */
	FVector									DeltaVelocity;

	FRigidBodyCollisionInfo() :
		BodyIndex(INDEX_NONE),
		BoneName(NAME_None)
	{}

	/** Utility to set up the body collision info from an FBodyInstance */
	ENGINE_API void SetFrom(const FBodyInstance* BodyInst, const FVector& InDeltaVelocity = FVector::ZeroVector);
	/** Get body instance */
	ENGINE_API FBodyInstance* GetBodyInstance() const;
};

/** One entry in the array of collision notifications pending execution at the end of the physics engine run. */
struct FCollisionNotifyInfo
{
	/** If this notification should be called for the Actor in Info0. */
	bool							bCallEvent0;

	/** If this notification should be called for the Actor in Info1. */
	bool							bCallEvent1;

	/** Information about the first object involved in the collision. */
	FRigidBodyCollisionInfo			Info0;

	/** Information about the second object involved in the collision. */
	FRigidBodyCollisionInfo			Info1;

	/** Information about the collision itself */
	FCollisionImpactData			RigidCollisionData;

	Chaos::FReal					SolverTime;

	FCollisionNotifyInfo() :
		bCallEvent0(false),
		bCallEvent1(false),
		SolverTime((Chaos::FReal)0.0)
	{}

	/** Check that is is valid to call a notification for this entry. Looks at the IsValid() flags on both Actors. */
	ENGINE_API bool IsValidForNotify() const;
};

namespace PhysCommand
{
	enum Type
	{
		Release,
		ReleasePScene,
		DeleteCPUDispatcher,
		DeleteSimEventCallback,
		DeleteContactModifyCallback,
		DeleteCCDContactModifyCallback,
		DeleteMbpBroadphaseCallback,
		Max
	};
}

/** Container used for physics tasks that need to be deferred from GameThread. This is not safe for general purpose multi-therading*/
class FPhysCommandHandler
{
public:

	~FPhysCommandHandler();
	
	/** Executes pending commands and clears buffer **/
	void ENGINE_API Flush();
	bool ENGINE_API HasPendingCommands();

private:

	/** Command to execute when physics simulation is done */
	struct FPhysPendingCommand
	{
		//union
		//{
		//} Pointer;

		PhysCommand::Type CommandType;
	};

	/** Execute all enqueued commands */
	void ExecuteCommands();

	/** Enqueue a command to the double buffer */
	void EnqueueCommand(const FPhysPendingCommand& Command);

	/** Array of commands waiting to execute once simulation is done */
	TArray<FPhysPendingCommand> PendingCommands;
};

/** Clears all linear forces on the body */
void ClearForces_AssumesLocked(FBodyInstance* BodyInstance, bool bAllowSubstepping);


/** Clears all torques on the body */
void ClearTorques_AssumesLocked(FBodyInstance* BodyInstance, bool bAllowSubstepping);

/**
* Return true if we should be running in single threaded mode, ala dedicated server
**/
inline bool PhysSingleThreadedMode()
{
	if (IsRunningDedicatedServer() || !FApp::ShouldUseThreadingForPerformance() || FPlatformMisc::NumberOfCores() < 3 || !FPlatformProcess::SupportsMultithreading() || FParse::Param(FCommandLine::Get(), TEXT("SingleThreadedPhysics")))
	{
		return true;
	}
	return false;
}

// Only used for legacy serialization (ver < VER_UE4_REMOVE_PHYS_SCALED_GEOM_CACHES)
class FKCachedConvexDataElement
{
public:
	TArray<uint8>	ConvexElementData;

	friend FArchive& operator<<( FArchive& Ar, FKCachedConvexDataElement& S )
	{
		S.ConvexElementData.BulkSerialize(Ar);
		return Ar;
	}
};

// Only used for legacy serialization (ver < VER_UE4_REMOVE_PHYS_SCALED_GEOM_CACHES)
class FKCachedConvexData
{
public:
	TArray<FKCachedConvexDataElement>	CachedConvexElements;

	friend FArchive& operator<<( FArchive& Ar, FKCachedConvexData& S )
	{
		Ar << S.CachedConvexElements;
		return Ar;
	}
};

// Only used for legacy serialization (ver < VER_UE4_ADD_BODYSETUP_GUID)
struct FKCachedPerTriData
{
	TArray<uint8> CachedPerTriData;

	friend FArchive& operator<<( FArchive& Ar, FKCachedPerTriData& S )
	{
		S.CachedPerTriData.BulkSerialize(Ar);
		return Ar;
	}
};

class FKConvexGeomRenderInfo
{
public:
	FStaticMeshVertexBuffers* VertexBuffers;
	FDynamicMeshIndexBuffer32* IndexBuffer;
	FLocalVertexFactory* CollisionVertexFactory;

	FKConvexGeomRenderInfo();

	/** Util to see if this render info has some valid geometry to render. */
	bool HasValidGeometry();
};

ENGINE_API bool	InitGamePhys();
ENGINE_API void	TermGamePhys();

/** Perform any deferred cleanup of resources (GPhysXPendingKillConvex etc) */
ENGINE_API void DeferredPhysResourceCleanup();


FTransform FindBodyTransform(AActor* Actor, FName BoneName);
FBox	FindBodyBox(AActor* Actor, FName BoneName);

/** Set of delegates to allowing hooking different parts of the physics engine */
class FPhysicsDelegates : public FPhysicsDelegatesCore
{
public:

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysicsAssetChanged, const UPhysicsAsset*);
	static ENGINE_API FOnPhysicsAssetChanged OnPhysicsAssetChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysSceneInit, FPhysScene*);
	static ENGINE_API FOnPhysSceneInit OnPhysSceneInit;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysSceneTerm, FPhysScene*);
	static ENGINE_API FOnPhysSceneTerm OnPhysSceneTerm;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPhysDispatchNotifications, FPhysScene*);
	static ENGINE_API FOnPhysDispatchNotifications OnPhysDispatchNotifications;
};
